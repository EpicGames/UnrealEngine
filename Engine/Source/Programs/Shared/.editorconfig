# NOTE: Requires **VS2019 16.3** or later

# EditorConfig is awesome:http://EditorConfig.org

# Dotnet code style settings:
[*.cs]



csharp_style_prefer_switch_expression = false
csharp_style_implicit_object_creation_when_type_is_apparent = true:warning
dotnet_remove_unnecessary_suppression_exclusions = all

# Sort using and Import directives with System.* appearing first
dotnet_sort_system_directives_first = true
dotnet_separate_import_directive_groups = false

# Avoid "this." and "Me." if not necessary
dotnet_style_qualification_for_field = false:error
dotnet_style_qualification_for_property = false:error
dotnet_style_qualification_for_method = false:error
dotnet_style_qualification_for_event = false:error

# Use language keywords instead of framework type names for type references
dotnet_style_predefined_type_for_locals_parameters_members = true:error
dotnet_style_predefined_type_for_member_access = false:error

# Suggest more modern language features when available
dotnet_style_object_initializer = false:suggestion
dotnet_style_collection_initializer = false:suggestion
dotnet_style_coalesce_expression = true:suggestion
dotnet_style_null_propagation = true:suggestion
dotnet_style_explicit_tuple_names = true:suggestion

# Whitespace options
dotnet_style_allow_multiple_blank_lines_experimental = false
dotnet_style_allow_statement_immediately_after_block_experimental = true

# Non-private static fields are PascalCase
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.symbols = non_private_static_fields
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.style = non_private_static_field_style

dotnet_naming_symbols.non_private_static_fields.applicable_kinds = field
dotnet_naming_symbols.non_private_static_fields.applicable_accessibilities = public, protected, internal, protected_internal, private_protected
dotnet_naming_symbols.non_private_static_fields.required_modifiers = static

dotnet_naming_style.non_private_static_field_style.capitalization = pascal_case

# Non-private readonly fields are PascalCase
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.symbols = non_private_readonly_fields
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.style = non_private_readonly_field_style

dotnet_naming_symbols.non_private_readonly_fields.applicable_kinds = field
dotnet_naming_symbols.non_private_readonly_fields.applicable_accessibilities = public, protected, internal, protected_internal, private_protected
dotnet_naming_symbols.non_private_readonly_fields.required_modifiers = readonly

dotnet_naming_style.non_private_readonly_field_style.capitalization = pascal_case

# Constants are PascalCase
dotnet_naming_rule.constants_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.constants_should_be_pascal_case.symbols = constants
dotnet_naming_rule.constants_should_be_pascal_case.style = constant_style

dotnet_naming_symbols.constants.applicable_kinds = field, local
dotnet_naming_symbols.constants.required_modifiers = const

dotnet_naming_style.constant_style.capitalization = pascal_case

# Static fields are camelCase and start with s_
dotnet_naming_rule.static_fields_should_be_camel_case.severity = suggestion
dotnet_naming_rule.static_fields_should_be_camel_case.symbols = static_fields
dotnet_naming_rule.static_fields_should_be_camel_case.style = static_field_style

dotnet_naming_symbols.static_fields.applicable_kinds = field
dotnet_naming_symbols.static_fields.required_modifiers = static

dotnet_naming_style.static_field_style.capitalization = camel_case
dotnet_naming_style.static_field_style.required_prefix = s_

# Instance fields are camelCase and start with _
dotnet_naming_rule.instance_fields_should_be_camel_case.severity = suggestion
dotnet_naming_rule.instance_fields_should_be_camel_case.symbols = instance_fields
dotnet_naming_rule.instance_fields_should_be_camel_case.style = instance_field_style

dotnet_naming_symbols.instance_fields.applicable_kinds = field

dotnet_naming_style.instance_field_style.capitalization = camel_case
dotnet_naming_style.instance_field_style.required_prefix = _

# Locals and parameters are camelCase
dotnet_naming_rule.locals_should_be_camel_case.severity = suggestion
dotnet_naming_rule.locals_should_be_camel_case.symbols = locals_and_parameters
dotnet_naming_rule.locals_should_be_camel_case.style = camel_case_style

dotnet_naming_symbols.locals_and_parameters.applicable_kinds = parameter, local

dotnet_naming_style.camel_case_style.capitalization = camel_case

# Local functions are PascalCase
dotnet_naming_rule.local_functions_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.local_functions_should_be_pascal_case.symbols = local_functions
dotnet_naming_rule.local_functions_should_be_pascal_case.style = local_function_style

dotnet_naming_symbols.local_functions.applicable_kinds = local_function

dotnet_naming_style.local_function_style.capitalization = pascal_case

# By default, name items with PascalCase
dotnet_naming_rule.members_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.members_should_be_pascal_case.symbols = all_members
dotnet_naming_rule.members_should_be_pascal_case.style = pascal_case_style

dotnet_naming_symbols.all_members.applicable_kinds = *

dotnet_naming_style.pascal_case_style.capitalization = pascal_case

# IDE0073: File header
dotnet_diagnostic.IDE0073.severity = warning
file_header_template = Copyright Epic Games, Inc. All Rights Reserved.

# IDE0035: Remove unreachable code
dotnet_diagnostic.IDE0035.severity = warning

# IDE0036: Order modifiers
dotnet_diagnostic.IDE0036.severity = warning

# IDE0043: Format string contains invalid placeholder
dotnet_diagnostic.IDE0043.severity = warning

# IDE0044: Make field readonly
dotnet_diagnostic.IDE0044.severity = warning

# IDE0028: Use collection initializers
dotnet_diagnostic.IDE0028.severity = warning

# IDE0031: Use null propagation
dotnet_diagnostic.IDE0031.severity = warning

# IDE0034: Simplify 'default' expression
dotnet_diagnostic.IDE0034.severity = warning

# IDE0054: Use compound assignment
dotnet_diagnostic.IDE0054.severity = warning

# IDE0062: Make local function static
dotnet_diagnostic.IDE0062.severity = warning

# IDE0071: Simplify interpolation
dotnet_diagnostic.IDE0071.severity = warning

# IDE0074: Use compound assignment
dotnet_diagnostic.IDE0074.severity = warning

# IDE0075: Simplify conditional expression
dotnet_diagnostic.IDE0075.severity = warning

# IDE0280: Use 'nameof'
dotnet_diagnostic.IDE0280.severity = warning

# IDE0300: Use collection expression for array
dotnet_diagnostic.IDE0300.severity = warning

# IDE0301: Use collection expression for empty
dotnet_diagnostic.IDE0301.severity = warning

# IDE0305: Use collection expression for fluent
dotnet_diagnostic.IDE0305.severity = warning

# IDE0330: Prefer 'System.Threading.Lock'
dotnet_diagnostic.IDE0330.severity = warning

# IDE0011: Add braces
csharp_prefer_braces = when_multiline:warning
# NOTE: We need the below severity entry for Add Braces due to https://github.com/dotnet/roslyn/issues/44201
dotnet_diagnostic.IDE0011.severity = warning

# IDE0040: Add accessibility modifiers
dotnet_diagnostic.IDE0040.severity = warning

# CONSIDER: Are IDE0051 and IDE0052 too noisy to be warnings for IDE editing scenarios? Should they be made build-only warnings?
# IDE0051: Remove unused private member
dotnet_diagnostic.IDE0051.severity = warning

# IDE0052: Remove unread private member
dotnet_diagnostic.IDE0052.severity = warning

# IDE0059: Unnecessary assignment to a value
dotnet_diagnostic.IDE0059.severity = warning

# IDE0060: Remove unused parameter
dotnet_diagnostic.IDE0060.severity = warning

# CA1012: Abstract types should not have public constructors
dotnet_diagnostic.CA1012.severity = warning

# IDE0005: Using directive is unnecessary
dotnet_diagnostic.IDE0005.severity = warning

# IDE0270: Null check can be simplified (if null check)
dotnet_diagnostic.IDE0270.severity = silent

# IDE0090: 'new' expression can be simplified
dotnet_diagnostic.IDE0090.severity = warning

# IDE1006: Enforce naming conventions
dotnet_diagnostic.IDE1006.severity = warning

# dotnet_style_allow_multiple_blank_lines_experimental
dotnet_diagnostic.IDE2000.severity = warning

# csharp_style_allow_embedded_statements_on_same_line_experimental
dotnet_diagnostic.IDE2001.severity = warning

# csharp_style_allow_blank_lines_between_consecutive_braces_experimental
dotnet_diagnostic.IDE2002.severity = warning

# dotnet_style_allow_statement_immediately_after_block_experimental
dotnet_diagnostic.IDE2003.severity = warning

# csharp_style_allow_blank_line_after_colon_in_constructor_initializer_experimental
dotnet_diagnostic.IDE2004.severity = warning

dotnet_style_require_accessibility_modifiers = false

# CSharp code style settings:
[*.cs]
csharp_prefer_simple_using_statement = false
csharp_style_prefer_range_operator = false

# Newline settings
csharp_new_line_before_open_brace = all
csharp_new_line_before_else = true
csharp_new_line_before_catch = true
csharp_new_line_before_finally = true
csharp_new_line_before_members_in_object_initializers = true
csharp_new_line_before_members_in_anonymous_types = true
csharp_new_line_between_query_expression_clauses = true

# Indentation preferences
csharp_indent_block_contents = true
csharp_indent_braces = false
csharp_indent_case_contents = true
csharp_indent_case_contents_when_block = true
csharp_indent_switch_labels = true
csharp_indent_labels = flush_left

# Whitespace options
csharp_style_allow_embedded_statements_on_same_line_experimental = false
csharp_style_allow_blank_lines_between_consecutive_braces_experimental = false
csharp_style_allow_blank_line_after_colon_in_constructor_initializer_experimental = false

# Prefer "var" everywhere
dotnet_diagnostic.IDE0007.severity = error
dotnet_diagnostic.IDE0008.severity = error
csharp_style_var_for_built_in_types = false:error
csharp_style_var_when_type_is_apparent = false:error
csharp_style_var_elsewhere = false:error

# Prefer method-like constructs to have a block body
csharp_style_expression_bodied_methods = false:none
csharp_style_expression_bodied_constructors = false:none
csharp_style_expression_bodied_operators = false:none

# Prefer property-like constructs to have an expression-body
csharp_style_expression_bodied_properties = true:error
csharp_style_expression_bodied_indexers = true:error
csharp_style_expression_bodied_accessors = true:error

# Suggest more modern language features when available
csharp_style_pattern_matching_over_is_with_cast_check = true:error
csharp_style_pattern_matching_over_as_with_null_check = false:error
csharp_style_inlined_variable_declaration = false:suggestion
csharp_style_throw_expression = true:error
csharp_style_conditional_delegate_call = true:suggestion

# Spacing
csharp_space_after_cast = false
csharp_space_after_colon_in_inheritance_clause = true
csharp_space_after_comma = true
csharp_space_after_dot = false
csharp_space_after_keywords_in_control_flow_statements = true
csharp_space_after_semicolon_in_for_statement = true
csharp_space_around_binary_operators = before_and_after
csharp_space_around_declaration_statements = do_not_ignore
csharp_space_before_colon_in_inheritance_clause = true
csharp_space_before_comma = false
csharp_space_before_dot = false
csharp_space_before_open_square_brackets = false
csharp_space_before_semicolon_in_for_statement = false
csharp_space_between_empty_square_brackets = false
csharp_space_between_method_call_empty_parameter_list_parentheses = false
csharp_space_between_method_call_name_and_opening_parenthesis = false
csharp_space_between_method_call_parameter_list_parentheses = false
csharp_space_between_method_declaration_empty_parameter_list_parentheses = false
csharp_space_between_method_declaration_name_and_open_parenthesis = false
csharp_space_between_method_declaration_parameter_list_parentheses = false
csharp_space_between_parentheses = false
csharp_space_between_square_brackets = false

# Blocks are allowed
csharp_prefer_braces = true:silent
csharp_preserve_single_line_blocks = true
csharp_preserve_single_line_statements = true

# Async methods should have "Async" suffix
dotnet_naming_rule.async_methods_end_in_async.symbols = any_async_methods
dotnet_naming_rule.async_methods_end_in_async.style = end_in_async
dotnet_naming_rule.async_methods_end_in_async.severity = warning

dotnet_naming_symbols.any_async_methods.applicable_kinds = method
dotnet_naming_symbols.any_async_methods.applicable_accessibilities = *
dotnet_naming_symbols.any_async_methods.required_modifiers = async

dotnet_naming_style.end_in_async.required_prefix = 
dotnet_naming_style.end_in_async.required_suffix = Async
dotnet_naming_style.end_in_async.capitalization = pascal_case
dotnet_naming_style.end_in_async.word_separator = 

# All Rules Enabled as build warnings
# Description: All Rules are enabled as build warnings. Rules with IsEnabledByDefault = false are force enabled as build warnings.

# Code files
[*.{cs,vb}]


# CA1000: Do not declare static members on generic types
dotnet_diagnostic.CA1000.severity = warning

# CA1001: Types that own disposable fields should be disposable
dotnet_diagnostic.CA1001.severity = warning

# CA1002: Do not expose generic lists
dotnet_diagnostic.CA1002.severity = none

# CA1003: Use generic event handler instances
dotnet_diagnostic.CA1003.severity = none

# CA1005: Avoid excessive parameters on generic types
dotnet_diagnostic.CA1005.severity = warning

# CA1008: Enums should have zero value
dotnet_diagnostic.CA1008.severity = warning

# CA1010: Generic interface should also be implemented
dotnet_diagnostic.CA1010.severity = warning

# CA1012: Abstract types should not have public constructors
dotnet_diagnostic.CA1012.severity = warning

# CA1014: Mark assemblies with CLSCompliant
dotnet_diagnostic.CA1014.severity = warning

# CA1016: Mark assemblies with assembly version
dotnet_diagnostic.CA1016.severity = warning

# CA1017: Mark assemblies with ComVisible
dotnet_diagnostic.CA1017.severity = warning

# CA1018: Mark attributes with AttributeUsageAttribute
dotnet_diagnostic.CA1018.severity = warning

# CA1019: Define accessors for attribute arguments
dotnet_diagnostic.CA1019.severity = warning

# CA1021: Avoid out parameters
dotnet_diagnostic.CA1021.severity = none

# CA1024: Use properties where appropriate
dotnet_diagnostic.CA1024.severity = none

# CA1027: Mark enums with FlagsAttribute
dotnet_diagnostic.CA1027.severity = warning

# CA1028: Enum Storage should be Int32
dotnet_diagnostic.CA1028.severity = warning

# CA1030: Use events where appropriate
dotnet_diagnostic.CA1030.severity = warning

# CA1031: Do not catch general exception types
dotnet_diagnostic.CA1031.severity = none

# CA1032: Implement standard exception constructors
dotnet_diagnostic.CA1032.severity = none

# CA1033: Interface methods should be callable by child types
dotnet_diagnostic.CA1033.severity = warning

# CA1034: Nested types should not be visible
dotnet_diagnostic.CA1034.severity = warning

# CA1036: Override methods on comparable types
dotnet_diagnostic.CA1036.severity = warning

# CA1040: Avoid empty interfaces
dotnet_diagnostic.CA1040.severity = warning

# CA1041: Provide ObsoleteAttribute message
dotnet_diagnostic.CA1041.severity = warning

# CA1043: Use Integral Or String Argument For Indexers
dotnet_diagnostic.CA1043.severity = warning

# CA1044: Properties should not be write only
dotnet_diagnostic.CA1044.severity = warning

# CA1045: Do not pass types by reference
dotnet_diagnostic.CA1045.severity = warning

# CA1046: Do not overload equality operator on reference types
dotnet_diagnostic.CA1046.severity = warning

# CA1047: Do not declare protected member in sealed type
dotnet_diagnostic.CA1047.severity = warning

# CA1050: Declare types in namespaces
dotnet_diagnostic.CA1050.severity = warning

# CA1051: Do not declare visible instance fields
dotnet_diagnostic.CA1051.severity = warning

# CA1052: Static holder types should be Static or NotInheritable
dotnet_diagnostic.CA1052.severity = warning

# CA1054: URI-like parameters should not be strings
dotnet_diagnostic.CA1054.severity = warning

# CA1055: URI-like return values should not be strings
dotnet_diagnostic.CA1055.severity = warning

# CA1056: URI-like properties should not be strings
dotnet_diagnostic.CA1056.severity = warning

# CA1058: Types should not extend certain base types
dotnet_diagnostic.CA1058.severity = warning

# CA1060: Move pinvokes to native methods class
dotnet_diagnostic.CA1060.severity = none

# CA1061: Do not hide base class methods
dotnet_diagnostic.CA1061.severity = warning

# CA1062: Validate arguments of public methods
dotnet_diagnostic.CA1062.severity = none

# CA1063: Implement IDisposable Correctly
dotnet_diagnostic.CA1063.severity = warning

# CA1064: Exceptions should be public
dotnet_diagnostic.CA1064.severity = warning

# CA1065: Do not raise exceptions in unexpected locations
dotnet_diagnostic.CA1065.severity = none

# CA1066: Implement IEquatable when overriding Object.Equals
dotnet_diagnostic.CA1066.severity = warning

# CA1067: Override Object.Equals(object) when implementing IEquatable<T>
dotnet_diagnostic.CA1067.severity = warning

# CA1068: CancellationToken parameters must come last
dotnet_diagnostic.CA1068.severity = warning

# CA1069: Enums values should not be duplicated
dotnet_diagnostic.CA1069.severity = warning

# CA1070: Do not declare event fields as virtual
dotnet_diagnostic.CA1070.severity = warning

# CA1200: Avoid using cref tags with a prefix
dotnet_diagnostic.CA1200.severity = warning

# CA1303: Do not pass literals as localized parameters
dotnet_diagnostic.CA1303.severity = none

# CA1304: Specify CultureInfo
dotnet_diagnostic.CA1304.severity = none

# CA1305: Specify IFormatProvider
dotnet_diagnostic.CA1305.severity = none

# CA1307: Specify StringComparison for clarity
dotnet_diagnostic.CA1307.severity = warning

# CA1308: Normalize strings to uppercase
dotnet_diagnostic.CA1308.severity = warning

# CA1309: Use ordinal string comparison
dotnet_diagnostic.CA1309.severity = warning

# CA1310: Specify StringComparison for correctness
dotnet_diagnostic.CA1310.severity = warning

# CA1401: P/Invokes should not be visible
dotnet_diagnostic.CA1401.severity = warning

# CA1416: Validate platform compatibility
dotnet_diagnostic.CA1416.severity = warning

# CA1417: Do not use 'OutAttribute' on string parameters for P/Invokes
dotnet_diagnostic.CA1417.severity = warning

# CA1418: Use valid platform string
dotnet_diagnostic.CA1418.severity = warning

# CA1419: Provide a parameterless constructor that is as visible as the containing type for concrete types derived from 'System.Runtime.InteropServices.SafeHandle'
dotnet_diagnostic.CA1419.severity = warning

# CA1501: Avoid excessive inheritance
dotnet_diagnostic.CA1501.severity = warning

# CA1502: Avoid excessive complexity
dotnet_diagnostic.CA1502.severity = none

# CA1505: Avoid unmaintainable code
dotnet_diagnostic.CA1505.severity = warning

# CA1506: Avoid excessive class coupling
dotnet_diagnostic.CA1506.severity = none

# CA1507: Use nameof to express symbol names
dotnet_diagnostic.CA1507.severity = warning

# CA1508: Avoid dead conditional code
dotnet_diagnostic.CA1508.severity = warning

# CA1509: Invalid entry in code metrics rule specification file
dotnet_diagnostic.CA1509.severity = warning

# CA1700: Do not name enum values 'Reserved'
dotnet_diagnostic.CA1700.severity = warning

# CA1707: Identifiers should not contain underscores
dotnet_diagnostic.CA1707.severity = warning

# CA1708: Identifiers should differ by more than case
dotnet_diagnostic.CA1708.severity = warning

# CA1710: Identifiers should have correct suffix
dotnet_diagnostic.CA1710.severity = warning

# CA1711: Identifiers should not have incorrect suffix
dotnet_diagnostic.CA1711.severity = none

# CA1712: Do not prefix enum values with type name
dotnet_diagnostic.CA1712.severity = warning

# CA1713: Events should not have 'Before' or 'After' prefix
dotnet_diagnostic.CA1713.severity = warning

# CA1715: Identifiers should have correct prefix
dotnet_diagnostic.CA1715.severity = warning

# CA1716: Identifiers should not match keywords
dotnet_diagnostic.CA1716.severity = warning

# CA1720: Identifier contains type name
dotnet_diagnostic.CA1720.severity = none

# CA1721: Property names should not match get methods
dotnet_diagnostic.CA1721.severity = warning

# CA1724: Type names should not match namespaces
dotnet_diagnostic.CA1724.severity = warning

# CA1725: Parameter names should match base declaration
dotnet_diagnostic.CA1725.severity = none

# CA1727: Use PascalCase for named placeholders
dotnet_diagnostic.CA1727.severity = warning

# CA1802: Use literals where appropriate
dotnet_diagnostic.CA1802.severity = warning

# CA1805: Do not initialize unnecessarily
dotnet_diagnostic.CA1805.severity = none

# CA1806: Do not ignore method results
dotnet_diagnostic.CA1806.severity = warning

# CA1810: Initialize reference type static fields inline
dotnet_diagnostic.CA1810.severity = warning

# CA1812: Avoid uninstantiated internal classes
dotnet_diagnostic.CA1812.severity = none

# CA1813: Avoid unsealed attributes
dotnet_diagnostic.CA1813.severity = warning

# CA1814: Prefer jagged arrays over multidimensional
dotnet_diagnostic.CA1814.severity = warning

# CA1815: Override equals and operator equals on value types
dotnet_diagnostic.CA1815.severity = none

# CA1816: Dispose methods should call SuppressFinalize
dotnet_diagnostic.CA1816.severity = warning

# CA1819: Properties should not return arrays (rationale: System.Text.Json serialization requires array properties)
dotnet_diagnostic.CA1819.severity = silent

# CA1820: Test for empty strings using string length
dotnet_diagnostic.CA1820.severity = warning

# CA1821: Remove empty Finalizers
dotnet_diagnostic.CA1821.severity = warning

# CA1822: Mark members as static
dotnet_diagnostic.CA1822.severity = warning

# CA1823: Avoid unused private fields
dotnet_diagnostic.CA1823.severity = warning

# CA1824: Mark assemblies with NeutralResourcesLanguageAttribute
dotnet_diagnostic.CA1824.severity = warning

# CA1825: Avoid zero-length array allocations
dotnet_diagnostic.CA1825.severity = warning

# CA1826: Do not use Enumerable methods on indexable collections
dotnet_diagnostic.CA1826.severity = warning

# CA1827: Do not use Count() or LongCount() when Any() can be used
dotnet_diagnostic.CA1827.severity = warning

# CA1828: Do not use CountAsync() or LongCountAsync() when AnyAsync() can be used
dotnet_diagnostic.CA1828.severity = warning

# CA1829: Use Length/Count property instead of Count() when available
dotnet_diagnostic.CA1829.severity = warning

# CA1830: Prefer strongly-typed Append and Insert method overloads on StringBuilder
dotnet_diagnostic.CA1830.severity = warning

# CA1831: Use AsSpan or AsMemory instead of Range-based indexers when appropriate
dotnet_diagnostic.CA1831.severity = warning

# CA1832: Use AsSpan or AsMemory instead of Range-based indexers when appropriate
dotnet_diagnostic.CA1832.severity = warning

# CA1833: Use AsSpan or AsMemory instead of Range-based indexers when appropriate
dotnet_diagnostic.CA1833.severity = warning

# CA1834: Consider using 'StringBuilder.Append(char)' when applicable
dotnet_diagnostic.CA1834.severity = warning

# CA1835: Prefer the 'Memory'-based overloads for 'ReadAsync' and 'WriteAsync'
dotnet_diagnostic.CA1835.severity = none

# CA1836: Prefer IsEmpty over Count
dotnet_diagnostic.CA1836.severity = warning

# CA1837: Use 'Environment.ProcessId'
dotnet_diagnostic.CA1837.severity = warning

# CA1838: Avoid 'StringBuilder' parameters for P/Invokes
dotnet_diagnostic.CA1838.severity = warning

# CA1839: Use 'Environment.ProcessPath'
dotnet_diagnostic.CA1839.severity = warning

# CA1840: Use 'Environment.CurrentManagedThreadId'
dotnet_diagnostic.CA1840.severity = warning

# CA1841: Prefer Dictionary.Contains methods
dotnet_diagnostic.CA1841.severity = warning

# CA1842: Do not use 'WhenAll' with a single task
dotnet_diagnostic.CA1842.severity = warning

# CA1843: Do not use 'WaitAll' with a single task
dotnet_diagnostic.CA1843.severity = warning

# CA1844: Provide memory-based overrides of async methods when subclassing 'Stream'
dotnet_diagnostic.CA1844.severity = warning

# CA1845: Use span-based 'string.Concat'
dotnet_diagnostic.CA1845.severity = none

# CA1846: Prefer 'AsSpan' over 'Substring'
dotnet_diagnostic.CA1846.severity = none

# CA1847: Use char literal for a single character lookup
dotnet_diagnostic.CA1847.severity = warning

# CA1848: Use the LoggerMessage delegates
dotnet_diagnostic.CA1848.severity = none

# CA1849: Call async methods when in an async method
dotnet_diagnostic.CA1849.severity = warning

# CA2000: Dispose objects before losing scope
dotnet_diagnostic.CA2000.severity = warning

# CA2002: Do not lock on objects with weak identity
dotnet_diagnostic.CA2002.severity = warning

# CA2007: Consider calling ConfigureAwait on the awaited task
dotnet_diagnostic.CA2007.severity = none

# CA2008: Do not create tasks without passing a TaskScheduler
dotnet_diagnostic.CA2008.severity = warning

# CA2009: Do not call ToImmutableCollection on an ImmutableCollection value
dotnet_diagnostic.CA2009.severity = warning

# CA2011: Avoid infinite recursion
dotnet_diagnostic.CA2011.severity = warning

# CA2012: Use ValueTasks correctly
dotnet_diagnostic.CA2012.severity = warning

# CA2013: Do not use ReferenceEquals with value types
dotnet_diagnostic.CA2013.severity = warning

# CA2014: Do not use stackalloc in loops
dotnet_diagnostic.CA2014.severity = warning

# CA2015: Do not define finalizers for types derived from MemoryManager<T>
dotnet_diagnostic.CA2015.severity = warning

# CA2016: Forward the 'CancellationToken' parameter to methods
dotnet_diagnostic.CA2016.severity = warning

# CA2017: Parameter count mismatch
dotnet_diagnostic.CA2017.severity = warning

# CA2018: 'Buffer.BlockCopy' expects the number of bytes to be copied for the 'count' argument
dotnet_diagnostic.CA2018.severity = warning

# CA2100: Review SQL queries for security vulnerabilities
dotnet_diagnostic.CA2100.severity = warning

# CA2101: Specify marshaling for P/Invoke string arguments
dotnet_diagnostic.CA2101.severity = none

# CA2109: Review visible event handlers
dotnet_diagnostic.CA2109.severity = warning

# CA2119: Seal methods that satisfy private interfaces
dotnet_diagnostic.CA2119.severity = warning

# CA2153: Do Not Catch Corrupted State Exceptions
dotnet_diagnostic.CA2153.severity = warning

# CA2200: Rethrow to preserve stack details
dotnet_diagnostic.CA2200.severity = warning

# CA2201: Do not raise reserved exception types
dotnet_diagnostic.CA2201.severity = none

# CA2207: Initialize value type static fields inline
dotnet_diagnostic.CA2207.severity = warning

# CA2208: Instantiate argument exceptions correctly
dotnet_diagnostic.CA2208.severity = warning

# CA2211: Non-constant fields should not be visible
dotnet_diagnostic.CA2211.severity = warning

# CA2213: Disposable fields should be disposed
dotnet_diagnostic.CA2213.severity = warning

# CA2214: Do not call overridable methods in constructors
dotnet_diagnostic.CA2214.severity = warning

# CA2215: Dispose methods should call base class dispose
dotnet_diagnostic.CA2215.severity = warning

# CA2216: Disposable types should declare finalizer
dotnet_diagnostic.CA2216.severity = warning

# CA2217: Do not mark enums with FlagsAttribute
dotnet_diagnostic.CA2217.severity = warning

# CA2218: Override GetHashCode on overriding Equals
dotnet_diagnostic.CA2218.severity = warning

# CA2219: Do not raise exceptions in finally clauses
dotnet_diagnostic.CA2219.severity = warning

# CA2224: Override Equals on overloading operator equals
dotnet_diagnostic.CA2224.severity = warning

# CA2225: Operator overloads have named alternates
dotnet_diagnostic.CA2225.severity = none

# CA2226: Operators should have symmetrical overloads
dotnet_diagnostic.CA2226.severity = warning

# CA2227: Collection properties should be read only
dotnet_diagnostic.CA2227.severity = warning

# CA2229: Implement serialization constructors
dotnet_diagnostic.CA2229.severity = warning

# CA2231: Overload operator equals on overriding value type Equals
dotnet_diagnostic.CA2231.severity = warning

# CA2234: Pass system uri objects instead of strings
dotnet_diagnostic.CA2234.severity = warning

# CA2235: Mark all non-serializable fields
dotnet_diagnostic.CA2235.severity = warning

# CA2237: Mark ISerializable types with serializable
dotnet_diagnostic.CA2237.severity = warning

# CA2241: Provide correct arguments to formatting methods
dotnet_diagnostic.CA2241.severity = warning

# CA2242: Test for NaN correctly
dotnet_diagnostic.CA2242.severity = warning

# CA2243: Attribute string literals should parse correctly
dotnet_diagnostic.CA2243.severity = warning

# CA2244: Do not duplicate indexed element initializations
dotnet_diagnostic.CA2244.severity = warning

# CA2245: Do not assign a property to itself
dotnet_diagnostic.CA2245.severity = warning

# CA2246: Assigning symbol and its member in the same statement
dotnet_diagnostic.CA2246.severity = warning

# CA2247: Argument passed to TaskCompletionSource constructor should be TaskCreationOptions enum instead of TaskContinuationOptions enum
dotnet_diagnostic.CA2247.severity = warning

# CA2248: Provide correct 'enum' argument to 'Enum.HasFlag'
dotnet_diagnostic.CA2248.severity = warning

# CA2249: Consider using 'string.Contains' instead of 'string.IndexOf'
dotnet_diagnostic.CA2249.severity = warning

# CA2250: Use 'ThrowIfCancellationRequested'
dotnet_diagnostic.CA2250.severity = warning

# CA2251: Use 'string.Equals'
dotnet_diagnostic.CA2251.severity = warning

# CA2252: This API requires opting into preview features
dotnet_diagnostic.CA2252.severity = warning

# CA2253: Named placeholders should not be numeric values
dotnet_diagnostic.CA2253.severity = warning

# CA2254: Template should be a static expression
dotnet_diagnostic.CA2254.severity = warning

# CA2255: The 'ModuleInitializer' attribute should not be used in libraries
dotnet_diagnostic.CA2255.severity = warning

# CA2256: All members declared in parent interfaces must have an implementation in a DynamicInterfaceCastableImplementation-attributed interface
dotnet_diagnostic.CA2256.severity = warning

# CA2257: Members defined on an interface with the 'DynamicInterfaceCastableImplementationAttribute' should be 'static'
dotnet_diagnostic.CA2257.severity = warning

# CA2258: Providing a 'DynamicInterfaceCastableImplementation' interface in Visual Basic is unsupported
dotnet_diagnostic.CA2258.severity = warning

# CA2300: Do not use insecure deserializer BinaryFormatter
dotnet_diagnostic.CA2300.severity = warning

# CA2301: Do not call BinaryFormatter.Deserialize without first setting BinaryFormatter.Binder
dotnet_diagnostic.CA2301.severity = warning

# CA2302: Ensure BinaryFormatter.Binder is set before calling BinaryFormatter.Deserialize
dotnet_diagnostic.CA2302.severity = warning

# CA2305: Do not use insecure deserializer LosFormatter
dotnet_diagnostic.CA2305.severity = warning

# CA2310: Do not use insecure deserializer NetDataContractSerializer
dotnet_diagnostic.CA2310.severity = warning

# CA2311: Do not deserialize without first setting NetDataContractSerializer.Binder
dotnet_diagnostic.CA2311.severity = warning

# CA2312: Ensure NetDataContractSerializer.Binder is set before deserializing
dotnet_diagnostic.CA2312.severity = warning

# CA2315: Do not use insecure deserializer ObjectStateFormatter
dotnet_diagnostic.CA2315.severity = warning

# CA2321: Do not deserialize with JavaScriptSerializer using a SimpleTypeResolver
dotnet_diagnostic.CA2321.severity = warning

# CA2322: Ensure JavaScriptSerializer is not initialized with SimpleTypeResolver before deserializing
dotnet_diagnostic.CA2322.severity = warning

# CA2326: Do not use TypeNameHandling values other than None
dotnet_diagnostic.CA2326.severity = warning

# CA2327: Do not use insecure JsonSerializerSettings
dotnet_diagnostic.CA2327.severity = warning

# CA2328: Ensure that JsonSerializerSettings are secure
dotnet_diagnostic.CA2328.severity = warning

# CA2329: Do not deserialize with JsonSerializer using an insecure configuration
dotnet_diagnostic.CA2329.severity = warning

# CA2330: Ensure that JsonSerializer has a secure configuration when deserializing
dotnet_diagnostic.CA2330.severity = warning

# CA2350: Do not use DataTable.ReadXml() with untrusted data
dotnet_diagnostic.CA2350.severity = warning

# CA2351: Do not use DataSet.ReadXml() with untrusted data
dotnet_diagnostic.CA2351.severity = warning

# CA2352: Unsafe DataSet or DataTable in serializable type can be vulnerable to remote code execution attacks
dotnet_diagnostic.CA2352.severity = warning

# CA2353: Unsafe DataSet or DataTable in serializable type
dotnet_diagnostic.CA2353.severity = warning

# CA2354: Unsafe DataSet or DataTable in deserialized object graph can be vulnerable to remote code execution attacks
dotnet_diagnostic.CA2354.severity = warning

# CA2355: Unsafe DataSet or DataTable type found in deserializable object graph
dotnet_diagnostic.CA2355.severity = warning

# CA2356: Unsafe DataSet or DataTable type in web deserializable object graph
dotnet_diagnostic.CA2356.severity = warning

# CA2361: Ensure auto-generated class containing DataSet.ReadXml() is not used with untrusted data
dotnet_diagnostic.CA2361.severity = warning

# CA2362: Unsafe DataSet or DataTable in auto-generated serializable type can be vulnerable to remote code execution attacks
dotnet_diagnostic.CA2362.severity = warning

# CA3001: Review code for SQL injection vulnerabilities
dotnet_diagnostic.CA3001.severity = warning

# CA3002: Review code for XSS vulnerabilities
dotnet_diagnostic.CA3002.severity = warning

# CA3003: Review code for file path injection vulnerabilities
dotnet_diagnostic.CA3003.severity = warning

# CA3004: Review code for information disclosure vulnerabilities
dotnet_diagnostic.CA3004.severity = warning

# CA3005: Review code for LDAP injection vulnerabilities
dotnet_diagnostic.CA3005.severity = warning

# CA3006: Review code for process command injection vulnerabilities
dotnet_diagnostic.CA3006.severity = warning

# CA3007: Review code for open redirect vulnerabilities
dotnet_diagnostic.CA3007.severity = warning

# CA3008: Review code for XPath injection vulnerabilities
dotnet_diagnostic.CA3008.severity = warning

# CA3009: Review code for XML injection vulnerabilities
dotnet_diagnostic.CA3009.severity = warning

# CA3010: Review code for XAML injection vulnerabilities
dotnet_diagnostic.CA3010.severity = warning

# CA3011: Review code for DLL injection vulnerabilities
dotnet_diagnostic.CA3011.severity = warning

# CA3012: Review code for regex injection vulnerabilities
dotnet_diagnostic.CA3012.severity = warning

# CA3061: Do Not Add Schema By URL
dotnet_diagnostic.CA3061.severity = warning

# CA3075: Insecure DTD processing in XML
dotnet_diagnostic.CA3075.severity = warning

# CA3076: Insecure XSLT script processing.
dotnet_diagnostic.CA3076.severity = warning

# CA3077: Insecure Processing in API Design, XmlDocument and XmlTextReader
dotnet_diagnostic.CA3077.severity = warning

# CA3147: Mark Verb Handlers With Validate Antiforgery Token
dotnet_diagnostic.CA3147.severity = warning

# CA5350: Do Not Use Weak Cryptographic Algorithms
dotnet_diagnostic.CA5350.severity = warning

# CA5351: Do Not Use Broken Cryptographic Algorithms
dotnet_diagnostic.CA5351.severity = warning

# CA5358: Review cipher mode usage with cryptography experts
dotnet_diagnostic.CA5358.severity = warning

# CA5359: Do Not Disable Certificate Validation
dotnet_diagnostic.CA5359.severity = warning

# CA5360: Do Not Call Dangerous Methods In Deserialization
dotnet_diagnostic.CA5360.severity = warning

# CA5361: Do Not Disable SChannel Use of Strong Crypto
dotnet_diagnostic.CA5361.severity = warning

# CA5362: Potential reference cycle in deserialized object graph
dotnet_diagnostic.CA5362.severity = warning

# CA5363: Do Not Disable Request Validation
dotnet_diagnostic.CA5363.severity = warning

# CA5364: Do Not Use Deprecated Security Protocols
dotnet_diagnostic.CA5364.severity = warning

# CA5365: Do Not Disable HTTP Header Checking
dotnet_diagnostic.CA5365.severity = warning

# CA5366: Use XmlReader for 'DataSet.ReadXml()'
dotnet_diagnostic.CA5366.severity = warning

# CA5367: Do Not Serialize Types With Pointer Fields
dotnet_diagnostic.CA5367.severity = warning

# CA5368: Set ViewStateUserKey For Classes Derived From Page
dotnet_diagnostic.CA5368.severity = warning

# CA5369: Use XmlReader for 'XmlSerializer.Deserialize()'
dotnet_diagnostic.CA5369.severity = warning

# CA5370: Use XmlReader for XmlValidatingReader constructor
dotnet_diagnostic.CA5370.severity = warning

# CA5371: Use XmlReader for 'XmlSchema.Read()'
dotnet_diagnostic.CA5371.severity = warning

# CA5372: Use XmlReader for XPathDocument constructor
dotnet_diagnostic.CA5372.severity = warning

# CA5373: Do not use obsolete key derivation function
dotnet_diagnostic.CA5373.severity = warning

# CA5374: Do Not Use XslTransform
dotnet_diagnostic.CA5374.severity = warning

# CA5375: Do Not Use Account Shared Access Signature
dotnet_diagnostic.CA5375.severity = warning

# CA5376: Use SharedAccessProtocol HttpsOnly
dotnet_diagnostic.CA5376.severity = warning

# CA5377: Use Container Level Access Policy
dotnet_diagnostic.CA5377.severity = warning

# CA5378: Do not disable ServicePointManagerSecurityProtocols
dotnet_diagnostic.CA5378.severity = warning

# CA5379: Ensure Key Derivation Function algorithm is sufficiently strong
dotnet_diagnostic.CA5379.severity = warning

# CA5380: Do Not Add Certificates To Root Store
dotnet_diagnostic.CA5380.severity = warning

# CA5381: Ensure Certificates Are Not Added To Root Store
dotnet_diagnostic.CA5381.severity = warning

# CA5382: Use Secure Cookies In ASP.NET Core
dotnet_diagnostic.CA5382.severity = warning

# CA5383: Ensure Use Secure Cookies In ASP.NET Core
dotnet_diagnostic.CA5383.severity = warning

# CA5384: Do Not Use Digital Signature Algorithm (DSA)
dotnet_diagnostic.CA5384.severity = warning

# CA5385: Use Rivest–Shamir–Adleman (RSA) Algorithm With Sufficient Key Size
dotnet_diagnostic.CA5385.severity = warning

# CA5386: Avoid hardcoding SecurityProtocolType value
dotnet_diagnostic.CA5386.severity = warning

# CA5387: Do Not Use Weak Key Derivation Function With Insufficient Iteration Count
dotnet_diagnostic.CA5387.severity = warning

# CA5388: Ensure Sufficient Iteration Count When Using Weak Key Derivation Function
dotnet_diagnostic.CA5388.severity = warning

# CA5389: Do Not Add Archive Item's Path To The Target File System Path
dotnet_diagnostic.CA5389.severity = warning

# CA5390: Do not hard-code encryption key (disabled: expensive interprocedural dataflow analyzer)
dotnet_diagnostic.CA5390.severity = none

# CA5391: Use antiforgery tokens in ASP.NET Core MVC controllers
dotnet_diagnostic.CA5391.severity = warning

# CA5392: Use DefaultDllImportSearchPaths attribute for P/Invokes
dotnet_diagnostic.CA5392.severity = none

# CA5393: Do not use unsafe DllImportSearchPath value
dotnet_diagnostic.CA5393.severity = warning

# CA5394: Do not use insecure randomness
dotnet_diagnostic.CA5394.severity = warning

# CA5395: Miss HttpVerb attribute for action methods
dotnet_diagnostic.CA5395.severity = warning

# CA5396: Set HttpOnly to true for HttpCookie
dotnet_diagnostic.CA5396.severity = warning

# CA5397: Do not use deprecated SslProtocols values
dotnet_diagnostic.CA5397.severity = warning

# CA5398: Avoid hardcoded SslProtocols values
dotnet_diagnostic.CA5398.severity = warning

# CA5399: HttpClients should enable certificate revocation list checks
dotnet_diagnostic.CA5399.severity = warning

# CA5400: Ensure HttpClient certificate revocation list check is not disabled
dotnet_diagnostic.CA5400.severity = warning

# CA5401: Do not use CreateEncryptor with non-default IV
dotnet_diagnostic.CA5401.severity = warning

# CA5402: Use CreateEncryptor with the default IV 
dotnet_diagnostic.CA5402.severity = warning

# CA5403: Do not hard-code certificate (disabled: expensive interprocedural dataflow analyzer)
dotnet_diagnostic.CA5403.severity = none

# CA5404: Do not disable token validation checks
dotnet_diagnostic.CA5404.severity = none

# CA5405: Do not always skip token validation in delegates
dotnet_diagnostic.CA5405.severity = warning

# VSTHRD003: Avoid awaiting or returning a Task representing work that was not started within your context as that can lead to deadlocks.
dotnet_diagnostic.VSTHRD003.severity = none

# MA0026: TODO comment
dotnet_diagnostic.MA0026.severity = none

# MA0048: File name must match type name
dotnet_diagnostic.MA0048.severity = none

# MA0004: Use Task.ConfigureAwait(false) if the current SynchronizationContext is not needed
dotnet_diagnostic.MA0004.severity = none

# MA0051: Method is too long
dotnet_diagnostic.MA0051.severity = none

# MA0002: Use an overload that has a IEqualityComparer<string> or IComparer<string> parameter
dotnet_diagnostic.MA0002.severity = none

# MA0016: Prefer using collection abstraction instead of implementation
dotnet_diagnostic.MA0016.severity = none

# MA0006: Use string.Equals instead of Equals operator
dotnet_diagnostic.MA0006.severity = none

# MA0061: Method overrides should not change default values
dotnet_diagnostic.MA0061.severity = none

# MA0134: Observe result of async calls
dotnet_diagnostic.MA0134.severity = none

# MA0011: Use an overload of 'ToString' that has a 'System.IFormatProvider' parameter
dotnet_diagnostic.MA0011.severity = none

# MA0009: Regular expressions should not be vulnerable to Denial of Service attacks
dotnet_diagnostic.MA0009.severity = none

# MA0099: Use Explicit enum value for 'SlackNotificationSink.ReactionFlags' instead of 0
dotnet_diagnostic.MA0099.severity = none

# MA0025: Implement the functionality (or raise NotSupportedException or PlatformNotSupportedException)
dotnet_diagnostic.MA0025.severity = none

# MA0015: Use an overload of 'System.ArgumentException' with the parameter name
dotnet_diagnostic.MA0015.severity = none

# MA0132: Do not convert implicitly to DateTimeOffset
dotnet_diagnostic.MA0132.severity = none

# MA0023: Add RegexOptions.ExplicitCapture to prevent capturing unneeded groups
dotnet_diagnostic.MA0023.severity = none

# MA0056: Do not call overridable members in constructor
dotnet_diagnostic.MA0056.severity = none

# MA0144: Use System.OperatingSystem to check the current OS
dotnet_diagnostic.MA0144.severity = none

# MA0046 - Use EventHandler<T> to declare events
dotnet_diagnostic.MA0046.severity = none

# MA0008: Add StructLayoutAttribute
dotnet_diagnostic.MA0008.severity = none

[*Test*.cs]

# CA1707: Identifiers should not contain underscores
dotnet_diagnostic.CA1707.severity = none

# CA5394: Do not use insecure randomness
dotnet_diagnostic.CA5394.severity = none
