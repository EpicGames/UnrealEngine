<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <Import
    Project="$([MSBuild]::GetPathOfFileAbove('Directory.Build.props', '$(MSBuildThisFileDirectory)../'))"
    Condition="'' != $([MSBuild]::GetPathOfFileAbove('Directory.Build.props', '$(MSBuildThisFileDirectory)../'))" />

  <Import Project="UnrealEngine.csproj.props" />

  <PropertyGroup>
    <!-- Current and future projects in Shared should enable nullable annotations,
      but this is not true of everything that imports UnrealEngine.csproj.props (yet). -->
    <Nullable>enable</Nullable>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="$(MSBuildThisFileDirectory)MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>

  <!-- Non-test project default properties -->
  <PropertyGroup Condition="!$(MSBuildProjectName.EndsWith('.Tests'))">
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
  </PropertyGroup>

  <!-- Test project default properties -->
  <PropertyGroup Condition="$(MSBuildProjectName.EndsWith('.Tests'))">
    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
    <!-- We don't require public documentation for test projects -->
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <!-- Test project default includes -->
  <ItemGroup Condition="$(MSBuildProjectName.EndsWith('.Tests')) OR $(MSBuildProjectName.EndsWith('.Fixture'))">
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="18.4.0">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestAdapter" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestFramework" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="coverlet.collector" Version="8.0.1">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
  </ItemGroup>
</Project>
