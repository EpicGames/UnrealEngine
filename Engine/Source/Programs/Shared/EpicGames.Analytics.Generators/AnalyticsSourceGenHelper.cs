// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;

namespace EpicGames.Analytics.Generators
{
	/// <summary>
	/// General source gen helper associated with analytics tables.
	/// </summary>
	public static class AnalyticsSourceGenHelper
	{
		/// <summary>
		/// Analytics Generator namespace.
		/// </summary>
		public const string AnalyticsTableGenNamespace = "EpicGames.Analytics.Generated";

		/// <summary>
		/// Analytics Generator prefix used in generated files.
		/// </summary>
		public const string AnalyticsTableGenPrefix = "AnalyticsTableGen";

		/// <summary>
		/// Shorthand prefix used in generated files.
		/// </summary>
		public const string AnalyticsTableGenSourcePrefix = "AGen";

		/// <summary>
		/// Generates the version file for the table gen.
		/// </summary>
		/// <returns>string representing the version.</returns>
		public static string GenerateAnalyticsTableGenVersionFile()
		{
			StringBuilder sb = new();
			sb.Append($$"""
				namespace {{AnalyticsTableGenNamespace}}
				{
					/// <summary>
					/// Descriptor class for generated metadata.
					/// </summary>
					public static class AnalyticsTableGen
					{
						/// <summary>
						/// The version of the table gen.
						/// </summary>
						public const int Version=1;
					}
				}
				""");

			return sb.ToString();
		}

		/// <summary>
		/// Constructs a analytics table source file from the provided object.
		/// </summary>
		/// <param name="analyticstableToGenerate">The analytics table definition.</param>
		/// <returns>The c-sharp representation of the object.</returns>
		public static string GenerateAnalyticsTable(AnalyticsTableToGenerate analyticstableToGenerate)
		{
			{
				StringBuilder sb = new();
				sb.Append($$"""
				namespace {{AnalyticsTableGenNamespace}}
				{
					/// <summary>
					/// Generated Table Descriptor for {{analyticstableToGenerate.Name}}.
					/// </summary>
					public static class {{analyticstableToGenerate.Name}}Gen
					{

				""");

				sb.AppendLine($"\t\t/// <summary>");
				sb.AppendLine($"\t\t///Table name.");
				sb.AppendLine($"\t\t/// </summary>");
				sb.AppendLine($"\t\tpublic const string TableName= \"{analyticstableToGenerate.TableName}\";");

				sb.AppendLine($"\t\t/// <summary>");
				sb.AppendLine($"\t\t///Schema name.");
				sb.AppendLine($"\t\t/// </summary>");
				sb.AppendLine($"\t\tpublic const string SchemaName= \"{analyticstableToGenerate.Schema}\";");

				foreach (Tuple<string, string> member in analyticstableToGenerate.ColumnsNames)
				{
					sb.AppendLine($"\t\t/// <summary>");
					sb.AppendLine($"\t\t/// Column descriptor for {member.Item1}.");
					sb.AppendLine($"\t\t/// </summary>");
					sb.AppendLine($"\t\tpublic const string {member.Item1} = \"{member.Item2}\";");
				}

				sb.AppendLine("	}");
				sb.AppendLine("}");

				return sb.ToString();
			}
		}
	}
}
