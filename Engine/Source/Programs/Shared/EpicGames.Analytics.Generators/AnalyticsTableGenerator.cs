// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Linq;
using System.Text;
using Microsoft.CodeAnalysis;
using Microsoft.CodeAnalysis.CSharp.Syntax;
using Microsoft.CodeAnalysis.Text;

namespace EpicGames.Analytics.Generators
{
	/// <summary>
	/// DTO that describes a analytics table.
	/// </summary>
	public readonly record struct AnalyticsTableToGenerate
	{
		/// <summary>
		/// The name of the telemetry record.
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// The schema/database name (e.g., "ingest").
		/// </summary>
		public readonly string Schema;

		/// <summary>
		/// The table name (e.g., "horde.state_job_summary").
		/// </summary>
		public readonly string TableName;

		/// <summary>
		/// The column names.
		/// </summary>
		public readonly List<Tuple<string, string>> ColumnsNames;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="name">Name of the telemetry record.</param>
		/// <param name="schema">Schema/database name.</param>
		/// <param name="tableName">Name of the table.</param>
		/// <param name="propsValues">List of the property and column name pairs.</param>
		public AnalyticsTableToGenerate(string name, string schema, string tableName, List<Tuple<string, string>> propsValues)
		{
			Name = name;
			Schema = schema;
			TableName = tableName;
			ColumnsNames = propsValues;
		}
	}

	/// <summary>
	/// Generator class.
	/// </summary>
	[Generator]
	public class AnalyticsTableGenerator : ISourceGenerator
	{
#pragma warning disable RS2008 // Enable analyzer release tracking
		private static readonly DiagnosticDescriptor s_descriptor = new(id: "HORDEATG01", title: "Unexpected Error", messageFormat: "Error for object: {0}", category: "Design", defaultSeverity: DiagnosticSeverity.Warning, isEnabledByDefault: true);
#pragma warning restore RS2008 // Enable analyzer release tracking

		/// <inheritdoc/>
		public void Initialize(GeneratorInitializationContext context)
		{
			context.RegisterForSyntaxNotifications(() => new SyntaxReceiver());
		}

		/// <inheritdoc/>
		public void Execute(GeneratorExecutionContext context)
		{
			if (context.SyntaxReceiver is not SyntaxReceiver)
			{
				return;
			}

			context.AddSource(AnalyticsSourceGenHelper.AnalyticsTableGenPrefix + ".g.cs", SourceText.From(AnalyticsSourceGenHelper.GenerateAnalyticsTableGenVersionFile(), Encoding.UTF8));

			// Search all named types in the compilation; we currently must do this because of an indirection in dependencies. Ideally this analyzer is applied to a EpicGames.Analytics.Telemetry project, which contains the neccesssary references.
			IEnumerable<INamedTypeSymbol> allTypes = GetAllNamedTypes(context.Compilation.GlobalNamespace);

			// Hint names become on-disk filenames (obj/.../generated/<asm>/<gen-fqn>/<hint>). We use
			// the simple type name only — not attg.Value.Name, which carries the namespace path —
			// to keep the full path under Windows MAX_PATH (260). The generated class identifier
			// still uses the namespaced name, so consumers (EpicGames_Analytics_*Gen references)
			// are unaffected. emittedHintNames disambiguates simple-name collisions across
			// namespaces with a deterministic counter suffix.
			HashSet<string> emittedHintNames = [];

			foreach (INamedTypeSymbol typeSymbol in allTypes)
			{
				ImmutableArray<AttributeData> attributes = typeSymbol.GetAttributes();
				if (attributes.Length == 0)
				{
					continue;
				}

				if (!attributes.Any(x => x.AttributeClass?.ToDisplayString() == "EpicGames.Analytics.Telemetry.AnalyticsTableGenAttribute"))
				{
					continue;
				}

				AnalyticsTableToGenerate? attg = BuildAnalyticsTable(typeSymbol);

				if (attg == null)
				{
					context.ReportDiagnostic(Diagnostic.Create(s_descriptor, Location.None, $"Failed to produce table for type: {typeSymbol.ToDisplayString()}"));
					continue;
				}

				string result = AnalyticsSourceGenHelper.GenerateAnalyticsTable(attg.Value);

				string hintBase = AnalyticsSourceGenHelper.AnalyticsTableGenSourcePrefix + "." + typeSymbol.Name;
				string hintName = hintBase + ".g.cs";
				int dedupeSuffix = 2;
				while (!emittedHintNames.Add(hintName))
				{
					hintName = $"{hintBase}_{dedupeSuffix}.g.cs";
					dedupeSuffix++;
				}

				context.AddSource(hintName, SourceText.From(result, Encoding.UTF8));
			}
		}

		private class SyntaxReceiver : ISyntaxReceiver
		{
			public List<TypeDeclarationSyntax> CandidateTypes { get; } = [];

			public void OnVisitSyntaxNode(SyntaxNode syntaxNode)
			{
				if (syntaxNode is TypeDeclarationSyntax tds && tds.AttributeLists.Count > 0)
				{
					CandidateTypes.Add(tds);
				}
			}
		}

#nullable enable
		private static string? BuildFullNamespace(INamespaceSymbol? symbol)
		{
			if (symbol == null)
			{
				return null;
			}
			string? retString = BuildFullNamespace(symbol.ContainingNamespace);

			return !String.IsNullOrEmpty(retString) ? $"{retString}_{symbol.Name}" : symbol.Name;
		}
#nullable disable

		private static AnalyticsTableToGenerate? BuildAnalyticsTable(INamedTypeSymbol telemetrySymbol)
		{
#nullable enable
			string? fullNamepsace = BuildFullNamespace(telemetrySymbol.ContainingNamespace);
			string telemetryName = !String.IsNullOrEmpty(fullNamepsace) ? $"{fullNamepsace}_{telemetrySymbol.Name}" : telemetrySymbol.Name;
			string? tableName = null;
			string? schema = null;

			List<Tuple<string, string>> columns = [];

			foreach (AttributeData attr in telemetrySymbol.GetAttributes())
			{
				if (attr.AttributeClass?.ToDisplayString() == "System.ComponentModel.DataAnnotations.Schema.TableAttribute")
				{
					if (attr.ConstructorArguments.Length > 0)
					{
						tableName = attr.ConstructorArguments[0].Value as string;
					}
					// Extract Schema from named arguments
					foreach (KeyValuePair<string, TypedConstant> namedArg in attr.NamedArguments)
					{
						if (namedArg.Key == "Schema")
						{
							schema = namedArg.Value.Value as string;
						}
					}
				}
			}

			ITypeSymbol? current = telemetrySymbol;
			while (current != null)
			{
				foreach (IPropertySymbol prop in current.GetMembers().OfType<IPropertySymbol>())
				{
					string propertyName = prop.Name;
					string? columnName = null;

					foreach (AttributeData attr in prop.GetAttributes())
					{
						if (attr.AttributeClass?.ToDisplayString() == "System.ComponentModel.DataAnnotations.Schema.ColumnAttribute")
						{
							if (attr.ConstructorArguments.Length > 0)
							{
								columnName = attr.ConstructorArguments[0].Value as string;
							}
						}
					}

					if (columnName != null)
					{
						columns.Add(new Tuple<string, string>(propertyName, columnName));
					}
				}

				current = current.BaseType;
			}

			return String.IsNullOrEmpty(tableName) ? null : new AnalyticsTableToGenerate(telemetryName, schema ?? String.Empty, tableName!, columns);
#nullable disable
		}

		private static IEnumerable<INamedTypeSymbol> GetAllNamedTypes(INamespaceSymbol ns)
		{
			foreach (INamedTypeSymbol type in ns.GetTypeMembers())
			{
				yield return type;

				foreach (INamedTypeSymbol nested in GetAllNestedTypes(type))
				{
					yield return nested;
				}
			}

			foreach (INamespaceSymbol childNs in ns.GetNamespaceMembers())
			{
				foreach (INamedTypeSymbol type in GetAllNamedTypes(childNs))
				{
					yield return type;
				}
			}
		}

		private static IEnumerable<INamedTypeSymbol> GetAllNestedTypes(INamedTypeSymbol type)
		{
			foreach (INamedTypeSymbol nested in type.GetTypeMembers())
			{
				yield return nested;

				foreach (INamedTypeSymbol deeper in GetAllNestedTypes(nested))
				{
					yield return deeper;
				}
			}
		}
	}
}