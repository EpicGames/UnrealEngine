<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <Deterministic>true</Deterministic>
    <RootNamespace>EpicGames.Analytics</RootNamespace>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Analytics.Telemetry\EpicGames.Analytics.Telemetry.csproj" />
    <ProjectReference Include="..\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>
</Project>
