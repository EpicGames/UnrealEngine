// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics.PerformanceTrends;
using EpicGames.Analytics.Telemetry;

namespace EpicGames.Analytics
{
	/// <summary>
	/// Telemetry record that describes key stats for performance tests.
	/// </summary>
	[AnalyticsTableGen]
	[TelemetryEvent(DefaultEventName)]
	[Table("horde.state_performance_key_stats", Schema = "ingest")]
	public record KeyStatTelemetryRecord : PerformanceTrendTelemetry
	{
		/// <summary>
		/// Default event name for the KeyStatTelemetryRecord.
		/// </summary>
		public const string DefaultEventName = "State.Performance.KeyStats";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		///Dynamic resolution max percentage.
		/// </summary>
		[Column("dynamic_resolution_percentage_max")]
		public int DynamicResolutionPercentageMax { get; init; }

		/// <summary>
		/// Csv id used to create this record.
		/// </summary>
		[Column("csvid")]
		public string? CsvId { get; init; }

		/// <summary>
		/// Average GPU time.
		/// </summary>
		[Column("gp_utime_avg")]
		public float GpuTimeAvg { get; init; }

		/// <summary>
		/// Max physical memory used.
		/// </summary>
		[Column("physical_used_mb_max")]
		public float PhysicalUsedMbMax { get; init; }

		/// <summary>
		/// Dynamic resolution through Nanite.
		/// </summary>
		[Column("dynamic_resolution_vsm_nanite")]
		public int DynamicResolutionVsmNanite { get; init; }

		/// <summary>
		/// MVP.
		/// </summary>
		[Column("mvp")]
		public float MVP { get; init; }

		/// <summary>
		/// Hitches per minute.
		/// </summary>
		[Column("hitches_min")]
		public float HitchesMin { get; init; }

		/// <summary>
		/// Hitch time percent.
		/// </summary>
		[Column("hitch_time_percent")]
		public float HitchTimePercent { get; init; }

		/// <summary>
		/// Game thread time average.
		/// </summary>
		[Column("game_threadtime_avg")]
		public float GameThreadtimeAvg { get; init; }

		/// <summary>
		/// Render thread time average.
		/// </summary>
		[Column("render_threadtime_avg")]
		public float RenderThreadtimeAvg { get; init; }

		/// <summary>
		/// Frame time average.
		/// </summary>
		[Column("frametime_avg")]
		public float FrametimeAvg { get; init; }

		/// <summary>
		/// Dynamic resolution percentage average.
		/// </summary>
		[Column("dynamic_resolution_percentage_avg")]
		public float DynamicResolutionPercentageAvg { get; init; }

		/// <summary>
		/// No arg constructor for ORM construction.
		/// </summary>
		/// <remarks>Used for ORM instantiation.</remarks>
		public KeyStatTelemetryRecord() : base()
		{
			EventName = DefaultEventName;
			SchemaVersion = CurrentSchemaVersion;
		}
	}
}