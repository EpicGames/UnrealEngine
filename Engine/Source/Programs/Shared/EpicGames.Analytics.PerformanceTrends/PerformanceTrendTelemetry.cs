// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Telemetry;

namespace EpicGames.Analytics.PerformanceTrends
{
	/// <summary>
	/// Utility class to handle standard telemetry operations, and metadata processing.
	/// </summary>
	public static class StandardTelemetryMetadata
	{
		#region -- Internal Constants --

		internal const string EngineVersion = "engineversion";
		internal const string EngineReleaseVersion = "enginereleaseversion";
		internal const string BuildVersion = "buildversion";
		internal const string Platform = "platform";
		internal const string TestId = "testid";
		internal const string TestName = "testname";
		internal const string TestConfigName = "testconfigname";
		internal const string GauntletTestType = "gauntlettesttype";
		internal const string GauntletSubTest = "gauntletsubtest";
		internal const string Branch = "branch";
		internal const string Changelist = "changelist";
		internal const string StartTimestamp = "starttimestamp";
		internal const string EndTimestamp = "endtimestamp";
		internal const string BuildSuffix = "buildsuffix";
		internal const string TestBuildIsPreflight = "testbuildispreflight";
		internal const string Cpu = "cpu";
		internal const string Gpu = "gpu";
		internal const string VerbatimRhiName = "verbatimrhiname";
		internal const string DeviceProfile = "deviceprofile";

		#endregion -- Internal Constants --

		#region -- Public Members --

		/// <summary>
		/// Sanitizes keys of the provided source hashtable.
		/// </summary>
		/// <param name="source">The source hashtable to sanitize.</param>
		/// <returns>A copy of the hashtable with all keys sanitized.</returns>
		public static Hashtable? SanitizeHashtable(Hashtable source)
		{
			if (source == null)
			{
				return null;
			}

			Hashtable sanitized = new(source.Count);

			foreach (DictionaryEntry entry in source)
			{
				if (entry.Key is string sourceKey)
				{
					string sanitizedKey = sourceKey.Trim().Replace("/", "_", StringComparison.Ordinal)
							.Replace(" ", "_", StringComparison.Ordinal)
							.Replace("<", "_lt_", StringComparison.Ordinal)
							.Replace(">", "_gt_", StringComparison.Ordinal);

					// Verify that after sanitization we don't already have this sourceKey, otherwise suffix the sourceKey.
					string finalKey = sanitizedKey;
					bool wasSanitized = sourceKey != finalKey;
					int suffix = 1;

					// If we have actually sanitized, it means we are at risk of flattening an existing sourceKey.
					// Check to see whether the final sourceKey already exists within our source; if so it means we created a sanitized sourceKey that overwrites something existing (because we know we've sanitized).
					// Check to see that we also haven't generated a sanitized sourceKey that overlaps with another sanitized sourceKey.
					while (wasSanitized && (source.ContainsKey(finalKey) || sanitized.ContainsKey(finalKey)))
					{
						suffix++;
						finalKey = $"{sanitizedKey}_{suffix}";
					}

					sanitized[finalKey] = entry.Value;
				}
				else
				{
					sanitized[entry.Key] = entry.Value;
				}
			}

			return sanitized;
		}

		/// <summary>
		/// The standard metadata keys.
		/// </summary>
		public static readonly string[] StandardMetadataKeys =
		[
				EngineVersion,
				EngineReleaseVersion,
				BuildVersion,
				Platform,
				TestId,
				TestName,
				TestConfigName,
				GauntletTestType,
				GauntletSubTest,
				Branch,
				Changelist,
				StartTimestamp,
				EndTimestamp,
				BuildSuffix,
				TestBuildIsPreflight,
				Gpu,
				Cpu,
				VerbatimRhiName,
				DeviceProfile
		];

		/// <summary>
		/// Creates a new hashtable only with the required metadata entries retained (if they exist).
		/// </summary>
		/// <param name="sourceTable">The source table.</param>
		/// <returns>Hashtable with the required metadata entries.</returns>
		public static Hashtable CreateFilteredHashtable(Hashtable sourceTable)
		{
			Hashtable returnTable = [];

			for (int i = 0; i < StandardMetadataKeys.Length; i++)
			{
				string key = StandardMetadataKeys[i];
				if (sourceTable.ContainsKey(key))
				{
					returnTable[key] = sourceTable[key];
				}
			}

			return returnTable;
		}

		#endregion -- Public Members --
	}

	/// <summary>
	/// Abstract record that describes the shared metadata for performance trend telemetry.
	/// </summary>
	/// <remarks>Whilst this table specifies <see cref="AnalyticsTableGenAttribute"/>, it does so for a narrow use in shared query context, and not against any specific <see cref="PerformanceTrendTelemetry"/> table.</remarks>
	[AnalyticsTableGen]
	[Table("default")]
	public abstract record PerformanceTrendTelemetry : HordeContextTelemetryRecord
	{
		#region -- Public Members --

		/// <summary>
		/// The engine version of the telemetry.
		/// </summary>
		[Column("engineversion")]
		public string? EngineVersion { get; init; }

		/// <summary>
		/// The engine release version of the telemetry.
		/// </summary>
		[Column("enginereleaseversion")]
		public string? EngineReleaseVersion { get; init; }

		/// <summary>
		/// The build version of the telemetry.
		/// </summary>
		[Column("buildversion")]
		public string? BuildVersion { get; init; }

		/// <summary>
		/// The platform of the telemetry.
		/// </summary>
		[Column("platform")]
		public string? Platform { get; init; }

		/// <summary>
		/// The test id for the telemetry, if applicable.
		/// </summary>
		[Column("testid")]
		public string? TestId { get; init; }

		/// <summary>
		/// The test name for the telemetry, if applicable.
		/// </summary>
		[Column("testname")]
		public string? TestName { get; init; }

		/// <summary>
		/// The test config name for the telemetry, if applicable.
		/// </summary>
		[Column("testconfigname")]
		public string? TestConfigName { get; init; }

		/// <summary>
		/// The test type, if applicable.
		/// </summary>
		[Column("gauntlettesttype")]
		public string? GauntletTestType { get; init; }

		/// <summary>
		/// The sub test type, if applicable.
		/// </summary>
		[Column("gauntletsubtest")]
		public string? GauntletSubTest { get; init; }

		/// <summary>
		/// Whether this represents collated data, or not.
		/// </summary>
		[Column("collated")]
		public bool Collated { get; init; }

		/// <summary>
		/// The branch the underlying test build was produced from.
		/// </summary>
		[Column("branch")]
		public string? Branch { get; init; }

		/// <summary>
		/// The changelist the underlying test build was produced from.
		/// </summary>
		[Column("changelist")]
		public int? Changelist { get; init; }

		/// <summary>
		/// The start timestamp of the performance trend session.
		/// </summary>
		/// <remarks>Time since epoch.</remarks>
		[Column("starttimestamp")]
		public long? StartTimestamp { get; init; }

		/// <summary>
		/// The end timestamp of the performance trend session.
		/// </summary>
		/// <remarks>Time since epoch.</remarks>
		[Column("endtimestamp")]
		public long? EndTimestamp { get; init; }

		/// <summary>
		/// The build suffix for the build string.
		/// </summary>
		/// <remarks>This typically refers to the test product name.</remarks>
		[Column("buildsuffix")]
		public string? BuildSuffix { get; init; }

		/// <summary>
		/// Int boolean representation of whether the test build was generated by preflight or not.
		/// </summary>
		[Column("testbuildispreflight")]
		public int TestBuildIsPreflight { get; init; }

		/// <summary>
		/// The CPU of the hardware the performance telemetry was generated from.
		/// </summary>
		[Column("cpu")]
		public string? Cpu { get; init; }

		/// <summary>
		/// The GPU of the hardware the performance telemetry was generated from.
		/// </summary>
		[Column("gpu")]
		public string? Gpu { get; init; }

		/// <summary>
		/// The RHI name of the hardware the performance telemetry was generated from.
		/// </summary>
		[Column("verbatimrhiname")]
		public string? VerbatimRhiName { get; init; }

		/// <summary>
		/// The device profile the performance telemetry was generated from.
		/// </summary>
		[Column("deviceprofile")]
		public string? DeviceProfile { get; init; }

		/// <summary>
		/// The data source of the performance trend telemetry.
		/// </summary>
		[Column("test_identity")]
		public string? TestIdentity { get; init; }

		/// <summary>
		/// Summary table name.
		/// </summary>
		[Column("summary_name")]
		public string SummaryName { get; init; }

		#region -- Computed Properties --

		/// <summary>
		/// The semantic platform the telemetry was produced on.
		/// </summary>
		/// <remarks>
		/// This represents the coalesced result of <see cref="TestConfigName"/> and <see cref="Platform"/>.
		/// <see cref="TestConfigName"/> represent the modern way of disambiguating between platforms (and within platforms - ex. Windows-Vulkan and !Vulkan), whereas <see cref="Platform"/> represents a historical model.
		/// </remarks>
		[Column("computed_platform")]
		public string? ComputedPlatform { get; init; }

		/// <summary>
		/// The stream the underlying build was produced from.
		/// </summary>
		/// <remarks>
		///	This represents the coalesced result of <see cref="HordeContextTelemetryRecord.StreamId"/> and <see cref="Branch"/>.
		///	Common local user invocations that result in telemetry emissions may not have a stream id, but will always have a branch.
		/// </remarks>
		[Column("computed_stream")]
		public string? ComputedStream { get; init; }

		#endregion -- Computed Properties --

		#endregion -- Public Members --

		/// <summary>
		/// No arg constructor for ORM construction.
		/// </summary>
		/// <remarks>Used for ORM instantiation.</remarks>
		protected PerformanceTrendTelemetry() : base()
		{
			SummaryName = String.Empty;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="source">The source table to construct the telemetry record from.</param>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version of the telemetry.</param>
		/// <param name="sessionId">The id used to correlate telemetry emitted in the same session.</param>
		/// <param name="sessionLabel">The label used to correlate telemetry emitted across multiple sessions.</param>
		/// <param name="isBuildMachine">True whether this is a build machine context, false otherwise.</param>
		/// <param name="hordeUrlStr">The horde url.</param>
		/// <param name="streamId">The stream id.</param>
		/// <param name="templateId">The template id.</param>
		/// <param name="jobId">The Horde job id.</param>
		/// <param name="stepId">The Horde step id.</param>
		/// <param name="commitIdOrdered">The ordered commitId.</param>
		/// <param name="commitId">The commitId.</param>
		/// <param name="summaryName">The summary name.</param>
		/// <param name="collated">Whether this record represents pre-aggregation.</param>
		/// <param name="testIdentity">The context in which the performance trend was generated.</param>
#pragma warning disable CA1054 // URI-like parameters should not be strings
		protected PerformanceTrendTelemetry(Hashtable source, string eventName, int schemaVersion, string? sessionId, string? sessionLabel, bool isBuildMachine, string? hordeUrlStr, string? streamId, string? templateId, string? jobId, string? stepId, int? commitIdOrdered, string? commitId, string summaryName, bool collated, string? testIdentity)
			: base(eventName, schemaVersion, sessionId, sessionLabel, isBuildMachine, hordeUrlStr, streamId, templateId, jobId, stepId, commitIdOrdered, commitId)
#pragma warning restore CA1054 // URI-like parameters should not be strings
		{
			SummaryName = summaryName;
			Collated = collated;
			TestIdentity = testIdentity;

			EngineVersion = GetString(source, StandardTelemetryMetadata.EngineVersion);
			EngineReleaseVersion = GetString(source, StandardTelemetryMetadata.EngineReleaseVersion);
			BuildVersion = GetString(source, StandardTelemetryMetadata.BuildVersion);
			Platform = GetString(source, StandardTelemetryMetadata.Platform);
			TestId = GetString(source, StandardTelemetryMetadata.TestId);
			TestName = GetString(source, StandardTelemetryMetadata.TestName);
			TestConfigName = GetString(source, StandardTelemetryMetadata.TestConfigName);
			GauntletTestType = GetString(source, StandardTelemetryMetadata.GauntletTestType);
			GauntletSubTest = GetString(source, StandardTelemetryMetadata.GauntletSubTest);
			Branch = GetString(source, StandardTelemetryMetadata.Branch);
			Changelist = GetInt(source, StandardTelemetryMetadata.Changelist);
			StartTimestamp = GetLong(source, StandardTelemetryMetadata.StartTimestamp);
			EndTimestamp = GetLong(source, StandardTelemetryMetadata.EndTimestamp);
			BuildSuffix = GetString(source, StandardTelemetryMetadata.BuildSuffix);
			TestBuildIsPreflight = GetInt(source, StandardTelemetryMetadata.TestBuildIsPreflight) ?? 0;
			Cpu = GetString(source, StandardTelemetryMetadata.Cpu);
			Gpu = GetString(source, StandardTelemetryMetadata.Gpu);
			VerbatimRhiName = GetString(source, StandardTelemetryMetadata.VerbatimRhiName);
			DeviceProfile = GetString(source, StandardTelemetryMetadata.DeviceProfile);
		}

		#region -- Private API --

		private static string? GetString(Hashtable table, string key)
		{
			if (!table.ContainsKey(key))
			{
				return null;
			}

			return table[key] as string;
		}

		private static int? GetInt(Hashtable table, string key)
		{
			if (!table.Contains(key))
			{
				return null;
			}

			try
			{
				object? value = table[key];

				if (value == null)
				{
					return null;
				}

				return Convert.ToInt32(value);
			}
			catch
			{
				return null;
			}
		}

		private static long? GetLong(Hashtable table, string key)
		{
			if (!table.Contains(key))
			{
				return null;
			}

			try
			{
				object? value = table[key];

				if (value == null)
				{
					return null;
				}

				return Convert.ToInt64(value);
			}
			catch
			{
				return null;
			}
		}

		#endregion -- Private API --
	}
}