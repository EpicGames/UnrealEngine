// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.ComponentModel.DataAnnotations.Schema;

namespace EpicGames.Analytics
{
	/// <summary>
	/// Describes the base elements required for telemetry.
	/// </summary>
	public abstract record AbstractTelemetryRecord
	{
		#region -- Constants --

		/// <summary>
		/// Canonical column name for <see cref="TelemetryTimestamp"/>. Read-side query code references this constant so consumer plugins (StructuredAnalytics, PerformanceTrends, ...) never carry a string literal that could drift from the wire format.
		/// </summary>
		public const string TelemetryTimestampColumnName = "telemetry_timestamp";

		#endregion -- Constants --

		#region -- Public Properties --

		/// <summary>
		/// The telemetry event name.
		/// </summary>
		[Column("event_name")]
		public string EventName { get; init; }

		/// <summary>
		/// The schema version of the telemetry.
		/// </summary>
		[Column("schema_version")]
		public int SchemaVersion { get; init; }

		/// <summary>
		/// When the telemetry record was created. Stamped to <see cref="DateTime.UtcNow"/> by the constructor so any record instance implicitly carries a value; producers can override via record-init for backfill / replay scenarios. Distinct from datarouter's <c>zz_received_timestamp</c> column, which captures the upstream pipeline's receive time.
		/// </summary>
		[Column(TelemetryTimestampColumnName)]
		public DateTime TelemetryTimestamp { get; init; }

		#endregion -- Public Properties --

		#region -- Public Api --

		/// <summary>
		/// Produces a hashtable representation of the telemetry record.
		/// </summary>
		/// <returns>Hashtable representing the underyling telemetry record.</returns>
		/// <remarks>
		///		This is used primarily for interop in <see cref="Hashtable"/> oriented systems.
		///		For more compositional telemetry structs, utilize <see cref="System.Text.Json.Serialization.JsonConverter"/>.
		/// </remarks>
		public Hashtable ToHashtable()
		{
			Hashtable result = [];
			WriteProperties(result);
			return result;
		}

		#endregion -- Public Api --

		#region -- Protected Api --

		/// <summary>
		/// No arg constructor for ORM construction.
		/// </summary>
		/// <remarks>used for ORM instantiation.</remarks>
		protected AbstractTelemetryRecord()
		{
			EventName = System.String.Empty;
			SchemaVersion = -1;
			TelemetryTimestamp = DateTime.UtcNow;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version.</param>
		protected AbstractTelemetryRecord(string eventName, int schemaVersion)
		{
			EventName = eventName;
			SchemaVersion = schemaVersion;
			TelemetryTimestamp = DateTime.UtcNow;
		}

		/// <summary>
		/// Template pattern method that appends serializable properties to the in hashtable.
		/// </summary>
		/// <param name="hashtable">The input hashtable to apply properties to.</param>
		/// <remarks>When overriding, a call to the base should be made.</remarks>
		protected virtual void WriteProperties(Hashtable hashtable)
		{
			hashtable[nameof(EventName)] = EventName;
			hashtable[nameof(SchemaVersion)] = SchemaVersion;
			hashtable[nameof(TelemetryTimestamp)] = TelemetryTimestamp;
		}

		#endregion -- Protected Api --
	}
}