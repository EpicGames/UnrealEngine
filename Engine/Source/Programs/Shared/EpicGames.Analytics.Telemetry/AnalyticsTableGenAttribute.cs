// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Analytics.Telemetry
{
	/// <summary>
	/// Attribute to annoatate that table generation should run on the type.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct)]
	public sealed class AnalyticsTableGenAttribute : System.Attribute { }
}