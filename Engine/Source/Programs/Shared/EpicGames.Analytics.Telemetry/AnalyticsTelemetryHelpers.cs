// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections;

namespace EpicGames.Analytics
{
	/// <summary>
	/// Common utilities for working with telemetry records.
	/// </summary>
	public static class AnalyticsTelemetryHelpers
	{
		/// <summary>
		/// Merges the right hashtable into the left, without mutating the left hashtable.
		/// </summary>
		/// <param name="left">The target hashtable to be merged into.</param>
		/// <param name="right">The hashtable to merge from.</param>
		/// <param name="overrideTarget">True to override keys in left, false otherwise.</param>
		/// <returns>The left hashtable with all the right table's elements merged in.</returns>
		public static Hashtable MergeLeftCopy(Hashtable left, Hashtable right, bool overrideTarget = true)
		{
			Hashtable result = new(left);

			MergeLeft(result, right, overrideTarget);

			return result;
		}

		/// <summary>
		/// Merges the right hashtable into the left.
		/// </summary>
		/// <param name="left">The target hashtable to be merged into.</param>
		/// <param name="right">The hashtable to merge from.</param>
		/// <param name="overrideTarget">True to override keys in left, false otherwise.</param>
		/// <returns>The left hashtable with all the right table's elements merged in. Overwrites the values in the left.</returns>
		public static Hashtable MergeLeft(Hashtable left, Hashtable right, bool overrideTarget = true)
		{
			if (left == right)
			{
				return left;
			}

			foreach (DictionaryEntry entry in right)
			{
				if (!overrideTarget && left.ContainsKey(entry.Key))
				{
					continue;
				}

				left[entry.Key] = entry.Value;
			}

			return left;
		}
	}
}