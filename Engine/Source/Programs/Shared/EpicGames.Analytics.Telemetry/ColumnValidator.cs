// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations.Schema;
using System.Reflection;
using System.Threading;

namespace EpicGames.Analytics
{
	/// <summary>
	/// Utility class used to validate the payload contents against specific record types, ensuring the presence of <see cref="ColumnAttribute"/> within different payload types.
	/// </summary>
	/// <typeparam name="T">The record type to validate against.</typeparam>
	public static class ColumnValidator<T>
	{
		private static readonly Lock s_initLock = new();
		private static HashSet<string>? s_columnKeys = null;
		private static HashSet<string>? s_propertyKeys = null;

		private static void InitializeKeys()
		{
			if (s_columnKeys != null && s_propertyKeys != null)
			{
				return;
			}

			lock (s_initLock)
			{
				HashSet<string> columnKeys = new(16, StringComparer.OrdinalIgnoreCase);
				HashSet<string> propertyKeys = new(16, StringComparer.OrdinalIgnoreCase);

				PropertyInfo[] props = typeof(T).GetProperties(
					BindingFlags.Instance |
					BindingFlags.Public |
					BindingFlags.NonPublic |
					BindingFlags.DeclaredOnly);

				foreach (PropertyInfo prop in props)
				{
					ColumnAttribute? columnAttr = prop.GetCustomAttribute<ColumnAttribute>();

					if (columnAttr != null)
					{
						// Safely fall back to the column key being the prop name if no name attribute.
						if (String.IsNullOrEmpty(columnAttr.Name))
						{
							columnKeys.Add(prop.Name.ToUpperInvariant());
						}
						else
						{
							columnKeys.Add(columnAttr.Name.ToUpperInvariant());
						}
						propertyKeys.Add(prop.Name.ToUpperInvariant());
					}
				}

				s_columnKeys = columnKeys;
				s_propertyKeys = propertyKeys;
			}
		}

		#region -- Private Static Api --

		private static HashSet<string> ReplicateAsHashSet(Hashtable source, bool fuzzyReplication)
		{
			HashSet<string> sourceKeys = new(source.Count, StringComparer.OrdinalIgnoreCase);
			foreach (object key in source.Keys)
			{
				if (key is string s)
				{
					// If fuzzy replication, we trim down the key and remove any unneccesary symbols since we tend to have clean property names.
					if (fuzzyReplication)
					{
						s = s.Trim()
							.Replace("_", "", StringComparison.Ordinal)
							.Replace(" ", "", StringComparison.Ordinal);
					}

					sourceKeys.Add(s);
				}
			}

			return sourceKeys;
		}

		private static bool HasCompleteKeySet(Hashtable source, HashSet<string> comparisonKeys, bool fuzzyMatch, List<string>? missingColumns)
		{
			HashSet<string> sourceKeys = ReplicateAsHashSet(source, fuzzyMatch);

			bool allPresent = true;
			foreach (string requiredKey in comparisonKeys!)
			{
				if (!sourceKeys.Contains(requiredKey))
				{
					missingColumns?.Add(requiredKey);
					allPresent = false;
				}
			}

			return allPresent;
		}

		#endregion -- Private Static Api --

		/// <summary>
		/// Validates whether an source payload of <see cref="Hashtable"/> type contains all the columns.
		/// </summary>
		/// <param name="source">The Hashtable to test.</param>
		/// <param name="missingColumns">List to store missing columns against.</param>
		/// <returns>True if all <see cref="ColumnAttribute.Name"/> in the type are contained within the source, false otherwise.</returns>
		/// <exception cref="ReflectionTypeLoadException">Can be thrown due to reflective inspection of <see cref="ColumnValidator{T}"/>.</exception>
		/// <remarks>This only validates at the current inheritance hierarchy of the <see cref="ColumnValidator{T}"/> in order to support narrow validation.</remarks>
#pragma warning disable CA1000 // Do not declare static members on generic types
		public static bool HasCompleteColumnSet(Hashtable source, List<string>? missingColumns)
#pragma warning restore CA1000 // Do not declare static members on generic types
		{
			if (s_columnKeys == null)
			{
				InitializeKeys();
			}

			// Column comparisons should *never* be fuzzy.
			return HasCompleteKeySet(source, s_columnKeys!, false, missingColumns);
		}

		/// <summary>
		/// Validates whether an source payload of <see cref="Hashtable"/> type contains all the columns.
		/// </summary>
		/// <param name="source">The Hashtable to test.</param>
		/// <param name="fuzzyMatch">Whether to use fuzzy matching for proprty comparison, or not.</param>
		/// <param name="missingColumns">List to store missing columns against.</param>
		/// <exception cref="ReflectionTypeLoadException">Can be thrown due to reflective inspection of <see cref="ColumnValidator{T}"/>.</exception>
		/// <returns>True if all properties annotated with <see cref="ColumnAttribute"/> in the type are contained within the source, false otherwise.</returns>
		/// <remarks>This only validates at the current inheritance hierarchy of the <see cref="ColumnValidator{T}"/> in order to support narrow validation.</remarks>
#pragma warning disable CA1000 // Do not declare static members on generic types
		public static bool HasCompletePropertySet(Hashtable source, bool fuzzyMatch, List<string>? missingColumns)
#pragma warning restore CA1000 // Do not declare static members on generic types
		{
			if (s_propertyKeys == null)
			{
				InitializeKeys();
			}

			return HasCompleteKeySet(source, s_propertyKeys!, fuzzyMatch, missingColumns);
		}
	}
}