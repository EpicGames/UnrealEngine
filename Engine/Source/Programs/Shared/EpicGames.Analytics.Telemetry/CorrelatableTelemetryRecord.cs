// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.ComponentModel.DataAnnotations.Schema;

namespace EpicGames.Analytics
{
	/// <summary>
	/// Record that describes a telemetry that can be correlated inter and intra session.
	/// </summary>
	public record CorrelatableTelemetryRecord : AbstractTelemetryRecord
	{
		#region -- Public Properties --

		/// <summary>
		/// A unique id used to correlate telemetry within a session.
		/// </summary>
		[Column("session_id")]
		public string? SessionId { get; init; }

		/// <summary>
		/// A label used to correlate telemetry across sessions.
		/// </summary>
		[Column("session_label")]
		public string? SessionLabel { get; init; }

		#endregion -- Public Properties --

		#region -- Public Api --

		/// <summary>
		/// No arg constructor for ORM construction.
		/// </summary>
		/// <remarks>used for ORM instantiation.</remarks>
		protected CorrelatableTelemetryRecord() : base() { }

		/// <summary>
		/// Constructor for correlatable telemetry.
		/// </summary>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version.</param>
		/// <param name="sessionId">The id used to correlate telemetry emitted in the same session.</param>
		/// <param name="sessionLabel">The label used to correlate telemetry emitted across multiple sessions.</param>
		public CorrelatableTelemetryRecord(string eventName, int schemaVersion, string? sessionId, string? sessionLabel) : base(eventName, schemaVersion)
		{
			SessionId = sessionId;
			SessionLabel = sessionLabel;
		}

		#endregion -- Public Api --

		#region -- Protected Api --

		/// <inheritdoc/>
		protected override void WriteProperties(Hashtable hashtable)
		{
			base.WriteProperties(hashtable);

			if (!String.IsNullOrEmpty(SessionId))
			{
				hashtable[nameof(SessionId)] = SessionId;
			}

			if (!String.IsNullOrEmpty(SessionLabel))
			{
				hashtable[nameof(SessionLabel)] = SessionLabel;
			}
		}

		#endregion -- Protected Api --
	}
}