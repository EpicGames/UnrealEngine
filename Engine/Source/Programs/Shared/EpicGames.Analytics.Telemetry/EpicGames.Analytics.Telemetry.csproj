<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <RootNamespace>EpicGames.Analytics</RootNamespace>
  </PropertyGroup>
</Project>
