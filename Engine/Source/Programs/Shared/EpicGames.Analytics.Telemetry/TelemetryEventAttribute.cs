// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Analytics.Telemetry
{
	/// <summary>
	/// Attribute to declare the event name for a telemetry record type.
	/// This allows mapping from event names (e.g., "State.JobSummary") to their corresponding types
	/// for schema resolution when processing JSON telemetry events.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct, AllowMultiple = false, Inherited = false)]
	public sealed class TelemetryEventAttribute : Attribute
	{
		/// <summary>
		/// The event name for this telemetry type (e.g., "State.JobSummary")
		/// </summary>
		public string EventName { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="eventName">The event name for this telemetry type</param>
		public TelemetryEventAttribute(string eventName)
		{
			EventName = eventName;
		}
	}
}
