// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.AspNetCore.Mvc;

namespace EpicGames.AspNet
{
	/// <summary>
	/// Provides methods for registering ASP.NET-specific compact binary converters
	/// </summary>
	public static class CbConvertersAspNet
	{
		/// <summary>
		/// Registers ASP.NET-specific converters for ProblemDetails and ValidationProblemDetails types
		/// </summary>
		public static void AddAspnetConverters()
		{
			CbConverter.AddConverter(typeof(ProblemDetails), new CbProblemDetailsConverter());
			CbConverter.AddConverter(typeof(ValidationProblemDetails), new CbValidationProblemDetailsConverter());
		}
	}

	/// <summary>
	/// Converter for asp.net problem details type
	/// </summary>
	class CbProblemDetailsConverter : CbConverter<ProblemDetails>
	{
		/// <inheritdoc/>
		public override ProblemDetails Read(CbField field)
		{
			if (!field.IsObject())
			{
				throw new CbException($"Error converting field \"{field.Name}\" to ProblemDetails. Expected CbObject.");
			}

			ProblemDetails result = new()
			{
				Title = field[new Utf8String("title")].AsString(),
				Detail = field[new Utf8String("detail")].AsString(),
				Type = field[new Utf8String("type")].AsString(),
				Instance = field[new Utf8String("instance")].AsString(),
				Status = field[new Utf8String("status")].AsInt32()
			};
			return result;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, ProblemDetails problemDetails)
		{
			writer.WriteObject(ToCbObject(problemDetails));
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, ProblemDetails problemDetails)
		{
			writer.WriteField(name, ToCbObject(problemDetails).AsField());
		}

		/// <summary>
		/// Converts a ProblemDetails instance to a CbObject representation
		/// </summary>
		/// <param name="problemDetails">The ProblemDetails to convert</param>
		/// <returns>A CbObject containing the serialized problem details</returns>
		private static CbObject ToCbObject(ProblemDetails problemDetails)
		{
			CbWriter objectWriter = new();
			objectWriter.BeginObject();

			objectWriter.WriteString(new Utf8String("title"), problemDetails.Title);
			if (!String.IsNullOrEmpty(problemDetails.Detail))
			{
				objectWriter.WriteString(new Utf8String("detail"), problemDetails.Detail);
			}

			if (!String.IsNullOrEmpty(problemDetails.Type))
			{
				objectWriter.WriteString(new Utf8String("type"), problemDetails.Type);
			}

			if (!String.IsNullOrEmpty(problemDetails.Instance))
			{
				objectWriter.WriteString(new Utf8String("instance"), problemDetails.Instance);
			}

			if (problemDetails.Status.HasValue)
			{
				objectWriter.WriteInteger(new Utf8String("status"), problemDetails.Status.Value);
			}

			objectWriter.EndObject();

			return objectWriter.ToObject();
		}
	}

	/// <summary>
	/// Converter for asp.net validation problem details type
	/// </summary>
	class CbValidationProblemDetailsConverter : CbConverter<ValidationProblemDetails>
	{
		/// <inheritdoc/>
		public override ValidationProblemDetails Read(CbField field)
		{
			if (!field.IsObject())
			{
				throw new CbException($"Error converting field \"{field.Name}\" to ValidationProblemDetails. Expected CbObject.");
			}

			Dictionary<string, string[]> errors = [];

			foreach (CbField error in field[new Utf8String("errors")].AsObject())
			{
				List<string> errorValuesList = [];
				
				foreach (CbField errorValue in error.AsArray())
				{
					errorValuesList.Add(errorValue.AsString());
				}
				
				errors[error.Name.ToString()] = [.. errorValuesList];
			}

			ValidationProblemDetails result = new()
			{
				Title = field[new Utf8String("title")].AsString(),
				Detail = field[new Utf8String("detail")].AsString(),
				Type = field[new Utf8String("type")].AsString(),
				Instance = field[new Utf8String("instance")].AsString(),
				Status = field[new Utf8String("status")].AsInt32(),
				Errors = errors
			};
			return result;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, ValidationProblemDetails validationProblemDetails)
		{
			writer.WriteObject(ToCbObject(validationProblemDetails));
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, ValidationProblemDetails validationProblemDetails)
		{
			writer.WriteField(name, ToCbObject(validationProblemDetails).AsField());
		}

		/// <summary>
		/// Converts a ValidationProblemDetails instance to a CbObject representation
		/// </summary>
		/// <param name="validationProblemDetails">The ValidationProblemDetails to convert</param>
		/// <returns>A CbObject containing the serialized validation problem details</returns>
		private static CbObject ToCbObject(ValidationProblemDetails validationProblemDetails)
		{
			CbWriter objectWriter = new();
			objectWriter.BeginObject();

			objectWriter.WriteString(new Utf8String("title"), validationProblemDetails.Title);
			if (!String.IsNullOrEmpty(validationProblemDetails.Detail))
			{
				objectWriter.WriteString(new Utf8String("detail"), validationProblemDetails.Detail);
			}

			if (!String.IsNullOrEmpty(validationProblemDetails.Type))
			{
				objectWriter.WriteString(new Utf8String("type"), validationProblemDetails.Type);
			}

			if (!String.IsNullOrEmpty(validationProblemDetails.Instance))
			{
				objectWriter.WriteString(new Utf8String("instance"), validationProblemDetails.Instance);
			}

			if (validationProblemDetails.Status.HasValue)
			{
				objectWriter.WriteInteger(new Utf8String("status"), validationProblemDetails.Status.Value);
			}

			if (validationProblemDetails.Errors.Count > 0)
			{
				objectWriter.BeginObject("errors");
				
				foreach (KeyValuePair<string, string[]> error in validationProblemDetails.Errors)
				{
					objectWriter.BeginArray(error.Key, CbFieldType.String);

					foreach (string errorValue in error.Value)
					{
						objectWriter.WriteStringValue(errorValue);
					}
					
					objectWriter.EndArray();
				}
				
				objectWriter.EndObject();
			}

			objectWriter.EndObject();

			return objectWriter.ToObject();
		}
	}
}
