// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.AspNetCore.Http;

namespace EpicGames.AspNet
{
	/// <summary>
	/// Utility methods for detecting and handling slow client send exceptions
	/// </summary>
	public static class ClientSendSlowExceptionUtil
	{
		/// <summary>
		/// Checks if a BadHttpRequestException indicates a slow client send and throws ClientSendSlowException if so
		/// </summary>
		/// <param name="e">The exception to check</param>
		/// <returns>True if the exception was not a slow send exception</returns>
		public static bool MaybeThrowSlowSendException(BadHttpRequestException e)
		{
			// if a user is sending data to slowly it will be terminated by asp.net and a bad http request exception is thrown
			// this type of exception can be thrown for multiple reasons so we check for the error message about MinRequestBodyDataRate
			// that setting can be set to reduce the number of bytes expected to be sent by a user - but that is likely a bad idea
			// as the most common reason for this happening is a client trying to send more data then what their uplink has bandwidth for.
			if (e.Message.Contains("MinRequestBodyDataRate", StringComparison.Ordinal))
			{
				throw new ClientSendSlowException(e);
			}

			// this exception also indicates a send issue, in this case the client sent fewer bytes then what it set as a content length
			// most likely caused by them shutting down their connection before finishing the entire send
			if (e.Message.Contains("Unexpected end of request content", StringComparison.Ordinal))
			{
				throw new ClientSendSlowException(e);
			}

			return true;
		}
	}

	/// <summary>
	/// Exception thrown when a client is sending data too slowly
	/// </summary>
	public class ClientSendSlowException : Exception
	{
		/// <summary>
		/// Initializes a new instance of the ClientSendSlowException class
		/// </summary>
		/// <param name="innerException">The inner exception that caused this exception</param>
		public ClientSendSlowException(Exception? innerException) : base("Client was sending data to slowly", innerException)
		{
		}
	}
}
