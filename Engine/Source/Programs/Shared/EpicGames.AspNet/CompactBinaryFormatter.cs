// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc.Formatters;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Primitives;
using Microsoft.Net.Http.Headers;

namespace EpicGames.AspNet
{
	/// <summary>
	/// Global constants
	/// </summary>
	public static partial class CustomMediaTypeNames
	{
		/// <summary>
		/// Media type for compact binary
		/// </summary>
		public const string UnrealCompactBinary = "application/x-ue-cb";

		/// <summary>
		/// Media type for compressed buffers
		/// </summary>
		public const string UnrealCompressedBuffer = "application/x-ue-comp";

		/// <summary>
		/// Media type for Jupiter inline payload
		/// </summary>
		public const string JupiterInlinedPayload = "application/x-jupiter-inline";

		/// <summary>
		/// Media type for compact binary packages
		/// </summary>
		public const string UnrealCompactBinaryPackage = "application/x-ue-cbpkg";
	}

	/// <summary>
	/// Converter to allow reading compact binary objects as request bodies
	/// </summary>
	public class CbInputFormatter : InputFormatter
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public CbInputFormatter()
		{
			SupportedMediaTypes.Add(MediaTypeHeaderValue.Parse(CustomMediaTypeNames.UnrealCompactBinary));
		}

		/// <inheritdoc/>
		protected override bool CanReadType(Type type)
		{
			return true;
		}

		/// <inheritdoc/>
		public override async Task<InputFormatterResult> ReadRequestBodyAsync(InputFormatterContext context)
		{
			// Buffer the data into an array
			byte[] data;
			try
			{
				using MemoryStream stream = new();
				await context.HttpContext.Request.Body.CopyToAsync(stream);
				data = stream.ToArray();
			}
			catch (BadHttpRequestException e)
			{
				ClientSendSlowExceptionUtil.MaybeThrowSlowSendException(e);
				throw;
			}

			// Serialize the object
			CbField field;
			try
			{
				field = new CbField(data);
			}
			catch (Exception ex)
			{
				ILogger<CbInputFormatter> logger = context.HttpContext.RequestServices.GetRequiredService<ILogger<CbInputFormatter>>();
				logger.LogError(ex, "Unable to parse compact binary: {Dump}", FormatHexDump(data, 256));
				foreach ((string name, StringValues values) in context.HttpContext.Request.Headers)
				{
					foreach (string? value in values)
					{
						logger.LogInformation("Header {Name}: {Value}", name, value);
					}
				}
				throw new Exception($"Unable to parse compact binary request: {FormatHexDump(data, 256)}", ex);
			}
			return await InputFormatterResult.SuccessAsync(CbSerializer.Deserialize(new CbField(data), context.ModelType)!);
		}

		/// <summary>
		/// Formats a byte array as a hexadecimal dump with line breaks every 16 bytes
		/// </summary>
		/// <param name="data">The byte array to format</param>
		/// <param name="maxLength">Maximum number of bytes to include in the dump</param>
		/// <returns>Formatted hexadecimal string with whitespace and line breaks</returns>
		static string FormatHexDump(byte[] data, int maxLength)
		{
			ReadOnlySpan<byte> span = data.AsSpan(0, Math.Min(data.Length, maxLength));
			string hexString = StringUtils.FormatHexString(span);

			char[] hexDump = new char[span.Length * 3];
			for (int idx = 0; idx < span.Length; idx++)
			{
				hexDump[(idx * 3) + 0] = ((idx & 15) == 0) ? '\n' : ' ';
				hexDump[(idx * 3) + 1] = hexString[(idx * 2) + 0];
				hexDump[(idx * 3) + 2] = hexString[(idx * 2) + 1];
			}
			return new string(hexDump);
		}
	}

	/// <summary>
	/// Converter to allow writing compact binary objects as responses
	/// </summary>
	public class CbOutputFormatter : OutputFormatter
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public CbOutputFormatter()
		{
			SupportedMediaTypes.Add(MediaTypeHeaderValue.Parse(CustomMediaTypeNames.UnrealCompactBinary));
		}

		/// <inheritdoc/>
		protected override bool CanWriteType(Type? type)
		{
			return true;
		}

		/// <inheritdoc/>
		public override async Task WriteResponseBodyAsync(OutputFormatterWriteContext context)
		{
			ReadOnlyMemory<byte> data;
			if (context.Object is CbObject obj)
			{
				data = obj.GetView();
			}
			else
			{
				data = CbSerializer.Serialize(context.ObjectType!, context.Object!).GetView();
			}
			await context.HttpContext.Response.BodyWriter.WriteAsync(data);
		}
	}

	/// <summary>
	/// Special version of <see cref="CbOutputFormatter"/> which returns native CbObject encoded data. Can be 
	/// inserted at a high priority in the output formatter list to prevent transcoding to json.
	/// </summary>
	public class CbPreferredOutputFormatter : CbOutputFormatter
	{
		/// <inheritdoc/>
		protected override bool CanWriteType(Type? type)
		{
			return type == typeof(CbObject);
		}
	}
}
