<Project Sdk="Microsoft.NET.Sdk.Web">
  <!-- Web SDK import order overrides OutputType from Directory.Build.props so it needs to be set back to Library -->
  <PropertyGroup>
    <OutputType>Library</OutputType>
  </PropertyGroup>

  <!-- Web SDK import order overrides configuration properties from UnrealEngine.csproj.props -->
  <PropertyGroup Condition="'$(Configuration)|$(Platform)' == 'Analyze|AnyCPU'">
    <GenerateDocumentationFile>true</GenerateDocumentationFile>
    <RunAnalyzersDuringBuild>true</RunAnalyzersDuringBuild>
    <EnforceCodeStyleInBuild>true</EnforceCodeStyleInBuild>
    <NuGetAudit>true</NuGetAudit>
    <NuGetAuditLevel>critical</NuGetAuditLevel>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>
</Project>
