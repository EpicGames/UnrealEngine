// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Globalization;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;

namespace EpicGames.AspNet
{
	/// <summary>
	/// Represents a server timing metric for performance monitoring
	/// </summary>
	public struct ServerTimingMetric
	{
		/// <summary>
		/// The name of the metric
		/// </summary>
		private readonly string _metricName;

		/// <summary>
		/// The duration in milliseconds
		/// </summary>
		private readonly double? _duration;

		/// <summary>
		/// Optional description of the metric
		/// </summary>
		private readonly string? _description;

		/// <summary>
		/// Cached string representation of the server timing metric
		/// </summary>
		private string? _serverTimingMetric;

		/// <summary>
		/// Initializes a new instance of the ServerTimingMetric struct
		/// </summary>
		/// <param name="metricName">The name of the metric</param>
		/// <param name="duration">The duration in milliseconds</param>
		/// <param name="description">Optional description of the metric</param>
		public ServerTimingMetric(string metricName, double? duration, string? description)
		{
			_metricName = metricName;
			_duration = duration;
			_description = description;

			_serverTimingMetric = null;
		}

		/// <summary>
		/// Converts the metric to its string representation
		/// </summary>
		/// <returns>Formatted server timing metric string</returns>
		public override string ToString()
		{
			if (_serverTimingMetric != null)
			{
				return _serverTimingMetric;
			}

			StringBuilder sb = new(_metricName);
			if (_duration != null)
			{
				sb.Append(";dur=");
				sb.Append(_duration.Value.ToString(CultureInfo.InvariantCulture));
			}

			if (!String.IsNullOrEmpty(_description))
			{
				sb.Append(";desc=\"");
				sb.Append(_description);
				sb.Append('"');
			}

			_serverTimingMetric = sb.ToString();

			return _serverTimingMetric;
		}
	}

	/// <summary>
	/// Interface for managing server timing metrics
	/// </summary>
	public interface IServerTiming
	{
		/// <summary>
		/// Adds a server timing metric
		/// </summary>
		/// <param name="metricName">The name of the metric</param>
		/// <param name="duration">The duration in milliseconds</param>
		/// <param name="description">Optional description</param>
		public void AddServerTimingMetric(string metricName, double? duration, string? description);

		/// <summary>
		/// Creates a scoped metric that measures duration automatically
		/// </summary>
		/// <param name="metricName">The name of the metric</param>
		/// <param name="description">Optional description</param>
		/// <returns>A disposable scoped metric</returns>
		public ServerTimingMetricScoped CreateServerTimingMetricScope(string metricName, string? description);

		/// <summary>
		/// Gets the collection of all recorded metrics
		/// </summary>
		public IReadOnlyCollection<ServerTimingMetric> Metrics { get; }
	}

	/// <summary>
	/// Scoped server timing metric that automatically records duration when disposed
	/// </summary>
	public sealed class ServerTimingMetricScoped : IDisposable
	{
		/// <summary>
		/// The timing manager that will receive the metric when disposed
		/// </summary>
		private readonly IServerTiming _timingManager;

		/// <summary>
		/// The name of the metric
		/// </summary>
		private readonly string _metricName;

		/// <summary>
		/// Optional description of the metric
		/// </summary>
		private readonly string? _description;

		/// <summary>
		/// The start time of the metric measurement
		/// </summary>
		private readonly DateTime _startTime;

		/// <summary>
		/// Initializes a new instance of the ServerTimingMetricScoped class
		/// </summary>
		/// <param name="timingManager">The timing manager that will receive the metric</param>
		/// <param name="metricName">The name of the metric</param>
		/// <param name="description">Optional description of the metric</param>
		internal ServerTimingMetricScoped(IServerTiming timingManager, string metricName, string? description)
		{
			_timingManager = timingManager;
			_metricName = metricName;
			_description = description;
			_startTime = DateTime.Now;
		}

		/// <summary>
		/// Disposes the scoped metric and records the duration
		/// </summary>
		public void Dispose()
		{
			TimeSpan duration = DateTime.Now - _startTime;
			_timingManager.AddServerTimingMetric(_metricName, duration.TotalMilliseconds, _description);
		}
	}

	/// <summary>
	/// Default implementation of server timing metrics
	/// </summary>
	public class ServerTiming : IServerTiming
	{
		/// <summary>
		/// Thread-safe collection of recorded metrics
		/// </summary>
		private readonly ConcurrentBag<ServerTimingMetric> _metrics = [];

		/// <inheritdoc/>
		public void AddServerTimingMetric(string metricName, double? duration, string? description)
		{
			_metrics.Add(new ServerTimingMetric(metricName, duration, description));
		}

		/// <inheritdoc/>
		public ServerTimingMetricScoped CreateServerTimingMetricScope(string metricName, string? description)
		{
			return new ServerTimingMetricScoped(this, metricName, description);
		}

		/// <inheritdoc/>
		public IReadOnlyCollection<ServerTimingMetric> Metrics => _metrics;
	}

	/// <summary>
	/// Middleware for adding server timing headers to HTTP responses
	/// </summary>
	public class ServerTimingMiddleware
	{
		/// <summary>
		/// The next request delegate in the middleware pipeline
		/// </summary>
		private readonly RequestDelegate _next;

		/// <summary>
		/// Initializes a new instance of the ServerTimingMiddleware class
		/// </summary>
		/// <param name="next">The next request delegate in the pipeline</param>
		public ServerTimingMiddleware(RequestDelegate next)
		{
			_next = next ?? throw new ArgumentNullException(nameof(next));
		}

		/// <summary>
		/// Invokes the middleware to add server timing headers
		/// </summary>
		/// <param name="context">The HTTP context</param>
		/// <returns>A task representing the asynchronous operation</returns>
		public async Task InvokeAsync(HttpContext context)
		{
			IServerTiming serverTiming = context.RequestServices.GetRequiredService<IServerTiming>();
			if (AllowsTrailers(context.Request) && context.Response.SupportsTrailers())
			{
				await HandleServerTimingAsTrailerHeaderAsync(context, serverTiming);
			}
			else
			{
				await HandleServerTimingAsResponseHeadersAsync(context, serverTiming);
			}
		}

		/// <summary>
		/// Checks if the request supports trailer headers
		/// </summary>
		/// <param name="request">The HTTP request</param>
		/// <returns>True if trailers are allowed</returns>
		public static bool AllowsTrailers(HttpRequest request)
		{
			return request.Headers.ContainsKey("TE") && request.Headers["TE"].Contains("trailers");
		}

		/// <summary>
		/// Handles server timing by adding metrics as HTTP trailer headers
		/// </summary>
		/// <param name="context">The HTTP context</param>
		/// <param name="serverTiming">The server timing instance containing metrics</param>
		/// <returns>A task representing the asynchronous operation</returns>
		private async Task HandleServerTimingAsTrailerHeaderAsync(HttpContext context, IServerTiming serverTiming)
		{
			context.Response.DeclareTrailer("Server-Timing");

			await _next(context);

			// we limit the server timing header to 10 metrics because otherwise we risk generating very large response headers for operations that do a lot of work
			string serverTimingValue = serverTiming.Metrics.Count != 0 ? String.Join(",", serverTiming.Metrics.Take(10)) : "";
			context.Response.AppendTrailer(
				"Server-Timing",
				serverTimingValue);
		}

		/// <summary>
		/// Handles server timing by adding metrics as HTTP response headers
		/// </summary>
		/// <param name="context">The HTTP context</param>
		/// <param name="serverTiming">The server timing instance containing metrics</param>
		/// <returns>A task representing the asynchronous operation</returns>
		private Task HandleServerTimingAsResponseHeadersAsync(HttpContext context, IServerTiming serverTiming)
		{
			context.Response.OnStarting(() =>
			{
				if (serverTiming.Metrics.Count != 0)
				{
					string serverTimingValue = String.Join(",", serverTiming.Metrics.Take(10));
					context.Response.Headers.Append("Server-Timing", serverTimingValue);
				}

				return Task.CompletedTask;
			});

			return _next(context);
		}
	}

	/// <summary>
	/// Extension methods for adding server timing services
	/// </summary>
	public static class ServerTimingServiceCollectionExtensions
	{
		/// <summary>
		/// Adds server timing services to the service collection
		/// </summary>
		/// <param name="services">The service collection</param>
		/// <returns>The service collection for chaining</returns>
		public static IServiceCollection AddServerTiming(this IServiceCollection services)
		{
			services.AddScoped<IServerTiming, ServerTiming>();

			return services;
		}
	}
}
