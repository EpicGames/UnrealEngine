<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="BenchmarkDotNet" Version="0.15.8" />
    <PackageReference Include="Microsoft.VisualStudio.DiagnosticsHub.BenchmarkDotNetDiagnosers" Version="18.6.37125.3" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
  </ItemGroup>
</Project>
