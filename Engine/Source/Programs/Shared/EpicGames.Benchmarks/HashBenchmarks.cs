// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using BenchmarkDotNet.Attributes;
using EpicGames.Core;
using Microsoft.VSDiagnostics;

namespace EpicGames.Benchmarks;

// For more information on the VS BenchmarkDotNet Diagnosers see https://learn.microsoft.com/visualstudio/profiling/profiling-with-benchmark-dotnet
[CPUUsageDiagnoser]
[MemoryDiagnoser]
public class HashBenchmarks
{
	[Params(1000, 10_000, 1_000_000)]
	public int DataLength { get; set; }

	private byte[]? _data;

	[GlobalSetup]
	[SuppressMessage("Security", "CA5394:Do not use insecure randomness", Justification = "Using Random for deterministic test data generation")]
	public void Setup()
	{
		_data = new byte[DataLength];
		new Random(0).NextBytes(_data);
	}

	[Benchmark]
	public IoHash IOHash()
	{
		return IoHash.Compute(_data);
	}

	[Benchmark]
	public ContentHash ContentHashSHA1()
	{
		return ContentHash.SHA1(_data!);
	}

	[Benchmark]
	public ContentHash ContentHashMD5()
	{
		return ContentHash.MD5(_data!);
	}

	[Benchmark]
	public Md5Hash MD5Hash()
	{
		return Md5Hash.Compute(_data!);
	}
}
