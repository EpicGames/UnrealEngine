// Copyright Epic Games, Inc. All Rights Reserved.

using BenchmarkDotNet.Running;

namespace EpicGames.Benchmarks;

internal class Program
{
	static void Main(string[] args)
	{
		_ = BenchmarkRunner.Run(typeof(Program).Assembly, args: args);
	}
}
