// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildBase;

namespace EpicGames.Build.Tests;

[TestClass]
public class DotnetProjectInfoTests
{
	[TestMethod]
	public void TestParseOwnProject()
	{
		FileReference ownProjectPath = GetOwnProjectPath();
		DotnetProjectInfo projectInfo;
		try
		{
			projectInfo = DotnetProjectInfo.Read(ownProjectPath, new Dictionary<string, string>());
		}
		catch (FileNotFoundException ex) when (ex.Message.Contains("System.Runtime", StringComparison.Ordinal))
		{
			Assert.Inconclusive("MSBuild SDK version mismatch - test requires matching .NET SDK version");
			return;
		}

		// Test properties from the Directory.Build.props file
		CollectionAssert.Contains(projectInfo.Properties.Keys.ToList(), "IsPackable");
		Assert.AreEqual("false", projectInfo.Properties["IsPackable"]);
	}

	[TestMethod]
	public void TestCanParseSameProjectRepeatedly()
	{
		FileReference ownProjectPath = GetOwnProjectPath();
		try
		{
			_ = DotnetProjectInfo.Read(ownProjectPath, new Dictionary<string, string>());
			_ = DotnetProjectInfo.Read(ownProjectPath, new Dictionary<string, string>());
			_ = DotnetProjectInfo.Read(ownProjectPath, new Dictionary<string, string>());
		}
		catch (FileNotFoundException ex) when (ex.Message.Contains("System.Runtime", StringComparison.Ordinal))
		{
			Assert.Inconclusive("MSBuild SDK version mismatch - test requires matching .NET SDK version");
		}
	}

	private static FileReference GetOwnProjectPath()
	{
		DirectoryReference? searchDirectory = new(Assembly.GetExecutingAssembly().Location);
		FileReference? maybeOwnProjectPath = null;
		while (searchDirectory is not null)
		{
			maybeOwnProjectPath = FileReference.Combine(searchDirectory, "EpicGames.Build.Tests.csproj");
			if (FileReference.Exists(maybeOwnProjectPath))
			{
				break;
			}

			searchDirectory = searchDirectory.ParentDirectory;
			maybeOwnProjectPath = null;
		}

		if (maybeOwnProjectPath is null)
		{
			Assert.Inconclusive("Could not find own project file to test");
		}

		return maybeOwnProjectPath;
	}
}
