<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Build\EpicGames.Build.csproj" />
  </ItemGroup>
</Project>
