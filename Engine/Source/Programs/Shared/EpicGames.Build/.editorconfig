# EditorConfig is awesome:http://EditorConfig.org

[*.cs]


# CA1012: Abstract types should not have public constructors
dotnet_diagnostic.CA1012.severity = none

# CA1051: Do not declare visible instance fields
dotnet_diagnostic.CA1051.severity = none

# CA1307: Specify StringComparison for clarity
dotnet_diagnostic.CA1307.severity = none

# CA1309: Use ordinal string comparison
dotnet_diagnostic.CA1309.severity = none

# CA1310: Specify StringComparison for correctness
dotnet_diagnostic.CA1310.severity = none

# CA2211: Non-constant fields should not be visible
dotnet_diagnostic.CA2211.severity = none

# CA2251: Use 'string.Equals'
dotnet_diagnostic.CA2251.severity = none

# IDE0025: Use expression body for properties
dotnet_diagnostic.IDE0025.severity = none

# IDE0036: Order modifiers
dotnet_diagnostic.IDE0036.severity = none

# IDE0049: Use framework type
dotnet_diagnostic.IDE0049.severity = none

# IDE1006: Naming Styles
dotnet_diagnostic.IDE1006.severity = none

# IDE0044: Add readonly modifier
dotnet_diagnostic.IDE0044.severity = none

# IDE2000: C#
dotnet_diagnostic.IDE2000.severity = none

# CA2227: Collection properties should be read only
dotnet_diagnostic.CA2227.severity = none

# CA1052: Static holder types should be Static or NotInheritable
dotnet_diagnostic.CA1052.severity = none

# IDE0005: Using directive is unnecessary.
dotnet_diagnostic.IDE0005.severity = none

# IDE0054: Use compound assignment
dotnet_diagnostic.IDE0054.severity = none

# CA1822: Mark members as static
dotnet_diagnostic.CA1822.severity = none

# CA2000: Dispose objects before losing scope
dotnet_diagnostic.CA2000.severity = none

# CA1825: Avoid zero-length array allocations
dotnet_diagnostic.CA1825.severity = none

# CA1707: Identifiers should not contain underscores
dotnet_diagnostic.CA1707.severity = none

# CA1034: Nested types should not be visible
dotnet_diagnostic.CA1034.severity = none

# CA1820: Test for empty strings using string length
dotnet_diagnostic.CA1820.severity = none

# CA1834: Consider using 'StringBuilder.Append(char)' when applicable
dotnet_diagnostic.CA1834.severity = none

# CA1508: Avoid dead conditional code
dotnet_diagnostic.CA1508.severity = none

# IDE0060: Remove unused parameter
dotnet_diagnostic.IDE0060.severity = none

# CA1819: Properties should not return arrays
dotnet_diagnostic.CA1819.severity = none

# CA1827: Do not use Count() or LongCount() when Any() can be used
dotnet_diagnostic.CA1827.severity = none

# CA1829: Use Length/Count property instead of Count() when available
dotnet_diagnostic.CA1829.severity = none

# CA2249: Consider using 'string.Contains' instead of 'string.IndexOf'
dotnet_diagnostic.CA2249.severity = none

# IDE0130: Namespace does not match folder structure
dotnet_diagnostic.IDE0130.severity = none
