// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealBuildBase
{
	/// <summary>
	/// Command to execute info.
	/// </summary>
	public class CommandInfo(string CommandNameIn)
	{
		public string CommandName { get; } = CommandNameIn;

		public List<string> Arguments { get; set; } = [];

		public override string ToString()
		{
			string Result = CommandName;
			Result += "(";
			for (int Index = 0; Index < Arguments.Count; ++Index)
			{
				if (Index > 0)
				{
					Result += ", ";
				}
				Result += Arguments[Index];
			}
			Result += ")";
			return Result;
		}
	}

	public partial class ParsedCommandLine(Dictionary<string, string> GlobalParameters, HashSet<string> IgnoredGlobalParameters)
	{
		/// <summary>
		/// Argument name and description for known arguments
		/// </summary>
		public Dictionary<string, string> GlobalParameters
		{
			get;
			private set;
		} = new Dictionary<string, string>(GlobalParameters, StringComparer.InvariantCultureIgnoreCase);

		public HashSet<string> IgnoredGlobalParameters
		{
			get;
			private set;
		} = new HashSet<string>(IgnoredGlobalParameters, StringComparer.InvariantCultureIgnoreCase);

		readonly HashSet<string> GlobalParameterValues = new(StringComparer.InvariantCultureIgnoreCase);

		public void SetGlobal(string Parameter)
		{
			if (IgnoredGlobalParameters.Contains(Parameter))
			{
				throw new Exception($"cannot set ignored global parameter {Parameter}");
			}

			if (!GlobalParameters.ContainsKey(Parameter))
			{
				throw new Exception($"unknown global parameter {Parameter}");
			}

			GlobalParameterValues.Add(Parameter);
		}

		public bool TrySetGlobal(string Parameter)
		{
			if (IgnoredGlobalParameters.Contains(Parameter))
			{
				return false;
			}

			if (!GlobalParameters.ContainsKey(Parameter))
			{
				return false;
			}

			GlobalParameterValues.Add(Parameter);
			return true;
		}

		public bool IsSetGlobal(string Parameter)
		{
			if (IgnoredGlobalParameters.Contains(Parameter))
			{
				throw new Exception($"cannot check if ignored global parameter {Parameter} is set");
			}

			if (!GlobalParameters.ContainsKey(Parameter))
			{
				throw new Exception($"unknown global parameter {Parameter}");
			}

			return GlobalParameterValues.Contains(Parameter);
		}

		Dictionary<string, object?> UncheckedParameters = new(StringComparer.InvariantCultureIgnoreCase);
		public void SetUnchecked(string Name, object? Value = null)
		{
			UncheckedParameters[Name] = Value;
		}

		public bool IsSetUnchecked(string Name)
		{
			return UncheckedParameters.ContainsKey(Name);
		}

		public object? GetValueUnchecked(string Name)
		{
			return UncheckedParameters.TryGetValue(Name, out object? Out) ? Out : null;
		}

		public bool IsParameterIgnored(string Parameter)
		{
			return IgnoredGlobalParameters.Contains(Parameter);
		}

		public List<CommandInfo> CommandsToExecute = [];
	}
}
