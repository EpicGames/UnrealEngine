// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildBase
{
	// NOTE: this needs to be kept in sync with EditorAnalytics.h and iPhonePackager.cs
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores", Justification = "Intentional")]
	public enum ExitCode
	{
		Error_UATNotFound = -1,
		Success = 0,
		Error_Unknown = 1,
		Error_Arguments = 2,
		Error_UnknownCommand = 3,
		Error_SDKNotFound = 10,
		Error_ProvisionNotFound = 11,
		Error_CertificateNotFound = 12,
		Error_ProvisionAndCertificateNotFound = 13,
		Error_InfoPListNotFound = 14,
		Error_KeyNotFoundInPList = 15,
		Error_ProvisionExpired = 16,
		Error_CertificateExpired = 17,
		Error_CertificateProvisionMismatch = 18,
		Error_CodeUnsupported = 19,
		Error_PluginsUnsupported = 20,
		Error_UnknownCookFailure = 25,
		Error_UnknownDeployFailure = 26,
		Error_UnknownBuildFailure = 27,
		Error_UnknownPackageFailure = 28,
		Error_UnknownLaunchFailure = 29,
		Error_StageMissingFile = 30,
		Error_FailedToCreateIPA = 31,
		Error_FailedToCodeSign = 32,
		Error_DeviceBackupFailed = 33,
		Error_AppUninstallFailed = 34,
		Error_AppInstallFailed = 35,
		Error_AppNotFound = 36,
		Error_StubNotSignedCorrectly = 37,
		Error_IPAMissingInfoPList = 38,
		Error_DeleteFile = 39,
		Error_DeleteDirectory = 40,
		Error_CreateDirectory = 41,
		Error_CopyFile = 42,
		Error_OnlyOneObbFileSupported = 50,
		Error_FailureGettingPackageInfo = 51,
		Error_OnlyOneTargetConfigurationSupported = 52,
		Error_ObbNotFound = 53,
		Error_AndroidBuildToolsPathNotFound = 54,
		Error_NoApkSuitableForArchitecture = 55,
		Error_FilesInstallFailed = 56,
		Error_RemoteCertificatesNotFound = 57,
		Error_LauncherFailed = 100,
		Error_UATLaunchFailure = 101,
		Error_FailedToDeleteStagingDirectory = 102,
		Error_MissingExecutable = 103,
		Error_DeviceNotSetupForDevelopment = 150,
		Error_DeviceOSNewerThanSDK = 151,
		Error_TestFailure = 152,
		Error_SymbolizedSONotFound = 153,
		Error_LicenseNotAccepted = 154,
		Error_AndroidOBBError = 155,
		Error_SDKInstallFailed = 200,
		Error_DeviceUpdateFailed = 201,
		Error_UBTFailure = 202
	};
}
