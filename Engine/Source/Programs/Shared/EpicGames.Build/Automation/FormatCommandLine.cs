// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealBuildBase
{
	public static partial class CommandLine
	{
		/// <summary>
		/// Converts a list of arguments to a string where each argument is separated with a space character.
		/// </summary>
		/// <param name="arguments">Arguments</param>
		/// <returns>Single string containing all arguments separated with a space.</returns>
		public static string FormatCommandLine(IEnumerable<string> arguments)
		{
			StringBuilder result = new();
			foreach (string argument in arguments)
			{
				if (result.Length > 0)
				{
					result.Append(' ');
				}
				result.Append(FormatArgumentForCommandLine(argument));
			}
			return result.ToString();
		}

		/// <summary>
		/// Format a single argument for passing on the command line, inserting quotes as necessary.
		/// </summary>
		/// <param name="argument">The argument to quote</param>
		/// <returns>The argument, with quotes if necessary</returns>
		public static string FormatArgumentForCommandLine(string argument)
		{
			// Check if the argument contains a space. If not, we can just pass it directly.
			int spaceIdx = argument.IndexOf(' ', StringComparison.Ordinal);
			if (spaceIdx == -1)
			{
				return argument;
			}

			// If the argument has quotes in it, the nested quotes must be escaped (unless already escaped)
			if (argument.Contains('"', StringComparison.Ordinal) && !argument.Contains("\\\"", StringComparison.Ordinal))
			{
				argument = argument.Replace("\"", "\\\"", StringComparison.Ordinal);
			}

			// If it does have a space, and it's formatted as an option (ie. -Something=), try to insert quotes after the equals character
			int equalsIdx = argument.IndexOf('=', StringComparison.Ordinal);
			return argument.StartsWith('-') && equalsIdx != -1 && equalsIdx < spaceIdx
				? $"{argument.Substring(0, equalsIdx)}=\"{argument.Substring(equalsIdx + 1)}\""
				: $"\"{argument}\"";
		}
	}
}
