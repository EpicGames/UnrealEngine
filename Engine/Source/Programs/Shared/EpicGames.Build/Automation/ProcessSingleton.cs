// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	public static class ProcessSingleton
	{
		/// <summary>
		/// Returns true/false based on whether this is the only instance
		/// running (checked at startup).
		/// </summary>
		public static bool IsSoleInstance { get; private set; }

		/// <summary>
		/// Runs the specified delegate checking if this is the only instance of the application.
		/// </summary>
		/// <param name="main"></param>
		/// <param name="bWaitForUATMutex"></param>
		/// <param name="logger"></param>
		public static async Task<ExitCode> RunSingleInstanceAsync(Func<Task<ExitCode>> main, bool bWaitForUATMutex, ILogger logger)
		{
			// Need to execute this logic on a background thread, since mutex ownership on Linux has thread affinity (ie. mutexes be released on the
			// same thread that acquires it, which is not guaranteed through an async continuation)
			TaskCompletionSource<ExitCode> result = new();
			Thread thread = new(() => RunSingleInstanceThread(main, result, bWaitForUATMutex, logger));
			thread.Start();
			return await result.Task;
		}

		static void RunSingleInstanceThread(Func<Task<ExitCode>> main, TaskCompletionSource<ExitCode> result, bool bWaitForUATMutex, ILogger logger)
		{
			try
			{
				bool allowMultipleInsances = (Environment.GetEnvironmentVariable("uebp_UATMutexNoWait") == "1");

				FileReference entryAssemblyLocation = FileReference.FromString(Assembly.GetEntryAssembly()?.GetOriginalLocation())
					?? Unreal.DotnetPath;

				string mutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath(entryAssemblyLocation.GetFileNameWithoutExtension(), entryAssemblyLocation);
				using (Mutex singleInstanceMutex = new(true, mutexName, out bool bCreatedMutex))
				{
					IsSoleInstance = bCreatedMutex;

					if (!IsSoleInstance && allowMultipleInsances == false)
					{
						if (bWaitForUATMutex)
						{
							logger.LogWarning("Another instance of UAT at '{File}' is running, and the -WaitForUATMutex parameter has been used. Waiting for other UAT to finish...", entryAssemblyLocation);
							int seconds = 0;
							while (WaitMutexNoExceptions(singleInstanceMutex, 15 * 1000) == false)
							{
								seconds += 15;
								logger.LogInformation("Still waiting for Mutex. {TimeSeconds} seconds has passed...", seconds);
							}
						}
						else
						{
							throw new Exception($"A conflicting instance of AutomationTool is already running. Current location: {entryAssemblyLocation}. A process manager may be used to determine the conflicting process and what tool may have launched it");
						}
					}

					ExitCode exitCode = Task.Run(() => main()).Result;

					if (IsSoleInstance)
					{
						singleInstanceMutex.ReleaseMutex();
					}

					result.SetResult(exitCode);
				}
			}
			catch (Exception ex)
			{
				result.TrySetException(ex);
			}
		}

		static bool WaitMutexNoExceptions(Mutex mutex, int timeoutMs = 15 * 1000)
		{
			try
			{
				return mutex.WaitOne(timeoutMs);
			}
			catch (AbandonedMutexException)
			{
				return true;
			}
		}
	}
}
