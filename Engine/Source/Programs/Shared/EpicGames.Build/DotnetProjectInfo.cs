// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;
using EpicGames.Core;
using EpicGames.MsBuild;
using Microsoft.Build.Definition;
using Microsoft.Build.Evaluation;

namespace UnrealBuildBase;

public interface IDotnetProjectInfo
{
	/// <summary>
	/// Evaluated properties from the project file
	/// </summary>
	public IReadOnlyDictionary<string, string> Properties { get; }

	/// <summary>
	/// Mapping of referenced assemblies to their 'CopyLocal' (aka 'Private') setting.
	/// </summary>
	public IReadOnlyDictionary<FileReference, bool> References { get; }

	/// <summary>
	/// Mapping of referenced projects to their 'CopyLocal' (aka 'Private') setting.
	/// </summary>
	public IReadOnlyDictionary<FileReference, bool> ProjectReferences { get; }

	/// <summary>
	/// List of compile references in the project.
	/// </summary>
	public IReadOnlyList<FileReference> CompileReferences { get; }

	/// <summary>
	/// Mapping of content to if they are flagged Always or Newer
	/// </summary>
	public IReadOnlyDictionary<FileReference, bool> ContentReferences { get; }

	/// <summary>
	/// Path to the CSProject file
	/// </summary>
	public FileReference ProjectPath { get; }
}

/// <summary>
/// Information from a C# project file parsed by MSBuild.
/// </summary>
public sealed record class DotnetProjectInfo : IDotnetProjectInfo
{
	private DotnetProjectInfo()
	{
	}

	/// <inheritdoc />
	public required IReadOnlyDictionary<string, string> Properties { get; init; }

	/// <inheritdoc />
	public required IReadOnlyDictionary<FileReference, bool> References { get; init; }

	/// <inheritdoc />
	public required IReadOnlyDictionary<FileReference, bool> ProjectReferences { get; init; }

	/// <inheritdoc />
	public required IReadOnlyList<FileReference> CompileReferences { get; init; }

	/// <inheritdoc />
	public required IReadOnlyDictionary<FileReference, bool> ContentReferences { get; init; }

	/// <inheritdoc />
	public required FileReference ProjectPath { get; init; }

	/// <summary>
	/// Reads project information for the given file.
	/// </summary>
	/// <param name="file">The project file to read</param>
	/// <param name="globalProperties">Initial set of property values</param>
	/// <returns>The parsed project info</returns>
	[MethodImpl(MethodImplOptions.NoInlining)]
	public static DotnetProjectInfo Read(FileReference file, IReadOnlyDictionary<string, string> globalProperties)
	{
		// First register MSBuild so we can use it to parse our project file. By not having the internal routine
		// inline, we avoid having the issue of the Microsoft.Build libraries being resolved prior to the build path
		// being set.
		MsBuildSupport.RegisterMsBuildPath(MsBuildRegistration.Instance);

		return ReadImpl(file, globalProperties);
	}

	/// <summary>
	/// Attempts to read project information for the given file.
	/// </summary>
	/// <param name="file">The project file to read</param>
	/// <param name="globalProperties">Initial set of property values</param>
	/// <param name="outProjectInfo">If successful, the parsed project info</param>
	/// <returns>True if the project was read successfully, false otherwise</returns>
	public static bool TryRead(
		FileReference file,
		IReadOnlyDictionary<string, string> globalProperties,
		[NotNullWhen(true)] out DotnetProjectInfo? outProjectInfo)
	{
		try
		{
			outProjectInfo = Read(file, globalProperties);
			return true;
		}
		catch
		{
			outProjectInfo = null;
			return false;
		}
	}

	[MethodImpl(MethodImplOptions.NoInlining)]
	private static DotnetProjectInfo ReadImpl(FileReference file, IReadOnlyDictionary<string, string> globalProperties)
	{
		using ProjectCollection projectCollection = new();

		// See Gauntlet.Automation.cs for context.
		const string includeGauntletPluginsKey = "EpicGamesIncludeGauntletPlugins";
		Dictionary<string, string> globalPropertiesWithDefaults = new(globalProperties);
		if (!globalProperties.ContainsKey(includeGauntletPluginsKey))
		{
			globalPropertiesWithDefaults[includeGauntletPluginsKey] = "false";
		}

		ProjectOptions projectOptions = new()
		{
			GlobalProperties = globalPropertiesWithDefaults,
			LoadSettings = ProjectLoadSettings.DoNotEvaluateElementsWithFalseCondition,
			ProjectCollection = projectCollection,
		};
		Project project = Project.FromFile(file.FullName, projectOptions);

		HashSet<string> targetFrameworks = [.. (project.GetProperty("TargetFrameworks")?.EvaluatedValue ?? String.Empty)
			.Split(';', StringSplitOptions.TrimEntries | StringSplitOptions.RemoveEmptyEntries)];
		if (targetFrameworks.Count > 0)
		{
			Dictionary<string, string> innerProps = project.GlobalProperties.ToDictionary(kvp => kvp.Key, kvp => kvp.Value);
#if NET10_0_OR_GREATER
			if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net10.0"))
			{
				innerProps["TargetFramework"] = "net10.0";
			}
#endif
#if NET9_0_OR_GREATER
			if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net9.0"))
			{
				innerProps["TargetFramework"] = "net9.0";
			}
#endif
#if NET8_0_OR_GREATER
			if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net8.0"))
			{
				innerProps["TargetFramework"] = "net8.0";
			}
#endif
			if (!innerProps.ContainsKey("TargetFramework"))
			{
				innerProps["TargetFramework"] = targetFrameworks.Order().Last();
			}
			project = new Project(project.FullPath, innerProps, project.ToolsVersion, project.ProjectCollection);
		}

		Dictionary<string, string> properties = project.Properties.ToDictionary(p => p.Name, p => p.EvaluatedValue);

		DirectoryReference projectDirectory = file.Directory;
		FileReference GetFullPath(string path) =>
			Path.IsPathFullyQualified(path) ? new FileReference(path) : FileReference.Combine(projectDirectory, path);

		bool IsPrivate(ProjectItem item) => !item.HasMetadata("Private") || Boolean.Parse(item.GetMetadataValue("Private"));

		// For references with hint paths, multiple values appear to be all loaded, checked that they're the correct assembly,
		// and the highest versioned assembly is used. It would be unreasonable to recreate this behaviour here, and it's not
		// clear how we would get MSBuild to evaluate this for us, so we should just warn if people do silly things like this.
		IEnumerable<string> unparseableReferences = project.GetItems("Reference")
			.Where(r => !r.HasMetadata("EmbedInteropTypes") || !Boolean.Parse(r.GetMetadataValue("EmbedInteropTypes")))
			.Where(r => r.HasMetadata("HintPath"))
			.GroupBy(r => r.EvaluatedInclude)
			.Where(grp => grp.DistinctBy(r => (hintPath: r.GetMetadataValue("HintPath"), isPrivate: IsPrivate(r))).Count() > 1)
			.Select(grp => grp.Key);
		string unparseableReferencesString = string.Join(", ", unparseableReferences);
		if (unparseableReferencesString.Length != 0)
		{
			throw new Exception($"Project '{file}' has multiple <Reference>s for the same Include but with different HintPath / Private values ({unparseableReferencesString}) - please fix.");
		}

		IEnumerable<ProjectItem> validReferences = project.GetItems("Reference")
			.Where(r => !r.HasMetadata("EmbedInteropTypes") || !Boolean.Parse(r.GetMetadataValue("EmbedInteropTypes")))
			.Where(r => r.HasMetadata("HintPath"))
			.Reverse()
			.DistinctBy(r => r.EvaluatedInclude);
		Dictionary<FileReference, bool> references = validReferences
			.ToDictionary(r => GetFullPath(r.GetMetadataValue("HintPath")), IsPrivate);

		// For project references, we deal with duplicate elements by taking the last value (hence the .Reverse call)
		// as this follows MSBuild's actual behaviour as documented.
		// https://learn.microsoft.com/en-us/visualstudio/msbuild/comparing-properties-and-items?view=visualstudio#property-and-item-evaluation-order
		IEnumerable<ProjectItem> dedupedProjectReferences = project.GetItems("ProjectReference")
			.Reverse()
			.DistinctBy(pr => pr.EvaluatedInclude);
		Dictionary<FileReference, bool> projectReferences = dedupedProjectReferences
			.ToDictionary(pr => GetFullPath(pr.EvaluatedInclude), IsPrivate);

		List<FileReference> compileReferences = [.. project.GetItems("Compile").Select(c => GetFullPath(c.EvaluatedInclude)).Distinct()];

		bool ShouldCopy(ProjectItem item)
		{
			string copyTo = item.GetMetadataValue("CopyToOutputDirectory");
			return copyTo.Equals("Always", StringComparison.OrdinalIgnoreCase) || copyTo.Equals("PreserveNewest", StringComparison.OrdinalIgnoreCase);
		}

		// For Content and None items, no matter the ordering, if any reference has CopyToOutputDirectory set to Always or PreserveNewest,
		// then they will be copied, even if later items explicitly set it to Never.
		// This contradicts the documented MSBuild behaviour, which is very cool.
		IEnumerable<(ProjectItem item, bool shouldCopy)> contentAndNoneRefs = project.GetItems("Content")
			.Concat(project.GetItems("None"))
			.GroupBy(cr => cr.EvaluatedInclude)
			.Select(grp => (item: grp.Last(), shouldCopy: grp.Any(ShouldCopy)));
		Dictionary<FileReference, bool> contentReferences = contentAndNoneRefs
			.ToDictionary(tup => GetFullPath(tup.item.EvaluatedInclude), tup => tup.shouldCopy);

		return new()
		{
			Properties = properties,
			References = references,
			ProjectReferences = projectReferences,
			CompileReferences = compileReferences,
			ContentReferences = contentReferences,
			ProjectPath = file,
		};
	}

	private class MsBuildRegistration : IMsBuildRegistration
	{
		internal static readonly MsBuildRegistration Instance = new();

		public DirectoryReference DotnetDirectory => Unreal.DotnetDirectory;

		public FileReference DotnetPath => Unreal.DotnetPath;
	}
}

public static class IDotnetProjectInfoExtensions
{
	/// <summary>
	/// Determines if this project is a .NET Core or .NET 5+ project
	/// </summary>
	public static bool IsDotnetCoreOrDotnet5Plus(this IDotnetProjectInfo self)
	{
		static bool IsDotnetCoreOrDotnet5Plus(string targetFramework)
		{
			if (targetFramework.Contains("netstandard", StringComparison.OrdinalIgnoreCase) || targetFramework.Contains("netcoreapp", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			else if (targetFramework.StartsWith("net", StringComparison.OrdinalIgnoreCase))
			{
				string[] versionSplit = targetFramework.Substring(3).Split('.');
				if (versionSplit.Length >= 1 && Int32.TryParse(versionSplit[0], out int majorVersion))
				{
					return majorVersion >= 5;
				}
			}
			return false;
		}

		if (self.Properties.TryGetValue("TargetFramework", out string? targetFramework))
		{
			return IsDotnetCoreOrDotnet5Plus(targetFramework);
		}

		if (self.Properties.TryGetValue("TargetFrameworks", out string? targetFrameworks))
		{
			return targetFramework?.Split(';', StringSplitOptions.RemoveEmptyEntries).Any(x => IsDotnetCoreOrDotnet5Plus(x.Trim())) ?? false;
		}

		return false;
	}

	/// <summary>
	/// Returns the assembly name used by this project
	/// </summary>
	/// <returns></returns>
	public static bool TryGetAssemblyName(this IDotnetProjectInfo self, [NotNullWhen(true)] out string? assemblyName)
	{
		if (self.Properties.TryGetValue("AssemblyName", out assemblyName))
		{
			return true;
		}
		else if (self.IsDotnetCoreOrDotnet5Plus())
		{
			assemblyName = self.ProjectPath.GetFileNameWithoutExtension();
			return true;
		}
		return false;
	}

	/// <summary>
	/// Determines all the compiled build products (executable, etc...) directly built from this project.
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="outputDir">The output directory</param>
	/// <param name="buildProducts">Receives the set of build products</param>
	public static void FindCompiledBuildProducts(this IDotnetProjectInfo self, DirectoryReference outputDir, HashSet<FileReference> buildProducts)
	{
		static void AddOptionalBuildProduct(FileReference buildProduct, HashSet<FileReference> buildProducts)
		{
			if (FileReference.Exists(buildProduct))
			{
				buildProducts.Add(buildProduct);
			}
		}

		string? outputType, assemblyName;
		if (self.Properties.TryGetValue("OutputType", out outputType) && self.TryGetAssemblyName(out assemblyName))
		{
			switch (outputType)
			{
				case "Exe":
				case "WinExe":
					string executableExtension = RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : "";
					buildProducts.Add(FileReference.Combine(outputDir, assemblyName + executableExtension));
					// dotnet outputs a apphost executable and a dll with the actual assembly
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".dll"), buildProducts);
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".pdb"), buildProducts);
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".exe.config"), buildProducts);
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".exe.mdb"), buildProducts);
					break;
				case "Library":
					buildProducts.Add(FileReference.Combine(outputDir, assemblyName + ".dll"));
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".pdb"), buildProducts);
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".dll.config"), buildProducts);
					AddOptionalBuildProduct(FileReference.Combine(outputDir, assemblyName + ".dll.mdb"), buildProducts);
					break;
			}
		}
	}

	/// <summary>
	/// Resolve the project's output directory
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="outputDir">If successful, receives the output directory</param>
	/// <returns>True if the output directory was found</returns>
	public static bool TryGetOutputDir(this IDotnetProjectInfo self, [NotNullWhen(true)] out DirectoryReference? outputDir)
	{
		string? outputPath;
		if (self.Properties.TryGetValue("OutputPath", out outputPath))
		{
			outputDir = DirectoryReference.Combine(self.ProjectPath.Directory, outputPath);
			return true;
		}
		else if (self.IsDotnetCoreOrDotnet5Plus())
		{
			string configuration = self.Properties.ContainsKey("Configuration") ? self.Properties["Configuration"] : "Development";
			outputDir = DirectoryReference.Combine(self.ProjectPath.Directory, "bin", configuration, self.Properties["TargetFramework"]);
			return true;
		}
		else
		{
			outputDir = null;
			return false;
		}
	}

	/// <summary>
	/// Get the ouptut file for this project
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="file">If successful, receives the assembly path</param>
	/// <returns>True if the output file was found</returns>
	public static bool TryGetOutputFile(this IDotnetProjectInfo self, [NotNullWhen(true)] out FileReference? file)
	{
		DirectoryReference? outputDir;
		if (!self.TryGetOutputDir(out outputDir))
		{
			file = null;
			return false;
		}

		string? assemblyName;
		if (!self.TryGetAssemblyName(out assemblyName))
		{
			file = null;
			return false;
		}

		file = FileReference.Combine(outputDir, assemblyName + ".dll");
		return true;
	}

	/// <summary>
	/// Resolve the project's output directory
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="baseDirectory">Base directory to resolve relative paths to</param>
	/// <returns>The configured output directory</returns>
	public static DirectoryReference GetOutputDir(this IDotnetProjectInfo self, DirectoryReference baseDirectory)
	{
		string? outputPath;
		if (self.Properties.TryGetValue("OutputPath", out outputPath))
		{
			return DirectoryReference.Combine(baseDirectory, outputPath);
		}
		else if (self.IsDotnetCoreOrDotnet5Plus())
		{
			string configuration = self.Properties.TryGetValue("Configuration", out string? value) ? value : "Development";
			return !self.Properties.TryGetValue("AppendTargetFrameworkToOutputPath", out string? appendFrameworkStr) || appendFrameworkStr.Equals("true", StringComparison.OrdinalIgnoreCase)
				? DirectoryReference.Combine(baseDirectory, "bin", configuration, self.Properties["TargetFramework"])
				: DirectoryReference.Combine(baseDirectory, "bin", configuration);
		}
		else
		{
			return baseDirectory;
		}
	}

	/// <summary>
	/// Adds the given file and any additional build products to the output set
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="outputFile">The assembly to add</param>
	/// <param name="outputFiles">Set to receive the file and support files</param>
	public static void AddReferencedAssemblyAndSupportFiles(this IDotnetProjectInfo self, FileReference outputFile, HashSet<FileReference> outputFiles)
	{
		outputFiles.Add(outputFile);

		FileReference symbolFile = outputFile.ChangeExtension(".pdb");
		if (FileReference.Exists(symbolFile))
		{
			outputFiles.Add(symbolFile);
		}

		FileReference documentationFile = outputFile.ChangeExtension(".xml");
		if (FileReference.Exists(documentationFile))
		{
			outputFiles.Add(documentationFile);
		}
	}

	/// <summary>
	/// Finds all build products from this project. This includes content and other assemblies marked to be copied local.
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="outputDir">The output directory</param>
	/// <param name="buildProducts">Receives the set of build products</param>
	/// <param name="projectFileToInfo">Map of project file to information, to resolve build products from referenced projects copied locally</param>
	public static void FindBuildProducts(
		this IDotnetProjectInfo self,
		DirectoryReference outputDir,
		HashSet<FileReference> buildProducts,
		Dictionary<FileReference, IDotnetProjectInfo> projectFileToInfo)
	{
		// Add the standard build products
		self.FindCompiledBuildProducts(outputDir, buildProducts);

		// Add the referenced assemblies which are marked to be copied into the output directory. This only happens for the main project, and does not happen for referenced projects.
		foreach (KeyValuePair<FileReference, bool> reference in self.References)
		{
			if (reference.Value)
			{
				FileReference outputFile = FileReference.Combine(outputDir, reference.Key.GetFileName());
				self.AddReferencedAssemblyAndSupportFiles(outputFile, buildProducts);
			}
		}

		// Copy the build products for any referenced projects. Note that this does NOT operate recursively.
		foreach (KeyValuePair<FileReference, bool> projectReference in self.ProjectReferences)
		{
			IDotnetProjectInfo? otherProjectInfo;
			if (projectFileToInfo.TryGetValue(projectReference.Key, out otherProjectInfo))
			{
				otherProjectInfo.FindCompiledBuildProducts(outputDir, buildProducts);
			}
		}

		// Add any copied content. This DOES operate recursively.
		self.FindCopiedContent(outputDir, buildProducts, projectFileToInfo);
	}

	/// <summary>
	/// Finds all content which will be copied into the output directory for this project. This includes content from any project references as "copy local" recursively (though MSBuild only traverses a single reference for actual binaries, in such cases)
	/// </summary>
	/// <param name="self">The target .NET project info</param>
	/// <param name="outputDir">The output directory</param>
	/// <param name="outputFiles">Receives the set of build products</param>
	/// <param name="projectFileToInfo">Map of project file to information, to resolve build products from referenced projects copied locally</param>
	private static void FindCopiedContent(
		this IDotnetProjectInfo self,
		DirectoryReference outputDir,
		HashSet<FileReference> outputFiles,
		Dictionary<FileReference, IDotnetProjectInfo> projectFileToInfo)
	{
		// Copy any referenced projects too.
		foreach (KeyValuePair<FileReference, bool> projectReference in self.ProjectReferences)
		{
			IDotnetProjectInfo? otherProjectInfo;
			if (projectFileToInfo.TryGetValue(projectReference.Key, out otherProjectInfo))
			{
				otherProjectInfo.FindCopiedContent(outputDir, outputFiles, projectFileToInfo);
			}
		}

		// Add the content which is copied to the output directory
		foreach (KeyValuePair<FileReference, bool> contentReference in self.ContentReferences)
		{
			FileReference contentFile = contentReference.Key;
			if (contentReference.Value)
			{
				outputFiles.Add(FileReference.Combine(outputDir, contentFile.GetFileName()));
			}
		}
	}
}
