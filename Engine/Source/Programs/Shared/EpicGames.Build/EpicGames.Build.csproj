<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <RootNamespace>UnrealBuildBase</RootNamespace>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
  </PropertyGroup>

  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Analyze|AnyCPU' ">
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\EpicGames.MsBuild\EpicGames.MsBuild.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.FileSystemGlobbing" Version="10.0.6" />
  </ItemGroup>
</Project>
