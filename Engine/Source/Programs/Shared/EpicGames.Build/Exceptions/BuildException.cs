// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

// This namespace is intentional for backwards compatibility.
// BuildException will eventually be marked Obsolete in favor of BuildLogEventException
namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for exceptions thrown by UnrealBuildTool and AutomationTool
	/// </summary>
	[SuppressMessage("Naming", "CA1724:Type names should not match namespaces", Justification = "Renaming would break public api")]
	public class BuildException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Message">The error message to display.</param>
		public BuildException(string Message)
			: base(Message)
		{
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="InnerException">An inner exception to wrap</param>
		/// <param name="Message">The error message to display.</param>
		public BuildException(Exception? InnerException, string Message)
			: base(Message, InnerException)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Format">Formatting string for the error message</param>
		/// <param name="Arguments">Arguments for the formatting string</param>
		public BuildException(string Format, params object?[] Arguments)
			: base(String.Format(Format, Arguments))
		{
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="InnerException">The inner exception being wrapped</param>
		/// <param name="Format">Format for the message string</param>
		/// <param name="Arguments">Format arguments</param>
		public BuildException(Exception InnerException, string Format, params object?[] Arguments)
			: base(String.Format(Format, Arguments), InnerException)
		{
		}

		/// <summary>
		/// Log BuildException with a provided ILogger
		/// </summary>
		/// <param name="Logger">The ILogger to use to log this exception</param>
		public virtual void LogException(ILogger Logger)
		{
			Logger.LogError(this, "{Ex}", ExceptionUtils.FormatException(this));
			Logger.LogDebug(this, "{Ex}", ExceptionUtils.FormatExceptionDetails(this));
		}

		/// <summary>
		/// Returns the string representing the exception. Our build exceptions do not show the callstack since they are used to report known error conditions.
		/// </summary>
		/// <returns>Message for the exception</returns>
		public override string ToString()
		{
			return Message;
		}
	}
}

