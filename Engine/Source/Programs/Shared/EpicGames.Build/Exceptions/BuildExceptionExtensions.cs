// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Extension methods for build exceptions
	/// </summary>
	public static class BuildExceptionExtensions
	{
		/// <summary>
		/// Log Exception with a provided ILogger
		/// </summary>
		/// <param name="ex">The exception to log</param>
		/// <param name="logger">The ILogger to use to log this exception</param>
		public static void LogException(this Exception ex, ILogger logger)
		{
			if (ex is UnrealBuildTool.BuildException buildException)
			{
				buildException.LogException(logger);
			}
			else if (ex is JsonException jsonException)
			{
				FileReference source = new(jsonException.Path ?? jsonException.Source ?? "unknown");
				LogValue fileValue = LogValue.SourceFile(source, source.GetFileName());
				long line = jsonException.LineNumber ?? 0;
				logger.LogError(KnownLogEvents.Compiler, "{File}({Line}): error: {Message}", fileValue, line, ExceptionUtils.FormatExceptionDetails(ex));
			}
			else if (ex is AggregateException aggregateException)
			{
				logger.LogInformation(ex, "Unhandled {Count} aggregate exceptions", aggregateException.InnerExceptions.Count);

				foreach (Exception innerException in aggregateException.InnerExceptions)
				{
					innerException.LogException(logger);
				}
			}
			else
			{
				logger.LogError(ex, "Unhandled exception: {Ex}", ExceptionUtils.FormatExceptionDetails(ex));
			}
		}

		/// <summary>
		/// Get the CompilationResult for a provided Exception
		/// </summary>
		/// <param name="ex">The exception to get the result for</param>
		/// <returns>CompilationResult</returns>
		public static CompilationResult GetCompilationResult(this Exception ex)
		{
			if (ex is CompilationResultException compilationResultsException)
			{
				return compilationResultsException.Result;
			}
			else if (ex is AggregateException aggregateException)
			{
				foreach (Exception innerException in aggregateException.InnerExceptions)
				{
					CompilationResult result = GetCompilationResult(innerException);
					if (result != CompilationResult.OtherCompilationError)
					{
						return result;
					}
				}
			}
			else if (ex.InnerException != null)
			{
				return GetCompilationResult(ex.InnerException);
			}
			return CompilationResult.OtherCompilationError;
		}
	}
}
