// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Implementation of <see cref="UnrealBuildTool.BuildException"/> that captures a full structured logging event.
	/// </summary>
	/// <param name="InnerException">The inner exception</param>
	/// <param name="Event">Event to construct from</param>
	public class BuildLogEventException(Exception? InnerException, LogEvent Event) : UnrealBuildTool.BuildException(InnerException, Event.ToString())
	{
		/// <summary>
		/// The event object
		/// </summary>
		public LogEvent Event { get; } = Event;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Event">Event to construct from</param>
		public BuildLogEventException(LogEvent Event)
			: this(null, Event)
		{
		}

		/// <inheritdoc/>
		public BuildLogEventException(string Format, params object[] Arguments)
			: this(LogEvent.Create(LogLevel.Error, Format, Arguments))
		{
		}

		/// <inheritdoc/>
		public BuildLogEventException(Exception? InnerException, string Format, params object[] Arguments)
			: this(InnerException, LogEvent.Create(LogLevel.Error, default, InnerException, Format, Arguments))
		{
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="EventId">Event id for the error</param>
		/// <param name="InnerException">Inner exception to wrap</param>
		/// <param name="Format">Structured logging format string</param>
		/// <param name="Arguments">Argument objects</param>
		public BuildLogEventException(Exception? InnerException, EventId EventId, string Format, params object[] Arguments)
			: this(InnerException, LogEvent.Create(LogLevel.Error, EventId, InnerException, Format, Arguments))
		{
		}

		/// <inheritdoc/>
		public override void LogException(ILogger Logger)
		{
			Logger.Log(Event.Level, Event.Id, Event, this, (s, e) => s.ToString());
			Logger.LogDebug(Event.Id, this, "{Ex}", ExceptionUtils.FormatExceptionDetails(this));
		}
	}
}
