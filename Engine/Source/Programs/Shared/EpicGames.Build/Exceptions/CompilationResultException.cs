// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Implementation of <see cref="BuildLogEventException"/> that will return a unique exit code.
	/// </summary>
	public class CompilationResultException : BuildLogEventException
	{
		/// <summary>
		/// The exit code associated with this exception
		/// </summary>
		public CompilationResult Result { get; }

		readonly bool HasMessage = true;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		public CompilationResultException(CompilationResult Result)
			: base(LogEvent.Create(LogLevel.Error, "{CompilationResult}", Result))
		{
			HasMessage = false;
			this.Result = Result;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="Event">Event to construct from</param>
		public CompilationResultException(CompilationResult Result, LogEvent Event)
			: base(Event)
		{
			this.Result = Result;
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="InnerException">The inner exception</param>
		/// <param name="Event">Event to construct from</param>
		public CompilationResultException(CompilationResult Result, Exception? InnerException, LogEvent Event)
			: base(InnerException, Event)
		{
			this.Result = Result;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="EventId">Event id for the error</param>
		/// <param name="Format">Formatting string for the error message</param>
		/// <param name="Arguments">Arguments for the formatting string</param>
		public CompilationResultException(CompilationResult Result, EventId EventId, string Format, params object[] Arguments)
			: base(LogEvent.Create(LogLevel.Error, EventId, Format, Arguments))
		{
			this.Result = Result;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="Format">Formatting string for the error message</param>
		/// <param name="Arguments">Arguments for the formatting string</param>
		public CompilationResultException(CompilationResult Result, string Format, params object[] Arguments)
			: base(LogEvent.Create(LogLevel.Error, Format, Arguments))
		{
			this.Result = Result;
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="InnerException">The inner exception being wrapped</param>
		/// <param name="Format">Format for the message string</param>
		/// <param name="Arguments">Format arguments</param>
		public CompilationResultException(CompilationResult Result, Exception? InnerException, string Format, params object[] Arguments)
			: base(InnerException, LogEvent.Create(LogLevel.Error, default, InnerException, Format, Arguments))
		{
			this.Result = Result;
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="Result">The resulting exit code</param>
		/// <param name="EventId">Event id for the error</param>
		/// <param name="InnerException">Inner exception to wrap</param>
		/// <param name="Format">Structured logging format string</param>
		/// <param name="Arguments">Argument objects</param>
		public CompilationResultException(CompilationResult Result, Exception? InnerException, EventId EventId, string Format, params object[] Arguments)
			: base(InnerException, LogEvent.Create(LogLevel.Error, EventId, InnerException, Format, Arguments))
		{
			this.Result = Result;
		}

		/// <inheritdoc/>
		public override void LogException(ILogger Logger)
		{
			if (HasMessage)
			{
				Logger.Log(Event.Level, Event.Id, Event, this, (s, e) => s.ToString());
			}
			Logger.LogDebug(Event.Id, this, "{Ex}", ExceptionUtils.FormatExceptionDetails(this));
		}
	}
}
