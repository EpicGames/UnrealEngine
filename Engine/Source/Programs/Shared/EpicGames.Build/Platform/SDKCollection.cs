// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Core
{
	public class SDKCollection(UEBuildPlatformSDK inPlatformSDK)
	{
		public virtual string DefaultName => "Sdk";
		public virtual bool IsValid(string version, SDKDescriptor info)
		{
			return PlatformSDK.IsVersionValid(version, info);
		}

		public UEBuildPlatformSDK PlatformSDK { get; init; } = inPlatformSDK;
		public List<SDKDescriptor> Sdks { get; init; } = [];
		public List<SDKDescriptor> FullSdks => [.. Sdks.Where(x => !String.Equals(x.Name, "AutoSdk", StringComparison.OrdinalIgnoreCase))];
		public SDKDescriptor? AutoSdk => Sdks.FirstOrDefault(x => String.Equals(x.Name, "AutoSdk", StringComparison.OrdinalIgnoreCase));

		public SDKCollection(SDKCollection other, UEBuildPlatformSDK platformSDK)
			: this(platformSDK)
		{
			foreach (SDKDescriptor desc in other.Sdks)
			{
				SetupSDK(desc.Name, desc.Min, desc.Max, desc.Current, desc.GroupName);
			}
		}

		// convenience for single unnamed Sdk range
		public SDKCollection(string? min, string? max, UEBuildPlatformSDK platformSDK)
			: this(platformSDK)
		{
			Sdks.Add(new SDKDescriptor(DefaultName, min, max, null, null, this));
		}

		// convenience for single unnamed Sdk current value
		public SDKCollection(string? current, UEBuildPlatformSDK platformSDK)
			: this(platformSDK)
		{
			if (current != null)
			{
				Sdks.Add(new SDKDescriptor(DefaultName, null, null, current, null, this));
			}
		}

		public bool AreAllManualSDKsValid()
		{
			IEnumerable<SDKDescriptor>? sdks = FullSdks;

			if (sdks == null)
			{
				return true;
			}

			// check if all the SDKs not in groups are valid (All returns true if empty set)
			bool bAllUngroupedAreValid = sdks
				.Where(desc => desc.GroupName == null)
				.All(desc => desc.Validity == SDKStatus.Valid);

			// for each groupname (ToHashSet removes duplicates), check if at least 1 SDK with that group name is valid
			bool bAllGroupsHaveOneValid = sdks
				.Where(desc => desc.GroupName != null)
				.Select(desc => desc.GroupName)
				.ToHashSet()
				.All(groupName => sdks.Any(desc => desc.GroupName == groupName && desc.Validity == SDKStatus.Valid));

			return bAllUngroupedAreValid && bAllGroupsHaveOneValid;
		}

		public bool IsAutoSDKValid()
		{
			return AutoSdk?.Validity == SDKStatus.Valid;
		}

		public string ToString(bool bIncludeCurrent = true)
		{
			// by default print something like "MinVersion=1.0, MaxVersion=2.0"
			return ToString("Version", "Version");
		}

		public string ToString(string descriptor, string? currentDescriptor = null)
		{
			if (Sdks.Count == 1)
			{
				return Sdks[0].ToString(descriptor, currentDescriptor, false);
			}
			return String.Join(", ", Sdks.Select(x => x.ToString(descriptor, currentDescriptor, true)));
		}

		public static string ToMultilineString(bool bIncludeCurrent = true)
		{
			return $"";// MinVersion={Min ?? ""}, MaxVersion={Max ?? ""}" + (bIncludeCurrent ? $", Current={Current ?? ""}" : "");
		}

		public void SetupSDK(string name, string? min, string? max, string? current, string? groupName)
		{
			Sdks.Add(new SDKDescriptor(name, min, max, current, groupName, this));
		}

		public void SetupCurrent(string name, string current)
		{
			Sdks.Add(new SDKDescriptor(name, null, null, current, null, this));
		}

		public bool UpdateCurrentForSingle(string name, string currentVersion)
		{
			if (PlatformSDK == null)
			{
				throw new Exception("Cannot call UpdateCurrentForSingle in a SDKCollection without a PlatformSDK set");
			}

			SDKDescriptor? info;

			// for ease, we assume that if there is only one "Sdk" or "Software" in this Collection, then any Name passed in will work with it
			// so just use that one entry
			if (Sdks.Count == 1 && (Sdks[0].Name == "Sdk" || Sdks[0].Name == "Software"))
			{
				info = Sdks[0];
			}
			else
			{
				info = Sdks.FirstOrDefault(x => String.Equals(x.Name, name, StringComparison.OrdinalIgnoreCase));
				if (info == null)
				{
					return false;
				}
			}

			info.UpdateCurrent(currentVersion, name, this);
			return true;
		}
	}
}
