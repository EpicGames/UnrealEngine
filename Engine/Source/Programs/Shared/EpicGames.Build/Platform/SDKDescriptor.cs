// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;

namespace EpicGames.Core
{
	/// <summary>
	/// Range of SDK versions that are valid for a platform
	/// </summary>
	public class SDKDescriptor
	{
		public required string Name { get; init; }
		public required string? Min { get; init; }
		public required string? Max { get; init; }
		public required string? Current { get; set; }

		// for SDKs where you need 1 of a set of SDKs (for instance, multiple toolchains for one platform), give them all the same GroupName
		// and Turnkey will make sure that at least one of them is installed for the SDK setup to be valid. Leave null otherwise
		public required string? GroupName { get; init; }

		public ulong MinInt { get; init; } = 0;
		public ulong MaxInt { get; init; } = UInt64.MaxValue;
		public ulong CurrentInt { get; set; } = 0;

		public SDKStatus Validity { get; set; } = SDKStatus.Invalid;

		[SetsRequiredMembers]
		public SDKDescriptor(string inName, string? inMin, string? inMax, string? inCurrent, string? inGroupName, SDKCollection? collection)
		{
			Name = inName;
			Min = inMin;
			Max = inMax;
			Current = inCurrent;
			GroupName = inGroupName;

			if (collection != null)
			{
				MinInt = collection.PlatformSDK.ConvertVersionToInt(Min, Name);
				// null will convert to 0, but we want MaxValue on null
				MaxInt = Max == null ? UInt64.MaxValue : collection.PlatformSDK.ConvertVersionToInt(Max, Name);
				CurrentInt = collection.PlatformSDK.ConvertVersionToInt(Current, Name);
				UpdateValidity(collection);
			}
		}

		[SetsRequiredMembers]
		public SDKDescriptor(string inName, string? inMin, string? inMax)
			: this(inName, inMin, inMax, null, null, null)
		{
		}

		[SetsRequiredMembers]
		public SDKDescriptor(string inName, string? inCurrent)
			: this(inName, null, null, inCurrent, null, null)
		{
		}

		public void UpdateCurrent(string inCurrent, string? hint, SDKCollection? collection)
		{
			Current = inCurrent;
			if (collection != null)
			{
				CurrentInt = collection.PlatformSDK.ConvertVersionToInt(Current, hint);
				UpdateValidity(collection);
			}
		}

		private void UpdateValidity(SDKCollection collection)
		{
			// instead of just checking the range numerically, we allow platforms to override the IsValid check, which makes this a little convoluted (calling back into the Collection 
			// so it can call back into the PlatformSDK with the proper SDK vs Software function)
			Validity = (Current != null && collection.IsValid(Current, this)) ? SDKStatus.Valid : SDKStatus.Invalid;
		}

		public string ToString(bool bIncludeCurrent = true)
		{
			return $"MinVersion={Min ?? ""}, MaxVersion={Max ?? ""}" + (bIncludeCurrent ? $", Current={Current ?? ""}" : "");
		}
		public string ToString(string descriptor, string? currentDescriptor, bool bIncludeName)
		{
			string displayName = bIncludeName ? "_" + Name : "";

			// if they are different, print something like:
			//   MinAllowed_Toolchain=1.0, MaxAllowed_Toolchain=2.0
			if (Min != Max)
			{
				return $"Min{descriptor}{displayName}={Min ?? ""}, Max{descriptor}{displayName}={Max ?? ""}" + (currentDescriptor != null ? $", Current{currentDescriptor}{displayName}={Current ?? ""}" : "");
			}
			// if they are the same, print something like:
			//   Allowed_Toolchain=1.0
			return $"{descriptor}{displayName}={Min}" + (currentDescriptor != null ? $", Current{currentDescriptor}{displayName}={Current ?? ""}" : "");
		}
	}
}
