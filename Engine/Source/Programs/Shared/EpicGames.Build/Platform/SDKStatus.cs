// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Core
{
	/// <summary>
	/// SDK installation status
	/// </summary>
	public enum SDKStatus
	{
		/// <summary>
		/// Desired SDK is installed and set up.
		/// </summary>
		Valid,

		/// <summary>
		/// Could not find the desired SDK, SDK setup failed, etc.
		/// </summary>
		Invalid,
	};
}
