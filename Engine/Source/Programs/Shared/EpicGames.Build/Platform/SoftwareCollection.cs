// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Core
{
	public class SoftwareCollection : SDKCollection
	{
		public SoftwareCollection(UEBuildPlatformSDK platformSDK)
			: base(platformSDK)
		{
		}
		public SoftwareCollection(string? min, string? max, UEBuildPlatformSDK platformSDK)
			: base(min, max, platformSDK)
		{
		}
		public SoftwareCollection(string? current, UEBuildPlatformSDK platformSDK)
			: base(current, platformSDK)
		{
		}

		public override string DefaultName => "Software";

		public override bool IsValid(string version, SDKDescriptor info)
		{
			return PlatformSDK.IsSoftwareVersionValid(version, info);
		}
	}
}
