// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace EpicGames.Core
{
	/// <summary>
	/// SDK for a platform
	/// </summary>
	public abstract class UEBuildPlatformSDK(ILogger inLogger)
	{
		// Public SDK handling, not specific to AutoSDK

		protected readonly ILogger Logger = inLogger;

		protected readonly HashSet<FileReference> LoadedConfigFiles = [];

		#region Global SDK Registration

		/// <summary>
		/// Registers the SDK for a given platform (as a string, but equivalent to UnrealTargetPlatform)
		/// </summary>
		/// <param name="sdk">SDK object</param>
		/// <param name="platformName">Platform name for this SDK</param>
		/// <param name="bIsSdkAllowedOnHost"></param>
		public static void RegisterSDKForPlatform(UEBuildPlatformSDK sdk, string platformName, bool bIsSdkAllowedOnHost)
		{
			// verify that neither platform or sdk were added before
			if (SDKRegistry.Any(x => x.Key == platformName || x.Value == sdk))
			{
				throw new Exception(String.Format("Re-registering SDK for {0}. All Platforms must have a unique SDK object", platformName));
			}

			SDKRegistry.Add(platformName, sdk);

			sdk.Init(platformName, bIsSdkAllowedOnHost);
		}

		private void Init(string inPlatformName, bool bInIsSdkAllowedOnHost)
		{
			PlatformName = inPlatformName;
			bIsSdkAllowedOnHost = bInIsSdkAllowedOnHost;

			// load the SDK config file
			if (bIsSdkAllowedOnHost)
			{
				LoadJsonFile(PlatformName);
			}

			// if the parent set up autosdk, the env vars will be wrong, but we can still get the manual SDK version from before it was setup
			string? parentManualSDKVersions = Environment.GetEnvironmentVariable(GetPlatformManualSDKSetupEnvVar());
			if (!String.IsNullOrEmpty(parentManualSDKVersions))
			{

				// we pass along __None to indicate the parent didn't have a manual sdk installed
				if (parentManualSDKVersions == "__None")
				{
					// empty it out
					CachedManualSDKVersions = [];
				}
				else
				{
					Logger.LogInformation("Processing parent manual sdk: {SDKVersion}", parentManualSDKVersions);
					CachedManualSDKVersions = JsonSerializer.Deserialize<Dictionary<string, string>>(parentManualSDKVersions) ?? [];
				}
			}
			else
			{
				// allow the platform to select the most appropriate manually installed SDK if it hasn't already been set up by AutoSDK
				bool bHasAutoSDK = PlatformSupportsAutoSDKs() && HasAutoSDKSystemEnabled(); // @todo: PlatformSupportsAutoSDKs & HasAutoSDKSystemEnabled check is not technically needed for manual sdk switching but the editor-side code is heavily tied into this and would require further refactoring first
				if ((bHasAutoSDK || !ManualSDKAutoSwitchRequiresAutoSDK) && !HasParentProcessSetupAutoSDK(out _))
				{
					if (TrySelectBestManualSDK(out string? selectedSDKVersion))
					{
						Logger.LogInformation("Auto-selected best installed sdk for {PlatformName} - {SDKVersion}", GetAutoSDKPlatformName(), selectedSDKVersion);
					}
				}

				CachedManualSDKVersions = [];

				// if there was no parent, get the SDK version before we run AutoSDK to get the manual version
				SDKCollection installedVersions = GetInstalledSDKVersions();
				foreach (SDKDescriptor desc in installedVersions.FullSdks)
				{
					CachedManualSDKVersions[desc.Name] = desc.Current!;
				}
			}
		}

		#endregion

		#region Main Public Interface/Utilties

		/// <summary>
		/// Retrieves a previously registered SDK for a given platform
		/// </summary>
		/// <param name="platformName">String name of the platform (equivalent to UnrealTargetPlatform)</param>
		/// <returns></returns>
		public static UEBuildPlatformSDK? GetSDKForPlatform(string platformName)
		{
			UEBuildPlatformSDK? sdk;
			SDKRegistry.TryGetValue(platformName, out sdk);

			return sdk;
		}

		public static T? GetSDKForPlatformOrMakeTemp<T>(string platformName) where T : UEBuildPlatformSDK
		{
			UEBuildPlatformSDK? sdk;
			SDKRegistry.TryGetValue(platformName, out sdk);

			// make a temp one if needed, this is not expected to happen often at all
			if (sdk == null)
			{
				if (!TempSDKRegistry.TryGetValue(platformName, out sdk))
				{
					object[] parameter = [Log.Logger];
					sdk = (UEBuildPlatformSDK)Activator.CreateInstance(typeof(T), parameter)!;
					// by setting this to false, we don't require any of the RequiredVersions to exist, but if they do, they will be read
					// this is useful on other platforms that 
					sdk.bIsSdkAllowedOnHost = false;
					sdk.LoadJsonFile(platformName);
					TempSDKRegistry.Add(platformName, sdk);
				}
			}
			return (T?)sdk;
		}

		/// <summary>
		/// Gets the set of all known SDKs
		/// </summary>
		public static UEBuildPlatformSDK[] AllPlatformSDKObjects => [.. SDKRegistry.Values];

		// True if at least one platform has had its version changed from the standard SDK version, which means it may not be compatible with other projects
		public static bool bHasAnySDKOverride = false;

		// True if this SDK has had its version changed from the standard SDK version, which means it may not be compatible with other projects
		public bool bHasSDKOverride = false;

		// Contains a list of projects that had per-project SDK overrides, used for validating conflicting SDKs
		public List<FileReference> ProjectsThatOverrodeSDK = [];

		// String name of the platform (will match an UnrealTargetPlatform)
		public string? PlatformName;

		// True if this Sdk is allowed to be used by this host - if not, we can skip a lot 
		public bool bIsSdkAllowedOnHost;

		// External dependencies for this Sdk (config files)
		public IEnumerable<FileItem> ExternalDependencies => LoadedConfigFiles.Select(FileItem.GetItemByFileReference);

		public SDKCollection GetAllSDKInfo()
		{
			SDKCollection allSdks = new(this);

			if (bIsSdkAllowedOnHost)
			{
				// walk over each one version the platform supports, and get it's current installed info
				foreach (SDKDescriptor desc in GetValidVersions().Sdks)
				{
					allSdks.SetupSDK(desc.Name, desc.Min, desc.Max, CachedManualSDKVersions.GetValueOrDefault(desc.Name), desc.GroupName);
				}

				// now get current AutoSDK version (if autosdk is set up, then the GetInstalledSDKVersion will return AutoSDK version)
				bool bIsAutoSDK = false;
				string? currentAutoSDKVersion = (PlatformSupportsAutoSDKs() && HasRequiredAutoSDKInstalled() == SDKStatus.Valid && HasSetupAutoSDK()) ? GetInstalledVersion(out bIsAutoSDK) : null;
				allSdks.SetupSDK("AutoSdk", GetMainVersion(), GetMainVersion(), currentAutoSDKVersion, null);

				// verify some assumptions
				if (currentAutoSDKVersion != null && bIsAutoSDK == false)
				{
					throw new Exception($"AutoSDK was indicated to be setup ({currentAutoSDKVersion}), but GetInstalledSDKVersion returned false for bIsAutoSDK");
				}
				if (currentAutoSDKVersion != null && currentAutoSDKVersion != GetMainVersion())
				{
					throw new Exception($"AutoSDK was indicated to be setup, but the version if returned ({currentAutoSDKVersion}) doesn't equal the MainVersion ({GetMainVersion()}");
				}
			}

			return allSdks;
		}

		public SDKDescriptor? GetSDKInfo(string sdkName)
		{
			return GetAllSDKInfo().Sdks.FirstOrDefault(x => String.Equals(x.Name, sdkName, StringComparison.OrdinalIgnoreCase));
		}

		public SDKDescriptor? GetSoftwareInfo(string? sdkName = null)
		{
			// todo: make this queryable?
			sdkName ??= "Software";
			return GetAllSoftwareInfo().Sdks.FirstOrDefault(x => String.Equals(x.Name, sdkName, StringComparison.OrdinalIgnoreCase));
		}

		public SDKCollection GetAllSoftwareInfo(string? deviceType = null, string? current = null)
		{
			SDKCollection allSoftwares = new(this);

			// walk over the valid software ranges, potentially setting the current verison if one was passed in
			// if there is only one known SDK named "Software" we allow any DeviceType
			IEnumerable<SDKDescriptor> softwareSDKs = GetValidSoftwareVersions().Sdks;

			foreach (SDKDescriptor desc in softwareSDKs)
			{
				// are we restricting this to one type?
				if (deviceType == null || String.Equals(desc.Name, deviceType, StringComparison.OrdinalIgnoreCase))
				{
					allSoftwares.SetupSDK(desc.Name, desc.Min, desc.Max, null, desc.GroupName);
				}

				if (current != null)
				{
					// set the current for this one (likely it will only be for DeviceType, but support passing in null DeviceType, and non-null Current
					allSoftwares.UpdateCurrentForSingle(desc.Name, current);
				}
			}

			return allSoftwares;
		}

		//public SDKCollection GetAllSoftwareInfo(string DeviceId)
		//{
		//	SDKCollection AllSoftwares =
		//}

		public string? GetInstalledVersion(out bool bIsAutoSDK)
		{
			bIsAutoSDK = HasSetupAutoSDK();
			return GetInstalledSDKVersion();
		}

		public string? GetInstalledVersion()
		{
			return GetInstalledSDKVersion();
		}

		public string? GetInstalledVersion(string sdkName)
		{
			return GetSDKInfo(sdkName)?.Current;
		}

		//		public void GetInstalledVersions(out string? ManualSDKVersion, out string? AutoSDKVersion)
		//		{
		//			// if we support AutoSDKs, then return both versions
		//			if (PlatformSupportsAutoSDKs())
		//			{
		//				AutoSDKVersion = (HasRequiredAutoSDKInstalled() == SDKStatus.Valid) ? GetInstalledSDKVersion() : null;
		////				AutoSDKVersion = GetInstalledSDKVersion();
		//			}
		//			else
		//			{
		//				AutoSDKVersion = null;
		//				if (CachedManualSDKVersion != GetInstalledSDKVersion())
		//				{
		//					throw new Exception("Manual SDK version changed, this is not supported yet");
		//				}
		//			}

		//			ManualSDKVersion = CachedManualSDKVersion;
		//		}

		public virtual bool IsVersionValid(string? version, string sdkType) => IsVersionValidInternal(version, sdkType, null);

		public virtual bool IsVersionValid(string? version, SDKDescriptor info) => IsVersionValidInternal(version, null, info);

		public virtual bool IsSoftwareVersionValid(string version, string deviceType) => IsSoftwareVersionValidInternal(version, deviceType, null);

		public virtual bool IsSoftwareVersionValid(string version, SDKDescriptor info) => IsSoftwareVersionValidInternal(version, null, info);

		public void ReactivateAutoSDK()
		{
			// @todo turnkey: this needs to force re-doing it, as it is likely a no-op, need to investigate what to clear out
			ManageAndValidateSDK();
		}

		#endregion

		#region Platform Overrides

		protected virtual string MainVersionKey => "MainVersion";
		protected virtual string MinVersionKey => "MinVersion";
		protected virtual string MaxVersionKey => "MaxVersion";

		protected virtual string BannedSdkVersionsKey => "BannedSdkVersions";

		/// <summary>
		/// Return the SDK version that the platform wants to use (AutoSDK dir must match this, full SDKs can be in a valid range)
		/// </summary>
		/// <returns></returns>
		public virtual string GetMainVersion()
		{
			// by default, look up in SDK config value
			return GetRequiredVersionFromConfig(MainVersionKey);
		}

		/// <summary>
		/// Gets the valid string range of Sdk versions. TryConvertVersionToInt() will need to succeed to make this usable for range checks
		/// </summary>
		/// <param name="minVersion">Smallest version allowed</param>
		/// <param name="maxVersion">Largest version allowed (inclusive)</param>
		protected virtual void GetValidVersionRange(out string? minVersion, out string? maxVersion)
		{
			// by default, use SDK config file values
			minVersion = GetVersionFromConfig(MinVersionKey);
			maxVersion = GetVersionFromConfig(MaxVersionKey);
		}

		protected virtual SDKCollection GetValidVersions()
		{
			// if the platform doesn't override this, then it only has one sdk, so put it into the collection as the single sdk (this is very much the standard behavior)
			GetValidVersionRange(out string? minVersion, out string? maxVersion);
			return new SDKCollection(minVersion, maxVersion, this);
		}

		/// <summary>
		/// Gets the valid string range of software/flash versions. TryConvertVersionToInt() will need to succeed to make this usable for range checks
		/// </summary>
		/// <param name="minVersion">Smallest version allowed, or null if no minmum (in other words, 0 - MaxVersion)</param>
		/// <param name="maxVersion">Largest version allowed (inclusive), or null if no maximum (in other words, MinVersion - infinity)y</param>
		protected virtual void GetValidSoftwareVersionRange(out string? minVersion, out string? maxVersion)
		{
			minVersion = GetVersionFromConfig("MinSoftwareVersion");
			maxVersion = GetVersionFromConfig("MaxSoftwareVersion");
		}

		protected virtual SoftwareCollection GetValidSoftwareVersions()
		{
			// if the platform doesn't override this, then it only has one sdk, so put it into the collection as the single software (this is very much the standard behavior)
			GetValidSoftwareVersionRange(out string? minVersion, out string? maxVersion);
			return new SoftwareCollection(minVersion, maxVersion, this);
		}

		/// <summary>
		/// Returns the installed SDK version, used to determine if up to date or not (
		/// </summary>
		/// <returns></returns>
		protected virtual string? GetInstalledSDKVersion()
		{
			if (GetInstalledSDKVersions().FullSdks.Count == 1)
			{
				return GetInstalledSDKVersions().FullSdks[0].Current;
			}
			throw new Exception($"This platform's SDK class {GetType()} did not implement GetInstalledSDKVersion(). That means it has multiple SDKs and you will need use GetAllSDKInfo() or GetInstalledVersion(SDKName)");
		}

		protected virtual SDKCollection GetInstalledSDKVersions()
		{
			if (bIsSdkAllowedOnHost)
			{
				// if the platform doesn't override this, then it only has one sdk, so put it into the collection as the single sdk (this is very much the standard behavior)
				string? version = GetInstalledSDKVersion();
				if (version != null)
				{
					return new SDKCollection(version, this);
				}
			}

			// if no manual version, then return an empty list of current sdks
			return new SDKCollection(this);
		}

		/// <summary>
		/// Returns a platform-specific version string that can be retrieved from anywhere without needing to typecast the SDK object
		/// </summary>
		/// <param name="versionType">Descriptor of the version you want</param>
		/// <returns>Version string, or empty string if not known/supported</returns>
		public virtual string GetPlatformSpecificVersion(string versionType)
		{
			return GetRequiredVersionFromConfig(versionType);
		}

		/// <summary>
		/// Return a list of full SDK versions that are already installed and can be quickly switched to without running any installers.
		/// </summary>
		/// <returns></returns>
		public virtual string[] GetAllInstalledSDKVersions()
		{
			return [];
		}

		/// <summary>
		/// Find the highest valid manually-installed SDK, prioritizing the main version if it's available. Requires that the platform SDK implements GetAllInstalledSDKVersions
		/// </summary>
		/// <returns>True if successful</returns>
		protected virtual string? FindBestInstalledSDKVersion()
		{
			string[] installedSDKVersions = GetAllInstalledSDKVersions();
			if (installedSDKVersions.Length == 0)
			{
				return null;
			}

			// see if the main version is available
			string mainVersion = GetMainVersion();
			if (installedSDKVersions.Contains(mainVersion))
			{
				return mainVersion;
			}

			// main version is not available - find highest valid version
			GetValidVersionRange(out string? minVersion, out string? maxVersion);
			if (TryConvertVersionToInt(minVersion, out ulong minVersionInt, null) && TryConvertVersionToInt(maxVersion, out ulong maxVersionInt, null))
			{
				List<Tuple<ulong, ulong>> bannedSdkVersionRanges = [];
				foreach (string sdkVersionPair in GetStringArrayFromConfig(BannedSdkVersionsKey))
				{
					string[] versions = sdkVersionPair.Split("-");
					if (versions.Length > 0 && versions.Length <= 2)
					{
						if (TryConvertVersionToInt(versions.First(), out ulong minInvalidVersion, null) && TryConvertVersionToInt(versions.Last(), out ulong maxInvalidVersion, null))
						{
							bannedSdkVersionRanges.Add(new(minInvalidVersion, maxInvalidVersion));
						}
					}
				}

				IGrouping<ulong, string>? resultGroup = installedSDKVersions
					.GroupBy(x => TryConvertVersionToInt(x, out ulong value, null) ? value : 0)            // group by integer version
					.Where(x => x.Key >= minVersionInt && x.Key <= maxVersionInt)                           // select only valid versions
					.OrderByDescending(x => x.Key)                                                         // sort highest version first
					.OrderBy(x => bannedSdkVersionRanges.Any(r => x.Key >= r.Item1 && x.Key <= r.Item2))     // put banned versions at the end of the list - they'll be excluded later
					.FirstOrDefault();
				if (resultGroup?.FirstOrDefault() != null && bannedSdkVersionRanges.Any(r => resultGroup.Key >= r.Item1 && resultGroup.Key <= r.Item2))
				{
					Logger.LogInformation("The best installed SDK for {Platform} is {Version} but this version is forbidden", PlatformName, resultGroup.FirstOrDefault());
					return null;
				}

				return resultGroup?.FirstOrDefault();
			}

			return null;
		}

		/// <summary>
		/// Switch to another version of the SDK than what GetInstalledSDKVersion() returns. This will be one of the versions returned from GetAllInstalledSDKVersions()
		/// </summary>
		/// <param name="version">String name of the version to switch to</param>
		/// <param name="bSwitchForThisProcessOnly">If true, only switch for this process (usually via process environment variable). If false, switch permanently (usually via system-wide environment variable)</param>
		/// <returns>True if successful</returns>
		public virtual bool SwitchToAlternateSDK(string version, bool bSwitchForThisProcessOnly)
		{
			return false;
		}

		/// <summary>
		/// Whether the automatic switching to a locally-installed manual SDK requires AutoSDK to be configured
		/// This is typically true unless the editor-side binaries do not depend on a version-specific SDK environment variable or path
		/// i.e. the SDK version is compiled into the binary and can be used directly.
		/// </summary>
		public virtual bool ManualSDKAutoSwitchRequiresAutoSDK => true;

		/// <summary>
		/// Allows a platform to switch a different version of a manually-installed SDK, if the platform supports side-by-side installations of different versions
		/// </summary>
		/// <returns>True if successful</returns>
		protected virtual bool TryGetEnvironmentForManualSDK(string selectedSDKVersion, out Dictionary<string, string>? envVarValues, out List<string>? pathAdds, out List<string>? pathRemoves)
		{
			envVarValues = null;
			pathAdds = null;
			pathRemoves = null;
			return false;
		}

		/// <summary>
		/// For a platform that doesn't use properly named AutoSDK directories, the directory name may not be convertible to an integer,
		/// and IsVersionValid checks could fail when checking AutoSDK version for an exact match. GetMainVersion() would return the 
		/// proper, integer-convertible version number of the SDK inside of the directory returned by GetAutoSDKDirectoryForMainVersion()
		/// </summary>
		/// <returns></returns>
		public virtual string GetAutoSDKDirectoryForMainVersion()
		{
			// if there is an AutoSDKDirectory property, use it, otherwise, use the MainVersion string
			return GetVersionFromConfig("AutoSDKDirectory") ?? GetMainVersion();
		}

		///// <summary>
		///// Gets the valid (integer) range of Sdk versions. Must be an integer to easily check a range vs a particular version
		///// </summary>
		///// <param name="MinVersion">Smallest version allowed</param>
		///// <param name="MaxVersion">Largest version allowed (inclusive)</param>
		///// <returns>True if the versions are valid, false if the platform is unable to convert its versions into an integer</returns>
		//public virtual void GetValidVersionRange(out UInt64 MinVersion, out UInt64 MaxVersion)
		//{
		//	string MinVersionString, MaxVersionString;
		//	GetValidVersionRange(out MinVersionString, out MaxVersionString);

		//	// failures to convert here are bad
		//	if (!TryConvertVersionToInt(MinVersionString, out MinVersion) || !TryConvertVersionToInt(MaxVersionString, out MaxVersion))
		//	{
		//		throw new Exception(string.Format("Unable to convert Min and Max valid versions to integers in {0} (Versions are {1} - {2})", GetType().Name, MinVersionString, MaxVersionString));
		//	}
		//}
		//public virtual void GetValidSoftwareVersionRange(out UInt64 MinVersion, out UInt64 MaxVersion)
		//{
		//	string? MinVersionString, MaxVersionString;
		//	GetValidSoftwareVersionRange(out MinVersionString, out MaxVersionString);

		//	MinVersion = UInt64.MinValue;
		//	MaxVersion = UInt64.MaxValue - 1; // MaxValue is always bad

		//	// failures to convert here are bad
		//	if ((MinVersionString != null && !TryConvertVersionToInt(MinVersionString, out MinVersion)) ||
		//		(MaxVersionString != null && !TryConvertVersionToInt(MaxVersionString, out MaxVersion)))
		//	{
		//		throw new Exception(string.Format("Unable to convert Min and Max valid Software versions to integers in {0} (Versions are {1} - {2})", GetType().Name, MinVersionString, MaxVersionString));
		//	}
		//}

		// Let platform override behavior to determine if a version is a valid (useful for non-numeric versions)
		protected virtual bool IsVersionValidInternal(string? version, string? sdkType, SDKDescriptor? versionInfo)
		{
			// we could have null if no SDK is installed at all, etc, which is always a failure
			if (version == null)
			{
				return false;
			}

			// AutoSDK must match the desired version exactly, since that is the only one we will use
			if (String.Equals(sdkType, "AutoSdk", StringComparison.OrdinalIgnoreCase))
			{
				// if integer version checking failed, then we can detect valid autosdk if the version matches the autosdk directory by name
				return String.Equals(version, GetAutoSDKDirectoryForMainVersion(), StringComparison.OrdinalIgnoreCase);
			}

			// look for a range for this hinted type of SDK
			versionInfo = versionInfo ?? GetSDKVersionForHint(GetValidVersions(), sdkType);

			// if we couldn't find the Info, we can't do anything, so fail
			if (versionInfo == null)
			{
				return false;
			}

			// convert it to an integer
			if (!TryConvertVersionToInt(version, out ulong intVersion, versionInfo.Name))
			{
				return false;
			}

			// short circuit range check if the Version is the desired version already
			if (intVersion == ConvertVersionToInt(GetMainVersion()))
			{
				return true;
			}

			// finally do the numeric comparison
			return intVersion >= versionInfo.MinInt && intVersion <= versionInfo.MaxInt;
		}

		protected virtual bool IsSoftwareVersionValidInternal(string? version, string? deviceType, SDKDescriptor? versionInfo)
		{
			// we could have null if no SDK is installed at all, etc, which is always a failure
			if (version == null)
			{
				return false;
			}

			// convert it to an integer
			if (!TryConvertVersionToInt(version, out ulong intVersion))
			{
				return false;
			}

			// look for a range for this hinted type of SDK
			versionInfo = versionInfo ?? GetSDKVersionForHint(GetValidSoftwareVersions(), deviceType);

			// if we couldn't find the Info, we can't do anything, so fail
			if (versionInfo == null)
			{
				return false;
			}

			// finally do the numeric comparison
			return intVersion >= versionInfo.MinInt && intVersion <= versionInfo.MaxInt;
		}

		/// <summary>
		/// Only the platform can convert a version string into an integer that is usable for comparison
		/// </summary>
		/// <param name="stringValue">Version that comes from the installed SDK or a Turnkey manifest or the like</param>
		/// <param name="outValue"></param>
		/// <param name="hint">A platform specific hint that can help guide conversion (usually SDKName or device type)</param>
		/// <returns>If the StringValue was able to be be converted to an integer</returns>
		public abstract bool TryConvertVersionToInt(string? stringValue, out ulong outValue, string? hint = null);

		/// <summary>
		/// Like TryConvertVersionToInt, but will throw an exception on failure
		/// </summary>
		/// <param name="stringValue">Version that comes from the PlatformSDK class (should not be used with Manifest or other user supplied versions)</param>
		/// <param name="hint">A platform specific hint that can help guide conversion (usually SDKName or device type)</param>
		/// <returns>The integer version of StringValue, can be used to compare against a valid range</returns>
		public ulong ConvertVersionToInt(string? stringValue, string? hint = null)
		{
			// quickly handle null input, which is not necessarily an error case
			if (stringValue == null)
			{
				return 0;
			}

			ulong Result;
			if (!TryConvertVersionToInt(stringValue, out Result, hint))
			{
				throw new Exception($"Unable to convert {GetType()} version '{stringValue}' to an integer. Likely this version was supplied by code, and is expected to be valid.");
			}
			return Result;
		}

		/// <summary>
		/// Compare two sdk versions, for Sort() purposes
		/// e.g. SdkVersions.Sort( (A,B) => PlatformSDK.SdkVersionsCompare(A,B) );
		/// </summary>
		/// <param name="stringValueA">First Version to compare</param>
		/// <param name="stringValueB">Second Version to compare</param>
		/// <param name="hint">A platform specific hint that can help guide conversion (usually SDKName or device type)</param>
		/// <returns>Comparison integer</returns>
		public int SdkVersionsCompare(string? stringValueA, string? stringValueB, string? hint = null)
		{
			TryConvertVersionToInt(stringValueA, out ulong valueA, hint);
			TryConvertVersionToInt(stringValueB, out ulong valueB, hint);
			if (valueA == valueB)
			{
				return 0;
			}
			else if (valueA < valueB)
			{
				return -1;
			}
			else
			{
				return 1;
			}
		}

		/// <summary>
		/// Allow the platform SDK to override the name it will use in AutoSDK, but default to the platform name
		/// </summary>
		/// <returns>The name of the directory to use inside the AutoSDK system</returns>
		public virtual string GetAutoSDKPlatformName()
		{
			return GetVersionFromConfig("AutoSDKPlatform") ?? PlatformName!;
		}

		#endregion

		#region Per-project SDK support

		private static List<FileReference> ProjectsToCheckForSDKOverrides = [];
		public static void InitializePerProjectSDKVersions(IEnumerable<FileReference> projectsToCheckForOverrides)
		{
			ProjectsToCheckForSDKOverrides = [.. projectsToCheckForOverrides];
		}

		#endregion

		#region Print SDK Info
		private static bool bHasShownTurnkey = false;
		private SDKStatus? SDKInfoValidity = null;
		public static bool bSuppressSDKWarnings = false;

		public virtual SDKStatus PrintSDKInfoAndReturnValidity(LogEventType verbosity = LogEventType.Console, LogFormatOptions options = LogFormatOptions.None,
			LogEventType errorVerbosity = LogEventType.Error, LogFormatOptions errorOptions = LogFormatOptions.None, bool bBriefInvalidSDKWarnings = false)
		{
			if (SDKInfoValidity != null)
			{
				return SDKInfoValidity.Value;
			}

			SDKCollection sdkInfo = GetAllSDKInfo();

			// will mark invalid below if needed
			SDKInfoValidity = SDKStatus.Valid;

			SDKDescriptor? autoSDKInfo = sdkInfo.AutoSdk;
			if (autoSDKInfo != null && autoSDKInfo.Validity == SDKStatus.Valid)
			{
				string platformSDKRoot = GetPathToPlatformAutoSDKs();
				Log.WriteLine(verbosity, options, "{0} using Auto SDK {1} from: {2} 0x{3:X}", PlatformName, autoSDKInfo.Current, Path.Combine(platformSDKRoot, GetAutoSDKDirectoryForMainVersion()), autoSDKInfo.CurrentInt);
			}
			else
			{
				if (sdkInfo.AreAllManualSDKsValid())
				{
					Log.WriteLine(verbosity, options, "{0} Installed SDK(s): {1}", PlatformName, sdkInfo.ToString(true));
				}
				else
				{
					SDKInfoValidity = SDKStatus.Invalid;

					StringBuilder msg = new();
					msg.AppendLine($"Unable to find valid SDK(s) for {PlatformName}:");

					foreach (SDKDescriptor desc in sdkInfo.Sdks)
					{
						if (desc.Validity == SDKStatus.Valid)
						{
							msg.AppendLine($"  {desc.Name} is valid ({desc}");
						}
						else
						{
							msg.Append($"  Found {desc.Name} Version");

							if (desc.Current != null)
							{
								msg.Append($"={desc.Current}");
							}

							msg.AppendLine($", {desc.ToString("Required", null, false)}.");
						}
					}

					if (!bBriefInvalidSDKWarnings && !bHasShownTurnkey)
					{
						msg.AppendLine("  If your Studio has it set up, you can run this command to find the SDK to install:");
						msg.AppendLine("    RunUAT Turnkey -command=InstallSdk -platform={0} -BestAvailable", PlatformName!);

						if ((errorOptions & LogFormatOptions.NoConsoleOutput) == LogFormatOptions.None)
						{
							bHasShownTurnkey = true;
						}
					}

					// Reducing warnings to log to help prevent warnings locally or in Horde about SDKs we might not currently be concerned about
					if (bSuppressSDKWarnings)
					{
						errorVerbosity = LogEventType.Log;
					}

					// always print errors to the screen
					Log.WriteLine(errorVerbosity, errorOptions, msg.ToString());
				}
			}

			return SDKInfoValidity.Value;
		}

		#endregion

		#region Private/Protected general functionality

		// this is the SDK version that was set before activating AutoSDK, since AutoSDK may remove ability to retrieve the Manual SDK version
		protected Dictionary<string, string> CachedManualSDKVersions = [];
		private static Dictionary<string, UEBuildPlatformSDK> SDKRegistry = [];

		// this map holds on to some temporary SDK objects that are generally used once and don't want to stick around, but they could be used multiple times, 
		// like in MicrosoftPlatofrmSDK.Version.cs, if one of the versions is needed, C# will go construct every version, needing the SDK multiple times
		private static Dictionary<string, UEBuildPlatformSDK> TempSDKRegistry = [];

		private static SDKDescriptor? GetSDKVersionForHint(SDKCollection collection, string? hint)
		{
			// if the hint is found, use it always
			SDKDescriptor? sdkDesc = collection.Sdks.FirstOrDefault(x => String.Equals(x.Name, hint, StringComparison.OrdinalIgnoreCase));
			if (sdkDesc != null)
			{
				return sdkDesc;
			}

			// only use AUtoSDK if explicitly asked for above, otherwise, remove it and look at what's left
			IEnumerable<SDKDescriptor> fullSdks = collection.FullSdks;

			// if there's one with the special name, then use it as it's generic (common case here)
			if (fullSdks.Count() == 1 && (fullSdks.First().Name == "Sdk" || fullSdks.First().Name == "Software"))
			{
				return fullSdks.First();
			}

			// finally we have to give up
			return null;
		}

		// cached SDK info from the SDK.json file
		private Dictionary<string, string> ConfigSDKVersions = new(StringComparer.OrdinalIgnoreCase);
		private Dictionary<string, string[]> ConfigSDKVersionArrays = new(StringComparer.OrdinalIgnoreCase);

		private void LoadJsonFile(string platform)
		{
			// fixup for Windows
			if (platform == "Win64")
			{
				platform = "Windows";
			}

			FileReference? MakeConfigFilename(DirectoryReference rootDir, bool bIsRequired)
			{
				FileReference platformExtensionLocation = FileReference.Combine(rootDir, "Platforms", platform, "Config", $"{platform}_SDK.json");
				if (FileReference.Exists(platformExtensionLocation))
				{
					return platformExtensionLocation;
				}
				FileReference standardLocation = FileReference.Combine(rootDir, "Config", platform, $"{platform}_SDK.json");
				if (FileReference.Exists(standardLocation))
				{
					return standardLocation;
				}
				if (bIsRequired)
				{
					throw new Exception($"Failed to find required SDK.json for {platform}. Looked in '{standardLocation}' and '{platformExtensionLocation}'.");
				}
				return null;
			}

			// if the SDK isn't allowed on the host, then allow it to not exist
			FileReference engineSDKConfigFile = MakeConfigFilename(Unreal.EngineDirectory, bIsSdkAllowedOnHost)!;

			// load the file, along with any chained group file
			ProcessJsonFile(engineSDKConfigFile, ConfigSDKVersions, ConfigSDKVersionArrays);

			// copy off the versions to the defaults, so we can check if overridden by the project
			Dictionary<string, string> defaultConfigSDKVersions = new(ConfigSDKVersions);
			foreach (string Key in ConfigSDKVersions.Keys)
			{
				defaultConfigSDKVersions[Key] = ConfigSDKVersions[Key];
			}

			// now read overrides into the array
			foreach (FileReference projectFile in ProjectsToCheckForSDKOverrides)
			{
				FileReference? projectSDKConfigFile = MakeConfigFilename(projectFile.Directory, false);
				if (projectSDKConfigFile != null)
				{
					// load a project's SDK file if thre is one, into a temp dictionary
					Dictionary<string, string> overrideConfigSDKVersions = new(StringComparer.OrdinalIgnoreCase);
					Dictionary<string, string[]> overrideConfigSDKVersionArrays = new(StringComparer.OrdinalIgnoreCase);
					ProcessJsonFile(projectSDKConfigFile, overrideConfigSDKVersions, overrideConfigSDKVersionArrays);
					if (overrideConfigSDKVersionArrays.Count > 0)
					{
						throw new Exception($"Overriding version arrays, in project '{projectFile.GetFileNameWithoutExtension()}', platform {platform} is not currently supported");
					}

					// currently only care about MainVersion in the overrides
					if (overrideConfigSDKVersions.TryGetValue(MainVersionKey, out string? overrideMainVersion))
					{
						// if different from default, then remmber it, and mark that it's been overridden
						if (!overrideMainVersion.Equals(defaultConfigSDKVersions[MainVersionKey], StringComparison.OrdinalIgnoreCase))
						{
							// now check if it was already overridden, in which case we have a conflict we can't resolve, so error
							if (ProjectsThatOverrodeSDK.Count > 0 && !overrideMainVersion.Equals(ConfigSDKVersions[MainVersionKey], StringComparison.OrdinalIgnoreCase))
							{
								throw new Exception($"Project {projectFile.GetFileNameWithoutAnyExtensions()} wants to override {platform} SDK to {overrideMainVersion}, but it was already overridden to version {ConfigSDKVersions[MainVersionKey]}");
							}

							Logger.LogWarning("Project {Project} is overriding {Platform} SDK to {OverrideMainVersion}", projectFile.GetFileNameWithoutAnyExtensions(), platform, overrideMainVersion);

							ConfigSDKVersions[MainVersionKey] = overrideMainVersion;
							bHasAnySDKOverride = true;
							bHasSDKOverride = true;
							ProjectsThatOverrodeSDK.Add(projectFile);
						}
					}
				}
			}
		}

		private void ProcessJsonFile(FileReference sdkConfigFile, Dictionary<string, string> versionMap, Dictionary<string, string[]> versionArrayMap)
		{
			// load the json
			if (!JsonObject.TryRead(sdkConfigFile, out JsonObject? jsonObject))
			{
				throw new Exception($"Failed to parse SDK version file '{sdkConfigFile}'");
			}

			LoadedConfigFiles.Add(sdkConfigFile);

			// look for special key to chain to other - after processing everything else (we only add from the "parent" settings not already in the map)
			List<string> parents = [];
			foreach (string key in jsonObject.KeyNames)
			{
				if (jsonObject.TryGetStringField(key, out string? stringValue))
				{
					if (key.Equals("ParentSDKFile", StringComparison.OrdinalIgnoreCase) ||
						key.Equals("IncludeSDKFile", StringComparison.OrdinalIgnoreCase))
					{
						stringValue = stringValue.Replace("$(EngineDir)", Unreal.EngineDirectory.ToString(), StringComparison.OrdinalIgnoreCase);
						parents.Add(stringValue);
					}
					// add this key if it's not already in the (case-insensitive) dictionary 
					versionMap.TryAdd(key, stringValue);
				}
				else if (jsonObject.TryGetStringArrayField(key, out string[]? ArrayValue))
				{
					versionArrayMap.Add(key, ArrayValue);
				}
			}

			// now load parents, filling in unset properties
			foreach (string parent in parents)
			{
				FileReference parentConfigFile = FileReference.Combine(sdkConfigFile.Directory, parent);
				ProcessJsonFile(parentConfigFile, versionMap, versionArrayMap);
			}
		}

		private static string GetHostSpecificVersionName(string versionName)
		{
			string hostPlatform = "Win64";
			if (OperatingSystem.IsMacOS())
			{
				hostPlatform = "Mac";
			}
			else if (OperatingSystem.IsLinux())
			{
				hostPlatform = "Linux";
			}
			return $"{versionName}_{hostPlatform}";
		}
		public string GetRequiredVersionFromConfig(string versionName)
		{
			// when bIsRequired is true, then we know it will return non-null
			return GetVersionFromConfig(versionName, bIsRequired: true)!;
		}

		public string? GetVersionFromConfig(string versionName, bool bIsRequired = false)
		{
			// look up both Version_Host and Version (Host specific version wins)
			string? version;
			if (ConfigSDKVersions!.TryGetValue(GetHostSpecificVersionName(versionName), out version) || ConfigSDKVersions.TryGetValue(versionName, out version))
			{
				return version;
			}

			if (bIsRequired)
			{
				throw new Exception($"Unable to find required SDK version '{versionName}' for platform {PlatformName}. Check your SDK.json files");
			}
			return null;
		}

		protected VersionNumber GetRequiredVersionNumberFromConfig(string versionName)
		{
			// required won't ever return null
			return GetVersionNumberFromConfig(versionName, true)!;
		}

		public VersionNumber? GetVersionNumberFromConfig(string versionName, bool bIsRequired = false)
		{
			string? versionString = GetVersionFromConfig(versionName, bIsRequired);

			if (versionString == null)
			{
				return null;
			}

			return VersionNumber.Parse(versionString);
		}

		private static VersionNumberRange? ParseVersionNumberRange(string range)
		{
			string[] versions = range.Split("-");
			if (versions.Length != 2)
			{
				return null;
			}

			return VersionNumberRange.Parse(versions[0], versions[1]);
		}

		public VersionNumberRange? GetRequiredVersionNumberRangeFromConfig(string versionName, bool bIsRequired = false)
		{
			// required won't ever return null
			return GetVersionNumberRangeFromConfig(versionName, true)!;
		}

		public VersionNumberRange? GetVersionNumberRangeFromConfig(string versionName, bool bIsRequired = false)
		{
			string? versionRange;
			if (!ConfigSDKVersions!.TryGetValue(GetHostSpecificVersionName(versionName), out versionRange) && !ConfigSDKVersions.TryGetValue(versionName, out versionRange))
			{
				if (bIsRequired)
				{
					throw new Exception($"Unable to find required SDK version range '{versionName}' for platform {PlatformName}. Check your SDK.json files");
				}
				return null;
			}

			VersionNumberRange? range = ParseVersionNumberRange(versionRange);
			if (range == null && bIsRequired)
			{
				throw new Exception($"Unable to parse the version number range for required version '{versionName}' for platform {PlatformName}. Check your SDK.json files");
			}
			return range;
		}

		public VersionNumberRange[] GetVersionNumberRangeArrayFromConfig(string versionName)
		{
			string[]? versionRanges;
			List<VersionNumberRange> ranges = [];
			if (ConfigSDKVersionArrays.TryGetValue(GetHostSpecificVersionName(versionName), out versionRanges) || ConfigSDKVersionArrays.TryGetValue(versionName, out versionRanges))
			{
				foreach (string versionRange in versionRanges)
				{
					VersionNumberRange? range = ParseVersionNumberRange(versionRange);
					if (range != null)
					{
						ranges.Add(range);
					}
				}
			}

			return [.. ranges];
		}

		public string[] GetStringArrayFromConfig(string name)
		{
			string[]? results;
			if (ConfigSDKVersionArrays.TryGetValue(GetHostSpecificVersionName(name), out results) || ConfigSDKVersionArrays.TryGetValue(name, out results))
			{
				return results;
			}

			return [];
		}

		#endregion

		// AutoSDKs handling portion

		#region protected AutoSDKs Utility

		/// <summary>
		/// Name of the file that holds currently install SDK version string
		/// </summary>
		protected const string CurrentlyInstalledSDKStringManifest = "CurrentlyInstalled.txt";

		/// <summary>
		/// name of the file that holds the last succesfully run SDK setup script version
		/// </summary>
		protected const string LastRunScriptVersionManifest = "CurrentlyInstalled.Version.txt";

		/// <summary>
		/// Filename of the script version file in each AutoSDK directory. Changing the contents of this file will force reinstallation of an AutoSDK.
		/// </summary>
		private const string ScriptVersionFilename = "Version.txt";

		/// <summary>
		/// Name of the file that holds environment variables of current SDK
		/// </summary>
		protected const string SDKEnvironmentVarsFile = "OutputEnvVars.txt";

		protected const string ManualSDKEnvironmentVarsFile = "ManualSDKEnvVars.txt";

		protected const string SDKRootEnvVar = "UE_SDKS_ROOT";

		protected const string AutoSetupEnvVar = "AutoSDKSetup";
		protected const string ManualSetupEnvVar = "ManualSDKSetup";

		private static string GetAutoSDKHostPlatform()
		{
			if (OperatingSystem.IsWindows())
			{
				return "Win64";
			}
			else if (OperatingSystem.IsMacOS())
			{
				return "Mac";
			}
			else if (OperatingSystem.IsLinux())
			{
				return "Linux";
			}
			throw new Exception("Unknown host platform!");
		}

		/// <summary>
		/// Whether platform supports switching SDKs during runtime
		/// </summary>
		/// <returns>true if supports</returns>
		protected virtual bool PlatformSupportsAutoSDKs()
		{
			return false;
		}

		private static bool bCheckedAutoSDKRootEnvVar = false;
		private static bool bAutoSDKSystemEnabled = false;
		private static bool HasAutoSDKSystemEnabled()
		{
			if (!bCheckedAutoSDKRootEnvVar)
			{
				string? sdkRoot = Environment.GetEnvironmentVariable(SDKRootEnvVar);
				if (sdkRoot != null)
				{
					bAutoSDKSystemEnabled = true;
				}
				bCheckedAutoSDKRootEnvVar = true;
			}
			return bAutoSDKSystemEnabled;
		}

		// Whether AutoSDK setup is safe. AutoSDKs will damage manual installs on some platforms.
		protected bool IsAutoSDKSafe()
		{
			return !IsAutoSDKDestructive() || !HasAnyManualInstall();
		}

		/// <summary>
		/// Gets the version number of the SDK setup script itself.  The version in the base should ALWAYS be the primary revision from the last refactor.
		/// If you need to force a rebuild for a given platform, modify the version file.
		/// </summary>
		/// <returns>Setup script version</returns>
		private string GetRequiredScriptVersionString()
		{
			const string UnspecifiedVersion = "UnspecifiedScriptVersion";

			string versionFilename = Path.Combine(GetPathToPlatformAutoSDKs(), GetAutoSDKDirectoryForMainVersion(), ScriptVersionFilename);
			return File.Exists(versionFilename)
				? File.ReadAllLines(versionFilename).FirstOrDefault() ?? UnspecifiedVersion
				: UnspecifiedVersion;
		}

		/// <summary>
		/// Returns path to platform SDKs
		/// </summary>
		/// <returns>Valid SDK string</returns>
		protected string GetPathToPlatformAutoSDKs()
		{
			string sdkPath = "";
			string? sdkRoot = Environment.GetEnvironmentVariable(SDKRootEnvVar);
			if (sdkRoot != null)
			{
				if (!String.IsNullOrEmpty(sdkRoot))
				{
					sdkPath = Path.Combine(sdkRoot, "Host" + GetAutoSDKHostPlatform(), GetAutoSDKPlatformName());
				}
			}
			return sdkPath;
		}

		/// <summary>
		/// Returns path to platform SDKs
		/// </summary>
		/// <returns>Valid SDK string</returns>
		public static bool TryGetHostPlatformAutoSDKDir([NotNullWhen(true)] out DirectoryReference? outPlatformDir)
		{
			string? sdkRoot = Environment.GetEnvironmentVariable(SDKRootEnvVar);
			if (String.IsNullOrEmpty(sdkRoot))
			{
				outPlatformDir = null;
				return false;
			}
			else
			{
				outPlatformDir = DirectoryReference.Combine(new DirectoryReference(sdkRoot), "Host" + GetAutoSDKHostPlatform());
				return true;
			}
		}

		/// <summary>
		/// Because most ManualSDK determination depends on reading env vars, if this process is spawned by a process that ALREADY set up
		/// AutoSDKs then all the SDK env vars will exist, and we will spuriously detect a Manual SDK. (children inherit the environment of the parent process).
		/// Therefore we write out an env variable to set in the command file (OutputEnvVars.txt) such that child processes can determine if their manual SDK detection
		/// is bogus.  Make it platform specific so that platforms can be in different states.
		/// </summary>
		protected string GetPlatformAutoSDKSetupEnvVar()
		{
			return GetAutoSDKPlatformName() + AutoSetupEnvVar;
		}
		protected string GetPlatformManualSDKSetupEnvVar()
		{
			return GetAutoSDKPlatformName() + ManualSetupEnvVar;
		}

		/// <summary>
		/// Gets currently installed version
		/// </summary>
		/// <param name="platformSDKRoot">absolute path to platform SDK root</param>
		/// <param name="outInstalledSDKVersionString">version string as currently installed</param>
		/// <param name="outInstalledSDKLevel"></param>
		/// <returns>true if was able to read it</returns>
		protected static bool GetCurrentlyInstalledSDKString(string platformSDKRoot, out string outInstalledSDKVersionString, out string outInstalledSDKLevel)
		{
			if (Directory.Exists(platformSDKRoot))
			{
				string versionFilename = Path.Combine(platformSDKRoot, CurrentlyInstalledSDKStringManifest);
				if (File.Exists(versionFilename))
				{
					using (StreamReader reader = new(versionFilename))
					{
						string? version = reader.ReadLine();
						string? type = reader.ReadLine();
						string? level = reader.ReadLine();
						if (String.IsNullOrEmpty(level))
						{
							level = "FULL";
						}

						// don't allow ManualSDK installs to count as an AutoSDK install version.
						if (type != null && type == "AutoSDK")
						{
							if (version != null)
							{
								outInstalledSDKVersionString = version;
								outInstalledSDKLevel = level;
								return true;
							}
						}
					}
				}
			}

			outInstalledSDKVersionString = "";
			outInstalledSDKLevel = "";
			return false;
		}

		/// <summary>
		/// Gets the version of the last successfully run setup script.
		/// </summary>
		/// <param name="platformSDKRoot">absolute path to platform SDK root</param>
		/// <param name="outLastRunScriptVersion">version string</param>
		/// <returns>true if was able to read it</returns>
		protected static bool GetLastRunScriptVersionString(string platformSDKRoot, out string outLastRunScriptVersion)
		{
			if (Directory.Exists(platformSDKRoot))
			{
				string versionFilename = Path.Combine(platformSDKRoot, LastRunScriptVersionManifest);
				if (File.Exists(versionFilename))
				{
					using (StreamReader reader = new(versionFilename))
					{
						string? version = reader.ReadLine();
						if (version != null)
						{
							outLastRunScriptVersion = version;
							return true;
						}
					}
				}
			}

			outLastRunScriptVersion = "";
			return false;
		}

		/// <summary>
		/// Sets currently installed version
		/// </summary>
		/// <param name="installedSDKVersionString">SDK version string to set</param>
		/// <param name="installedSDKLevelString"></param>
		/// <returns>true if was able to set it</returns>
		protected bool SetCurrentlyInstalledAutoSDKString(string installedSDKVersionString, string installedSDKLevelString)
		{
			string platformSDKRoot = GetPathToPlatformAutoSDKs();
			if (Directory.Exists(platformSDKRoot))
			{
				string versionFilename = Path.Combine(platformSDKRoot, CurrentlyInstalledSDKStringManifest);
				if (File.Exists(versionFilename))
				{
					File.Delete(versionFilename);
				}

				using (StreamWriter writer = File.CreateText(versionFilename))
				{
					writer.WriteLine(installedSDKVersionString);
					writer.WriteLine("AutoSDK");
					writer.WriteLine(installedSDKLevelString);
					return true;
				}
			}

			return false;
		}

		protected void SetupManualSDK()
		{
			if (PlatformSupportsAutoSDKs() && HasAutoSDKSystemEnabled())
			{
				string installedSDKVersionString = GetAutoSDKDirectoryForMainVersion();
				string platformSDKRoot = GetPathToPlatformAutoSDKs();
				if (!Directory.Exists(platformSDKRoot))
				{
					Directory.CreateDirectory(platformSDKRoot);
				}

				{
					string versionFilename = Path.Combine(platformSDKRoot, CurrentlyInstalledSDKStringManifest);
					if (File.Exists(versionFilename))
					{
						File.Delete(versionFilename);
					}

					string envVarFile = Path.Combine(platformSDKRoot, SDKEnvironmentVarsFile);
					if (File.Exists(envVarFile))
					{
						File.Delete(envVarFile);
					}

					using (StreamWriter writer = File.CreateText(versionFilename))
					{
						writer.WriteLine(installedSDKVersionString);
						writer.WriteLine("ManualSDK");
						writer.WriteLine("FULL");
					}
				}
			}
		}

		public static void ClearManualSDKEnvVarCache()
		{
			string manualSDKEnvironmentVarsPath = Path.Combine(Unreal.EngineDirectory.ToString(), "Intermediate", ManualSDKEnvironmentVarsFile);
			if (File.Exists(manualSDKEnvironmentVarsPath))
			{
				File.Delete(manualSDKEnvironmentVarsPath);
			}
		}

		private bool TrySelectBestManualSDK(out string? selectedSDKVersion)
		{
			// find the best valid manual sdk
			selectedSDKVersion = FindBestInstalledSDKVersion();
			if (selectedSDKVersion == null)
			{
				return false;
			}

			// query the platform for environment changes
			if (!TryGetEnvironmentForManualSDK(selectedSDKVersion, out Dictionary<string, string>? envVarValues, out List<string>? pathAdds, out List<string>? pathRemoves))
			{
				return false;
			}

			// apply environment variables
			if (envVarValues != null)
			{
				foreach (KeyValuePair<string, string> envVarValue in envVarValues)
				{
					Environment.SetEnvironmentVariable(envVarValue.Key, envVarValue.Value);
				}
			}

			// apply PATH modifications
			if (pathRemoves != null || pathAdds != null)
			{
				string origPathVar = Environment.GetEnvironmentVariable("PATH") ?? String.Empty;
				IEnumerable<string> pathVars = origPathVar.Split(Path.PathSeparator);
				if (pathRemoves != null)
				{
					pathVars = pathVars.Except(pathRemoves, FileUtils.PlatformPathComparer);
				}
				if (pathAdds != null)
				{
					pathVars = pathVars.Except(pathAdds, FileUtils.PlatformPathComparer); // remove all of the ADDs so that if this function is executed multiple times, the paths will be guaranteed to be in the same order after each run.
					pathVars = pathVars.Union(pathAdds, FileUtils.PlatformPathComparer);
				}

				string newPathVar = String.Join(Path.PathSeparator, pathVars);
				Environment.SetEnvironmentVariable("PATH", newPathVar);
			}

			// write all environment modifications
			if ((envVarValues != null && envVarValues.Count > 0) || (pathRemoves != null && pathRemoves.Count > 0) || (pathAdds != null && pathAdds.Count > 0))
			{
				string manualSDKEnvironmentVarsPath = Path.Combine(Unreal.EngineDirectory.ToString(), "Intermediate", ManualSDKEnvironmentVarsFile);

				Directory.CreateDirectory(Path.GetDirectoryName(manualSDKEnvironmentVarsPath)!);
				using (StreamWriter writer = File.AppendText(manualSDKEnvironmentVarsPath))
				{
					// write environment variables
					if (envVarValues != null)
					{
						foreach (KeyValuePair<string, string> envVarValue in envVarValues)
						{
							writer.WriteLine($"{envVarValue.Key}={envVarValue.Value}");
							Logger.LogDebug("Setting variable '{Name}' to '{Value}'", envVarValue.Key, envVarValue.Value);
						}
					}

					// write PATH modifications
					if (pathRemoves != null)
					{
						foreach (string pathVar in pathRemoves)
						{
							writer.WriteLine($"strippath={pathVar}");
							Logger.LogDebug("Removing Path: '{Path}'", pathVar);
						}
					}
					if (pathAdds != null)
					{
						foreach (string pathVar in pathAdds)
						{
							writer.WriteLine($"addpath={pathVar}");
							Logger.LogDebug("Adding Path: '{Path}'", pathVar);
						}
					}
				}
			}

			return true;
		}

		protected bool SetLastRunAutoSDKScriptVersion(string lastRunScriptVersion)
		{
			string platformSDKRoot = GetPathToPlatformAutoSDKs();
			if (Directory.Exists(platformSDKRoot))
			{
				string versionFilename = Path.Combine(platformSDKRoot, LastRunScriptVersionManifest);
				if (File.Exists(versionFilename))
				{
					File.Delete(versionFilename);
				}

				using (StreamWriter writer = File.CreateText(versionFilename))
				{
					writer.WriteLine(lastRunScriptVersion);
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Returns Hook names as needed by the platform
		/// (e.g. can be overridden with custom executables or scripts)
		/// </summary>
		/// <param name="hook">Hook type</param>
		protected virtual string GetHookExecutableName(SDKHookType hook)
		{
			return OperatingSystem.IsWindows()
				? hook == SDKHookType.Uninstall ? "unsetup.bat" : "setup.bat"
				: hook == SDKHookType.Uninstall ? "unsetup.sh" : "setup.sh";
		}

		/// <summary>
		/// Whether the hook must be run with administrator privileges.
		/// </summary>
		/// <param name="hook">Hook for which to check the required privileges.</param>
		/// <returns>true if the hook must be run with administrator privileges.</returns>
		protected virtual bool DoesHookRequireAdmin(SDKHookType hook)
		{
			return true;
		}

		private void LogAutoSDKHook(object sender, DataReceivedEventArgs args)
		{
			if (args.Data != null)
			{
				LogFormatOptions options = Log.OutputLevel >= LogEventType.Verbose ? LogFormatOptions.None : LogFormatOptions.NoConsoleOutput;
				Log.WriteLine(LogEventType.Log, options, args.Data);
			}
		}

		private static string[] AutoSDKLevels =
		[
			"NONE",
			"BUILD",
			"PACKAGE",
			"RUN",
			"FULL",
		];

		private static bool IsSDKLevelAtLeast(string value, string comparedTo)
		{
			int valueIndex = AutoSDKLevels.FindIndex(x => x.Equals(value, StringComparison.OrdinalIgnoreCase));
			int comparedIndex = AutoSDKLevels.FindIndex(x => x.Equals(comparedTo, StringComparison.OrdinalIgnoreCase));
			if (valueIndex < 0 || comparedIndex < 0)
			{
				throw new Exception($"Passed in a bad value to IsSDKLevelAtLeast: {value}, {comparedTo}");
			}
			return valueIndex >= comparedIndex;
		}

		private static string GetAutoSDKLevelForPlatform(string platformSDKRoot)
		{
			// the last component is the platform name
			string platformName = Path.GetFileName(platformSDKRoot).ToUpper();

			// parse the envvar
			Dictionary<string, string> platformSpecificLevels = [];
			string? detailedSettings = Environment.GetEnvironmentVariable("UE_AUTOSDK_SPECIFIC_LEVELS"); // "Android=PACKAGE;GDk=RuN;PS5=BUILD";
			if (!String.IsNullOrEmpty(detailedSettings) && detailedSettings.Contains(platformName, StringComparison.InvariantCultureIgnoreCase))
			{
				foreach (string detail in detailedSettings.ToUpper().Split(';'))
				{
					string[] tokens = detail.Split('=');
					// validate the level string
					if (AutoSDKLevels.Contains(tokens[1]))
					{
						platformSpecificLevels.Add(tokens[0], tokens[1]);
					}
				}
			}

			string finalAutoSDKLevel = "FULL";
			if (platformSpecificLevels.TryGetValue(platformName, out string? value))
			{
				finalAutoSDKLevel = value;
			}
			else
			{
				string? defaultLevel = Environment.GetEnvironmentVariable("UE_AUTOSDK_DEFAULT_LEVEL");
				if (!String.IsNullOrEmpty(defaultLevel) && AutoSDKLevels.Contains(defaultLevel, StringComparer.InvariantCultureIgnoreCase))
				{
					finalAutoSDKLevel = defaultLevel.ToUpper();
				}
			}

			return finalAutoSDKLevel;
		}

		/// <summary>
		/// Runs install/uninstall hooks for SDK
		/// </summary>
		/// <param name="platformSDKRoot">absolute path to platform SDK root</param>
		/// <param name="sdkVersionString">version string to run for (can be empty!)</param>
		/// <param name="autoSDKLevel"></param>
		/// <param name="hook">which one of hooks to run</param>
		/// <param name="bHookCanBeNonExistent">whether a non-existing hook means failure</param>
		/// <returns>true if succeeded</returns>
		protected virtual bool RunAutoSDKHooks(string platformSDKRoot, string sdkVersionString, string autoSDKLevel, SDKHookType hook, bool bHookCanBeNonExistent = true)
		{
			if (!IsAutoSDKSafe())
			{
				Logger.LogDebug("{Platform} attempted to run SDK hook which could have damaged manual SDK install!", GetAutoSDKPlatformName());
				return false;
			}
			if (!String.IsNullOrEmpty(sdkVersionString))
			{
				string sdkDirectory = Path.Combine(platformSDKRoot, sdkVersionString);
				string hookExe = Path.Combine(sdkDirectory, GetHookExecutableName(hook));

				if (File.Exists(hookExe))
				{
					Logger.LogDebug("Running {Hook} hook {HookExe}", hook, hookExe);
					using ITimelineEvent _ = Timeline.ScopeEvent("RunAutoSDKHooks");

					// run it
					using Process hookProcess = new();
					hookProcess.StartInfo.WorkingDirectory = sdkDirectory;
					hookProcess.StartInfo.FileName = hookExe;
					hookProcess.StartInfo.Arguments = autoSDKLevel;
					hookProcess.StartInfo.WindowStyle = ProcessWindowStyle.Hidden;

					bool bHookRequiresAdmin = DoesHookRequireAdmin(hook);
					if (bHookRequiresAdmin)
					{
						// installers may require administrator access to succeed. so run as an admin.
						hookProcess.StartInfo.Verb = "runas";

						//Forcing the old .Net Framework default to prevent processes from failing
						hookProcess.StartInfo.UseShellExecute = true;
					}
					else
					{
						hookProcess.StartInfo.UseShellExecute = false;
						hookProcess.StartInfo.RedirectStandardOutput = true;
						hookProcess.StartInfo.RedirectStandardError = true;
						hookProcess.OutputDataReceived += LogAutoSDKHook;
						hookProcess.ErrorDataReceived += LogAutoSDKHook;
					}

					//using (ScopedTimer HookTimer = new ScopedTimer("Time to run hook: ", LogEventType.Log))
					{
						hookProcess.Start();
						if (!bHookRequiresAdmin)
						{
							hookProcess.BeginOutputReadLine();
							hookProcess.BeginErrorReadLine();
						}
						hookProcess.WaitForExit();
					}

					if (hookProcess.ExitCode != 0)
					{
						Logger.LogDebug("Hook exited uncleanly (returned {ExitCode}), considering it failed.", hookProcess.ExitCode);

						// Don't fail on Wine, because error propagation behaves differently
						// for batch scripts when compared to Windows. This sometimes results
						// in non-zero exit codes after a script completes successfully.
						if (!RuntimePlatform.IsRunningOnWine())
						{
							return false;
						}
					}

					return true;
				}
				else
				{
					Logger.LogDebug("File {HookExe} does not exist", hookExe);
				}
			}
			else
			{
				Logger.LogDebug("Version string is blank for {SdkRoot}. Can't determine {Hook} hook.", platformSDKRoot, hook.ToString());
			}

			return bHookCanBeNonExistent;
		}

		/// <summary>
		/// Loads environment variables from SDK
		/// If any commands are added or removed the handling needs to be duplicated in
		/// TargetPlatformManagerModule.cpp
		/// </summary>
		/// <param name="platformSDKRoot">absolute path to platform SDK</param>
		/// <returns>true if succeeded</returns>
		protected bool SetupEnvironmentFromAutoSDK(string platformSDKRoot)
		{
			string envVarFile = Path.Combine(platformSDKRoot, SDKEnvironmentVarsFile);
			if (File.Exists(envVarFile))
			{
				using (StreamReader reader = new(envVarFile))
				{
					List<string> pathAdds = [];
					List<string> pathRemoves = [];

					List<string> envVarNames = [];
					List<string> envVarValues = [];

					bool bNeedsToWriteAutoSetupEnvVar = true;
					string platformSetupEnvVar = GetPlatformAutoSDKSetupEnvVar();
					for (; ; )
					{
						string? variableString = reader.ReadLine();
						if (variableString == null)
						{
							break;
						}

						string[] parts = variableString.Split('=');
						if (parts.Length != 2)
						{
							Logger.LogDebug("Incorrect environment variable declaration:");
							Logger.LogDebug("{VariableString}", variableString);
							return false;
						}

						if (String.Equals(parts[0], "strippath", StringComparison.OrdinalIgnoreCase))
						{
							pathRemoves.Add(parts[1]);
						}
						else if (String.Equals(parts[0], "addpath", StringComparison.OrdinalIgnoreCase))
						{
							pathAdds.Add(parts[1]);
						}
						else
						{
							if (String.Equals(parts[0], platformSetupEnvVar, StringComparison.Ordinal))
							{
								bNeedsToWriteAutoSetupEnvVar = false;
							}
							// convenience for setup.bat writers.  Trim any accidental whitespace from variable names/values.
							envVarNames.Add(parts[0].Trim());
							envVarValues.Add(parts[1].Trim());
						}
					}

					// don't actually set anything until we successfully validate and read all values in.
					// we don't want to set a few vars, return a failure, and then have a platform try to
					// build against a manually installed SDK with half-set env vars.
					for (int i = 0; i < envVarNames.Count; ++i)
					{
						string envVarName = envVarNames[i];
						string envVarValue = envVarValues[i];
						Logger.LogDebug("Setting variable '{Name}' to '{Value}'", envVarName, envVarValue);
						Environment.SetEnvironmentVariable(envVarName, envVarValue);
					}

					// actually perform the PATH stripping / adding.
					string? origPathVar = Environment.GetEnvironmentVariable("PATH");
					string pathDelimiter = OperatingSystem.IsWindows() ? ";" : ":";
					string[] pathVars = [];
					if (!String.IsNullOrEmpty(origPathVar))
					{
						pathVars = origPathVar.Split(pathDelimiter.ToCharArray());
					}
					else
					{
						Logger.LogDebug("Path environment variable is null during AutoSDK");
					}

					List<string> modifiedPathVars = [.. pathVars];

					// perform removes first, in case they overlap with any adds.
					foreach (string pathRemove in pathRemoves)
					{
						foreach (string pathVar in pathVars)
						{
							if (pathVar.Contains(pathRemove, StringComparison.OrdinalIgnoreCase))
							{
								Logger.LogDebug("Removing Path: '{Path}'", pathVar);
								modifiedPathVars.Remove(pathVar);
							}
						}
					}

					// remove all the of ADDs so that if this function is executed multiple times, the paths will be guaranteed to be in the same order after each run.
					// If we did not do this, a 'remove' that matched some, but not all, of our 'adds' would cause the order to change.
					foreach (string pathAdd in pathAdds)
					{
						foreach (string pathVar in pathVars)
						{
							if (String.Equals(pathAdd, pathVar, StringComparison.OrdinalIgnoreCase))
							{
								Logger.LogDebug("Removing Path: '{Path}'", pathVar);
								modifiedPathVars.Remove(pathVar);
							}
						}
					}

					// perform adds, but don't add duplicates
					foreach (string pathAdd in pathAdds)
					{
						if (!modifiedPathVars.Contains(pathAdd))
						{
							Logger.LogDebug("Adding Path: '{Path}'", pathAdd);
							modifiedPathVars.Add(pathAdd);
						}
					}

					string modifiedPath = String.Join(pathDelimiter, modifiedPathVars);
					Environment.SetEnvironmentVariable("PATH", modifiedPath);

					reader.Close();

					// write out environment variable command so any process using this commandfile will mark itself as having had autosdks set up.
					// avoids child processes spuriously detecting manualsdks.
					if (bNeedsToWriteAutoSetupEnvVar)
					{
						// write out the manual sdk version since child processes won't be able to detect manual with AutoSDK messing up env vars
						using (StreamWriter writer = File.AppendText(envVarFile))
						{
							writer.WriteLine("{0}={1}", platformSetupEnvVar, "1");
						}
						// set the variable in the local environment in case this process spawns any others.
						Environment.SetEnvironmentVariable(platformSetupEnvVar, "1");
					}

					// make sure we know that we've modified the local environment, invalidating manual installs for this run.
					bLocalProcessSetupAutoSDK = true;

					// tell any child processes what our manual versions were before setting up autosdk
					string valueToWrite = CachedManualSDKVersions.Count > 0 ? JsonSerializer.Serialize(CachedManualSDKVersions) : "__None";
					Environment.SetEnvironmentVariable(GetPlatformManualSDKSetupEnvVar(), valueToWrite);

					return true;
				}
			}
			else
			{
				Logger.LogDebug("Cannot set up environment for {SdkRoot} because command file {EnvVarFile} does not exist.", platformSDKRoot, envVarFile);
			}

			return false;
		}

		protected void InvalidateCurrentlyInstalledAutoSDK()
		{
			string platformSDKRoot = GetPathToPlatformAutoSDKs();
			if (Directory.Exists(platformSDKRoot))
			{
				string sdkFilename = Path.Combine(platformSDKRoot, CurrentlyInstalledSDKStringManifest);
				if (File.Exists(sdkFilename))
				{
					File.Delete(sdkFilename);
				}

				string versionFilename = Path.Combine(platformSDKRoot, LastRunScriptVersionManifest);
				if (File.Exists(versionFilename))
				{
					File.Delete(versionFilename);
				}

				string envVarFile = Path.Combine(platformSDKRoot, SDKEnvironmentVarsFile);
				if (File.Exists(envVarFile))
				{
					File.Delete(envVarFile);
				}
			}
		}

		/// <summary>
		/// Currently installed AutoSDK is written out to a text file in a known location.
		/// This function just compares the file's contents with the current requirements.
		/// </summary>
		public SDKStatus HasRequiredAutoSDKInstalled()
		{
			if (PlatformSupportsAutoSDKs() && HasAutoSDKSystemEnabled())
			{
				string autoSDKRoot = GetPathToPlatformAutoSDKs();
				if (!String.IsNullOrEmpty(autoSDKRoot))
				{
					string desiredSDKLevel = GetAutoSDKLevelForPlatform(autoSDKRoot);
					// if the user doesn't want AutoSDK for this platform, then return that it is not installed, even if it actually is
					if (String.Equals(desiredSDKLevel, "NONE", StringComparison.OrdinalIgnoreCase))
					{
						return SDKStatus.Invalid;
					}

					// check script version so script fixes can be propagated without touching every build machine's CurrentlyInstalled file manually.
					if (GetLastRunScriptVersionString(autoSDKRoot, out string currentScriptVersionString) && currentScriptVersionString == GetRequiredScriptVersionString())
					{
						// check to make sure OutputEnvVars doesn't need regenerating
						string envVarFile = Path.Combine(autoSDKRoot, SDKEnvironmentVarsFile);
						bool bEnvVarFileExists = File.Exists(envVarFile);

						string currentSDKString;
						string currentSDKLevel;
						if (bEnvVarFileExists && GetCurrentlyInstalledSDKString(autoSDKRoot, out currentSDKString, out currentSDKLevel))
						{
							// match version
							if (currentSDKString == GetAutoSDKDirectoryForMainVersion())
							{
								if (IsSDKLevelAtLeast(currentSDKLevel, desiredSDKLevel))
								{
									return SDKStatus.Valid;
								}
							}
						}
					}
				}
			}
			return SDKStatus.Invalid;
		}

		// This tracks if we have already checked the sdk installation.
		private int SDKCheckStatus = -1;

		// true if we've ever overridden the process's environment with AutoSDK data.  After that, manual installs cannot be considered valid ever again.
		private bool bLocalProcessSetupAutoSDK = false;

		protected bool HasSetupAutoSDK()
		{
			return bLocalProcessSetupAutoSDK || HasParentProcessSetupAutoSDK(out _);
		}

		protected bool HasParentProcessSetupAutoSDK([NotNullWhen(true)] out string? outAutoSDKSetupValue)
		{
			string autoSDKSetupVarName = GetPlatformAutoSDKSetupEnvVar();
			outAutoSDKSetupValue = Environment.GetEnvironmentVariable(autoSDKSetupVarName);

			if (!String.IsNullOrEmpty(outAutoSDKSetupValue))
			{
				return true;
			}
			return false;
		}

		public SDKStatus HasRequiredManualSDK()
		{
			// 			if (HasSetupAutoSDK())
			// 			{
			// 				return SDKStatus.Invalid;
			// 			}
			//
			//			// manual installs are always invalid if we have modified the process's environment for AutoSDKs
			return HasRequiredManualSDKInternal();
		}

		// for platforms with destructive AutoSDK.  Report if any manual sdk is installed that may be damaged by an autosdk.
		protected virtual bool HasAnyManualInstall()
		{
			return false;
		}

		// tells us if the user has a valid manual install.
		protected virtual SDKStatus HasRequiredManualSDKInternal()
		{
			return GetAllSDKInfo().AreAllManualSDKsValid() ? SDKStatus.Valid : SDKStatus.Invalid;
			//string? ManualSDKVersion;
			//GetInstalledVersions(out ManualSDKVersion, out _);

			//return IsVersionValid(ManualSDKVersion, bForAutoSDK:false) ? SDKStatus.Valid : SDKStatus.Invalid;
		}

		// some platforms will fail if there is a manual install that is the WRONG manual install.
		protected virtual bool AllowInvalidManualInstall()
		{
			return true;
		}

		// platforms can choose if they prefer a correct the the AutoSDK install over the manual install.
		protected virtual bool PreferAutoSDK()
		{
			return true;
		}

		// some platforms don't support parallel SDK installs.  AutoSDK on these platforms will
		// actively damage an existing manual install by overwriting files in it.  AutoSDK must NOT
		// run any setup if a manual install exists in this case.
		protected virtual bool IsAutoSDKDestructive()
		{
			return false;
		}

		/// <summary>
		/// Runs batch files if necessary to set up required AutoSDK.
		/// AutoSDKs are SDKs that have not been setup through a formal installer, but rather come from
		/// a source control directory, or other local copy.
		/// </summary>
		private void SetupAutoSDK()
		{
			if (IsAutoSDKSafe() && PlatformSupportsAutoSDKs() && HasAutoSDKSystemEnabled())
			{
				// run installation for autosdk if necessary.
				if (HasRequiredAutoSDKInstalled() == SDKStatus.Invalid)
				{
					string autoSDKRoot = GetPathToPlatformAutoSDKs();

					string desiredSDKLevel = GetAutoSDKLevelForPlatform(autoSDKRoot);
					// if the user doesn't want AutoSDK for this platform, then do nothing
					if (String.Equals(desiredSDKLevel, "NONE", StringComparison.OrdinalIgnoreCase))
					{
						Logger.LogDebug("Skipping AutoSDK for {PlatformName} because NONE was specified as the desired AutoSDK level", PlatformName);
						return;
					}

					//reset check status so any checking sdk status after the attempted setup will do a real check again.
					SDKCheckStatus = -1;

					GetCurrentlyInstalledSDKString(autoSDKRoot, out string currentSDKString, out string currentSDKLevel);

					// switch over (note that version string can be empty)
					if (!RunAutoSDKHooks(autoSDKRoot, currentSDKString, currentSDKLevel, SDKHookType.Uninstall))
					{
						Logger.LogDebug("Failed to uninstall currently installed SDK {SdkVersion}", currentSDKString);
						InvalidateCurrentlyInstalledAutoSDK();
						return;
					}
					// delete Manifest file to avoid multiple uninstalls
					InvalidateCurrentlyInstalledAutoSDK();

					if (!RunAutoSDKHooks(autoSDKRoot, GetAutoSDKDirectoryForMainVersion(), desiredSDKLevel, SDKHookType.Install, false))
					{
						Logger.LogDebug("Failed to install required SDK {SdkVersion}.  Attempting to uninstall", GetAutoSDKDirectoryForMainVersion());
						RunAutoSDKHooks(autoSDKRoot, GetAutoSDKDirectoryForMainVersion(), desiredSDKLevel, SDKHookType.Uninstall, false);
						return;
					}

					string envVarFile = Path.Combine(autoSDKRoot, SDKEnvironmentVarsFile);
					if (!File.Exists(envVarFile))
					{
						Logger.LogDebug("Installation of required SDK {SdkVersion}.  Did not generate Environment file {EnvVarFile}", GetAutoSDKDirectoryForMainVersion(), envVarFile);
						RunAutoSDKHooks(autoSDKRoot, GetAutoSDKDirectoryForMainVersion(), desiredSDKLevel, SDKHookType.Uninstall, false);
						return;
					}

					SetCurrentlyInstalledAutoSDKString(GetAutoSDKDirectoryForMainVersion(), desiredSDKLevel);
					SetLastRunAutoSDKScriptVersion(GetRequiredScriptVersionString());
				}

				// fixup process environment to match autosdk
				SetupEnvironmentFromAutoSDK();
			}
		}

		/// <summary>
		/// Allows the platform to optionally returns a path to the internal SDK
		/// </summary>
		/// <returns>Valid path to the internal SDK, null otherwise</returns>
		public virtual string? GetInternalSDKPath()
		{
			return null;
		}

		#endregion

		#region public AutoSDKs Utility

		/// <summary>
		/// Enum describing types of hooks a platform SDK can have
		/// </summary>
		public enum SDKHookType
		{
			Install,
			Uninstall
		};

		/* Whether or not we should try to automatically switch SDKs when asked to validate the platform's SDK state. */
		public static bool bAllowAutoSDKSwitching = true;

		public SDKStatus SetupEnvironmentFromAutoSDK()
		{
			string platformSDKRoot = GetPathToPlatformAutoSDKs();

			// load environment variables from current SDK
			if (!SetupEnvironmentFromAutoSDK(platformSDKRoot))
			{
				Logger.LogDebug("Failed to load environment from required SDK {SdkRoot}", GetAutoSDKDirectoryForMainVersion());
				InvalidateCurrentlyInstalledAutoSDK();
				return SDKStatus.Invalid;
			}
			return SDKStatus.Valid;
		}

		/// <summary>
		/// Whether the required external SDKs are installed for this platform.
		/// Could be either a manual install or an AutoSDK.
		/// </summary>
		public SDKStatus HasRequiredSDKsInstalled()
		{
			// avoid redundant potentially expensive SDK checks.
			if (SDKCheckStatus == -1)
			{
				bool bHasManualSDK = HasRequiredManualSDK() == SDKStatus.Valid;
				bool bHasAutoSDK = HasRequiredAutoSDKInstalled() == SDKStatus.Valid;

				// Per-Platform implementations can choose how to handle non-Auto SDK detection / handling.
				SDKCheckStatus = (bHasManualSDK || bHasAutoSDK) ? 1 : 0;
			}
			return SDKCheckStatus == 1 ? SDKStatus.Valid : SDKStatus.Invalid;
		}

		// Arbitrates between manual SDKs and setting up AutoSDK based on program options and platform preferences.
		public void ManageAndValidateSDK()
		{
			// do not modify installed manifests if parent process has already set everything up.
			// this avoids problems with determining IsAutoSDKSafe and doing an incorrect invalidate.
			if (bAllowAutoSDKSwitching && !HasParentProcessSetupAutoSDK(out _))
			{
				bool bSetSomeSDK = false;
				bool bHasRequiredManualSDK = HasRequiredManualSDK() == SDKStatus.Valid;
				if (IsAutoSDKSafe() && (PreferAutoSDK() || !bHasRequiredManualSDK))
				{
					SetupAutoSDK();
					bSetSomeSDK = true;
				}

				//Setup manual SDK if autoSDK setup was skipped or failed for whatever reason.
				if (bHasRequiredManualSDK && (HasRequiredAutoSDKInstalled() != SDKStatus.Valid))
				{
					SetupManualSDK();
					bSetSomeSDK = true;
				}

				if (!bSetSomeSDK)
				{
					InvalidateCurrentlyInstalledAutoSDK();
				}
			}

			// print all SDKs to log file (errors will print out later for builds and generateprojectfiles)
			PrintSDKInfoAndReturnValidity(LogEventType.Log, LogFormatOptions.NoConsoleOutput, LogEventType.Log, LogFormatOptions.NoConsoleOutput, bBriefInvalidSDKWarnings: true);
		}
		#endregion

	}
}
