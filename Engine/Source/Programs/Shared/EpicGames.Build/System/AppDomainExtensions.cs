// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace UnrealBuildBase
{
	/// <summary>
	/// Extensions to AppDomain
	/// </summary>
	public static class AppDomainExtensions
	{
		/// <summary>
		/// Using reflection to find all subclasses for a class.
		/// Will check which assembly that class is in, then traverse that assmbly plus all assemblies depending on that assembly
		/// to find the base classes. This is much more efficient than parsing all loaded assemblies to find subclasses
		/// </summary>
		/// <param name="domain">AppDomain to use</param>
		/// <param name="baseClass">Base class to use</param>
		public static IEnumerable<Type> EnumerateSubclasses(this AppDomain domain, Type baseClass)
		{
			Assembly baseAssembly = baseClass.Assembly;
			AssemblyName baseAssemblyName = baseAssembly.GetName();

			// Only search in assemblies that are either baseClass's assembly or assemblies depending on it
			foreach (Assembly assembly in domain.GetAssemblies().Where(x => !x.IsDynamic))
			{
				if (!ReferenceEquals(assembly, baseAssembly))
				{
					try
					{
						if (!assembly
							.GetReferencedAssemblies()
							.Any(x => AssemblyName.ReferenceMatchesDefinition(x, baseAssemblyName)))
						{
							continue;
						}
					}
					catch
					{
						continue;
					}
				}

				Type[] types = assembly.SafeGetLoadedTypes();
				foreach (Type t in types.Where(t => t.IsClass && !t.IsAbstract && t.IsSubclassOf(baseClass)))
				{
					yield return t;
				}
			}
		}
	}
}