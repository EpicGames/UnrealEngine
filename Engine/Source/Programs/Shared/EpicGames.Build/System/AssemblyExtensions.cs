// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Reflection;

namespace UnrealBuildBase
{
	/// <summary>
	/// Extensions to Assembly
	/// </summary>
	public static class AssemblyExtensions
	{
		/// <summary>
		/// Safely get all loaded types from an assembly.
		/// </summary>
		/// <param name="assembly">Assembly to use</param>
		/// <returns></returns>
		public static Type[] SafeGetLoadedTypes(this Assembly assembly)
		{
			try
			{
				return assembly.GetTypes();
			}
			catch (ReflectionTypeLoadException e)
			{
				return [.. e.Types.OfType<Type>()];
			}
		}
	}
}
