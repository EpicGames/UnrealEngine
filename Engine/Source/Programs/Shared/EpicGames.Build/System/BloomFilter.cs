// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildBase
{
	/// <summary>
	/// Simple bloom filter implementation which can be used to optimize searches by early out on 
	/// </summary>
	public sealed class BloomFilter
	{
		const uint ByteCount = 128;
		private readonly byte[] _bits = new byte[ByteCount];

		/// <summary>
		/// Add string to filter
		/// </summary>
		/// <param name="s"></param>
		public void Add(ReadOnlySpan<char> s)
		{
			uint hash = (uint)IoHash.Compute(s).GetHashCode();
			int bitIndex = (int)(hash % (ByteCount * 8));
			int byteIndex = bitIndex / 8;
			int bitInByte = bitIndex - byteIndex * 8;
			_bits[byteIndex] |= (byte)(1 << bitInByte);
		}

		/// <summary>
		/// Add multiple strings to filter
		/// </summary>
		/// <param name="strings"></param>
		public void AddRange(IEnumerable<string> strings)
		{
			foreach (string s in strings)
			{
				Add(s);
			}
		}

		/// <summary>
		/// Will return false if string is guaranteed to not exist, and true if it could exist
		/// </summary>
		/// <param name="s"></param>
		public bool MightContain(ReadOnlySpan<char> s)
		{
			uint hash = (uint)IoHash.Compute(s).GetHashCode();
			int bitIndex = (int)(hash % (ByteCount * 8));
			int byteIndex = bitIndex / 8;
			int bitInByte = bitIndex - byteIndex * 8;
			return (_bits[byteIndex] & (1 << bitInByte)) != 0;
		}

		/// <summary>
		/// Will traverse a string to find substrings separated with separator and then test them
		/// Useful if you for example want to check if a a string might exist somewhere in a path
		/// </summary>
		/// <param name="span"></param>
		/// <param name="separator"></param>
		public bool MightContainSubString(ReadOnlySpan<char> span, char separator)
		{
			int separatorIndex;
			while ((separatorIndex = span.IndexOf(separator)) != -1)
			{
				if (MightContain(span.Slice(0, separatorIndex)))
				{
					return true;
				}
				span = span.Slice(separatorIndex + 1);
			}
			return MightContain(span);
		}
	}
}