// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.MsBuild;
using Microsoft.Extensions.FileSystemGlobbing;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	public static class CompileScriptModule
	{
		[Flags]
		public enum BuildFlags
		{
			/// <summary>
			/// No build flags specified
			/// </summary>
			None = 0,

			/// <summary>
			/// Do not allow compiles of any projects
			/// </summary>
			NoCompile = 1 << 0,

			/// <summary>
			/// If not compiling, generate error if target does not exist
			/// </summary>
			ErrorOnMissingTarget = 1 << 1,

			/// <summary>
			/// When NoCompile options are specified, this flag will test to see if the build records
			/// are valid.  But the results of that test are currently ignored.  Only the existence of the
			/// target path is considered.
			/// 
			/// When NoCompile options are not specified, then the validity of the build records will be used
			/// to test to see if project files need building.  If ForceCompile is specified, then this flag
			/// serves no useful purpose.  If this flag is not specified, then ForceCompile is assumed.
			/// </summary>
			UseBuildRecords = 1 << 2,

			/// <summary>
			/// Force compile when the NoCompile options aren't specified regardless of the 
			/// state of the build records.
			/// </summary>
			ForceCompile = 1 << 3,
		}

		class Hook : CsProjBuildHook
		{
			private ILogger Logger;
			private ConcurrentDictionary<string, DateTime> WriteTimes = [];
			private string BuildRecordDirectory;
			private Dictionary<FileReference, CsProjBuildRecordEntry> ValidBuildRecords = [];

			public Hook(ILogger InLogger, Rules.RulesFileType InRulesFileType)
			{
				Logger = InLogger;
				switch (InRulesFileType)
				{
					case Rules.RulesFileType.AutomationModule:
						BuildRecordDirectory = "ScriptModules";
						break;
					case Rules.RulesFileType.UbtPlugin:
						BuildRecordDirectory = "ScriptModules.UBT";
						break;
					default:
						throw new ArgumentException("Unexpected rules file type");
				}
			}

			public DateTime GetLastWriteTime(DirectoryReference BasePath, string RelativeFilePath)
			{
				return GetLastWriteTime(BasePath.FullName, RelativeFilePath);
			}

			public DateTime GetLastWriteTime(string BasePath, string RelativeFilePath)
			{
				string NormalizedPath = Path.GetFullPath(RelativeFilePath, BasePath);
				return WriteTimes.GetOrAdd(NormalizedPath, File.GetLastWriteTime);
			}

			public DirectoryReference GetBuildRecordDirectory(DirectoryReference BasePath)
			{
				return DirectoryReference.Combine(BasePath, "Intermediate", BuildRecordDirectory);
			}

			public void ValidateRecursively(Dictionary<FileReference, CsProjBuildRecordEntry> BuildRecords, FileReference ProjectPath)
			{
				ValidateBuildRecordRecursively(BuildRecords, ProjectPath, this, Logger);
			}

			public void SetValidBuildRecords(IReadOnlyDictionary<FileReference, CsProjBuildRecordEntry> Records)
			{
				ValidBuildRecords = new(Records.Where(x => x.Value.Status == CsProjBuildRecordStatus.Valid));
			}

			IReadOnlyDictionary<FileReference, CsProjBuildRecordEntry> CsProjBuildHook.ValidBuildRecords => ValidBuildRecords;

			DirectoryReference CsProjBuildHook.EngineDirectory => Unreal.EngineDirectory;

			DirectoryReference IMsBuildRegistration.DotnetDirectory => Unreal.DotnetDirectory;

			FileReference IMsBuildRegistration.DotnetPath => Unreal.DotnetPath;
		}

		/// <summary>
		/// Return the target paths from the collection of build records
		/// </summary>
		/// <param name="BuildRecords">Input build records</param>
		/// <returns>Set of target files</returns>
		public static HashSet<FileReference> GetTargetPaths(IReadOnlyDictionary<FileReference, CsProjBuildRecordEntry> BuildRecords)
		{
			return [.. BuildRecords.Select(GetTargetPath)];
		}

		/// <summary>
		/// Return the target path for the given build record
		/// </summary>
		/// <param name="BuildRecord">Build record</param>
		/// <returns>File reference for the target</returns>
		public static FileReference GetTargetPath(KeyValuePair<FileReference, CsProjBuildRecordEntry> BuildRecord)
		{
			return FileReference.Combine(BuildRecord.Key.Directory, BuildRecord.Value.BuildRecord.TargetPath!);
		}

		/// <summary>
		/// Locates script modules, builds them if necessary, returns set of .dll files
		/// </summary>
		/// <param name="RulesFileType"></param>
		/// <param name="ScriptsForProjectFileName"></param>
		/// <param name="AdditionalScriptsFolders"></param>
		/// <param name="AutomationProjectIncludeFilter"></param>
		/// <param name="bForceCompile"></param>
		/// <param name="bNoCompile"></param>
		/// <param name="bUseBuildRecords"></param>
		/// <param name="bBuildSuccess"></param>
		/// <param name="Logger"></param>
		/// <returns>Collection of all the projects.  They will have been compiled.</returns>
		public static HashSet<FileReference> InitializeScriptModules(Rules.RulesFileType RulesFileType,
			string? ScriptsForProjectFileName, List<string>? AdditionalScriptsFolders, List<string>? AutomationProjectIncludeFilter, bool bForceCompile, bool bNoCompile, bool bUseBuildRecords,
			out bool bBuildSuccess, ILogger Logger)
		{
			Stopwatch Timer = Stopwatch.StartNew();
			List<DirectoryReference> GameDirectories = GetGameDirectories(ScriptsForProjectFileName, Logger);
			List<DirectoryReference> AdditionalDirectories = GetAdditionalDirectories(AdditionalScriptsFolders);
			List<DirectoryReference> GameBuildDirectories = GetAdditionalBuildDirectories(GameDirectories);

			// List of directories used to locate Intermediate/ScriptModules dirs for writing build records
			List<DirectoryReference> BaseDirectories =
			[
				Unreal.EngineDirectory,
				.. GameDirectories,
				.. AdditionalDirectories,
			];
			BaseDirectories = [.. BaseDirectories.Distinct()];
			Logger.LogDebug("Find {Count} directories time: {TimeSeconds:0.00} s", BaseDirectories.Count, Timer.Elapsed.TotalSeconds);

			Timer.Restart();
			HashSet<FileReference> FoundProjects =
				[.. Rules.FindAllRulesSourceFiles(RulesFileType,
				// Project scripts require source engine builds
				GameFolders: Unreal.IsEngineInstalled() ? GameDirectories : [],
				ForeignPlugins: null, AdditionalSearchPaths: [.. AdditionalDirectories, .. GameBuildDirectories])];
			if (AutomationProjectIncludeFilter != null && AutomationProjectIncludeFilter.Count > 0)
			{
				FoundProjects = [.. FoundProjects.Where(x => (AutomationProjectIncludeFilter.Contains(x.GetFileNameWithoutAnyExtensions(), StringComparer.OrdinalIgnoreCase)))];
			}
			Logger.LogDebug("Find {Count} projects time: {TimeSeconds:0.00} s", FoundProjects.Count, Timer.Elapsed.TotalSeconds);

			BuildFlags BuildFlags = BuildFlags.None;
			if (bForceCompile)
			{
				BuildFlags |= BuildFlags.ForceCompile;
			}
			if (bNoCompile)
			{
				BuildFlags |= BuildFlags.NoCompile;
			}
			if (bUseBuildRecords)
			{
				BuildFlags |= BuildFlags.UseBuildRecords;
			}

			Dictionary<FileReference, CsProjBuildRecordEntry> BuildResults;
			if (Unreal.IsEngineInstalled())
			{
				// Warn if not using build records or force compiling
				if (BuildFlags.HasFlag(BuildFlags.ForceCompile))
				{
					Logger.LogWarning("ForceCompile not supported if Unreal.IsEngineInstalled() == true");
				}
				if (!BuildFlags.HasFlag(BuildFlags.UseBuildRecords))
				{
					Logger.LogWarning("Disabling UseBuildRecords not supported if Unreal.IsEngineInstalled() == true");
				}
				BuildFlags EngineBuildFlags = BuildFlags | BuildFlags.UseBuildRecords | BuildFlags.NoCompile | BuildFlags.ErrorOnMissingTarget;

				bool bEngineBuildSuccess = false;
				Dictionary<FileReference, CsProjBuildRecordEntry> EngineBuildResults = Build(
					RulesFileType,
					[.. FoundProjects.Where(x => x.IsUnderDirectory(Unreal.EngineDirectory))],
					[.. BaseDirectories.Where(x => x.IsUnderDirectory(Unreal.EngineDirectory))],
					null,
					EngineBuildFlags,
					out bEngineBuildSuccess,
					Logger);

				bool bProjectBuildSuccess = false;
				BuildFlags ProjectBuildFlags = BuildFlags | BuildFlags.UseBuildRecords;
				// ForceCompile not supported for installed builds
				if (ProjectBuildFlags.HasFlag(BuildFlags.ForceCompile))
				{
					ProjectBuildFlags &= ~BuildFlags.ForceCompile;
				}
				Dictionary<FileReference, CsProjBuildRecordEntry> ProjectBuildResults = Build(
					RulesFileType,
					[.. FoundProjects.Where(x => !x.IsUnderDirectory(Unreal.EngineDirectory))],
					BaseDirectories,
					null,
					ProjectBuildFlags,
					out bProjectBuildSuccess,
					Logger);

				bBuildSuccess = bEngineBuildSuccess && bProjectBuildSuccess;
				BuildResults = new(EngineBuildResults);
				foreach (KeyValuePair<FileReference, CsProjBuildRecordEntry> Item in ProjectBuildResults)
				{
					if (!BuildResults.ContainsKey(Item.Key))
					{
						BuildResults.Add(Item.Key, Item.Value);
					}
				}
			}
			else
			{
				BuildResults = Build(RulesFileType, FoundProjects, BaseDirectories, null, BuildFlags, out bBuildSuccess, Logger);
			}

			return GetTargetPaths(BuildResults);
		}

		/// <summary>
		/// Test to see if all the given projects are up-to-date
		/// </summary>
		/// <param name="RulesFileType">Type of script modules being tested</param>
		/// <param name="FoundProjects">Collection of projects to test</param>
		/// <param name="BaseDirectories">Base directories of the projects</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if all of the projects are up to date</returns>
		public static bool AreScriptModulesUpToDate(Rules.RulesFileType RulesFileType, HashSet<FileReference> FoundProjects, List<DirectoryReference> BaseDirectories, ILogger Logger)
		{
			CsProjBuildHook Hook = new Hook(Logger, RulesFileType);

			// Load existing build records, validating them only if (re)compiling script projects is an option
			Dictionary<FileReference, CsProjBuildRecordEntry> BuildRecords = LoadExistingBuildRecords(Hook, BaseDirectories, Logger);
			foreach (FileReference Project in FoundProjects)
			{
				ValidateBuildRecordRecursively(BuildRecords, Project, Hook, Logger);
			}

			// If all found records are valid, we can return their targets directly
			Dictionary<FileReference, CsProjBuildRecordEntry> ValidBuildRecords = new(BuildRecords.Where(x => x.Value.Status == CsProjBuildRecordStatus.Valid));
			return FoundProjects.All(x => ValidBuildRecords.ContainsKey(x));
		}

		/// <summary>
		/// Locates script modules, builds them if necessary, returns set of .dll files
		/// </summary>
		/// <param name="RulesFileType"></param>
		/// <param name="FoundProjects">Projects to be compiled</param>
		/// <param name="BaseDirectories">Base directories for all the projects</param>
		/// <param name="CustomGlobalProperties">Additional MSBuild global properties to set while building projects</param>
		/// <param name="BuildFlags">Collection of flags to customize compilation</param>
		/// <param name="bBuildSuccess"></param>
		/// <param name="Logger"></param>
		/// <returns>Collection of all the projects.  They will have been compiled.</returns>
		public static Dictionary<FileReference, CsProjBuildRecordEntry> Build(Rules.RulesFileType RulesFileType,
			HashSet<FileReference> FoundProjects, IEnumerable<DirectoryReference> BaseDirectories, IReadOnlyDictionary<string, IEnumerable<string>>? CustomGlobalProperties,
			BuildFlags BuildFlags, out bool bBuildSuccess, ILogger Logger)
		{
			CsProjBuildHook Hook = new Hook(Logger, RulesFileType);

			// Load existing build records, validating them only if (re)compiling script projects is an option
			Dictionary<FileReference, CsProjBuildRecordEntry> BuildRecords = LoadExistingBuildRecords(Hook, BaseDirectories, Logger);

			// Only validate the build records if requested
			if (BuildFlags.HasFlag(BuildFlags.UseBuildRecords))
			{
				if (Unreal.IsEngineInstalled())
				{
					// Validate build records only for projects to be compiled
					foreach (FileReference Project in FoundProjects)
					{
						ValidateBuildRecordRecursively(BuildRecords, Project, Hook, Logger);
					}
				}
				else
				{
					// Validate build records for all existing records
					foreach (FileReference Project in BuildRecords.Keys)
					{
						ValidateBuildRecordRecursively(BuildRecords, Project, Hook, Logger);
					}
				}
			}
			else
			{
				// If not using build records, mark all existing records as invalid and force compile
				foreach (CsProjBuildRecordEntry Record in BuildRecords.Values)
				{
					Record.Status = CsProjBuildRecordStatus.Invalid;
				}
				BuildFlags |= BuildFlags.ForceCompile;
			}

			// If we are not compiling, then test to see if the targets exist
			if (BuildFlags.HasFlag(BuildFlags.NoCompile))
			{
				string FilterExtension = String.Empty;
				switch (RulesFileType)
				{
					case Rules.RulesFileType.AutomationModule:
						FilterExtension = ".Automation.json";
						break;
					case Rules.RulesFileType.UbtPlugin:
						FilterExtension = ".ubtplugin.json";
						break;
					default:
						throw new Exception("Unsupported rules file type");
				}
				Dictionary<FileReference, CsProjBuildRecordEntry> OutRecords = new(BuildRecords.Count);
				foreach (KeyValuePair<FileReference, CsProjBuildRecordEntry> Record in BuildRecords.Where(x => x.Value.BuildRecordFile.HasExtension(FilterExtension)))
				{
					FileReference TargetPath = FileReference.Combine(Record.Key.Directory, Record.Value.BuildRecord.TargetPath!);

					if (FileReference.Exists(TargetPath))
					{
						OutRecords.Add(Record.Key, Record.Value);
					}
					else
					{
						if (!BuildFlags.HasFlag(BuildFlags.ErrorOnMissingTarget))
						{
							// when -NoCompile is on the command line, try to run with whatever is available
							Logger.LogWarning("Script module \"{TargetPath}\" not found for record \"{BuildRecordPath}\"", TargetPath, Record.Value.BuildRecordFile);
						}
						else
						{
							// when the engine is installed, expect to find a built target assembly for every record that was found
							throw new Exception($"Script module \"{TargetPath}\" not found for record \"{Record.Value.BuildRecordFile}\"");
						}
					}
				}
				bBuildSuccess = true;
				return OutRecords;
			}
			else
			{
				// when the engine is not installed, delete any build record .json file that is not valid
				foreach (CsProjBuildRecordEntry Entry in BuildRecords.Values.Where(x => x.Status == CsProjBuildRecordStatus.Invalid))
				{
					if (Entry.BuildRecordFile != null)
					{
						Logger.LogDebug("Deleting invalid build record \"{BuildRecordPath}\"", Entry.BuildRecordFile);
						FileReference.Delete(Entry.BuildRecordFile);
					}
				}
			}

			if (!BuildFlags.HasFlag(BuildFlags.ForceCompile))
			{
				// If all found records are valid, we can return their targets directly
				Dictionary<FileReference, CsProjBuildRecordEntry> ValidBuildRecords = new(BuildRecords.Where(x => x.Value.Status == CsProjBuildRecordStatus.Valid));
				if (FoundProjects.All(x => ValidBuildRecords.ContainsKey(x)))
				{
					bBuildSuccess = true;
					return new(ValidBuildRecords.Where(x => FoundProjects.Contains(x.Key)));
				}
				Hook.SetValidBuildRecords(ValidBuildRecords);
			}

			// Fall back to the slower approach: use msbuild to load csproj files & build as necessary
			return Build(FoundProjects, BuildFlags.HasFlag(BuildFlags.ForceCompile), out bBuildSuccess, Hook, BaseDirectories, CustomGlobalProperties, Logger);
		}

		/// <summary>
		/// This method exists purely to prevent EpicGames.MsBuild from being loaded until the absolute last moment.
		/// If it is placed in the caller directly, then when the caller is invoked, the assembly will be loaded resulting
		/// in the possible Microsoft.Build.Framework load issue later on is this method isn't invoked.
		/// </summary>
		[MethodImpl(MethodImplOptions.NoInlining)]
		static Dictionary<FileReference, CsProjBuildRecordEntry> Build(HashSet<FileReference> FoundProjects,
			bool bForceCompile, out bool bBuildSuccess, CsProjBuildHook Hook, IEnumerable<DirectoryReference> BaseDirectories,
			IReadOnlyDictionary<string, IEnumerable<string>>? CustomGlobalProperties, ILogger Logger)
		{
			return CsProjBuilder.Build(FoundProjects, bForceCompile, out bBuildSuccess, Hook, BaseDirectories, CustomGlobalProperties ?? ImmutableDictionary<string, IEnumerable<string>>.Empty, Logger);
		}

		/// <summary>
		/// Find and load existing build record .json files from any Intermediate/ScriptModules found in the provided lists
		/// </summary>
		/// <param name="Hook">Hook object used to query directory</param>
		/// <param name="BaseDirectories"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		static Dictionary<FileReference, CsProjBuildRecordEntry> LoadExistingBuildRecords(CsProjBuildHook Hook, IEnumerable<DirectoryReference> BaseDirectories, ILogger Logger)
		{
			Dictionary<FileReference, CsProjBuildRecordEntry> LoadedBuildRecords = [];

			foreach (DirectoryReference Directory in BaseDirectories)
			{
				DirectoryReference IntermediateDirectory = Hook.GetBuildRecordDirectory(Directory);
				if (!DirectoryReference.Exists(IntermediateDirectory))
				{
					continue;
				}

				foreach (FileReference JsonFile in DirectoryReference.EnumerateFiles(IntermediateDirectory, "*.json"))
				{
					CsProjBuildRecord? BuildRecord = default;

					// filesystem errors or json parsing might result in an exception. If that happens, we fall back to the
					// slower path - if compiling, buildrecord files will be re-generated; other filesystem errors may persist
					try
					{
						BuildRecord = JsonSerializer.Deserialize<CsProjBuildRecord>(FileReference.ReadAllText(JsonFile));
						Logger.LogTrace("Loaded script module build record {JsonFile}", JsonFile);
					}
					catch (Exception ex)
					{
						Logger.LogWarning(ex, "[{JsonFile}] Failed to load build record: {Message}", JsonFile, ex.Message);
					}

					if (BuildRecord != null && BuildRecord.ProjectPath != null)
					{
						CsProjBuildRecordEntry Entry = new(
							new(Path.GetFullPath(BuildRecord.ProjectPath, JsonFile.Directory.FullName)),
							JsonFile,
							BuildRecord);
						LoadedBuildRecords.Add(Entry.ProjectFile, Entry);
					}
					else
					{
						// Delete the invalid build record
						Logger.LogWarning("Deleting invalid build record {JsonFile}", JsonFile);
					}
				}
			}

			return LoadedBuildRecords;
		}

		private static bool ValidateGlobbedFiles(DirectoryReference ProjectDirectory, List<CsProjBuildRecord.Glob> Globs, HashSet<string> Dependencies, out string ValidationFailureMessage)
		{
			string[] AllFiles = [.. Directory.EnumerateFiles(ProjectDirectory.FullName, "*", SearchOption.AllDirectories)];

			// First, evaluate globs

			// Files are grouped by ItemType (e.g. Compile, EmbeddedResource) to ensure that Exclude and
			// Remove act as expected.
			Dictionary<string, HashSet<string>> Files = [];
			foreach (CsProjBuildRecord.Glob Glob in Globs)
			{
				HashSet<string>? TypedFiles;
				if (!Files.TryGetValue(Glob.ItemType!, out TypedFiles))
				{
					TypedFiles = [];
					Files.Add(Glob.ItemType!, TypedFiles);
				}

				Matcher matcher = new();
				if (Glob.Include?.Count > 0)
				{
					matcher.AddIncludePatterns(Glob.Include);
				}
				if (Glob.Exclude?.Count > 0)
				{
					matcher.AddExcludePatterns(Glob.Exclude);
				}

				// Not using GetResultsInFullPath, as internally it's super slow
				// Match returns filenames separated with '/'
				if (OperatingSystem.IsWindows())
				{
					TypedFiles.UnionWith(matcher.Match(ProjectDirectory.FullName, AllFiles).Files.Select(x => x.Path.Replace('/', Path.DirectorySeparatorChar)));
				}
				else
				{
					TypedFiles.UnionWith(matcher.Match(ProjectDirectory.FullName, AllFiles).Files.Select(x => x.Path));
				}

				if (Glob.Remove?.Count > 0)
				{
					// Matcher.Match() doesn't handle inconsistent path separators correctly - which is why globs
					// are normalized when they are added to CsProjBuildRecord
					Matcher removeMatcher = new();
					removeMatcher.AddIncludePatterns(Glob.Remove);
					TypedFiles.RemoveWhere(F => removeMatcher.Match(F).HasMatches);
				}
			}

			// Then, validation that our evaluation matches what we're comparing against

			// Look for extra files that were found.
			HashSet<string> newFiles = [.. Files.Values.SelectMany(x => x)];
			newFiles.ExceptWith(Dependencies);
			ValidationFailureMessage = newFiles.Count > 0 ? $"Found additional files: {String.Join(Environment.NewLine, newFiles.Order())}" : String.Empty;

			return newFiles.Count == 0;
		}

		private static bool ValidateBuildRecord(CsProjBuildRecord BuildRecord, DirectoryReference ProjectDirectory, out string ValidationFailureMessage, CsProjBuildHook Hook)
		{
			if (BuildRecord.Version != CsProjBuildRecord.CurrentVersion)
			{
				ValidationFailureMessage =
					$"version does not match: build record has version {BuildRecord.Version}; current version is {CsProjBuildRecord.CurrentVersion}";
				return false;
			}

			if (BuildRecord.RuntimeVersion != Environment.Version)
			{
				ValidationFailureMessage =
					$"runtime version does not match: build record has version {BuildRecord.RuntimeVersion}; current version is {Environment.Version}";
				return false;
			}

			if (BuildRecord.TargetConfiguration != CsProjBuildRecord.CurrentConfiguration)
			{
				ValidationFailureMessage =
					$"recorded target configuration ({BuildRecord.TargetConfiguration}) does not match ({CsProjBuildRecord.CurrentConfiguration})";
				return false;
			}

			string TargetRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, BuildRecord.TargetPath!);
			string NormalizedPath = Path.GetFullPath(BuildRecord.TargetPath!, ProjectDirectory.FullName);
			if (!File.Exists(NormalizedPath))
			{
				ValidationFailureMessage = $"{TargetRelativePath} not found";
				return false;
			}

			DateTime TargetWriteTime = Hook.GetLastWriteTime(ProjectDirectory, BuildRecord.TargetPath!);
			if (BuildRecord.TargetBuildTime != TargetWriteTime)
			{
				ValidationFailureMessage =
					$"recorded target build time ({BuildRecord.TargetBuildTime}) does not match {TargetRelativePath} ({TargetWriteTime})";
				return false;
			}

			foreach (string Dependency in BuildRecord.Dependencies)
			{
				NormalizedPath = Path.GetFullPath(Dependency, ProjectDirectory.FullName);
				if (!File.Exists(NormalizedPath))
				{
					ValidationFailureMessage = $"{Dependency} not found";
					return false;
				}

				if (Hook.GetLastWriteTime(ProjectDirectory, Dependency) > TargetWriteTime)
				{
					ValidationFailureMessage = $"{Dependency} is newer than {TargetRelativePath}";
					return false;
				}
			}

			if (!ValidateGlobbedFiles(ProjectDirectory, BuildRecord.Globs, BuildRecord.Dependencies, out ValidationFailureMessage))
			{
				return false;
			}

			return true;
		}

		static void ValidateBuildRecordRecursively(
			Dictionary<FileReference, CsProjBuildRecordEntry> BuildRecords,
			FileReference ProjectPath, CsProjBuildHook Hook, ILogger Logger)
		{
			// Was a build record loaded for this project path? (relevant when considering referenced projects)
			if (!BuildRecords.TryGetValue(ProjectPath, out CsProjBuildRecordEntry? Entry))
			{
				Logger.LogTrace("Found project {ProjectPath} with no existing build record", ProjectPath);
				return;
			}

			// Ignore if the status is known
			if (Entry.Status != CsProjBuildRecordStatus.Unknown)
			{
				// Project validity has already been determined
				return;
			}

			// If the engine is installed, and the project being validated is under the engine directory, treat it as valid
			// so it is not built as a dependency when building an external project.
			// It is assumed these projects exist and have already been verified.
			if (Unreal.IsEngineInstalled() && ProjectPath.IsUnderDirectory(Unreal.EngineDirectory))
			{
				Entry.Status = CsProjBuildRecordStatus.Valid;
				return;
			}

			// Is this particular build record valid?
			if (!ValidateBuildRecord(Entry.BuildRecord, ProjectPath.Directory, out string ValidationFailureMessage, Hook))
			{
				string ProjectRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, ProjectPath.FullName);
				Logger.LogTrace("[{ProjectRelativePath}] {ValidationFailureMessage}", ProjectRelativePath, ValidationFailureMessage);
				Entry.Status = CsProjBuildRecordStatus.Invalid;
				return;
			}

			// Are all referenced build records valid?
			foreach (CsProjBuildRecordRef ReferencedProject in Entry.BuildRecord.ProjectReferencesAndTimes)
			{
				FileReference FullProjectPath = new(Path.GetFullPath(ReferencedProject.ProjectPath, ProjectPath.Directory.FullName));
				ValidateBuildRecordRecursively(BuildRecords, FullProjectPath, Hook, Logger);

				if (!BuildRecords.TryGetValue(FullProjectPath, out CsProjBuildRecordEntry? RefEntry))
				{
					string ProjectRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, ProjectPath.FullName);
					string DependencyRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, FullProjectPath.FullName);
					Logger.LogTrace("[{ProjectRelativePath}] Existing output is not valid because dependency {DependencyRelativePath} could not be found", ProjectRelativePath, DependencyRelativePath);
					Entry.Status = CsProjBuildRecordStatus.Invalid;
					return;
				}

				if (RefEntry.Status != CsProjBuildRecordStatus.Valid)
				{
					string ProjectRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, ProjectPath.FullName);
					string DependencyRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, FullProjectPath.FullName);
					Logger.LogTrace("[{ProjectRelativePath}] Existing output is not valid because dependency {DependencyRelativePath} is not valid", ProjectRelativePath, DependencyRelativePath);
					Entry.Status = CsProjBuildRecordStatus.Invalid;
					return;
				}

				// Ensure that the dependency was not built more recently than the project
				if (ReferencedProject.TargetBuildTime == DateTime.MinValue)
				{
					if (Entry.BuildRecord.TargetBuildTime < RefEntry.BuildRecord.TargetBuildTime)
					{
						string ProjectRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, ProjectPath.FullName);
						string DependencyRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, FullProjectPath.FullName);
						Logger.LogTrace("[{ProjectRelativePath}] Existing output is not valid because dependency {DependencyRelativePath} is newer", ProjectRelativePath, DependencyRelativePath);
						Entry.Status = CsProjBuildRecordStatus.Invalid;
						return;
					}
				}
				else
				{
					if (ReferencedProject.TargetBuildTime != RefEntry.BuildRecord.TargetBuildTime)
					{
						string ProjectRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, ProjectPath.FullName);
						string DependencyRelativePath = Path.GetRelativePath(Unreal.EngineDirectory.FullName, FullProjectPath.FullName);
						Logger.LogTrace("[{ProjectRelativePath}] Existing dependency output time stamp for {DependencyRelativePath} does not match expected value", ProjectRelativePath, DependencyRelativePath);
						Entry.Status = CsProjBuildRecordStatus.Invalid;
						return;
					}
				}
			}

			Entry.Status = CsProjBuildRecordStatus.Valid;
		}

		static List<DirectoryReference> GetGameDirectories(string? ScriptsForProjectFileName, ILogger Logger)
		{
			List<DirectoryReference> GameDirectories = [];

			if (String.IsNullOrEmpty(ScriptsForProjectFileName))
			{
				GameDirectories = [.. NativeProjectsBase.EnumerateProjectFiles(Logger).Select(x => x.Directory)];
			}
			else
			{
				DirectoryReference ScriptsDir = new(Path.GetDirectoryName(ScriptsForProjectFileName)!);
				ScriptsDir = DirectoryReference.FindCorrectCase(ScriptsDir);
				GameDirectories.Add(ScriptsDir);
			}
			return GameDirectories;
		}

		static List<DirectoryReference> GetAdditionalDirectories(List<string>? AdditionalScriptsFolders) =>
			AdditionalScriptsFolders == null ? [] :
				[.. AdditionalScriptsFolders.Select(x => DirectoryReference.FindCorrectCase(new DirectoryReference(x)))];

		static List<DirectoryReference> GetAdditionalBuildDirectories(List<DirectoryReference> GameDirectories) =>
			[.. GameDirectories.Select(x => DirectoryReference.Combine(x, "Build")).Where(x => DirectoryReference.Exists(x))];
	}
}

