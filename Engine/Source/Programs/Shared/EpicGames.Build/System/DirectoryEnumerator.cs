// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Runtime.InteropServices;

namespace UnrealBuildBase
{
	/// <summary>
	/// Platform optimized utility class to enumerate directory to fetch information needed by FileItem and DirectoryItem
	/// </summary>
	public static class DirectoryEnumerator
	{
		/// <summary>
		/// Struct representing a file system entry
		/// </summary>
		/// <param name="Name"></param>
		/// <param name="Size"></param>
		/// <param name="Attributes"></param>
		/// <param name="CreationTimeUtc"></param>
		/// <param name="LastWriteTimeUtc"></param>
		public readonly record struct Entry(string Name, long Size, FileAttributes Attributes, DateTime CreationTimeUtc, DateTime LastWriteTimeUtc);

		/// <summary>
		/// Enumerate a directory. If directory does not exist it will return null
		/// </summary>
		/// <param name="directoryPath">Path to directory</param>
		/// <param name="bufferSize">Buffer size used internally to reduce number of kernel calls</param>
		/// <returns>List of files</returns>
		public static unsafe List<Entry>? Enumerate(string directoryPath, int bufferSize = 128 * 1024)
		{
			if (OperatingSystem.IsWindows())
			{
				string ntPath;
				if (directoryPath[0] == '\\')
				{
					ntPath = @"\??\UNC\" + directoryPath.TrimStart('\\'); // UNC -> \??\UNC\server\share\...
				}
				else
				{
					ntPath = @"\??\" + directoryPath; // Drive -> \??\C:\...
				}

				IntPtr dirHandle = IntPtr.Zero;

				fixed (char* p = ntPath)
				{
					UNICODE_STRING name = new()
					{
						Length = (ushort)(ntPath.Length * sizeof(char)),
						MaximumLength = (ushort)((ntPath.Length + 1) * sizeof(char)),
						Buffer = p
					};

					OBJECT_ATTRIBUTES oa = new()
					{
						Length = (uint)sizeof(OBJECT_ATTRIBUTES),
						RootDirectory = IntPtr.Zero,
						ObjectName = &name,
						Attributes = OBJ_CASE_INSENSITIVE,
						SecurityDescriptor = IntPtr.Zero,
						SecurityQualityOfService = IntPtr.Zero
					};

					long alloc = 0;

					int status = NtCreateFile(out dirHandle, FILE_LIST_DIRECTORY | SYNCHRONIZE, ref oa, out IO_STATUS_BLOCK iosb, ref alloc, 0u, FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE, FILE_OPEN, FILE_DIRECTORY_FILE | FILE_SYNCHRONOUS_IO_NONALERT, IntPtr.Zero, 0u);
					if (status != STATUS_SUCCESS)
					{
						return null; //ThrowIo(status, $"NtCreateFile('{directoryPath}')");
					}
				}

				try
				{
					List<Entry> entries = new(256);

					if (bufferSize <= 128 * 1024)
					{
						byte* buf = stackalloc byte[bufferSize];
						EnumerateWithBuffer(dirHandle, buf, bufferSize, entries);
					}
					else
					{
						byte[] rented = ArrayPool<byte>.Shared.Rent(bufferSize);
						try
						{
							fixed (byte* buf = rented)
							{
								EnumerateWithBuffer(dirHandle, buf, bufferSize, entries);
							}
						}
						finally
						{
							ArrayPool<byte>.Shared.Return(rented);
						}
					}

					return entries;
				}
				finally
				{
					if (dirHandle != IntPtr.Zero)
					{
						int v = NtClose(dirHandle);
					}
				}
			}
			else
			{
				List<Entry> entries = new(256);
				foreach (FileSystemInfo entry in (new DirectoryInfo(directoryPath)).EnumerateFileSystemInfos())
				{
					long size = (entry.Attributes & FileAttributes.Directory) == 0 ? ((FileInfo)entry).Length : 0;
					entries.Add(new Entry(entry.Name, size, entry.Attributes, entry.CreationTimeUtc, entry.LastWriteTimeUtc));
				}

				return entries;
			}
		}

		/// <summary>
		/// Try to get file from path
		/// </summary>
		/// <param name="path"></param>
		/// <param name="entry"></param>
		/// <returns>Returns false if file is not found or is a directory</returns>
		public static bool TryGetFileFromPath(string path, out Entry entry)
		{
			if (OperatingSystem.IsWindows())
			{
				return TryGetEntryFromPath(path, out entry) && (entry.Attributes & FileAttributes.Directory) == 0;
			}
			FileInfo info = new(path);
			if (!info.Exists)
			{
				entry = default;
				return false;
			}
			entry = new("", info.Length, info.Attributes, info.CreationTimeUtc, info.LastWriteTimeUtc);
			return true;
		}

		/// <summary>
		/// Try to get directory from path
		/// </summary>
		/// <param name="path"></param>
		/// <param name="entry"></param>
		/// <returns>Returns false if directory is not found or is a file</returns>
		public static bool TryGetDirectoryFromPath(string path, out Entry entry)
		{
			if (OperatingSystem.IsWindows())
			{
				return TryGetEntryFromPath(path, out entry) && (entry.Attributes & FileAttributes.Directory) != 0;
			}
			DirectoryInfo info = new(path);
			if (!info.Exists)
			{
				entry = default;
				return false;
			}
			entry = new("", 0, info.Attributes, info.CreationTimeUtc, info.LastWriteTimeUtc);
			return true;
		}

		private static bool TryGetEntryFromPath(string path, out Entry entry)
		{
			if (!GetFileAttributesExW(path, GET_FILEEX_INFO_LEVELS.GetFileExInfoStandard, out WIN32_FILE_ATTRIBUTE_DATA data))
			{
				entry = default;
				return false;
			}

			FileAttributes attrs = (FileAttributes)data.dwFileAttributes;
			long size = ((long)data.nFileSizeHigh << 32) | data.nFileSizeLow;
			DateTime c = DateTimeOffset.FromFileTime(((long)data.ftCreationTime.dwHighDateTime << 32) | data.ftCreationTime.dwLowDateTime).UtcDateTime;
			DateTime w = DateTimeOffset.FromFileTime(((long)data.ftLastWriteTime.dwHighDateTime << 32) | data.ftLastWriteTime.dwLowDateTime).UtcDateTime;

			entry = new("", size, attrs, c, w);
			return true;
		}

		private static unsafe void EnumerateWithBuffer(IntPtr dirHandle, byte* buf, int bufferSize, List<Entry> entries)
		{
			bool restart = true;

			while (true)
			{
				int status = NtQueryDirectoryFile(dirHandle, IntPtr.Zero, IntPtr.Zero, IntPtr.Zero, out IO_STATUS_BLOCK iosbLocal, buf, (uint)bufferSize, FILE_INFORMATION_CLASS.FileFullDirectoryInformation, false, null, restart);

				if (status == STATUS_NO_MORE_FILES)
				{
					break;
				}

				if (status != STATUS_SUCCESS)
				{
					ThrowIo(status, "NtQueryDirectoryFile");
				}

				byte* cursor = buf;
				while (true)
				{
					FILE_FULL_DIR_INFORMATION* info = (FILE_FULL_DIR_INFORMATION*)cursor;

					int nameBytes = checked((int)info->FileNameLength);
					char* namePtr = &info->FileName;
					int nameChars = nameBytes / 2;

					if (!(nameChars == 1 && namePtr[0] == '.') && !(nameChars == 2 && namePtr[0] == '.' && namePtr[1] == '.'))
					{
						string name = new(namePtr, 0, nameChars);
						entries.Add(new Entry(name, info->EndOfFile, (FileAttributes)info->FileAttributes, DateTimeOffset.FromFileTime(info->CreationTime).UtcDateTime, DateTimeOffset.FromFileTime(info->LastWriteTime).UtcDateTime));
					}

					if (info->NextEntryOffset == 0)
					{
						break;
					}

					cursor += info->NextEntryOffset;
				}

				restart = false;
			}
		}

		private static void ThrowIo(int ntstatus, string op)
		{
			int win32 = RtlNtStatusToDosError(ntstatus);
			Win32Exception ex = new(unchecked(win32));
			throw new IOException($"{op} Failed: 0x{ntstatus:X8} ({ex.Message})", ex);
		}

		private const int STATUS_SUCCESS = 0x00000000;
		private const int STATUS_NO_MORE_FILES = unchecked((int)0x80000006);

		private const uint OBJ_CASE_INSENSITIVE = 0x00000040;

		private const uint FILE_LIST_DIRECTORY = 0x0001;
		private const uint FILE_SHARE_READ = 0x00000001;
		private const uint FILE_SHARE_WRITE = 0x00000002;
		private const uint FILE_SHARE_DELETE = 0x00000004;
		private const uint FILE_OPEN = 0x00000001;
		private const uint FILE_DIRECTORY_FILE = 0x00000001;
		private const uint FILE_SYNCHRONOUS_IO_NONALERT = 0x00000020;
		private const uint SYNCHRONIZE = 0x00100000;

		[DllImport("ntdll.dll")]
		private static extern int NtClose(IntPtr handle);

		[DllImport("ntdll.dll", ExactSpelling = true)]
		private static extern int RtlNtStatusToDosError(int status);

		[DllImport("ntdll.dll", ExactSpelling = true)]
		private static extern unsafe int NtCreateFile(
			out IntPtr FileHandle,
			uint DesiredAccess,
			ref OBJECT_ATTRIBUTES ObjectAttributes,
			out IO_STATUS_BLOCK IoStatusBlock,
			ref long AllocationSize,
			uint FileAttributes,
			uint ShareAccess,
			uint CreateDisposition,
			uint CreateOptions,
			IntPtr EaBuffer,
			uint EaLength);

		[DllImport("ntdll.dll", ExactSpelling = true)]
		private static extern unsafe int NtQueryDirectoryFile(
			IntPtr FileHandle,
			IntPtr Event,
			IntPtr ApcRoutine,
			IntPtr ApcContext,
			out IO_STATUS_BLOCK IoStatusBlock,
			void* FileInformation,
			uint Length,
			FILE_INFORMATION_CLASS FileInformationClass,
			[MarshalAs(UnmanagedType.U1)] bool ReturnSingleEntry,
			UNICODE_STRING* FileName,
			[MarshalAs(UnmanagedType.U1)] bool RestartScan);

		[DllImport("kernel32.dll", CharSet = CharSet.Unicode, SetLastError = true)]
		private static extern bool GetFileAttributesExW(string lpFileName, GET_FILEEX_INFO_LEVELS fInfoLevelId, out WIN32_FILE_ATTRIBUTE_DATA fileData);

		private enum GET_FILEEX_INFO_LEVELS : int
		{
			GetFileExInfoStandard = 0
		}

		[StructLayout(LayoutKind.Sequential)]
		private struct FILETIME
		{
			public uint dwLowDateTime;
			public uint dwHighDateTime;
		}

		[StructLayout(LayoutKind.Sequential)]
		private struct WIN32_FILE_ATTRIBUTE_DATA
		{
			public uint dwFileAttributes;
			public FILETIME ftCreationTime;
			public FILETIME ftLastAccessTime;
			public FILETIME ftLastWriteTime;
			public uint nFileSizeHigh;
			public uint nFileSizeLow;
		}

		[StructLayout(LayoutKind.Sequential)]
		private unsafe struct UNICODE_STRING
		{
			public ushort Length;
			public ushort MaximumLength;
			public char* Buffer;
		}

		[StructLayout(LayoutKind.Sequential)]
		private unsafe struct OBJECT_ATTRIBUTES
		{
			public uint Length;
			public IntPtr RootDirectory;
			public UNICODE_STRING* ObjectName;
			public uint Attributes;
			public IntPtr SecurityDescriptor;
			public IntPtr SecurityQualityOfService;
		}

		[StructLayout(LayoutKind.Sequential)]
		private struct IO_STATUS_BLOCK
		{
			public IntPtr Status;
			public nuint Information;
		}

		private enum FILE_INFORMATION_CLASS : int
		{
			FileFullDirectoryInformation = 2,
		}

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		private struct FILE_FULL_DIR_INFORMATION
		{
			public uint NextEntryOffset;
			public uint FileIndex;
			public long CreationTime;
			public long LastAccessTime;
			public long LastWriteTime;
			public long ChangeTime;
			public long EndOfFile;
			public long AllocationSize;
			public uint FileAttributes;
			public uint FileNameLength;
			public uint EaSize;
			public char FileName;
		}
	}
}
