// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Security;
using EpicGames.Core;

namespace UnrealBuildBase
{
	/// <summary>
	/// Stores the state of a directory. May or may not exist.
	/// </summary>
	public class DirectoryItem
	{
		/// <summary>
		/// Full path to the directory on disk
		/// </summary>
		public required DirectoryReference Location { get; init; }

		/// <summary>
		/// The name of this directory
		/// </summary>
		public required string Name { get; init; }

		/// <summary>
		/// Accessor for map of name to subdirectory item
		/// </summary>
		Dictionary<string, DirectoryItem> Directories => _cache.Value.Item1;

		/// <summary>
		/// Accessor for map of name to file
		/// </summary>
		Dictionary<string, FileItem> Files => _cache.Value.Item2;

		/// <summary>
		/// The full name of this directory
		/// </summary>
		public string FullName => Location.FullName;

		/// <summary>
		/// Whether the directory exists or not
		/// </summary>
		public bool Exists => Update().Exists;

		/// <summary>
		/// The last write time of the file.
		/// </summary>
		public DateTime LastWriteTimeUtc => Update().LastWriteTimeUtc;

		/// <summary>
		/// The creation time of the file.
		/// </summary>
		public DateTime CreationTimeUtc => Update().CreationTimeUtc;

		/// <summary>
		/// Gets the parent directory item
		/// </summary>
		public DirectoryItem? GetParentDirectoryItem()
		{
			DirectoryReference? parent = Location.ParentDirectory;
			return parent != null ? GetItemByDirectoryReference(parent) : null;
		}

		/// <summary>
		/// Gets a new directory item by combining the existing directory item with the given path fragments
		/// </summary>
		/// <param name="BaseDirectory">Base directory to append path fragments to</param>
		/// <param name="Fragments">The path fragments to append</param>
		/// <returns>Directory item corresponding to the combined path</returns>
		public static DirectoryItem Combine(DirectoryItem BaseDirectory, params string[] Fragments)
			=> GetItemByDirectoryReference(DirectoryReference.Combine(BaseDirectory.Location, Fragments));

		/// <summary>
		/// Finds or creates a directory item from its location
		/// </summary>
		/// <param name="Location">Path to the directory</param>
		/// <returns>The directory item for this location</returns>
		public static DirectoryItem GetItemByPath(string Location) => GetItemByDirectoryReference(new(Location));

		/// <summary>
		/// Finds or creates a directory item from its location
		/// </summary>
		/// <param name="Location">Path to the directory</param>
		/// <returns>The directory item for this location</returns>
		public static DirectoryItem GetItemByDirectoryReference(DirectoryReference Location)
			=> LocationToItem.GetOrAdd(Location, static loc => new DirectoryItem(loc, loc.GetDirectoryName(), null));

		/// <summary>
		/// Finds or creates a directory item from a DirectoryInfo object
		/// </summary>
		/// <param name="Info">Path to the directory</param>
		/// <returns>The directory item for this location</returns>
		public static DirectoryItem GetItemByDirectoryInfo(DirectoryInfo Info) => GetItemByDirectoryReference(new(Info));

		/// <summary>
		/// Reset the contents of the directory and allow them to be fetched again
		/// </summary>
		public void ResetCachedInfo()
		{
			_info = null;

			if (_cache.IsValueCreated)
			{
				(Dictionary<string, DirectoryItem> dirs, Dictionary<string, FileItem> files) = _cache.Value;
				foreach (DirectoryItem SubDirectory in dirs.Values)
				{
					SubDirectory.ResetCachedInfo();
				}
				foreach (FileItem File in files.Values)
				{
					File.ResetCachedInfo();
				}
				_cache = new(Scan);
			}
		}

		/// <summary>
		/// This function can be used to force the cache to be populated with files that do not yet exist.
		/// Useful when wanting to run things in parallel and one thread write files while the other uses the list of files in the directory.
		/// </summary>
		/// <param name="files"></param>
		public void PrepopulateCache(List<string> files)
		{
			Dictionary<string, FileItem> cacheFiles = new(files.Count, FileSystemReference.Comparer);
			foreach (string file in files)
			{
				FileReference fileRef = FileReference.Combine(Location, file);
				FileItem fileItem = FileItem.GetItemByFileReference(fileRef);
				fileItem.Update(new() { Attributes = FileAttributes.Normal, Size = long.MaxValue }, true);
				cacheFiles.TryAdd(file, fileItem);
			}

			_cache = new((new(FileSystemReference.Comparer), cacheFiles));

			// Update ourself and all our ancestors - if we're pretending we have extant files then they must all exist too.
			DirectoryItem? toUpdate = this;
			DirectoryItem? childDir = null;
			while (toUpdate is not null)
			{
				if (childDir is not null)
				{
					lock (toUpdate)
					{
						toUpdate.Directories[childDir.Name] = childDir;
					}
				}
				// This has to come after the above because it triggers Scan which resets Exists back to false.
				if (!toUpdate.Exists)
				{
					toUpdate.Update(new DirectoryEnumerator.Entry { Attributes = FileAttributes.Directory }, force: true);
				}

				childDir = toUpdate;
				toUpdate = toUpdate.GetParentDirectoryItem();
			}
		}

		/// <summary>
		/// Resets the cached info, if the DirectoryInfo is not found don't create a new entry
		/// </summary>
		public static void ResetCachedInfo(string Path)
		{
			if (LocationToItem.TryGetValue(new DirectoryReference(Path), out DirectoryItem? Result))
			{
				Result.ResetCachedInfo();
			}
		}

		/// <summary>
		/// Resets all cached directory info. Significantly reduces performance; do not use unless strictly necessary.
		/// </summary>
		public static void ResetAllCachedInfo_SLOW()
		{
			LocationToItem.Values.AsParallel().ForAll(Item =>
			{
				Item._info = null;
				Item._cache = new(Item.Scan);
			});
			FileItem.ResetAllCachedInfo_SLOW();
		}

		/// <summary>
		/// Caches the subdirectories of this directories
		/// </summary>
		public bool CacheDirectories() => _cache.Value.Item1 != null;

		/// <summary>
		/// Enumerates all the subdirectories
		/// </summary>
		/// <returns>Sequence of subdirectory items</returns>
		public IEnumerable<DirectoryItem> EnumerateDirectories()
		{
			CacheDirectories();
			return Directories.Values;
		}

		/// <summary>
		/// Attempts to get a sub-directory by name
		/// </summary>
		/// <param name="Name">Name of the directory</param>
		/// <param name="OutDirectory">If successful receives the matching directory item with this name</param>
		/// <returns>True if the file exists, false otherwise</returns>
		public bool TryGetDirectory(string Name, [NotNullWhen(true)] out DirectoryItem? OutDirectory)
		{
			if (Name.Length > 0 && Name[0] == '.')
			{
				if (Name.Length == 1)
				{
					OutDirectory = this;
					return true;
				}
				else if (Name.Length == 2 && Name[1] == '.')
				{
					OutDirectory = GetParentDirectoryItem();
					return OutDirectory != null;
				}
			}

			CacheDirectories();
			return Directories.TryGetValue(Name, out OutDirectory);
		}

		[SetsRequiredMembers]
		DirectoryItem(DirectoryReference location, string name, Info? info)
		{
			Location = location;
			Name = name;
			_info = info;

			_cache = new(Scan);
		}

		Info Update()
		{
			if (_info != null)
			{
				return _info;
			}
			Info newInfo = new();
			if (DirectoryEnumerator.TryGetDirectoryFromPath(FullName, out DirectoryEnumerator.Entry entry))
			{
				newInfo.Exists = true;
				newInfo.LastWriteTimeUtc = entry.LastWriteTimeUtc;
				newInfo.CreationTimeUtc = entry.CreationTimeUtc;
			}
			_info = newInfo;
			return newInfo;
		}

		internal void Update(DirectoryEnumerator.Entry entry, bool force)
		{
			if (_info == null || force)
			{
				_info = new() { Exists = entry.Attributes != FileAttributes.None, LastWriteTimeUtc = entry.LastWriteTimeUtc, CreationTimeUtc = entry.CreationTimeUtc };
			}
		}

		/// <summary>
		/// Scans the directory for directories and files, used for lazy initialization.
		/// </summary>
		/// <returns></returns>
		(Dictionary<string, DirectoryItem>, Dictionary<string, FileItem>) Scan()
		{
			if (_info != null && !_info.Exists)
			{
				return (new(FileSystemReference.Comparer), new(FileSystemReference.Comparer));
			}

			try
			{
				List<DirectoryEnumerator.Entry>? entries = DirectoryEnumerator.Enumerate(FullName);

				if (entries == null)
				{
					_info = new Info() { Exists = false };
					return (new(FileSystemReference.Comparer), new(FileSystemReference.Comparer));
				}
				int dirCount = 0;
				foreach (DirectoryEnumerator.Entry entry in entries)
				{
					if ((entry.Attributes & FileAttributes.Directory) != 0)
					{
						++dirCount;
					}
				}

				Dictionary<string, DirectoryItem>? newDirs = new(dirCount, FileSystemReference.Comparer);
				Dictionary<string, FileItem>? newFiles = new(entries.Count - dirCount, FileSystemReference.Comparer);

				foreach (DirectoryEnumerator.Entry entry in entries)
				{
					if ((entry.Attributes & FileAttributes.Directory) != 0)
					{
						Info info = new() { Exists = true, LastWriteTimeUtc = entry.LastWriteTimeUtc, CreationTimeUtc = entry.CreationTimeUtc };
						DirectoryReference loc = DirectoryReference.Combine(Location, entry.Name);
						DirectoryItem dir = LocationToItem.GetOrAdd(loc, loc => new DirectoryItem(loc, entry.Name, info));

						// There are folders in linux sdk that has files with same name but different casing.
						// Ideally FileReference.Comparer should be case sensitive on linux/mac but I don't dare changing that right now
						newDirs.TryAdd(entry.Name, dir);
					}
					else
					{
						FileReference loc = FileReference.Combine(Location, entry.Name);
						FileItem file = FileItem.UniqueSourceFileMap.GetOrAdd(loc, loc => new FileItem(loc, entry.Name, new() { Attributes = entry.Attributes, Length = entry.Size, LastWriteTimeUtc = entry.LastWriteTimeUtc, CreationTimeUtc = entry.CreationTimeUtc }, this));
						file.Update(entry, false);

						// There are folders in linux sdk that has files with same name but different casing.
						// Ideally FileReference.Comparer should be case sensitive on linux/mac but I don't dare changing that right now
						newFiles.TryAdd(entry.Name, file);
					}
				}

				return (newDirs, newFiles);
			}
			catch (DirectoryNotFoundException)
			{
			}
			catch (SecurityException)
			{
			}
			catch (UnauthorizedAccessException)
			{
			}

			return ([], []);
		}

		/// <summary>
		/// Caches the files in this directory
		/// </summary>
		public bool CacheFiles() => _cache.Value.Item2 != null;

		/// <summary>
		/// Enumerates all the files
		/// </summary>
		/// <returns>Sequence of FileItems</returns>
		public IEnumerable<FileItem> EnumerateFiles()
		{
			CacheFiles();
			return Files.Values;
		}

		/// <summary>
		/// Check if this directory contains any files
		/// </summary>
		/// <param name="searchOption">Directory search options</param>
		/// <returns>True if this directory has files</returns>
		public bool ContainsFiles(SearchOption searchOption = SearchOption.TopDirectoryOnly)
		{
			return searchOption == SearchOption.TopDirectoryOnly ? EnumerateFiles().Any(x => x.Exists) : (EnumerateFiles().Any(x => x.Exists) || EnumerateDirectories().Any(x => x.ContainsFiles(searchOption)));
		}

		/// <summary>
		/// Attempts to get a file from this directory by name. Unlike creating a file item and checking whether it exists, this will
		/// not create a permanent FileItem object if it does not exist.
		/// </summary>
		/// <param name="Name">Name of the file</param>
		/// <param name="OutFile">If successful receives the matching file item with this name</param>
		/// <returns>True if the file exists, false otherwise</returns>
		public bool TryGetFile(string Name, [NotNullWhen(true)] out FileItem? OutFile)
		{
			CacheFiles();
			return Files.TryGetValue(Name, out OutFile);
		}

		/// <summary>
		/// Formats this object as a string for debugging
		/// </summary>
		/// <returns>Location of the directory</returns>
		public override string ToString() => Location.ToString();

		/// <summary>
		/// Sort file items by full path
		/// </summary>
		public static readonly IComparer<DirectoryItem?> SortByPath = new SortByPathComparer();

		/// <summary>
		/// Writes out all the enumerated files full names sorted to OutFile
		/// </summary>
		public static void WriteDebugFileWithAllEnumeratedFiles(string OutFile)
		{
			SortedSet<string> AllFiles = [];
			foreach (DirectoryItem Item in LocationToItem.Values)
			{
				if (Item.Files != null)
				{
					foreach (FileItem File in Item.EnumerateFiles())
					{
						AllFiles.Add(File.FullName);
					}
				}
			}
			File.WriteAllLines(OutFile, AllFiles);
		}

		class Info
		{
			internal bool Exists;
			internal DateTime LastWriteTimeUtc;
			internal DateTime CreationTimeUtc;
		}

		Info? _info;
		Lazy<(Dictionary<string, DirectoryItem>, Dictionary<string, FileItem>)> _cache;
		static ConcurrentDictionary<DirectoryReference, DirectoryItem> LocationToItem = [];

		class SortByPathComparer : IComparer<DirectoryItem?>
		{
			public int Compare(DirectoryItem? a, DirectoryItem? b)
			{
				if (a != null && b != null)
				{
					return a.Location.CompareTo(b.Location);
				}
				if (a == b)
				{
					return 0;
				}
				if (a is null)
				{
					return -1;
				}
				return 1;
			}
		}
	}

	/// <summary>
	/// Helper functions for serialization
	/// </summary>
	public static class DirectoryItemExtensionMethods
	{
		/// <summary>
		/// Read a directory item from a binary archive
		/// </summary>
		/// <param name="Reader">Reader to serialize data from</param>
		/// <returns>Instance of the serialized directory item</returns>
		public static DirectoryItem? ReadDirectoryItem(this BinaryArchiveReader Reader)
		{
			// Use lambda that doesn't require anything to be captured thus eliminating an allocation.
			return Reader.ReadObjectReference(Reader => DirectoryItem.GetItemByDirectoryReference(Reader.ReadDirectoryReferenceNotNull()));
		}

		/// <summary>
		/// Write a directory item to a binary archive
		/// </summary>
		/// <param name="Writer">Writer to serialize data to</param>
		/// <param name="DirectoryItem">Directory item to write</param>
		public static void WriteDirectoryItem(this BinaryArchiveWriter Writer, DirectoryItem DirectoryItem)
		{
			// Use lambda that doesn't require anything to be captured thus eliminating an allocation.
			Writer.WriteObjectReference(DirectoryItem, (Writer, DirectoryItem) => Writer.WriteDirectoryReference(DirectoryItem.Location));
		}

		/// <summary>
		/// Writes a directory reference  to a binary archive
		/// </summary>
		/// <param name="Writer">The writer to output data to</param>
		/// <param name="Directory">The item to write</param>
		public static void WriteCompactDirectoryReference(this BinaryArchiveWriter Writer, DirectoryReference Directory)
		{
			DirectoryItem Item = DirectoryItem.GetItemByDirectoryReference(Directory);
			Writer.WriteDirectoryItem(Item);
		}

		/// <summary>
		/// Reads a directory reference from a binary archive
		/// </summary>
		/// <param name="Reader">Reader to serialize data from</param>
		/// <returns>New directory reference instance</returns>
		public static DirectoryReference ReadCompactDirectoryReference(this BinaryArchiveReader Reader)
		{
			return Reader.ReadDirectoryItem()!.Location;
		}
	}
}
