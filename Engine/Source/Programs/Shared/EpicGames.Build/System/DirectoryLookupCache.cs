// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using EpicGames.Core;
using System.Linq;

namespace UnrealBuildBase
{
	public static class DirectoryLookupCache
	{
		public static bool FileExists(FileReference File)
		{
			return FileItem.GetItemByFileReference(File).Exists;
		}

		public static bool DirectoryExists(DirectoryReference Directory)
		{
			return DirectoryItem.GetItemByDirectoryReference(Directory).Exists;
		}

		public static bool DirectoryExistsAndContainsFiles(DirectoryReference Directory, SearchOption searchOption = SearchOption.TopDirectoryOnly)
		{
			DirectoryItem item = DirectoryItem.GetItemByDirectoryReference(Directory);
			return item.Exists && item.ContainsFiles(searchOption);
		}

		public static IEnumerable<FileReference> EnumerateFiles(DirectoryReference Directory)
		{
			return DirectoryItem.GetItemByDirectoryReference(Directory).EnumerateFiles().Select(x => x.Location);
		}

		public static IEnumerable<DirectoryReference> EnumerateDirectories(DirectoryReference Directory)
		{
			return DirectoryItem.GetItemByDirectoryReference(Directory).EnumerateDirectories().Select(x => x.Location);
		}

		public static void InvalidateCachedDirectory(DirectoryReference Directory)
		{
			DirectoryItem.GetItemByDirectoryReference(Directory).ResetCachedInfo();
		}
	}
}
