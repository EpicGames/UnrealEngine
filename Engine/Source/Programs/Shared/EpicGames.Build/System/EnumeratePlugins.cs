// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;

namespace UnrealBuildBase
{
	public class PluginsBase
	{
		/// <summary>
		/// Cache of plugin filenames under each directory
		/// </summary>
		static ConcurrentDictionary<DirectoryReference, Lazy<List<FileReference>>> PluginFileCache = new();

		/// <summary>
		/// Cache of all UBT plugins that have been built.  When generating projects, there is a good chance that a UBT project
		/// might need to be rebuilt.
		/// </summary>
		static Dictionary<FileReference, FileReference> UbtPluginFileCache = [];

		/// <summary>
		/// Invalidate cached plugin data so that we can pickup new things
		/// Warning: Will make subsequent plugin lookups and directory scans slow until the caches are repopulated
		/// </summary>
		public static void InvalidateCache_SLOW()
		{
			PluginFileCache = new();
			DirectoryItem.ResetAllCachedInfo_SLOW();
		}

		/// <summary>
		/// Enumerates all the plugin files available to the given project
		/// </summary>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <returns>List of project files</returns>
		public static List<FileReference> EnumeratePlugins(FileReference? ProjectFile)
		{
			List<DirectoryReference> BaseDirs = [.. Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Plugins")];
			if (ProjectFile != null)
			{
				BaseDirs.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Plugins"));
				BaseDirs.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Mods"));
			}
			return [.. BaseDirs.SelectMany(x => EnumeratePlugins(x))];
		}

		/// <summary>
		/// Find paths to all the plugins under a given parent directory (recursively)
		/// </summary>
		/// <param name="ParentDirectory">Parent directory to look in. Plugins will be found in any *subfolders* of this directory.</param>
		public static List<FileReference> EnumeratePlugins(DirectoryReference ParentDirectory)
		{
			return PluginFileCache.GetOrAdd(ParentDirectory, new Lazy<List<FileReference>>(() =>
			{
				List<FileReference> FileNames = [];

				DirectoryItem ParentDirectoryItem = DirectoryItem.GetItemByDirectoryReference(ParentDirectory);
				if (ParentDirectoryItem.Exists)
				{
					ParallelQueue Queue = new();
					EnumeratePluginsInternal(ParentDirectoryItem, FileNames, Queue);
					Queue.Drain();
				}

				// Sort the filenames to ensure that the plugin order is deterministic; otherwise response files will change with each build.
				FileNames.Sort();
				return FileNames;
			})).Value;
		}

		/// <summary>
		/// Find paths to all the plugins under a given parent directory (recursively)
		/// </summary>
		/// <param name="ParentDirectory">Parent directory to look in. Plugins will be found in any *subfolders* of this directory.</param>
		/// <param name="FileNames">List of filenames. Will have all the discovered .uplugin files appended to it.</param>
		/// <param name="Queue">Queue for tasks to be executed</param>
		static void EnumeratePluginsInternal(DirectoryItem ParentDirectory, List<FileReference> FileNames, ParallelQueue Queue)
		{
			foreach (DirectoryItem ChildDirectory in ParentDirectory.EnumerateDirectories())
			{
				bool bSearchSubDirectories = true;
				foreach (FileItem PluginFile in ChildDirectory.EnumerateFiles())
				{
					if (PluginFile.HasExtension(".uplugin"))
					{
						lock (FileNames)
						{
							FileNames.Add(PluginFile.Location);
						}
						bSearchSubDirectories = false;
					}
					else if (PluginFile.Name == ".ubtignore")
					{
						bSearchSubDirectories = false;
					}
				}

				if (bSearchSubDirectories)
				{
					Queue.Enqueue(() => EnumeratePluginsInternal(ChildDirectory, FileNames, Queue));
				}
			}
		}

		/// <summary>
		/// Enumerates all the UBT plugin files available to the given project
		/// </summary>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <returns>The collection of UBT project files</returns>
		public static List<FileReference> EnumerateUbtPlugins(FileReference? ProjectFile)
		{
			List<DirectoryReference> PluginDirectories =
			[
				.. Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Build"),
				.. Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source"),
				.. Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Plugins"),
			];
			if (ProjectFile != null)
			{
				PluginDirectories.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Build"));
				PluginDirectories.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Source"));
				PluginDirectories.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Plugins"));
				PluginDirectories.AddRange(Unreal.GetExtensionDirs(ProjectFile.Directory, "Mods"));
			}
			Rules.Populate(PluginDirectories);
			List<FileReference> Plugins = Rules.Get(PluginDirectories, Rules.RulesFileType.UbtPlugin);
			return Plugins;
		}

		/// <summary>
		/// Build the given collection of plugins
		/// </summary>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <param name="FoundPlugins">Collection of plugins to build</param>
		/// <param name="CustomGlobalProperties">Additional MSBuild global properties to set while building the plugins</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Plugins">Collection of built plugins</param>
		/// <returns>True if the plugins compiled</returns>
		public static bool BuildUbtPlugins(FileReference? ProjectFile, IEnumerable<FileReference> FoundPlugins,
			IReadOnlyDictionary<string, IEnumerable<string>>? CustomGlobalProperties, ILogger Logger, out (FileReference ProjectFile, FileReference TargetAssembly)[]? Plugins)
		{
			lock (UbtPluginFileCache)
			{

				bool bBuildSuccess = true;
				Plugins = null;

				// Collect the list of plugins already build
				List<FileReference> Built = [.. FoundPlugins.Where(P => UbtPluginFileCache.ContainsKey(P))];

				// Collect the list of plugins not build
				List<FileReference> NotBuilt = [.. FoundPlugins.Where(P => !UbtPluginFileCache.ContainsKey(P))];

				// If we have anything left to build, then build it
				if (NotBuilt.Count > 0)
				{
					if (Unreal.IsEngineInstalled())
					{
						bBuildSuccess = BuildUbtPluginsInternal(PluginSet.EngineOnly, ProjectFile, Built, NotBuilt, CustomGlobalProperties, Logger);
						bBuildSuccess |= BuildUbtPluginsInternal(PluginSet.ProjectOnly, ProjectFile, Built, NotBuilt, CustomGlobalProperties, Logger);
					}
					else
					{
						bBuildSuccess =
							BuildUbtPluginsInternal(PluginSet.Both, ProjectFile, Built, NotBuilt, CustomGlobalProperties, Logger);
					}
				}

				Plugins = [.. Built.Where(P => UbtPluginFileCache.ContainsKey(P)).Select(P => (P, UbtPluginFileCache[P])).OrderBy(X => X.P.FullName)];
				return bBuildSuccess;
			}
		}

		private enum PluginSet
		{
			EngineOnly,
			ProjectOnly,
			Both,
		}

		private static bool BuildUbtPluginsInternal(PluginSet PluginSet, FileReference? ProjectFile,
			List<FileReference> Built, IEnumerable<FileReference> NotBuilt, IReadOnlyDictionary<string, IEnumerable<string>>? CustomGlobalProperties, ILogger Logger)
		{
			// Collect the work to be done
			IEnumerable<FileReference>? ToBuild = null;
			List<DirectoryReference> BaseDirectories = [];
			CompileScriptModule.BuildFlags BuildFlags = CompileScriptModule.BuildFlags.UseBuildRecords;
			switch (PluginSet)
			{
				case PluginSet.EngineOnly:
					BaseDirectories.Add(Unreal.EngineDirectory);
					BuildFlags |= CompileScriptModule.BuildFlags.NoCompile | CompileScriptModule.BuildFlags.ErrorOnMissingTarget;
					ToBuild = NotBuilt.Where(P => P.IsUnderDirectory(Unreal.EngineDirectory));
					break;
				case PluginSet.ProjectOnly:
					if (ProjectFile != null)
					{
						BaseDirectories.Add(ProjectFile.Directory);
						ToBuild = NotBuilt.Where(P => !P.IsUnderDirectory(Unreal.EngineDirectory));
					}
					break;
				case PluginSet.Both:
					BaseDirectories.Add(Unreal.EngineDirectory);
					if (ProjectFile != null)
					{
						BaseDirectories.Add(ProjectFile.Directory);
					}
					ToBuild = NotBuilt;
					break;
			}

			// Just return if there is nothing
			if (ToBuild == null || !ToBuild.Any() || BaseDirectories.Count == 0)
			{
				return true;
			}

			// Build the things needing to be built
			bool bBuildSuccess = true;
			Dictionary<FileReference, CsProjBuildRecordEntry>? BuiltPlugins =
				CompileScriptModule.Build(Rules.RulesFileType.UbtPlugin, [.. NotBuilt],
				BaseDirectories, CustomGlobalProperties, CompileScriptModule.BuildFlags.UseBuildRecords, out bBuildSuccess, Logger);

			// Add any built plugins back into the cache
			foreach (KeyValuePair<FileReference, CsProjBuildRecordEntry> BuiltPlugin in BuiltPlugins)
			{
				Built.Add(BuiltPlugin.Key);
				UbtPluginFileCache.Add(BuiltPlugin.Key, CompileScriptModule.GetTargetPath(BuiltPlugin));
			}
			return bBuildSuccess;
		}
	}
}
