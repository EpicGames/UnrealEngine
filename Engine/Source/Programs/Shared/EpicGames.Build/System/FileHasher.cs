// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// FileHasher constructor
	/// </summary>
	/// <param name="logger">Optional logger</param>
	public class FileHasher(ILogger? logger = null)
	{
		private readonly struct CachedDigest(FileItem item, IoHash digest)
		{
			[JsonPropertyName("l")]
			public long Length { get; init; } = item.Length;
			[JsonPropertyName("w")]
			public long LastWriteTimeUtc { get; init; } = item.LastWriteTimeUtc.Ticks;
			[JsonPropertyName("d")]
			public IoHash Digest { get; init; } = digest;

			/// <summary>
			/// Check if a digest is up to date for a FileItem
			/// </summary>
			/// <param name="item">The FileItem to check</param>
			/// <returns>If the digest is up to date</returns>
			public bool UpToDate(FileItem item) => item.Exists && Length == item.Length && LastWriteTimeUtc == item.LastWriteTimeUtc.Ticks;
		}

		// Dictionary containing cached digests, to prevent rehashing data if the file is unchanged
		ConcurrentDictionary<string, CachedDigest> CachedDigests = new();

		/// <summary>
		/// Saves cached digests to disk.
		/// </summary>
		/// <param name="location">The location to save the digest data (json)</param>
		public async Task Save(FileReference location)
		{
			using FileStream stream = FileReference.Open(location, FileMode.Create);
			await JsonSerializer.SerializeAsync(stream, new SortedDictionary<string, CachedDigest>(CachedDigests));
			logger?.LogDebug("Saved {Entries} cached digest entries", CachedDigests.Count);
		}

		/// <summary>
		/// Loads cached digests from disk, replacing existing cache.
		/// </summary>
		/// <param name="location">The location to load the digest data from (json)</param>
		public async Task Load(FileReference location)
		{
			if (!FileReference.Exists(location))
			{
				return;
			}

			using FileStream stream = FileReference.Open(location, FileMode.Open, FileAccess.Read, FileShare.Read);
			ConcurrentDictionary<string, CachedDigest>? loadedDigests = await JsonSerializer.DeserializeAsync<ConcurrentDictionary<string, CachedDigest>>(stream);
			if (loadedDigests != null)
			{
				CachedDigests = loadedDigests;
				logger?.LogDebug("Loaded {Entries} cached digest entries", CachedDigests.Count);
			}
		}

		/// <summary>
		/// Purges stale cached digests.
		/// </summary>
		public void PurgeStale()
		{
			int count = CachedDigests.Count;
			IEnumerable<KeyValuePair<string, CachedDigest>> valid = CachedDigests.Where((x) => x.Value.UpToDate(FileItem.GetItemByFileReference(new(x.Key))));
			CachedDigests = new(valid);
			logger?.LogDebug("Purged {Entries} stale cached digest entries", count - CachedDigests.Count);
		}

		/// <summary>
		/// Get the IoHash digest of a file's contents.
		/// </summary>
		/// <param name="item">The FileItem to digest</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <remarks>A cache is maintained of digests and a file will not be rehashed if unchanged.</remarks>
		/// <returns>The IoHash digest, or IoHash.Zero if the file doesn't exist.</returns>
		public async Task<IoHash> GetDigestAsync(FileItem item, CancellationToken cancellationToken = default)
		{
			if (CachedDigests.TryGetValue(item.FullName, out CachedDigest CachedHash))
			{
				if (CachedHash.UpToDate(item))
				{
					// Hash already calculated
					return CachedHash.Digest;
				}
			}

			if (!item.Exists)
			{
				return IoHash.Zero;
			}

			CachedDigest digest = await ComputeDigest(item, logger, cancellationToken);
			return CachedDigests.AddOrUpdate(item.FullName, digest, (key, oldValue) => digest).Digest;
		}

		/// <summary>
		/// In systems where the IoHash is already available, this allows it to be updated directly and
		/// not recomputed.
		/// </summary>
		/// <param name="item">The updated file item information</param>
		/// <param name="hash">The updated hash</param>
		public void SetDigest(FileItem item, IoHash hash)
		{
			CachedDigest digest = new(item, hash);
			CachedDigests.AddOrUpdate(item.FullName, digest, (key, oldValue) => digest);
		}

		/// <summary>
		/// Get the IoHash digest of a file's contents.
		/// </summary>
		/// <param name="location">The FileReference to digest</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <remarks>A cache is maintained of digests and a file will not be rehashed if unchanged.</remarks>
		/// <returns>The IoHash digest, or IoHash.Zero if the file doesn't exist.</returns>
		public Task<IoHash> GetDigestAsync(FileReference location, CancellationToken cancellationToken = default) => GetDigestAsync(FileItem.GetItemByFileReference(location), cancellationToken);

		public IoHash GetDigest(FileItem item) => GetDigestAsync(item).Result;
		public IoHash GetDigest(FileReference location) => GetDigestAsync(location).Result;

		static async Task<CachedDigest> ComputeDigest(FileItem item, ILogger? logger, CancellationToken CancellationToken = default)
		{
			using FileStream stream = FileReference.Open(item.Location, FileMode.Open, FileAccess.Read, FileShare.Read);
			CachedDigest digest = new(item, await IoHash.ComputeAsync(stream, item.Length, CancellationToken));
			logger?.LogDebug("Computed IoHash {Digest} File {Location} Size {Size} LastWrite {LastWrite}", digest.Digest, item.FullName, item.Length, item.LastWriteTimeUtc);
			return digest;
		}
	}
}
