// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Represents a file on disk that is used as an input or output of a build action. FileItem instances are unique for a given path. Use FileItem.GetItemByFileReference 
	/// to get the FileItem for a specific path.
	/// </summary>
	public class FileItem
	{
		/// <summary>
		/// The directory containing this file
		/// </summary>
		Lazy<DirectoryItem> _cachedDirectory;

		/// <summary>
		/// Location of this file
		/// </summary>
		public required FileReference Location { get; init; }

		/// <summary>
		/// Name of this file
		/// </summary>
		public required string Name { get; init; }

		/// <summary>
		/// Full name of this file
		/// </summary>
		public string FullName => Location.FullName;

		/// <summary>
		/// Accessor for the absolute path to the file
		/// </summary>
		public string AbsolutePath => Location.FullName;

		/// <summary>
		/// Gets the directory that this file is in
		/// </summary>
		public DirectoryItem Directory => _cachedDirectory.Value;

		/// <summary>
		/// Whether the file exists.
		/// </summary>
		public bool Exists => Update().Attributes != 0;

		/// <summary>
		/// Size of the file if it exists, otherwise -1
		/// </summary>
		public long Length => UpdateCheckPlaceholder("Length").Length;

		/// <summary>
		/// The attributes for this file
		/// </summary>
		public FileAttributes Attributes => Update().Attributes;

		/// <summary>
		/// The last write time of the file.
		/// </summary>
		public DateTime LastWriteTimeUtc => UpdateCheckPlaceholder("LastWriteTimeUtc").LastWriteTimeUtc;

		/// <summary>
		/// The creation time of the file.
		/// </summary>
		public DateTime CreationTimeUtc => UpdateCheckPlaceholder("CreationTimeUtc").CreationTimeUtc;

		/// <summary>
		/// Determines if the file has the given extension
		/// </summary>
		/// <param name="Extension">The extension to check for</param>
		/// <returns>True if the file has the given extension, false otherwise</returns>
		public bool HasExtension(string Extension) => Location.HasExtension(Extension);

		/// <summary>
		/// Gets the directory containing this file
		/// </summary>
		/// <returns>DirectoryItem for the directory containing this file</returns>
		public DirectoryItem GetDirectoryItem() => Directory;

		/// <summary>
		/// Updates the cached directory for this file. Used by DirectoryItem when enumerating files, to avoid having to look this up later.
		/// </summary>
		/// <param name="Directory">The directory that this file is in</param>
		public void UpdateCachedDirectory(DirectoryItem Directory)
		{
			Debug.Assert(Directory.Location == Location.Directory);
			if (!_cachedDirectory.IsValueCreated)
			{
				_cachedDirectory = new(Directory);
			}
		}

		/// <summary>
		/// Returns true if file is a placeholder file which can only be used to check if it exists. Length, create/write date are not valid
		/// </summary>
		public bool IsPlaceholder => Update().Length == long.MaxValue;

		/// <summary>
		/// Gets a FileItem corresponding to the given path
		/// </summary>
		/// <param name="FilePath">Path for the FileItem</param>
		/// <returns>The FileItem that represents the given file path.</returns>
		public static FileItem GetItemByPath(string FilePath) => GetItemByFileReference(new(FilePath));

		/// <summary>
		/// Gets a FileItem for a given path
		/// </summary>
		/// <param name="Info">Information about the file</param>
		/// <returns>The FileItem that represents the given a full file path.</returns>
		public static FileItem GetItemByFileInfo(FileInfo Info) => GetItemByFileReference(new(Info));

		/// <summary>
		/// Gets a FileItem for a given path
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <returns>The FileItem that represents the given a full file path.</returns>
		public static FileItem GetItemByFileReference(FileReference Location)
			=> UniqueSourceFileMap.GetOrAdd(Location, static loc => new FileItem(loc, loc.GetFileName(), null, null));

		/// <summary>
		/// Deletes the file.
		/// </summary>
		public void Delete(ILogger Logger)
		{
			Debug.Assert(Exists);

			int MaxRetryCount = 3;
			int DeleteTryCount = 0;
			bool bFileDeletedSuccessfully = false;
			do
			{
				// If this isn't the first time through, sleep a little before trying again
				if (DeleteTryCount > 0)
				{
					Thread.Sleep(1000);
				}
				DeleteTryCount++;
				try
				{
					// Delete the destination file if it exists
					FileInfo DeletedFileInfo = new(AbsolutePath);
					if (DeletedFileInfo.Exists)
					{
						DeletedFileInfo.IsReadOnly = false;
						DeletedFileInfo.Delete();
					}
					// Success!
					bFileDeletedSuccessfully = true;
				}
				catch (Exception Ex)
				{
					Logger.LogInformation(Ex, "Failed to delete file '{Location}'", Location);
					Logger.LogInformation("    Exception: {Message}", Ex.Message);
					if (DeleteTryCount < MaxRetryCount)
					{
						Logger.LogInformation("Attempting to retry...");
					}
					else
					{
						Logger.LogError("ERROR: Exhausted all retries!");
					}
				}
			}
			while (!bFileDeletedSuccessfully && (DeleteTryCount < MaxRetryCount));
		}

		/// <summary>
		/// Resets the cached file info
		/// </summary>
		public void ResetCachedInfo()
		{
			_info = null;
		}

		/// <summary>
		/// Resets the cached info, if the FileInfo is not found don't create a new entry
		/// </summary>
		public static void ResetCachedInfo(string Path)
		{
			if (UniqueSourceFileMap.TryGetValue(new(Path), out FileItem? Result))
			{
				Result.ResetCachedInfo();
			}
		}

		/// <summary>
		/// Resets all cached file info. Significantly reduces performance; do not use unless strictly necessary.
		/// </summary>
		public static void ResetAllCachedInfo_SLOW()
		{
			UniqueSourceFileMap.Values.AsParallel().ForAll(Item => Item.ResetCachedInfo());
		}

		/// <summary>
		/// Return the path to this FileItem to debugging
		/// </summary>
		/// <returns>Absolute path to this file item</returns>
		public override string ToString() => Location.ToString();

		/// <summary>
		/// Sort file items by full path
		/// </summary>
		public static readonly IComparer<FileItem?> SortByPath = new SortByPathComparer();

		/// <summary>
		/// For debugging purposes, to know if info has been populated
		/// </summary>
		public bool HasInfo => _info != null;

		[SetsRequiredMembers]
		internal FileItem(FileReference Location, string name, Info? info, DirectoryItem? directory)
		{
			this.Location = Location;
			Name = name;
			_info = info;

			_cachedDirectory = directory != null
				? new(directory)
				: new(() => DirectoryItem.GetItemByDirectoryReference(Location.Directory));
		}

		private Info Update()
		{
			if (_info != null)
			{
				return _info;
			}

			Info newInfo = new();
			if (DirectoryEnumerator.TryGetFileFromPath(FullName, out DirectoryEnumerator.Entry entry))
			{
				newInfo.Attributes = entry.Attributes;
				newInfo.Length = entry.Size;
				newInfo.LastWriteTimeUtc = entry.LastWriteTimeUtc;
				newInfo.CreationTimeUtc = entry.CreationTimeUtc;
			}
			_info = newInfo;
			return newInfo;
		}

		private Info UpdateCheckPlaceholder(string property)
		{
			Info info = Update();
			if (info.Length == long.MaxValue)
			{
				throw new Exception($"{property} can't be accessed, file {FullName} is placeholder");
			}
			return info;
		}

		internal void Update(DirectoryEnumerator.Entry entry, bool force)
		{
			if (_info == null || force)
			{
				_info = new() { Attributes = entry.Attributes, Length = entry.Size, LastWriteTimeUtc = entry.LastWriteTimeUtc, CreationTimeUtc = entry.CreationTimeUtc };
			}
		}

		/// <summary>
		/// A case-insensitive dictionary that's used to map each unique file name to a single FileItem object.
		/// </summary>
		internal static ConcurrentDictionary<FileReference, FileItem> UniqueSourceFileMap = new(-1, 100000);// [];

		internal class Info
		{
			internal FileAttributes Attributes;
			internal long Length;
			internal DateTime LastWriteTimeUtc;
			internal DateTime CreationTimeUtc;
		}

		Info? _info;

		class SortByPathComparer : IComparer<FileItem?>
		{
			public int Compare(FileItem? a, FileItem? b)
			{
				if (a != null && b != null)
				{
					return a.Location.CompareTo(b.Location);
				}
				if (a == b)
				{
					return 0;
				}
				if (a is null)
				{
					return -1;
				}
				return 1;
			}
		}
	}

	/// <summary>
	/// Helper functions for serialization
	/// </summary>
	public static class FileItemExtensionMethods
	{
		/// <summary>
		/// Read a file item from a binary archive
		/// </summary>
		/// <param name="Reader">Reader to serialize data from</param>
		/// <returns>Instance of the serialized file item</returns>
		public static FileItem? ReadFileItem(this BinaryArchiveReader Reader)
		{
			return Reader.ReadObjectReference(Reader => FileItem.GetItemByFileReference(Reader.ReadFileReference()));
		}

		/// <summary>
		/// Write a file item to a binary archive
		/// </summary>
		/// <param name="Writer">Writer to serialize data to</param>
		/// <param name="FileItem">File item to write</param>
		public static void WriteFileItem(this BinaryArchiveWriter Writer, FileItem? FileItem)
		{
			Writer.WriteObjectReference(FileItem!, () => Writer.WriteFileReference(FileItem!.Location));
		}

		/// <summary>
		/// Read a file item as a DirectoryItem and name. This is slower than reading it directly, but results in a significantly smaller archive
		/// where most files are in the same directories.
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		/// <returns>FileItem read from the archive</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		static FileItem ReadCompactFileItemData(this BinaryArchiveReader Reader)
		{
			DirectoryItem Directory = Reader.ReadDirectoryItem()!;
			string Name = Reader.ReadString()!;
			FileReference fileRef = FileReference.Combine(Directory.Location, Name);
			FileItem FileItem = FileItem.UniqueSourceFileMap.GetOrAdd(fileRef, loc => new FileItem(loc, Name, null, Directory));
			FileItem.UpdateCachedDirectory(Directory);
			return FileItem;
		}

		/// <summary>
		/// Read a file item in a format which de-duplicates directory names.
		/// </summary>
		/// <param name="Reader">Reader to serialize data from</param>
		/// <returns>Instance of the serialized file item</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		public static FileItem ReadCompactFileItem(this BinaryArchiveReader Reader)
		{
			// Use lambda that doesn't require anything to be captured thus eliminating an allocation.
			return Reader.ReadObjectReference(ReadCompactFileItemData)!;
		}

		/// <summary>
		/// Writes a file item in a format which de-duplicates directory names.
		/// </summary>
		/// <param name="Writer">Writer to serialize data to</param>
		/// <param name="FileItem">File item to write</param>
		public static void WriteCompactFileItem(this BinaryArchiveWriter Writer, FileItem FileItem)
		{
			// Use lambda that doesn't require anything to be captured thus eliminating an allocation.
			Writer.WriteObjectReference<FileItem>(FileItem, (Writer, FileItem) =>
			{
				Writer.WriteDirectoryItem(FileItem.GetDirectoryItem());
				Writer.WriteString(FileItem.Name);
			});
		}
	}
}
