// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.Metrics;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace UnrealBuildBase
{
	/// <summary>
	/// Helper methods for single instance mutexes.
	/// </summary>
	public static class GlobalSingleInstanceMutex
	{
		/// <summary>
		/// Gets the name of a mutex unique for the given path
		/// </summary>
		/// <param name="name">Base name of the mutex</param>
		/// <param name="uniquePath">Path to identify a unique mutex</param>
		public static string GetUniqueMutexForPath(string name, string uniquePath)
		{
			// generate an IoHash of the path, as GetHashCode is not guaranteed to generate a stable hash
			return $"Global\\{name}_{IoHash.Compute(uniquePath.ToUpperInvariant())}";
		}

		/// <summary>
		/// Gets the name of a mutex unique for the given path
		/// </summary>
		/// <param name="name">Base name of the mutex</param>
		/// <param name="uniquePath">Path to identify a unique mutex</param>
		public static string GetUniqueMutexForPath(string name, FileSystemReference? uniquePath) => GetUniqueMutexForPath(name, uniquePath?.FullName ?? String.Empty);

		/// <summary>
		/// Tries to acquire a global mutex with the given name if it's available, and throws if it is not.
		/// </summary>
		/// <param name="name">Name of the mutex to acquire.</param>
		/// <returns>The acquired mutex.</returns>
		/// <exception cref="BuildLogEventException">Thrown when the mutex cannot be acquired.</exception>
		public static IDisposable AcquireNowOrThrow(string name)
		{
			if (!SingleInstanceMutex.TryAcquire(name, out IDisposable? mutex))
			{
				throw new BuildLogEventException(new CompilationResultException(CompilationResult.ConflictingInstance), "A conflicting instance of {Mutex} is already running.", name);
			}
			return mutex;
		}
	}
}
