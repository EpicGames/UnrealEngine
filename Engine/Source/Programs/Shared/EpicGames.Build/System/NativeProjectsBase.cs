// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Utility functions for querying native projects (ie. those found via a .uprojectdirs query)
	/// </summary>
	public class NativeProjectsBase
	{
		/// <summary>
		/// Object used for synchronizing access to static fields
		/// </summary>
		static protected Lock LockObject = new();

		/// <summary>
		/// The native project base directories
		/// </summary>
		static HashSet<DirectoryReference>? CachedBaseDirectories;

		/// <summary>
		/// Cached list of project files within all the base directories
		/// </summary>
		static HashSet<FileReference>? CachedProjectFiles;

		/// <summary>
		/// Clear our cached properties. Generally only needed if your script has modified local files...
		/// </summary>
		public static void ClearCacheBase()
		{
			CachedBaseDirectories = null;
			CachedProjectFiles = null;
		}

		/// <summary>
		/// Retrieve the list of base directories for native projects
		/// </summary>
		public static IEnumerable<DirectoryReference> EnumerateBaseDirectories(ILogger Logger)
		{
			if (CachedBaseDirectories == null)
			{
				lock (LockObject)
				{
					if (CachedBaseDirectories == null)
					{
						HashSet<DirectoryReference> BaseDirs = [];
						foreach (FileReference RootFile in DirectoryLookupCache.EnumerateFiles(Unreal.RootDirectory))
						{
							if (RootFile.HasExtension(".uprojectdirs"))
							{
								foreach (string Line in File.ReadAllLines(RootFile.FullName))
								{
									string TrimLine = Line.Trim();
									if (!TrimLine.StartsWith(';'))
									{
										DirectoryReference BaseProjectDir = DirectoryReference.Combine(Unreal.RootDirectory, TrimLine);
										if (BaseProjectDir.IsUnderDirectory(Unreal.RootDirectory))
										{
											BaseDirs.Add(BaseProjectDir);
										}
										else
										{
											Logger.LogWarning("Project search path '{SearchPath}' referenced by '{ProjectDirFile}' is not under '{RootDir}', ignoring.", TrimLine, RootFile, Unreal.RootDirectory);
										}
									}
								}
							}
						}
						CachedBaseDirectories = BaseDirs;
					}
				}
			}
			return CachedBaseDirectories;
		}

		/// <summary>
		/// Returns a list of all the projects
		/// </summary>
		/// <returns>List of projects</returns>
		public static IEnumerable<FileReference> EnumerateProjectFiles(ILogger Logger)
		{
			if (CachedProjectFiles == null)
			{
				lock (LockObject)
				{
					if (CachedProjectFiles == null)
					{
						HashSet<FileReference> ProjectFiles = [];
						foreach (DirectoryReference BaseDirectory in EnumerateBaseDirectories(Logger))
						{
							if (DirectoryLookupCache.DirectoryExists(BaseDirectory))
							{
								foreach (DirectoryReference SubDirectory in DirectoryLookupCache.EnumerateDirectories(BaseDirectory))
								{
									// Ignore system directories (specifically for Mac, since temporary folders are created here)
									string DirectoryName = SubDirectory.GetDirectoryName();
									if (DirectoryName.StartsWith('.'))
									{
										continue;
									}

									foreach (FileReference File in DirectoryLookupCache.EnumerateFiles(SubDirectory))
									{
										if (File.HasExtension(".uproject"))
										{
											ProjectFiles.Add(File);
										}
									}
								}
							}
						}
						CachedProjectFiles = ProjectFiles;
					}
				}
			}
			return CachedProjectFiles;
		}

		/// <summary>
		/// Searches base directories for an existing relative pathed file
		/// </summary>
		/// <param name="File">File to search for</param>
		/// <param name="Logger"></param>
		/// <returns>A FileReference for the existing file, otherwise null</returns>
		public static FileReference? FindRelativeFileReference(string File, ILogger Logger)
		{
			return EnumerateBaseDirectories(Logger)
					.Select(x => FileReference.Combine(x, File))
					.FirstOrDefault(x => FileReference.Exists(x));
		}

		/// <summary>
		/// Searches base directories for an existing relative pathed directory
		/// </summary>
		/// <param name="Directory">Directory to search for</param>
		/// <param name="Logger"></param>
		/// <returns>A DirectoryReference for the existing directory, otherwise null</returns>
		public static DirectoryReference? FindRelativeDirectoryReference(string Directory, ILogger Logger)
		{
			return EnumerateBaseDirectories(Logger)
					.Select(x => DirectoryReference.Combine(x, Directory))
					.FirstOrDefault(x => DirectoryReference.Exists(x));
		}

		/// <summary>
		/// Takes a project name (e.g "ShooterGame") or path and attempt to find the existing uproject file in the base directories
		/// </summary>
		/// <param name="Project">Project to search, either a name or a .uproject path</param>
		/// <param name="Logger"></param>
		/// <returns>A FileReference to an existing .uproject file, otherwise null</returns>
		public static FileReference? FindProjectFile(string Project, ILogger Logger)
		{
			// Handle absolute paths, or relative paths from the current working directory
			if (File.Exists(Project))
			{
				return new FileReference(Project);
			}

			if (Path.IsPathFullyQualified(Project))
			{
				// Absolute path not found, return null instead of searching
				return null;
			}

			string ProjectName = Path.GetFileNameWithoutExtension(Project);

			// Search known .uprojects by name, then as relative path, then relative path for Project/Project.uproject
			return EnumerateProjectFiles(Logger).FirstOrDefault(x => String.Equals(x.GetFileNameWithoutExtension(), ProjectName, StringComparison.OrdinalIgnoreCase)) ??
				FindRelativeFileReference(Project, Logger) ??
				FindRelativeFileReference(Path.Combine(ProjectName, $"{ProjectName}.uproject"), Logger);
		}

		/// <summary>
		/// Checks if a given project is a native project
		/// </summary>
		/// <param name="ProjectFile">The project file to check</param>
		/// <param name="Logger"></param>
		/// <returns>True if the given project is a native project</returns>
		public static bool IsNativeProject(FileReference ProjectFile, ILogger Logger)
		{
			EnumerateProjectFiles(Logger);
			return CachedProjectFiles!.Contains(ProjectFile);
		}
	}
}
