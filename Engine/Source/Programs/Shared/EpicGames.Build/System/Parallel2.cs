// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.ExceptionServices;
using System.Threading;
using System.Threading.Tasks;

namespace UnrealBuildBase
{
	/// <summary>
	/// Parallel2 is a collection of concurrent utilities designed for wall time throughput.
	/// Proven to reduce wall time in UBT when running lots of Parallel.ForEach etc.
	/// </summary>
	public static class Parallel2
	{
		/// <summary>
		/// Loops through elements in an enumerator and run them in parallel.
		/// There is a very important difference compared to normal Parallel.ForEach
		/// and that is that there is no "join" of scheduled tasks.
		/// This will make Parallel2.ForEach much faster when there are lots of queued tasks since it will not wait
		/// for scheduled tasks before exiting.
		/// Parallel2.ForEach is much more friendly to nesting lots of parallel foreach.
		/// </summary>
		/// <typeparam name="TSource"></typeparam>
		/// <param name="source">enumerator to iterate elements</param>
		/// <param name="body">action to execute on each element</param>
		/// <param name="helperCount">Helper count. -1 means it will use as many cores as needed</param>
		/// <param name="useTasks">Use tasks or threads</param>
		public static void ForEach<TSource>(IEnumerable<TSource> source, Action<TSource> body, int helperCount = -1, bool useTasks = true)
		{
			using ManualResetEventSlim Handle = new(initialState: false, spinCount: 20);

			ForEachContext<TSource> context = new() { Enumerator = source.GetEnumerator(), Handle = Handle };

			void WorkerLoop()
			{
				int finished = 0;
				ExceptionDispatchInfo? exception = null;
				while (true)
				{
					Monitor.Enter(context);

					if (!context.Done)
					{
						if (!context.Enumerator.MoveNext())
						{
							context.Done = true;
						}
					}

					context.Finished += finished;

					if (context.Done)
					{
						context.Exception ??= exception;

						if (context.Finished == context.Count)
						{
							++context.Finished; // Just to prevent another Set call
							context.Handle.Set();
						}

						Monitor.Exit(context);
						break;
					}

					++context.Count;

					TSource? c = context.Enumerator.Current;

					Monitor.Exit(context);

					try
					{
						body(c);
					}
					catch (Exception ex)
					{
						exception ??= ExceptionDispatchInfo.Capture(ex);
					}
					finished = 1;
				}
			}

			if (helperCount == -1)
			{
				helperCount = MaxHelperCount - 1;
			}

			if (helperCount > 0 && source.TryGetNonEnumeratedCount(out int count))
			{
				helperCount = Math.Min(count, helperCount);
				if (helperCount > 0)
				{
					--helperCount;
				}
			}

			for (int i = 0; i != helperCount; ++i)
			{
				if (useTasks)
				{
					Task.Run(WorkerLoop);
				}
				else
				{
					(new Thread(WorkerLoop)).Start();
				}
			}

			WorkerLoop();

			context.Handle.Wait();

			context.Exception?.Throw();
		}

		/// <summary>
		/// Read information about Parallel2.ForEach
		/// </summary>
		/// <param name="from">Start index</param>
		/// <param name="to">End index</param>
		/// <param name="body">action to execute on each index</param>
		/// <param name="helperCount">Helper count. -1 means it will use as many cores as needed</param>
		/// <param name="useTasks">Use tasks or threads</param>
		public static void For(int from, int to, Action<int> body, int helperCount = -1, bool useTasks = true)
		{
			int distance = to - from;
			if (distance <= 1)
			{
				if (distance == 1)
				{
					body(from);
				}
				return;
			}

			using ManualResetEventSlim Handle = new(initialState: false, spinCount: 20);

			ForContext context = new() { Handle = Handle, Index = from, End = to, Finished = from };

			void WorkerLoop()
			{
				while (true)
				{
					int index = Interlocked.Increment(ref context.Index) - 1;
					if (index >= context.End)
					{
						return;
					}

					try
					{
						body(index);
					}
					catch (Exception ex)
					{
						Interlocked.CompareExchange(ref context.Exception, ExceptionDispatchInfo.Capture(ex), null);
					}

					if (Interlocked.Increment(ref context.Finished) == context.End)
					{
						context.Handle.Set();
						return;
					}
				}
			}

			if (helperCount == -1)
			{
				helperCount = MaxHelperCount - 1;
			}

			if (helperCount > 0)
			{
				helperCount = Math.Min(distance, helperCount);
				if (helperCount > 0)
				{
					--helperCount;
				}
			}

			for (int i = 0; i != helperCount; ++i)
			{
				if (useTasks)
				{
					Task.Run(WorkerLoop);
				}
				else
				{
					(new Thread(WorkerLoop)).Start();
				}
			}

			WorkerLoop();

			context.Handle.Wait();

			context.Exception?.Throw();
		}

		private class ForEachContext<TSource>
		{
			internal required IEnumerator<TSource> Enumerator;
			internal required ManualResetEventSlim Handle;
			internal int Count;
			internal int Finished;
			internal bool Done;
			internal ExceptionDispatchInfo? Exception;
		}

		private class ForContext
		{
			internal required ManualResetEventSlim Handle;
			internal int Index;
			internal int End;
			internal int Finished;
			internal ExceptionDispatchInfo? Exception;
		}

		public static readonly int MaxHelperCount = Math.Max(Environment.ProcessorCount - 1, 0);
	}

	/// <summary>
	/// Parallel queue. Enqueue actions and then call Drain() to execute actions in parallel
	/// Actions are allowed to enqueue more actions from inside the actions
	/// There is a very important implementation detail
	/// and that is that there is no "join" of scheduled tasks. It will really exit as early as possible
	/// </summary>
	public sealed class ParallelQueue : IDisposable
	{
		/// <summary>
		/// Enqueue work which will directly be picked up by workers if inside Run
		/// <param name="body">action to run</param>
		/// </summary>
		public void Enqueue(Action body)
		{
			if (Volatile.Read(ref _accepting) == 0)
			{
				throw new Exception("Queue is closed");
			}

			Interlocked.Increment(ref _outstanding);
			_queue.Enqueue(body);
			_available.Release();
		}

		/// <summary>
		/// Drain the queue. Can only be done once
		/// </summary>
		/// <param name="helperCount">Helper count. -1 means it will use as many cores as needed</param>
		/// <param name="useTasks">Use tasks or threads</param>
		public void Drain(int helperCount = -1, bool useTasks = true)
		{
			if (_queue.IsEmpty)
			{
				return;
			}

			_helperCount = helperCount == -1 ? MaxHelperCount - 1 : helperCount;
			_workerCount = _helperCount + 1;

			void WorkerLoop()
			{
				while (true)
				{
					_available.Wait();

					if (Volatile.Read(ref _done) == 1)
					{
						break;
					}

					if (!_queue.TryDequeue(out Action? action))
					{
						continue;
					}

					try
					{
						action();
					}
					catch (Exception ex)
					{
						_exception = ExceptionDispatchInfo.Capture(ex);
					}

					if (Interlocked.Decrement(ref _outstanding) != 0)
					{
						continue;
					}

					Interlocked.Exchange(ref _accepting, 0);

					if (Volatile.Read(ref _outstanding) == 0 && Interlocked.Exchange(ref _done, 1) == 0)
					{
						_available.Release(_workerCount);
					}
				}
			}

			for (int i = 0; i != _helperCount; ++i)
			{
				if (useTasks)
				{
					Task.Run(WorkerLoop);
				}
				else
				{
					(new Thread(WorkerLoop)).Start();
				}
			}

			WorkerLoop();

			_exception?.Throw();
		}

		public void Dispose()
		{
			// Note, we can not dispose _available here if there are tasks since it needs to outlive
			// all started tasks which could start after ParallelQueue goes out of scope.
			if (_helperCount == 0)
			{
				_available.Dispose();
			}
		}

		private readonly ConcurrentQueue<Action> _queue = new();
		private readonly SemaphoreSlim _available = new(0);
		private int _outstanding;
		private int _accepting = 1;
		private int _done;
		private int _helperCount;
		private int _workerCount;
		private ExceptionDispatchInfo? _exception;

		public static readonly int MaxHelperCount = Math.Max(Environment.ProcessorCount - 1, 0);
	}
}