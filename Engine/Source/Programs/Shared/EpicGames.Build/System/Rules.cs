// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;

namespace UnrealBuildBase
{
	public class Rules
	{
		/// <summary>
		/// Enum for types of rules files. Should match extensions in RulesFileExtensions.
		/// </summary>
		public enum RulesFileType
		{
			/// <summary>
			/// .build.cs files
			/// </summary>
			Module,

			/// <summary>
			/// .target.cs files
			/// </summary>
			Target,

			/// <summary>
			/// .automation.csproj files
			/// </summary>
			AutomationModule,

			/// <summary>
			/// .ubtplugin.csproj files
			/// </summary>
			UbtPlugin,
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="RulesFileType"></param>
		/// <param name="GameFolders"></param>
		/// <param name="ForeignPlugins"></param>
		/// <param name="AdditionalSearchPaths"></param>
		/// <param name="bIncludeEngine"></param>
		/// <param name="bIncludeTempTargets">Whether to include targets generated by UAT to accomodate content-only projects that need to be compiled to include plugins</param>
		/// <returns></returns>
		public static List<FileReference> FindAllRulesSourceFiles(RulesFileType RulesFileType, List<DirectoryReference>? GameFolders, List<FileReference>? ForeignPlugins, List<DirectoryReference>? AdditionalSearchPaths, bool bIncludeEngine = true, bool bIncludeTempTargets = true)
		{
			List<DirectoryReference> Folders = [];

			// Add all engine source (including third party source)
			if (bIncludeEngine)
			{
				Folders.AddRange(Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source"));
			}

			// @todo plugin: Disallow modules from including plugin modules as dependency modules? (except when the module is part of that plugin)

			// Get all the root folders for plugins
			List<DirectoryReference> RootFolders = [];
			if (bIncludeEngine)
			{
				RootFolders.AddRange(Unreal.GetExtensionDirs(Unreal.EngineDirectory));
			}
			if (GameFolders != null)
			{
				RootFolders.AddRange(GameFolders.SelectMany(x => Unreal.GetExtensionDirs(x)));
			}

			// Find all the plugin source and tests directories
			Parallel2.ForEach(RootFolders, (RootFolder) =>
			{
				DirectoryReference PluginsFolder = DirectoryReference.Combine(RootFolder, "Plugins");
				foreach (FileReference PluginFile in PluginsBase.EnumeratePlugins(PluginsFolder))
				{
					lock (Folders)
					{
						Folders.Add(DirectoryReference.Combine(PluginFile.Directory, "Source"));
						Folders.Add(DirectoryReference.Combine(PluginFile.Directory, "Tests"));
						Folders.Add(DirectoryReference.Combine(PluginFile.Directory, "Build")); // this is to pick up plugin's C# automation projects					
					}
				}
			});

			// Add all the extra plugin folders
			if (ForeignPlugins != null)
			{
				foreach (FileReference ForeignPlugin in ForeignPlugins)
				{
					Folders.Add(DirectoryReference.Combine(ForeignPlugin.Directory, "Source"));
				}
			}

			// Add in the game folders to search
			if (GameFolders != null)
			{
				foreach (DirectoryReference GameFolder in GameFolders)
				{
					Folders.AddRange(Unreal.GetExtensionDirs(GameFolder, "Source"));

					if (bIncludeTempTargets)
					{
						DirectoryReference GameIntermediateSourceFolder = DirectoryReference.Combine(GameFolder, "Intermediate", "Source");
						Folders.Add(GameIntermediateSourceFolder);
					}
				}
			}

			// Process the additional search path, if sent in, only for paths that exist
			if (AdditionalSearchPaths != null)
			{
				Folders.AddRange(AdditionalSearchPaths.Where(x => x != null && DirectoryReference.Exists(x)));
			}

			Populate(Folders);
			List<FileReference> files = Get(Folders, RulesFileType);
			DistinctSortedList(files); // Seems like same path is added multiple times and Get is not removing dups, just sorting
			return files;
		}

		/// <summary>
		/// Invalidate the cache for the givcen directory
		/// </summary>
		/// <param name="DirectoryPath">Directory to invalidate</param>
		public static void InvalidateRulesFileCache(string DirectoryPath)
		{
			DirectoryReference dirRef = new(DirectoryPath);
			lock (Root)
			{
				Directory? dir = FindDirectory(dirRef, false);
				if (dir != null)
				{
					dir.IsScanned = false;
					dir.Files = null;
					dir.SubDirectories = null;
				}
			}
			DirectoryLookupCache.InvalidateCachedDirectory(dirRef);
		}

		class Files
		{
			public List<FileReference> ModuleRules = [];
			public List<FileReference> TargetRules = [];
			public List<FileReference> AutomationModules = [];
			public List<FileReference> UbtPlugins = [];
		}

		class Directory
		{
			internal Files? Files;
			internal Dictionary<string, Directory>? SubDirectories;
			internal bool IsScanned;
		}
		static Directory Root = new();
		static ReaderWriterLockSlim RootLock = new();

		static Directory? FindDirectory(DirectoryReference dirRef, bool createIfNotFound)
		{
			string[] parts = dirRef.FullName.Split(Path.DirectorySeparatorChar);

			Directory parentDir = Root;
			for (int i = 0; i != parts.Length; ++i)
			{
				parentDir.SubDirectories ??= new(FileSystemReference.Comparer);
				if (!parentDir.SubDirectories.TryGetValue(parts[i], out Directory? dir))
				{
					if (!createIfNotFound)
					{
						return null;
					}
					dir = new Directory();
					parentDir.SubDirectories[parts[i]] = dir;
				}
				parentDir = dir;
			}

			return parentDir;
		}

		/// <summary>
		/// Populate rule files from provided directories
		/// </summary>
		/// <param name="dirs">Directories to traverse</param>
		public static void Populate(IEnumerable<DirectoryReference> dirs)
		{
			RootLock.EnterWriteLock();
			ParallelQueue? queue = null;
			try
			{
				foreach (DirectoryReference dirRef in dirs)
				{
					Directory dir = FindDirectory(dirRef, true)!;

					if (dir.IsScanned)
					{
						continue;
					}

					dir.IsScanned = true;
					queue ??= new();

					queue.Enqueue(() =>
					{
						DirectoryItem dirItem = DirectoryItem.GetItemByDirectoryReference(dirRef);
						PopulateRecursively(dirItem, dir, queue);
					});
				}
				queue?.Drain();
			}
			catch (Exception)
			{
				throw;
			}
			finally
			{
				queue?.Dispose();
				RootLock.ExitWriteLock();
			}
		}

		/// <summary>
		/// Get all rule files of type. Requires populate to have been run for this to work
		/// </summary>
		/// <param name="dirs">Directories to traverse</param>
		/// <param name="type">Type to return</param>
		/// <returns></returns>
		public static List<FileReference> Get(IEnumerable<DirectoryReference> dirs, RulesFileType type)
		{
			RootLock.EnterReadLock();
			try
			{
				List<FileReference> files = [];
				foreach (DirectoryReference dirRef in dirs)
				{
					Directory? dir = FindDirectory(dirRef, false);
					if (dir != null)
					{
						GetRecursively(files, dir, type);
					}
				}
				files.Sort(static (A, B) => A.CompareTo(B));
				return files;
			}
			catch (Exception)
			{
				throw;
			}
			finally
			{
				RootLock.ExitReadLock();
			}
		}

		private static void DistinctSortedList(List<FileReference> files)
		{
			if (files.Count <= 1)
			{
				return;
			}

			int write = 1;
			FileReference prev = files[0];

			for (int i = 1; i < files.Count; i++)
			{
				FileReference x = files[i];
				if (x != prev)
				{
					files[write++] = x;
					prev = x;
				}
			}

			if (write < files.Count)
			{
				files.RemoveRange(write, files.Count - write);
			}
		}

		private static void GetRecursively(List<FileReference> outFiles, Directory dir, RulesFileType type)
		{
			if (dir.Files != null)
			{
				if (type == RulesFileType.Module)
				{
					outFiles.AddRange(dir.Files.ModuleRules);
				}
				else if (type == RulesFileType.Target)
				{
					outFiles.AddRange(dir.Files.TargetRules);
				}
				else if (type == RulesFileType.AutomationModule)
				{
					outFiles.AddRange(dir.Files.AutomationModules);
				}
				else if (type == RulesFileType.UbtPlugin)
				{
					outFiles.AddRange(dir.Files.UbtPlugins);
				}
				else
				{
					throw new Exception($"Unhandled rules type: {type}");
				}
			}
			if (dir.SubDirectories != null)
			{
				foreach (KeyValuePair<string, Directory> kv in dir.SubDirectories)
				{
					GetRecursively(outFiles, kv.Value, type);
				}
			}
		}

		private static void PopulateRecursively(DirectoryItem dirItem, Directory dir, ParallelQueue queue)
		{
			bool bSearchSubFolders = true;
			bool bIsPlugin = false;
			bool bIsCsProj = false;

			foreach (FileItem File in dirItem.EnumerateFiles())
			{
				if (File.HasExtension(".build.cs"))
				{
					dir.Files ??= new();
					dir.Files.ModuleRules.Add(File.Location);
					bSearchSubFolders = false;
				}
				else if (File.HasExtension(".target.cs"))
				{
					dir.Files ??= new();
					dir.Files.TargetRules.Add(File.Location);
				}
				else if (File.HasExtension(".automation.csproj"))
				{
					dir.Files ??= new();
					dir.Files.AutomationModules.Add(File.Location);
					bSearchSubFolders = false;
				}
				else if (File.HasExtension(".ubtplugin.csproj"))
				{
					dir.Files ??= new();
					dir.Files.UbtPlugins.Add(File.Location);
					bSearchSubFolders = false;
				}
				else if (File.HasExtension(".uplugin"))
				{
					bIsPlugin = true;
					bSearchSubFolders = false;
				}
				else if (File.HasExtension(".csproj"))
				{
					bIsCsProj = true;
				}
				else if (File.Name == ".ubtignore")
				{
					bSearchSubFolders = false;
				}
			}

			bool checkExisting = dir.SubDirectories != null;

			void QueueSubDir(DirectoryItem subDirItem)
			{
				if (checkExisting && dir.SubDirectories!.TryGetValue(subDirItem.Name, out Directory? subDir))
				{
					if (subDir.IsScanned)
					{
						return;
					}
				}
				else
				{
					subDir = new();
					dir.SubDirectories ??= new(FileSystemReference.Comparer);
					dir.SubDirectories.Add(subDirItem.Name, subDir);
				}
				subDir.IsScanned = true;

				queue.Enqueue(() => PopulateRecursively(subDirItem, subDir, queue));
			}

			if (bIsPlugin)
			{
				if (dirItem.TryGetDirectory("Source", out DirectoryItem? sourceDirItem))
				{
					QueueSubDir(sourceDirItem);
				}
				if (dirItem.TryGetDirectory("Build", out DirectoryItem? buildDirItem))
				{
					QueueSubDir(buildDirItem);
				}
			}

			// If we didn't find anything to stop the search, search all the subdirectories too
			if (!bSearchSubFolders)
			{
				return;
			}

			foreach (DirectoryItem subDirectory in dirItem.EnumerateDirectories())
			{
				if (bIsCsProj)
				{
					if (subDirectory.Name == "bin" || subDirectory.Name == "obj") // Intermediate files.. we don't want to enter here
					{
						continue;
					}
				}
				else if (subDirectory.Name.StartsWith("__External", StringComparison.Ordinal)) // This is a bit annoying but version control likes to leave around these huuuge hierarchies of empty directories
				{
					//Console.WriteLine("Most likely leftovers: " + Directory.Location);
					continue;
				}
				//else if (subDirectory.Name == "Intermediate")
				//{
				//	Console.WriteLine("Most likely leftovers: " + Directory.Location);
				//}

				QueueSubDir(subDirectory);
			}
		}
	}
}
