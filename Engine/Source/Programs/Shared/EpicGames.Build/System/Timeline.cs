// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildBase
{
	/// <summary>
	/// Public interface for a timeline scope. Should be disposed to exit the scope.
	/// </summary>
	public interface ITimelineEvent : IDisposable
	{
		void Finish();
	}

	/// <summary>
	/// Tracks simple high-level timing data
	/// </summary>
	public static class Timeline
	{
		/// <summary>
		/// A marker in the timeline
		/// </summary>
		[DebuggerDisplay("{Name}")]
		class Event : ITimelineEvent
		{
			/// <summary>
			/// Name of the marker
			/// </summary>
			public readonly string Name;

			/// <summary>
			/// Time at which the event ocurred
			/// </summary>
			public readonly TimeSpan StartTime;

			/// <summary>
			/// Time at which the event ended
			/// </summary>
			public TimeSpan? FinishTime;

			/// <summary>
			/// The trace span for external tracing
			/// </summary>
			public readonly ITraceSpan Span;

			/// <summary>
			/// Optional output logger
			/// </summary>
			ILogger? Logger;

			/// <summary>
			/// Optional output verbosity
			/// </summary>
			LogLevel? Verbosity;

			uint Id = 0;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="Name">Event name</param>
			/// <param name="StartTime">Time of the event</param>
			/// <param name="FinishTime">Finish time for the event. May be null.</param>
			/// <param name="Logger">Logger. May be null.</param>
			/// <param name="Verbosity">Verbosity. May be null.</param>
			public Event(string Name, TimeSpan StartTime, TimeSpan? FinishTime = null, ILogger? Logger = null, LogLevel? Verbosity = null)
			{
				this.Name = Name;
				this.StartTime = StartTime;
				this.FinishTime = FinishTime;
				this.Logger = Logger;
				this.Verbosity = Verbosity;
				Span = TraceSpan.Create(Name);

				if (OnEventBegin != null)
				{
					Id = OnEventBegin.Invoke(Name);
				}
			}

			/// <summary>
			/// Finishes the current event
			/// </summary>
			public void Finish()
			{
				if (!FinishTime.HasValue)
				{
					if (Id != 0 && OnEventEnd != null)
					{
						OnEventEnd.Invoke(Id);
					}

					FinishTime = Stopwatch.Elapsed;
					if (Logger != null && Verbosity != null)
					{
						double Duration = ((TimeSpan)FinishTime - StartTime).TotalSeconds;
						Logger.Log((LogLevel)Verbosity, "{Name} took {Duration:0.00}s", Name, Duration);
					}
				}
				Span.Dispose();
			}

			/// <summary>
			/// Disposes of the current event
			/// </summary>
			public void Dispose()
			{
				Finish();
			}
		}

		public static Func<string, uint>? OnEventBegin;
		public static Action<uint>? OnEventEnd;

		/// <summary>
		/// The stopwatch used for timing
		/// </summary>
		static Stopwatch Stopwatch = new();

		/// <summary>
		/// The recorded events
		/// </summary>
		static List<Event> Events = [];

		/// <summary>
		/// Property for the total time elapsed
		/// </summary>
		public static TimeSpan Elapsed => Stopwatch.Elapsed;

		/// <summary>
		/// Start the stopwatch
		/// </summary>
		public static void Start()
		{
			Stopwatch.Restart();
		}

		/// <summary>
		/// Stop the stopwatch
		/// </summary>
		public static void Stop()
		{
			Stopwatch.Stop();
		}

		/// <summary>
		/// Records a timeline marker with the given name
		/// </summary>
		/// <param name="Name">The marker name</param>
		public static void AddEvent(string Name)
		{
			TimeSpan Time = Stopwatch.Elapsed;
			lock (Events)
			{
				Events.Add(new Event(Name, Time, Time));
			}
		}

		/// <summary>
		/// Enters a scope event with the given name. Should be disposed to terminate the scope.
		/// </summary>
		/// <param name="Name">Name of the event</param>
		/// <returns>Event to track the length of the event</returns>
		public static ITimelineEvent ScopeEvent(string Name)
		{
			Event Event = new(Name, Stopwatch.Elapsed, null);
			lock (Events)
			{
				Events.Add(Event);
			}
			return Event;
		}

		/// <summary>
		/// Enters a scope event with the given name. Should be disposed to terminate the scope.
		/// Also logs the finish of the event at the provided verbosity.
		/// </summary>
		/// <param name="Name">Name of the event</param>
		/// <param name="Logger">Logger.</param>
		/// <param name="Verbosity">LogLevel of the event.</param>
		/// <returns>Event to track the length of the event</returns>
		public static ITimelineEvent ScopeEvent(string Name, ILogger Logger, LogLevel Verbosity = LogLevel.Information)
		{
			Event Event = new(Name, Stopwatch.Elapsed, null, Logger, Verbosity);
			lock (Events)
			{
				Events.Add(Event);
			}
			return Event;
		}

		/// <summary>
		/// Prints this information to the log
		/// </summary>
		/// <param name="MinKnownTime">Instrumented time limit for printing the labelled event at given verbosity. May be null.</param>
		/// <param name="MaxUnknownTime">Non-instrumented time limit for inserting an "unknown" event</param>
		/// <param name="MaxDepth">Maximum tree depth to print.</param>
		/// <param name="Verbosity">LogLevel of the event.</param>
		/// <param name="Logger">Logger.</param>
		public static void Print(TimeSpan? MinKnownTime, TimeSpan MaxUnknownTime, int? MaxDepth, LogLevel Verbosity, ILogger Logger)
		{
			// Print the start time
			Logger.Log(Verbosity, "");
			Logger.Log(Verbosity, "Timeline:");

			// Create the root event
			TimeSpan FinishTime = Stopwatch.Elapsed;

			List<Event> OuterEvents =
			[
				new Event("<Root>", TimeSpan.Zero, FinishTime)
			];

			// Print out all the child events
			TimeSpan LastTime = TimeSpan.Zero;
			lock (Events)
			{
				for (int EventIdx = 0; EventIdx < Events.Count; EventIdx++)
				{
					Event Event = Events[EventIdx];

					// Skip inner events when MaxDepth has been reached
					if (MaxDepth != null && OuterEvents.Count == MaxDepth)
					{
						Event OuterEvent = OuterEvents.Last();
						if (Event.StartTime < OuterEvent.FinishTime!.Value)
						{
							continue;
						}
						LastTime = OuterEvent.FinishTime.Value;
					}

					// Pop events off the stack
					for (; OuterEvents.Count > 1; OuterEvents.RemoveAt(OuterEvents.Count - 1))
					{
						Event OuterEvent = OuterEvents.Last();
						if (Event.StartTime < OuterEvent.FinishTime!.Value)
						{
							break;
						}
						UpdateLastEventTime(ref LastTime, OuterEvent.FinishTime.Value, MaxUnknownTime, OuterEvents, Verbosity, Logger);
					}

					// If there's a gap since the last event, print an unknown marker
					UpdateLastEventTime(ref LastTime, Event.StartTime, MaxUnknownTime, OuterEvents, Verbosity, Logger);

					// Print this event
					Print(Event.StartTime, Event.FinishTime, MinKnownTime, Event.Name, OuterEvents, Verbosity, Logger);

					// Push it onto the stack
					if (Event.FinishTime.HasValue)
					{
						if (EventIdx + 1 < Events.Count && Events[EventIdx + 1].StartTime < Event.FinishTime.Value)
						{
							OuterEvents.Add(Event);
						}
						else
						{
							LastTime = Event.FinishTime.Value;
						}
					}
				}
			}

			// Remove everything from the stack
			for (; OuterEvents.Count > 0; OuterEvents.RemoveAt(OuterEvents.Count - 1))
			{
				UpdateLastEventTime(ref LastTime, OuterEvents.Last().FinishTime!.Value, MaxUnknownTime, OuterEvents, Verbosity, Logger);
			}

			// Print the finish time
			Logger.Log(Verbosity, "[{Time,7}]", FormatTime(FinishTime));
		}

		/// <summary>
		/// Updates the last event time
		/// </summary>
		/// <param name="LastTime"></param>
		/// <param name="NewTime"></param>
		/// <param name="MaxUnknownTime"></param>
		/// <param name="OuterEvents"></param>
		/// <param name="Verbosity"></param>
		/// <param name="Logger"></param>
		static void UpdateLastEventTime(ref TimeSpan LastTime, TimeSpan NewTime, TimeSpan MaxUnknownTime, List<Event> OuterEvents, LogLevel Verbosity, ILogger Logger)
		{
			const string UnknownEvent = "<unknown>";
			if (NewTime - LastTime > MaxUnknownTime)
			{
				Print(LastTime, NewTime, null, UnknownEvent, OuterEvents, Verbosity, Logger);
			}
			LastTime = NewTime;
		}

		/// <summary>
		/// Prints an individual event to the log
		/// </summary>
		/// <param name="StartTime">Start time for the event</param>
		/// <param name="FinishTime">Finish time for the event. May be null.</param>
		/// <param name="MinKnownTime">MinKnownTime for printing at the provided verbosity. May be null.</param>
		/// <param name="Label">Event name</param>
		/// <param name="OuterEvents">List of all the start times for parent events</param>
		/// <param name="Verbosity">Verbosity for the output</param>
		/// <param name="Logger">Logger for output</param>
		static void Print(TimeSpan StartTime, TimeSpan? FinishTime, TimeSpan? MinKnownTime, string Label, List<Event> OuterEvents, LogLevel Verbosity, ILogger Logger)
		{
			StringBuilder Prefix = new();

			for (int Idx = 0; Idx < OuterEvents.Count - 1; Idx++)
			{
				Prefix.AppendFormat(" {0,7}          ", FormatTime(StartTime - OuterEvents[Idx].StartTime));
			}

			Prefix.AppendFormat("[{0,7}]", FormatTime(StartTime - OuterEvents[^1].StartTime));

			bool UseDebugVerbosity = MinKnownTime != null;
			if (!FinishTime.HasValue)
			{
				Prefix.AppendFormat("{0,8}", "???");
			}
			else if (FinishTime.Value == StartTime)
			{
				Prefix.Append(" ------ ");
			}
			else
			{
				Prefix.AppendFormat("{0,8}", "+" + FormatTime(FinishTime.Value - StartTime));
				UseDebugVerbosity &= (FinishTime.Value - StartTime) < MinKnownTime;
			}

			LogLevel PrintVerbosity = UseDebugVerbosity && Verbosity > LogLevel.Debug ? LogLevel.Debug : Verbosity;
			Logger.Log(PrintVerbosity, "{Prefix} {Label}", Prefix.ToString(), Label);
		}

		/// <summary>
		/// Formats a timespan in milliseconds
		/// </summary>
		/// <param name="Time">The time to format</param>
		/// <returns>Formatted timespan</returns>
		static string FormatTime(TimeSpan Time)
		{
			int TotalMilliseconds = (int)Time.TotalMilliseconds;
			return String.Format("{0}.{1:000}", TotalMilliseconds / 1000, TotalMilliseconds % 1000);
		}
	}

	/// <summary>
	/// Will add and remove to OnEvent
	/// </summary>
	public readonly struct TimelineEventScope : IDisposable
	{
		readonly Func<string, uint>? _begin;
		readonly Action<uint>? _end;

		/// <summary>
		/// Ctor
		/// </summary>
		/// <param name="begin">OnEventBegin</param>
		/// <param name="end">OnEventEnd</param>
		public TimelineEventScope(Func<string, uint>? begin, Action<uint>? end)
		{
			_begin = begin;
			_end = end;
			Timeline.OnEventBegin += _begin;
			Timeline.OnEventEnd += _end;
		}

		/// <summary>
		/// Dispose
		/// </summary>
		public void Dispose()
		{
			Timeline.OnEventBegin -= _begin;
			Timeline.OnEventEnd -= _end;
		}
	}
}
