// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using EpicGames.Core;

namespace UnrealBuildBase
{
	public static class Unreal
	{
		private static DirectoryReference FindRootDirectory()
		{
			if (LocationOverride.RootDirectory != null)
			{
				return DirectoryReference.FindCorrectCase(LocationOverride.RootDirectory);
			}

			string? overrideArg = Environment.GetCommandLineArgs().FirstOrDefault(x => x?.StartsWith("-rootdirectory=", StringComparison.OrdinalIgnoreCase) ?? false, null);
			if (overrideArg != null)
			{
				string[] parts = overrideArg.Split('=', 2);
				return new(Path.GetFullPath(parts[1]));
			}

			// This base library may be used - and so be launched - from more than one location (at time of writing, UnrealBuildTool and AutomationTool)
			// Programs that use this assembly must be located under "Engine/Binaries/DotNET" and so we look for that sequence of directories in that path of the executing assembly

			// Use the EntryAssembly (the application path), rather than the ExecutingAssembly (the library path)
			string assemblyLocation = Assembly.GetEntryAssembly()!.GetOriginalLocation();

			DirectoryReference? foundRootDirectory = DirectoryReference.FindCorrectCase(DirectoryReference.FromString(assemblyLocation)!);

			// Search up through the directory tree for the deepest instance of the sub-path "Engine/Binaries/DotNET"
			while (foundRootDirectory != null)
			{
				if (String.Equals("DotNET", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
				{
					foundRootDirectory = foundRootDirectory.ParentDirectory;
					if (foundRootDirectory != null && String.Equals("Binaries", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
					{
						foundRootDirectory = foundRootDirectory.ParentDirectory;
						if (foundRootDirectory != null && String.Equals("Engine", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
						{
							foundRootDirectory = foundRootDirectory.ParentDirectory;
							break;
						}
						continue;
					}
					continue;
				}
				foundRootDirectory = foundRootDirectory.ParentDirectory;
			}

			// Search up through the directory tree for the deepest instance of the sub-path "Engine/Source/Programs"
			if (foundRootDirectory == null)
			{
				foundRootDirectory = DirectoryReference.FindCorrectCase(DirectoryReference.FromString(assemblyLocation)!);
				while (foundRootDirectory != null)
				{
					if (String.Equals("Programs", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
					{
						foundRootDirectory = foundRootDirectory.ParentDirectory;
						if (foundRootDirectory != null && String.Equals("Source", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
						{
							foundRootDirectory = foundRootDirectory.ParentDirectory;
							if (foundRootDirectory != null && String.Equals("Engine", foundRootDirectory.GetDirectoryName(), StringComparison.OrdinalIgnoreCase))
							{
								foundRootDirectory = foundRootDirectory.ParentDirectory;
								break;
							}
							continue;
						}
						continue;
					}
					foundRootDirectory = foundRootDirectory.ParentDirectory;
				}
			}

			if (foundRootDirectory == null)
			{
				throw new Exception($"This code requires that applications using it are launched from a path containing \"Engine/Binaries/DotNET\" or \"Engine/Source/Programs\". This application was launched from {Path.GetDirectoryName(assemblyLocation)}");
			}

			// Confirm that we've found a valid root directory, by testing for the existence of a well-known file
			FileReference expectedExistingFile = FileReference.Combine(foundRootDirectory, "Engine", "Build", "Build.version");
			if (!FileReference.Exists(expectedExistingFile))
			{
				throw new Exception($"Expected file \"Engine/Build/Build.version\" was not found at {expectedExistingFile.FullName}");
			}

			return foundRootDirectory;
		}

		private static FileReference FindUnrealBuildToolDll()
		{
			// Return the entry assembly location if it is UnrealBuildTool
			FileReference? entryDll = FileReference.FromString(Assembly.GetEntryAssembly()?.GetOriginalLocation());
			if (entryDll?.GetFileName().Equals("UnrealBuildTool.dll", StringComparison.OrdinalIgnoreCase) == true)
			{
				return FileReference.FindCorrectCase(entryDll);
			}

			// UnrealBuildTool.dll is assumed to be located under {RootDirectory}/Engine/Binaries/DotNET/UnrealBuildTool/
			FileReference unrealBuildToolDllPath = FileReference.Combine(EngineDirectory, "Binaries", "DotNET", "UnrealBuildTool", "UnrealBuildTool.dll");

			unrealBuildToolDllPath = FileReference.FindCorrectCase(unrealBuildToolDllPath);

			if (!FileReference.Exists(unrealBuildToolDllPath))
			{
				throw new Exception($"Unable to find UnrealBuildTool.dll in the expected location at {unrealBuildToolDllPath.FullName}");
			}

			return unrealBuildToolDllPath;
		}

		private const string DotnetVersionDirectory = "10.0";

		private static string FindRelativeDotnetDirectory(RuntimePlatform.Type hostPlatform)
		{
			string platform;
			string architecture;

			switch (hostPlatform)
			{
				case RuntimePlatform.Type.Linux: platform = "linux"; break;
				case RuntimePlatform.Type.Mac: platform = "mac"; break;
				case RuntimePlatform.Type.Windows: platform = "win"; break;
				default: throw new PlatformNotSupportedException($"Unsupported host platform {hostPlatform}");
			}

			switch (RuntimeInformation.ProcessArchitecture)
			{
				case Architecture.Arm64: architecture = "arm64"; break;
				case Architecture.X64: architecture = "x64"; break;
				default: throw new PlatformNotSupportedException($"Unsupported host architecture {hostPlatform} {RuntimeInformation.ProcessArchitecture}");
			}

			return Path.Combine("Binaries", "ThirdParty", "DotNet", DotnetVersionDirectory, $"{platform}-{architecture}");
		}

		private static string FindRelativeDotnetDirectory() => FindRelativeDotnetDirectory(RuntimePlatform.Current);

		private static Version GetDotnetFileVersion()
		{
			if (!FileReference.Exists(DotnetPath))
			{
				throw new FileNotFoundException("Unable to query FileVersion", DotnetPath.FullName);
			}
			FileVersionInfo versionInfo = FileVersionInfo.GetVersionInfo(DotnetPath.FullName);
			return new Version(versionInfo.FileMajorPart, versionInfo.FileMinorPart, versionInfo.FileBuildPart);
		}

		/// <summary>
		/// Relative path to the dotnet executable from EngineDir
		/// </summary>
		/// <returns></returns>
		public static string RelativeDotnetDirectory => s_relativeDotnetDirectory.Value;
		private static readonly Lazy<string> s_relativeDotnetDirectory = new(FindRelativeDotnetDirectory);

		private static DirectoryReference FindDotnetDirectory() => DirectoryReference.Combine(EngineDirectory, RelativeDotnetDirectory);

		/// <summary>
		/// The full name of the root UE directory
		/// </summary>
		public static DirectoryReference RootDirectory => s_rootDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_rootDirectory = new(FindRootDirectory);

		/// <summary>
		/// The full name of the Engine directory
		/// </summary>
		public static DirectoryReference EngineDirectory => s_engineDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_engineDirectory = new(() => DirectoryReference.Combine(RootDirectory, "Engine"));

		/// <summary>
		/// The full name of the Engine/Source directory
		/// </summary>
		public static DirectoryReference EngineSourceDirectory => s_engineSourceDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_engineSourceDirectory = new(() => DirectoryReference.Combine(EngineDirectory, "Source"));

		/// <summary>
		/// Returns the Application Settings Directory path. This matches FPlatformProcess::ApplicationSettingsDir().
		/// </summary>
		public static DirectoryReference ApplicationSettingDirectory => s_applicationSettingDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_applicationSettingDirectory = new(GetApplicationSettingDirectory);

		/// <summary>
		/// Returns the User Settings Directory path. This matches FPlatformProcess::UserSettingsDir().
		/// </summary>
		public static DirectoryReference UserSettingDirectory => s_userSettingDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_userSettingDirectory = new(GetUserSettingDirectory);

		/// <summary>
		/// Returns the User Directory path. This matches FPlatformProcess::UserDir().
		/// </summary>
		public static DirectoryReference? UserDirectory => s_userDirectory.Value;
		private static readonly Lazy<DirectoryReference?> s_userDirectory = new(GetUserDirectory);

		/// <summary>
		/// Writable engine directory. Uses the user's settings folder for installed builds.
		/// </summary>
		public static DirectoryReference WritableEngineDirectory => s_writableEngineDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_writableEngineDirectory = new(() => IsEngineInstalled() ? DirectoryReference.Combine(UserSettingDirectory, "UnrealEngine") : EngineDirectory);

		/// <summary>
		/// The engine saved programs directory
		/// </summary>
		public static DirectoryReference EngineProgramSavedDirectory => s_engineProgramSavedDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_engineProgramSavedDirectory = new(() => IsEngineInstalled() ? UserSettingDirectory : DirectoryReference.Combine(EngineDirectory, "Programs"));

		/// <summary>
		/// The path to UBT
		/// </summary>
		[Obsolete("Deprecated in UE5.1; to launch UnrealBuildTool, use this dll as the first argument with DonetPath")]
		public static FileReference UnrealBuildToolPath => s_unrealBuildToolDllPath.Value.ChangeExtension(RuntimePlatform.ExeExtension);

		/// <summary>
		/// The path to UBT
		/// </summary>
		public static FileReference UnrealBuildToolDllPath => s_unrealBuildToolDllPath.Value;
		private static readonly Lazy<FileReference> s_unrealBuildToolDllPath = new(FindUnrealBuildToolDll);

		/// <summary>
		/// The directory containing the bundled .NET installation
		/// </summary>
		public static DirectoryReference DotnetDirectory => s_dotnetDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_dotnetDirectory = new(FindDotnetDirectory);

		/// <summary>
		/// The path of the bundled dotnet executable
		/// </summary>
		public static FileReference DotnetPath => s_dotnetPath.Value;
		private static readonly Lazy<FileReference> s_dotnetPath = new(() => FileReference.Combine(DotnetDirectory, $"dotnet{RuntimePlatform.ExeExtension}"));

		/// <summary>
		/// The version of the bundled dotnet executable
		/// </summary>
		public static Version DotnetVersion => s_dotnetVersion.Value;
		private static readonly Lazy<Version> s_dotnetVersion = new(GetDotnetFileVersion);

		/// <summary>
		/// Returns true if the application is running on a build machine
		/// </summary>
		/// <returns>True if running on a build machine</returns>
		public static bool IsBuildMachine() => s_isBuildMachine.Value;
		private static readonly Lazy<bool> s_isBuildMachine = new(() => String.Equals("1", Environment.GetEnvironmentVariable("IsBuildMachine")?.Trim(), StringComparison.Ordinal));

		/// <summary>
		/// Returns true if the application is running using installed Engine components
		/// </summary>
		/// <returns>True if running using installed Engine components</returns>
		public static bool IsEngineInstalled() => s_isEngineInstalled.Value;
		private static readonly Lazy<bool> s_isEngineInstalled = new(() => FileReference.Exists(FileReference.Combine(EngineDirectory, "Build", "InstalledBuild.txt")));

		/// <summary>
		/// If we are running with an installed project, specifies the path to it
		/// </summary>
		private static readonly Lazy<FileReference?> s_installedProjectFile = new(() =>
		{
			FileReference installedProjectLocationFile = FileReference.Combine(EngineDirectory, "Build", "InstalledProjectBuild.txt");
			return FileReference.Exists(installedProjectLocationFile)
				? FileReference.Combine(RootDirectory, FileReference.ReadAllText(installedProjectLocationFile).Trim())
				: null;
		});

		/// <summary>
		/// The original root directory that was used to compile the installed engine
		/// Used to remap source code paths when debugging.
		/// </summary>
		public static DirectoryReference OriginalCompilationRootDirectory => s_originalCompilationRootDirectory.Value;
		private static readonly Lazy<DirectoryReference> s_originalCompilationRootDirectory = new(FindOriginalCompilationRootDirectory);

		/// <summary>
		/// Returns where another platform's Dotnet is located
		/// </summary>
		/// <param name="hostPlatform"></param>
		/// <returns></returns>
		public static DirectoryReference FindDotnetDirectoryForPlatform(RuntimePlatform.Type hostPlatform) => DirectoryReference.Combine(EngineDirectory, FindRelativeDotnetDirectory(hostPlatform));

		/// <summary>
		/// Returns true if the application is running using an installed project (ie. a mod kit)
		/// </summary>
		/// <returns>True if running using an installed project</returns>
		public static bool IsProjectInstalled() => s_installedProjectFile.Value != null;

		/// <summary>
		/// Gets the installed project file
		/// </summary>
		/// <returns>Location of the installed project file</returns>
		public static FileReference? GetInstalledProjectFile() => s_installedProjectFile.Value;

		/// <summary>
		/// Checks whether the given file is under an installed directory, and should not be overridden
		/// </summary>
		/// <param name="file">File to test</param>
		/// <returns>True if the file is part of the installed distribution, false otherwise</returns>
		public static bool IsFileInstalled(FileReference file)
		{
			if (IsEngineInstalled() && file.IsUnderDirectory(EngineDirectory))
			{
				return true;
			}
			if (IsProjectInstalled() && file.IsUnderDirectory(s_installedProjectFile.Value!.Directory))
			{
				return true;
			}
			return false;
		}

		private static DirectoryReference FindOriginalCompilationRootDirectory()
		{
			if (IsEngineInstalled())
			{
				// Load Engine\Intermediate\Build\BuildRules\*RulesManifest.json
				DirectoryReference buildRules = DirectoryReference.Combine(EngineDirectory, "Intermediate", "Build", "BuildRules");
				FileReference? rulesManifest = DirectoryReference.EnumerateFiles(buildRules, "*RulesManifest.json").FirstOrDefault();
				if (rulesManifest != null)
				{
					JsonObject manifest = JsonObject.Read(rulesManifest);
					if (manifest.TryGetStringArrayField("SourceFiles", out string[]? sourceFiles))
					{
						FileReference? sourceFile = FileReference.FromString(sourceFiles.FirstOrDefault());
						if (sourceFile != null && !sourceFile.IsUnderDirectory(EngineDirectory))
						{
							// Walk up parent directory until Engine is found
							DirectoryReference? directory = sourceFile.Directory;
							while (directory != null && !directory.IsRootDirectory())
							{
								if (directory.GetDirectoryName() == "Engine" && directory.ParentDirectory != null)
								{
									return directory.ParentDirectory;
								}

								directory = directory.ParentDirectory;
							}
						}
					}
				}
			}

			return RootDirectory;
		}

		public static class LocationOverride
		{
			/// <summary>
			/// If set, this value will be used to populate Unreal.RootDirectory
			/// </summary>
			public static DirectoryReference? RootDirectory { get; set; } = null;
		}

		// A subset of the functionality in DataDrivenPlatformInfo.GetAllPlatformInfos() - finds the DataDrivenPlatformInfo.ini files and records their existence, but does not parse them
		// (perhaps DataDrivenPlatformInfo.GetAllPlatformInfos() could be modified to use this data to avoid an additional search through the filesystem)
		private static readonly Lazy<HashSet<string>> s_iniPresentForPlatform = new(() =>
		{
			HashSet<string> set = new(StringComparer.OrdinalIgnoreCase);
			// find all platform directories (skipping NFL/NoRedist)
			foreach (DirectoryReference engineConfigDir in GetExtensionDirs(EngineDirectory, "Config", bIncludeRestrictedDirectories: false))
			{
				// look through all config dirs looking for the data driven ini file
				foreach (string filePath in Directory.EnumerateFiles(engineConfigDir.FullName, "DataDrivenPlatformInfo.ini", SearchOption.AllDirectories))
				{
					FileReference fileRef = new(filePath);

					// get the platform name from the path
					string iniPlatformName;
					if (fileRef.IsUnderDirectory(DirectoryReference.Combine(EngineDirectory, "Config")))
					{
						// Foo/Engine/Config/<Platform>/DataDrivenPlatformInfo.ini
						iniPlatformName = Path.GetFileName(Path.GetDirectoryName(filePath))!;
					}
					else
					{
						// Foo/Engine/Platforms/<Platform>/Config/DataDrivenPlatformInfo.ini
						iniPlatformName = Path.GetFileName(Path.GetDirectoryName(Path.GetDirectoryName(filePath)))!;
					}

					// DataDrivenPlatformInfo.GetAllPlatformInfos() checks that [DataDrivenPlatformInfo] section exists as part of validating that the file exists
					// This code should probably behave the same way.
					set.Add(iniPlatformName);
				}
			}

			return set;
		});

		private static bool DataDrivenPlatformInfoIniIsPresent(string platformName) => s_iniPresentForPlatform.Value.Contains(platformName);

		// cached dictionary of BaseDir to extension directories
		private static readonly ConcurrentDictionary<DirectoryReference, Lazy<(List<DirectoryReference>, List<DirectoryReference>)>> s_cachedExtensionDirectories = new();

		/// <summary>
		/// Finds all the extension directories for the given base directory. This includes platform extensions and restricted folders.
		/// </summary>
		/// <param name="baseDir">Location of the base directory</param>
		/// <param name="bIncludePlatformDirectories">If true, platform subdirectories are included (will return platform directories under Restricted dirs, even if bIncludeRestrictedDirectories is false)</param>
		/// <param name="bIncludeRestrictedDirectories">If true, restricted (NotForLicensees, NoRedist) subdirectories are included</param>
		/// <param name="bIncludeBaseDirectory">If true, BaseDir is included</param>
		/// <returns>List of extension directories, including the given base directory</returns>
		public static List<DirectoryReference> GetExtensionDirs(DirectoryReference baseDir, bool bIncludePlatformDirectories = true, bool bIncludeRestrictedDirectories = true, bool bIncludeBaseDirectory = true)
		{
			(List<DirectoryReference>, List<DirectoryReference>) cachedDirs = s_cachedExtensionDirectories.GetOrAdd(baseDir, new Lazy<(List<DirectoryReference>, List<DirectoryReference>)>(() =>
			{
				(List<DirectoryReference>, List<DirectoryReference>) newCachedDirs = new([], []);

				DirectoryItem baseDirItem = DirectoryItem.GetItemByDirectoryReference(baseDir);
				if (baseDirItem.TryGetDirectory("Platforms", out DirectoryItem? platformExtensionBaseDir))
				{
					newCachedDirs.Item1.AddRange(platformExtensionBaseDir.EnumerateDirectories().Select(d => d.Location));
				}

				if (baseDirItem.TryGetDirectory("Restricted", out DirectoryItem? restrictedBaseDir))
				{
					IEnumerable<DirectoryItem> restrictedDirs = restrictedBaseDir.EnumerateDirectories();
					newCachedDirs.Item2.AddRange(restrictedDirs.Select(d => d.Location));

					// also look for nested platforms in the restricted
					foreach (DirectoryItem restrictedDir in restrictedDirs)
					{
						if (restrictedDir.TryGetDirectory("Platforms", out DirectoryItem? restrictedPlatformExtensionBaseDir))
						{
							newCachedDirs.Item1.AddRange(restrictedPlatformExtensionBaseDir.EnumerateDirectories().Select(d => d.Location));
						}
					}
				}

				// remove any platform directories in non-engine locations if the engine doesn't have the platform 
				if (baseDir != EngineDirectory && newCachedDirs.Item1.Count > 0)
				{
					// if the DDPI.ini file doesn't exist, we haven't synced the platform, so just skip this directory
					newCachedDirs.Item1.RemoveAll(x => !DataDrivenPlatformInfoIniIsPresent(x.GetDirectoryName()));
				}
				return newCachedDirs;
			})).Value;

			// now return what the caller wanted (always include BaseDir)
			List<DirectoryReference> extensionDirs = [];
			if (bIncludeBaseDirectory)
			{
				extensionDirs.Add(baseDir);
			}
			if (bIncludePlatformDirectories)
			{
				extensionDirs.AddRange(cachedDirs.Item1);
			}
			if (bIncludeRestrictedDirectories)
			{
				extensionDirs.AddRange(cachedDirs.Item2);
			}
			return extensionDirs;
		}

		/// <summary>
		/// Finds all the extension directories for the given base directory. This includes platform extensions and restricted folders.f
		/// </summary>
		/// <param name="baseDir">Location of the base directory</param>
		/// <param name="subDir">The subdirectory to find</param>
		/// <param name="bIncludePlatformDirectories">If true, platform subdirectories are included (will return platform directories under Restricted dirs, even if bIncludeRestrictedDirectories is false)</param>
		/// <param name="bIncludeRestrictedDirectories">If true, restricted (NotForLicensees, NoRedist) subdirectories are included</param>
		/// <param name="bIncludeBaseDirectory">If true, baseDir is included</param>
		/// <returns>List of extension directories, including the given base directory</returns>
		public static List<DirectoryReference> GetExtensionDirs(DirectoryReference baseDir, string subDir, bool bIncludePlatformDirectories = true, bool bIncludeRestrictedDirectories = true, bool bIncludeBaseDirectory = true)
		{
			return [.. GetExtensionDirs(baseDir, bIncludePlatformDirectories, bIncludeRestrictedDirectories, bIncludeBaseDirectory)
				.Select(x => DirectoryReference.Combine(x, subDir))
				.Where(DirectoryReference.Exists)];
		}

		/// <summary>
		/// Returns the Application Settings Directory path. This matches FPlatformProcess::ApplicationSettingsDir().
		/// </summary>
		private static DirectoryReference GetApplicationSettingDirectory() =>
			OperatingSystem.IsMacOS() || OperatingSystem.IsLinux()
				? new DirectoryReference(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), "Epic"))
				: new DirectoryReference(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.CommonApplicationData), "Epic"));

		/// <summary>
		/// Returns the User Settings Directory path. This matches FPlatformProcess::UserSettingsDir().
		/// </summary>
		private static DirectoryReference GetUserSettingDirectory()
		{
			if (OperatingSystem.IsMacOS() || OperatingSystem.IsLinux())
			{
				// Mac and Linux use the same folder for UserSettingsDir and ApplicationSettingsDir
				return GetApplicationSettingDirectory();
			}
			else
			{
				// Not all user accounts have a local application data directory (eg. SYSTEM, used by Jenkins for builds).
				List<Environment.SpecialFolder> dataFolders = [
					Environment.SpecialFolder.LocalApplicationData,
					Environment.SpecialFolder.CommonApplicationData
				];
				foreach (Environment.SpecialFolder dataFolder in dataFolders)
				{
					string directoryName = Environment.GetFolderPath(dataFolder);
					if (!String.IsNullOrEmpty(directoryName))
					{
						return new(directoryName);
					}
				}
			}

			return DirectoryReference.Combine(EngineDirectory, "Saved");
		}

		/// <summary>
		/// Returns the User Directory path. This matches FPlatformProcess::UserDir().
		/// </summary>
		private static DirectoryReference? GetUserDirectory()
		{
			// Some user accounts (eg. SYSTEM on Windows) don't have a home directory. Ignore them if Environment.GetFolderPath() returns an empty string.
			string personalFolder = Environment.GetFolderPath(Environment.SpecialFolder.Personal);
			if (!String.IsNullOrEmpty(personalFolder))
			{
				return OperatingSystem.IsMacOS() || OperatingSystem.IsLinux()
					? new(Path.Combine(personalFolder, "Documents"))
					: new(personalFolder);
			}
			return null;
		}

		/// <summary>
		/// The current Machine name
		/// </summary>
		public static string MachineName => s_machineName.Value;
		private static readonly Lazy<string> s_machineName = new(() =>
		{
			try
			{
				// this likely can't fail, but just in case, fallback to Environment implementation
				string machineName = System.Net.Dns.GetHostName();

				if (OperatingSystem.IsMacOS() && machineName.EndsWith(".local", StringComparison.OrdinalIgnoreCase))
				{
					machineName = machineName.Replace(".local", String.Empty, StringComparison.OrdinalIgnoreCase);
				}
				return machineName;
			}
			catch (Exception)
			{
			}
			return Environment.MachineName;
		});
	}
}
