<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.BuildGraph\EpicGames.BuildGraph.csproj" />
  </ItemGroup>
</Project>
