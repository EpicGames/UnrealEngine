// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.BuildGraph.Expressions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.BuildGraph.Tests
{
	[TestClass]
	public class GraphTests
	{
		static Task UpdateVersionFiles() => Task.CompletedTask;
		static Task CompileShooterGameWin64() => Task.CompletedTask;
		static Task CookShooterGameWin64() => Task.CompletedTask;
		static Task PackageShooterGameWin64() => Task.CompletedTask;

		static object Evaluate(BgExpr expr)
		{
			(byte[] data, BgThunkDef[] methods) = BgCompiler.Compile(expr);
			BgInterpreter interpreter = new(data, methods, new Dictionary<string, string>());
			return interpreter.Evaluate();
		}

		[TestMethod]
		public void NodeTest()
		{
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());

			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).Requires(nodeSpec1);

			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).Requires(BgList<BgFileSet>.Create(nodeSpec2.DefaultOutput));

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();
			Assert.AreEqual("Cook Shooter Game Win64", node3.Name);
			Assert.AreEqual(2, node3.InputDependencies.Count);
			Assert.AreEqual(1, node3.Inputs.Count);
			Assert.AreEqual(0, node3.OptionalInputs.Count);

			BgNodeDef node2 = node3.Inputs[0].ProducingNode;
			Assert.AreEqual("Compile Shooter Game Win64", node2.Name);
			Assert.AreEqual(1, node2.InputDependencies.Count);
			Assert.AreEqual(1, node2.Inputs.Count);
			Assert.AreEqual(0, node2.OptionalInputs.Count);

			BgNodeDef node1 = node2.Inputs[0].ProducingNode;
			Assert.AreEqual("Update Version Files", node1.Name);
			Assert.AreEqual(0, node1.InputDependencies.Count);
			Assert.AreEqual(0, node1.Inputs.Count);
			Assert.AreEqual(0, node1.OptionalInputs.Count);
		}

		[TestMethod]
		public void NodeWithOptionalRequiresTest()
		{
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());

			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).OptionalRequires(nodeSpec1);

			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).OptionalRequires(BgList<BgFileSet>.Create(nodeSpec2.DefaultOutput));

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();
			Assert.AreEqual("Cook Shooter Game Win64", node3.Name);
			Assert.AreEqual(0, node3.InputDependencies.Count);
			Assert.AreEqual(0, node3.Inputs.Count);
			Assert.AreEqual(1, node3.OptionalInputs.Count);

			BgNodeDef node2 = node3.OptionalInputs[0].ProducingNode;
			Assert.AreEqual("Compile Shooter Game Win64", node2.Name);
			Assert.AreEqual(0, node2.InputDependencies.Count);
			Assert.AreEqual(0, node2.Inputs.Count);
			Assert.AreEqual(1, node2.OptionalInputs.Count);

			BgNodeDef node1 = node2.OptionalInputs[0].ProducingNode;
			Assert.AreEqual("Update Version Files", node1.Name);
			Assert.AreEqual(0, node1.InputDependencies.Count);
			Assert.AreEqual(0, node1.Inputs.Count);
			Assert.AreEqual(0, node1.OptionalInputs.Count);
		}

		[TestMethod]
		public void NodeWithRequiresAndOptionalRequiresTest()
		{
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());

			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).Requires(nodeSpec1);

			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).Requires(nodeSpec1);

			BgNode nodeSpec4 = agent.AddNode(x => PackageShooterGameWin64()).
				Requires(BgList<BgFileSet>.Create(nodeSpec2.DefaultOutput)).
				OptionalRequires(BgList<BgFileSet>.Create(nodeSpec3.DefaultOutput));

			BgNodeDef node4 = ((BgObjectDef)Evaluate(nodeSpec4)).Deserialize<BgNodeExpressionDef>();
			Assert.AreEqual("Package Shooter Game Win64", node4.Name);
			Assert.AreEqual(2, node4.InputDependencies.Count);
			Assert.AreEqual(1, node4.Inputs.Count);
			Assert.AreEqual(1, node4.OptionalInputs.Count);

			BgNodeDef node3 = node4.OptionalInputs[0].ProducingNode;
			Assert.AreEqual("Cook Shooter Game Win64", node3.Name);
			Assert.AreEqual(1, node3.InputDependencies.Count);
			Assert.AreEqual(1, node3.Inputs.Count);
			Assert.AreEqual(0, node3.OptionalInputs.Count);

			BgNodeDef node2 = node4.Inputs[0].ProducingNode;
			Assert.AreEqual("Compile Shooter Game Win64", node2.Name);
			Assert.AreEqual(1, node2.InputDependencies.Count);
			Assert.AreEqual(1, node2.Inputs.Count);
			Assert.AreEqual(0, node2.OptionalInputs.Count);

			BgNodeDef node1 = node2.Inputs[0].ProducingNode;
			Assert.AreEqual("Update Version Files", node1.Name);
			Assert.AreEqual(0, node1.InputDependencies.Count);
			Assert.AreEqual(0, node1.Inputs.Count);
			Assert.AreEqual(0, node1.OptionalInputs.Count);
		}

		[TestMethod]
		public void OptionalRequiresChainsThruRequiredDependenciesTest()
		{
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).Requires(nodeSpec1);
			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).OptionalRequires(nodeSpec2);

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();

			// Node 3 has no required dependencies
			Assert.IsEmpty(node3.InputDependencies);
			Assert.IsEmpty(node3.Inputs);

			// Flattened outputs from Node 2; includes Node 2's output + Node 1's output
			Assert.HasCount(2, node3.OptionalInputs);
			Assert.IsTrue(node3.OptionalInputs.Any(x => x.ProducingNode.Name == "Update Version Files"));
			Assert.IsTrue(node3.OptionalInputs.Any(x => x.ProducingNode.Name == "Compile Shooter Game Win64"));

			// Recursive chain; Node 1 + Node 2
			Assert.HasCount(2, node3.OptionalInputDependencies);
			Assert.IsTrue(node3.OptionalInputDependencies.Any(x => x.Name == "Update Version Files"));
			Assert.IsTrue(node3.OptionalInputDependencies.Any(x => x.Name == "Compile Shooter Game Win64"));
		}

		[TestMethod]
		public void RequiredDependencyChainsThruOptionalDependenciesTest()
		{
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).OptionalRequires(nodeSpec1);
			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).Requires(nodeSpec2);

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();

			// When Node 3 requires Node 2 includes:
			// - Node 2 InputDependencies outputs (none)
			// - Node 2 OptionalInputDependencies outputs (Node 1)
			// - Node 2 Outputs
			// Node 3's InputDependencies includes both Node 1 and Node 2
			Assert.HasCount(2, node3.InputDependencies);
			Assert.IsTrue(node3.InputDependencies.Any(x => x.Name == "Compile Shooter Game Win64"));
			Assert.IsTrue(node3.InputDependencies.Any(x => x.Name == "Update Version Files"));

			// Node 3 should have no optional dependencies
			Assert.IsEmpty(node3.OptionalInputDependencies);

			// Node 2's optional dependency on Node 1
			BgNodeDef node2 = node3.InputDependencies.First(x => x.Name == "Compile Shooter Game Win64");
			Assert.IsEmpty(node2.InputDependencies);
			Assert.HasCount(1, node2.OptionalInputDependencies);
			Assert.IsTrue(node2.OptionalInputDependencies.Any(x => x.Name == "Update Version Files"));
		}

		[TestMethod]
		public void GetDirectOptionalInputDependenciesExcludesTransitiveRequiredTest()
		{
			// A --req--> B --opt--> C
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).Requires(nodeSpec1);
			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).OptionalRequires(nodeSpec2);

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();

			// Flattened contains both
			Assert.HasCount(2, node3.OptionalInputDependencies);

			// Direct should only contain B (Compile Shooter Game Win64)
			// A (Update Version Files) is transitive through B's required chain
			List<BgNodeDef> directOptionalDeps = [.. node3.GetDirectOptionalInputDependencies()];
			Assert.HasCount(1, directOptionalDeps);
			Assert.AreEqual("Compile Shooter Game Win64", directOptionalDeps[0].Name);
		}

		[TestMethod]
		public void GetDirectInputDependenciesExcludesTransitiveOptionalTest()
		{
			// A --opt--> B --req--> C
			BgAgent agent = new("name", "type");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).OptionalRequires(nodeSpec1);
			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).Requires(nodeSpec2);

			BgNodeDef node3 = ((BgObjectDef)Evaluate(nodeSpec3)).Deserialize<BgNodeExpressionDef>();

			// Flattened contains both
			Assert.HasCount(2, node3.InputDependencies);

			// Direct should only contain B (Compile Shooter Game Win64)
			// A (Update Version Files) is transitive through B's optional chain
			List<BgNodeDef> directInputDeps = [.. node3.GetDirectInputDependencies()];
			Assert.HasCount(1, directInputDeps);
			Assert.AreEqual("Compile Shooter Game Win64", directInputDeps[0].Name);
		}

		[TestMethod]
		public void AgentTest()
		{
			BgAgent expr = new("TestAgent", BgList<BgString>.Create("win64", "incremental"));

			BgAgentDef agent = ((BgObjectDef)Evaluate(expr)).Deserialize<BgAgentDef>();
			Assert.AreEqual("TestAgent", agent.Name);

			Assert.AreEqual(2, agent.PossibleTypes.Count);
			Assert.AreEqual("win64", agent.PossibleTypes[0]);
			Assert.AreEqual("incremental", agent.PossibleTypes[1]);
			/*
						Assert.AreEqual(1, agent.Nodes.Count);

						BgNode node = agent.Nodes[0];
						Assert.AreEqual("Update Version Files", node.Name);
			*/
		}

		[TestMethod]
		public void AggregateTest()
		{
			BgAgent agent = new("test", "test");

			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64());

			BgLabel labelSpec = new("name", "category");
			BgAggregate aggregateSpec = new("All nodes", BgList.Create(nodeSpec1, nodeSpec2), labelSpec);

			BgAggregateExpressionDef aggregateExpr = ((BgObjectDef)Evaluate(aggregateSpec)).Deserialize<BgAggregateExpressionDef>();
			BgAggregateDef aggregate = aggregateExpr.ToAggregateDef();
			Assert.AreEqual("All nodes", aggregate.Name);

			BgNodeDef node1 = aggregate.RequiredNodes.OrderBy(x => x.Name).ElementAt(0);
			Assert.AreEqual("Compile Shooter Game Win64", node1.Name);

			BgNodeDef node2 = aggregate.RequiredNodes.OrderBy(x => x.Name).ElementAt(1);
			Assert.AreEqual("Update Version Files", node2.Name);

			BgLabelDef? label = aggregateExpr.Label;
			Assert.IsNotNull(label);
			Assert.AreEqual("name", label!.DashboardName);
			Assert.AreEqual("category", label!.DashboardCategory);
		}

		[TestMethod]
		public void LabelTest()
		{
			BgLabel labelSpec = new("name", "category", "ugsBadge", "ugsProject", BgLabelChange.Code);

			BgLabelDef label = ((BgObjectDef)Evaluate(labelSpec)).Deserialize<BgLabelDef>();
			Assert.AreEqual("name", label.DashboardName);
			Assert.AreEqual("category", label.DashboardCategory);
			Assert.AreEqual("ugsBadge", label.UgsBadge);
			Assert.AreEqual("ugsProject", label.UgsProject);
			Assert.AreEqual(BgLabelChange.Code, label.Change);
		}

		[TestMethod]
		public void GraphTest()
		{
			BgAgent agent = new("test", "test");

			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64());

			BgAggregate aggregateSpec = new("All nodes", BgList.Create(nodeSpec1, nodeSpec2), new BgLabel("name", "category"));
			BgGraph graphSpec = new(BgList.Create(nodeSpec1, nodeSpec2), BgList.Create(aggregateSpec));

			BgGraphDef graph = ((BgObjectDef)Evaluate(graphSpec)).Deserialize<BgGraphExpressionDef>().ToGraphDef();
			Assert.AreEqual(2, graph.NameToNode.Count);
			Assert.AreEqual(1, graph.NameToAggregate.Count);
			Assert.AreEqual(1, graph.Labels.Count);

			BgNodeDef node1 = graph.NameToNode["Update Version Files"];

			BgNodeDef node2 = graph.NameToNode["Compile Shooter Game Win64"];

			BgLabelDef label = graph.Labels[0];
			Assert.AreEqual("name", label!.DashboardName);
			Assert.AreEqual("category", label!.DashboardCategory);
			Assert.IsTrue(label!.RequiredNodes.Contains(node1));
			Assert.IsTrue(label!.RequiredNodes.Contains(node2));
			Assert.IsTrue(label!.IncludedNodes.Contains(node1));
			Assert.IsTrue(label!.IncludedNodes.Contains(node2));
		}

		[TestMethod]
		public void TryResolveInputReferenceIncludesOptionalInputsTest()
		{
			BgAgent agent = new("test", "test");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).Requires(nodeSpec1);
			BgNode nodeSpec3 = agent.AddNode(x => CookShooterGameWin64()).OptionalRequires(nodeSpec2);

			BgGraph graphSpec = new(BgList.Create(nodeSpec1, nodeSpec2, nodeSpec3), BgList<BgAggregate>.Empty);
			BgGraphDef graph = ((BgObjectDef)Evaluate(graphSpec)).Deserialize<BgGraphExpressionDef>().ToGraphDef();

			// Resolving by node name should return outputs + inputs + optional inputs
			bool resolved = graph.TryResolveInputReference("Cook Shooter Game Win64", out BgNodeOutput[]? outputs);
			Assert.IsTrue(resolved);
			Assert.IsNotNull(outputs);
			Assert.HasCount(3, outputs);

			string[] tagNames = [.. outputs.Select(x => x.TagName)];
			Assert.Contains("#Cook Shooter Game Win64", tagNames);
			Assert.Contains("#Compile Shooter Game Win64", tagNames);
			Assert.Contains("#Update Version Files", tagNames);
		}

		[TestMethod]
		public void TryResolveInputReferenceAggregateIncludesOptionalInputsTest()
		{
			BgAgent agent = new("test", "test");
			BgNode nodeSpec1 = agent.AddNode(x => UpdateVersionFiles());
			BgNode nodeSpec2 = agent.AddNode(x => CompileShooterGameWin64()).OptionalRequires(nodeSpec1);

			BgAggregate aggregateSpec = new("All nodes", BgList.Create(nodeSpec2));
			BgGraph graphSpec = new(BgList.Create(nodeSpec1, nodeSpec2), BgList.Create(aggregateSpec));
			BgGraphDef graph = ((BgObjectDef)Evaluate(graphSpec)).Deserialize<BgGraphExpressionDef>().ToGraphDef();

			// Resolving aggregate should include optional inputs from its required nodes
			bool resolved = graph.TryResolveInputReference("All nodes", out BgNodeOutput[]? outputs);
			Assert.IsTrue(resolved);
			Assert.IsNotNull(outputs);
			Assert.HasCount(2, outputs);

			string[] tagNames = [.. outputs.Select(x => x.TagName)];
			Assert.Contains("#Compile Shooter Game Win64", tagNames);
			Assert.Contains("#Update Version Files", tagNames);
		}
	}
}
