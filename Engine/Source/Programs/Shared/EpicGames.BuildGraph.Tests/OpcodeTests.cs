// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.BuildGraph.Expressions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.BuildGraph.Tests
{
	[TestClass]
	public class OpcodeTests
	{
		static object Evaluate(BgExpr expr)
		{
			(byte[] data, BgThunkDef[] methods) = BgCompiler.Compile(expr);
			BgInterpreter interpreter = new(data, methods, new Dictionary<string, string>());
			return interpreter.Evaluate();
		}

		static BgOptionDef EvaluateOption(BgExpr expr)
		{
			(byte[] data, BgThunkDef[] methods) = BgCompiler.Compile(expr);
			BgInterpreter interpreter = new(data, methods, new Dictionary<string, string>());
			interpreter.Evaluate();
			return interpreter.OptionDefs[0];
		}

		#region Boolean opcodes

		[TestMethod]
		public void BoolFalse()
		{
			object result = Evaluate((BgBool)false);
			Assert.AreEqual(false, result);
		}

		[TestMethod]
		public void BoolTrue()
		{
			object result = Evaluate((BgBool)true);
			Assert.AreEqual(true, result);
		}

		[TestMethod]
		public void BoolNot()
		{
			object result1 = Evaluate(!(BgBool)true);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate(!(BgBool)false);
			Assert.AreEqual(true, result2);
		}

		[TestMethod]
		public void BoolAnd()
		{
			object result1 = Evaluate((BgBool)false & (BgBool)false);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate((BgBool)false & (BgBool)true);
			Assert.AreEqual(false, result2);

			object result3 = Evaluate((BgBool)true & (BgBool)false);
			Assert.AreEqual(false, result3);

			object result4 = Evaluate((BgBool)true & (BgBool)true);
			Assert.AreEqual(true, result4);
		}

		[TestMethod]
		public void BoolOr()
		{
			object result1 = Evaluate((BgBool)false | (BgBool)false);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate((BgBool)false | (BgBool)true);
			Assert.AreEqual(true, result2);

			object result3 = Evaluate((BgBool)true | (BgBool)false);
			Assert.AreEqual(true, result3);

			object result4 = Evaluate((BgBool)true | (BgBool)true);
			Assert.AreEqual(true, result4);
		}

		[TestMethod]
		public void BoolXor()
		{
			object result1 = Evaluate((BgBool)false ^ (BgBool)false);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate((BgBool)false ^ (BgBool)true);
			Assert.AreEqual(true, result2);

			object result3 = Evaluate((BgBool)true ^ (BgBool)false);
			Assert.AreEqual(true, result3);

			object result4 = Evaluate((BgBool)true ^ (BgBool)true);
			Assert.AreEqual(false, result4);
		}

		[TestMethod]
		public void BoolEq()
		{
			object result1 = Evaluate((BgBool)false == (BgBool)false);
			Assert.AreEqual(true, result1);

			object result2 = Evaluate((BgBool)false == (BgBool)true);
			Assert.AreEqual(false, result2);

			object result3 = Evaluate((BgBool)true == (BgBool)false);
			Assert.AreEqual(false, result3);

			object result4 = Evaluate((BgBool)true == (BgBool)true);
			Assert.AreEqual(true, result4);
		}

		[TestMethod]
		public void BoolOption()
		{
			BgBoolOptionDef result = (BgBoolOptionDef)EvaluateOption(new BgBoolOption("boolOption", "description", defaultValue: true));
			Assert.AreEqual("boolOption", result.Name);
			Assert.AreEqual("description", result.Description);
			Assert.AreEqual(true, result.DefaultValue);
		}

		#endregion

		#region Integer opcodes

		[TestMethod]
		public void IntLiteral()
		{
			object result = Evaluate((BgInt)123);
			Assert.AreEqual(123, result);
		}

		[TestMethod]
		public void IntEq()
		{
			object result1 = Evaluate((BgInt)123 == (BgInt)124);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate((BgInt)123 == (BgInt)123);
			Assert.AreEqual(true, result2);

			object result3 = Evaluate((BgInt)123 == (BgInt)122);
			Assert.AreEqual(false, result3);
		}

		[TestMethod]
		public void IntLt()
		{
			object result1 = Evaluate((BgInt)123 < (BgInt)124);
			Assert.AreEqual(true, result1);

			object result2 = Evaluate((BgInt)123 < (BgInt)123);
			Assert.AreEqual(false, result2);

			object result3 = Evaluate((BgInt)123 < (BgInt)122);
			Assert.AreEqual(false, result3);
		}

		[TestMethod]
		public void IntGt()
		{
			object result1 = Evaluate((BgInt)123 > (BgInt)124);
			Assert.AreEqual(false, result1);

			object result2 = Evaluate((BgInt)123 > (BgInt)123);
			Assert.AreEqual(false, result2);

			object result3 = Evaluate((BgInt)123 > (BgInt)122);
			Assert.AreEqual(true, result3);
		}

		[TestMethod]
		public void IntAdd()
		{
			object result = Evaluate((BgInt)1 + (BgInt)2);
			Assert.AreEqual(3, result);
		}

		[TestMethod]
		public void IntMultiply()
		{
			object result = Evaluate((BgInt)3 * (BgInt)7);
			Assert.AreEqual(21, result);
		}

		[TestMethod]
		public void IntDivide()
		{
			object result = Evaluate((BgInt)21 / (BgInt)3);
			Assert.AreEqual(7, result);
		}

		[TestMethod]
		public void IntModulo()
		{
			object result = Evaluate((BgInt)23 % (BgInt)7);
			Assert.AreEqual(2, result);
		}

		[TestMethod]
		public void IntNegate()
		{
			object result = Evaluate(-(BgInt)123);
			Assert.AreEqual(-123, result);
		}

		[TestMethod]
		public void IntOption()
		{
			BgIntOptionDef result = (BgIntOptionDef)EvaluateOption(new BgIntOption("intOption", "description", defaultValue: 1, minValue: 4, maxValue: 5));
			Assert.AreEqual("intOption", result.Name);
			Assert.AreEqual("description", result.Description);
			Assert.AreEqual(1, result.DefaultValue);
			Assert.AreEqual(4, result.MinValue);
			Assert.AreEqual(5, result.MaxValue);
		}

		#endregion
		#region String opcodes

		[TestMethod]
		public void StrEmpty()
		{
			object result = Evaluate(BgString.Empty);
			Assert.AreEqual(String.Empty, result);
		}

		[TestMethod]
		public void StrLiteral()
		{
			object result = Evaluate((BgString)"hello world");
			Assert.AreEqual("hello world", result);
		}

		[TestMethod]
		public void StrCompare()
		{
			{
				object result1 = Evaluate((BgString)"hello world" == (BgString)"hello world");
				Assert.AreEqual(true, result1);

				object result2 = Evaluate((BgString)"hello world" == (BgString)"HELLO WORLD");
				Assert.AreEqual(false, result2);

				object result3 = Evaluate((BgString)"HELLO WORLD" == (BgString)"hello world");
				Assert.AreEqual(false, result3);

				object result4 = Evaluate((BgString)"HELLO WORLD" == (BgString)"HELLO WORLD");
				Assert.AreEqual(true, result4);
			}

			{
				int result1 = (int)Evaluate(BgString.Compare("hello world", "hello world", StringComparison.Ordinal));
				Assert.IsTrue(result1 == 0);

				int result2 = (int)Evaluate(BgString.Compare("hello world", "HELLO WORLD", StringComparison.Ordinal));
				Assert.IsTrue(result2 > 0);

				int result3 = (int)Evaluate(BgString.Compare("HELLO WORLD", "hello world", StringComparison.Ordinal));
				Assert.IsTrue(result3 < 0);

				int result4 = (int)Evaluate(BgString.Compare("HELLO WORLD", "HELLO WORLD", StringComparison.Ordinal));
				Assert.IsTrue(result4 == 0);
			}

			{
				int result1 = (int)Evaluate(BgString.Compare("hello world", "hello world", StringComparison.OrdinalIgnoreCase));
				Assert.IsTrue(result1 == 0);

				int result2 = (int)Evaluate(BgString.Compare("hello world", "HELLO WORLD", StringComparison.OrdinalIgnoreCase));
				Assert.IsTrue(result2 == 0);

				int result3 = (int)Evaluate(BgString.Compare("HELLO WORLD", "hello world", StringComparison.OrdinalIgnoreCase));
				Assert.IsTrue(result3 == 0);

				int result4 = (int)Evaluate(BgString.Compare("HELLO WORLD", "HELLO WORLD", StringComparison.OrdinalIgnoreCase));
				Assert.IsTrue(result4 == 0);

				int result5 = (int)Evaluate(BgString.Compare("HELLO WORLD", "HELLO WORLD2", StringComparison.OrdinalIgnoreCase));
				Assert.IsTrue(result5 < 0);
			}
		}

		[TestMethod]
		public void StrConcat()
		{
			object result = Evaluate((BgString)"hello" + (BgString)" " + (BgString)"world");
			Assert.AreEqual("hello world", result);
		}

		[TestMethod]
		public void StrFormat()
		{
			object result = Evaluate(BgString.Format("result is {0}", (BgInt)1 + (BgInt)2));
			Assert.AreEqual("result is 3", result);
		}

		[TestMethod]
		public void StrSplit()
		{
			object result = Evaluate(((BgString)"1+2+3").Split("+"));
			Assert.IsTrue(Enumerable.SequenceEqual((IEnumerable<object>)result, ["1", "2", "3"]));
		}

		[TestMethod]
		public void StrJoin()
		{
			object result = Evaluate(BgString.Join("+", BgList<BgString>.Create("1", "2", "3")));
			Assert.AreEqual("1+2+3", result);
		}

		[TestMethod]
		public void StrMatch()
		{
			object result1 = Evaluate(((BgString)"hello world").Match("wor"));
			Assert.AreEqual(true, result1);

			object result2 = Evaluate(((BgString)"hello world").Match("war"));
			Assert.AreEqual(false, result2);

			object result3 = Evaluate(((BgString)"hello world").Match(@"^hello"));
			Assert.AreEqual(true, result3);

			object result4 = Evaluate(((BgString)"hello world").Match(@"^world"));
			Assert.AreEqual(false, result4);
		}

		[TestMethod]
		public void StrReplace()
		{
			object result1 = Evaluate(((BgString)"hello world").Replace("(w[orl]+d)", "-> $1"));
			Assert.AreEqual("hello -> world", result1);
		}

		[TestMethod]
		public void StrOption()
		{
			BgStringOptionDef result = (BgStringOptionDef)EvaluateOption(new BgStringOption("stringOption", "description", defaultValue: "value", pattern: "abc", patternFailed: "def", values: BgList<BgString>.Create("value1", "value2"), valueDescriptions: BgList<BgString>.Create("desc1", "desc2")));
			Assert.AreEqual("stringOption", result.Name);
			Assert.AreEqual("description", result.Description);
			Assert.AreEqual("value", result.DefaultValue);
			Assert.AreEqual("abc", result.Pattern);
			Assert.AreEqual("def", result.PatternFailed);
			Assert.IsTrue(result.Values!.SequenceEqual(["value1", "value2"]));
			Assert.IsTrue(result.ValueDescriptions!.SequenceEqual(["desc1", "desc2"]));
		}

		#endregion

		#region Enum opcodes

		enum TestEnum
		{
			ValueIs123 = 123,
			ValueIs456 = 456
		}

		[TestMethod]
		public void EnumConstant()
		{
			object result = Evaluate((BgEnum<TestEnum>)TestEnum.ValueIs123);
			Assert.AreEqual(123, result);
		}

		[TestMethod]
		public void EnumParse()
		{
			object result1 = Evaluate(BgEnum<TestEnum>.Parse(nameof(TestEnum.ValueIs123)));
			Assert.AreEqual((int)TestEnum.ValueIs123, result1);

			object result2 = Evaluate(BgEnum<TestEnum>.Parse(nameof(TestEnum.ValueIs456)));
			Assert.AreEqual((int)TestEnum.ValueIs456, result2);

			Assert.ThrowsExactly<InvalidDataException>(() => Evaluate(BgEnum<TestEnum>.Parse("Other")));
		}

		[TestMethod]
		public void EnumToString()
		{
			object result1 = Evaluate(((BgEnum<TestEnum>)TestEnum.ValueIs123).ToBgString());
			Assert.AreEqual(nameof(TestEnum.ValueIs123), result1);

			object result2 = Evaluate(((BgEnum<TestEnum>)TestEnum.ValueIs456).ToBgString());
			Assert.AreEqual(nameof(TestEnum.ValueIs456), result2);

			object result3 = Evaluate(((BgEnum<TestEnum>)(TestEnum)789).ToBgString());
			Assert.AreEqual("789", result3);
		}

		#endregion

		#region List opcodes

		[TestMethod]
		public void ListEmpty()
		{
			object result = Evaluate(BgList<BgInt>.Empty);
			Assert.IsTrue(Enumerable.SequenceEqual([], (IEnumerable<object>)result));
		}

		[TestMethod]
		public void ListCount()
		{
			object result = Evaluate(BgList<BgInt>.Empty.Add(1, 2, 3).Count);
			Assert.AreEqual(3, result);
		}

		[TestMethod]
		public void ListElement()
		{
			object result = Evaluate(BgList<BgInt>.Create(1, 2, 3)[2]);
			Assert.AreEqual(3, result);
		}

		[TestMethod]
		public void ListPush()
		{
			object result1 = Evaluate(BgList<BgInt>.Empty.Add(1));
			Assert.IsTrue(Enumerable.SequenceEqual([1], (IEnumerable<object>)result1));

			object result2 = Evaluate(BgList<BgInt>.Empty.Add(1, 2).Add(3));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2, 3], (IEnumerable<object>)result2));
		}

		[TestMethod]
		public void ListConcat()
		{
			object result = Evaluate(BgList<BgInt>.Concat(BgList<BgInt>.Create(1, 2), BgList<BgInt>.Create(3)));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2, 3], (IEnumerable<object>)result));
		}

		[TestMethod]
		public void ListUnion()
		{
			object result1 = Evaluate(BgList<BgInt>.Create(1, 2).Union(BgList<BgInt>.Create(3)));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2, 3], (IEnumerable<object>)result1));

			object result2 = Evaluate(BgList<BgInt>.Create(1, 2, 3).Union(BgList<BgInt>.Create(3)));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2, 3], (IEnumerable<object>)result2));
		}

		[TestMethod]
		public void ListExcept()
		{
			object result1 = Evaluate(BgList<BgInt>.Create(1, 2).Except(BgList<BgInt>.Create(3)));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2], (IEnumerable<object>)result1));

			object result2 = Evaluate(BgList<BgInt>.Create(1, 2, 3).Except(BgList<BgInt>.Create(3)));
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2], (IEnumerable<object>)result2));
		}

		[TestMethod]
		public void ListSelect()
		{
			object result = Evaluate(BgList<BgInt>.Create(1, 1, 2, 2, 3).Select(x => x * 9));
			Assert.IsTrue(Enumerable.SequenceEqual([9, 9, 18, 18, 27], (IEnumerable<object>)result));
		}

		[TestMethod]
		public void ListWhere()
		{
			object result = Evaluate(BgList<BgInt>.Create(1, 1, 2, 2, 3).Where(x => x >= 2));
			Assert.IsTrue(Enumerable.SequenceEqual([2, 2, 3], (IEnumerable<object>)result));
		}

		[TestMethod]
		public void ListDistinct()
		{
			object result = Evaluate(BgList<BgInt>.Concat(BgList<BgInt>.Empty.Add(1, 1, 2, 2, 3), BgList<BgInt>.Empty.Add(3)).Distinct());
			Assert.IsTrue(Enumerable.SequenceEqual([1, 2, 3], (IEnumerable<object>)result));
		}

		[TestMethod]
		public void ListOption()
		{
			BgListOptionDef result = (BgListOptionDef)EvaluateOption(new BgListOption("stringOption", "description", defaultValue: "value", values: BgList<BgString>.Create("value1", "value2"), valueDescriptions: BgList<BgString>.Create("desc1", "desc2")));
			Assert.AreEqual("stringOption", result.Name);
			Assert.AreEqual("description", result.Description);
			Assert.AreEqual("value", result.DefaultValue);
			Assert.IsTrue(result.Values!.SequenceEqual(["value1", "value2"]));
			Assert.IsTrue(result.ValueDescriptions!.SequenceEqual(["desc1", "desc2"]));
		}

		#endregion

		#region Objects

		class TestObject
		{
			public int IntegerValue { get; set; } = 1234;
			public string StringValue { get; set; } = "hello world";
		}

		[TestMethod]
		public void ObjEmpty()
		{
			object result = Evaluate(BgObject<TestObject>.Empty);
			Assert.AreEqual(0, ((BgObjectDef)result).Properties.Count);
		}

		[TestMethod]
		public void ObjSet()
		{
			object result1 = Evaluate(BgObject<TestObject>.Empty.Set<BgInt, int>(x => x.IntegerValue, 999));
			Assert.AreEqual(999, ((BgObjectDef)result1).Get(nameof(TestObject.IntegerValue), null));

			object result2 = Evaluate(BgObject<TestObject>.Empty.Set<BgString, string>(x => x.StringValue, "foo"));
			Assert.AreEqual("foo", ((BgObjectDef)result2).Get(nameof(TestObject.StringValue), null));
		}

		[TestMethod]
		public void ObjGet()
		{
			object result1 = Evaluate(BgObject<TestObject>.Empty.Get<BgInt, int>(x => x.IntegerValue, 1234));
			Assert.AreEqual(1234, result1);

			object result2 = Evaluate(BgObject<TestObject>.Empty.Get<BgString, string>(x => x.StringValue, "hello world"));
			Assert.AreEqual("hello world", result2);
		}

		#endregion

		#region Generic

		[TestMethod]
		public void Throw()
		{
			Assert.ThrowsExactly<BgBytecodeException>(() => Evaluate(BgExpr.Throw<BgInt>("this is an exception")));
		}

		static Task NativeMethod(BgInt a, BgString b, int c, string d) => throw new NotImplementedException();
		static Task<BgFileSet> NativeMethodWithReturnValue(BgInt a, BgString b, int c, string d) => throw new NotImplementedException();

		[TestMethod]
		public void Thunk()
		{
			BgInt a = (BgInt)100 + (BgInt)20 + (BgInt)3;
			BgString b = (BgString)"4" + (BgString)"5" + (BgString)"6";

			{
				BgThunk thunk = BgThunk.Create(() => NativeMethod(a, b, 3, "4"));
				Assert.IsTrue(ReferenceEquals(thunk.Arguments[0], a));
				Assert.IsTrue(ReferenceEquals(thunk.Arguments[1], b));
				Assert.AreEqual(3, thunk.Arguments[2]);
				Assert.AreEqual("4", thunk.Arguments[3]);

				BgThunkDef thunkDef = (BgThunkDef)Evaluate(thunk);
				Assert.IsTrue(thunkDef.Arguments[0] is BgIntConstantExpr constInt && constInt.Value == 123);
				Assert.IsTrue(thunkDef.Arguments[1] is BgStringConstantExpr constStr && constStr.Value == "456");
				Assert.AreEqual(3, thunkDef.Arguments[2]);
				Assert.AreEqual("4", thunkDef.Arguments[3]);
			}
			{
				BgThunk<BgFileSet> thunk = BgThunk.Create(() => NativeMethodWithReturnValue(a, b, 3, "4"));
				Assert.IsTrue(ReferenceEquals(thunk.Arguments[0], a));
				Assert.IsTrue(ReferenceEquals(thunk.Arguments[1], b));
				Assert.AreEqual(3, thunk.Arguments[2]);
				Assert.AreEqual("4", thunk.Arguments[3]);

				BgThunkDef thunkDef = (BgThunkDef)Evaluate(thunk);
				Assert.IsTrue(thunkDef.Arguments[0] is BgIntConstantExpr constInt && constInt.Value == 123);
				Assert.IsTrue(thunkDef.Arguments[1] is BgStringConstantExpr constStr && constStr.Value == "456");
				Assert.AreEqual(3, thunkDef.Arguments[2]);
				Assert.AreEqual("4", thunkDef.Arguments[3]);
			}
		}

		#endregion

		#region Functions

		[TestMethod]
		public void Call()
		{
			BgFunc<BgBool> func = new(() => true);
			object result1 = Evaluate(func.Call());
			Assert.AreEqual(true, result1);

			BgFunc<BgInt, BgInt> func2 = new(x => (x * 3) + 2);
			object result2 = Evaluate(func2.Call(7));
			Assert.AreEqual(23, result2);

			BgFunc<BgInt, BgInt> func3a = new(x => x * 3);
			BgFunc<BgInt, BgInt> func3b = new(x => func3a.Call(x) + 2);
			object result3 = Evaluate(func3b.Call(7));
			Assert.AreEqual(23, result3);

			BgFunc<BgInt, BgInt, BgInt> func4 = new((x, y) => (x + y));
			object result4 = Evaluate(func4.Call(7, 9));
			Assert.AreEqual(16, result4);
		}

		#endregion
	}
}
