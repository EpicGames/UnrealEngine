// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.BuildGraph.Tests
{
	[TestClass]
	public class SerializationTests
	{
		class NestedObject
		{
			public int Value { get; set; }
		}

		class RootObject
		{
			public NestedObject? Nested { get; set; }
			public List<string>? StringList { get; set; }
			public List<NestedObject> NestedList { get; } = [];
		}

		[TestMethod]
		public void BasicTests()
		{
			BgObjectDef def = new BgObjectDef().Set(nameof(RootObject.Nested), new BgObjectDef().Set(nameof(NestedObject.Value), 123));
			RootObject obj = def.Deserialize<RootObject>();
			Assert.AreEqual(123, obj.Nested!.Value);
		}

		[TestMethod]
		public void StringListTests()
		{
			BgObjectDef def = new BgObjectDef().Set(nameof(RootObject.StringList), new[] { "hello", "world" });
			RootObject obj = def.Deserialize<RootObject>();
			Assert.IsNotNull(obj.StringList);
			Assert.AreEqual(2, obj.StringList!.Count);
			Assert.AreEqual("hello", obj.StringList[0]);
			Assert.AreEqual("world", obj.StringList[1]);
		}

		[TestMethod]
		public void ObjectListTests()
		{
			BgObjectDef def = new BgObjectDef().Set(nameof(RootObject.NestedList), new[] { new BgObjectDef().Set(nameof(NestedObject.Value), 123) });
			RootObject obj = def.Deserialize<RootObject>();
			Assert.AreEqual(1, obj.NestedList.Count);
			Assert.AreEqual(123, obj.NestedList[0].Value);
		}
	}
}
