# Dotnet code style settings:
[*.cs]

# CA2227: Collection properties should be read only
dotnet_diagnostic.CA2227.severity = none

# CA1000: Do not declare static members on generic types
dotnet_diagnostic.CA1000.severity = none

# CA1822: Mark members as static
dotnet_diagnostic.CA1822.severity = none

# CA1005: Avoid excessive parameters on generic types
dotnet_diagnostic.CA1005.severity = none

# CA1043: Use Integral Or String Argument For Indexers
dotnet_diagnostic.CA1043.severity = none
