// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Stores a list of nodes which can be executed on a single agent
	/// </summary>
	[DebuggerDisplay("{Name}")]
	[BgObject(typeof(BgAgentDefSerializer))]
	public class BgAgentDef
	{
		/// <summary>
		/// Name of this agent. Used for display purposes in a build system.
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Array of valid agent types that these nodes may run on. When running in the build system, this determines the class of machine that should
		/// be selected to run these nodes. The first defined agent type for this branch will be used.
		/// </summary>
		public List<string> PossibleTypes { get; } = [];

		/// <summary>
		/// List of nodes in this agent group.
		/// </summary>
		public List<BgNodeDef> Nodes { get; set; } = [];

		/// <summary>
		/// Diagnostics to output if executing this agent
		/// </summary>
		public List<BgDiagnosticDef> Diagnostics { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of this agent group</param>
		public BgAgentDef(string name)
		{
			Name = name;
		}
	}

	class BgAgentDefSerializer : BgObjectSerializer<BgAgentDef>
	{
		/// <inheritdoc/>
		public override BgAgentDef Deserialize(BgObjectDef<BgAgentDef> obj)
		{
			BgAgentDef agent = new(obj.Get(x => x.Name, ""));
			obj.CopyTo(agent);
			return agent;
		}
	}
}
