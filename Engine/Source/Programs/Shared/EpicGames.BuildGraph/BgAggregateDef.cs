// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Defines a agggregate within a graph, which give the combined status of one or more job steps, and allow building several steps at once.
	/// </summary>
	public class BgAggregateDef
	{
		/// <summary>
		/// Name of this badge
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Set of nodes that must be run for this label to be shown.
		/// </summary>
		public HashSet<BgNodeDef> RequiredNodes { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of this aggregate</param>
		public BgAggregateDef(string name)
		{
			Name = name;
		}

		/// <summary>
		/// Get the name of this label
		/// </summary>
		/// <returns>The name of this label</returns>
		public override string ToString()
		{
			return Name;
		}
	}

	/// <summary>
	/// Aggregate that was created from bytecode
	/// </summary>
	[BgObject(typeof(BgAggregateExpressionDefSerializer))]
	public class BgAggregateExpressionDef
	{
		/// <inheritdoc cref="BgAggregateDef.Name"/>
		public string Name { get; set; }

		/// <inheritdoc cref="BgAggregateDef.RequiredNodes"/>
		public List<BgNodeExpressionDef> RequiredNodes { get; } = [];

		/// <summary>
		/// Labels to add this aggregate to
		/// </summary>
		public BgLabelDef? Label { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgAggregateExpressionDef(string name)
		{
			Name = name;
		}

		/// <summary>
		/// Construct a BgAggregateDef
		/// </summary>
		/// <returns></returns>
		public BgAggregateDef ToAggregateDef()
		{
			BgAggregateDef aggregate = new(Name);
			aggregate.RequiredNodes.UnionWith(RequiredNodes);
			return aggregate;
		}
	}

	class BgAggregateExpressionDefSerializer : BgObjectSerializer<BgAggregateExpressionDef>
	{
		/// <inheritdoc/>
		public override BgAggregateExpressionDef Deserialize(BgObjectDef<BgAggregateExpressionDef> obj)
		{
			BgAggregateExpressionDef aggregate = new(obj.Get(x => x.Name, ""));
			obj.CopyTo(aggregate);
			return aggregate;
		}
	}
}
