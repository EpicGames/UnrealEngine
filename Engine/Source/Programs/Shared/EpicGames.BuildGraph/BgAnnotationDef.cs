// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Defines an annotation key/value pair for BuildGraph nodes
	/// </summary>
	public class BgAnnotationDef
	{
		/// <summary>
		/// Name, or key, of the annotation
		/// </summary>
		public string Name { get; set; }
		
		/// <summary>
		/// Value to be paired with the annotation name
		/// </summary>
		public string Value { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgAnnotationDef(string name, string value)
		{
			Name = name;
			Value = value;
		}
		
		/// <summary>
		/// Simply the annotation name.
		/// </summary>
		public override string ToString()
		{
			return Name;
		}
	}
}