// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Defines an artifact produced by the build
	/// </summary>
	public class BgArtifactDef
	{
		/// <summary>
		/// Name of this artifact
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Type of this artifact
		/// </summary>
		public string? Type { get; }

		/// <summary>
		/// Description for the artifact
		/// </summary>
		public string? Description { get; }

		/// <summary>
		/// Base path for files included in the artifact. Will be detected from the files specified if not set.
		/// </summary>
		public string? BasePath { get; }

		/// <summary>
		/// Node that produces the artifact. Either this or TagName should be set.
		/// </summary>
		public string? NodeName { get; }

		/// <summary>
		/// Tag to use for the artifact. Uses the artifact name by default.
		/// </summary>
		public string? TagName { get; }

		/// <summary>
		/// Keys that can be used to find the artifact
		/// </summary>
		public IReadOnlyList<string> Keys { get; }

		/// <summary>
		/// Metadata for the artifact
		/// </summary>
		public IReadOnlyList<string> Metadata { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the artifact</param>
		/// <param name="type">Type of the artifact</param>
		/// <param name="description">Description for the artifact</param>
		/// <param name="basePath">Base path for files included in the artifact</param>
		/// <param name="nodeName">Name of the node producing this artifact</param>
		/// <param name="tagName">Name of the tag producing this artifact</param>
		/// <param name="keys">Keys that can be used to find the artifact</param>
		/// <param name="metadata">Metadata for the artifact</param>
		public BgArtifactDef(string name, string? type, string? description, string? basePath, string? nodeName, string? tagName, IReadOnlyList<string> keys, IReadOnlyList<string> metadata)
		{
			Name = name;
			Type = type;
			Description = description;
			BasePath = basePath;
			NodeName = nodeName;
			TagName = tagName;
			Keys = keys;
			Metadata = metadata;
		}

		/// <summary>
		/// Get the name of this badge
		/// </summary>
		/// <returns>The name of this badge</returns>
		public override string ToString()
		{
			return Name;
		}
	}
}
