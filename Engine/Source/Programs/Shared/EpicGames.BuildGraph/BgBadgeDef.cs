// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Defines a badge which gives an at-a-glance summary of part of the build, and can be displayed in UGS
	/// </summary>
	public class BgBadgeDef
	{
		/// <summary>
		/// Name of this badge
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Depot path to the project that this badge applies to. Used for filtering in UGS.
		/// </summary>
		public string Project { get; }

		/// <summary>
		/// The changelist to post the badge for
		/// </summary>
		public int Change { get; }

		/// <summary>
		/// Set of nodes that this badge reports the status of
		/// </summary>
		public HashSet<BgNodeDef> Nodes { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inName">Name of this report</param>
		/// <param name="inProject">Depot path to the project that this badge applies to</param>
		/// <param name="inChange">The changelist to post the badge for</param>
		public BgBadgeDef(string inName, string inProject, int inChange)
		{
			Name = inName;
			Project = inProject;
			Change = inChange;
		}

		/// <summary>
		/// Get the name of this badge
		/// </summary>
		/// <returns>The name of this badge</returns>
		public override string ToString()
		{
			return Name;
		}
	}
}
