// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Helper class for writing BuildGraph bytecode to a buffer.
	/// </summary>
	public abstract class BgBytecodeWriter
	{
		/// <summary>
		/// Writes an opcode to the output
		/// </summary>
		/// <param name="opcode"></param>
		public abstract void WriteOpcode(BgOpcode opcode);

		/// <summary>
		/// Writes a string to the output
		/// </summary>
		/// <param name="str"></param>
		public abstract void WriteString(string str);

		/// <summary>
		/// Writes a reference to an interned string to the output
		/// </summary>
		/// <param name="name">Name to write</param>
		public abstract void WriteName(string name);

		/// <summary>
		/// Writes a signed integer value to the output
		/// </summary>
		/// <param name="value"></param>
		public abstract void WriteSignedInteger(long value);

		/// <summary>
		/// Writes an unsigned integer to the output
		/// </summary>
		/// <param name="value"></param>
		public void WriteUnsignedInteger(int value) => WriteUnsignedInteger((ulong)value);

		/// <summary>
		/// Writes an unsigned integer to the output
		/// </summary>
		/// <param name="value"></param>
		public abstract void WriteUnsignedInteger(ulong value);

		/// <summary>
		/// Writes an expression to the output buffer
		/// </summary>
		/// <param name="expr"></param>
		public abstract void WriteExpr(BgExpr expr);

		/// <summary>
		/// Writes an expression as a standalone fragment, encoding just the fragment index into the output stream
		/// </summary>
		/// <param name="expr"></param>
		public abstract void WriteExprAsFragment(BgExpr expr);

		/// <summary>
		/// Writes a thunk to native code.
		/// </summary>
		/// <param name="thunk">Method to be called</param>
		public abstract void WriteThunk(BgThunkDef thunk);
	}
}
