// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.CompilerServices;
using System.Text;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Version numbers for bytecode streams
	/// </summary>
	enum BgBytecodeVersion
	{
		Current = 0,
	}

	/// <summary>
	/// Helper class for writing BuildGraph bytecode to a buffer.
	/// </summary>
	public static class BgCompiler
	{
		class ReferenceEqualityComparer : IEqualityComparer<object>
		{
			public static readonly ReferenceEqualityComparer Instance = new();

			public new bool Equals([AllowNull] object x, [AllowNull] object y) => ReferenceEquals(x, y);

			[SuppressMessage("MicrosoftCodeAnalysisCorrectness", "RS1024:Compare symbols correctly", Justification = "<Pending>")]
			public int GetHashCode([DisallowNull] object obj) => RuntimeHelpers.GetHashCode(obj);
		}

		class FragmentCollector : BgBytecodeWriter
		{
			readonly HashSet<BgExpr> _uniqueExprs = new(ReferenceEqualityComparer.Instance);
			readonly List<BgExpr> _fragments;
			readonly Dictionary<BgExpr, int> _exprToFragmentIndex;

			public FragmentCollector(List<BgExpr> fragments, Dictionary<BgExpr, int> exprToFragmentIdx)
			{
				_fragments = fragments;
				_exprToFragmentIndex = exprToFragmentIdx;
			}

			/// <inheritdoc/>
			public override void WriteExpr(BgExpr expr)
			{
				if ((expr.Flags & BgExprFlags.ForceFragment) != 0 && false)
				{
					RegisterFragment(expr);
				}
				else if (_uniqueExprs.Add(expr) || (expr.Flags & BgExprFlags.NotInterned) != 0)
				{
					expr.Write(this);
				}
				else
				{
					RegisterFragment(expr);
				}
			}

			/// <inheritdoc/>
			public override void WriteExprAsFragment(BgExpr expr)
			{
				RegisterFragment(expr);

				if (_uniqueExprs.Add(expr))
				{
					expr.Write(this);
				}
			}

			/// <summary>
			/// Registers an expression for compilation into a new fragment
			/// </summary>
			/// <param name="expr"></param>
			void RegisterFragment(BgExpr expr)
			{
				if (!_exprToFragmentIndex.ContainsKey(expr))
				{
					int index = _exprToFragmentIndex.Count;
					_fragments.Add(expr);
					_exprToFragmentIndex[expr] = index;
				}
			}

			/// <inheritdoc/>
			public override void WriteOpcode(BgOpcode opcode) { }

			/// <inheritdoc/>
			public override void WriteName(string str) { }

			/// <inheritdoc/>
			public override void WriteString(string str) { }

			/// <inheritdoc/>
			public override void WriteSignedInteger(long value) { }

			/// <inheritdoc/>
			public override void WriteUnsignedInteger(ulong value) { }

			/// <inheritdoc/>
			public override void WriteThunk(BgThunkDef handler) { }
		}

		class ForwardWriter : BgBytecodeWriter
		{
			readonly ByteArrayBuilder _builder;
			readonly Dictionary<BgExpr, int> _exprToFragmentIdx;
			readonly List<string> _names;
			readonly List<BgThunkDef> _thunks;
			readonly Dictionary<string, int> _nameToIndex = [];

			public ForwardWriter(ByteArrayBuilder builder, Dictionary<BgExpr, int> exprToFragmentIdx, List<string> names, List<BgThunkDef> thunks)
			{
				_builder = builder;
				_exprToFragmentIdx = exprToFragmentIdx;
				_names = names;
				_thunks = thunks;

				for (int idx = 0; idx < _names.Count; idx++)
				{
					_nameToIndex[_names[idx]] = idx;
				}
			}

			/// <inheritdoc/>
			public override void WriteExpr(BgExpr expr)
			{
				int index;
				if (_exprToFragmentIdx.TryGetValue(expr, out index))
				{
					WriteOpcode(BgOpcode.Jump);
					WriteUnsignedInteger((ulong)index);
				}
				else
				{
					expr.Write(this);
				}
			}

			/// <inheritdoc/>
			public override void WriteExprAsFragment(BgExpr expr)
			{
				WriteUnsignedInteger(_exprToFragmentIdx[expr]);
			}

			/// <inheritdoc/>
			public override void WriteOpcode(BgOpcode opcode)
			{
				_builder.WriteUInt8((byte)opcode);
			}

			/// <inheritdoc/>
			public override void WriteName(string name)
			{
				int index;
				if (!_nameToIndex.TryGetValue(name, out index))
				{
					index = _names.Count;
					_names.Add(name);
					_nameToIndex.Add(name, index);
				}
				WriteUnsignedInteger((ulong)index);
			}

			/// <inheritdoc/>
			public override void WriteString(string str)
			{
				int textLength = Encoding.UTF8.GetByteCount(str);
				int lengthLength = VarInt.MeasureUnsigned(textLength);

				Span<byte> buffer = _builder.GetSpanAndAdvance(lengthLength + textLength);
				VarInt.WriteUnsigned(buffer, textLength);
				Encoding.UTF8.GetBytes(str, buffer.Slice(lengthLength));
			}

			/// <inheritdoc/>
			public override void WriteSignedInteger(long value) => _builder.WriteSignedVarInt(value);

			/// <inheritdoc/>
			public override void WriteUnsignedInteger(ulong value) => _builder.WriteUnsignedVarInt(value);

			/// <inheritdoc/>
			public override void WriteThunk(BgThunkDef thunk)
			{
				_builder.WriteUnsignedVarInt(_thunks.Count);
				_thunks.Add(thunk);
			}
		}

		/// <summary>
		/// Compiles the given expression into bytecode
		/// </summary>
		/// <param name="expr">Expression to compile</param>
		/// <returns>Compiled bytecode for the expression, suitable for passing to <see cref="BgInterpreter"/></returns>
		public static (byte[], BgThunkDef[]) Compile(BgExpr expr)
		{
			List<BgExpr> fragments = [];
			Dictionary<BgExpr, int> exprToFragmentIndex = new(ReferenceEqualityComparer.Instance);

			// Figure out which expressions need to be compiled into separate fragments
			FragmentCollector collector = new(fragments, exprToFragmentIndex);
			collector.WriteExprAsFragment(expr);

			// Write all the fragments
			ByteArrayBuilder code = new();
			List<int> fragmentLengths = new(fragments.Count);

			List<BgThunkDef> thunks = [];
			List<string> names = [];

			ForwardWriter writer = new(code, exprToFragmentIndex, names, thunks);
			for (int idx = 0; idx < fragments.Count; idx++)
			{
				int fragmentOffset = code.Length;
				fragments[idx].Write(writer);
				fragmentLengths.Add(code.Length - fragmentOffset);
			}

			// Create the header
			ByteArrayBuilder header = new();
			header.WriteUnsignedVarInt((int)BgBytecodeVersion.Current);
			header.WriteVariableLengthArray(names, x => header.WriteString(x));
			header.WriteVariableLengthArray(fragmentLengths, x => header.WriteUnsignedVarInt(x));

			// Append them together
			byte[] output = new byte[header.Length + code.Length];
			header.CopyTo(output);
			code.CopyTo(output.AsSpan(header.Length));

			return (output, thunks.ToArray());
		}
	}
}
