// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Context for executing a node method
	/// </summary>
	public abstract class BgContext
	{
		/// <summary>
		/// The stream executing the current build
		/// </summary>
		public abstract string Stream { get; }

		/// <summary>
		/// Changelist being built
		/// </summary>
		public abstract int Change { get; }

		/// <summary>
		/// The code changelist currently being built
		/// </summary>
		public abstract int CodeChange { get; }

		/// <summary>
		/// Version number for the engine
		/// </summary>
		public abstract (int Major, int Minor, int Patch) EngineVersion { get; }

		/// <summary>
		/// Whether this machine is a builder
		/// </summary>
		public abstract bool IsBuildMachine { get; }

		/// <summary>
		/// All outputs for the node
		/// </summary>
		public HashSet<FileReference> BuildProducts { get; } = [];

		readonly Dictionary<string, FileSet> _tagNameToFileSet;

		/// <summary>
		/// Constructor
		/// </summary>
		protected BgContext(Dictionary<string, FileSet> tagNameToFileSet)
		{
			_tagNameToFileSet = tagNameToFileSet;
		}

		/// <summary>
		/// Resolve a boolean expression to a value
		/// </summary>
		/// <param name="expr">The boolean expression</param>
		/// <returns>Value of the expression</returns>
		public bool Get(BgBool expr) => ((BgBoolConstantExpr)expr).Value;

		/// <summary>
		/// Resolve an integer expression to a value
		/// </summary>
		/// <param name="expr">The integer expression</param>
		/// <returns>Value of the expression</returns>
		public int Get(BgInt expr) => ((BgIntConstantExpr)expr).Value;

		/// <summary>
		/// Resolve a string expression to a value
		/// </summary>
		/// <param name="expr">The string expression</param>
		/// <returns>Value of the expression</returns>
		public string Get(BgString expr) => ((BgStringConstantExpr)expr).Value;

		/// <summary>
		/// Resolves an enum expression to a value
		/// </summary>
		/// <typeparam name="TEnum">The enum type</typeparam>
		/// <param name="expr">Enum expression</param>
		/// <returns>The enum value</returns>
		public TEnum Get<TEnum>(BgEnum<TEnum> expr) where TEnum : struct => ((BgEnumConstantExpr<TEnum>)expr).Value;

		/// <summary>
		/// Resolve a list of enums to a value
		/// </summary>
		/// <typeparam name="TEnum">The enum type</typeparam>
		/// <param name="expr">Enum expression</param>
		/// <returns>The enum value</returns>
		public List<TEnum> Get<TEnum>(BgList<BgEnum<TEnum>> expr) where TEnum : struct => [.. ((BgListConstantExpr<BgEnum<TEnum>>)expr).Value.Select(x => ((BgEnumConstantExpr<TEnum>)x).Value)];

		/// <summary>
		/// Resolve a list of strings
		/// </summary>
		/// <param name="expr">List expression</param>
		/// <returns></returns>
		public List<string> Get(BgList<BgString> expr) => (((BgListConstantExpr<BgString>)expr).Value).ConvertAll(x => ((BgStringConstantExpr)x).Value);

		/// <summary>
		/// Resolve a file set
		/// </summary>
		/// <param name="fileSet">The token expression</param>
		/// <returns>Set of files for the token</returns>
		public FileSet Get(BgFileSet fileSet)
		{
			FileSet result = FileSet.Empty;

			BgFileSetInputExpr input = (BgFileSetInputExpr)fileSet;
			foreach (BgNodeOutput output in input.Outputs)
			{
				FileSet set = _tagNameToFileSet[output.TagName];
				result = FileSet.Union(result, set);
			}

			return result;
		}

		/// <summary>
		/// Resolve a file set
		/// </summary>
		/// <param name="fileSets">The token expression</param>
		/// <returns>Set of files for the token</returns>
		public FileSet Get(BgList<BgFileSet> fileSets)
		{
			FileSet result = FileSet.Empty;
			foreach (BgFileSet fileSet in ((BgListConstantExpr<BgFileSet>)fileSets).Value)
			{
				result += Get(fileSet);
			}
			return result;
		}
	}
}
