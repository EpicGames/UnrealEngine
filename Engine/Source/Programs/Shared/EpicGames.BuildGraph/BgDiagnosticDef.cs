// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Diagnostic message from the graph script. These messages are parsed at startup, then culled along with the rest of the graph nodes before output. Doing so
	/// allows errors and warnings which are only output if a node is part of the graph being executed.
	/// </summary>
	public class BgDiagnosticDef
	{
		/// <summary>
		/// File containing the diagnostic
		/// </summary>
		public string File { get; }

		/// <summary>
		/// Line number containing the diagnostic
		/// </summary>
		public int Line { get; }

		/// <summary>
		/// The diagnostic event type
		/// </summary>
		public LogLevel Level { get; }

		/// <summary>
		/// The message to display
		/// </summary>
		public string Message { get; }

		/// <summary>
		/// Report the diagnostic during execution
		/// </summary>
		public bool ReportOnExecution { get; }

		/// <summary>
		/// An optional set of node names that when <see cref="ReportOnExecution"/> is true limits the reporting of the diagnostic
		/// to only this set of nodes
		/// </summary>
		public string[] ReportOnNodes { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgDiagnosticDef(string file, int line, LogLevel level, string message, bool reportOnExecution, string[] reportOnNodes)
		{
			File = file;
			Line = line;
			Level = level;
			Message = message;
			ReportOnExecution = reportOnExecution;
			ReportOnNodes = reportOnNodes;
		}
	}
}
