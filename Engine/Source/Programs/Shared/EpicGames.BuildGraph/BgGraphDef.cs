// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

[assembly: InternalsVisibleTo("EpicGames.BuildGraph.Tests")]

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Options for how the graph should be printed
	/// </summary>
	[Flags]
	public enum GraphPrintOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Includes a list of the graph options
		/// </summary>
		ShowCommandLineOptions = 0x1,

		/// <summary>
		/// Includes the list of dependencies for each node
		/// </summary>
		ShowDependencies = 0x2,

		/// <summary>
		/// Includes the list of notifiers for each node
		/// </summary>
		ShowNotifications = 0x4,
	}

	/// <summary>
	/// Definition of a graph.
	/// </summary>
	public class BgGraphDef
	{
		/// <summary>
		/// List of options, in the order they were specified
		/// </summary>
		public List<BgOptionDef> Options { get; } = [];

		/// <summary>
		/// List of agents containing nodes to execute
		/// </summary>
		public List<BgAgentDef> Agents { get; } = [];

		/// <summary>
		/// Mapping from name to agent
		/// </summary>
		public Dictionary<string, BgAgentDef> NameToAgent { get; set; } = new Dictionary<string, BgAgentDef>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Mapping of names to the corresponding node.
		/// </summary>
		public Dictionary<string, BgNodeDef> NameToNode { get; set; } = new Dictionary<string, BgNodeDef>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Mapping of names to the corresponding report.
		/// </summary>
		public Dictionary<string, BgReport> NameToReport { get; private set; } = new Dictionary<string, BgReport>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Mapping of names to their corresponding node output.
		/// </summary>
		public Dictionary<string, BgNodeOutput> TagNameToNodeOutput { get; private set; } = new Dictionary<string, BgNodeOutput>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Mapping of aggregate names to their respective nodes
		/// </summary>
		public Dictionary<string, BgAggregateDef> NameToAggregate { get; private set; } = new Dictionary<string, BgAggregateDef>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Mapping of artifact names to their definitions. Artifacts will be produced from matching tag names.
		/// </summary>
		public List<BgArtifactDef> Artifacts { get; } = [];

		/// <summary>
		/// List of badges that can be displayed for this build
		/// </summary>
		public List<BgBadgeDef> Badges { get; } = [];

		/// <summary>
		/// List of labels that can be displayed for this build
		/// </summary>
		public List<BgLabelDef> Labels { get; } = [];

		/// <summary>
		/// Diagnostics at graph scope
		/// </summary>
		public List<BgDiagnosticDef> Diagnostics { get; } = [];

		/// <summary>
		/// Default constructor
		/// </summary>
		public BgGraphDef()
		{
		}

		/// <summary>
		/// Checks whether a given name already exists
		/// </summary>
		/// <param name="name">The name to check.</param>
		/// <returns>True if the name exists, false otherwise.</returns>
		public bool ContainsName(string name)
		{
			return NameToNode.ContainsKey(name) || NameToReport.ContainsKey(name) || NameToAggregate.ContainsKey(name);
		}

		/// <summary>
		/// Gets diagnostics from all graph structures
		/// </summary>
		/// <returns>List of diagnostics</returns>
		public List<BgDiagnosticDef> GetAllDiagnostics()
		{
			List<BgDiagnosticDef> diagnostics = [.. Diagnostics];
			foreach (BgAgentDef agent in Agents)
			{
				diagnostics.AddRange(agent.Diagnostics);
				foreach (BgNodeDef node in agent.Nodes)
				{
					diagnostics.AddRange(node.Diagnostics);
				}
			}
			return diagnostics;
		}

		/// <summary>
		/// Tries to resolve the given name to one or more nodes. Checks for aggregates, and actual nodes.
		/// </summary>
		/// <param name="name">The name to search for</param>
		/// <param name="outNodes">If the name is a match, receives an array of nodes and their output names</param>
		/// <returns>True if the name was found, false otherwise.</returns>
		public bool TryResolveReference(string name, [NotNullWhen(true)] out BgNodeDef[]? outNodes)
		{
			// Check if it's a tag reference or node reference
			if (name.StartsWith('#'))
			{
				// Check if it's a regular node or output name
				BgNodeOutput? output;
				if (TagNameToNodeOutput.TryGetValue(name, out output))
				{
					outNodes = [output.ProducingNode];
					return true;
				}
			}
			else
			{
				// Check if it's a regular node or output name
				BgNodeDef? node;
				if (NameToNode.TryGetValue(name, out node))
				{
					outNodes = [node];
					return true;
				}

				// Check if it's an aggregate name
				BgAggregateDef? aggregate;
				if (NameToAggregate.TryGetValue(name, out aggregate))
				{
					outNodes = [.. aggregate.RequiredNodes];
					return true;
				}

				// Check if it's a group name
				BgAgentDef? agent;
				if (NameToAgent.TryGetValue(name, out agent))
				{
					outNodes = [.. agent.Nodes];
					return true;
				}
			}

			// Otherwise fail
			outNodes = null;
			return false;
		}

		/// <summary>
		/// Tries to resolve the given name to one or more node outputs. Checks for aggregates, and actual nodes.
		/// </summary>
		/// <param name="name">The name to search for</param>
		/// <param name="outOutputs">If the name is a match, receives an array of nodes and their output names</param>
		/// <returns>True if the name was found, false otherwise.</returns>
		public bool TryResolveInputReference(string name, [NotNullWhen(true)] out BgNodeOutput[]? outOutputs)
		{
			// Check if it's a tag reference or node reference
			if (name.StartsWith('#'))
			{
				// Check if it's a regular node or output name
				if (TagNameToNodeOutput.TryGetValue(name, out BgNodeOutput? output))
				{
					outOutputs = [output];
					return true;
				}
			}
			else
			{
				// Check if it's a regular node or output name
				if (NameToNode.TryGetValue(name, out BgNodeDef? node))
				{
					outOutputs = [.. node.Outputs.Union(node.Inputs).Union(node.OptionalInputs)];
					return true;
				}

				// Check if it's an aggregate name
				if (NameToAggregate.TryGetValue(name, out BgAggregateDef? aggregate))
				{
					outOutputs = [.. aggregate.RequiredNodes.SelectMany(x => x.Outputs.Union(x.Inputs).Union(x.OptionalInputs)).Distinct()];
					return true;
				}
			}

			// Otherwise fail
			outOutputs = null;
			return false;
		}

		static void AddDependencies(BgNodeDef node, HashSet<BgNodeDef> retainNodes)
		{
			if (retainNodes.Add(node))
			{
				foreach (BgNodeDef inputDependency in node.InputDependencies)
				{
					AddDependencies(inputDependency, retainNodes);
				}

				foreach (BgNodeDef optionalInputDependency in node.OptionalInputDependencies)
				{
					AddDependencies(optionalInputDependency, retainNodes);
				}
			}
		}

		/// <summary>
		/// Cull the graph to only include the given nodes and their dependencies
		/// </summary>
		/// <param name="targetNodes">A set of target nodes to build</param>
		public void Select(IEnumerable<BgNodeDef> targetNodes)
		{
			// Find this node and all its dependencies
			HashSet<BgNodeDef> retainNodes = [];
			foreach (BgNodeDef targetNode in targetNodes)
			{
				AddDependencies(targetNode, retainNodes);
			}

			// Remove all the nodes which are not marked to be kept
			foreach (BgAgentDef agent in Agents)
			{
				agent.Nodes = [.. agent.Nodes.Where(x => retainNodes.Contains(x))];
			}

			// Remove all the empty agents
			Agents.RemoveAll(x => x.Nodes.Count == 0);

			// Trim down the list of nodes for each report to the ones that are being built
			foreach (BgReport report in NameToReport.Values)
			{
				report.Nodes.RemoveWhere(x => !retainNodes.Contains(x));
			}

			// Remove all the empty reports
			NameToReport = NameToReport.Where(x => x.Value.Nodes.Count > 0).ToDictionary(pair => pair.Key, pair => pair.Value, StringComparer.InvariantCultureIgnoreCase);

			// Remove all the order dependencies which are no longer part of the graph. Since we don't need to build them, we don't need to wait for them
			foreach (BgNodeDef node in retainNodes)
			{
				node.OrderDependencies.RemoveAll(x => !retainNodes.Contains(x));
			}

			// Create a new list of aggregates for everything that's left
			Dictionary<string, BgAggregateDef> newNameToAggregate = new(NameToAggregate.Comparer);
			foreach (BgAggregateDef aggregate in NameToAggregate.Values)
			{
				if (aggregate.RequiredNodes.All(x => retainNodes.Contains(x)))
				{
					newNameToAggregate[aggregate.Name] = aggregate;
				}
			}
			NameToAggregate = newNameToAggregate;

			// Remove any labels that are no longer valid
			foreach (BgLabelDef label in Labels)
			{
				label.RequiredNodes.RemoveWhere(x => !retainNodes.Contains(x));
				label.IncludedNodes.RemoveWhere(x => !retainNodes.Contains(x));
			}
			Labels.RemoveAll(x => x.RequiredNodes.Count == 0);

			// Remove any badges which do not have all their dependencies
			Badges.RemoveAll(x => x.Nodes.Any(y => !retainNodes.Contains(y)));

			// Rebuild the tag name to output dictionary
			Dictionary<string, BgNodeOutput> newTagNameToNodeOutput = new(TagNameToNodeOutput.Comparer);
			foreach (BgNodeOutput output in Agents.SelectMany(x => x.Nodes).SelectMany(x => x.Outputs))
			{
				newTagNameToNodeOutput.Add(output.TagName, output);
			}
			TagNameToNodeOutput = newTagNameToNodeOutput;

			// Remove any artifacts whose outputs are not produced
			Artifacts.RemoveAll(x => x.TagName != null && !TagNameToNodeOutput.ContainsKey(x.TagName));

			// Remove any artifacts whose nodes are no longer run
			HashSet<string> newNodeNames = Agents.SelectMany(x => x.Nodes).Select(x => x.Name).ToHashSet(StringComparer.OrdinalIgnoreCase);
			Artifacts.RemoveAll(x => x.NodeName != null && !newNodeNames.Contains(x.NodeName));
		}

		/// <summary>
		/// Export the build graph to a Json file, for parallel execution by the build system
		/// </summary>
		/// <param name="file">Output file to write</param>
		/// <param name="completedNodes">Set of nodes which have been completed</param>
		public void Export(FileReference file, HashSet<BgNodeDef> completedNodes)
		{
			// Find all the nodes which we're actually going to execute. We'll use this to filter the graph.
			HashSet<BgNodeDef> nodesToExecute = [];
			foreach (BgNodeDef node in Agents.SelectMany(x => x.Nodes))
			{
				if (!completedNodes.Contains(node))
				{
					nodesToExecute.Add(node);
				}
			}

			// Open the output file
			using (JsonWriter jsonWriter = new(file.FullName))
			{
				jsonWriter.WriteObjectStart();

				// Write all the agents
				jsonWriter.WriteArrayStart("Groups");
				foreach (BgAgentDef agent in Agents)
				{
					BgNodeDef[] nodes = [.. agent.Nodes.Where(x => nodesToExecute.Contains(x))];
					if (nodes.Length > 0)
					{
						jsonWriter.WriteObjectStart();
						jsonWriter.WriteValue("Name", agent.Name);
						jsonWriter.WriteArrayStart("Agent Types");
						foreach (string agentType in agent.PossibleTypes)
						{
							jsonWriter.WriteValue(agentType);
						}
						jsonWriter.WriteArrayEnd();
						jsonWriter.WriteArrayStart("Nodes");
						foreach (BgNodeDef node in nodes)
						{
							jsonWriter.WriteObjectStart();
							jsonWriter.WriteValue("Name", node.Name);
							jsonWriter.WriteValue("DependsOn", String.Join(";", node.GetDirectOrderDependencies().Where(x => nodesToExecute.Contains(x))));
							jsonWriter.WriteValue("RunEarly", node.RunEarly);
							jsonWriter.WriteValue("AllowRetry", node.AllowRetry);
							jsonWriter.WriteObjectStart("Notify");
							jsonWriter.WriteValue("Default", String.Join(";", node.NotifyUsers));
							jsonWriter.WriteValue("Submitters", String.Join(";", node.NotifySubmitters));
							jsonWriter.WriteValue("Warnings", node.NotifyOnWarnings);
							jsonWriter.WriteObjectEnd();
							jsonWriter.WriteObjectEnd();
						}
						jsonWriter.WriteArrayEnd();
						jsonWriter.WriteObjectEnd();
					}
				}
				jsonWriter.WriteArrayEnd();

				// Write all the badges
				jsonWriter.WriteArrayStart("Badges");
				foreach (BgBadgeDef badge in Badges)
				{
					BgNodeDef[] dependencies = [.. badge.Nodes.Where(x => nodesToExecute.Contains(x))];
					if (dependencies.Length > 0)
					{
						// Reduce that list to the smallest subset of direct dependencies
						HashSet<BgNodeDef> directDependencies = [.. dependencies];
						foreach (BgNodeDef dependency in dependencies)
						{
							directDependencies.ExceptWith(dependency.OrderDependencies);
						}

						jsonWriter.WriteObjectStart();
						jsonWriter.WriteValue("Name", badge.Name);
						if (!String.IsNullOrEmpty(badge.Project))
						{
							jsonWriter.WriteValue("Project", badge.Project);
						}
						if (badge.Change != 0)
						{
							jsonWriter.WriteValue("Change", badge.Change);
						}
						jsonWriter.WriteValue("AllDependencies", String.Join(";", Agents.SelectMany(x => x.Nodes).Where(x => dependencies.Contains(x)).Select(x => x.Name)));
						jsonWriter.WriteValue("DirectDependencies", String.Join(";", directDependencies.Select(x => x.Name)));
						jsonWriter.WriteObjectEnd();
					}
				}
				jsonWriter.WriteArrayEnd();

				// Write all the triggers and reports. 
				jsonWriter.WriteArrayStart("Reports");
				foreach (BgReport report in NameToReport.Values)
				{
					BgNodeDef[] dependencies = [.. report.Nodes.Where(x => nodesToExecute.Contains(x))];
					if (dependencies.Length > 0)
					{
						// Reduce that list to the smallest subset of direct dependencies
						HashSet<BgNodeDef> directDependencies = [.. dependencies];
						foreach (BgNodeDef dependency in dependencies)
						{
							directDependencies.ExceptWith(dependency.OrderDependencies);
						}

						jsonWriter.WriteObjectStart();
						jsonWriter.WriteValue("Name", report.Name);
						jsonWriter.WriteValue("AllDependencies", String.Join(";", Agents.SelectMany(x => x.Nodes).Where(x => dependencies.Contains(x)).Select(x => x.Name)));
						jsonWriter.WriteValue("DirectDependencies", String.Join(";", directDependencies.Select(x => x.Name)));
						jsonWriter.WriteValue("Notify", String.Join(";", report.NotifyUsers));
						jsonWriter.WriteValue("IsTrigger", false);
						jsonWriter.WriteObjectEnd();
					}
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Export the build graph to a Json file for parsing by Horde
		/// </summary>
		/// <param name="file">Output file to write</param>
		public void ExportForHorde(FileReference file)
		{
			DirectoryReference.CreateDirectory(file.Directory);
			using (JsonWriter jsonWriter = new(file.FullName))
			{
				jsonWriter.WriteObjectStart();

				jsonWriter.WriteArrayStart("Artifacts");
				foreach (BgArtifactDef artifact in Artifacts)
				{
					jsonWriter.WriteObjectStart();
					jsonWriter.WriteValue("Name", artifact.Name);
					if (!String.IsNullOrEmpty(artifact.Type))
					{
						jsonWriter.WriteValue("Type", artifact.Type);
					}
					if (!String.IsNullOrEmpty(artifact.Description))
					{
						jsonWriter.WriteValue("Description", artifact.Description);
					}
					if (!String.IsNullOrEmpty(artifact.BasePath))
					{
						jsonWriter.WriteValue("BasePath", artifact.BasePath);
					}

					if (artifact.Keys.Count > 0)
					{
						jsonWriter.WriteArrayStart("Keys");
						foreach (string key in artifact.Keys)
						{
							jsonWriter.WriteValue(key);
						}
						jsonWriter.WriteArrayEnd();
					}

					if (artifact.Metadata.Count > 0)
					{
						jsonWriter.WriteArrayStart("Metadata");
						foreach (string metadata in artifact.Metadata)
						{
							jsonWriter.WriteValue(metadata);
						}
						jsonWriter.WriteArrayEnd();
					}

					if (!String.IsNullOrEmpty(artifact.NodeName))
					{
						jsonWriter.WriteValue("NodeName", artifact.NodeName);
					}
					if (!String.IsNullOrEmpty(artifact.TagName))
					{
						jsonWriter.WriteValue("OutputName", artifact.TagName);
					}

					jsonWriter.WriteObjectEnd();
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteArrayStart("Groups");
				foreach (BgAgentDef agent in Agents)
				{
					jsonWriter.WriteObjectStart();
					jsonWriter.WriteArrayStart("Types");
					foreach (string possibleType in agent.PossibleTypes)
					{
						jsonWriter.WriteValue(possibleType);
					}
					jsonWriter.WriteArrayEnd();
					jsonWriter.WriteArrayStart("Nodes");
					foreach (BgNodeDef node in agent.Nodes)
					{
						jsonWriter.WriteObjectStart();
						jsonWriter.WriteValue("Name", node.Name);
						jsonWriter.WriteValue("RunEarly", node.RunEarly);
						jsonWriter.WriteValue("AllowRetry", node.AllowRetry);
						jsonWriter.WriteValue("Warnings", node.NotifyOnWarnings);

						jsonWriter.WriteArrayStart("Inputs");
						foreach (BgNodeOutput input in node.Inputs)
						{
							jsonWriter.WriteValue(input.TagName);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteArrayStart("OptionalInputs");
						foreach (BgNodeOutput optionalInput in node.OptionalInputs)
						{
							jsonWriter.WriteValue(optionalInput.TagName);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteArrayStart("Outputs");
						foreach (BgNodeOutput output in node.Outputs)
						{
							jsonWriter.WriteValue(output.TagName);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteArrayStart("InputDependencies");
						foreach (string inputDependency in node.GetDirectInputDependencies().Select(x => x.Name))
						{
							jsonWriter.WriteValue(inputDependency);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteArrayStart("OptionalInputDependencies");
						foreach (string optionalInputDependency in node.GetDirectOptionalInputDependencies().Select(x => x.Name))
						{
							jsonWriter.WriteValue(optionalInputDependency);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteArrayStart("OrderDependencies");
						foreach (string orderDependency in node.GetDirectOrderDependencies().Select(x => x.Name))
						{
							jsonWriter.WriteValue(orderDependency);
						}
						jsonWriter.WriteArrayEnd();

						jsonWriter.WriteObjectStart("Annotations");
						foreach (BgAnnotationDef annotation in node.Annotations)
						{
							jsonWriter.WriteValue(annotation.Name, annotation.Value);
						}
						jsonWriter.WriteObjectEnd();

						jsonWriter.WriteObjectEnd();
					}
					jsonWriter.WriteArrayEnd();
					jsonWriter.WriteObjectEnd();
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteArrayStart("Aggregates");
				foreach (BgAggregateDef aggregate in NameToAggregate.Values)
				{
					jsonWriter.WriteObjectStart();
					jsonWriter.WriteValue("Name", aggregate.Name);
					jsonWriter.WriteArrayStart("Nodes");
					foreach (BgNodeDef requiredNode in aggregate.RequiredNodes.OrderBy(x => x.Name))
					{
						jsonWriter.WriteValue(requiredNode.Name);
					}
					jsonWriter.WriteArrayEnd();
					jsonWriter.WriteObjectEnd();
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteArrayStart("Labels");
				foreach (BgLabelDef label in Labels)
				{
					jsonWriter.WriteObjectStart();
					if (!String.IsNullOrEmpty(label.DashboardName))
					{
						jsonWriter.WriteValue("Name", label.DashboardName);
					}
					if (!String.IsNullOrEmpty(label.DashboardCategory))
					{
						jsonWriter.WriteValue("Category", label.DashboardCategory);
					}
					if (!String.IsNullOrEmpty(label.UgsBadge))
					{
						jsonWriter.WriteValue("UgsBadge", label.UgsBadge);
					}
					if (!String.IsNullOrEmpty(label.UgsProject))
					{
						jsonWriter.WriteValue("UgsProject", label.UgsProject);
					}
					if (label.Change != BgLabelChange.Current)
					{
						jsonWriter.WriteValue("Change", label.Change.ToString());
					}

					jsonWriter.WriteArrayStart("RequiredNodes");
					foreach (BgNodeDef requiredNode in label.RequiredNodes.OrderBy(x => x.Name))
					{
						jsonWriter.WriteValue(requiredNode.Name);
					}
					jsonWriter.WriteArrayEnd();
					jsonWriter.WriteArrayStart("IncludedNodes");
					foreach (BgNodeDef includedNode in label.IncludedNodes.OrderBy(x => x.Name))
					{
						jsonWriter.WriteValue(includedNode.Name);
					}
					jsonWriter.WriteArrayEnd();
					jsonWriter.WriteObjectEnd();
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteArrayStart("Badges");
				foreach (BgBadgeDef badge in Badges)
				{
					HashSet<BgNodeDef> dependencies = badge.Nodes;
					if (dependencies.Count > 0)
					{
						// Reduce that list to the smallest subset of direct dependencies
						HashSet<BgNodeDef> directDependencies = [.. dependencies];
						foreach (BgNodeDef dependency in dependencies)
						{
							directDependencies.ExceptWith(dependency.OrderDependencies);
						}

						jsonWriter.WriteObjectStart();
						jsonWriter.WriteValue("Name", badge.Name);
						if (!String.IsNullOrEmpty(badge.Project))
						{
							jsonWriter.WriteValue("Project", badge.Project);
						}
						if (badge.Change != 0)
						{
							jsonWriter.WriteValue("Change", badge.Change);
						}
						jsonWriter.WriteValue("Dependencies", String.Join(";", directDependencies.Select(x => x.Name)));
						jsonWriter.WriteObjectEnd();
					}
				}
				jsonWriter.WriteArrayEnd();

				jsonWriter.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Print the contents of the graph
		/// </summary>
		/// <param name="completedNodes">Set of nodes which are already complete</param>
		/// <param name="printOptions">Options for how to print the graph</param>
		/// <param name="logger"></param>
		public void Print(HashSet<BgNodeDef> completedNodes, GraphPrintOptions printOptions, ILogger logger)
		{
			// Print the options
			if ((printOptions & GraphPrintOptions.ShowCommandLineOptions) != 0)
			{
				// Get the list of messages
				List<KeyValuePair<string, string>> parameters = [];
				foreach (BgOptionDef option in Options)
				{
					string name = String.Format("-set:{0}=...", option.Name);

					StringBuilder description = new(option.Description);

					string? defaultValue = null;
					if (option is BgBoolOptionDef boolOption)
					{
						defaultValue = boolOption.DefaultValue.ToString();
					}
					else if (option is BgIntOptionDef intOption)
					{
						defaultValue = intOption.DefaultValue.ToString();
					}
					else if (option is BgStringOptionDef stringOption)
					{
						defaultValue = stringOption.DefaultValue;
					}

					if (!String.IsNullOrEmpty(defaultValue))
					{
						description.AppendFormat(" (Default: {0})", defaultValue);
					}

					parameters.Add(new KeyValuePair<string, string>(name, description.ToString()));
				}

				// Format them to the log
				if (parameters.Count > 0)
				{
					logger.LogInformation("");
					logger.LogInformation("Options:");
					logger.LogInformation("");

					List<string> lines = [];
					HelpUtils.FormatTable(parameters, 4, 24, HelpUtils.WindowWidth - 20, lines);

					foreach (string line in lines)
					{
						logger.Log(LogLevel.Information, "{Line}", line);
					}
				}
			}

			// Output all the triggers in order
			logger.LogInformation("");
			logger.LogInformation("Graph:");
			foreach (BgAgentDef agent in Agents)
			{
				logger.LogInformation("        Agent: {Name} ({Types})", agent.Name, String.Join(";", agent.PossibleTypes));
				foreach (BgNodeDef node in agent.Nodes)
				{
					logger.LogInformation("            Node: {Name}{Type}{Retry}", node.Name, completedNodes.Contains(node) ? " (completed)" : node.RunEarly ? " (early)" : "", !node.AllowRetry ? " (retry disabled)" : "");
					if (printOptions.HasFlag(GraphPrintOptions.ShowDependencies))
					{
						HashSet<BgNodeDef> inputDependencies = [.. node.GetDirectInputDependencies()];
						foreach (BgNodeDef inputDependency in inputDependencies)
						{
							logger.LogInformation("                input> {Name}", inputDependency.Name);
						}
						HashSet<BgNodeDef> optionalInputDependencies = [.. node.GetDirectOptionalInputDependencies()];
						foreach (BgNodeDef optionalInputDependency in optionalInputDependencies)
						{
							logger.LogInformation("                optional> {Name}", optionalInputDependency.Name);
						}
						HashSet<BgNodeDef> orderDependencies = [.. node.GetDirectOrderDependencies()];
						foreach (BgNodeDef orderDependency in orderDependencies.Except(inputDependencies).Except(optionalInputDependencies))
						{
							logger.LogInformation("                after> {Name}", orderDependency.Name);
						}
					}
					if (printOptions.HasFlag(GraphPrintOptions.ShowNotifications))
					{
						string label = node.NotifyOnWarnings ? "warnings" : "errors";
						foreach (string user in node.NotifyUsers)
						{
							logger.LogInformation("                {Name}> {User}", label, user);
						}
						foreach (string submitter in node.NotifySubmitters)
						{
							logger.LogInformation("                {Name}> submitters to {Submitter}", label, submitter);
						}
					}
				}
			}
			logger.LogInformation("");

			// Print out all the non-empty aggregates
			BgAggregateDef[] aggregates = [.. NameToAggregate.Values.OrderBy(x => x.Name)];
			if (aggregates.Length > 0)
			{
				logger.LogInformation("Aggregates:");
				foreach (string aggregateName in aggregates.Select(x => x.Name))
				{
					logger.LogInformation("    {Aggregate}", aggregateName);
				}
				logger.LogInformation("");
			}

			// Print all the produced artifacts
			BgArtifactDef[] artifacts = [.. Artifacts.OrderBy(x => x.Name)];
			if (artifacts.Length > 0)
			{
				logger.LogInformation("Artifacts:");
				foreach (BgArtifactDef artifact in artifacts.OrderBy(x => x.Description))
				{
					logger.LogInformation("    {Name}", artifact.Name);
				}
				logger.LogInformation("");
			}
		}
	}

	/// <summary>
	/// Definition of a graph from bytecode. Can be converted to regular graph definition.
	/// </summary>
	public class BgGraphExpressionDef
	{
		/// <summary>
		/// Nodes for the graph
		/// </summary>
		public List<BgNodeExpressionDef> Nodes { get; } = [];

		/// <summary>
		/// Aggregates for the graph
		/// </summary>
		public List<BgAggregateExpressionDef> Aggregates { get; } = [];

		/// <summary>
		/// Creates a graph definition from this template
		/// </summary>
		/// <returns></returns>
		public BgGraphDef ToGraphDef()
		{
			List<BgNodeExpressionDef> nodes = [.. Nodes];

			BgGraphDef graph = new();
			foreach (BgAggregateExpressionDef aggregate in Aggregates)
			{
				graph.NameToAggregate[aggregate.Name] = aggregate.ToAggregateDef();
				nodes.AddRange(aggregate.RequiredNodes.Select(x => (BgNodeExpressionDef)x));
			}

			HashSet<BgNodeDef> uniqueNodes = [];
			HashSet<BgAgentDef> uniqueAgents = [];
			foreach (BgNodeExpressionDef node in nodes)
			{
				RegisterNode(graph, node, uniqueNodes, uniqueAgents);
			}

			HashSet<BgLabelDef> labels = [];
			foreach (BgNodeExpressionDef node in nodes)
			{
				foreach (BgLabelDef label in node.Labels)
				{
					label.RequiredNodes.Add(node);
					label.IncludedNodes.Add(node);
					labels.Add(label);
				}
			}
			foreach (BgAggregateExpressionDef aggregate in Aggregates)
			{
				if (aggregate.Label != null)
				{
					aggregate.Label.RequiredNodes.UnionWith(aggregate.RequiredNodes);
					aggregate.Label.IncludedNodes.UnionWith(aggregate.RequiredNodes);
					labels.Add(aggregate.Label);
				}
			}
			graph.Labels.AddRange(labels);
			return graph;
		}

		static void RegisterNode(BgGraphDef graph, BgNodeExpressionDef node, HashSet<BgNodeDef> uniqueNodes, HashSet<BgAgentDef> uniqueAgents)
		{
			if (uniqueNodes.Add(node))
			{
				foreach (BgNodeExpressionDef inputNode in node.InputDependencies.OfType<BgNodeExpressionDef>())
				{
					RegisterNode(graph, inputNode, uniqueNodes, uniqueAgents);
				}
				foreach (BgNodeExpressionDef optionalInputNode in node.OptionalInputDependencies.OfType<BgNodeExpressionDef>())
				{
					RegisterNode(graph, optionalInputNode, uniqueNodes, uniqueAgents);
				}

				BgAgentDef agent = node.Agent;
				if (uniqueAgents.Add(agent))
				{
					graph.Agents.Add(agent);
					graph.NameToAgent.Add(agent.Name, agent);
				}

				agent.Nodes.Add(node);
				graph.NameToNode.Add(node.Name, node);
			}
		}
	}
}
