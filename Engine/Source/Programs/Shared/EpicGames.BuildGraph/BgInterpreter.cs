// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Exception thrown by the runtime
	/// </summary>
	public sealed class BgBytecodeException : Exception
	{
		/// <summary>
		/// Source file that the error was thrown from
		/// </summary>
		public string SourceFile { get; }

		/// <summary>
		/// Line number that threw the exception
		/// </summary>
		public int SourceLine { get; }

		/// <summary>
		/// Message to display
		/// </summary>
		public string Diagnostic { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="sourceFile"></param>
		/// <param name="sourceLine"></param>
		/// <param name="diagnostic"></param>
		public BgBytecodeException(string sourceFile, int sourceLine, string diagnostic)
		{
			SourceFile = sourceFile;
			SourceLine = sourceLine;
			Diagnostic = diagnostic;
		}

		/// <inheritdoc/>
		public override string ToString() => $"{SourceFile}({SourceLine}): {Diagnostic}";
	}

	/// <summary>
	/// Interprets compiled buildgraph bytecode
	/// </summary>
	public class BgInterpreter
	{
		enum BgArg
		{
			None,
			Value,
			ArgList,
			Fragment,
			Name,
			Thunk,
			StringLiteral,
			VarIntSigned,
			VarIntUnsigned,
		}

		readonly struct BgOpcodeInfo
		{
			readonly uint _data;

			public BgOpcodeInfo(BgArg arg1) : this(1, arg1, BgArg.None, BgArg.None) { }
			public BgOpcodeInfo(BgArg arg1, BgArg arg2) : this(2, arg1, arg2, BgArg.None) { }
			public BgOpcodeInfo(BgArg arg1, BgArg arg2, BgArg arg3) : this(3, arg1, arg2, arg3) { }

			private BgOpcodeInfo(int count, BgArg arg1, BgArg arg2, BgArg arg3)
			{
				_data = (uint)arg1 | ((uint)arg2 << 4) | ((uint)arg3 << 8) | ((uint)count << 12);
			}

			public int ArgCount => (int)((_data >> 12) & 0x7);

			public BgArg Arg0 => GetArg(0);
			public BgArg Arg1 => GetArg(1);
			public BgArg Arg2 => GetArg(2);

			public BgArg GetArg(int idx) => (BgArg)((_data >> (idx * 4)) & 0xf);
		}

		class Frame
		{
			public int Offset { get; set; }
			public IReadOnlyList<object> Arguments { get; }
			public Dictionary<int, object> Objects { get; set; }

			public Frame(Frame other)
			{
				Offset = other.Offset;
				Arguments = other.Arguments;
				Objects = other.Objects;
			}

			public Frame(int offset, IReadOnlyList<object> arguments)
			{
				Offset = offset;
				Arguments = arguments;
				Objects = [];
			}
		}

		readonly byte[] _data;
		readonly BgThunkDef[] _thunks;
		readonly string[] _names;
		readonly IReadOnlyDictionary<string, string> _options;
		readonly BgBytecodeVersion _version;
		readonly int[] _fragments;
		readonly List<BgOptionDef> _optionDefs = [];

		/// <summary>
		/// The option definitions that were parsed during execution
		/// </summary>
		public IReadOnlyList<BgOptionDef> OptionDefs => _optionDefs;

		static readonly BgOpcodeInfo[] s_opcodeTable = CreateOpcodeLookup();

		static BgOpcodeInfo[] CreateOpcodeLookup()
		{
			BgOpcodeInfo[] opcodes = new BgOpcodeInfo[256];

			opcodes[(int)BgOpcode.BoolFalse] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.BoolTrue] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.BoolNot] = new BgOpcodeInfo(BgArg.Value);
			opcodes[(int)BgOpcode.BoolAnd] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.BoolOr] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.BoolXor] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.BoolEq] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.BoolOption] = new BgOpcodeInfo(BgArg.Value);

			opcodes[(int)BgOpcode.IntLiteral] = new BgOpcodeInfo(BgArg.VarIntSigned);
			opcodes[(int)BgOpcode.IntEq] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntLt] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntGt] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntAdd] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntMultiply] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntDivide] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntModulo] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.IntNegate] = new BgOpcodeInfo(BgArg.Value);
			opcodes[(int)BgOpcode.IntOption] = new BgOpcodeInfo(BgArg.Value);

			opcodes[(int)BgOpcode.StrEmpty] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.StrLiteral] = new BgOpcodeInfo(BgArg.StringLiteral);
			opcodes[(int)BgOpcode.StrCompare] = new BgOpcodeInfo(BgArg.Value, BgArg.Value, BgArg.VarIntUnsigned);
			opcodes[(int)BgOpcode.StrConcat] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.StrFormat] = new BgOpcodeInfo(BgArg.Value, BgArg.ArgList);
			opcodes[(int)BgOpcode.StrSplit] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.StrJoin] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.StrMatch] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.StrReplace] = new BgOpcodeInfo(BgArg.Value, BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.StrOption] = new BgOpcodeInfo(BgArg.Value);

			opcodes[(int)BgOpcode.EnumConstant] = new BgOpcodeInfo(BgArg.VarIntSigned);
			opcodes[(int)BgOpcode.EnumParse] = new BgOpcodeInfo(BgArg.Value, BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.EnumToString] = new BgOpcodeInfo(BgArg.Value, BgArg.Value, BgArg.Value);

			opcodes[(int)BgOpcode.ListEmpty] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.ListPush] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListPushLazy] = new BgOpcodeInfo(BgArg.Value, BgArg.Fragment);
			opcodes[(int)BgOpcode.ListCount] = new BgOpcodeInfo(BgArg.Value);
			opcodes[(int)BgOpcode.ListElement] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListConcat] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListUnion] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListExcept] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListSelect] = new BgOpcodeInfo(BgArg.Value, BgArg.Fragment);
			opcodes[(int)BgOpcode.ListWhere] = new BgOpcodeInfo(BgArg.Value, BgArg.Fragment);
			opcodes[(int)BgOpcode.ListDistinct] = new BgOpcodeInfo(BgArg.Value);
			opcodes[(int)BgOpcode.ListContains] = new BgOpcodeInfo(BgArg.Value, BgArg.Value);
			opcodes[(int)BgOpcode.ListLazy] = new BgOpcodeInfo(BgArg.Fragment);
			opcodes[(int)BgOpcode.ListOption] = new BgOpcodeInfo(BgArg.Value);

			opcodes[(int)BgOpcode.ObjEmpty] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.ObjGet] = new BgOpcodeInfo(BgArg.Value, BgArg.Name, BgArg.Value);
			opcodes[(int)BgOpcode.ObjSet] = new BgOpcodeInfo(BgArg.Value, BgArg.Name, BgArg.Value);

			opcodes[(int)BgOpcode.Call] = new BgOpcodeInfo(BgArg.Fragment, BgArg.ArgList);
			opcodes[(int)BgOpcode.Argument] = new BgOpcodeInfo(BgArg.VarIntUnsigned);
			opcodes[(int)BgOpcode.Jump] = new BgOpcodeInfo(BgArg.Fragment);

			opcodes[(int)BgOpcode.Choose] = new BgOpcodeInfo(BgArg.Value, BgArg.Fragment, BgArg.Fragment);
			opcodes[(int)BgOpcode.Throw] = new BgOpcodeInfo(BgArg.StringLiteral, BgArg.VarIntUnsigned, BgArg.Value);
			opcodes[(int)BgOpcode.Null] = new BgOpcodeInfo();
			opcodes[(int)BgOpcode.Thunk] = new BgOpcodeInfo(BgArg.Thunk, BgArg.ArgList);

			return opcodes;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data"></param>
		/// <param name="thunks">Thunks to native code</param>
		/// <param name="options">Options for evaluating the graph</param>
		public BgInterpreter(byte[] data, BgThunkDef[] thunks, IReadOnlyDictionary<string, string> options)
		{
			_data = data;
			_thunks = thunks;
			_options = options;

			MemoryReader reader = new(data);

			_version = (BgBytecodeVersion)reader.ReadUnsignedVarInt();
			_names = reader.ReadVariableLengthArray(() => reader.ReadString());
			int[] fragmentLengths = reader.ReadVariableLengthArray(() => (int)reader.ReadUnsignedVarInt());

			int offset = data.Length - reader.RemainingMemory.Length;

			_fragments = new int[fragmentLengths.Length];
			for (int idx = 0; idx < _fragments.Length; idx++)
			{
				_fragments[idx] = offset;
				offset += fragmentLengths[idx];
			}
		}

		/// <summary>
		/// Evaluates the graph
		/// </summary>
		public object Evaluate()
		{
			// Take the given parameters, evaluate the graph that was produced (creating a map of all named entites)
			// Filter the map to the required target
			// Evaluate it fully into a BgGraph
			return Evaluate(new Frame(_fragments[0], []));
		}

		object Evaluate(Frame frame)
		{
			BgOpcode opcode = ReadOpcode(frame);
			BgOpcodeInfo opcodeInfo = s_opcodeTable[(int)opcode];

			object arg0 = ReadArgument(frame, opcodeInfo.Arg0);
			object arg1 = ReadArgument(frame, opcodeInfo.Arg1);
			object arg2 = ReadArgument(frame, opcodeInfo.Arg2);

			return Evaluate(frame, opcode, arg0, arg1, arg2);
		}

		object ReadArgument(Frame frame, BgArg type)
		{
			switch (type)
			{
				case BgArg.None:
					return null!;
				case BgArg.Value:
					return Evaluate(frame);
				case BgArg.Name:
					return ReadName(frame);
				case BgArg.Fragment:
					return ReadFragment(frame);
				case BgArg.VarIntSigned:
					return ReadSignedInteger(frame);
				case BgArg.VarIntUnsigned:
					return ReadUnsignedInteger(frame);
				case BgArg.StringLiteral:
					return ReadString(frame);
				case BgArg.ArgList:
					{
						int argCount = (int)ReadUnsignedInteger(frame);
						object[] argList = new object[argCount];

						for (int argIdx = 0; argIdx < argCount; argIdx++)
						{
							argList[argIdx] = Evaluate(frame);
						}

						return argList;
					}
				case BgArg.Thunk:
					return _thunks[(int)ReadUnsignedInteger(frame)];
				default:
					throw new InvalidDataException();
			}
		}

		object Evaluate(Frame frame, BgOpcode opcode, object arg0, object arg1, object arg2)
		{
			switch (opcode)
			{
				#region Bool opcodes

				case BgOpcode.BoolFalse:
					return false;
				case BgOpcode.BoolTrue:
					return true;
				case BgOpcode.BoolNot:
					return !(bool)arg0;
				case BgOpcode.BoolAnd:
					{
						bool lhs = (bool)arg0;
						bool rhs = (bool)arg1;
						return lhs & rhs;
					}
				case BgOpcode.BoolOr:
					{
						bool lhs = (bool)arg0;
						bool rhs = (bool)arg1;
						return lhs | rhs;
					}
				case BgOpcode.BoolXor:
					{
						bool lhs = (bool)arg0;
						bool rhs = (bool)arg1;
						return lhs ^ rhs;
					}
				case BgOpcode.BoolEq:
					{
						bool lhs = (bool)arg0;
						bool rhs = (bool)arg1;
						return lhs == rhs;
					}
				case BgOpcode.BoolOption:
					{
						BgObjectDef obj = (BgObjectDef)arg0;

						BgBoolOptionDef option = obj.Deserialize<BgBoolOptionDef>();
						_optionDefs.Add(option);

						bool value = option.DefaultValue;
						if (_options.TryGetValue(option.Name, out string? str))
						{
							value = Boolean.Parse(str);
						}

						return value;
					}

				#endregion
				#region Integer opcodes

				case BgOpcode.IntLiteral:
					return arg0;
				case BgOpcode.IntEq:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs == rhs;
					}
				case BgOpcode.IntLt:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs < rhs;
					}
				case BgOpcode.IntGt:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs > rhs;
					}
				case BgOpcode.IntAdd:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs + rhs;
					}
				case BgOpcode.IntMultiply:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs * rhs;
					}
				case BgOpcode.IntDivide:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs / rhs;
					}
				case BgOpcode.IntModulo:
					{
						int lhs = (int)arg0;
						int rhs = (int)arg1;
						return lhs % rhs;
					}
				case BgOpcode.IntNegate:
					return -(int)arg0;
				case BgOpcode.IntOption:
					{
						BgObjectDef obj = (BgObjectDef)arg0;

						BgIntOptionDef option = obj.Deserialize<BgIntOptionDef>();
						_optionDefs.Add(option);

						int value = option.DefaultValue;
						if (_options.TryGetValue(option.Name, out string? str))
						{
							value = Int32.Parse(str);
						}

						return value;
					}

				#endregion
				#region String opcodes

				case BgOpcode.StrEmpty:
					return String.Empty;
				case BgOpcode.StrLiteral:
					return arg0;
				case BgOpcode.StrCompare:
					{
						string lhs = (string)arg0;
						string rhs = (string)arg1;
						StringComparison comparison = (StringComparison)(ulong)arg2;
						return String.Compare(lhs, rhs, comparison);
					}
				case BgOpcode.StrConcat:
					{
						string lhs = (string)arg0;
						string rhs = (string)arg1;
						return lhs + rhs;
					}
				case BgOpcode.StrFormat:
					{
						string format = (string)arg0;
						object?[] arguments = (object?[])arg1;

						return String.Format(format, arguments);
					}
				case BgOpcode.StrSplit:
					{
						string source = (string)arg0;
						string separator = (string)arg1;
						return source.Split(separator, StringSplitOptions.RemoveEmptyEntries);
					}
				case BgOpcode.StrJoin:
					{
						string lhs = (string)arg0;
						IEnumerable<object> rhs = (IEnumerable<object>)arg1;
						return String.Join(lhs, rhs);
					}
				case BgOpcode.StrMatch:
					{
						string input = (string)arg0;
						string pattern = (string)arg1;
						return Regex.IsMatch(input, pattern);
					}
				case BgOpcode.StrReplace:
					{
						string input = (string)arg0;
						string pattern = (string)arg1;
						string replacement = (string)arg2;
						return Regex.Replace(input, pattern, replacement);
					}
				case BgOpcode.StrOption:
					{
						BgObjectDef obj = (BgObjectDef)arg0;

						BgStringOptionDef option = obj.Deserialize<BgStringOptionDef>();
						_optionDefs.Add(option);

						string value = option.DefaultValue;
						if (_options.TryGetValue(option.Name, out string? result))
						{
							value = result;
						}

						return value;
					}

				#endregion
				#region Enum opcodes

				case BgOpcode.EnumConstant:
					return (int)arg0;
				case BgOpcode.EnumParse:
					{
						string name = (string)arg0;
						List<string> names = GetListArg<string>(arg1);
						List<int> values = GetListArg<int>(arg2);

						for (int idx = 0; idx < names.Count; idx++)
						{
							if (String.Equals(names[idx], name, StringComparison.OrdinalIgnoreCase))
							{
								return values[idx];
							}
						}

						throw new InvalidDataException($"Unable to parse enum '{name}'");
					}
				case BgOpcode.EnumToString:
					{
						int value = (int)arg0;
						List<string> names = GetListArg<string>(arg1);
						List<int> values = GetListArg<int>(arg2);

						for (int idx = 0; idx < names.Count; idx++)
						{
							if (value == values[idx])
							{
								return names[idx];
							}
						}

						return $"{value}";
					}

				#endregion
				#region List opcodes

				case BgOpcode.ListEmpty:
					return Enumerable.Empty<object>();
				case BgOpcode.ListPush:
					{
						IEnumerable<object> list = (IEnumerable<object>)arg0;
						object item = arg1;

						return list.Concat([item]);
					}
				case BgOpcode.ListPushLazy:
					{
						IEnumerable<object> list = (IEnumerable<object>)arg0;
						int fragment = (int)arg1;

						IEnumerable<object> item = LazyEvaluateItem(new Frame(frame), fragment);
						return list.Concat(item);
					}
				case BgOpcode.ListCount:
					{
						IEnumerable<object> list = (IEnumerable<object>)arg0;
						return list.Count();
					}
				case BgOpcode.ListElement:
					{
						IEnumerable<object> list = (IEnumerable<object>)arg0;
						int index = (int)arg1;
						return list.ElementAt(index);
					}
				case BgOpcode.ListConcat:
					{
						IEnumerable<object> lhs = (IEnumerable<object>)arg0;
						IEnumerable<object> rhs = (IEnumerable<object>)arg1;
						return Enumerable.Concat(lhs, rhs);
					}
				case BgOpcode.ListUnion:
					{
						IEnumerable<object> lhs = (IEnumerable<object>)arg0;
						IEnumerable<object> rhs = (IEnumerable<object>)arg1;
						return lhs.Union(rhs);
					}
				case BgOpcode.ListExcept:
					{
						IEnumerable<object> lhs = (IEnumerable<object>)arg0;
						IEnumerable<object> rhs = (IEnumerable<object>)arg1;
						return lhs.Except(rhs);
					}
				case BgOpcode.ListSelect:
					{
						IEnumerable<object> source = (IEnumerable<object>)arg0;
						int fragment = (int)arg1;

						return source.Select(x => Call(fragment, [x]));
					}
				case BgOpcode.ListWhere:
					{
						IEnumerable<object> source = (IEnumerable<object>)arg0;
						int fragment = (int)arg1;

						return source.Where(x => (bool)Call(fragment, [x]));
					}
				case BgOpcode.ListDistinct:
					{
						IEnumerable<object> source = (IEnumerable<object>)arg0;
						return source.Distinct();
					}
				case BgOpcode.ListContains:
					{
						IEnumerable<object> source = (IEnumerable<object>)arg0;
						object item = arg1;
						return source.Contains(item);
					}
				case BgOpcode.ListLazy:
					{
						int fragment = (int)arg0;
						return LazyEvaluateList(new Frame(frame), fragment);
					}
				case BgOpcode.ListOption:
					{
						BgObjectDef obj = (BgObjectDef)arg0;

						BgListOptionDef option = obj.Deserialize<BgListOptionDef>();
						_optionDefs.Add(option);

						string? value = option.DefaultValue ?? String.Empty;
						if (_options.TryGetValue(option.Name, out string? result))
						{
							value = result;
						}

						return value!.Split(['+', ';'], StringSplitOptions.RemoveEmptyEntries).Select(x => (object)x).ToList();
					}

				#endregion
				#region Object opcodes

				case BgOpcode.ObjEmpty:
					return new BgObjectDef();
				case BgOpcode.ObjGet:
					{
						BgObjectDef obj = (BgObjectDef)arg0;
						string name = (string)arg1;
						object defaultValue = arg2;

						object? value;
						if (!obj.Properties.TryGetValue(name, out value))
						{
							value = defaultValue;
						}

						return value!;
					}
				case BgOpcode.ObjSet:
					{
						BgObjectDef obj = (BgObjectDef)arg0;
						string name = (string)arg1;
						object value = arg2;
						return obj.Set(name, value);
					}

				#endregion
				#region Function opcodes

				case BgOpcode.Call:
					{
						int function = (int)arg0;
						object[] arguments = (object[])arg1;
						return Evaluate(new Frame(_fragments[function], arguments));
					}
				case BgOpcode.Argument:
					{
						int index = (int)(ulong)arg0;
						return frame.Arguments[index];
					}
				case BgOpcode.Jump:
					{
						int fragment = (int)arg0;
						return Jump(frame, fragment);
					}

				#endregion
				#region Generic opcodes

				case BgOpcode.Choose:
					{
						bool condition = (bool)arg0;

						int fragmentIfTrue = (int)arg1;
						int fragmentIfFalse = (int)arg2;

						return Jump(frame, condition ? fragmentIfTrue : fragmentIfFalse);
					}
				case BgOpcode.Throw:
					{
						string sourceFile = (string)arg0;
						int sourceLine = (int)(ulong)arg1;
						string message = (string)arg2;
						throw new BgBytecodeException(sourceFile, sourceLine, message);
					}
				case BgOpcode.Null:
					return null!;
				case BgOpcode.Thunk:
					{
						BgThunkDef method = (BgThunkDef)arg0;
						object[] arguments = (object[])arg1;

						object[] thunkArgs = new object[arguments.Length];

						ParameterInfo[] parameters = method.Method.GetParameters();
						for (int idx = 0; idx < arguments.Length; idx++)
						{
							object value = arguments[idx];
							if (typeof(BgExpr).IsAssignableFrom(parameters[idx].ParameterType))
							{
								thunkArgs[idx] = BgType.Constant(parameters[idx].ParameterType, value);
							}
							else
							{
								thunkArgs[idx] = method.Arguments[idx]!;
							}
						}

						return new BgThunkDef(method.Method, thunkArgs);
					}

				#endregion

				default:
					throw new InvalidDataException($"Invalid opcode: {opcode}");
			}
		}

		static List<T> GetListArg<T>(object value)
		{
			return [.. ((IEnumerable<object>)value).Select(x => (T)x)];
		}

		IEnumerable<object> LazyEvaluateItem(Frame frame, int fragment)
		{
			yield return Jump(frame, fragment);
		}

		IEnumerable<object> LazyEvaluateList(Frame frame, int fragment)
		{
			IEnumerable<object> result = (IEnumerable<object>)Jump(frame, fragment);
			foreach (object item in result)
			{
				yield return item;
			}
		}

		object Jump(Frame frame, int fragment)
		{
			object? result;
			if (!frame.Objects.TryGetValue(fragment, out result))
			{
				int prevOffset = frame.Offset;

				frame.Offset = _fragments[fragment];

				result = Evaluate(frame);
				frame.Objects.Add(fragment, result);

				frame.Offset = prevOffset;
			}
			return result;
		}

		object Call(int fragment, object[] arguments)
		{
			int offset = _fragments[fragment];
			return Evaluate(new Frame(offset, arguments));
		}

		int ReadFragment(Frame frame)
		{
			return (int)ReadUnsignedInteger(frame);
		}

		/// <summary>
		/// Reads an opcode from the input stream
		/// </summary>
		/// <returns>The opcode that was read</returns>
		BgOpcode ReadOpcode(Frame frame)
		{
			BgOpcode opcode = (BgOpcode)_data[frame.Offset++];
			return opcode;
		}

		/// <summary>
		/// Reads a name from the input stream
		/// </summary>
		string ReadName(Frame frame)
		{
			int index = (int)ReadUnsignedInteger(frame);
			return _names[index];
		}

		/// <summary>
		/// Reads a string from the input stream
		/// </summary>
		string ReadString(Frame frame)
		{
			ReadOnlySpan<byte> buffer = _data.AsSpan(frame.Offset);

			int length = (int)VarInt.ReadUnsigned(buffer, out int bytesRead);
			frame.Offset += bytesRead;

			string text = Encoding.UTF8.GetString(buffer.Slice(bytesRead, length));
			frame.Offset += length;

			return text;
		}

		/// <summary>
		/// Writes a signed integer value to the output
		/// </summary>
		/// <returns>the value that was read</returns>
		int ReadSignedInteger(Frame frame)
		{
			ulong encoded = ReadUnsignedInteger(frame);
			return DecodeSignedInteger(encoded);
		}

		/// <summary>
		/// Read an unsigned integer value from the input
		/// </summary>
		/// <returns>The value that was read</returns>
		ulong ReadUnsignedInteger(Frame frame)
		{
			ulong value = VarInt.ReadUnsigned(_data.AsSpan(frame.Offset), out int bytesRead);
			frame.Offset += bytesRead;
			return value;
		}

		/// <summary>
		/// Decode a signed integer using the lower bit for the sign flag, allowing us to encode it more efficiently as a <see cref="VarInt"/>
		/// </summary>
		/// <param name="value">Value to be decoded</param>
		/// <returns>The decoded value</returns>
		static int DecodeSignedInteger(ulong value)
		{
			if ((value & 1) != 0)
			{
				return -(int)(value >> 1);
			}
			else
			{
				return (int)(value >> 1);
			}
		}

		/// <summary>
		/// Disassemble the current script to a logger
		/// </summary>
		/// <param name="logger"></param>
		public void Disassemble(ILogger logger)
		{
			logger.LogInformation("Version: {Version}", _version);
			for (int idx = 0; idx < _fragments.Length; idx++)
			{
				logger.LogInformation("");
				logger.LogInformation("Fragment {Idx}:", idx);
				Disassemble(new Frame(_fragments[idx], []), logger);
			}
		}

		void Disassemble(Frame frame, ILogger logger)
		{
			BgOpcode opcode = Trace(frame, null, ReadOpcode, logger);
			BgOpcodeInfo opcodeInfo = s_opcodeTable[(int)opcode];

			for (int idx = 0; idx < opcodeInfo.ArgCount; idx++)
			{
				BgArg arg = opcodeInfo.GetArg(idx);
				switch (arg)
				{
					case BgArg.Value:
						Disassemble(frame, logger);
						break;
					case BgArg.Name:
						Trace(frame, ReadName, x => $"name: {x}", logger);
						break;
					case BgArg.Fragment:
						TraceFragment(frame, logger);
						break;
					case BgArg.VarIntSigned:
						Trace(frame, "int:", ReadSignedInteger, logger);
						break;
					case BgArg.VarIntUnsigned:
						Trace(frame, "uint:", ReadUnsignedInteger, logger);
						break;
					case BgArg.StringLiteral:
						Trace(frame, ReadString, x => $"\"{x}\"", logger);
						break;
					case BgArg.ArgList:
						{
							int argCount = (int)Trace(frame, "count", ReadUnsignedInteger, logger);
							for (int argIdx = 0; argIdx < argCount; argIdx++)
							{
								Disassemble(frame, logger);
							}
							break;
						}
					case BgArg.Thunk:
						Trace(frame, "thunk", ReadUnsignedInteger, logger);
						break;
					default:
						throw new InvalidDataException();
				}
			}
		}

		int TraceFragment(Frame frame, ILogger logger)
		{
			return Trace(frame, "-> Fragment", ReadFragment, logger);
		}

		T Trace<T>(Frame frame, string? type, Func<Frame, T> readValue, ILogger logger)
		{
			int offset = frame.Offset;
			T value = readValue(frame);
			int length = frame.Offset - offset;

			string valueAndType = (type == null) ? $"{value}" : $"{type} {value}";
			Trace<T>(offset, length, valueAndType, logger);
			return value;
		}

		T Trace<T>(Frame frame, Func<Frame, T> readValue, Func<T, object> formatValue, ILogger logger)
		{
			int offset = frame.Offset;
			T value = readValue(frame);
			int length = frame.Offset - offset;

			Trace<T>(offset, length, formatValue(value), logger);
			return value;
		}

		void Trace<T>(int offset, int length, object value, ILogger logger)
		{
			string bytes = String.Join(" ", _data.AsSpan(offset, length).ToArray().Select(x => $"{x:x2}"));
			logger.LogInformation("{Offset,6}: {Value,-20} {Bytes}", offset, value, bytes);
		}
	}
}
