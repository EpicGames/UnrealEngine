// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Which changelist to show a UGS badge for
	/// </summary>
	public enum BgLabelChange
	{
		/// <summary>
		/// The current changelist being built
		/// </summary>
		Current,

		/// <summary>
		/// The last code changelist
		/// </summary>
		Code,
	}

	/// <summary>
	/// Defines a label within a graph. Labels are similar to badges, and give the combined status of one or more job steps. Unlike badges, they
	/// separate the requirements for its status and optional nodes to be included in its status, allowing this to be handled externally.
	/// </summary>
	public class BgLabelDef
	{
		/// <summary>
		/// Name of this badge
		/// </summary>
		public string? DashboardName { get; set; }

		/// <summary>
		/// Category for this label
		/// </summary>
		public string? DashboardCategory { get; set; }

		/// <summary>
		/// Name of the badge in UGS
		/// </summary>
		public string? UgsBadge { get; set; }

		/// <summary>
		/// Path to the project folder in UGS
		/// </summary>
		public string? UgsProject { get; set; }

		/// <summary>
		/// Which change to show the badge for
		/// </summary>
		public BgLabelChange Change { get; set; }

		/// <summary>
		/// Set of nodes that must be run for this label to be shown.
		/// </summary>
		public HashSet<BgNodeDef> RequiredNodes { get; } = [];

		/// <summary>
		/// Set of nodes that will be included in this label if present.
		/// </summary>
		public HashSet<BgNodeDef> IncludedNodes { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public BgLabelDef()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inDashboardName">Name of this label</param>
		/// <param name="inDashboardCategory">Type of this label</param>
		/// <param name="inUgsBadge">The UGS badge name</param>
		/// <param name="inUgsProject">Project to display this badge for</param>
		/// <param name="inChange">The change to show this badge on in UGS</param>
		public BgLabelDef(string? inDashboardName = null, string? inDashboardCategory = null, string? inUgsBadge = null, string? inUgsProject = null, BgLabelChange inChange = BgLabelChange.Current)
		{
			DashboardName = inDashboardName;
			DashboardCategory = inDashboardCategory;
			UgsBadge = inUgsBadge;
			UgsProject = inUgsProject;
			Change = inChange;
		}

		/// <summary>
		/// Get the name of this label
		/// </summary>
		/// <returns>The name of this label</returns>
		public override string ToString()
		{
			if (!String.IsNullOrEmpty(DashboardName))
			{
				return String.Format("{0}/{1}", DashboardCategory, DashboardName);
			}
			else
			{
				return UgsBadge ?? "Unknown";
			}
		}
	}
}
