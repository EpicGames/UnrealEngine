// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Reference to an output tag from a particular node
	/// </summary>
	public class BgNodeOutput
	{
		/// <summary>
		/// The node which produces the given output
		/// </summary>
		public BgNodeDef ProducingNode { get; }

		/// <summary>
		/// Name of the tag
		/// </summary>
		public string TagName { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="producingNode">Node which produces the given output</param>
		/// <param name="tagName">Name of the tag</param>
		public BgNodeOutput(BgNodeDef producingNode, string tagName)
		{
			ProducingNode = producingNode;
			TagName = tagName;
		}

		/// <summary>
		/// Returns a string representation of this output for debugging purposes
		/// </summary>
		/// <returns>The name of this output</returns>
		public override string ToString()
		{
			return String.Format("{0} [{1}]", TagName, ProducingNode.Name);
		}
	}

	/// <summary>
	/// Describes a dependency on a node output
	/// </summary>
	[BgObject(typeof(BgNodeOutputExprDefSerializer))]
	public class BgNodeOutputExprDef
	{
		/// <summary>
		/// The producing node
		/// </summary>
		public BgNodeExpressionDef ProducingNode { get; }

		/// <summary>
		/// The output index. -1 means all inputs and outputs for the node.
		/// </summary>
		public int OutputIndex { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="producingNode"></param>
		/// <param name="outputIndex"></param>
		public BgNodeOutputExprDef(BgNodeExpressionDef producingNode, int outputIndex)
		{
			ProducingNode = producingNode;
			OutputIndex = outputIndex;
		}

		/// <summary>
		/// Flattens this expression to a list of outputs
		/// </summary>
		/// <returns></returns>
		public IEnumerable<BgNodeOutput> Flatten()
		{
			if (OutputIndex == -1)
			{
				return [.. ProducingNode.InputDependencies.SelectMany(x => x.Outputs), .. ProducingNode.OptionalInputDependencies.SelectMany(x => x.Outputs), .. ProducingNode.Outputs];
			}
			else
			{
				return [ProducingNode.Outputs[OutputIndex]];
			}
		}
	}

	class BgNodeOutputExprDefSerializer : BgObjectSerializer<BgNodeOutputExprDef>
	{
		/// <inheritdoc/>
		public override BgNodeOutputExprDef Deserialize(BgObjectDef<BgNodeOutputExprDef> obj)
		{
			return new BgNodeOutputExprDef(obj.Get(x => x.ProducingNode, null!), obj.Get(x => x.OutputIndex, -1));
		}
	}

	/// <summary>
	/// Defines a node, a container for tasks and the smallest unit of execution that can be run as part of a build graph.
	/// </summary>
	public class BgNodeDef
	{
		/// <summary>
		/// The node's name
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Thunk to execute this node.
		/// </summary>
		public BgThunkDef? Thunk { get; }

		/// <summary>
		/// Array of inputs which this node requires to run
		/// </summary>
		public List<BgNodeOutput> Inputs { get; } = [];

		/// <summary>
		/// Array of inputs which this node is not dependent on to run
		/// </summary>
		public List<BgNodeOutput> OptionalInputs { get; } = [];

		/// <summary>
		/// Array of outputs produced by this node
		/// </summary>
		public IReadOnlyList<BgNodeOutput> Outputs { get; }

		/// <summary>
		/// Nodes which this node has input dependencies on
		/// </summary>
		public List<BgNodeDef> InputDependencies { get; } = [];

		/// <summary>
		/// Nodes which this has optional input dependencies on
		/// </summary>
		public List<BgNodeDef> OptionalInputDependencies { get; } = [];

		/// <summary>
		/// Nodes which this node needs to run after
		/// </summary>
		public List<BgNodeDef> OrderDependencies { get; } = [];

		/// <summary>
		/// Tokens which must be acquired for this node to run
		/// </summary>
		public List<FileReference> RequiredTokens { get; } = [];

		/// <summary>
		/// List of email addresses to notify if this node fails.
		/// </summary>
		public HashSet<string> NotifyUsers { get; set; } = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// If set, anyone that has submitted to one of the given paths will be notified on failure of this node
		/// </summary>
		public HashSet<string> NotifySubmitters { get; set; } = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// Whether to start this node as soon as its dependencies are satisfied, rather than waiting for all of its agent's dependencies to be met.
		/// </summary>
		public bool RunEarly { get; set; } = false;

		/// <summary>
		/// Whether this node can be retried.
		/// </summary>
		public bool AllowRetry { get; set; } = true;

		/// <summary>
		/// Whether to ignore warnings produced by this node
		/// </summary>
		public bool NotifyOnWarnings { get; set; } = true;

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public List<BgAnnotationDef> Annotations { get; } = [];

		/// <summary>
		/// Ignore modified files matching the patterns provided
		/// </summary>
		public List<string> IgnoreModified { get; set; } = [];

		/// <summary>
		/// Diagnostics to output if executing this node
		/// </summary>
		public List<BgDiagnosticDef> Diagnostics { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public BgNodeDef(string name, BgThunkDef? thunk, IReadOnlyList<string> outputNames)
		{
			Name = name;
			Thunk = thunk;

			List<BgNodeOutput> allOutputs =
			[
				new BgNodeOutput(this, "#" + Name),
				.. outputNames.Where(x => !String.Equals(x, Name, StringComparison.OrdinalIgnoreCase)).Select(x => new BgNodeOutput(this, x)),
			];
			Outputs = [.. allOutputs];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">The name of this node</param>
		/// <param name="inputs">Inputs that this node depends on</param>
		/// <param name="optionalInputs">Inputs that this node can optionally depend on</param>
		/// <param name="outputNames">Names of the outputs that this node produces</param>
		/// <param name="inputDependencies">Nodes which this node is dependent on for its inputs</param>
		/// <param name="optionalInputDependencies">Nodes which this node is optionally dependent on for its inputs</param>
		/// <param name="orderDependencies">Nodes which this node needs to run after. Should include all input dependencies.</param>
		/// <param name="requiredTokens">Optional tokens which must be required for this node to run</param>
		/// <param name="ignoreModified">File patterns to ignore when checking for modified timestamps</param>
		public BgNodeDef(string name, IReadOnlyList<BgNodeOutput> inputs, IReadOnlyList<BgNodeOutput> optionalInputs, IReadOnlyList<string> outputNames, IReadOnlyList<BgNodeDef> inputDependencies, IReadOnlyList<BgNodeDef> optionalInputDependencies, IReadOnlyList<BgNodeDef> orderDependencies, IReadOnlyList<FileReference> requiredTokens, IReadOnlyList<string> ignoreModified)
			: this(name, null, outputNames)
		{
			Name = name;
			Inputs.AddRange(inputs);
			OptionalInputs.AddRange(optionalInputs);
			InputDependencies.AddRange(inputDependencies);
			OptionalInputDependencies.AddRange(optionalInputDependencies);
			OrderDependencies.AddRange(orderDependencies);
			RequiredTokens.AddRange(requiredTokens);
			IgnoreModified.AddRange(ignoreModified);
		}

		/// <summary>
		/// Returns the default output for this node, which includes all build products
		/// </summary>
		public BgNodeOutput DefaultOutput => Outputs[0];

		/// <summary>
		/// Determines the minimal set of direct input dependencies for this node to run
		/// </summary>
		/// <returns>Sequence of nodes that are direct inputs to this node</returns>
		public IEnumerable<BgNodeDef> GetDirectInputDependencies()
		{
			HashSet<BgNodeDef> directDependencies = [.. InputDependencies];
			foreach (BgNodeDef inputDependency in InputDependencies)
			{
				directDependencies.ExceptWith(inputDependency.InputDependencies);
				directDependencies.ExceptWith(inputDependency.OptionalInputDependencies);
			}
			return directDependencies;
		}

		/// <summary>
		/// Determines the minimal set of direct optional input dependencies for this node to run
		/// </summary>
		/// <returns>Sequence of nodes that are direct optional inputs to this node</returns>
		public IEnumerable<BgNodeDef> GetDirectOptionalInputDependencies()
		{
			HashSet<BgNodeDef> directDependencies = [.. OptionalInputDependencies];
			foreach (BgNodeDef optionalInputDependency in OptionalInputDependencies)
			{
				directDependencies.ExceptWith(optionalInputDependency.OptionalInputDependencies);
				directDependencies.ExceptWith(optionalInputDependency.InputDependencies);
			}
			return directDependencies;
		}

		/// <summary>
		/// Determines the minimal set of direct order dependencies for this node to run
		/// </summary>
		/// <returns>Sequence of nodes that are direct order dependencies of this node</returns>
		public IEnumerable<BgNodeDef> GetDirectOrderDependencies()
		{
			HashSet<BgNodeDef> directDependencies = [.. OrderDependencies];
			foreach (BgNodeDef orderDependency in OrderDependencies)
			{
				directDependencies.ExceptWith(orderDependency.OrderDependencies);
			}
			return directDependencies;
		}

		/// <summary>
		/// Returns the name of this node
		/// </summary>
		/// <returns>The name of this node</returns>
		public override string ToString()
		{
			return Name;
		}
	}

	/// <summary>
	/// Node constructed from a bytecode expression
	/// </summary>
	[BgObject(typeof(BgNodeExpressionDefSerializer))]
	public class BgNodeExpressionDef : BgNodeDef
	{
		/// <summary>
		/// Agent declaring this node
		/// </summary>
		public BgAgentDef Agent { get; }

		/// <summary>
		/// Labels to add this node to
		/// </summary>
		public List<BgLabelDef> Labels { get; } = [];

		/// <summary>
		/// Input expressions
		/// </summary>
		public List<BgNodeOutputExprDef> InputExprs { get; } = [];
		
		/// <summary>
		/// Optional input expressions
		/// </summary>
		public List<BgNodeOutputExprDef> OptionalInputExprs { get; } = [];

		/// <summary>
		/// Order expressions
		/// </summary>
		public List<BgNodeExpressionDef> OrderExprs { get; } = [];
		
		/// <summary>
		/// Number of outputs from this node
		/// </summary>
		public int OutputCount { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgNodeExpressionDef(BgAgentDef agent, string name, BgThunkDef thunk, int outputCount)
			: base(name, thunk, GetOutputNames(name, outputCount))
		{
			Agent = agent;
			OutputCount = outputCount;
		}

		static string[] GetOutputNames(string name, int numOutputs)
		{
			return [.. Enumerable.Range(0, numOutputs).Select(x => BgNode.GetDefaultTagName(name, x))];
		}
	}

	class BgNodeExpressionDefSerializer : BgObjectSerializer<BgNodeExpressionDef>
	{
		/// <inheritdoc/>
		public override BgNodeExpressionDef Deserialize(BgObjectDef<BgNodeExpressionDef> obj)
		{
			BgNodeExpressionDef node = new(obj.Get(x => x.Agent, null!), obj.Get(x => x.Name, ""), obj.Get(x => x.Thunk!, null!), obj.Get(x => x.OutputCount, 0));
			obj.CopyTo(node);

			node.Inputs.AddRange(node.InputExprs.SelectMany(x => x.Flatten()));
			node.OptionalInputs.AddRange(node.OptionalInputExprs.SelectMany(x => x.Flatten()));

			HashSet<BgNodeDef> inputDependencies = [];
			foreach (BgNodeOutput input in node.Inputs)
			{
				inputDependencies.Add(input.ProducingNode);
				inputDependencies.UnionWith(input.ProducingNode.InputDependencies);
			}

			inputDependencies.ExceptWith(node.InputDependencies);
			node.InputDependencies.AddRange(inputDependencies);
			
			node.OrderDependencies.AddRange(node.OrderExprs);

			HashSet<BgNodeDef> optionalInputDependencies = [];
			foreach (BgNodeOutput optionalInput in node.OptionalInputs)
			{
				optionalInputDependencies.Add(optionalInput.ProducingNode);
				optionalInputDependencies.UnionWith(optionalInput.ProducingNode.InputDependencies);
				optionalInputDependencies.UnionWith(optionalInput.ProducingNode.OptionalInputDependencies);
			}

			optionalInputDependencies.ExceptWith(node.OptionalInputDependencies);
			node.OptionalInputDependencies.AddRange(optionalInputDependencies);

			return node;
		}
	}
}
