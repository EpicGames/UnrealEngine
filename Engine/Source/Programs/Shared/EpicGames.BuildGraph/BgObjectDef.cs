// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Linq;
using System.Linq.Expressions;
using System.Reflection;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Value of an object
	/// </summary>
	public class BgObjectDef
	{
		/// <summary>
		/// Deserialized object value
		/// </summary>
		private object? _value;

		/// <summary>
		/// Properties for the object
		/// </summary>
		public ImmutableDictionary<string, object?> Properties { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgObjectDef()
		{
			Properties = ImmutableDictionary<string, object?>.Empty.WithComparers(StringComparer.Ordinal);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BgObjectDef(ImmutableDictionary<string, object?> properties)
		{
			Properties = properties;
		}

		/// <summary>
		/// Sets a property value
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <param name="value">New value for the property</param>
		/// <returns>Object definition with the new properties</returns>
		public BgObjectDef Set(string name, object? value)
		{
			return new BgObjectDef(Properties.SetItem(name, value));
		}

		/// <summary>
		/// Gets a property value
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <param name="defaultValue">Default value to return if the property is not set</param>
		/// <returns></returns>
		public object? Get(string name, object? defaultValue)
		{
			object? value;
			if (!Properties.TryGetValue(name, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Creates a strongly typed object instance
		/// </summary>
		/// <typeparam name="T">Type of the object</typeparam>
		/// <returns></returns>
		public BgObjectDef<T> WithType<T>() => new(Properties);

		/// <summary>
		/// Cache of serializer instances
		/// </summary>
		static readonly ConcurrentDictionary<Type, BgObjectSerializer> s_typeToSerializer = new();

		/// <summary>
		/// Deserialize an object of the given type, using the default serializer
		/// </summary>
		/// <param name="type">Type of the object to create</param>
		/// <returns>Deserialized instance of the type</returns>
		public object Deserialize(Type type)
		{
			if (_value == null)
			{
				BgObjectSerializer? serializer;
				if (!s_typeToSerializer.TryGetValue(type, out serializer))
				{
					Type serializerType = type.GetCustomAttribute<BgObjectAttribute>()?.SerializerType ?? typeof(BgDefaultObjectSerializer<>).MakeGenericType(type);
					serializer = (BgObjectSerializer)Activator.CreateInstance(serializerType)!;
					serializer = s_typeToSerializer.GetOrAdd(type, serializer);
				}
				_value = serializer.Deserialize(this);
			}
			return _value;
		}

		/// <summary>
		/// Deserialize an object of the given type
		/// </summary>
		/// <typeparam name="T">Type to deserialize</typeparam>
		/// <returns>Deserialized instance of the type</returns>
		public T Deserialize<T>() => (T)Deserialize(typeof(T));
	}

	/// <summary>
	/// Strongly typed instance of <see cref="BgObjectDef"/>
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class BgObjectDef<T> : BgObjectDef
	{
		class PropertySetter
		{
			protected PropertyInfo Property { get; }

			public PropertySetter(PropertyInfo property) => Property = property;

			public virtual void SetValue(object instance, object? value) => Property.SetValue(instance, ConvertValue(value, Property.PropertyType));
		}

		class CollectionPropertySetter<TElement> : PropertySetter
		{
			public CollectionPropertySetter(PropertyInfo property)
				: base(property)
			{
			}

			public override void SetValue(object instance, object? value)
			{
				ICollection<TElement>? list = (ICollection<TElement>?)Property.GetValue(instance)!;
				if (list == null)
				{
					list = (ICollection<TElement>)Activator.CreateInstance(Property.PropertyType)!;
					Property.SetValue(instance, list);
				}

				IEnumerable<object> elements = (IEnumerable<object>)value!;
				foreach (object element in elements)
				{
					if (element is IEnumerable<TElement> range)
					{
						foreach (object? rangeElement in range)
						{
							list.Add((TElement)ConvertValue(rangeElement, typeof(TElement))!);
						}
					}
					else
					{
						list.Add((TElement)ConvertValue(element, typeof(TElement))!);
					}
				}
			}
		}

		static readonly IReadOnlyDictionary<string, PropertySetter> s_nameToSetter = GetPropertyMap();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgObjectDef()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BgObjectDef(ImmutableDictionary<string, object?> properties)
			: base(properties)
		{
		}

		/// <summary>
		/// Gets a property value
		/// </summary>
		/// <typeparam name="TValue"></typeparam>
		/// <param name="property"></param>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public TValue Get<TValue>(Expression<Func<T, TValue>> property, TValue defaultValue)
		{
			MemberExpression member = ((MemberExpression)property.Body);
			PropertyInfo propertyInfo = (PropertyInfo)member.Member;
			string name = propertyInfo.GetCustomAttribute<BgPropertyAttribute>()?.Name ?? propertyInfo.Name;
			return (TValue)ConvertValue(Get(name, defaultValue), typeof(TValue))!;
		}

		/// <summary>
		/// Copies properties from the object value to an instance
		/// </summary>
		/// <param name="instance"></param>
		public void CopyTo(object instance)
		{
			foreach ((string name, object? value) in Properties)
			{
				if (s_nameToSetter.TryGetValue(name, out PropertySetter? propertySetter))
				{
					propertySetter.SetValue(instance, value);
				}
			}
		}

		/// <summary>
		/// Convert a value to a particular type
		/// </summary>
		/// <param name="value">Value to convert</param>
		/// <param name="type">Target type</param>
		/// <returns>The converted value</returns>
		static object? ConvertValue(object? value, Type type)
		{
			if (value == null)
			{
				return null;
			}
			else if (value.GetType() == type)
			{
				return value;
			}
			else if (type.IsClass && type != typeof(string) && type != typeof(BgNodeOutput))
			{
				return ((BgObjectDef)value).Deserialize(type);
			}
			else
			{
				return value;
			}
		}

		/// <summary>
		/// Creates a map from name to property info
		/// </summary>
		/// <returns></returns>
		static IReadOnlyDictionary<string, PropertySetter> GetPropertyMap()
		{
			Dictionary<string, PropertySetter> nameToProperty = new(StringComparer.Ordinal);

			PropertyInfo[] properties = typeof(T).GetProperties(BindingFlags.Public | BindingFlags.Instance);
			foreach (PropertyInfo property in properties)
			{
				PropertySetter? setter = CreateSetter(property);
				if (setter != null)
				{
					BgPropertyAttribute? attribute = property.GetCustomAttribute<BgPropertyAttribute>();
					string name = attribute?.Name ?? property.Name;
					nameToProperty[name] = setter;
				}
			}

			return nameToProperty;
		}

		static PropertySetter? CreateSetter(PropertyInfo property)
		{
			Type? collectionInterface = property.PropertyType.GetInterfaces().FirstOrDefault(x => x.IsGenericType && x.GetGenericTypeDefinition() == typeof(ICollection<>));
			if (collectionInterface != null)
			{
				Type setterType = typeof(CollectionPropertySetter<>).MakeGenericType(typeof(T), collectionInterface.GetGenericArguments()[0]);
				return (PropertySetter)Activator.CreateInstance(setterType, property)!;
			}

			if (property.SetMethod != null)
			{
				return new PropertySetter(property);
			}

			return null;
		}
	}

	/// <summary>
	/// Attribute marking that a property should be serialized to BuildGraph
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class BgPropertyAttribute : Attribute
	{
		/// <summary>
		/// Name of the property. If unspecified, the property name will be used.
		/// </summary>
		public string? Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgPropertyAttribute(string? name = null)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Attribute marking the type of serializer for an object
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class BgObjectAttribute : Attribute
	{
		/// <summary>
		/// The serailizer to use for this object
		/// </summary>
		public Type SerializerType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="serializerType">The serializer type</param>
		public BgObjectAttribute(Type serializerType)
		{
			SerializerType = serializerType;
		}
	}

	/// <summary>
	/// Base class for deserializing objects from untyped <see cref="BgObjectDef"/> instances
	/// </summary>
	public abstract class BgObjectSerializer
	{
		/// <summary>
		/// Deserialize an object from the given property bag
		/// </summary>
		/// <param name="obj">Properties for the object</param>
		/// <returns>New object instance</returns>
		public abstract object Deserialize(BgObjectDef obj);
	}

	/// <summary>
	/// Strongly typed base class for deserializing objects
	/// </summary>
	/// <typeparam name="T">The object type</typeparam>
	public abstract class BgObjectSerializer<T> : BgObjectSerializer
	{
		/// <inheritdoc/>
		public sealed override object Deserialize(BgObjectDef obj) => Deserialize(obj.WithType<T>())!;

		/// <summary>
		/// Typed deserialization method
		/// </summary>
		/// <param name="obj">Properties for the object</param>
		/// <returns>Object instance</returns>
		public abstract T Deserialize(BgObjectDef<T> obj);
	}

	/// <summary>
	/// Default serializer for objects with a default constructor
	/// </summary>
	/// <typeparam name="T">Object type</typeparam>
	public class BgDefaultObjectSerializer<T> : BgObjectSerializer<T> where T : new()
	{
		/// <inheritdoc/>
		public override T Deserialize(BgObjectDef<T> obj)
		{
			T result = new();
			obj.CopyTo(result);
			return result;
		}
	}
}
