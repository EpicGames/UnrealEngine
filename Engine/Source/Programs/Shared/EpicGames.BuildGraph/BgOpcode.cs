// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Opcodes for graph expressions. Each opcode is followed by 1-3 operands
	/// </summary>
	public enum BgOpcode
	{
		#region Boolean opcodes

		/// <summary>
		/// Pushes a boolean 'false' value onto the stack
		/// </summary>
		BoolFalse = 0x00,

		/// <summary>
		/// Pushes a boolean 'true' value onto the stack
		/// </summary>
		BoolTrue = 0x01,

		/// <summary>
		/// Logical NOT
		/// </summary>
		BoolNot = 0x02,

		/// <summary>
		/// Logical AND
		/// </summary>
		BoolAnd = 0x03,

		/// <summary>
		/// Logical OR
		/// </summary>
		BoolOr = 0x04,

		/// <summary>
		/// Logical XOR
		/// </summary>
		BoolXor = 0x05,

		/// <summary>
		/// Tests whether two boolean values are equal
		/// </summary>
		BoolEq = 0x06,

		/// <summary>
		/// A boolean option
		/// </summary>
		BoolOption = 0x07,

		/// <summary>
		/// Converts a bool to a string
		/// </summary>
		BoolToString = 0x08,

		#endregion

		#region Integer opcodes

		/// <summary>
		/// An integer literal. Opcode is followed by a 32-bit little-endian integer.
		/// </summary>
		IntLiteral = 0x10,

		/// <summary>
		/// Tests whether two integers are equal
		/// </summary>
		IntEq = 0x11,

		/// <summary>
		/// Pops an integer from the stack, tests whether it is less than zero, and pushes a bool onto the stack with the result. 
		/// </summary>
		IntLt = 0x12,

		/// <summary>
		/// Pops an integer from the stack, tests whether it is greater than zero, and pushes a bool onto the stack with the result. 
		/// </summary>
		IntGt = 0x13,

		/// <summary>
		/// Adds two integers together
		/// </summary>
		IntAdd = 0x14,

		/// <summary>
		/// Multiplies two integers
		/// </summary>
		IntMultiply = 0x15,

		/// <summary>
		/// Divides one integer by another
		/// </summary>
		IntDivide = 0x16,

		/// <summary>
		/// Computes the modulo of one integer with another
		/// </summary>
		IntModulo = 0x17,

		/// <summary>
		/// Negates an integer
		/// </summary>
		IntNegate = 0x18,

		/// <summary>
		/// An integer option between two values (see <see cref="Expressions.BgIntOption"/>)
		/// </summary>
		IntOption = 0x19,

		/// <summary>
		/// Converts a bool to a string
		/// </summary>
		IntToString = 0x1a,

		#endregion

		#region String opcodes

		/// <summary>
		/// An empty string literal
		/// </summary>
		StrEmpty = 0x20,

		/// <summary>
		/// A string literal. Opcode is followed by a UTF-8 encoded string with length.
		/// </summary>
		StrLiteral = 0x21,

		/// <summary>
		/// Compares two strings for equality, using a <see cref="System.StringComparison"/> value encoded into the byte stream as an unsigned <see cref="EpicGames.Core.VarInt"/>.
		/// </summary>
		StrCompare = 0x22,

		/// <summary>
		/// Concatenates two strings and returns the result
		/// </summary>
		StrConcat = 0x23,

		/// <summary>
		/// Format a string, similar to <see cref="System.String.Format(System.String, System.Object?[])"/>
		/// </summary>
		StrFormat = 0x24,

		/// <summary>
		/// Joins a list of strings (first argument) with a separator (second argument)
		/// </summary>
		StrJoin = 0x25,

		/// <summary>
		/// Splits a string by a delimiter
		/// </summary>
		StrSplit = 0x26,

		/// <summary>
		/// Tests whether a string (first argument) matches a regular expression (second argument)
		/// </summary>
		StrMatch = 0x27,

		/// <summary>
		/// Returns a string with all ocurrences of the second argument in the first argument replaced with the third argument
		/// </summary>
		StrReplace = 0x28,

		/// <summary>
		/// A string option (see <see cref="Expressions.BgStringOption"/>)
		/// </summary>
		StrOption = 0x29,

		#endregion

		#region

		/// <summary>
		/// Literal enum value, as an integer.
		/// </summary>
		EnumConstant = 0x30,

		/// <summary>
		/// Parses a string as an enum
		/// </summary>
		EnumParse = 0x31,

		/// <summary>
		/// Converts an enum to a string
		/// </summary>
		EnumToString = 0x32,

		#endregion

		#region List opcodes

		/// <summary>
		/// Creates an empty list
		/// </summary>
		ListEmpty = 0x40,

		/// <summary>
		/// Adds an item to a list
		/// </summary>
		ListPush = 0x41,

		/// <summary>
		/// Adds a lazily evaluated item to a list
		/// </summary>
		ListPushLazy = 0x42,

		/// <summary>
		/// Gets the length of a list
		/// </summary>
		ListCount = 0x43,

		/// <summary>
		/// Gets the element of a list at an index. Second argument is an integer expression.
		/// </summary>
		ListElement = 0x44,

		/// <summary>
		/// Concatenates two lists
		/// </summary>
		ListConcat = 0x45,

		/// <summary>
		/// Creates the union of two lists
		/// </summary>
		ListUnion = 0x46,

		/// <summary>
		/// Creates the set excluding another set of items
		/// </summary>
		ListExcept = 0x47,

		/// <summary>
		/// Call a function on each element of a list, returning the transformed list
		/// </summary>
		ListSelect = 0x48,

		/// <summary>
		/// Select elements from a list using a predicate
		/// </summary>
		ListWhere = 0x49,

		/// <summary>
		/// Selects all the unique entries in the list. The comparer for the list type is determined by the type of the first element.
		/// </summary>
		ListDistinct = 0x4a,

		/// <summary>
		/// Determines whether a list contains a particular item
		/// </summary>
		ListContains = 0x4b,

		/// <summary>
		/// Indicates that the list should only be evaluated when the first item is enumerated from it.
		/// </summary>
		ListLazy = 0x4c,

		/// <summary>
		/// An option providing a list of strings (see <see cref="Expressions.BgListOption"/>
		/// </summary>
		ListOption = 0x4d,

		#endregion

		#region Object opcodes

		/// <summary>
		/// An empty object
		/// </summary>
		ObjEmpty = 0x60,

		/// <summary>
		/// Gets the value of a property
		/// </summary>
		ObjGet = 0x61,

		/// <summary>
		/// Sets the value of a property
		/// </summary>
		ObjSet = 0x62,

		#endregion

		#region Function opcodes

		/// <summary>
		/// Takes an operand indicating the number of arguments, and offset of the function to call. Pops the arguments from the evaluation stack, pushes them onto the function stack, and jumps to the given offset.
		/// </summary>
		Call = 0xd0,

		/// <summary>
		/// Fetches a numbered argument from the functions stack frame.
		/// </summary>
		Argument = 0xd1,

		/// <summary>
		/// Jumps to another fragment without creating a new stack frame
		/// </summary>
		Jump = 0xd2,

		#endregion

		#region Generic opcodes

		/// <summary>
		/// Chooses between two operands based on a boolean parameter.
		/// </summary>
		Choose = 0xe0,

		/// <summary>
		/// Throws an exception
		/// </summary>
		Throw = 0xe1,

		/// <summary>
		/// Null value for an optional expression
		/// </summary>
		Null = 0xe2,

		/// <summary>
		/// A native method invocation, represented as an index into the method table
		/// </summary>
		Thunk = 0xe3,

		#endregion
	}
}
