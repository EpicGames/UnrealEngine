// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Represents a graph option. These are expanded during preprocessing, but are retained in order to display help messages.
	/// </summary>
	public abstract class BgOptionDef
	{
		/// <summary>
		/// Name of this option
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Label for the option in the dashboard
		/// </summary>
		public string? Label { get; set; }

		/// <summary>
		/// Description for this option
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">The name of this option</param>
		protected BgOptionDef(string name)
		{
			Name = name;
		}

		/// <summary>
		/// Returns the default argument value
		/// </summary>
		/// <returns>Default argument value</returns>
		public abstract string? GetDefaultArgument();

		/// <summary>
		/// Returns a name of this option for debugging
		/// </summary>
		/// <returns>Name of the option</returns>
		public override string ToString()
		{
			return Name;
		}
	}

	/// <summary>
	/// Definition of a boolean option
	/// </summary>
	[BgObject(typeof(BgBoolOptionSerializer))]
	public class BgBoolOptionDef : BgOptionDef
	{
		/// <summary>
		/// Default value for the option
		/// </summary>
		public bool DefaultValue { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">The name of this option</param>
		public BgBoolOptionDef(string name)
			: base(name)
		{
		}

		/// <inheritdoc/>
		public override string? GetDefaultArgument() => DefaultValue.ToString();
	}

	/// <summary>
	/// Serializer for bool options
	/// </summary>
	class BgBoolOptionSerializer : BgObjectSerializer<BgBoolOptionDef>
	{
		/// <inheritdoc/>
		public override BgBoolOptionDef Deserialize(BgObjectDef<BgBoolOptionDef> obj)
		{
			BgBoolOptionDef option = new(obj.Get(x => x.Name, null!));
			obj.CopyTo(option);
			return option;
		}
	}

	/// <summary>
	/// Definition of an integer option
	/// </summary>
	[BgObject(typeof(BgIntOptionSerializer))]
	public class BgIntOptionDef : BgOptionDef
	{
		/// <summary>
		/// Default value for the option
		/// </summary>
		public int DefaultValue { get; set; }

		/// <summary>
		/// Minimum allowed value
		/// </summary>
		public int? MinValue { get; set; }

		/// <summary>
		/// Maximum allowed value
		/// </summary>
		public int? MaxValue { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">The name of this option</param>
		public BgIntOptionDef(string name)
			: base(name)
		{
		}

		/// <inheritdoc/>
		public override string? GetDefaultArgument() => DefaultValue.ToString();
	}

	/// <summary>
	/// Serializer for int options
	/// </summary>
	class BgIntOptionSerializer : BgObjectSerializer<BgIntOptionDef>
	{
		/// <inheritdoc/>
		public override BgIntOptionDef Deserialize(BgObjectDef<BgIntOptionDef> obj)
		{
			BgIntOptionDef option = new(obj.Get(x => x.Name, null!));
			obj.CopyTo(option);
			return option;
		}
	}

	/// <summary>
	/// Style for a string option
	/// </summary>
	public enum BgStringOptionStyle
	{
		/// <summary>
		/// Free-form text entry
		/// </summary>
		Text,

		/// <summary>
		/// List of options
		/// </summary>
		DropList,
	}

	/// <summary>
	/// Definition of a string option
	/// </summary>
	[BgObject(typeof(BgStringOptionSerializer))]
	public class BgStringOptionDef : BgOptionDef
	{
		/// <summary>
		/// Default value for the option
		/// </summary>
		public string DefaultValue { get; set; } = String.Empty;

		/// <summary>
		/// Style for this option
		/// </summary>
		public BgStringOptionStyle Style { get; }

		/// <summary>
		/// Regex for validating values for the option
		/// </summary>
		public string? Pattern { get; set; }

		/// <summary>
		/// Message to display if validation fails
		/// </summary>
		public string? PatternFailed { get; set; }

		/// <summary>
		/// List of values to choose from
		/// </summary>
		public List<string>? Values { get; set; }

		/// <summary>
		/// Matching list of descriptions for each value
		/// </summary>
		public List<string>? ValueDescriptions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the option</param>
		public BgStringOptionDef(string name)
			: base(name)
		{
		}

		/// <inheritdoc/>
		public override string? GetDefaultArgument() => DefaultValue;
	}

	/// <summary>
	/// Serializer for string options
	/// </summary>
	class BgStringOptionSerializer : BgObjectSerializer<BgStringOptionDef>
	{
		/// <inheritdoc/>
		public override BgStringOptionDef Deserialize(BgObjectDef<BgStringOptionDef> obj)
		{
			BgStringOptionDef option = new(obj.Get(x => x.Name, null!));
			obj.CopyTo(option);
			return option;
		}
	}

	/// <summary>
	/// Style for a list option
	/// </summary>
	public enum BgListOptionStyle
	{
		/// <summary>
		/// List of checkboxes
		/// </summary>
		CheckList = 0,

		/// <summary>
		/// Tag picker
		/// </summary>
		TagPicker = 1,
	}

	/// <summary>
	/// A list option definition
	/// </summary>
	[BgObject(typeof(BgListOptionSerializer))]
	public class BgListOptionDef : BgOptionDef
	{
		/// <summary>
		/// Style for this list box
		/// </summary>
		public BgListOptionStyle Style { get; }

		/// <summary>
		/// Default value for the option
		/// </summary>
		public string? DefaultValue { get; set; }

		/// <summary>
		/// List of values to choose from
		/// </summary>
		public List<string>? Values { get; set; }

		/// <summary>
		/// Matching list of descriptions for each value
		/// </summary>
		public List<string>? ValueDescriptions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the option</param>
		public BgListOptionDef(string name)
			: base(name)
		{
		}

		/// <inheritdoc/>
		public override string? GetDefaultArgument() => DefaultValue;
	}

	/// <summary>
	/// Serializer for string options
	/// </summary>
	class BgListOptionSerializer : BgObjectSerializer<BgListOptionDef>
	{
		/// <inheritdoc/>
		public override BgListOptionDef Deserialize(BgObjectDef<BgListOptionDef> obj)
		{
			BgListOptionDef option = new(obj.Get(x => x.Name, null!));
			obj.CopyTo(option);
			return option;
		}
	}
}
