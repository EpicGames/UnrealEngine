// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Defines a report to be generated as part of the build.
	/// </summary>
	public class BgReport
	{
		/// <summary>
		/// Name of this trigger
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Set of nodes to include in the report
		/// </summary>
		public HashSet<BgNodeDef> Nodes { get; } = [];

		/// <summary>
		/// List of users to notify with this report
		/// </summary>
		public HashSet<string> NotifyUsers { get; } = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inName">Name of this report</param>
		public BgReport(string inName)
		{
			Name = inName;
		}

		/// <summary>
		/// Get the name of this report
		/// </summary>
		/// <returns>The name of this report</returns>
		public override string ToString()
		{
			return Name;
		}
	}
}
