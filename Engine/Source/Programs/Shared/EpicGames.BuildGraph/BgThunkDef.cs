// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Reflection;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Information about the method bound to execute a node
	/// </summary>
	public class BgThunkDef
	{
		/// <summary>
		/// Method to call
		/// </summary>
		public MethodInfo Method { get; }

		/// <summary>
		/// Arguments to the method
		/// </summary>
		public IReadOnlyList<object?> Arguments { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgThunkDef(MethodInfo method, IReadOnlyList<object?> arguments)
		{
			Method = method;
			Arguments = arguments;
		}
	}

	/// <summary>
	/// Outputs from a thunk
	/// </summary>
	public class BgThunkOutputDef
	{
		/// <summary>
		/// The thunk definition
		/// </summary>
		public BgThunkDef Thunk { get; }

		/// <summary>
		/// Output index from the thunk
		/// </summary>
		public int Index { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="thunk"></param>
		/// <param name="index"></param>
		public BgThunkOutputDef(BgThunkDef thunk, int index)
		{
			Thunk = thunk;
			Index = index;
		}
	}
}
