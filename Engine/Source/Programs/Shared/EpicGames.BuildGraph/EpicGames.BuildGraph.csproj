<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>
</Project>
