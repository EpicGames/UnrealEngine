// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Describes an agent that can execute execute build steps
	/// </summary>
	public class BgAgent : BgExpr
	{
		/// <summary>
		/// Name of the agent
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// List of agent types to select from, in order of preference. The first agent type supported by a stream will be used.
		/// </summary>
		public BgList<BgString> Types { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgAgent(BgString name, BgString type)
			: this(name, BgList.Create(type))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BgAgent(BgString name, BgList<BgString> types)
			: base(BgExprFlags.ForceFragment)
		{
			Name = name;
			Types = types;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgAgentDef> obj = BgObject<BgAgentDef>.Empty;
			obj = obj.Set(x => x.Name, Name);
			obj = obj.Set(x => x.PossibleTypes, Types);
			writer.WriteExpr(obj);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{Agent}";
	}
}
