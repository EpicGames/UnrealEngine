// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Specification for an aggregate target in the graph
	/// </summary>
	public class BgAggregate : BgExpr
	{
		/// <summary>
		/// Name of the aggregate
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Outputs required for the aggregate
		/// </summary>
		public BgList<BgNode> Requires { get; }

		/// <summary>
		/// Label to apply to this aggregate
		/// </summary>
		public BgLabel? Label { get; }

		/// <summary>
		/// Constructor.
		/// </summary>
		public BgAggregate(BgString name, params BgNode[] requires)
			: this(name, BgList.Create(requires))
		{
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		public BgAggregate(BgString name, params BgList<BgNode>[] requires)
			: this(name, BgList.Concat(requires))
		{
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		public BgAggregate(BgString name, BgList<BgNode> requires, string label)
			: this(name, requires, new BgLabel(label))
		{
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		public BgAggregate(BgString name, BgList<BgNode> requires, BgLabel? label = null)
			: base(BgExprFlags.None)
		{
			Name = name;
			Requires = requires;
			Label = label;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgAggregateExpressionDef> obj = BgObject<BgAggregateExpressionDef>.Empty;
			obj = obj.Set(x => x.Name, Name);
			obj = obj.Set(x => x.RequiredNodes, Requires);
			if (Label != null)
			{
				obj = obj.Set(x => x.Label, Label);
			}
			writer.WriteExpr(obj);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => Name;
	}
}
