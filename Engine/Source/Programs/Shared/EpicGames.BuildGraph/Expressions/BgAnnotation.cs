// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph.Expressions;

/// <summary>
/// Specification for a annotation
/// </summary>
public class BgAnnotation : BgExpr
{
	/// <summary>
	/// Name, or key, of the annotation
	/// </summary>
	public BgString Name { get; }

	/// <summary>
	/// Value to be paired with the annotation name
	/// </summary>
	public BgString Value { get; }

	/// <inheritdoc/>
	public BgAnnotation(BgString name, BgString value)
		: base(BgExprFlags.ForceFragment)
	{
		Name = name;
		Value = value;
	}

	/// <inheritdoc/>
	public override void Write(BgBytecodeWriter writer)
	{
		BgObject<BgAnnotationDef> obj = BgObject<BgAnnotationDef>.Empty;
		obj = obj.Set(x => x.Name, Name);
		obj = obj.Set(x => x.Value, Value);
		writer.WriteExpr(obj);
	}

	/// <inheritdoc/>
	public override BgString ToBgString() => "{Annotation}";
}