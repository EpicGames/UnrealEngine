// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for expressions returning a boolean value 
	/// </summary>
	[BgType(typeof(BgBoolType))]
	public abstract class BgBool : BgExpr
	{
		/// <summary>
		/// Constant value for false
		/// </summary>
		public static BgBool False { get; } = false;

		/// <summary>
		/// Constant value for true
		/// </summary>
		public static BgBool True { get; } = true;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags">Flags for this expression</param>
		protected BgBool(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <summary>
		/// Implict conversion operator from a boolean literal
		/// </summary>
		public static implicit operator BgBool(bool value)
		{
			return new BgBoolConstantExpr(value);
		}

		/// <inheritdoc/>
		public static BgBool operator !(BgBool inner) => new BgBoolNotExpr(inner);

		/// <inheritdoc/>
		public static BgBool operator &(BgBool lhs, BgBool rhs) => new BgBoolBinaryExpr(BgOpcode.BoolAnd, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator |(BgBool lhs, BgBool rhs) => new BgBoolBinaryExpr(BgOpcode.BoolOr, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator ^(BgBool lhs, BgBool rhs) => new BgBoolBinaryExpr(BgOpcode.BoolXor, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator ==(BgBool lhs, BgBool rhs) => new BgBoolBinaryExpr(BgOpcode.BoolEq, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator !=(BgBool lhs, BgBool rhs) => !(lhs == rhs);

		/// <inheritdoc/>
		public sealed override bool Equals(object? obj) => throw new InvalidOperationException();

		/// <inheritdoc/>
		public sealed override int GetHashCode() => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override BgString ToBgString() => new BgBoolToBgStringExpr(this);
	}

	/// <summary>
	/// Type traits for a <see cref="BgBool"/>
	/// </summary>
	class BgBoolType : BgType<BgBool>
	{
		/// <inheritdoc/>
		public override BgBool Constant(object value) => new BgBoolConstantExpr((bool)value);

		/// <inheritdoc/>
		public override BgBool Wrap(BgExpr expr) => new BgBoolWrappedExpr(expr);
	}

	#region Expression classes

	class BgBoolNotExpr : BgBool
	{
		public BgBool Inner { get; }

		public BgBoolNotExpr(BgBool inner)
			: base(inner.Flags & BgExprFlags.Eager)
		{
			Inner = inner;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.BoolNot);
			writer.WriteExpr(Inner);
		}
	}

	class BgBoolBinaryExpr : BgBool
	{
		public BgOpcode Opcode { get; }
		public BgBool Lhs { get; }
		public BgBool Rhs { get; }

		public BgBoolBinaryExpr(BgOpcode opcode, BgBool lhs, BgBool rhs)
			: base(lhs.Flags & rhs.Flags & BgExprFlags.Eager)
		{
			Opcode = opcode;
			Lhs = lhs;
			Rhs = rhs;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(Opcode);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
		}
	}

	class BgBoolConstantExpr : BgBool
	{
		public bool Value { get; }

		public BgBoolConstantExpr(bool value)
			: base(BgExprFlags.NotInterned | BgExprFlags.Eager)
		{
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer) => writer.WriteOpcode(Value ? BgOpcode.BoolTrue : BgOpcode.BoolFalse);
	}

	class BgBoolWrappedExpr : BgBool
	{
		public BgExpr Expr { get; }

		public BgBoolWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	class BgBoolToBgStringExpr : BgString
	{
		public BgExpr Expr { get; }

		public BgBoolToBgStringExpr(BgExpr expr)
			: base(expr.Flags & BgExprFlags.Eager)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.BoolToString);
			writer.WriteExpr(Expr);
		}
	}

	#endregion

	/// <summary>
	/// A boolean option expression
	/// </summary>
	public class BgBoolOption : BgBool
	{
		/// <summary>
		/// Name of the option
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Label to display next to the option
		/// </summary>
		public BgString? Label { get; }

		/// <summary>
		/// Help text to display for the user
		/// </summary>
		public BgString? Description { get; }

		/// <summary>
		/// Default value for the option
		/// </summary>
		public BgBool? DefaultValue { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgBoolOption(BgString name, BgString? description = null, BgBool? defaultValue = null)
			: this(name, null, description, defaultValue)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BgBoolOption(BgString name, BgString? label, BgString? description, BgBool? defaultValue)
			: base(BgExprFlags.None)
		{
			Name = name;
			Label = label;
			Description = description;
			DefaultValue = defaultValue;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.BoolOption);
			writer.WriteExpr(CreateOptionsObject());
		}

		BgObject<BgBoolOptionDef> CreateOptionsObject()
		{
			BgObject<BgBoolOptionDef> option = BgObject<BgBoolOptionDef>.Empty;
			option = option.Set(x => x.Name, Name);
			if (Label is not null)
			{
				option = option.Set(x => x.Label, Label);
			}
			if (Description is not null)
			{
				option = option.Set(x => x.Description, Description);
			}
			if (DefaultValue is not null)
			{
				option = option.Set(x => x.DefaultValue, DefaultValue);
			}
			return option;
		}
	}
}
