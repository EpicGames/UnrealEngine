// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for expressions returning a string value 
	/// </summary>
	[BgType(typeof(BgEnumType<>))]
	public abstract class BgEnum<TEnum> : BgExpr where TEnum : struct
	{
		/// <inheritdoc/>
		public Type EnumType => typeof(TEnum);

		/// <summary>
		/// Names of this enum
		/// </summary>
		public static BgList<BgString> Names { get; } = BgList.Create(Enum.GetNames(typeof(TEnum)).Select(x => (BgString)x));

		/// <summary>
		/// Values of this enum
		/// </summary>
		public static BgList<BgInt> Values { get; } = BgList.Create(((TEnum[])Enum.GetValues(typeof(TEnum))).Select(x => (BgInt)(int)(object)x));

		/// <summary>
		/// Constructor
		/// </summary>
		protected BgEnum(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <summary>
		/// Implicit conversion from a regular enum type
		/// </summary>
		public static implicit operator BgEnum<TEnum>(TEnum value)
		{
			return new BgEnumConstantExpr<TEnum>(value);
		}

		/// <summary>
		/// Explicit conversion from a string value
		/// </summary>
		public static BgEnum<TEnum> Parse(BgString value)
		{
			return new BgEnumParseExpr<TEnum>(value);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => new BgEnumToStringExpr<TEnum>(this);
	}

	/// <summary>
	/// Type traits for a <see cref="BgEnum{TEnum}"/>
	/// </summary>
	class BgEnumType<TEnum> : BgType<BgEnum<TEnum>> where TEnum : struct
	{
		/// <inheritdoc/>
		public override BgEnum<TEnum> Constant(object value)
		{
			return new BgEnumConstantExpr<TEnum>((TEnum)value);
		}

		/// <inheritdoc/>
		public override BgEnum<TEnum> Wrap(BgExpr expr)
		{
			throw new NotImplementedException();
		}
	}

	#region Expression classes

	class BgEnumConstantExpr<TEnum> : BgEnum<TEnum> where TEnum : struct
	{
		public TEnum Value { get; }

		public BgEnumConstantExpr(TEnum value)
			: base(BgExprFlags.NotInterned | BgExprFlags.Eager)
		{
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.EnumConstant);
			writer.WriteSignedInteger((int)(object)Value);
		}
	}

	class BgEnumParseExpr<TEnum> : BgEnum<TEnum> where TEnum : struct
	{
		public BgString Value { get; }

		public BgEnumParseExpr(BgString value)
			: base(value.Flags & BgExprFlags.Eager)
		{
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.EnumParse);
			writer.WriteExpr(Value);
			writer.WriteExpr(Names);
			writer.WriteExpr(Values);
		}
	}

	class BgEnumToStringExpr<TEnum> : BgString where TEnum : struct
	{
		public BgEnum<TEnum> Expr { get; }

		public BgEnumToStringExpr(BgEnum<TEnum> expr)
			: base(expr.Flags & BgExprFlags.Eager)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.EnumToString);
			writer.WriteExpr(Expr);
			writer.WriteExpr(BgEnum<TEnum>.Names);
			writer.WriteExpr(BgEnum<TEnum>.Values);
		}
	}

	class BgEnumWrappedExpr<TEnum> : BgEnum<TEnum> where TEnum : struct
	{
		public BgExpr Expr { get; }

		public BgEnumWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			Expr.Write(writer);
		}
	}

	#endregion
}
