// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.CompilerServices;
using EpicGames.BuildGraph.Expressions;

namespace EpicGames.BuildGraph
{
	/// <summary>
	/// Flags for an expression
	/// </summary>
	[Flags]
	public enum BgExprFlags
	{
		/// <summary>
		/// No flags set
		/// </summary>
		None = 0,

		/// <summary>
		/// Indicates that the expression should never be interned (ie. encoded to a separate fragment), and will always be duplicated in the bytecode. 
		/// Trivial constants with short encodings typically have this flag set.
		/// </summary>
		NotInterned = 1,

		/// <summary>
		/// Always eagerly evaluate this expression
		/// </summary>
		Eager = 2,

		/// <summary>
		/// Force this expression to be stored in a separate fragment. Can help improves readability of the disassembly output.
		/// </summary>
		ForceFragment = 4,
	}

	/// <summary>
	/// Base class for computable expressions
	/// </summary>
	public abstract class BgExpr
	{
		/// <summary>
		/// Null value
		/// </summary>
		public static BgExpr Null { get; } = new BgNullExpr();

		/// <summary>
		/// Flags for this expression
		/// </summary>
		public BgExprFlags Flags { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags"></param>
		protected BgExpr(BgExprFlags flags)
		{
			Flags = flags;
		}

		/// <summary>
		/// Throws an exception
		/// </summary>
		/// <typeparam name="T">Type of the expression to masquerade as</typeparam>
		/// <param name="message">Message to display</param>
		/// <param name="sourcePath">Path to the source file declaring this diagnostic. Automatically set by the runtime.</param>
		/// <param name="sourceLine">Line number in the source file declaring this diagnostic. Automatically set by the runtime.</param>
		public static T Throw<T>(BgString message, [CallerFilePath] string sourcePath = "", [CallerLineNumber] int sourceLine = 0) where T : BgExpr
		{
			return BgType.Wrap<T>(new BgThrowExpr(sourcePath, sourceLine, message));
		}

		/// <summary>
		/// Chooses between two values based on a condition
		/// </summary>
		/// <typeparam name="T">Type of the expression to choose between</typeparam>
		/// <param name="condition">Condition to check</param>
		/// <param name="valueIfTrue">Value to return if the condition is true</param>
		/// <param name="valueIfFalse">Value to return if the condition is false</param>
		/// <returns>The chosen value</returns>
		public static T Choose<T>(BgBool condition, T valueIfTrue, T valueIfFalse) where T : BgExpr
		{
			return BgType.Wrap<T>(new BgChooseExpr<T>(condition, valueIfTrue, valueIfFalse));
		}

		/// <summary>
		/// Serialize the expression to an output stream
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public abstract void Write(BgBytecodeWriter writer);

		/// <summary>
		/// Convert the value of the expression to a string
		/// </summary>
		public abstract BgString ToBgString();
	}

	class BgChooseExpr<T> : BgExpr where T : BgExpr
	{
		public BgBool Condition { get; }
		public T ValueIfTrue { get; }
		public T ValueIfFalse { get; }

		public BgChooseExpr(BgBool condition, T valueIfTrue, T valueIfFalse)
			: base(BgExprFlags.None)
		{
			Condition = condition;
			ValueIfTrue = valueIfTrue;
			ValueIfFalse = valueIfFalse;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Choose);
			writer.WriteExpr(Condition);
			writer.WriteExprAsFragment(ValueIfTrue);
			writer.WriteExprAsFragment(ValueIfFalse);
		}

		public override BgString ToBgString() => throw new InvalidOperationException();
	}

	class BgThrowExpr : BgExpr
	{
		public string SourcePath { get; }
		public int SourceLine { get; }
		public BgString Message { get; }

		public BgThrowExpr(string sourcePath, int sourceLine, BgString message)
			: base(BgExprFlags.None)
		{
			SourcePath = sourcePath;
			SourceLine = sourceLine;
			Message = message;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Throw);
			writer.WriteString(SourcePath);
			writer.WriteUnsignedInteger(SourceLine);
			writer.WriteExpr(Message);
		}

		public override BgString ToBgString() => Message;
	}

	class BgNullExpr : BgExpr
	{
		public BgNullExpr()
			: base(BgExprFlags.NotInterned)
		{
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Null);
		}

		public override BgString ToBgString() => "null";
	}

	/// <summary>
	/// Extension methods for expressions
	/// </summary>
	public static class BgExprExtensions
	{
		/// <summary>
		/// Chose an expression if a condition evaluates to true
		/// </summary>
		/// <typeparam name="T">Type of the expression</typeparam>
		/// <param name="expr">The expression value</param>
		/// <param name="condition">Condition to test</param>
		/// <param name="value">Value to return if the condition is true</param>
		/// <returns>New expression</returns>
		public static T If<T>(this T expr, BgBool condition, T value) where T : BgExpr
		{
			return BgExpr.Choose(condition, value, expr);
		}

		/// <summary>
		/// Chose an expression if a condition evaluates to true
		/// </summary>
		/// <typeparam name="T">Type of the expression</typeparam>
		/// <param name="expr">The expression value</param>
		/// <param name="condition">Condition to test</param>
		/// <param name="func">Function to apply to the expression if the condition is true</param>
		/// <returns>New expression</returns>
		public static T If<T>(this T expr, BgBool condition, Func<T, T> func) where T : BgExpr
		{
			return BgExpr.Choose(condition, func(expr), expr);
		}
	}
}
