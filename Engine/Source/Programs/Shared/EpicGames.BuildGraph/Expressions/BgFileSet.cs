// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Represents a placeholder for the output from a node, which can be exchanged for the artifacts produced by a node at runtime
	/// </summary>
	[BgType(typeof(BgFileSetType))]
	public abstract class BgFileSet : BgExpr
	{
		/// <summary>
		/// Constant empty fileset
		/// </summary>
		public static BgFileSet Empty { get; } = new BgFileSetOutputExpr(FileSet.Empty);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags"></param>
		protected BgFileSet(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <summary>
		/// Implicit conversion from a regular fileset
		/// </summary>
		/// <param name="fileSet"></param>
		public static implicit operator BgFileSet(FileSet fileSet)
		{
			return new BgFileSetOutputExpr(fileSet);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{FileSet}";
	}

	/// <summary>
	/// Traits for a <see cref="BgFileSet"/>
	/// </summary>
	class BgFileSetType : BgType<BgFileSet>
	{
		/// <inheritdoc/>
		public override BgFileSet Constant(object value)
		{
			BgObjectDef obj = (BgObjectDef)value;
			BgNodeOutput[] outputs = [.. obj.Deserialize<BgNodeOutputExprDef>().Flatten()];
			return new BgFileSetInputExpr(outputs);
		}

		/// <inheritdoc/>
		public override BgFileSet Wrap(BgExpr expr) => new BgFileSetWrappedExpr(expr);
	}

	#region Expression classes

	class BgFileSetInputExpr : BgFileSet
	{
		public IReadOnlyList<BgNodeOutput> Outputs { get; }

		public BgFileSetInputExpr(BgNodeOutput[] outputs)
			: base(BgExprFlags.None)
		{
			Outputs = outputs;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			throw new NotImplementedException();
		}
	}

	/// <summary>
	/// /
	/// </summary>
	public class BgFileSetOutputExpr : BgFileSet
	{
		/// <summary>
		/// 
		/// </summary>
		public FileSet Value { get; }

		/// <summary>
		/// 
		/// </summary>
		/// <param name="value"></param>
		public BgFileSetOutputExpr(FileSet value)
			: base(BgExprFlags.NotInterned)
		{
			Value = value;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			throw new NotImplementedException();
		}
	}

	class BgFileSetFromNodeExpr : BgFileSet
	{
		public BgNode Node { get; }

		public BgFileSetFromNodeExpr(BgNode node)
			: base(BgExprFlags.NotInterned)
		{
			Node = node;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgNodeOutputExprDef> obj = BgObject<BgNodeOutputExprDef>.Empty;
			obj = obj.Set(x => x.ProducingNode, Node);
			writer.WriteExpr(obj);
		}
	}

	class BgFileSetFromNodeOutputExpr : BgFileSet
	{
		public BgNode Node { get; }
		public int OutputIndex { get; }

		public BgFileSetFromNodeOutputExpr(BgNode node, int outputIndex)
			: base(BgExprFlags.NotInterned)
		{
			Node = node;
			OutputIndex = outputIndex;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgNodeOutputExprDef> obj = BgObject<BgNodeOutputExprDef>.Empty;
			obj = obj.Set(x => x.ProducingNode, Node);
			obj = obj.Set(x => x.OutputIndex, (BgInt)OutputIndex);
			writer.WriteExpr(obj);
		}
	}

	class BgFileSetWrappedExpr : BgFileSet
	{
		public BgExpr Expr { get; }

		public BgFileSetWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	#endregion
}

