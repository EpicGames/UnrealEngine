// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for function expressions
	/// </summary>
	public abstract class BgFunc
	{
		/// <summary>
		/// The function expression.
		/// </summary>
		public BgExpr Body { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="body">Expression to be evaluated</param>
		protected BgFunc(BgExpr body)
		{
			// Evaluate the body twice, and determine which expressions are captured outside this context
			// Lexical scope can include those expressions too - if they are declared locally, they will not be duplicated
			Body = body;
		}
	}

	/// <summary>
	/// A function taking no arguments
	/// </summary>
	/// <typeparam name="TResult">Result type</typeparam>
	public class BgFunc<TResult> : BgFunc
		where TResult : BgExpr
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="func">Function to construct from</param>
		public BgFunc(Func<TResult> func)
			: base(func())
		{
		}

		/// <summary>
		/// Implicit conversion from a regular C# function
		/// </summary>
		public static implicit operator BgFunc<TResult>(Func<TResult> func) => new(func);

		/// <summary>
		/// Call the function with the given arguments
		/// </summary>
		/// <returns>Result from the function</returns>
		public TResult Call() => BgType.Wrap<TResult>(new BgFuncCallExpr<TResult>(this));
	}

	/// <summary>
	/// A function taking a single argument
	/// </summary>
	/// <typeparam name="TArg">Type of the function argument</typeparam>
	/// <typeparam name="TResult">Result type</typeparam>
	public class BgFunc<TArg, TResult> : BgFunc
		where TArg : BgExpr
		where TResult : BgExpr
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="func">Function to construct from</param>
		public BgFunc(Func<TArg, TResult> func)
			: base(func(BgType.Wrap<TArg>(new BgFuncArgumentExpr(0))))
		{
		}

		/// <summary>
		/// Implicit conversion from a regular C# function
		/// </summary>
		public static implicit operator BgFunc<TArg, TResult>(Func<TArg, TResult> func) => new(func);

		/// <summary>
		/// Call the function with the given arguments
		/// </summary>
		/// <param name="arg">Argument to pass to the function</param>
		/// <returns>Result from the function</returns>
		public TResult Call(TArg arg) => BgType.Wrap<TResult>(new BgFuncCallExpr<TResult>(this, arg));
	}

	/// <summary>
	/// A function taking two arguments
	/// </summary>
	/// <typeparam name="TArg1">Type of the first function argument</typeparam>
	/// <typeparam name="TArg2">Type of the second function argument</typeparam>
	/// <typeparam name="TResult">Result type</typeparam>
	public class BgFunc<TArg1, TArg2, TResult> : BgFunc
		where TArg1 : BgExpr
		where TArg2 : BgExpr
		where TResult : BgExpr
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="func">Function to construct from</param>
		public BgFunc(Func<TArg1, TArg2, TResult> func)
			: base(func(BgType.Wrap<TArg1>(new BgFuncArgumentExpr(0)), BgType.Wrap<TArg2>(new BgFuncArgumentExpr(1))))
		{
		}

		/// <summary>
		/// Implicit conversion from a regular C# function
		/// </summary>
		public static implicit operator BgFunc<TArg1, TArg2, TResult>(Func<TArg1, TArg2, TResult> func) => new(func);

		/// <summary>
		/// Call the function with the given arguments
		/// </summary>
		/// <param name="arg1">First argument to the function</param>
		/// <param name="arg2">Second argument to the function</param>
		/// <returns>Result from the function</returns>
		public TResult Call(TArg1 arg1, TArg2 arg2) => BgType.Wrap<TResult>(new BgFuncCallExpr<TResult>(this, arg1, arg2));
	}

	#region Expression classes

	class BgFuncArgumentExpr : BgExpr
	{
		uint Index { get; }

		public BgFuncArgumentExpr(uint index)
			: base(BgExprFlags.NotInterned)
		{
			Index = index;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Argument);
			writer.WriteUnsignedInteger(Index);
		}

		public override BgString ToBgString() => "{Function}";
	}

	class BgFuncCallExpr<T> : BgExpr where T : BgExpr
	{
		readonly BgFunc _function;
		readonly BgExpr[] _arguments;

		public BgFuncCallExpr(BgFunc function, params BgExpr[] arguments)
			: base(BgExprFlags.None)
		{
			_function = function;
			_arguments = arguments;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Call);
			writer.WriteExprAsFragment(_function.Body);
			writer.WriteUnsignedInteger((uint)_arguments.Length);
			foreach (BgExpr argument in _arguments)
			{
				argument.Write(writer);
			}
		}

		public override BgString ToBgString() => throw new InvalidOperationException();
	}

	#endregion
}
