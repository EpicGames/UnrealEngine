// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Specification for a graph in fluent syntax
	/// </summary>
	public class BgGraph : BgExpr
	{
		/// <summary>
		/// Nodes for the graph
		/// </summary>
		public BgList<BgNode> Nodes { get; }

		/// <summary>
		/// Aggregates for the graph
		/// </summary>
		public BgList<BgAggregate> Aggregates { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgGraph(BgList<BgNode> nodes, BgList<BgAggregate> aggregates)
			: base(BgExprFlags.ForceFragment)
		{
			Nodes = nodes;
			Aggregates = aggregates;
		}

		/// <summary>
		/// Implicit conversion from a node spec
		/// </summary>
		public static implicit operator BgGraph(BgNode node)
		{
			return new BgGraph(node, BgList<BgAggregate>.Empty);
		}

		/// <summary>
		/// Implicit conversion from a list of node specs
		/// </summary>
		public static implicit operator BgGraph(BgList<BgNode> nodes)
		{
			return new BgGraph(nodes, BgList<BgAggregate>.Empty);
		}

		/// <summary>
		/// Implicit conversion from an aggregate spec
		/// </summary>
		public static implicit operator BgGraph(BgAggregate aggregate)
		{
			return new BgGraph(BgList<BgNode>.Empty, aggregate);
		}

		/// <summary>
		/// Implicit conversion from a list of node specs
		/// </summary>
		public static implicit operator BgGraph(BgList<BgAggregate> aggregates)
		{
			return new BgGraph(BgList<BgNode>.Empty, aggregates);
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgGraphExpressionDef> obj = BgObject<BgGraphExpressionDef>.Empty;
			obj = obj.Set(x => x.Nodes, Nodes);
			obj = obj.Set(x => x.Aggregates, Aggregates);
			writer.WriteExpr(obj);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{Graph}";
	}
}
