// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for expressions returning a 32-bit integer value 
	/// </summary>
	[BgType(typeof(BgIntType))]
	public abstract class BgInt : BgExpr
	{
		/// <summary>
		/// Implicit conversion from an integer value
		/// </summary>
		public static implicit operator BgInt(int value)
		{
			return new BgIntConstantExpr(value);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags">Flags for this expression</param>
		protected BgInt(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <inheritdoc/>
		public static BgInt operator -(BgInt value) => new BgIntUnaryExpr(BgOpcode.IntNegate, value);

		/// <inheritdoc/>
		public static BgInt operator +(BgInt lhs, BgInt rhs) => new BgIntBinaryExpr(BgOpcode.IntAdd, lhs, rhs);

		/// <inheritdoc/>
		public static BgInt operator -(BgInt lhs, BgInt rhs) => new BgIntBinaryExpr(BgOpcode.IntAdd, lhs, -rhs);

		/// <inheritdoc/>
		public static BgInt operator *(BgInt lhs, BgInt rhs) => new BgIntBinaryExpr(BgOpcode.IntMultiply, lhs, rhs);

		/// <inheritdoc/>
		public static BgInt operator /(BgInt lhs, BgInt rhs) => new BgIntBinaryExpr(BgOpcode.IntDivide, lhs, rhs);

		/// <inheritdoc/>
		public static BgInt operator %(BgInt lhs, BgInt rhs) => new BgIntBinaryExpr(BgOpcode.IntModulo, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator <(BgInt lhs, BgInt rhs) => new BgIntTestExpr(BgOpcode.IntLt, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator >(BgInt lhs, BgInt rhs) => new BgIntTestExpr(BgOpcode.IntGt, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator ==(BgInt lhs, BgInt rhs) => new BgIntTestExpr(BgOpcode.IntEq, lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator !=(BgInt lhs, BgInt rhs) => !(lhs == rhs);

		/// <inheritdoc/>
		public static BgBool operator <=(BgInt lhs, BgInt rhs) => !(lhs > rhs);

		/// <inheritdoc/>
		public static BgBool operator >=(BgInt lhs, BgInt rhs) => !(lhs < rhs);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override int GetHashCode() => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override BgString ToBgString()
		{
			throw new NotImplementedException();
		}
	}

	/// <summary>
	/// Traits for a <see cref="BgInt"/>
	/// </summary>
	class BgIntType : BgType<BgInt>
	{
		/// <inheritdoc/>
		public override BgInt Constant(object value) => new BgIntConstantExpr((int)value);

		/// <inheritdoc/>
		public override BgInt Wrap(BgExpr expr) => new BgIntWrappedExpr(expr);
	}

	#region Expression classes

	class BgIntUnaryExpr : BgInt
	{
		public BgOpcode Opcode { get; }
		public BgInt Value { get; }

		public BgIntUnaryExpr(BgOpcode opcode, BgInt value)
			: base(value.Flags & BgExprFlags.Eager)
		{
			Opcode = opcode;
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(Opcode);
			writer.WriteExpr(Value);
		}
	}

	class BgIntBinaryExpr : BgInt
	{
		public BgOpcode Opcode { get; }
		public BgInt Lhs { get; }
		public BgInt Rhs { get; }

		public BgIntBinaryExpr(BgOpcode opcode, BgInt lhs, BgInt rhs)
			: base(lhs.Flags & rhs.Flags & BgExprFlags.Eager)
		{
			Opcode = opcode;
			Lhs = lhs;
			Rhs = rhs;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(Opcode);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
		}
	}

	class BgIntTestExpr : BgBool
	{
		public BgOpcode Opcode { get; }
		public BgInt Lhs { get; }
		public BgInt Rhs { get; }

		public BgIntTestExpr(BgOpcode opcode, BgInt lhs, BgInt rhs)
			: base(lhs.Flags & rhs.Flags & BgExprFlags.Eager)
		{
			Opcode = opcode;
			Lhs = lhs;
			Rhs = rhs;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(Opcode);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
		}
	}

	class BgIntChooseExpr : BgInt
	{
		public BgBool Condition { get; }
		public BgInt ValueIfTrue { get; }
		public BgInt ValueIfFalse { get; }

		public BgIntChooseExpr(BgBool condition, BgInt valueIfTrue, BgInt valueIfFalse)
			: base(BgExprFlags.None)
		{
			Condition = condition;
			ValueIfTrue = valueIfTrue;
			ValueIfFalse = valueIfFalse;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Choose);
			writer.WriteExpr(Condition);
			writer.WriteExpr(ValueIfTrue);
			writer.WriteExpr(ValueIfFalse);
		}
	}

	class BgIntConstantExpr : BgInt
	{
		public int Value { get; }

		public BgIntConstantExpr(int value)
			: base(BgExprFlags.NotInterned | BgExprFlags.Eager)
		{
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.IntLiteral);
			writer.WriteSignedInteger(Value);
		}
	}

	class BgIntWrappedExpr : BgInt
	{
		BgExpr Expr { get; }

		public BgIntWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	class BgIntToBgStringExpr : BgString
	{
		BgInt Expr { get; }

		public BgIntToBgStringExpr(BgInt expr)
			: base(expr.Flags & BgExprFlags.Eager)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.IntToString);
			writer.WriteExpr(Expr);
		}
	}

	#endregion

	/// <summary>
	/// An integer option expression
	/// </summary>
	public class BgIntOption : BgInt
	{
		/// <summary>
		/// Name of the option
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Label to display next to the option
		/// </summary>
		public BgString? Label { get; }

		/// <summary>
		/// Help text to display for the user
		/// </summary>
		public BgString? Description { get; }

		/// <summary>
		/// Default value for the option
		/// </summary>
		public BgInt? DefaultValue { get; }

		/// <summary>
		/// Minimum allowed value
		/// </summary>
		public BgInt? MinValue { get; }

		/// <summary>
		/// Maximum allowed value
		/// </summary>
		public BgInt? MaxValue { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgIntOption(string name, BgString? description = null, BgInt? defaultValue = null, BgInt? minValue = null, BgInt? maxValue = null, BgString? label = null)
			: base(BgExprFlags.None)
		{
			Name = name;
			Label = label;
			Description = description;
			DefaultValue = defaultValue;
			MinValue = minValue;
			MaxValue = maxValue;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.IntOption);
			writer.WriteExpr(CreateOptionsObject());
		}

		BgObject<BgIntOptionDef> CreateOptionsObject()
		{
			BgObject<BgIntOptionDef> option = BgObject<BgIntOptionDef>.Empty;
			option = option.Set(x => x.Name, Name);
			if (Label is not null)
			{
				option = option.Set(x => x.Label, Label);
			}
			if (Description is not null)
			{
				option = option.Set(x => x.Description, Description);
			}
			if (DefaultValue is not null)
			{
				option = option.Set(x => x.DefaultValue, DefaultValue);
			}
			if (MinValue is not null)
			{
				option = option.Set(x => x.MinValue, MinValue);
			}
			if (MaxValue is not null)
			{
				option = option.Set(x => x.MaxValue, MaxValue);
			}
			return option;
		}
	}
}
