// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Specification for a label
	/// </summary>
	public class BgLabel : BgExpr
	{
		/// <summary>
		/// Name of this badge
		/// </summary>
		public BgString? DashboardName { get; }

		/// <summary>
		/// Category for this label
		/// </summary>
		public BgString? DashboardCategory { get; }

		/// <summary>
		/// Name of the badge in UGS
		/// </summary>
		public BgString? UgsBadge { get; }

		/// <summary>
		/// Path to the project folder in UGS
		/// </summary>
		public BgString? UgsProject { get; }

		/// <summary>
		/// Which change to show the badge for
		/// </summary>
		public BgEnum<BgLabelChange>? Change { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgLabel(BgString? name = null, BgString? category = null, BgString? ugsBadge = null, BgString? ugsProject = null, BgEnum<BgLabelChange>? change = null)
			: base(BgExprFlags.ForceFragment)
		{
			DashboardName = name;
			DashboardCategory = category;
			UgsBadge = ugsBadge;
			UgsProject = ugsProject;
			Change = change;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgLabelDef> obj = BgObject<BgLabelDef>.Empty;
			if (DashboardName is not null)
			{
				obj = obj.Set(x => x.DashboardName, DashboardName);
			}
			if (DashboardCategory is not null)
			{
				obj = obj.Set(x => x.DashboardCategory, DashboardCategory);
			}
			if (UgsBadge is not null)
			{
				obj = obj.Set(x => x.UgsBadge, UgsBadge);
			}
			if (UgsProject is not null)
			{
				obj = obj.Set(x => x.UgsProject, UgsProject);
			}
			if (Change is not null)
			{
				obj = obj.Set(x => x.Change, Change);
			}
			writer.WriteExpr(obj);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{Label}";
	}
}
