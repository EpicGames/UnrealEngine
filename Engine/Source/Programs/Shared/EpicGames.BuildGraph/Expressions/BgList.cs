// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Utility methods for lists
	/// </summary>
	public static class BgList
	{
		/// <summary>
		/// Gets an empty list of the given type
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <returns></returns>
		public static BgList<T> Empty<T>() where T : BgExpr => BgList<T>.Empty;

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<BgString> Create(IEnumerable<string> items) => BgList<BgString>.Create(items.Select(x => (BgString)x));

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<BgString> Create(params string[] items) => BgList<BgString>.Create(items.Select(x => (BgString)x));

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<T> Create<T>(IEnumerable<T> items) where T : BgExpr => BgList<T>.Create(items);

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<T> Create<T>(params T[] items) where T : BgExpr => BgList<T>.Create(items);

		/// <summary>
		/// Concatenates two lists together
		/// </summary>
		/// <param name="lhs"></param>
		/// <param name="rhs"></param>
		/// <returns></returns>
		public static BgList<T> Concat<T>(BgList<T> lhs, BgList<T> rhs) where T : BgExpr => BgList<T>.Concat(lhs, rhs);

		/// <summary>
		/// Concatenates two lists together
		/// </summary>
		/// <param name="others"></param>
		/// <returns></returns>
		public static BgList<T> Concat<T>(params BgList<T>[] others) where T : BgExpr => BgList<T>.Concat(others);
	}

	/// <summary>
	/// Abstract base class for expressions returning an immutable list of values
	/// </summary>
	[BgType(typeof(BgListType<>))]
	public abstract class BgList<T> : BgExpr where T : BgExpr
	{
		/// <inheritdoc/>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public Type ElementType => typeof(T);

		/// <summary>
		/// Constant representation of an empty list
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public static BgList<T> Empty { get; } = new BgListEmptyExpr<T>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags">Flags for this expression</param>
		protected BgList(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <inheritdoc/>
		public T this[BgInt index] => BgType.Wrap<T>(new BgListElementExpr<T>(this, index));

		/// <summary>
		/// Implicit conversion operator from a single value
		/// </summary>
		public static implicit operator BgList<T>(T item) => Create(item);

		/// <summary>
		/// Implicit conversion operator from an array of values
		/// </summary>
		public static implicit operator BgList<T>(T[] items) => Create(items);

		/// <summary>
		/// Implicit conversion operator from a list of values
		/// </summary>
		public static implicit operator BgList<T>(List<T> items) => Create(items);

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<T> Create(IEnumerable<T> items) => Empty.Add(items);

		/// <summary>
		/// Crates a list from an array of values
		/// </summary>
		/// <param name="items">Sequence to construct from</param>
		/// <returns></returns>
		public static BgList<T> Create(params T[] items) => Empty.Add(items);

		/// <summary>
		/// Concatenates two lists together
		/// </summary>
		/// <param name="lhs"></param>
		/// <param name="rhs"></param>
		/// <returns></returns>
		public static BgList<T> Concat(BgList<T> lhs, BgList<T> rhs) => new BgListConcatExpr<T>(lhs, rhs);

		/// <summary>
		/// Concatenates two lists together
		/// </summary>
		/// <param name="others"></param>
		/// <returns></returns>
		public static BgList<T> Concat(params BgList<T>[] others)
		{
			BgList<T> result = Empty;
			for (int idx = 0; idx < others.Length; idx++)
			{
				result = Concat(result, others[idx]);
			}
			return result;
		}

		/// <summary>
		/// Gets the length of this list
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public BgInt Count => new BgListCountExpr<T>(this);

		/// <summary>
		/// Adds an item to the end of the list, returning the new list
		/// </summary>
		/// <param name="item">Items to add</param>
		/// <returns>New list containing the given items</returns>
		public BgList<T> Add(T item) => new BgListPushExpr<T>(this, item);

		/// <summary>
		/// Adds items to the end of the list, returning the new list
		/// </summary>
		/// <param name="items">Items to add</param>
		/// <returns>New list containing the given items</returns>
		public BgList<T> Add(params T[] items) => Add((IEnumerable<T>)items);

		/// <inheritdoc cref="Add(T[])"/>
		public BgList<T> Add(BgList<T> list) => Concat(this, list);

		/// <inheritdoc cref="Add(T[])"/>
		public BgList<T> Add(IEnumerable<T> items)
		{
			BgList<T> list = this;
			foreach (T item in items)
			{
				list = list.Add(item);
			}
			return list;
		}

		/// <inheritdoc cref="Union(BgList{T})"/>
		public BgList<T> Union(params T[] items) => Union(Create(items));

		/// <inheritdoc cref="Union(BgList{T})"/>
		public BgList<T> Union(IEnumerable<T> items) => Union(Create(items));

		/// <summary>
		/// Creates the union of this list with another
		/// </summary>
		/// <param name="items">Items to add</param>
		/// <returns>Union with the given items</returns>
		public BgList<T> Union(BgList<T> items) => new BgListUnionExpr<T>(this, items);

		/// <summary>
		/// Removes the given items from this list
		/// </summary>
		/// <param name="items">Items to remove</param>
		/// <returns>New list without the given items</returns>
		public BgList<T> Except(BgList<T> items) => new BgListExceptExpr<T>(this, items);

		/// <summary>
		/// Removes any duplicate items from the list. The first item in the list is retained in its original order.
		/// </summary>
		/// <returns>New list containing the distinct items.</returns>
		public BgList<T> Distinct() => new BgListDistinctExpr<T>(this);

		/// <inheritdoc cref="Enumerable.Select{TSource, TResult}(IEnumerable{TSource}, Func{TSource, TResult})"/>
		public BgList<TResult> Select<TResult>(Func<T, TResult> function) where TResult : BgExpr => new BgListSelectExpr<T, TResult>(this, function);

		/// <inheritdoc cref="Enumerable.Select{TSource, TResult}(IEnumerable{TSource}, Func{TSource, TResult})"/>
		public BgList<TResult> Select<TResult>(BgFunc<T, TResult> function) where TResult : BgExpr => new BgListSelectExpr<T, TResult>(this, function);

		/// <inheritdoc cref="Enumerable.Where{TSource}(IEnumerable{TSource}, Func{TSource, Boolean})"/>
		public BgList<T> Where(Func<T, BgBool> predicate) => new BgListWhereExpr<T>(this, predicate);

		/// <inheritdoc cref="Enumerable.Contains{TSource}(IEnumerable{TSource}, TSource)"/>
		public BgBool Contains(T item) => new BgListContainsExpr<T>(this, item);

		/// <summary>
		/// Creates a lazily evaluated copy of this list, if it's not constant
		/// </summary>
		/// <returns></returns>
		public BgList<T> Lazy() => new BgListLazyEval<T>(this);

		/// <inheritdoc/>
		public override BgString ToBgString() => "{List}";
	}

	/// <summary>
	/// Traits for a <see cref="BgList{T}"/>
	/// </summary>
	class BgListType<T> : BgType<BgList<T>> where T : BgExpr
	{
		/// <inheritdoc/>
		public override BgList<T> Constant(object value)
		{
			IEnumerable<object> elements = (IEnumerable<object>)value;

			List<T> items = [];
			foreach (object element in elements)
			{
				items.Add(BgType.Constant<T>(element));
			}

			return new BgListConstantExpr<T>(items);
		}
		//		=> new BgListConstantExpr<T>(((IEnumerable<object>)value).Select(x => BgType.Constant<T>(x)).ToList());

		/// <inheritdoc/>
		public override BgList<T> Wrap(BgExpr expr) => new BgListWrappedExpr<T>(expr);
	}

	#region Expression classes

	class BgListEmptyExpr<T> : BgList<T> where T : BgExpr
	{
		public BgListEmptyExpr()
			: base(BgExprFlags.NotInterned)
		{
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListEmpty);
		}
	}

	class BgListConstantExpr<T> : BgList<T> where T : BgExpr
	{
		public IReadOnlyList<T> Value { get; }

		public BgListConstantExpr(IEnumerable<T> value)
			: base(BgExprFlags.NotInterned)
		{
			Value = [.. value];
		}

		public override void Write(BgBytecodeWriter writer)
		{
			throw new NotImplementedException();
		}
	}

	class BgListPushExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> List { get; }
		public T Item { get; }

		public BgListPushExpr(BgList<T> list, T item)
			: base(BgExprFlags.None)
		{
			List = list;
			Item = item;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			if ((Item.Flags & BgExprFlags.Eager) != 0)
			{
				writer.WriteOpcode(BgOpcode.ListPush);
				writer.WriteExpr(List);
				writer.WriteExpr(Item);
			}
			else
			{
				writer.WriteOpcode(BgOpcode.ListPushLazy);
				writer.WriteExpr(List);
				writer.WriteExprAsFragment(Item);
			}
		}
	}

	class BgListCountExpr<T> : BgInt where T : BgExpr
	{
		public BgList<T> List { get; }

		public BgListCountExpr(BgList<T> list)
			: base(BgExprFlags.None)
		{
			List = list;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListCount);
			writer.WriteExpr(List);
		}
	}

	class BgListConcatExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> Lhs { get; }
		public BgList<T> Rhs { get; }

		public BgListConcatExpr(BgList<T> lhs, BgList<T> rhs)
			: base(BgExprFlags.None)
		{
			Lhs = lhs;
			Rhs = rhs.Lazy();
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListConcat);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
		}
	}

	class BgListElementExpr<T> : BgExpr where T : BgExpr
	{
		public BgList<T> List { get; }
		public BgInt Index { get; }

		public BgListElementExpr(BgList<T> list, BgInt index)
			: base(BgExprFlags.None)
		{
			List = list;
			Index = index;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListElement);
			writer.WriteExpr(List);
			writer.WriteExpr(Index);
		}

		public override BgString ToBgString() => throw new InvalidOperationException();
	}

	class BgListDistinctExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> Source { get; }

		public BgListDistinctExpr(BgList<T> source)
			: base(BgExprFlags.None)
		{
			Source = source;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListDistinct);
			writer.WriteExpr(Source);
		}
	}

	class BgListSelectExpr<TIn, TOut> : BgList<TOut> where TIn : BgExpr where TOut : BgExpr
	{
		public BgList<TIn> Source { get; }
		public BgFunc<TIn, TOut> Function { get; }

		public BgListSelectExpr(BgList<TIn> source, BgFunc<TIn, TOut> function)
			: base(BgExprFlags.None)
		{
			Source = source;
			Function = function;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListSelect);
			writer.WriteExpr(Source);
			writer.WriteExprAsFragment(Function.Body);
		}
	}

	class BgListWhereExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> Source { get; }
		public BgFunc<T, BgBool> Predicate { get; }

		public BgListWhereExpr(BgList<T> source, BgFunc<T, BgBool> predicate)
			: base(BgExprFlags.None)
		{
			Source = source;
			Predicate = predicate;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListWhere);
			writer.WriteExpr(Source);
			writer.WriteExprAsFragment(Predicate.Body);
		}
	}

	class BgListUnionExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> Input { get; }
		public BgList<T> Other { get; }

		public BgListUnionExpr(BgList<T> input, BgList<T> other)
			: base(BgExprFlags.None)
		{
			Input = input;
			Other = other.Lazy();
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListUnion);
			writer.WriteExpr(Input);
			writer.WriteExpr(Other);
		}
	}

	class BgListExceptExpr<T> : BgList<T> where T : BgExpr
	{
		public BgList<T> Input { get; }
		public BgList<T> Other { get; }

		public BgListExceptExpr(BgList<T> input, BgList<T> other)
			: base(BgExprFlags.None)
		{
			Input = input;
			Other = other;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListExcept);
			writer.WriteExpr(Input);
			writer.WriteExpr(Other);
		}
	}

	class BgListContainsExpr<T> : BgBool where T : BgExpr
	{
		public BgList<T> List { get; }
		public T Item { get; }

		public BgListContainsExpr(BgList<T> list, T item)
			: base(BgExprFlags.None)
		{
			List = list;
			Item = item;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListContains);
			writer.WriteExpr(List);
			writer.WriteExpr(Item);
		}
	}

	class BgListLazyEval<T> : BgList<T> where T : BgExpr
	{
		BgList<T> List { get; }

		public BgListLazyEval(BgList<T> list)
			: base(BgExprFlags.None)
		{
			List = list;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListLazy);
			writer.WriteExprAsFragment(List);
		}
	}

	class BgListWrappedExpr<T> : BgList<T> where T : BgExpr
	{
		BgExpr Expr { get; }

		public BgListWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	#endregion

	/// <summary>
	/// A list option expression
	/// </summary>
	public class BgListOption : BgList<BgString>
	{
		/// <summary>
		/// Name of the option
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Label to display next to the option
		/// </summary>
		public BgString? Label { get; }

		/// <summary>
		/// Help text to display for the user
		/// </summary>
		public BgString? Description { get; }

		/// <summary>
		/// Style for this list box
		/// </summary>
		public BgEnum<BgListOptionStyle> Style { get; }

		/// <summary>
		/// Default value for the option
		/// </summary>
		public BgString? DefaultValue { get; set; }

		/// <summary>
		/// List of values to choose from
		/// </summary>
		public BgList<BgString>? Values { get; set; }

		/// <summary>
		/// Matching list of descriptions for each value
		/// </summary>
		public BgList<BgString>? ValueDescriptions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgListOption(string name, BgString? description = null, BgString? defaultValue = null, BgListOptionStyle style = BgListOptionStyle.CheckList, BgList<BgString>? values = null, BgList<BgString>? valueDescriptions = null, BgString? label = null)
			: base(BgExprFlags.None)
		{
			Name = name;
			Label = label;
			Description = description;
			DefaultValue = defaultValue;
			Style = style;
			Values = values;
			ValueDescriptions = valueDescriptions;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ListOption);
			writer.WriteExpr(CreateOptionsObject());
		}

		BgObject<BgListOptionDef> CreateOptionsObject()
		{
			BgObject<BgListOptionDef> option = BgObject<BgListOptionDef>.Empty;
			option = option.Set(x => x.Name, Name);
			if (Label is not null)
			{
				option = option.Set(x => x.Label, Label);
			}
			if (Description is not null)
			{
				option = option.Set(x => x.Description, Description);
			}
			if (DefaultValue is not null)
			{
				option = option.Set(x => x.DefaultValue, DefaultValue);
			}
			if (Style is not null)
			{
				option = option.Set(x => x.Style, Style);
			}
			if (Values is not null)
			{
				option = option.Set(x => x.Values, Values);
			}
			if (ValueDescriptions is not null)
			{
				option = option.Set(x => x.ValueDescriptions, ValueDescriptions);
			}
			return option;
		}
	}
}
