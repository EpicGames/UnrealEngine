// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Linq.Expressions;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Exception for constructing nodes
	/// </summary>
	public sealed class BgNodeException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		public BgNodeException(string message) : base(message)
		{
		}
	}

	/// <summary>
	/// Speecifies the node name for a method. Parameters from the method may be embedded in the name using the {ParamName} syntax.
	/// </summary>
	[AttributeUsage(AttributeTargets.Method)]
	public sealed class BgNodeNameAttribute : Attribute
	{
		/// <summary>
		/// The format string
		/// </summary>
		public string Template { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="template">Format string for the name</param>
		public BgNodeNameAttribute(string template)
		{
			Template = template;
		}
	}

	/// <summary>
	/// Specification for a node to execute
	/// </summary>
	public class BgNode : BgExpr
	{
		/// <summary>
		/// Name of the node
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Thunk to native code to execute the node
		/// </summary>
		public BgThunk Thunk { get; }

		/// <summary>
		/// Number of outputs from this node
		/// </summary>
		public int OutputCount { get; protected set; }

		/// <summary>
		/// The default output of this node. Includes all other outputs. 
		/// </summary>
		public BgFileSet DefaultOutput { get; }

		/// <summary>
		/// Agent for the node to be run on
		/// </summary>
		public BgAgent Agent { get; }

		/// <summary>
		/// Tokens for inputs of this node
		/// </summary>
		public BgList<BgFileSet> Inputs { get; private set; } = BgList.Empty<BgFileSet>();

		/// <summary>
		/// Tokens for optional inputs of this node
		/// </summary>
		public BgList<BgFileSet> OptionalInputs { get; private set; } = BgList.Empty<BgFileSet>();

		/// <summary>
		/// Weak dependency on outputs that must be generated for the node to run, without making those dependencies inputs.
		/// </summary>
		public BgList<BgNode> Fences { get; private set; } = BgList.Empty<BgNode>();

		/// <summary>
		/// Whether this node should start running as soon as its dependencies are ready, even if other nodes in the same agent are not.
		/// </summary>
		public BgBool RunEarly { get; private set; } = BgBool.False;

		/// <summary>
		/// Labels that this node contributes to
		/// </summary>
		public BgList<BgLabel> Labels { get; private set; } = BgList.Empty<BgLabel>();
		
		/// <summary>
		/// Custom annotations for this node
		/// </summary>
		public BgList<BgAnnotation> Annotations { get; private set; } = BgList.Empty<BgAnnotation>();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgNode(BgThunk thunk, BgAgent agent, BgString? name = null)
			: base(BgExprFlags.ForceFragment)
		{
			Name = name ?? GetDefaultNodeName(thunk);
			Thunk = thunk;
			Agent = agent;
			DefaultOutput = new BgFileSetFromNodeOutputExpr(this, 0);
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="node"></param>
		public BgNode(BgNode node)
			: base(BgExprFlags.ForceFragment)
		{
			Name = node.Name;
			Thunk = node.Thunk;
			Agent = node.Agent;
			DefaultOutput = new BgFileSetFromNodeOutputExpr(this, 0);
			Inputs = node.Inputs;
			OptionalInputs = node.OptionalInputs;
			Fences = node.Fences;
			RunEarly = node.RunEarly;
			Labels = node.Labels;
			Annotations = node.Annotations;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			BgObject<BgNodeExpressionDef> obj = BgObject<BgNodeExpressionDef>.Empty;
			obj = obj.Set(x => x.Name, Name);
			obj = obj.Set(x => x.Agent, Agent);
			obj = obj.Set(x => x.Thunk, Thunk);
			obj = obj.Set(x => x.OutputCount, (BgInt)OutputCount);
			obj = obj.Set(x => x.InputExprs, Inputs);
			obj = obj.Set(x => x.OptionalInputExprs, OptionalInputs);
			obj = obj.Set(x => x.OrderExprs, Fences);
			obj = obj.Set(x => x.OrderDependencies, Fences);
			obj = obj.Set(x => x.RunEarly, RunEarly);
			obj = obj.Set(x => x.Labels, Labels);
			obj = obj.Set(x => x.Annotations, Annotations);
			writer.WriteExpr(obj);
		}

		/// <summary>
		/// Creates a copy of this node and updates the given parameters
		/// </summary>
		/// <param name="inputs"></param>
		/// <param name="optionalInputs"></param>
		/// <param name="fences"></param>
		/// <param name="runEarly"></param>
		/// <param name="labels"></param>
		/// <param name="annotations"></param>
		/// <returns></returns>
		internal BgNode Modify(
			BgList<BgFileSet>? inputs = null,
			BgList<BgFileSet>? optionalInputs = null,
			BgList<BgNode>? fences = null,
			BgBool? runEarly = null,
			BgList<BgLabel>? labels = null,
			BgList<BgAnnotation>? annotations = null
		)
		{
			BgNode node = Clone();
			if (inputs is not null)
			{
				node.Inputs = inputs;
			}
			if (optionalInputs is not null)
			{
				node.OptionalInputs = optionalInputs;
			}
			if (fences is not null)
			{
				node.Fences = fences;
			}
			if (runEarly is not null)
			{
				node.RunEarly = runEarly;
			}
			if (labels is not null)
			{
				node.Labels = labels;
			}
			if (annotations is not null)
			{
				node.Annotations = annotations;
			}
			return node;
		}

		/// <summary>
		/// Clone this node
		/// </summary>
		/// <returns>Clone of this node</returns>
		protected virtual BgNode Clone() => new(this);

		/// <summary>
		/// Gets the default tag name for the numbered output index
		/// </summary>
		/// <param name="name">Name of the node</param>
		/// <param name="index">Index of the output. Index zero is the default, others are explicit.</param>
		/// <returns></returns>
		internal static string GetDefaultTagName(string name, int index)
		{
			return $"#{name}${index}";
		}

		static BgString GetDefaultNodeName(BgThunk thunk)
		{
			// Check if it's got an attribute override for the node name
			BgNodeNameAttribute? nameAttr = thunk.Method.GetCustomAttribute<BgNodeNameAttribute>();
			if (nameAttr != null)
			{
				return GetNodeNameFromTemplate(nameAttr.Template, thunk.Method.GetParameters(), thunk.Arguments);
			}
			else
			{
				return GetNodeNameFromMethodName(thunk.Method.Name);
			}
		}

		static BgString GetNodeNameFromTemplate(string template, ParameterInfo[] parameters, IReadOnlyList<object?> arguments)
		{
			// Create a list of lazily computed string fragments which comprise the evaluated name
			List<BgString> fragments = [];

			int lastIdx = 0;
			for (int nextIdx = 0; nextIdx < template.Length; nextIdx++)
			{
				if (template[nextIdx] == '{')
				{
					if (nextIdx + 1 < template.Length && template[nextIdx + 1] == '{')
					{
						fragments.Add(template.Substring(lastIdx, nextIdx - lastIdx));
						lastIdx = ++nextIdx;
					}
					else
					{
						fragments.Add(template.Substring(lastIdx, nextIdx - lastIdx));
						nextIdx++;

						int endIdx = template.IndexOf('}', nextIdx);
						if (endIdx == -1)
						{
							throw new BgNodeException($"Unterminated parameter expression for {nameof(BgNodeNameAttribute)} in {template}");
						}

						StringView paramName = new(template, nextIdx, endIdx - nextIdx);

						int paramIdx = Array.FindIndex(parameters, x => x.Name != null && paramName.Equals(x.Name, StringComparison.Ordinal));
						if (paramIdx == -1)
						{
							throw new BgNodeException($"Unable to find parameter named {paramName} in {template}");
						}

						object? arg = arguments[paramIdx];
						if (typeof(BgExpr).IsAssignableFrom(parameters[paramIdx].ParameterType))
						{
							fragments.Add(((BgExpr)arg!).ToBgString());
						}
						else if (arg != null)
						{
							fragments.Add(arg.ToString() ?? String.Empty);
						}

						lastIdx = nextIdx = endIdx + 1;
					}
				}
				else if (template[nextIdx] == '}')
				{
					if (nextIdx + 1 < template.Length && template[nextIdx + 1] == '{')
					{
						fragments.Add(template.Substring(lastIdx, nextIdx - lastIdx));
						lastIdx = ++nextIdx;
					}
				}
			}
			fragments.Add(template.Substring(lastIdx, template.Length - lastIdx));

			if (fragments.Count == 1)
			{
				return fragments[0];
			}
			else
			{
				return BgString.Join(BgString.Empty, fragments);
			}
		}

		/// <summary>
		/// Inserts spaces into a PascalCase method name to create a node name
		/// </summary>
		public static string GetNodeNameFromMethodName(string methodName)
		{
			StringBuilder name = new();
			name.Append(methodName[0]);

			int length = methodName.Length;
			if (length > 5 && methodName.EndsWith("Async", StringComparison.Ordinal))
			{
				length -= 5;
			}

			bool bIsAcronym = false;
			for (int idx = 1; idx < length; idx++)
			{
				bool bLastIsUpper = Char.IsUpper(methodName[idx - 1]);
				bool bNextIsUpper = Char.IsUpper(methodName[idx]);
				if (bLastIsUpper && bNextIsUpper)
				{
					bIsAcronym = true;
				}
				else if (bIsAcronym)
				{
					name.Insert(name.Length - 2, ' ');
					bIsAcronym = false;
				}
				else if (!bLastIsUpper && bNextIsUpper)
				{
					name.Append(' ');
				}
				name.Append(methodName[idx]);
			}

			return name.ToString();
		}

		/// <summary>
		/// Implicit conversion to a fileset
		/// </summary>
		/// <param name="node"></param>
		public static implicit operator BgFileSet(BgNode node)
		{
			return new BgFileSetFromNodeExpr(node);
		}

		/// <summary>
		/// Implicit conversion to a fileset
		/// </summary>
		/// <param name="node"></param>
		public static implicit operator BgList<BgFileSet>(BgNode node)
		{
			return (BgFileSet)node;
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => Name ?? BgString.Empty;
	}

	/// <summary>
	/// Nodespec with a typed return value
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class BgNode<T> : BgNode
	{
		/// <summary>
		/// Output from this node
		/// </summary>
		public T Output { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgNode(BgThunk<T> thunk, BgAgent agent, BgString? name = null)
			: base(thunk, agent, name)
		{
			Output = CreateOutput();
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		public BgNode(BgNode<T> other)
			: base(other)
		{
			Output = CreateOutput();
		}

		/// <summary>
		/// Clone this node
		/// </summary>
		/// <returns>Clone of this node</returns>
		protected override BgNode Clone() => new BgNode<T>(this);

		T CreateOutput()
		{
			Type type = typeof(T);
			if (IsValueTuple(type))
			{
				BgExpr[] outputs = CreateOutputExprs(type.GetGenericArguments());
				OutputCount = outputs.Length;
				return (T)Activator.CreateInstance(type, outputs)!;
			}
			else
			{
				BgExpr[] outputs = CreateOutputExprs([type]);
				OutputCount = outputs.Length;
				return (T)(object)outputs[0];
			}
		}

		BgExpr[] CreateOutputExprs(Type[] types)
		{
			BgExpr[] outputs = new BgExpr[types.Length];
			for (int idx = 0; idx < types.Length; idx++)
			{
				outputs[idx] = CreateOutputExpr(types[idx], idx);
			}
			return outputs;
		}

		BgExpr CreateOutputExpr(Type type, int index)
		{
			if (type == typeof(BgFileSet))
			{
				// When BgNodeDef is loaded it automatically inserts another
				// output into index 0 that represents all outputs. The actual
				// outputs from the node are inserted after this automatically
				// generated output. That means that what is index 0 here will
				// map to the automatically generated output. To compensate for
				// this we need to increment all output indexes by 1 to match
				// the index they will actually be once BgNodeDef loads them.
				return new BgFileSetFromNodeOutputExpr(this, index + 1);
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		internal static bool IsValueTuple(Type returnType)
		{
			if (returnType.IsGenericType)
			{
				Type genericType = returnType.GetGenericTypeDefinition();
				if (genericType.FullName != null && genericType.FullName.StartsWith("System.ValueTuple`", StringComparison.Ordinal))
				{
					return true;
				}
			}
			return false;
		}
	}

	/// <summary>
	/// Extension methods for BgNode types
	/// </summary>
	public static class BgNodeExtensions
	{
		/// <summary>
		/// Creates a node builder for the given agent
		/// </summary>
		/// <param name="agent">Agent to run the node</param>
		/// <param name="func">Function to execute</param>
		/// <returns>Node builder</returns>
		public static BgNode AddNode(this BgAgent agent, Expression<Func<BgContext, Task>> func)
		{
			BgThunk thunk = BgThunk.Create(func);
			return new BgNode(thunk, agent);
		}
		

		/// <summary>
		/// Creates a node builder for the given agent
		/// </summary>
		/// <param name="agent">Agent to run the node</param>
		/// <param name="name"></param>
		/// <param name="func">Function to execute</param>
		/// <returns>Node builder</returns>
		public static BgNode AddNode(this BgAgent agent, BgString name, Expression<Func<BgContext, Task>> func)
		{
			BgThunk thunk = BgThunk.Create(func);
			return new BgNode(thunk, agent, name);
		}

		/// <summary>
		/// Creates a node builder for the given agent
		/// </summary>
		/// <param name="agent">Agent to run the node</param>
		/// <param name="func">Function to execute</param>
		/// <returns>Node builder</returns>
		public static BgNode<T> AddNode<T>(this BgAgent agent, Expression<Func<BgContext, Task<T>>> func)
		{
			BgThunk<T> thunk = BgThunk.Create(func);
			return new BgNode<T>(thunk, agent);
		}
		
		/// <summary>
		/// Creates a node builder for the given agent
		/// </summary>
		/// <param name="agent">Agent to run the node</param>
		/// <param name="name"></param>
		/// <param name="func">Function to execute</param>
		/// <returns>Node builder</returns>
		public static BgNode<T> AddNode<T>(this BgAgent agent, BgString name, Expression<Func<BgContext, Task<T>>> func)
		{
			BgThunk<T> thunk = BgThunk.Create(func);
			return new BgNode<T>(thunk, agent, name);
		}

		/// <summary>
		/// Add dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T Requires<T>(this T node, params BgNode[] inputs) where T : BgNode
		{
			return (T)node.Modify(inputs: node.Inputs.Add(inputs.Select(x => (BgFileSet)x)));
		}

		/// <summary>
		/// Add optional dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as optional inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T OptionalRequires<T>(this T node, params BgNode[] inputs) where T : BgNode
		{
			return (T)node.Modify(optionalInputs: node.OptionalInputs.Add(inputs.Select(x => (BgFileSet)x)));
		}

		/// <summary>
		/// Add dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T Requires<T>(this T node, params BgFileSet[] inputs) where T : BgNode
		{
			return (T)node.Modify(inputs: node.Inputs.Add(inputs));
		}

		/// <summary>
		/// Add optional dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as optional inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T OptionalRequires<T>(this T node, params BgFileSet[] inputs) where T : BgNode
		{
			return (T)node.Modify(optionalInputs: node.OptionalInputs.Add(inputs));
		}

		/// <summary>
		/// Add dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T Requires<T>(this T node, BgList<BgFileSet> inputs) where T : BgNode
		{
			return (T)node.Modify(inputs: node.Inputs.Add(inputs));
		}

		/// <summary>
		/// Add optional dependencies onto other nodes or outputs. Outputs from the given tokens will be copied to the current machine before execution of the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as optional inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T OptionalRequires<T>(this T node, BgList<BgFileSet> inputs) where T : BgNode
		{
			return (T)node.Modify(optionalInputs: node.OptionalInputs.Add(inputs));
		}

		/// <summary>
		/// Add weak dependencies onto other nodes or outputs. The producing nodes must complete successfully if they are part of the graph, but outputs from them will not be 
		/// transferred to the machine running this node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="inputs">Files to add as inputs</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T After<T>(this T node, params BgNode[] inputs) where T : BgNode
		{
			return (T)node.Modify(fences: node.Fences.Add(inputs));
		}

		/// <summary>
		/// Add a label to the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="label">The label to add to the node.</param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T AddLabel<T>(this T node, BgLabel label) where T : BgNode
		{
			return (T)node.Modify(labels: node.Labels.Add(label));
		}
		
		/// <summary>
		/// Allows the node to run early.
		/// </summary>
		public static T RunEarly<T>(this T node) where T : BgNode
		{
			return (T)node.Modify(runEarly: true);
		}
		
		/// <summary>
		/// Add an annotation to the node.
		/// </summary>
		/// <param name="node">The node to modify</param>
		/// <param name="annotation"></param>
		/// <returns>The current node spec, to allow chaining calls</returns>
		public static T AddAnnotation<T>(this T node, BgAnnotation annotation) where T : BgNode
		{
			return (T)node.Modify(annotations: node.Annotations.Add(annotation));
		}
	}
}
