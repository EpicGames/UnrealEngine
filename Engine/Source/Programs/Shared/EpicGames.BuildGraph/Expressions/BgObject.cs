// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Immutable;
using System.Linq.Expressions;
using System.Reflection;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for expressions returning an object
	/// </summary>
	/// <typeparam name="T">The native type which mirrors this object</typeparam>
	[BgType(typeof(BgObjectType<>))]
	public abstract class BgObject<T> : BgExpr
	{
		/// <summary>
		/// Constant representation of an empty object
		/// </summary>
		public static BgObject<T> Empty { get; } = new BgObjectEmptyExpr<T>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags">Flags for this expression</param>
		protected BgObject(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <summary>
		/// Sets the value of a property in the object
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		/// <returns>New object with the field set</returns>
		public BgObject<T> Set(string name, BgExpr value) => new BgObjectSetExpr<T>(this, name, value);

		/// <summary>
		/// Sets the value of a property in the object
		/// </summary>
		/// <param name="property">Name of the field</param>
		/// <param name="value">Value for the field</param>
		/// <returns>New object with the field set</returns>
		public BgObject<T> Set<TExpr, TNative>(Expression<Func<T, TNative>> property, TExpr value) where TExpr : BgExpr
		{
			MemberExpression member = ((MemberExpression)property.Body);
			PropertyInfo propertyInfo = (PropertyInfo)member.Member;
			string name = propertyInfo.GetCustomAttribute<BgPropertyAttribute>()?.Name ?? propertyInfo.Name;
			return Set(name, value);
		}

		/// <summary>
		/// Gets the value of a field in the object
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="defaultValue">Default value for the field, if it's not defined</param>
		/// <returns>Value of the field</returns>
		public TExpr Get<TExpr>(string name, TExpr defaultValue) where TExpr : BgExpr => BgType.Wrap<TExpr>(new BgObjectGetExpr<T>(this, name, defaultValue));

		/// <summary>
		/// Gets the value of a field in the object
		/// </summary>
		/// <param name="property">Expression indicating the property to retrieve</param>
		/// <param name="defaultValue">Default value for the property</param>
		/// <returns>Value of the field</returns>
		public TExpr Get<TExpr, TNative>(Expression<Func<T, TNative>> property, TExpr defaultValue) where TExpr : BgExpr
		{
			MemberExpression member = ((MemberExpression)property.Body);
			PropertyInfo propertyInfo = (PropertyInfo)member.Member;
			string name = propertyInfo.GetCustomAttribute<BgPropertyAttribute>()?.Name ?? propertyInfo.Name;
			return Get(name, defaultValue);
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{Object}";
	}

	/// <summary>
	/// Traits for a <see cref="BgList{T}"/>
	/// </summary>
	class BgObjectType<T> : BgType<BgObject<T>> where T : new()
	{
		/// <inheritdoc/>
		public override BgObject<T> Constant(object value) => new BgObjectConstantExpr<T>((ImmutableDictionary<string, object>)value);

		/// <inheritdoc/>
		public override BgObject<T> Wrap(BgExpr expr) => new BgObjectWrappedExpr<T>(expr);
	}

	#region Expression classes

	class BgObjectEmptyExpr<T> : BgObject<T>
	{
		public BgObjectEmptyExpr()
			: base(BgExprFlags.NotInterned)
		{
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ObjEmpty);
		}
	}

	class BgObjectConstantExpr<T> : BgObject<T>
	{
		public ImmutableDictionary<string, object> Fields { get; }

		public BgObjectConstantExpr(ImmutableDictionary<string, object> fields)
			: base(BgExprFlags.NotInterned)
		{
			Fields = fields;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			throw new NotImplementedException();
		}
	}

	class BgObjectSetExpr<T> : BgObject<T>
	{
		public BgObject<T> Object { get; }
		public string Name { get; }
		public BgExpr Value { get; }

		public BgObjectSetExpr(BgObject<T> obj, string name, BgExpr value)
			: base(BgExprFlags.NotInterned)
		{
			Object = obj;
			Name = name;
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ObjSet);
			writer.WriteExpr(Object);
			writer.WriteName(Name);
			writer.WriteExpr(Value);
		}
	}

	class BgObjectGetExpr<T> : BgObject<T>
	{
		public BgObject<T> Object { get; }
		public string Name { get; }
		public BgExpr DefaultValue { get; }

		public BgObjectGetExpr(BgObject<T> obj, string name, BgExpr defaultValue)
			: base(BgExprFlags.NotInterned)
		{
			Object = obj;
			Name = name;
			DefaultValue = defaultValue;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.ObjGet);
			writer.WriteExpr(Object);
			writer.WriteName(Name);
			writer.WriteExpr(DefaultValue);
		}
	}

	class BgObjectWrappedExpr<T> : BgObject<T>
	{
		BgExpr Expr { get; }

		public BgObjectWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	#endregion
}
