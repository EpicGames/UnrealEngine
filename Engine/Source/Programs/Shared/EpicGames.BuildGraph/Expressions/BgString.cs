// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Abstract base class for expressions returning a string value 
	/// </summary>
	[BgType(typeof(BgStringType))]
	public abstract class BgString : BgExpr
	{
		/// <summary>
		/// Constant value for an empty string
		/// </summary>
		public static BgString Empty { get; } = new BgStringEmptyExpr();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags">Flags for this expression</param>
		protected BgString(BgExprFlags flags)
			: base(flags)
		{
		}

		/// <summary>
		/// Implicit conversion from a regular string type
		/// </summary>
		public static implicit operator BgString(string value)
		{
			return new BgStringConstantExpr(value);
		}

		/// <inheritdoc cref="String.Equals(String?, String?, StringComparison)"/>
		public static BgBool Equals(BgString lhs, BgString rhs, StringComparison comparison = StringComparison.CurrentCulture) => Compare(lhs, rhs, comparison) == 0;

		/// <inheritdoc cref="String.Compare(String?, String?, StringComparison)"/>
		public static BgInt Compare(BgString lhs, BgString rhs, StringComparison comparison = StringComparison.CurrentCulture) => new BgStringCompareExpr(lhs, rhs, comparison);

		/// <inheritdoc cref="String.Join{T}(String?, IEnumerable{T})"/>
		public static BgString Join(BgString separator, BgList<BgString> values) => new BgStringJoinExpr(separator, values);

		/// <inheritdoc cref="String.Split(String?, StringSplitOptions)"/>
		public BgList<BgString> Split(BgString separator) => new BgStringSplitExpr(this, separator);

		/// <inheritdoc cref="String.Format(String, Object?[])"/>
		public static BgString Format(string format, params BgExpr[] args) => new BgStringFormatExpr(format, args);

		/// <inheritdoc/>
		public static BgString operator +(BgString lhs, BgString rhs) => new BgStringConcatExpr(lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator ==(BgString lhs, BgString rhs) => Equals(lhs, rhs);

		/// <inheritdoc/>
		public static BgBool operator !=(BgString lhs, BgString rhs) => !Equals(lhs, rhs);

		/// <summary>
		/// Appens another string to this one
		/// </summary>
		public BgString Append(BgString other) => this + other;

		/// <inheritdoc/>
		public BgBool Match(BgString pattern) => new BgStringMatchExpr(this, pattern);

		/// <inheritdoc/>
		public BgString Replace(BgString pattern, BgString replace) => new BgStringReplaceExpr(this, pattern, replace);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override int GetHashCode() => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override BgString ToBgString() => this;
	}

	/// <summary>
	/// Traits implementation for <see cref="BgString"/>
	/// </summary>
	class BgStringType : BgType<BgString>
	{
		/// <inheritdoc/>
		public override BgString Constant(object value) => new BgStringConstantExpr((string)value);

		/// <inheritdoc/>
		public override BgString Wrap(BgExpr expr) => new BgStringWrappedExpr(expr);
	}

	#region Expression classes

	class BgStringCompareExpr : BgInt
	{
		public BgString Lhs { get; }
		public BgString Rhs { get; }
		public StringComparison Comparison { get; }

		public BgStringCompareExpr(BgString lhs, BgString rhs, StringComparison comparison)
			: base(lhs.Flags & rhs.Flags & BgExprFlags.Eager)
		{
			Lhs = lhs;
			Rhs = rhs;
			Comparison = comparison;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrCompare);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
			writer.WriteUnsignedInteger((uint)Comparison);
		}

		public override string ToString() => $"Compare({Lhs}, {Rhs})";
	}

	class BgStringConcatExpr : BgString
	{
		public BgString Lhs { get; }
		public BgString Rhs { get; }

		public BgStringConcatExpr(BgString lhs, BgString rhs)
			: base(lhs.Flags & rhs.Flags & BgExprFlags.Eager)
		{
			Lhs = lhs;
			Rhs = rhs;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrConcat);
			writer.WriteExpr(Lhs);
			writer.WriteExpr(Rhs);
		}

		public override string ToString() => $"Concat({Lhs}, {Rhs})";
	}

	class BgStringJoinExpr : BgString
	{
		public BgString Separator { get; }
		public BgList<BgString> Values { get; }

		public BgStringJoinExpr(BgString separator, BgList<BgString> values)
			: base(BgExprFlags.None)
		{
			Separator = separator;
			Values = values;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrJoin);
			writer.WriteExpr(Separator);
			writer.WriteExpr(Values);
		}

		public override string ToString() => $"Join({Separator}, {Values})";
	}

	class BgStringSplitExpr : BgList<BgString>
	{
		public BgString Source { get; }
		public BgString Separator { get; }

		public BgStringSplitExpr(BgString source, BgString separator)
			: base(source.Flags & separator.Flags & BgExprFlags.Eager)
		{
			Source = source;
			Separator = separator;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrSplit);
			writer.WriteExpr(Source);
			writer.WriteExpr(Separator);
		}

		public override string ToString() => $"Split({Source}, {Separator})";
	}

	class BgStringMatchExpr : BgBool
	{
		public BgString Input { get; }
		public BgString Pattern { get; }

		public BgStringMatchExpr(BgString input, BgString pattern)
			: base(input.Flags & BgExprFlags.Eager)
		{
			Input = input;
			Pattern = pattern;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrMatch);
			writer.WriteExpr(Input);
			writer.WriteExpr(Pattern);
		}
	}

	class BgStringReplaceExpr : BgString
	{
		public BgString Input { get; }
		public BgString Pattern { get; }
		public BgString Replacement { get; }

		public BgStringReplaceExpr(BgString input, BgString pattern, BgString replacement)
			: base(input.Flags & replacement.Flags & BgExprFlags.Eager)
		{
			Input = input;
			Pattern = pattern;
			Replacement = replacement;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrReplace);
			writer.WriteExpr(Input);
			writer.WriteExpr(Pattern);
			writer.WriteExpr(Replacement);
		}
	}

	class BgStringFormatExpr : BgString
	{
		readonly BgString _format;
		readonly BgExpr[] _arguments;

		public BgStringFormatExpr(BgString format, BgExpr[] arguments)
			: base(BgExprFlags.None)
		{
			_format = format;
			_arguments = arguments;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrFormat);
			writer.WriteExpr(_format);
			writer.WriteUnsignedInteger((ulong)_arguments.Length);
			foreach (BgExpr argument in _arguments)
			{
				writer.WriteExpr(argument);
			}
		}
	}

	class BgStringEmptyExpr : BgString
	{
		public BgStringEmptyExpr()
			: base(BgExprFlags.NotInterned | BgExprFlags.Eager)
		{
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrEmpty);
		}
	}

	class BgStringConstantExpr : BgString
	{
		public string Value { get; }

		public BgStringConstantExpr(string value)
			: base(BgExprFlags.Eager)
		{
			Value = value;
		}

		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrLiteral);
			writer.WriteString(Value);
		}
	}

	class BgStringWrappedExpr : BgString
	{
		BgExpr Expr { get; }

		public BgStringWrappedExpr(BgExpr expr)
			: base(expr.Flags)
		{
			Expr = expr;
		}

		public override void Write(BgBytecodeWriter writer) => Expr.Write(writer);
	}

	#endregion

	/// <summary>
	/// Style for a string option
	/// </summary>
	public enum BgStringOptionStyle
	{
		/// <summary>
		/// Free-form text entry
		/// </summary>
		Text,

		/// <summary>
		/// List of options
		/// </summary>
		DropList,
	}

	/// <summary>
	/// A string option expression
	/// </summary>
	public class BgStringOption : BgString
	{
		/// <summary>
		/// Name of the option
		/// </summary>
		public BgString Name { get; }

		/// <summary>
		/// Label to display next to the option
		/// </summary>
		public BgString? Label { get; }

		/// <summary>
		/// Help text to display for the user
		/// </summary>
		public BgString? Description { get; }

		/// <summary>
		/// Default value for the option
		/// </summary>
		public BgString? DefaultValue { get; set; }

		/// <summary>
		/// Style for this option
		/// </summary>
		public BgEnum<BgStringOptionStyle>? Style { get; }

		/// <summary>
		/// Regex for validating values for the option
		/// </summary>
		public BgString? Pattern { get; set; }

		/// <summary>
		/// Message to display if validation fails
		/// </summary>
		public BgString? PatternFailed { get; set; }

		/// <summary>
		/// List of values to choose from
		/// </summary>
		public BgList<BgString>? Values { get; set; }

		/// <summary>
		/// Matching list of descriptions for each value
		/// </summary>
		public BgList<BgString>? ValueDescriptions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgStringOption(string name, BgString? description = null, BgString? defaultValue = null, BgEnum<BgStringOptionStyle>? style = null, BgString? pattern = null, BgString? patternFailed = null, BgList<BgString>? values = null, BgList<BgString>? valueDescriptions = null, BgString? label = null)
			: base(BgExprFlags.None)
		{
			Name = name;
			Label = label;
			Description = description;
			Style = style;
			DefaultValue = defaultValue;
			Pattern = pattern;
			PatternFailed = patternFailed;
			Values = values;
			ValueDescriptions = valueDescriptions;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.StrOption);
			writer.WriteExpr(CreateOptionsObject());
		}

		BgObject<BgStringOptionDef> CreateOptionsObject()
		{
			BgObject<BgStringOptionDef> option = BgObject<BgStringOptionDef>.Empty;
			option = option.Set(x => x.Name, Name);
			if (Label is not null)
			{
				option = option.Set(x => x.Label, Label);
			}
			if (Description is not null)
			{
				option = option.Set(x => x.Description, Description);
			}
			if (DefaultValue is not null)
			{
				option = option.Set(x => x.DefaultValue, DefaultValue);
			}
			if (Style is not null)
			{
				option = option.Set(x => x.Style, Style);
			}
			if (Pattern is not null)
			{
				option = option.Set(x => x.Pattern, Pattern);
			}
			if (PatternFailed is not null)
			{
				option = option.Set(x => x.PatternFailed, PatternFailed);
			}
			if (Values is not null)
			{
				option = option.Set(x => x.Values, Values);
			}
			if (ValueDescriptions is not null)
			{
				option = option.Set(x => x.ValueDescriptions, ValueDescriptions);
			}
			return option;
		}
	}
}
