// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq.Expressions;
using System.Reflection;
using System.Threading.Tasks;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Thunks to a native method from within an expression. These objects contain an index into a thunk table that is not persisted to the bytecode.
	/// </summary>
	public class BgThunk : BgExpr
	{
		/// <summary>
		/// Method to call
		/// </summary>
		public MethodInfo Method { get; }

		/// <summary>
		/// Arguments for the method
		/// </summary>
		public IReadOnlyList<object?> Arguments { get; }

		/// <summary>
		/// Creates a method closure from the given expression
		/// </summary>
		/// <param name="expr">Method call expression</param>
		/// <returns></returns>
		public static BgThunk Create(Expression<Func<Task>> expr)
		{
			MethodCallExpression call = (MethodCallExpression)expr.Body;
			return new BgThunk(call);
		}

		/// <summary>
		/// Creates a method closure from the given expression
		/// </summary>
		/// <typeparam name="TRet">Type of the return value</typeparam>
		/// <param name="expr">Method call expression</param>
		/// <returns></returns>
		public static BgThunk<TRet> Create<TRet>(Expression<Func<Task<TRet>>> expr)
		{
			MethodCallExpression call = (MethodCallExpression)expr.Body;
			return new BgThunk<TRet>(call);
		}

		/// <summary>
		/// Creates a method closure from the given expression
		/// </summary>
		/// <typeparam name="TArg">Parameter to the method</typeparam>
		/// <param name="expr">Method call expression</param>
		/// <returns></returns>
		public static BgThunk Create<TArg>(Expression<Func<TArg, Task>> expr)
		{
			MethodCallExpression call = (MethodCallExpression)expr.Body;
			return new BgThunk(call);
		}

		/// <summary>
		/// Creates a method closure from the given expression
		/// </summary>
		/// <typeparam name="TArg">Parameter to the method</typeparam>
		/// <typeparam name="TRet">Type of the return value</typeparam>
		/// <param name="expr">Method call expression</param>
		/// <returns></returns>
		public static BgThunk<TRet> Create<TArg, TRet>(Expression<Func<TArg, Task<TRet>>> expr)
		{
			MethodCallExpression call = (MethodCallExpression)expr.Body;
			return new BgThunk<TRet>(call);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="call">Call expression</param>
		public BgThunk(MethodCallExpression call)
			: base(BgExprFlags.None)
		{
			Method = call.Method;
			Arguments = GetArguments(call);
		}

		static object?[] GetArguments(MethodCallExpression call)
		{
			object?[] args = new object?[call.Arguments.Count];
			for (int idx = 0; idx < call.Arguments.Count; idx++)
			{
				Expression argumentExpr = call.Arguments[idx];
				if (argumentExpr is ParameterExpression parameterExpr)
				{
					if (parameterExpr.Type != typeof(BgContext))
					{
						throw new BgNodeException($"Unable to determine type of parameter '{parameterExpr.Name}'");
					}
				}
				else
				{
					Delegate compiled = Expression.Lambda(argumentExpr).Compile();
					args[idx] = compiled.DynamicInvoke();
				}
			}
			return args;
		}

		/// <inheritdoc/>
		public override void Write(BgBytecodeWriter writer)
		{
			writer.WriteOpcode(BgOpcode.Thunk);
			writer.WriteThunk(new BgThunkDef(Method, Arguments));

			ParameterInfo[] parameters = Method.GetParameters();
			writer.WriteUnsignedInteger(parameters.Length);

			for (int idx = 0; idx < parameters.Length; idx++)
			{
				if (typeof(BgExpr).IsAssignableFrom(parameters[idx].ParameterType))
				{
					writer.WriteExpr((BgExpr)Arguments[idx]!);
				}
				else
				{
					writer.WriteOpcode(BgOpcode.Null);
				}
			}
		}

		/// <inheritdoc/>
		public override BgString ToBgString() => "{Method}";
	}

	/// <summary>
	/// Wraps a native method that returns a value
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class BgThunk<T> : BgThunk
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="expr"></param>
		public BgThunk(MethodCallExpression expr)
			: base(expr)
		{
		}
	}
}
