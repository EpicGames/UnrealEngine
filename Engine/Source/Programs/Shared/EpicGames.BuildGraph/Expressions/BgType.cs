// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Reflection;

namespace EpicGames.BuildGraph.Expressions
{
	/// <summary>
	/// Interface for type traits
	/// </summary>
	public interface IBgType
	{
		/// <summary>
		/// Creates a constant expression wrapping the given value
		/// </summary>
		/// <param name="value">Value to wrap</param>
		/// <returns></returns>
		public abstract BgExpr Constant(object value);
	}

	/// <summary>
	/// Base class for type functions
	/// </summary>
	public abstract class BgType<T> : IBgType where T : BgExpr
	{
		/// <inheritdoc/>
		BgExpr IBgType.Constant(object value) => Constant(value);

		/// <inheritdoc cref="IBgType.Constant(Object)"/>
		public abstract T Constant(object value);

		/// <summary>
		/// Wraps an untyped expression as a strongly typed value
		/// </summary>
		/// <param name="expr">Expression to wrap</param>
		/// <returns></returns>
		public abstract T Wrap(BgExpr expr);
	}

	/// <summary>
	/// Attribute used to specify the converter class to use for a type
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	sealed class BgTypeAttribute : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type Type { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type"></param>
		public BgTypeAttribute(Type type)
		{
			Type = type;
		}
	}

	/// <summary>
	/// Utility methods for type traits
	/// </summary>
	public abstract class BgType
	{
		class BgTypeCache<T> where T : BgExpr
		{
			public static BgType<T> Instance { get; } = (BgType<T>)CreateInstance(typeof(T));
		}

		static readonly ConcurrentDictionary<Type, IBgType> s_typeCache = new();

		/// <summary>
		/// Create a converter instance for the given type
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		static object CreateInstance(Type type)
		{
			BgTypeAttribute converterAttr = type.GetCustomAttribute<BgTypeAttribute>(true) ?? throw new InvalidOperationException($"Missing [BgType] attribute on {type.Name}");

			Type converterType = converterAttr.Type;
			if (converterType.IsGenericType)
			{
				converterType = converterType.MakeGenericType(type.GetGenericArguments());
			}

			return Activator.CreateInstance(converterType)!;
		}

		/// <summary>
		/// Gets the type interface for a type
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		public static IBgType Get(Type type)
		{
			return s_typeCache.GetOrAdd(type, CreateTypeInstance);
		}

		static IBgType CreateTypeInstance(Type type)
		{
			Type concreteType = typeof(BgTypeCache<>).MakeGenericType(type);
			return (IBgType)concreteType.GetProperty(nameof(BgTypeCache<BgInt>.Instance))!.GetValue(null)!;
		}

		/// <summary>
		/// Create a traits instance for the given type
		/// </summary>
		/// <returns></returns>
		public static BgType<T> Get<T>() where T : BgExpr
		{
			return BgTypeCache<T>.Instance;
		}

		/// <summary>
		/// Wraps an expression as a different type
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="expr"></param>
		/// <returns></returns>
		public static T Wrap<T>(BgExpr expr) where T : BgExpr
		{
			return Get<T>().Wrap(expr);
		}

		/// <summary>
		/// Creates a constant of the given type
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static T Constant<T>(object value) where T : BgExpr
		{
			return Get<T>().Constant(value);
		}

		/// <summary>
		/// Creates a constant of the given type
		/// </summary>
		/// <param name="type"></param>
		/// <param name="value"></param>
		/// <returns></returns>
		public static BgExpr Constant(Type type, object value)
		{
			return Get(type).Constant(value);
		}
	}
}
