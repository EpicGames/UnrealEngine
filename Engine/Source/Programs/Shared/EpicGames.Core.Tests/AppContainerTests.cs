// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Runtime.InteropServices;
using System.Security.AccessControl;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

// Suppress call sites not available on all platforms
// OS check is done in test class initializer below
#pragma warning disable CA1416

[TestClass]
public class AppContainerTests
{
	private const string FileContent = "foobar";
	private const string RegeditExePath = @"c:\Windows\regedit.exe";
	private const string DisplayName = "Unreal Engine Horde Test";
	private const string Description = "Created by Unreal Engine tests";

	private static int s_idCounter = 1000;
	private static readonly HashSet<string> s_allocatedNames = [];
	private readonly DirectoryInfo _tempDir;
	private readonly FileInfo _tempFile;

	public AppContainerTests()
	{
		string uniqueTempDir = Path.Join(Path.GetTempPath(), Path.GetRandomFileName());
		string uniqueTempFile = Path.Join(uniqueTempDir, Path.GetRandomFileName());
		_tempDir = Directory.CreateDirectory(uniqueTempDir);
		_tempFile = new FileInfo(uniqueTempFile);
		File.WriteAllText(uniqueTempFile, FileContent);
	}

	[ClassInitialize]
	public static void ClassInit(TestContext _)
	{
		if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
		{
			Assert.Inconclusive("AppContainer tests only run on Windows");
		}
	}

	[TestCleanup]
	public void Cleanup()
	{
		foreach (string containerName in s_allocatedNames)
		{
			AppContainer.Delete(containerName);
		}

		_tempDir.Delete(true);
	}

	[TestMethod]
	public void CreateOrUseExisting()
	{
		string containerName = GetName(20);
		AppContainer.Delete(containerName);
		using AppContainer ac = AppContainer.Create(containerName, DisplayName, Description);
		using AppContainer sameAc = AppContainer.Create(containerName, DisplayName, Description);

		Assert.ThrowsExactly<AppContainerException>(() =>
		{
			using AppContainer errored = AppContainer.Create(containerName, DisplayName, Description, true);
		});
		Assert.AreEqual(ac.ContainerName, sameAc.ContainerName);
	}

	[TestMethod]
	public async Task FileSystem_CanRead_AllowedDir_Async()
	{
		using AppContainer appContainer = Create();
		Assert.IsTrue(await IsFileReadableAsync(null, _tempFile.FullName));
		Assert.IsFalse(await IsFileReadableAsync(appContainer, _tempFile.FullName));
		appContainer.AddDirectoryAccess(_tempDir, FileSystemRights.FullControl);
		Assert.IsTrue(await IsFileReadableAsync(appContainer, _tempFile.FullName));
	}

	[TestMethod]
	public async Task FileSystem_CannotRead_DefaultTempDir_Async()
	{
		using AppContainer appContainer = Create();
		Assert.IsTrue(await IsFileReadableAsync(null));
		Assert.IsFalse(await IsFileReadableAsync(appContainer));
	}

	[TestMethod]
	public async Task FileSystem_CanRead_WindowsDir_Async()
	{
		// Windows dir can be read by restricted apps by default
		using AppContainer appContainer = Create();
		Assert.IsTrue(await IsFileReadableAsync(null, RegeditExePath, "run in DOS mode"));
		Assert.IsTrue(await IsFileReadableAsync(appContainer, RegeditExePath, "run in DOS mode"));
	}

	private static async Task<bool> IsFileReadableAsync(AppContainer? appContainer, string? filename = null, string? matchString = FileContent)
	{
		if (filename == null)
		{
			filename = Path.GetTempFileName();
			await File.WriteAllTextAsync(filename, FileContent);
		}

		// Use built-in Windows tool findstr.exe as an external process to launch and read from the file system with
		// findstr.exe finds a string in the given file. If exit code is zero, file could be read and string was found.
		string exe = @"c:\Windows\System32\findstr.exe";

		if (appContainer != null)
		{
			string destFileName = Path.Join(appContainer.GetFolderPath().FullName, "target.exe");
			File.Copy(exe, destFileName, true);
			exe = destFileName;
		}

		(int exitCode, string _) = await RunProcessAsync(exe, $"\"{matchString}\" {filename}", appContainer);

		return exitCode switch
		{
			0 => true,
			1 => false,
			2 => false,
			_ => throw new Exception("Unknown exit code: " + exitCode)
		};
	}

	private static string GetName(int? id = null)
	{
		id ??= Interlocked.Increment(ref s_idCounter);
		string containerName = "EpicGames-UE-Test-" + id;
		s_allocatedNames.Add(containerName);
		return containerName;
	}

	private static AppContainer Create()
	{
		string containerName = GetName();
		AppContainer.Delete(containerName);
		return AppContainer.Create(containerName, DisplayName, Description);
	}

	private static async Task<(int exitCode, string output)> RunProcessAsync(string fileName, string commandLine, AppContainer? appContainer)
	{
		using ManagedProcess process = new(null, fileName, commandLine, null, null, null, ProcessPriorityClass.Normal, appContainer);
		using MemoryStream ms = new(500);
		await process.CopyToAsync((buffer, offset, length) => { ms.Write(buffer, offset, length); }, 4096, CancellationToken.None);
		await process.WaitForExitAsync(CancellationToken.None);
		return (process.ExitCode, Encoding.UTF8.GetString(ms.ToArray()));
	}
}

#pragma warning restore CA1416