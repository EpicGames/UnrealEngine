// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public sealed class AsyncThreadPoolWorkQueueTests : IDisposable
{
	private readonly AsyncThreadPoolWorkQueue _queue = new(4);

	public void Dispose()
	{
		_queue.Dispose();
	}

	[TestMethod]
	public async Task ExecuteEnqueuedTasksAsync()
	{
		int c = 0;
		await _queue.EnqueueAsync((_) =>
		{
			Interlocked.Increment(ref c);
			return Task.CompletedTask;
		});
		await _queue.EnqueueAsync((_) =>
		{
			Interlocked.Increment(ref c);
			return Task.CompletedTask;
		});
		await _queue.ExecuteAsync();
		Assert.AreEqual(2, c);
	}

	[TestMethod]
	public async Task ExecuteManyEnqueuedTasksAsync()
	{
		int c = 0;
		int numTasks = 100000;

		for (int i = 0; i < numTasks; i++)
		{
			await _queue.EnqueueAsync((_) =>
			{
				Interlocked.Increment(ref c);
				return Task.CompletedTask;
			});
		}

		await _queue.ExecuteAsync();
		Assert.AreEqual(numTasks, c);
	}

	[TestMethod]
	public async Task ExecuteNestedEnqueuedTasksAsync()
	{
		bool task1 = false;
		bool task2 = false;
		bool task3 = false;

		await _queue.EnqueueAsync(async (ct1) =>
		{
			await Task.Delay(10, ct1);
			await _queue.EnqueueAsync(async (ct2) =>
			{
				await Task.Delay(10, ct2);
				await _queue.EnqueueAsync(async (ct3) =>
				{
					await Task.Delay(10, ct3);
					task3 = true;
				});
				task2 = true;
			});
			task1 = true;
		});

		await _queue.ExecuteAsync();
		Assert.IsTrue(task1);
		Assert.IsTrue(task2);
		Assert.IsTrue(task3);
	}

	[Ignore]
	[TestMethod]
	public async Task VerifyMaxConcurrencyAsync()
	{
		const int NumTasks = 10000;
		const int NumWorkers = 3;
		int c = 0;
		using AsyncThreadPoolWorkQueue queue = new(NumWorkers);

		HashSet<int> uniqueTaskIds = new(NumTasks);
		for (int i = 0; i < NumTasks; i++)
		{
			await queue.EnqueueAsync((_) =>
			{
				uniqueTaskIds.Add(Task.CurrentId!.Value);
				Interlocked.Increment(ref c);
				return Task.CompletedTask;
			});
		}

		await queue.ExecuteAsync();

		Assert.AreEqual(NumTasks, c);
		Assert.AreEqual(NumWorkers, uniqueTaskIds.Count);
	}
}

