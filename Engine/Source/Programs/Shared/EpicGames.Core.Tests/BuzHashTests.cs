// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Cryptography;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class BuzHashTests
	{
		[TestMethod]
		public void AddSub()
		{
			byte[] bytes = RandomNumberGenerator.GetBytes(4096);

			const int WindowLength = 32;

			uint rollingHash = BuzHash.Add(0, bytes.AsSpan(0, WindowLength - 1));
			for (int idx = WindowLength; idx < bytes.Length; idx++)
			{
				rollingHash = BuzHash.Add(rollingHash, bytes[idx - 1]);

				uint newRollingHash = BuzHash.Add(0, bytes.AsSpan(idx - WindowLength, WindowLength));
				Assert.AreEqual(rollingHash, newRollingHash);

				rollingHash = BuzHash.Sub(rollingHash, bytes[idx - WindowLength], WindowLength);
			}
		}

		[TestMethod]
		public void AddSubStepped()
		{
			byte[] bytes = RandomNumberGenerator.GetBytes(4096);

			const int WindowLength = 32;
			const int WindowStep = 8;

			uint rollingHash = BuzHash.Add(0, bytes.AsSpan(0, WindowLength - WindowStep));
			for (int idx = WindowLength; idx < bytes.Length; idx += WindowStep)
			{
				rollingHash = BuzHash.Add(rollingHash, bytes.AsSpan(idx - WindowStep, WindowStep));

				uint newRollingHash = BuzHash.Add(0, bytes.AsSpan(idx - WindowLength, WindowLength));
				Assert.AreEqual(rollingHash, newRollingHash);

				rollingHash = BuzHash.Sub(rollingHash, bytes.AsSpan(idx - WindowLength, WindowStep), WindowLength);
			}
		}

		[TestMethod]
		public void Candidates()
		{
			Random random = new(0);

			byte[] bytes = new byte[16 * 1024 * 1024];
			random.NextBytes(bytes);

			const int MinSize = 16 * 1024;
			const int TargetSize = 32 * 1024;
			const int MaxSize = 128 * 1024;

			List<(int Offset, uint Hash)> expected = BuzHash.FindSplitPoints(bytes, MinSize, MaxSize, TargetSize);
			List<(int Offset, uint Hash)> candidates = BuzHash.FindCandidateSplitPoints(bytes, MinSize, BuzHash.GetThreshold(TargetSize - MinSize));

			int idx = 0;

			int lastOffset = 0;
			foreach ((int offset, _) in candidates)
			{
				while (offset >= lastOffset + MaxSize)
				{
					Assert.AreEqual(lastOffset + MaxSize, expected[idx].Offset);
					idx++;
					lastOffset += MaxSize;
				}
				if (offset >= lastOffset + MinSize)
				{
					Assert.AreEqual(offset, expected[idx].Offset);
					idx++;
					lastOffset = offset;
				}
			}

			Assert.AreEqual(idx, expected.Count);
		}
	}
}
