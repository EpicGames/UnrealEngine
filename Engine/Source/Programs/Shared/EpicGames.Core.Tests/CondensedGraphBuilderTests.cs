// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	/// <summary>
	/// Tests for CondensedGraphBuilder, ported from the C++ automation test
	/// FCondensedGraphTest in Editor/UnrealEd/Private/Cooker/Algo/GraphConvert.cpp.
	/// </summary>
	[TestClass]
	public class CondensedGraphBuilderTests
	{
		// -------------------------------------------------------------------------
		// Test state (reset between cases via Clear())
		// -------------------------------------------------------------------------

		/// <summary>
		/// Sparse adjacency list: _adjacency[v] contains the outgoing edge targets for vertex v.
		/// </summary>
		private readonly List<int[]> _adjacency = [];

		/// <summary>
		/// Expected SCC membership for vertices that participate in cycles.
		/// Vertices not listed here are expected to be singleton SCCs.
		/// </summary>
		private readonly List<int[]> _expectedComponents = [];

		// -------------------------------------------------------------------------
		// Helpers mirroring the C++ AddVertex / AddExpectedComponent / Clear lambdas
		// -------------------------------------------------------------------------

		private void AddVertex(int v, params int[] edges)
		{
			while (_adjacency.Count <= v)
			{
				_adjacency.Add([]);
			}
			_adjacency[v] = edges;
		}

		private void AddExpectedComponent(params int[] vertices)
		{
			_expectedComponents.Add(vertices);
		}

		private void Clear()
		{
			_adjacency.Clear();
			_expectedComponents.Clear();
		}

		// -------------------------------------------------------------------------
		// ConfirmResults — mirrors the C++ ConfirmResults lambda
		// -------------------------------------------------------------------------

		private void ConfirmResults(string testCaseName)
		{
			// Build the edge list from _adjacency.
			List<Tuple<int, int>> edges = [];
			for (int v = 0; v < _adjacency.Count; ++v)
			{
				foreach (int u in _adjacency[v])
				{
					edges.Add(Tuple.Create(v, u));
				}
			}

			CondensedGraphBuilder<int> builder = new(edges);
			bool result = builder.Build();

			// (1) Verify the return value matches whether we declared any expected components.
			bool expectedResult = _expectedComponents.Count > 0;
			Assert.AreEqual(expectedResult, result,
				$"[{testCaseName}] Build() returned {result} but expected {expectedResult}.");

			List<int[]>                  sCCs            = builder.GetSCCs();
			IReadOnlyDictionary<int, int> nodeToSCC      = builder.GetNodeToSCCIndex();
			List<int[]>                  condensedEdges  = builder.GetCondensedEdges();

			int numVertices   = _adjacency.Count;
			int numOutVertices = sCCs.Count;

			// (2) Verify that OutVertexToInVertices and InVertexToOutVertex are consistent:
			//     every vertex appears in exactly one SCC and the two mappings agree.
			int[] actualInVertexToOutVertex = new int[numVertices];
			bool[] assigned = new bool[numVertices];

			int componentIndex = 0;
			foreach (int[] componentVertices in sCCs)
			{
				foreach (int inVertex in componentVertices)
				{
					Assert.IsFalse(assigned[inVertex],
						$"[{testCaseName}] OutVertexToInVertices has vertex {inVertex} in multiple SCCs " +
						$"(seen again in SCC {componentIndex}, previously in SCC {actualInVertexToOutVertex[inVertex]}).");

					assigned[inVertex] = true;
					actualInVertexToOutVertex[inVertex] = componentIndex;
				}
				++componentIndex;
			}

			for (int inVertex = 0; inVertex < numVertices; ++inVertex)
			{
				Assert.IsTrue(assigned[inVertex],
					$"[{testCaseName}] OutVertexToInVertices is missing vertex {inVertex}.");

				Assert.AreEqual(actualInVertexToOutVertex[inVertex], nodeToSCC[inVertex],
					$"[{testCaseName}] GetNodeToSCCIndex()[{inVertex}] = {nodeToSCC[inVertex]} " +
					$"but GetSCCs() places it in SCC {actualInVertexToOutVertex[inVertex]}.");
			}

			// (3) Finish building ExpectedOutVertexToInVertices: any vertex not explicitly
			//     declared is assumed to be a singleton SCC. Then verify actual memberships.
			int[] expectedInVertexToOutVertex = new int[numVertices];
			List<List<int>> allExpectedComponents = [];

			Array.Clear(assigned, 0, numVertices);
			int expComponentIndex = 0;
			foreach (int[] comp in _expectedComponents)
			{
				List<int> compList = [.. comp];
				allExpectedComponents.Add(compList);
				foreach (int v in comp)
				{
					assigned[v] = true;
					expectedInVertexToOutVertex[v] = expComponentIndex;
				}
				++expComponentIndex;
			}
			for (int inVertex = 0; inVertex < numVertices; ++inVertex)
			{
				if (!assigned[inVertex])
				{
					expectedInVertexToOutVertex[inVertex] = expComponentIndex++;
					allExpectedComponents.Add([inVertex]);
				}
			}

			// For each vertex, check its SCC has the same sorted members as expected.
			Array.Clear(assigned, 0, numVertices);
			for (int inVertex = 0; inVertex < numVertices; ++inVertex)
			{
				if (assigned[inVertex])
				{
					continue;
				}

				int actualSCCIndex   = actualInVertexToOutVertex[inVertex];
				int expectedSCCIndex = expectedInVertexToOutVertex[inVertex];

				int[] actualMembers   = [.. sCCs[actualSCCIndex].OrderBy(x => x)];
				int[] expectedMembers = [.. allExpectedComponents[expectedSCCIndex].OrderBy(x => x)];

				CollectionAssert.AreEqual(expectedMembers, actualMembers,
					$"[{testCaseName}] Vertex {inVertex} is in SCC {{{String.Join(",", actualMembers)}}} " +
					$"but expected SCC {{{String.Join(",", expectedMembers)}}}.");

				foreach (int m in actualMembers)
				{
					assigned[m] = true;
				}
			}

			if (result)
			{
				// (4) Every input edge must be either within-SCC or present in the Condensed.
				for (int inSourceVertex = 0; inSourceVertex < numVertices; ++inSourceVertex)
				{
					int outSourceVertex = actualInVertexToOutVertex[inSourceVertex];
					foreach (int inEdgeVertex in _adjacency[inSourceVertex])
					{
						int expectedOutEdgeVertex = actualInVertexToOutVertex[inEdgeVertex];
						if (expectedOutEdgeVertex == outSourceVertex)
						{
							continue;
						}
						Assert.IsTrue(condensedEdges[outSourceVertex].Contains(expectedOutEdgeVertex),
							$"[{testCaseName}] Missing Condensed edge: expected SCC {outSourceVertex} -> " +
							$"SCC {expectedOutEdgeVertex} (from input edge {inSourceVertex} -> {inEdgeVertex}).");
					}
				}

				// (5) No extra Condensed edges: for each SCC, compute the expected edge set
				//     by unioning all member outgoing edges and verify exact match.
				for (int outVertex = 0; outVertex < numOutVertices; ++outVertex)
				{
					HashSet<int> expectedOutEdges = [];
					foreach (int inVertex in sCCs[outVertex])
					{
						foreach (int sourceEdgeVertex in _adjacency[inVertex])
						{
							int outEdgeVertex = actualInVertexToOutVertex[sourceEdgeVertex];
							if (outEdgeVertex != outVertex)
							{
								expectedOutEdges.Add(outEdgeVertex);
							}
						}
					}

					int[] sortedActual   = [.. condensedEdges[outVertex].OrderBy(x => x)];
					int[] sortedExpected = [.. expectedOutEdges.OrderBy(x => x)];

					CollectionAssert.AreEqual(sortedExpected, sortedActual,
						$"[{testCaseName}] SCC {outVertex}: expected edges " +
						$"{{{String.Join(",", sortedExpected)}}} but got " +
						$"{{{String.Join(",", sortedActual)}}}.");
				}

				// (6) Verify Condensed is topologically sorted root-to-leaf:
				//     every vertex reachable from SCC[Root] must have index >= Root.
				for (int root = 0; root < numOutVertices; ++root)
				{
					HashSet<int> seen = [root];
					Stack<int> toVisit = new();
					toVisit.Push(root);

					while (toVisit.Count > 0)
					{
						int cur = toVisit.Pop();
						foreach (int neighbor in condensedEdges[cur])
						{
							if (seen.Add(neighbor))
							{
								toVisit.Push(neighbor);
							}
						}
					}

					foreach (int reachableVertex in seen)
					{
						Assert.IsTrue(reachableVertex >= root,
							$"[{testCaseName}] SCC {reachableVertex} is reachable from SCC {root} " +
							$"but has a lower index (output is not root-to-leaf sorted).");
					}
				}
			}
			else
			{
				// (7) When Build() returns false, GetCondensedEdges() must be empty.
				Assert.AreEqual(0, condensedEdges.Count,
					$"[{testCaseName}] Expected GetCondensedEdges() to be empty when Build() returns false.");
			}
		}

		// -------------------------------------------------------------------------
		// Test cases — direct ports of the C++ FCondensedGraphTest cases
		// -------------------------------------------------------------------------

		[TestMethod]
		public void EachNodeDependsOnThePreviousOne()
		{
			Clear();
			AddVertex(0);
			AddVertex(1, 0);
			AddVertex(2, 1);
			ConfirmResults("Each node depends on the previous one");
		}

		[TestMethod]
		public void EachNodeDependsOnTheNextOne()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 2);
			AddVertex(2);
			ConfirmResults("Each node depends on the next one");
		}

		[TestMethod]
		public void SelfReferences()
		{
			Clear();
			AddVertex(0, 0);
			AddVertex(1, 0, 1);
			AddVertex(2, 1, 2);
			ConfirmResults("SelfReferences");
		}

		[TestMethod]
		public void SketchedOutExample1()
		{
			//              6
			//             / \
			//            5   7
			//           / \   \
			//          0   1   8
			//           \ / \   \
			//            3   4   |
			//             \   \ /
			//              \   9
			//               \ /
			//                2
			Clear();
			AddVertex(6, 5, 7);
			AddVertex(5, 0, 1);
			AddVertex(7, 8);
			AddVertex(0, 3);
			AddVertex(1, 3, 4);
			AddVertex(8, 9);
			AddVertex(3, 2);
			AddVertex(4, 9);
			AddVertex(9, 2);
			AddVertex(2);
			ConfirmResults("SketchedOutExample1");
		}

		[TestMethod]
		public void SimpleCycle()
		{
			Clear();
			AddVertex(0, 0, 1);
			AddVertex(1, 0, 1);
			AddExpectedComponent(0, 1);
			ConfirmResults("Simple cycle");
		}

		[TestMethod]
		public void ShortCycleInALongCycle()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 0, 1, 2);
			AddVertex(2, 2, 0);
			AddExpectedComponent(0, 1, 2);
			ConfirmResults("Short cycle in a long cycle");
		}

		[TestMethod]
		public void CycleInRootDependingOnChain()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 0, 2);
			AddVertex(2, 3);
			AddVertex(3);
			AddExpectedComponent(0, 1);
			ConfirmResults("Cycle in the root and with the root cycle depending on a chain of non-cycle verts");
		}

		[TestMethod]
		public void CycleInRootDependingOnChainReverse()
		{
			Clear();
			AddVertex(0);
			AddVertex(1, 0);
			AddVertex(2, 1, 3);
			AddVertex(3, 2);
			AddExpectedComponent(2, 3);
			ConfirmResults("Cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse");
		}

		[TestMethod]
		public void CycleAtLeafWithChainFromRoot()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 2);
			AddVertex(2, 3);
			AddVertex(3, 2);
			AddExpectedComponent(2, 3);
			ConfirmResults("Cycle at a leaf and a chain from the root depending on that cycle");
		}

		[TestMethod]
		public void CycleInRootWithChainDependents()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 0);
			AddVertex(2, 1);
			AddVertex(3, 2);
			AddExpectedComponent(0, 1);
			ConfirmResults("Cycle in the root and with the root cycle depending on a chain of non-cycle verts");
		}

		[TestMethod]
		public void VertexDependentUponACycle()
		{
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 2, 3);
			AddVertex(2, 1, 3);
			AddVertex(3, 1, 2);
			AddExpectedComponent(1, 2, 3);
			ConfirmResults("Vertex dependent upon a cycle");
		}

		[TestMethod]
		public void OneCycleDependentUponAnother()
		{
			// 0 -> 1 -> 2 -> 3 -> 1
			//           |
			//           v
			//      4 -> 5 -> 6 -> 4
			Clear();
			AddVertex(0, 1);
			AddVertex(1, 2);
			AddVertex(2, 3, 5);
			AddVertex(3, 1);
			AddVertex(4, 5);
			AddVertex(5, 6);
			AddVertex(6, 4);
			AddExpectedComponent(1, 2, 3);
			AddExpectedComponent(4, 5, 6);
			ConfirmResults("One cycle dependent upon another");
		}

		[TestMethod]
		public void MutuallyReachableSetProblem1()
		{
			// 5 -> 0 -> (1 -> 2 -> 1)
			//      |          |
			//      |          v
			//      |          5
			//      v
			//     (3 -> 4 -> 3)
			Clear();
			AddVertex(0, 1, 3);
			AddVertex(1, 2);
			AddVertex(2, 1, 5);
			AddVertex(3, 4);
			AddVertex(4, 3);
			AddVertex(5, 0);
			AddExpectedComponent(0, 1, 2, 5);
			AddExpectedComponent(3, 4);
			ConfirmResults("MutuallyReachableSet Problem1");
		}

		[TestMethod]
		public void FullyConnectedGraph()
		{
			Clear();
			List<int> allButItself = [0, 1, 2, 3, 4, 5];
			int[] all = [0, 1, 2, 3, 4, 5];
			for (int vertex = 0; vertex < 6; ++vertex)
			{
				allButItself.Remove(vertex);
				AddVertex(vertex, [.. allButItself]);
				allButItself.Add(vertex);
			}
			AddExpectedComponent(all);
			ConfirmResults("FullyConnectedGraph");
		}

		[TestMethod]
		public void CycleOfCycles()
		{
			// 0 - 1 - 2 - 0
			// |
			// 3 - 4 - 5 - 3
			//         |
			// 6 - 7 - 8 - 6
			//     |
			//     1
			Clear();
			AddVertex(0, 1, 3);
			AddVertex(1, 2);
			AddVertex(2, 0);
			AddVertex(3, 4);
			AddVertex(4, 5);
			AddVertex(5, 3, 8);
			AddVertex(6, 7);
			AddVertex(7, 1, 8);
			AddVertex(8, 6);
			AddExpectedComponent(0, 1, 2, 3, 4, 5, 6, 7, 8);
			ConfirmResults("Cycle of cycles");
		}

		[TestMethod]
		public void TreeOfCycles()
		{
			// 0 ----- 1 ----- 2 ----- 0
			// |       |       |
			// |       0       |
			// |               |
			// 5 - 4 - 3 - 5   6 - 7 - 6
			//     |\              |
			//     | 5             |
			//     |               |
			//     8 - 9 - 10 - 9  11
			//     |   |   |
			//     10  8   8
			Clear();
			AddVertex(0, 1, 5);
			AddVertex(1, 0, 2);
			AddVertex(2, 0, 6);
			AddVertex(3, 5);
			AddVertex(4, 3, 5, 8);
			AddVertex(5, 4);
			AddVertex(6, 7);
			AddVertex(7, 6, 11);
			AddVertex(8, 9, 10);
			AddVertex(9, 8, 10);
			AddVertex(10, 8, 9);
			AddVertex(11);
			AddExpectedComponent(0, 1, 2);
			AddExpectedComponent(3, 4, 5);
			AddExpectedComponent(6, 7);
			AddExpectedComponent(8, 9, 10);
			ConfirmResults("Tree of cycles");
		}
	}
}
