<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <TreatWarningsAsErrors>False</TreatWarningsAsErrors>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>
</Project>
