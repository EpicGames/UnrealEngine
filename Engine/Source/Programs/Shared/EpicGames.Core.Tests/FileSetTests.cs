// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class FileSetTests
	{
		private readonly DirectoryReference _tempDir;

		public FileSetTests()
		{
			_tempDir = CreateTempDir();
		}

		[TestMethod]
		public void FileSetContainsAllFiles()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));
			Assert.IsTrue(allFiles.Count() == 4);
		}

		[TestMethod]
		public void FileSetFilterContainsOnlyFileTypeWildcard()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			// Make sure we have filtered to just what is in a sub-directory
			FileSet justMdFile = allFiles.Filter("*.md");
			Assert.IsTrue(justMdFile.Count() == 1);

			// Make sure filtered fileset contains root level file and sub-directory file
			FileSet justTxtFile = allFiles.Filter("*.txt");
			Assert.IsTrue(justTxtFile.Count() == 2);
		}

		[TestMethod]
		public void FileSetFilterExcludesWildcardWithMinus()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			FileSet withoutMd = allFiles.Filter("*;-*.md");
			Assert.IsTrue(withoutMd.Count() == 3);

			FileSet withoutTxtFile = allFiles.Filter("*;-*.txt");
			Assert.IsTrue(withoutTxtFile.Count() == 2);
		}

		[TestMethod]
		public void FileSetExceptExcludesFileTypeWildcard()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			// Make sure except properly excludes files in root directory and sub-directories
			FileSet withoutMd = allFiles.Except("*.md");
			Assert.IsTrue(withoutMd.Count() == 3);

			// Make sure except works with files in the root as well as sub-directories
			FileSet withoutTxtFile = allFiles.Except("*.txt");
			Assert.IsTrue(withoutTxtFile.Count() == 2);
		}

		[TestMethod]
		public void FileSetFilterContainsDirectoryName()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			FileSet results = allFiles.Filter("dir1/dir1.txt");
			Assert.IsTrue(results.Count() == 1);
		}

		[TestMethod]
		public void FileSetFilterContainsMultipleDirectoryName()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			FileSet results = allFiles.Filter("dir1/dir2/dir2.md");
			Assert.IsTrue(results.Count() == 1);
		}

		[TestMethod]
		public void FileSetFilterContainsTripleDotSuffix()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			FileSet results = allFiles.Filter("dir1/...");
			Assert.IsTrue(results.Count() == 2);
		}

		[TestMethod]
		public void FileSetFilterContainsTripleDotPrefixAndSuffix()
		{
			FileSet allFiles = FileSet.FromDirectory(new DirectoryReference(_tempDir.FullName));

			FileSet results = allFiles.Filter(".../dir2/...");
			Assert.IsTrue(results.Count() == 1);
		}

		[TestInitialize]
		public void SetupTempDir()
		{
			string rootTxt = Path.Join(_tempDir.FullName, "root.txt");
			File.WriteAllText(rootTxt, "placeholder");

			string rootPdb = Path.Join(_tempDir.FullName, "rootPdb.pdb");
			File.WriteAllText(rootPdb, "placeholder");

			string dir1 = Path.Join(_tempDir.FullName, "dir1");
			Directory.CreateDirectory(dir1);

			string dir1Txt = Path.Join(dir1, "dir1.txt");
			File.WriteAllText(dir1Txt, "placeholder");

			string dir2 = Path.Join(dir1, "dir2");
			Directory.CreateDirectory(dir2);

			string dir2Md = Path.Join(dir2, "dir2.md");
			File.WriteAllText(dir2Md, "placeholder");
		}

		[TestCleanup]
		public void RemoveTempDir()
		{
			if (Directory.Exists(_tempDir.FullName))
			{
				Directory.Delete(_tempDir.FullName, true);
			}
		}

		private static DirectoryReference CreateTempDir()
		{
			string tempDir = Path.Join(Path.GetTempPath(), "epicgames-core-tests-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);
			return new DirectoryReference(tempDir);
		}
	}
}
