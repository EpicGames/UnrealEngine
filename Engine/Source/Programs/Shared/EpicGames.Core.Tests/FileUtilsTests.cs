// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using Microsoft.Win32.SafeHandles;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class FileUtilsTest
	{
		private readonly DirectoryReference _tempDir;

		public FileUtilsTest()
		{
			_tempDir = CreateTempDir();
		}

		[TestMethod]
		public void ForceDeleteDirectoryNormalPath()
		{
			string subDir = Path.Join(_tempDir.FullName, "Foo");
			Directory.CreateDirectory(subDir);
			FileUtils.ForceDeleteDirectory(subDir);
			Assert.IsFalse(Directory.Exists(subDir));
		}

		[TestMethod]
		public void ForceDeleteDirectoryLongPath()
		{
			string subDir = CreateSubDirWithLongPath();
			FileUtils.ForceDeleteDirectory(subDir);
			Assert.IsFalse(Directory.Exists(subDir));
		}

		[TestMethod]
		public void ForceDeleteDirectoryContentsNormalPath()
		{
			string subDir = Path.Join(_tempDir.FullName, "Foo");
			Directory.CreateDirectory(subDir);
			string filePath = Path.Join(subDir, "file.txt");
			File.WriteAllText(filePath, "placeholder");

			FileUtils.ForceDeleteDirectoryContents(subDir);
			Assert.IsTrue(Directory.Exists(subDir));
			Assert.IsFalse(File.Exists(filePath));
		}

		[TestMethod]
		public void ForceDeleteDirectoryContentsLongPath()
		{
			string subDir = CreateSubDirWithLongPath();
			string filePath = Path.Join(subDir, "file.txt");
			File.WriteAllText(filePath, "placeholder");

			FileUtils.ForceDeleteDirectoryContents(subDir);
			Assert.IsTrue(Directory.Exists(subDir));
			Assert.IsFalse(File.Exists(filePath));
		}

		[TestCleanup]
		public void RemoveTempDir()
		{
			if (Directory.Exists(_tempDir.FullName))
			{
				Directory.Delete(_tempDir.FullName, true);
			}
		}

		private string CreateSubDirWithLongPath()
		{
			const string LongDirNameA = "ThisIsAVeryLongDirectoryName";
			const string LongDirNameB = "AnotherLongDirectoryNameThatIsUsed";
			string subDir = Path.Join(_tempDir.FullName, "Foo", LongDirNameA, LongDirNameB, LongDirNameA, LongDirNameB, LongDirNameA, LongDirNameB, LongDirNameA, LongDirNameB);
			Assert.IsTrue(subDir.Length > 260, "Longer than Windows MAX_PATH");
			Directory.CreateDirectory(subDir);
			return subDir;
		}

		internal static DirectoryReference CreateTempDir()
		{
			string tempDir = Path.Join(Path.GetTempPath(), "epicgames-core-tests-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);
			return new DirectoryReference(tempDir);
		}
	}

	[TestClass]
	[SupportedOSPlatform("windows")]
	public class FileUtilsWindowsTest
	{
		private readonly DirectoryReference _tempDir = FileUtilsTest.CreateTempDir();

		[TestInitialize]
		public void CheckWindows()
		{
			if (!OperatingSystem.IsWindows())
			{
				Assert.Inconclusive("This test only runs on Windows");
			}
		}

		[TestCleanup]
		public void RemoveTempDir()
		{
			if (Directory.Exists(_tempDir.FullName))
			{
				Directory.Delete(_tempDir.FullName, true);
			}
		}

		[TestMethod]
		public void ForceDeleteDirectory_LockedFile_IncludesLockInfo()
		{
			string subDir = Path.Join(_tempDir.FullName, "LockedDir");
			Directory.CreateDirectory(subDir);
			string tempFile = Path.Join(subDir, "locked_file.txt");
			File.WriteAllText(tempFile, "test content");

			using FileStream lockedStream = new(tempFile, FileMode.Open, FileAccess.Read, FileShare.None);
			WrappedFileOrDirectoryException ex = Assert.ThrowsExactly<WrappedFileOrDirectoryException>(() => FileUtils.ForceDeleteDirectory(subDir));

			// Verify the exception includes process ID, image path, and command line of the locking process
			string msg = ex.Message;
			StringAssert.Contains(msg, "locked_file.txt", StringComparison.Ordinal);
			StringAssert.Contains(msg, "Processes with open handles", StringComparison.Ordinal);
			StringAssert.Contains(msg, $"{Environment.ProcessId}: ", StringComparison.Ordinal);
			StringAssert.Contains(msg, "Command line:", StringComparison.Ordinal);
		}

		[TestMethod]
		public void ForceDeleteFileWin32_NormalFile_Deleted()
		{
			string filePath = Path.Join(_tempDir.FullName, "normal.txt");
			File.WriteAllText(filePath, "content");
			FileUtils.ForceDeleteFileWin32(filePath);
			Assert.IsFalse(File.Exists(filePath));
		}

		[TestMethod]
		public void ForceDeleteFileWin32_NonExistentFile_NoThrow()
		{
			string filePath = Path.Join(_tempDir.FullName, "does_not_exist.txt");
			FileUtils.ForceDeleteFileWin32(filePath);
		}

		[TestMethod]
		public void ForceDeleteFileWin32_ReadOnlyFile_Deleted()
		{
			string filePath = Path.Join(_tempDir.FullName, "readonly.txt");
			File.WriteAllText(filePath, "content");
			File.SetAttributes(filePath, FileAttributes.ReadOnly);
			FileUtils.ForceDeleteFileWin32(filePath);
			Assert.IsFalse(File.Exists(filePath));
		}

		[TestMethod]
		public void ForceDeleteFileWin32_LockedFile_ThrowsWithLockInfo()
		{
			string filePath = Path.Join(_tempDir.FullName, "locked.txt");
			File.WriteAllText(filePath, "content");

			using FileStream lockedStream = new(filePath, FileMode.Open, FileAccess.Read, FileShare.None);
			WrappedFileOrDirectoryException ex = Assert.ThrowsExactly<WrappedFileOrDirectoryException>(() => FileUtils.ForceDeleteFileWin32(filePath));

			string msg = ex.Message;
			StringAssert.Contains(msg, "locked.txt", StringComparison.Ordinal);
			StringAssert.Contains(msg, "Processes with open handles", StringComparison.Ordinal);
			StringAssert.Contains(msg, $"{Environment.ProcessId}: ", StringComparison.Ordinal);
		}

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		static extern SafeFileHandle CreateFileW(string lpFileName, uint dwDesiredAccess, uint dwShareMode, IntPtr lpSecurityAttributes, uint dwCreationDisposition, uint dwFlagsAndAttributes, IntPtr hTemplateFile);

		static SafeFileHandle HoldDirectoryHandle(string path)
		{
			SafeFileHandle handle = CreateFileW(path, 0x80000000 /* GENERIC_READ */, 0x01 /* FILE_SHARE_READ */, IntPtr.Zero, 3 /* OPEN_EXISTING */, 0x02000000 /* FILE_FLAG_BACKUP_SEMANTICS */, IntPtr.Zero);
			Assert.IsFalse(handle.IsInvalid, $"Failed to open directory handle on {path}");
			return handle;
		}

		static void AssertDiagnosticInfo(string message)
		{
			Assert.IsTrue(
				message.Contains("Directory is empty but could not be removed", StringComparison.Ordinal)
				|| message.Contains("Processes with open handles", StringComparison.Ordinal)
				|| message.Contains("Failed to gather lock diagnostics", StringComparison.Ordinal),
				$"Expected diagnostic info in message but got: {message}");
		}

		[TestMethod]
		public void ForceDeleteDirectory_ReadOnlySubdirectory_Deleted()
		{
			string subDir = Path.Join(_tempDir.FullName, "Parent");
			string childDir = Path.Join(subDir, "ReadOnlyChild");
			Directory.CreateDirectory(childDir);
			File.SetAttributes(childDir, File.GetAttributes(childDir) | FileAttributes.ReadOnly);

			FileUtils.ForceDeleteDirectory(subDir);
			Assert.IsFalse(Directory.Exists(subDir));
		}

		[TestMethod]
		public void ForceDeleteDirectory_ReadOnlyDirectory_Deleted()
		{
			string subDir = Path.Join(_tempDir.FullName, "ReadOnlyDir");
			Directory.CreateDirectory(subDir);
			File.SetAttributes(subDir, File.GetAttributes(subDir) | FileAttributes.ReadOnly);

			FileUtils.ForceDeleteDirectory(subDir);
			Assert.IsFalse(Directory.Exists(subDir));
		}

		[TestMethod]
		public void ForceDeleteDirectory_OpenDirectoryHandle_ThrowsWithDiagnostics()
		{
			string subDir = Path.Join(_tempDir.FullName, "HandleDir");
			Directory.CreateDirectory(subDir);

			using SafeFileHandle _ = HoldDirectoryHandle(subDir);

			WrappedFileOrDirectoryException ex = Assert.ThrowsExactly<WrappedFileOrDirectoryException>(() => FileUtils.ForceDeleteDirectory(subDir));
			AssertDiagnosticInfo(ex.Message);
		}

		[TestMethod]
		public void GetFileLockInfoWin32_ReturnsCommandLineAndParentInfo()
		{
			string tempFile = Path.Combine(_tempDir.FullName, "locked_file.txt");
			File.WriteAllText(tempFile, "test content");

			// Create a file and lock it from this process and verify we can detect the lock
			using FileStream lockedStream = new (tempFile, FileMode.Open, FileAccess.Read, FileShare.None);
			List<FileLockInfoWin32> lockInfoList = FileUtils.GetFileLockInfoWin32(tempFile);
			Assert.IsNotEmpty(lockInfoList);

			// Find our process (the current test runner)
			int currentPid = Environment.ProcessId;
			FileLockInfoWin32? ourLock = lockInfoList.Find(l => l.ProcessId == currentPid);
			Assert.IsNotNull(ourLock);

			// Verify command line is present (should contain dotnet or testhost)
			Assert.IsNotNull(ourLock.CommandLine);
			Assert.IsGreaterThan(0, ourLock.CommandLine.Length);

			// Verify parent process info is present
			Assert.IsGreaterThan(0, ourLock.ParentProcessId);

			// Verify ToString() includes all the info in a readable format
			string lockString = ourLock.ToString();
			Assert.IsTrue(lockString.Contains(ourLock.ProcessId.ToString(), StringComparison.Ordinal));
			Assert.IsTrue(lockString.Contains("Command line:", StringComparison.Ordinal));
			Assert.IsTrue(lockString.Contains("Parent:", StringComparison.Ordinal));

			// Check multi-line formatting
			string[] lines = lockString.Split('\n');
			Assert.IsGreaterThanOrEqualTo(2, lines.Length);
		}
	}
}
