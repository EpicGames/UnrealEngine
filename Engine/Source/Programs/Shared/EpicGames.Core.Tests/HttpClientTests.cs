// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public class HttpClientExtensionTests
{
	[TestMethod]
	public void RedactedUriTest()
	{
		AssertRedactedUri("http://a.com/path?foo=bar&hello=world", "http://a.com/path?foo=bar&hello=world");
		AssertRedactedUri("http://a.com/path?AWSAccessKeyId=redacted", "http://a.com/path?AWSAccessKeyId=bar");
		AssertRedactedUri("http://a.com/path?AWSACCESSKEYID=redacted&foo=bar", "http://a.com/path?AWSACCESSKEYID=bar&foo=bar");
		AssertRedactedUri("http://a.com/path", "http://a.com/path");
	}

	private static void AssertRedactedUri(string expectedUri, string actualUri)
	{
		using HttpRequestMessage req = new(HttpMethod.Get, actualUri);
		Assert.AreEqual(expectedUri, req.RedactedRequestUri()!.ToString());
	}
}
