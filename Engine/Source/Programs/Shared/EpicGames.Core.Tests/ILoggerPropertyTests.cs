// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

public sealed class StubLogScope(object? state) : IDisposable
{
	public object? State { get; } = state;
	public bool IsDisposed { get; private set; }

	public void Dispose()
	{
		IsDisposed = true;
	}
}

public class SpyLogger : ILogger
{
	public List<object?> ReceivedLogStates { get; } = [];
	public List<StubLogScope> ScopesStarted { get; } = [];

	public void Log<TState>(LogLevel logLevel, EventId eventId, TState? state, Exception? exception, Func<TState, Exception?, string> formatter)
	{
		ReceivedLogStates.Add(state);
	}

	public bool IsEnabled(LogLevel logLevel)
	{
		return true;
	}

	public IDisposable? BeginScope<TState>(TState state) where TState : notnull
	{
		StubLogScope scope = new(state);
		ScopesStarted.Add(scope);
		return scope;
	}
}

public sealed class FakeLoggerProvider(ILogger logger) : ILoggerProvider
{
	public ILogger CreateLogger(string categoryName)
	{
		return logger;
	}

	public void Dispose()
	{
	}
}

[TestClass]
public class ILoggerPropertyTests
{
	[TestMethod]
	public void AttachPropertiesToScopeTest()
	{
		SpyLogger logger = new();
		using LoggerFactory loggerFactory = new();
		using ILoggerProvider provider = new FakeLoggerProvider(logger);
		loggerFactory.AddProvider(provider);

		using (logger.WithProperty("user", "foobar").BeginScope())
		{
			logger.LogInformation("Some log info");
		}

		Dictionary<string, object> scopeState = (Dictionary<string, object>)logger.ScopesStarted[0].State!;
		Assert.AreEqual("foobar", scopeState["user"]);
	}
}
