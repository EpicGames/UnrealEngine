// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Text.Json;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class JsonExtensionsTests
	{
		sealed class BufferSegment : ReadOnlySequenceSegment<byte>
		{
			public BufferSegment(ReadOnlyMemory<byte> memory) => Memory = memory;

			public BufferSegment Append(ReadOnlyMemory<byte> memory)
			{
				BufferSegment next = new(memory) { RunningIndex = RunningIndex + Memory.Length };
				Next = next;
				return next;
			}
		}

		static string GetUtf8StringFromSingleSegment(string jsonLiteral)
		{
			byte[] json = Encoding.UTF8.GetBytes(jsonLiteral);
			Utf8JsonReader reader = new(json);
			reader.Read();
			return Encoding.UTF8.GetString(reader.GetUtf8String());
		}

		static string GetUtf8StringFromMultiSegment(string jsonLiteral, int splitPoint)
		{
			byte[] json = Encoding.UTF8.GetBytes(jsonLiteral);
			if (splitPoint < 0)
			{
				splitPoint = json.Length / 2;
			}
			byte[] firstPart = json[..splitPoint];
			byte[] secondPart = json[splitPoint..];
			BufferSegment first = new(firstPart);
			BufferSegment last = first.Append(secondPart);
			ReadOnlySequence<byte> sequence = new(first, 0, last, secondPart.Length);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			return Encoding.UTF8.GetString(reader.GetUtf8String());
		}

		[TestMethod]
		[DataRow("\"hello-world\"", "hello-world", DisplayName = "no escapes")]
		[DataRow("\"hello\\\\world\\\"foo\\/bar\"", "hello\\world\"foo/bar", DisplayName = "backslash escapes")]
		[DataRow("\"\\u0041\\u0042\\u0043\"", "ABC", DisplayName = "unicode escapes")]
		[DataRow("\"\"", "", DisplayName = "empty string")]
		public void SingleSegment(string jsonLiteral, string expected)
		{
			Assert.AreEqual(expected, GetUtf8StringFromSingleSegment(jsonLiteral));
		}

		[TestMethod]
		[DataRow("\"step-output/test-stream/1/test-artifact/67890abcdef#pkt=4005760,29050&exp=1\"", "step-output/test-stream/1/test-artifact/67890abcdef#pkt=4005760,29050&exp=1", -1, DisplayName = "no escapes")]
		[DataRow("\"hello\\\\world\\\"foo\\/bar\"", "hello\\world\"foo/bar", -1, DisplayName = "backslash escapes")]
		[DataRow("\"AB\\u0043DE\"", "ABCDE", 4, DisplayName = "split at escape boundary")]
		[DataRow("\"\"", "", 1, DisplayName = "empty string")]
		public void MultiSegment(string jsonLiteral, string expected, int splitPoint)
		{
			Assert.AreEqual(expected, GetUtf8StringFromMultiSegment(jsonLiteral, splitPoint));
		}
	}
}
