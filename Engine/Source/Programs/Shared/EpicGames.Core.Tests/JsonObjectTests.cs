// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class JsonObjectTests
	{
		private string _testJsonText = null!;
		private JsonObject _validTestObject = null!;
		private Enemy _validJsonDataSource = null!;
		private static DirectoryReference s_tempDirectory = null!;
		private static JsonSerializerOptions s_jsonOptions = null!;

		public JsonObjectTests()
		{
			s_tempDirectory = CreateTempDir();
			// This is to make sure that enums are serialized as strings in the case that they were written as.
			s_jsonOptions = new();
			s_jsonOptions.Converters.Add(new JsonStringEnumConverter(null));
			s_jsonOptions.WriteIndented = true;
		}

		[TestCleanup]
		public void RemoveTempDir()
		{
			if (Directory.Exists(s_tempDirectory.FullName))
			{
				Directory.Delete(s_tempDirectory.FullName, true);
			}
		}

		[TestInitialize]
		public void InitializeTest()
		{
			Enemy enemy1 = CreateValidJsonDataSource();
			_testJsonText = JsonSerializer.Serialize(enemy1, s_jsonOptions);
			_validTestObject = CreateValidJsonObject();
			_validJsonDataSource = enemy1;
		}

		[TestMethod]
		public void ParseValidJsonText()
		{
			JsonObject.Parse(_testJsonText);
		}

		[TestMethod]
		public void ParseInvalidJsonText()
		{
			MakeTestJsonTextInvalid();
			Assert.Throws<JsonException>(() => JsonObject.Parse(_testJsonText));
		}

		[TestMethod]
		public void ParseEmptyText()
		{
			Assert.Throws<JsonException>(() => JsonObject.Parse(""));
		}

		[TestMethod]
		public void TryParseValidJsonText()
		{
			JsonObject? testObject;
			Assert.IsTrue(JsonObject.TryParse(_testJsonText, out testObject));
			Assert.IsNotNull(testObject);
		}

		[TestMethod]
		public void TryParseInvalidJsonText()
		{
			MakeTestJsonTextInvalid();
			JsonObject? testObject;
			Assert.IsFalse(JsonObject.TryParse(_testJsonText, out testObject));
			Assert.IsNull(testObject);
		}

		[TestMethod]
		public void TryParseEmptyJsonText()
		{
			JsonObject? testObject;
			Assert.IsFalse(JsonObject.TryParse("", out testObject));
			Assert.IsNull(testObject);
		}

		[TestMethod]
		public void ReadValidJsonText()
		{
			FileReference inputFileReference = CreateTempJsonFile("Valid.json", _testJsonText);
			JsonObject testObject = JsonObject.Read(inputFileReference);
			Assert.IsNotNull(testObject);
		}

		[TestMethod]
		public void ReadInvalidJsonText()
		{
			MakeTestJsonTextInvalid();
			FileReference inputFileReference = CreateTempJsonFile("Invalid.json", _testJsonText);
			Assert.Throws<JsonException>(() => JsonObject.Read(inputFileReference));
		}

		[TestMethod]
		public void ReadEmptyJsonText()
		{
			FileReference inputFileReference = CreateTempJsonFile("Empty.json", "");
			Assert.Throws<JsonException>(() => JsonObject.Read(inputFileReference));
		}

		[TestMethod]
		public void TryReadValidJsonText()
		{
			FileReference inputFileReference = CreateTempJsonFile("Valid.json", _testJsonText);
			JsonObject? testObject;
			Assert.IsTrue(JsonObject.TryRead(inputFileReference, out testObject));
			Assert.IsNotNull(testObject);
		}

		[TestMethod]
		public void TryReadInvalidJsonText()
		{
			MakeTestJsonTextInvalid();
			FileReference inputFileReference = CreateTempJsonFile("Invalid.json", _testJsonText);
			JsonObject? testObject;
			Assert.IsFalse(JsonObject.TryRead(inputFileReference, out testObject));
			Assert.IsNull(testObject);
		}

		[TestMethod]
		public void TryReadEmptyJsonText()
		{
			FileReference inputFileReference = CreateTempJsonFile("Empty.json", "");
			JsonObject? testObject;
			Assert.IsFalse(JsonObject.TryRead(inputFileReference, out testObject));
			Assert.IsNull(testObject);
		}

		[TestMethod]
		public void GetIntegerValidField()
		{
			string fieldName = "Hp";
			int correctValue = _validJsonDataSource.Hp;
			Assert.AreEqual(correctValue, _validTestObject.GetIntegerField(fieldName));
			int outValue;
			Assert.IsTrue(_validTestObject.TryGetIntegerField(fieldName, out outValue));
			Assert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetIntegerInvalidField()
		{
			string fieldName = "Invalid";
			int correctValue = _validJsonDataSource.Hp;
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetIntegerField(fieldName));
			int outValue;
			Assert.IsFalse(_validTestObject.TryGetIntegerField(fieldName, out outValue));
			Assert.AreNotEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetStringValidField()
		{
			string fieldName = "Name";
			string correctValue = _validJsonDataSource.Name;
			Assert.AreEqual(correctValue, _validTestObject.GetStringField(fieldName));
			string? outValue;
			Assert.IsTrue(_validTestObject.TryGetStringField(fieldName, out outValue));
			Assert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetStringInvalidField()
		{
			string fieldName = "Invalid";
			string correctValue = _validJsonDataSource.Name;
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetStringField(fieldName));
			string? outValue;
			Assert.IsFalse(_validTestObject.TryGetStringField(fieldName, out outValue));
			Assert.AreNotEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetObjectValidField()
		{
			string fieldName = "StrongestAttack";
			JsonObject correctValue = _validJsonDataSource.StrongestAttack.ToJsonObject();
			JsonObject returnValue = _validTestObject.GetObjectField(fieldName);
			Assert.AreEqual(correctValue, returnValue);
			JsonObject? outValue;
			Assert.IsTrue(_validTestObject.TryGetObjectField(fieldName, out outValue));
			Assert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetObjectInvalidField()
		{
			string fieldName = "Invalid";
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetObjectField(fieldName));
			JsonObject? outValue;
			Assert.IsFalse(_validTestObject.TryGetObjectField(fieldName, out outValue));
			Assert.IsNull(outValue);
		}

		[TestMethod]
		public void GetObjectArrayValidField()
		{
			string fieldName = "Attacks";
			JsonObject[] correctValue = [.. _validJsonDataSource.Attacks.Select(x => x.ToJsonObject())];
			JsonObject[] result = _validTestObject.GetObjectArrayField(fieldName);
			CollectionAssert.AreEqual(correctValue, result);
			JsonObject[]? outValue;
			Assert.IsTrue(_validTestObject.TryGetObjectArrayField(fieldName, out outValue));
			CollectionAssert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetObjectArrayInvalidField()
		{
			string fieldName = "Invalid";
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetObjectArrayField(fieldName));
			JsonObject[]? outValue;
			Assert.IsFalse(_validTestObject.TryGetObjectArrayField(fieldName, out outValue));
			Assert.IsNull(outValue);
		}

		[TestMethod]
		public void GetStringArrayValidField()
		{
			string fieldName = "Skills";
			string[] correctValue = [.. _validJsonDataSource.Skills];
			string[] result = _validTestObject.GetStringArrayField(fieldName);
			CollectionAssert.AreEqual(correctValue, result);
			string[]? outValue;
			Assert.IsTrue(_validTestObject.TryGetStringArrayField(fieldName, out outValue));
			CollectionAssert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetStringArrayInvalidField()
		{
			string fieldName = "Invalid";
			string[] correctValue = [.. _validJsonDataSource.Skills];
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetStringArrayField(fieldName));
			string[]? outValue;
			Assert.IsFalse(_validTestObject.TryGetStringArrayField(fieldName, out outValue));
			Assert.IsNull(outValue);
		}

		[TestMethod]
		public void GetEnumValidField()
		{
			string fieldName = "CurrentStatusEffect";
			StatusEffect correctValue = _validJsonDataSource.CurrentStatusEffect;
			Assert.AreEqual(correctValue, _validTestObject.GetEnumField<StatusEffect>(fieldName));
			StatusEffect outValue;
			Assert.IsTrue(_validTestObject.TryGetEnumField<StatusEffect>(fieldName, out outValue));
			Assert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetEnumInvalidField()
		{
			string fieldName = "Invalid";
			StatusEffect correctValue = _validJsonDataSource.CurrentStatusEffect;
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetEnumField<StatusEffect>(fieldName));
			StatusEffect outValue;
			Assert.IsFalse(_validTestObject.TryGetEnumField<StatusEffect>(fieldName, out outValue));
			Assert.AreEqual(default, outValue);
		}

		[TestMethod]
		public void GetEnumArrayValidField()
		{
			string fieldName = "ImmuneStatusEffects";
			StatusEffect[] correctValue = [.. _validJsonDataSource.ImmuneStatusEffects];
			StatusEffect[]? outValue;
			Assert.IsTrue(_validTestObject.TryGetEnumArrayField<StatusEffect>(fieldName, out outValue));
			CollectionAssert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetEnumArrayInvalidField()
		{
			string fieldName = "Invalid";
			StatusEffect[] correctValu = [.. _validJsonDataSource.ImmuneStatusEffects];
			StatusEffect[]? outValue;
			Assert.IsFalse(_validTestObject.TryGetEnumArrayField(fieldName, out outValue));
			Assert.IsNull(outValue);
			CollectionAssert.AreNotEqual(correctValu, outValue);
		}

		[TestMethod]
		public void GetBoolValidField()
		{
			string fieldName = "IsAlive";
			bool correctValue = _validJsonDataSource.IsAlive;
			Assert.AreEqual(correctValue, _validTestObject.GetBoolField(fieldName));
			bool outValue;
			Assert.IsTrue(_validTestObject.TryGetBoolField(fieldName, out outValue));
			Assert.AreEqual(correctValue, outValue);
		}

		[TestMethod]
		public void GetBoolInvalidField()
		{
			string fieldName = "Invalid";
			Assert.ThrowsExactly<JsonException>(() => _validTestObject.GetBoolField(fieldName));
			bool outValue = false;
			Assert.IsFalse(_validTestObject.TryGetBoolField(fieldName, out outValue));
			Assert.IsFalse(outValue);
		}

		[TestMethod]
		public void ContainsField()
		{
			string fieldName1 = "Name";
			string fieldName2 = "Invalid";
			Assert.IsTrue(_validTestObject.ContainsField(fieldName1));
			Assert.IsFalse(_validTestObject.ContainsField(fieldName2));
		}

		[TestMethod]
		public void SetIntegerField()
		{
			string fieldName = "Hp";
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			int setValue = 60;
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			Assert.AreEqual(setValue, _validTestObject.GetIntegerField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			Assert.AreEqual(setValue, _validTestObject.GetIntegerField(newFieldName));
		}

		[TestMethod]
		public void SetDoubleField()
		{
			string fieldName = "CriticalHitRate";
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			double setValue = 1.0;
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			Assert.AreEqual(setValue, _validTestObject.GetDoubleField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			Assert.AreEqual(setValue, _validTestObject.GetDoubleField(newFieldName));
		}

		[TestMethod]
		public void SetStringField()
		{
			string fieldName = "Name";
			string setValue = "NewValue";
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			Assert.AreEqual(setValue, _validTestObject.GetStringField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			Assert.AreEqual(setValue, _validTestObject.GetStringField(newFieldName));

			string nullFieldName = "Null";
			string? nullString = null;
			Assert.IsFalse(_validTestObject.ContainsField(nullFieldName));
			_validTestObject.AddOrSetFieldValue(nullFieldName, nullString);
			Assert.AreEqual("", _validTestObject.GetStringField(nullFieldName));
		}

		[TestMethod]
		public void SetStringArrayField()
		{
			string fieldName = "Skills";
			string[] setValue = ["SetValue1", "SetValue2", "SetValue3", "SetValue4"];
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			CollectionAssert.AreEqual(setValue, _validTestObject.GetStringArrayField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			CollectionAssert.AreEqual(setValue, _validTestObject.GetStringArrayField(newFieldName));

			// test array of strings that contains null. Null should be set as ""
			string nullFieldName = "Null";
			string?[] nullStringArray = ["String1", null, "String2"];
			string[] correctNullStringArray = ["String1", "", "String2"];
			Assert.IsFalse(_validTestObject.ContainsField(nullFieldName));
			_validTestObject.AddOrSetFieldValue(nullFieldName, nullStringArray);
			CollectionAssert.AreEqual(correctNullStringArray, _validTestObject.GetStringArrayField(nullFieldName));

			// Test for passing in an empty string array. This should still be added and contained within the object 
			// This test is very important for ModuleDescriptor 
			string emptyFieldName = "Empty";
			Assert.IsFalse(_validTestObject.ContainsField(emptyFieldName));
			string[] emptyStringArray = [];
			_validTestObject.AddOrSetFieldValue(emptyFieldName, emptyStringArray);
			Assert.IsTrue(_validTestObject.ContainsField(emptyFieldName));
			string[] returnArray = _validTestObject.GetStringArrayField(emptyFieldName);
			Assert.IsTrue(returnArray.Length == 0);
		}

		[TestMethod]
		public void SetObjectField()
		{
			string fieldName = "StrongestAttack";
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			EnemyAttack setValue = new("Ultima", 300, StatusEffect.None);
			JsonObject setValueObject = setValue.ToJsonObject();
			_validTestObject.AddOrSetFieldValue(fieldName, setValue.ToJsonObject());
			Assert.AreEqual(setValueObject, _validTestObject.GetObjectField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue.ToJsonObject());
			Assert.AreEqual(setValueObject, _validTestObject.GetObjectField(newFieldName));
		}

		[TestMethod]
		public void SetObjectArrayField()
		{
			string fieldName = "Attacks";
			EnemyAttack[] attacks =
			[
				new EnemyAttack("Attack1", 40, StatusEffect.Sleep),
				new EnemyAttack("Attack2", 100, StatusEffect.Burn)
			];
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			JsonObject[] setValue = [.. attacks.Select(x => x.ToJsonObject())];
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			CollectionAssert.AreEqual(setValue, _validTestObject.GetObjectArrayField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			CollectionAssert.AreEqual(setValue, _validTestObject.GetObjectArrayField(newFieldName));
		}

		[TestMethod]
		public void SetEnumField()
		{
			string fieldName = "CurrentStatusEffect";
			StatusEffect setValue = StatusEffect.Burn;
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			Assert.AreEqual(setValue, _validTestObject.GetEnumField<StatusEffect>(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			Assert.AreEqual(setValue, _validTestObject.GetEnumField<StatusEffect>(newFieldName));
		}

		[TestMethod]
		public void SetEnumArrayField()
		{
			string fieldName = "ImmuneStatusEffects";
			StatusEffect[] setValue = [StatusEffect.Sleep, StatusEffect.Poison, StatusEffect.Frozen, StatusEffect.Burn];
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			StatusEffect[]? outValue;
			Assert.IsTrue(_validTestObject.TryGetEnumArrayField(fieldName, out outValue));
			CollectionAssert.AreEqual(setValue, outValue);

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			StatusEffect[]? outValue2;
			Assert.IsTrue(_validTestObject.TryGetEnumArrayField(newFieldName, out outValue2));
			CollectionAssert.AreEqual(setValue, outValue2);

			// Test passing in an empty array
			string emptyFieldName = "Empty";
			StatusEffect[] setValueEmpty = [];
			Assert.IsFalse(_validTestObject.ContainsField(emptyFieldName));
			_validTestObject.AddOrSetFieldValue(emptyFieldName, setValueEmpty);
			Assert.IsTrue(_validTestObject.ContainsField(emptyFieldName));
			StatusEffect[]? outValue4;
			Assert.IsTrue(_validTestObject.TryGetEnumArrayField<StatusEffect>(emptyFieldName, out outValue4));
			CollectionAssert.AreEqual(setValueEmpty, outValue4);
		}

		[TestMethod]
		public void SetEmptyArrayField()
		{
			string emptyFieldName = "Empty";
			Assert.IsFalse(_validTestObject.ContainsField(emptyFieldName));
			string[] setValue = [];
			_validTestObject.AddOrSetFieldValue(emptyFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(emptyFieldName));
			string[] returnValue = _validTestObject.GetStringArrayField(emptyFieldName);
			CollectionAssert.AreEqual(setValue, returnValue);
			Assert.IsTrue(returnValue.Length == 0);
		}

		[TestMethod]
		public void SetBoolField()
		{
			string fieldName = "IsAlive";
			bool setValue = false;
			Assert.IsTrue(_validTestObject.ContainsField(fieldName));
			_validTestObject.AddOrSetFieldValue(fieldName, setValue);
			Assert.AreEqual(setValue, _validTestObject.GetBoolField(fieldName));

			string newFieldName = "New";
			Assert.IsFalse(_validTestObject.ContainsField(newFieldName));
			_validTestObject.AddOrSetFieldValue(newFieldName, setValue);
			Assert.IsTrue(_validTestObject.ContainsField(newFieldName));
			Assert.AreEqual(setValue, _validTestObject.GetBoolField(newFieldName));
		}

		[TestMethod]
		public void Constructor()
		{
			JsonObject obj = new();
			Assert.AreEqual(0, obj.KeyNames.Count());
		}

		[TestMethod]
		public void ToJsonString()
		{
			string stringToMatch = new(_testJsonText.ToCharArray());
			string[] lines = stringToMatch.Split([Environment.NewLine], StringSplitOptions.None);
			StringBuilder jsonStringBuilder = new();
			foreach (string line in lines)
			{
				// Utf8JsonWriter uses 2 spaces for indents, we replace them with tabs here 
				int numLeadingSpaces = line.TakeWhile(x => x == ' ').Count();
				int numLeadingTabs = numLeadingSpaces / 2;
				jsonStringBuilder.Append('\t', numLeadingTabs);
				jsonStringBuilder.AppendLine(line.Substring(numLeadingSpaces));
			}
			stringToMatch = jsonStringBuilder.ToString();

			string jsonObjectString = _validTestObject.ToJsonString();
			Assert.AreEqual(stringToMatch, jsonObjectString);
		}

		[TestMethod]
		public void ToStringEscapeCharacters()
		{
			string escapeCharacterFieldName = "Escape";
			// .NET Json serialization escapes certian characters like +,<,>,~,& for security reasons. We need to make sure those come through 
			string escapeString = "++MyTest+Test<>`&";
			StringBuilder builder = new();
			builder.AppendLine("{");
			builder.AppendLine($"\t\"{escapeCharacterFieldName}\": \"{escapeString}\"");
			builder.AppendLine("}");
			string correctString = builder.ToString();
			JsonObject jsonObject = new();
			jsonObject.AddOrSetFieldValue(escapeCharacterFieldName, escapeString);
			string jsonString = jsonObject.ToJsonString();
			Assert.AreEqual(correctString, jsonString);
		}
		[TestMethod]
		public void KeyNameCaseSensitivity()
		{
			// Legacy behavior is that we can look up keys in a case insensitive way 
			string fieldName = "Hp";
			string caseInsensitiveFieldName = "hp";
			int correctValue = _validJsonDataSource.Hp;
			Assert.AreEqual(correctValue, _validTestObject.GetIntegerField(fieldName));
			Assert.AreEqual(_validTestObject.GetIntegerField(fieldName), _validTestObject.GetIntegerField(caseInsensitiveFieldName));
			int outValue;
			int caseInsensitiveOutValue;
			Assert.IsTrue(_validTestObject.TryGetIntegerField(fieldName, out outValue));
			Assert.IsTrue(_validTestObject.TryGetIntegerField(caseInsensitiveFieldName, out caseInsensitiveOutValue));
			Assert.AreEqual(correctValue, outValue);
			Assert.AreEqual(outValue, caseInsensitiveOutValue);
		}

		private static Enemy CreateValidJsonDataSource()
		{
			Enemy enemy1 = new();
			enemy1.Name = "Monster";
			enemy1.Hp = 100;
			enemy1.CriticalHitRate = 3.0;

			// Add attacks
			enemy1.Attacks.Add(new EnemyAttack("Slash", 10));
			enemy1.Attacks.Add(new EnemyAttack("Ice Blade", 20, StatusEffect.Frozen));

			enemy1.StrongestAttack = new EnemyAttack("Ultimate", 100, StatusEffect.Poison);
			// Add skills 
			enemy1.Skills = ["Skill 1", "Skill 2"];

			// Add status effects
			enemy1.ImmuneStatusEffects.Add(StatusEffect.Frozen);
			enemy1.ImmuneStatusEffects.Add(StatusEffect.Sleep);

			enemy1.IsAlive = true;
			enemy1.CurrentStatusEffect = StatusEffect.Poison;

			return enemy1;
		}

		private JsonObject CreateValidJsonObject()
		{
			JsonObject validObject = JsonObject.Parse(_testJsonText);
			return validObject;
		}

		private static FileReference CreateTempJsonFile(string fileName, string? fileContent)
		{
			string inputFile = Path.Join(s_tempDirectory.FullName, fileName);
			FileReference inputFileReference = new(inputFile);
			File.WriteAllText(inputFile, fileContent);
			return inputFileReference;
		}

		private void MakeTestJsonTextInvalid()
		{
			_testJsonText += @"blah}blah";
		}

		private static DirectoryReference CreateTempDir()
		{
			string tempDir = Path.Join(Path.GetTempPath(), "epicgames-core-tests-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);
			return new DirectoryReference(tempDir);
		}
	}

	enum StatusEffect
	{
		Frozen,
		Poison,
		Burn,
		Sleep,
		None
	}

	class Enemy
	{
		public string Name { get; set; } = "";
		public int Hp { get; set; } = 0;
		public double CriticalHitRate { get; set; } = 0.0;
		public List<EnemyAttack> Attacks { get; set; } = [];
		public EnemyAttack StrongestAttack { get; set; } = null!;
		public List<StatusEffect> ImmuneStatusEffects { get; set; } = [];
		public StatusEffect CurrentStatusEffect { get; set; } = StatusEffect.None;
		public List<string> Skills { get; set; } = [];
		public bool IsAlive { get; set; } = false;
	}

	class EnemyAttack(string name, int damage, StatusEffect effect = StatusEffect.None)
	{
		public JsonObject ToJsonObject()
		{
			JsonObject obj = new();
			obj.AddOrSetFieldValue("Name", Name);
			obj.AddOrSetFieldValue("Damage", Damage);
			obj.AddOrSetFieldValue<StatusEffect>("Effect", Effect);
			return obj;
		}
		public string Name { get; set; } = name;
		public int Damage { get; set; } = damage;
		public StatusEffect Effect { get; set; } = effect;
	}
}
