// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.TestTools.UnitTesting;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using OpenTracing;
using System.Threading.Tasks;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class JsonTracerTests
	{
		[TestMethod]
		public void Run()
		{
			JsonTracer tracer = new();
			using (IScope scope1 = tracer.BuildSpan("bogusOp1").StartActive())
			{
			}
			using (IScope scope2 = tracer.BuildSpan("bogusOp2").StartActive())
			{
			}

			Assert.AreEqual(2, tracer.GetFinishedSpans().Count);
		}

		[TestMethod]
		public void SpansWithTags()
		{
			JsonTracer tracer = new();
			using (IScope scope1 = tracer.BuildSpan("bogusOp1").WithTag("age", 94).StartActive())
			{
			}
			using (IScope scope2 = tracer.BuildSpan("bogusOp1").WithTag("foo", "bar").StartActive())
			{
			}

			List<JsonTracerSpan> spans = tracer.GetFinishedSpans();

			Assert.AreEqual(2, spans.Count);
			Assert.AreEqual("bogusOp1", spans[0].OperationName);
		}

		[TestMethod]
		public void SerializeToJson()
		{
			JsonTracer tracer = new();
			using (IScope scope1 = tracer.BuildSpan("bogusOp1").WithTag("age", 94).WithTag("foo", "bar").StartActive())
			{
			}
			using (IScope scope2 = tracer.BuildSpan("bogusOp2").WithTag("Service", "myService").WithTag("Resource", "myResource").StartActive())
			{
			}

			using StringWriter sw = new();
			using JsonWriter jw = new(sw);
			tracer.GetFinishedSpansAsJson(jw);
			byte[] data = Encoding.ASCII.GetBytes(sw.ToString());

			using (JsonDocument document = JsonDocument.Parse(data.AsMemory()))
			{
				JsonElement root = document.RootElement;
				JsonElement spansElement = root.GetProperty("Spans");
				List<JsonElement> spansList = [.. spansElement.EnumerateArray()];
				Console.WriteLine(spansList[0]);
				Assert.AreEqual("bogusOp1", spansList[0].GetProperty("Name").GetString());
				Assert.IsNotNull(spansList[0].GetProperty("StartTime").GetDateTime());
				Assert.IsNotNull(spansList[0].GetProperty("FinishTime").GetDateTime());
				Assert.AreEqual("myService", spansList[1].GetProperty("Service").GetString());
				Assert.AreEqual("myResource", spansList[1].GetProperty("Resource").GetString());
			}
		}

		[TestMethod]
		public void SpansWithParallelFor()
		{
			JsonTracer tracer = new();
			using (IScope parentScope = tracer.BuildSpan("Parent").StartActive())
			{
				Parallel.For(0, 100, index =>
				{
					using (IScope childScope = tracer.BuildSpan("Child").StartActive())
					{
						Assert.AreEqual(tracer.ActiveSpan, childScope.Span);
						Assert.AreEqual(((JsonTracerSpan)childScope.Span).ParentId, ((JsonTracerSpan)parentScope.Span).Context.SpanId);
					}
					Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
				});
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
			}
			Assert.AreEqual(null, tracer.ActiveSpan);
		}

		private static async Task<bool> AsyncSpanTestAsync(JsonTracer tracer, IScope parentScope)
		{
			for (int index = 0; index < 100; index++)
			{
				using (IScope asyncChildScope = tracer.BuildSpan("AsyncChild").StartActive())
				{
					await Task.Run(() =>
					{
						using (IScope taskChildScope = tracer.BuildSpan("TaskChild").StartActive())
						{
							Assert.AreEqual(tracer.ActiveSpan, taskChildScope.Span);
							Assert.AreEqual(((JsonTracerSpan)taskChildScope.Span).ParentId, ((JsonTracerSpan)asyncChildScope.Span).Context.SpanId);
							return true;
						}
					});
					Assert.AreEqual(tracer.ActiveSpan, asyncChildScope.Span);
					Assert.AreEqual(((JsonTracerSpan)asyncChildScope.Span).ParentId, ((JsonTracerSpan)parentScope.Span).Context.SpanId);
				}
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
			}
			return true;
		}

		[TestMethod]
		public void SpansWithAsync()
		{
			JsonTracer tracer = new();
			using (IScope parentScope = tracer.BuildSpan("Parent").StartActive())
			{
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
				AsyncSpanTestAsync(tracer, parentScope).Wait();
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
			}

			Assert.AreEqual(null, tracer.ActiveSpan);
		}

		private static async Task ForEachAsyncSpanTestAsync(JsonTracer tracer, IScope parentScope)
		{
			int[] values = [.. Enumerable.Range(0, 10000)];
			await Parallel.ForEachAsync(values, new ParallelOptions { MaxDegreeOfParallelism = 10000 }, async (i, token) =>
			{
				using (IScope childScope = tracer.BuildSpan("Child").StartActive())
				{
					Assert.AreEqual(tracer.ActiveSpan, childScope.Span);
					Assert.AreEqual(((JsonTracerSpan)childScope.Span).ParentId, ((JsonTracerSpan)parentScope.Span).Context.SpanId);
					await Task.Delay(10, token);
				}
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
			});
		}

		[TestMethod]
		public void SpansWithParallelForEachAsync()
		{
			JsonTracer tracer = new();
			using (IScope parentScope = tracer.BuildSpan("Parent").StartActive())
			{
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
				ForEachAsyncSpanTestAsync(tracer, parentScope).Wait();
				Assert.AreEqual(tracer.ActiveSpan, parentScope.Span);
			}

			Assert.AreEqual(null, tracer.ActiveSpan);
		}
	}
}
