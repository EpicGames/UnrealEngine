// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public class LogEventParserTests
{
	[TestMethod]
	public void TestAllMatchersAreTried()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([highest, lowest]);
			parser.WriteLine(LineToLog);
		}

		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);
	}

	[TestMethod]
	public void TestHighestPriorityMatcherWins()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([highest, lowest]);
			parser.WriteLine(LineToLog);
		}

		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LineToLog, capture.Events[0].Message);
		Assert.AreEqual(highest.Level, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestTogglingOneMatcherOnce()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine(LineToLog);

		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);

		parser.WriteLine("<-- logeventparser suspend highest -->");
		parser.WriteLine(LineToLog);

		CollectionAssert.AreEquivalent<ILogEventMatcher>(parser.EnabledMatchers, [lowest], ReferenceEqualityComparer.Instance);
		// Highest should not have gotten another call.
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(2, lowest.MatchCalls);

		Assert.AreEqual(2, capture.Events.Count);

		Assert.AreEqual(LineToLog, capture.Events[^1].Message);
		// Lowest should have matched the event.
		Assert.AreEqual(lowest.Level, capture.Events[^1].Level);

		parser.WriteLine("<-- logeventparser resume highest -->");
		parser.WriteLine(LineToLog);

		// Highest should have gotten another call after being restored.
		Assert.AreEqual(2, highest.MatchCalls);
		Assert.AreEqual(3, lowest.MatchCalls);

		Assert.AreEqual(3, capture.Events.Count);

		Assert.AreEqual(LineToLog, capture.Events[^1].Message);
		// Highest should have matched the event.
		Assert.AreEqual(highest.Level, capture.Events[^1].Level);
	}

	[TestMethod]
	public void TestTogglingAllMatchersIndividuallyOnce()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser suspend highest -->");
		parser.WriteLine("<-- logeventparser suspend lowest -->");
		parser.WriteLine(LineToLog);

		Assert.AreEqual(0, parser.EnabledMatchers.Count());
		// Neither should have been called.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		// We should still have one event.
		Assert.AreEqual(1, capture.Events.Count);
		// The event should have a zero ID.
		Assert.AreEqual(new EventId(0), capture.Events[0].Id);
	}

	[TestMethod]
	public void TestTogglingAllWildcardAndReenablingOneMatcher()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser suspend * -->");
		parser.WriteLine(LineToLog);

		// Neither should have been called.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		// We should still have one event.
		Assert.AreEqual(1, capture.Events.Count);
		// The event should have a zero ID.
		Assert.AreEqual(new EventId(0), capture.Events[0].Id);

		parser.WriteLine("<-- logeventparser resume lowest -->");
		parser.WriteLine(LineToLog);

		// Lowest should have been called.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);

		Assert.AreEqual(LineToLog, capture.Events[^1].Message);
		// Lowest should have matched the event.
		Assert.AreEqual(lowest.Level, capture.Events[^1].Level);
	}

	[TestMethod]
	public void TestPoppingUndoesToggles()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser suspend * -->");
		parser.WriteLine("<-- logeventparser resume lowest -->");
		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine(LineToLog);

		// Neither should have been called after we popped the resume.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine(LineToLog);

		// Both should have been called after the pop.
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);
	}

	[TestMethod]
	public void TestLayersStackCorrectly()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser suspend * -->");
		parser.WriteLine("<-- logeventparser suspend * -->");
		parser.WriteLine("<-- logeventparser resume lowest -->");
		parser.WriteLine(LineToLog);

		// Neither should have been called after we only resumed once but suspended twice.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine(LineToLog);

		// Neither should have been called as we should still have a wildcard disable as the top layer.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine(LineToLog);

		// Now we've popped the disable, the matchers should be enabled again.
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);
	}

	[TestMethod]
	public void TestAllowsEnablingBeforeDisabling()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser resume lowest -->");
		parser.WriteLine("<-- logeventparser suspend * -->");
		parser.WriteLine(LineToLog);

		// Lowest should be enabled because it was explicitly enabled as many times as the wildcard was disabled.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);

		parser.WriteLine("<-- logeventparser pop -->");
		parser.WriteLine(LineToLog);

		// Both should now be enabled.
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(2, lowest.MatchCalls);
	}

	[TestMethod]
	public void TestMismatchedLayerNamesWarns()
	{
		LoggerCapture capture = new();
		TestMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine("<-- logeventparser layer=thing1 suspend * -->");
		parser.WriteLine("<-- logeventparser layer=thing2 suspend * -->");
		parser.WriteLine("<-- logeventparser layer=thing1 pop -->");

		// Neither should have been called.
		Assert.AreEqual(0, highest.MatchCalls);
		Assert.AreEqual(0, lowest.MatchCalls);

		// We should have one event, the warning issued.
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual("[LogEventParser] Given layer 'thing1' is not the same as last pushed layer 'thing2', please check your logic.", capture.Events[^1].Message);
		Assert.AreEqual(LogLevel.Warning, capture.Events[^1].Level);

		parser.WriteLine("<-- logeventparser pop -->");

		// An anonymous pop of a non-anonymous layer is fine and should not warn again.
		Assert.AreEqual(1, capture.Events.Count);

		parser.WriteLine(LineToLog);

		// The pop that warned should still have gone through, so matchers should be enabled again.
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);
	}

	[TestMethod]
	public void TestHangingLines()
	{
		LoggerCapture capture = new();
		TestHangingLinesMatcher highest = new(LogEventPriority.Highest, LogLevel.Critical);
		TestMatcher lowest = new(LogEventPriority.Lowest, LogLevel.Debug, 3);

		using LogEventParser parser = new(capture);
		parser.AddMatchers([highest, lowest]);

		parser.WriteLine(LineToLog);
		Assert.AreEqual(1, highest.MatchCalls);
		Assert.AreEqual(1, lowest.MatchCalls);
		// 0 because of lowest 3 line count requirement
		Assert.AreEqual(0, capture.Events.Count);

		parser.WriteLine("\t" + LineToLog);
		Assert.AreEqual(2, highest.MatchCalls);
		Assert.AreEqual(2, lowest.MatchCalls);
		// 0 because of lowest 3 line count requirement
		Assert.AreEqual(0, capture.Events.Count);

		parser.WriteLine("\t" + LineToLog);
		Assert.AreEqual(3, highest.MatchCalls);
		Assert.AreEqual(3, lowest.MatchCalls);
		// 3 because lowest 3 line count requirement is met
		Assert.AreEqual(3, capture.Events.Count);

		parser.WriteLine("\t" + LineToLog);
		Assert.AreEqual(4, highest.MatchCalls);
		// 3 because highest it incomplete hanging
		Assert.AreEqual(3, lowest.MatchCalls);
		// 4 because of highest was incomplete hanging
		Assert.AreEqual(4, capture.Events.Count);

		parser.WriteLine(LineToLog);
		// 6 because highest gets called again once complete
		Assert.AreEqual(6, highest.MatchCalls);
		Assert.AreEqual(4, lowest.MatchCalls);
		// 4 because of lowest 3 line count requirement
		Assert.AreEqual(4, capture.Events.Count);

		parser.WriteLine(LineToLog);
		Assert.AreEqual(7, highest.MatchCalls);
		Assert.AreEqual(5, lowest.MatchCalls);
		// 4 because of lowest 3 line count requirement
		Assert.AreEqual(4, capture.Events.Count);

		parser.WriteLine(LineToLog);
		Assert.AreEqual(9, highest.MatchCalls);
		Assert.AreEqual(7, lowest.MatchCalls);
		// 5 because lowest 3 line count requirement is met and is not hanging
		Assert.AreEqual(5, capture.Events.Count);

		parser.Flush();
		Assert.AreEqual(11, highest.MatchCalls);
		Assert.AreEqual(9, lowest.MatchCalls);
		// 7 because flushed
		Assert.AreEqual(7, capture.Events.Count);
	}

	#region Ignore Pattern Tests

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_EngineDir()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*test.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		Assert.AreEqual(1, files.Count);
		Assert.IsTrue(files[0].FullName.Contains("Engine", StringComparison.Ordinal));
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_ProjectDir()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*engine.*");
		workspace.CreatePatternFile("MyGame/Build/Horde/IgnorePatterns.txt", ".*game.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		Assert.AreEqual(2, files.Count);
		Assert.IsTrue(files.Any(f => f.FullName.Contains("Engine", StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains("MyGame", StringComparison.Ordinal)));
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_RestrictedAndPlatforms()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*engine.*");
		workspace.CreatePatternFile("Engine/Restricted/ConsoleX/Build/Horde/IgnorePatterns.txt", ".*consolex.*");
		workspace.CreatePatternFile("MyGame/Platforms/PS5/Build/Horde/IgnorePatterns.txt", ".*ps5.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		Assert.AreEqual(3, files.Count);
		Assert.IsTrue(files.Any(f => f.FullName.Contains(Path.Combine("Engine", "Build"), StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains("ConsoleX", StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains("PS5", StringComparison.Ordinal)));
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_ProjectRestrictedSubdirs()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("MyGame/Build/Horde/IgnorePatterns.txt", ".*game.*");
		workspace.CreatePatternFile("MyGame/Restricted/Internal/Build/Horde/IgnorePatterns.txt", ".*internal.*");
		workspace.CreatePatternFile("MyGame/Platforms/Switch/Build/Horde/IgnorePatterns.txt", ".*switch.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		Assert.AreEqual(3, files.Count);
		Assert.IsTrue(files.Any(f => f.FullName.Contains(Path.Combine("MyGame", "Build"), StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains("Internal", StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains("Switch", StringComparison.Ordinal)));
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_EmptyWorkspace()
	{
		using TempWorkspace workspace = new();

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		Assert.AreEqual(0, files.Count);
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_EngineDirNotDuplicated()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*test.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir);

		// Engine should appear only once even though it's both the explicit engine dir and a top-level dir
		int engineCount = files.Count(f => f.FullName.Contains(Path.Combine("Engine", "Build", "Horde", "IgnorePatterns.txt"), StringComparison.Ordinal));
		Assert.AreEqual(1, engineCount);
	}

	[TestMethod]
	public void TestDiscoverIgnorePatternFiles_CustomEnginePath()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("CustomEngine/Build/Horde/IgnorePatterns.txt", ".*custom.*");
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*default.*");

		List<FileReference> files = LogEventParser.DiscoverIgnorePatternFiles(workspace.Dir, "CustomEngine");

		// Both should be discovered - CustomEngine as the engine dir, Engine as a top-level dir
		Assert.AreEqual(2, files.Count);
		Assert.IsTrue(files.Any(f => f.FullName.Contains("CustomEngine", StringComparison.Ordinal)));
		Assert.IsTrue(files.Any(f => f.FullName.Contains(Path.Combine("Engine", "Build"), StringComparison.Ordinal)));
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_ValidPatternsAsync()
	{
		using TempWorkspace workspace = new();
		FileReference file = workspace.CreatePatternFile("patterns.txt",
			".*warning.*\n.*error X4000.*\n.*LNK4099.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsAsync(file);

		Assert.AreEqual(3, parser.IgnorePatterns.Count);
		// Verify source file is tracked on each entry
		foreach (IgnorePatternEntry entry in parser.IgnorePatterns)
		{
			Assert.AreEqual(file.FullName, entry.SourceFile);
		}
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_CommentsAndBlankLinesAsync()
	{
		using TempWorkspace workspace = new();
		FileReference file = workspace.CreatePatternFile("patterns.txt",
			"# This is a comment\n" +
			"\n" +
			".*warning.*\n" +
			"   # Indented comment\n" +
			"  \n" +
			".*error.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsAsync(file);

		// Only non-comment, non-blank lines should be loaded
		Assert.AreEqual(2, parser.IgnorePatterns.Count);
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_InvalidRegexAsync()
	{
		using TempWorkspace workspace = new();
		FileReference file = workspace.CreatePatternFile("patterns.txt",
			".*valid_pattern.*\n" +
			"[invalid_unclosed_bracket\n" +
			".*another_valid.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsAsync(file);

		// Invalid regex should be skipped, valid ones still loaded
		Assert.AreEqual(2, parser.IgnorePatterns.Count);
		// Warning should have been logged about the invalid pattern
		Assert.IsTrue(capture.Events.Any(e => e.Level == LogLevel.Warning));
	}

	[TestMethod]
	public void TestIgnorePattern_SuppressesMatchedError()
	{
		LoggerCapture capture = new();
		TestMatcher matcher = new(LogEventPriority.Highest, LogLevel.Error);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([matcher]);
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*hello.*", RegexOptions.Compiled), "test"));
			parser.WriteLine(LineToLog); // LineToLog is "hello"
		}

		// The error should be suppressed - line is logged at Information level (not Error)
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_DoesNotSuppressNonMatchingLine()
	{
		LoggerCapture capture = new();
		TestMatcher matcher = new(LogEventPriority.Highest, LogLevel.Error);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([matcher]);
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*goodbye.*", RegexOptions.Compiled), "test"));
			parser.WriteLine(LineToLog); // LineToLog is "hello", doesn't match ".*goodbye.*"
		}

		// The error should NOT be suppressed since the ignore pattern doesn't match
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Error, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_MultiplePatterns()
	{
		LoggerCapture capture = new();
		TestMatcher matcher = new(LogEventPriority.Highest, LogLevel.Error);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([matcher]);
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*unrelated.*", RegexOptions.Compiled), "test"));
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*hello.*", RegexOptions.Compiled), "test"));
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*other.*", RegexOptions.Compiled), "test"));
			parser.WriteLine(LineToLog);
		}

		// Second pattern should match, suppressing the error
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_StructuredProperties()
	{
		LoggerCapture capture = new();
		TestMatcher matcher = new(LogEventPriority.Highest, LogLevel.Error);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([matcher]);
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*hello.*", RegexOptions.Compiled), "/workspace/Engine/Build/Horde/IgnorePatterns.txt"));
			parser.WriteLine(LineToLog);
		}

		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);

		// Verify IgnoredLine LogValue with structured properties
		Assert.IsNotNull(capture.Events[0].Properties);
		KeyValuePair<string, object?>? ignoredLineProperty = capture.Events[0].Properties?.FirstOrDefault(p => p.Key == "IgnoredLine");
		Assert.IsNotNull(ignoredLineProperty);
		LogValue ignoredLineValue = (LogValue)ignoredLineProperty.Value.Value!;
		Assert.AreEqual(LogValueType.IgnoredLine, ignoredLineValue.Type);
		Assert.IsNotNull(ignoredLineValue.Properties);
		Assert.AreEqual("/workspace/Engine/Build/Horde/IgnorePatterns.txt", ignoredLineValue.Properties[new Utf8String("ignorePatternFile")]?.ToString());
		Assert.AreEqual(".*hello.*", ignoredLineValue.Properties[new Utf8String("ignorePattern")]?.ToString());
		Assert.AreEqual("Error", ignoredLineValue.Properties[new Utf8String("originalLevel")]?.ToString());
	}

	[TestMethod]
	public void TestIgnorePattern_RegexTimeoutHandled()
	{
		LoggerCapture capture = new();
		TestMatcher matcher = new(LogEventPriority.Highest, LogLevel.Error);

		using (LogEventParser parser = new(capture))
		{
			parser.AddMatchers([matcher]);
			// Pathological pattern with very short timeout to trigger RegexMatchTimeoutException
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(@"(a+)+$", RegexOptions.None, TimeSpan.FromTicks(1)), "test"));
			// String that causes catastrophic backtracking: many 'a's followed by non-'a'
			parser.WriteLine(new string('a', 50) + "!");
		}

		// The error should NOT be suppressed because the pattern either timed out or didn't match.
		// In either case, the line should appear as an error event.
		Assert.IsTrue(capture.Events.Any(e => e.Level == LogLevel.Error));
	}

	[TestMethod]
	public async Task TestReadIgnorePatternsFromWorkspaceAsync()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*engine_warning.*");
		workspace.CreatePatternFile("MyGame/Build/Horde/IgnorePatterns.txt", ".*game_warning.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsFromWorkspaceAsync(workspace.Dir);

		Assert.AreEqual(2, parser.IgnorePatterns.Count);
	}

	[TestMethod]
	public async Task TestReadIgnorePatternsFromWorkspace_WithRestrictedDirsAsync()
	{
		using TempWorkspace workspace = new();
		workspace.CreatePatternFile("Engine/Build/Horde/IgnorePatterns.txt", ".*engine.*");
		workspace.CreatePatternFile("Engine/Restricted/Internal/Build/Horde/IgnorePatterns.txt", ".*internal.*");
		workspace.CreatePatternFile("MyGame/Build/Horde/IgnorePatterns.txt", ".*game.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsFromWorkspaceAsync(workspace.Dir);

		Assert.AreEqual(3, parser.IgnorePatterns.Count);
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_MissingAdditionalFilesAsync()
	{
		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		FileReference missingFile = new(Path.Combine(Path.GetTempPath(), Guid.NewGuid().ToString("N"), "nonexistent.txt"));
		await parser.ReadIgnorePatternsAsync([missingFile]);

		// No patterns should be loaded
		Assert.AreEqual(0, parser.IgnorePatterns.Count);
		// Warning should have been logged about missing file
		Assert.IsTrue(capture.Events.Any(e => e.Level == LogLevel.Warning));
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_AdditionalFilesLoadedAsync()
	{
		using TempWorkspace workspace = new();
		FileReference file1 = workspace.CreatePatternFile("extra/patterns1.txt", ".*pattern1.*");
		FileReference file2 = workspace.CreatePatternFile("extra/patterns2.txt", ".*pattern2.*\n.*pattern3.*");

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsAsync([file1, file2]);

		Assert.AreEqual(3, parser.IgnorePatterns.Count);
	}

	[TestMethod]
	public async Task TestReadIgnorePatterns_MixedValidAndMissingAdditionalFilesAsync()
	{
		using TempWorkspace workspace = new();
		FileReference validFile = workspace.CreatePatternFile("extra/patterns.txt", ".*valid.*");
		FileReference missingFile = new(Path.Combine(workspace.Dir.FullName, "nonexistent.txt"));

		LoggerCapture capture = new();
		using LogEventParser parser = new(capture);
		await parser.ReadIgnorePatternsAsync([validFile, missingFile]);

		// Valid file patterns loaded, missing file logged as warning
		Assert.AreEqual(1, parser.IgnorePatterns.Count);
		Assert.IsTrue(capture.Events.Any(e => e.Level == LogLevel.Warning));
	}

	#endregion

	#region Structured JSON Ignore Pattern Tests

	// Minimal valid structured JSON log events for testing
	private const string JsonErrorLine = "{\"level\":\"Error\",\"message\":\"LogCompile: Error: C1234: undefined symbol 'foo'\",\"properties\":{\"id\":100},\"format\":\"{message}\"}";
	private const string JsonWarningLine = "{\"level\":\"Warning\",\"message\":\"LogCompile: Warning: variable 'x' is unused\",\"properties\":{\"id\":100},\"format\":\"{message}\"}";
	private const string JsonInfoLine = "{\"level\":\"Information\",\"message\":\"LogCompile: Display: Build succeeded\",\"properties\":{},\"format\":\"{message}\"}";

	[TestMethod]
	public void TestIgnorePattern_SuppressesStructuredJsonError()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*C1234.*", RegexOptions.Compiled), "test.txt"));
			parser.WriteLine(JsonErrorLine);
		}

		// The error should be suppressed - logged at Information instead of Error
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_SuppressesStructuredJsonWarning()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*variable.*unused.*", RegexOptions.Compiled), "test.txt"));
			parser.WriteLine(JsonWarningLine);
		}

		// The warning should be suppressed
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_DoesNotSuppressNonMatchingStructuredJsonError()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*unrelated_pattern.*", RegexOptions.Compiled), "test.txt"));
			parser.WriteLine(JsonErrorLine);
		}

		// The error should NOT be suppressed since the pattern doesn't match
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Error, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_DoesNotCheckStructuredJsonInformationEvents()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			// Pattern matches the info message but should NOT be checked (optimization: only Warning+ checked)
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*Build succeeded.*", RegexOptions.Compiled), "test.txt"));
			parser.WriteLine(JsonInfoLine);
		}

		// Information events pass through without ignore pattern checking
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_StructuredJsonViaWriteData()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*C1234.*", RegexOptions.Compiled), "test.txt"));
			// WriteData goes through AddLine path (byte-based), not WriteLine
			byte[] data = System.Text.Encoding.UTF8.GetBytes(JsonErrorLine + "\n");
			parser.WriteData(data);
		}

		// The error should be suppressed through the WriteData/AddLine path too
		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);
	}

	[TestMethod]
	public void TestIgnorePattern_StructuredJsonPreservesIgnoreMetadata()
	{
		LoggerCapture capture = new();

		using (LogEventParser parser = new(capture))
		{
			parser.IgnorePatterns.Add(new IgnorePatternEntry(new Regex(".*C1234.*", RegexOptions.Compiled), "/workspace/Engine/Build/Horde/IgnorePatterns.txt"));
			parser.WriteLine(JsonErrorLine);
		}

		Assert.AreEqual(1, capture.Events.Count);
		Assert.AreEqual(LogLevel.Information, capture.Events[0].Level);

		// Verify IgnoredLine LogValue with structured properties
		Assert.IsNotNull(capture.Events[0].Properties);
		KeyValuePair<string, object?>? ignoredLineProperty = capture.Events[0].Properties?.FirstOrDefault(p => p.Key == "IgnoredLine");
		Assert.IsNotNull(ignoredLineProperty);
		LogValue ignoredLineValue = (LogValue)ignoredLineProperty.Value.Value!;
		Assert.AreEqual(LogValueType.IgnoredLine, ignoredLineValue.Type);
		Assert.IsNotNull(ignoredLineValue.Properties);
		Assert.AreEqual("/workspace/Engine/Build/Horde/IgnorePatterns.txt", ignoredLineValue.Properties[new Utf8String("ignorePatternFile")]?.ToString());
		Assert.AreEqual(".*C1234.*", ignoredLineValue.Properties[new Utf8String("ignorePattern")]?.ToString());
		Assert.AreEqual("Error", ignoredLineValue.Properties[new Utf8String("originalLevel")]?.ToString());
	}

	#endregion

	private const string LogLineProperty = "LogLine";
	private const string LineToLog = "hello";

	private class LoggerCapture : ILogger
	{
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null!;

		public bool IsEnabled(LogLevel logLevel) => true;

		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			LogEvent logEvent = LogEvent.Read(JsonLogEvent.FromLoggerState(logLevel, eventId, state, exception, formatter).Data.Span);
			if (logEvent.Message.StartsWith("[LogEventParser]", StringComparison.Ordinal) && logLevel == LogLevel.Trace)
			{
				return;
			}

			KeyValuePair<string, object?>[] items = [new KeyValuePair<string, object?>(LogLineProperty, _logLineIndex)];
			logEvent.Properties = (logEvent.Properties == null) ? items : Enumerable.Concat(logEvent.Properties, items);
			Events.Add(logEvent);

			_logLineIndex++;
		}

		public List<LogEvent> Events { get; } = [];

		private int _logLineIndex;
	}

	private class TestMatcher(LogEventPriority priority, LogLevel level, int requiredLineCount = 1) : ILogEventMatcher
	{
		public LogEventMatch? Match(ILogCursor cursor)
		{
			++MatchCalls;

			string? line = cursor.CurrentLine;
			Assert.IsNotNull(line);
			for (int index = 1; index < requiredLineCount; ++index)
			{
				_ = cursor[index];
			}

			LogEventBuilder builder = new(cursor);
			return builder.ToMatch(Priority, Level, new EventId(999));
		}

		public LogEventPriority Priority { get; } = priority;
		public LogLevel Level { get; } = level;
		public int MatchCalls { get; private set; } = 0;

		public string GetMatcherName() => Priority.ToString();
	}

	private class TestHangingLinesMatcher(LogEventPriority priority, LogLevel level, int requiredLineCount = 1) : ILogEventMatcher
	{
		public LogEventMatch? Match(ILogCursor cursor)
		{
			++MatchCalls;

			string? line = cursor.CurrentLine;
			Assert.IsNotNull(line);
			for (int index = 1; index < requiredLineCount; ++index)
			{
				_ = cursor[index];
			}

			LogEventBuilder builder = new(cursor);
			return builder.ToMatchWithHangingLines(Priority, Level, new EventId(999));
		}

		public LogEventPriority Priority { get; } = priority;
		public LogLevel Level { get; } = level;
		public int MatchCalls { get; private set; } = 0;

		public string GetMatcherName() => Priority.ToString();
	}

	/// <summary>
	/// Helper class for creating temporary workspace directory structures for testing
	/// </summary>
	private class TempWorkspace : IDisposable
	{
		public DirectoryReference Dir { get; }

		public TempWorkspace()
		{
			Dir = new DirectoryReference(Path.Combine(Path.GetTempPath(), "HordeTest_" + Guid.NewGuid().ToString("N")));
			Directory.CreateDirectory(Dir.FullName);
		}

		public FileReference CreatePatternFile(string relativePath, string contents)
		{
			string fullPath = Path.Combine(Dir.FullName, relativePath);
			Directory.CreateDirectory(Path.GetDirectoryName(fullPath)!);
			File.WriteAllText(fullPath, contents);
			return new FileReference(fullPath);
		}

		public void Dispose()
		{
			try
			{
				Directory.Delete(Dir.FullName, true);
			}
			catch
			{
				// Best effort cleanup
			}
		}
	}
}
