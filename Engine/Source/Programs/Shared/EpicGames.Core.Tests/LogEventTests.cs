// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.Text;
using System.Text.Json;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class LogEventTests
	{
		class TestClass
		{
			public int Foo { get; set; }
			public string? Bar { get; set; }

			public override string ToString()
				=> $"TestClass({Foo},{Bar})";
		}

		[TestMethod]
		public void ArgumentTests()
		{
			CaptureLogger logger = new();
			logger.LogInformation("Test {Value} {@Value}", new TestClass { Foo = 123, Bar = "456" }, new TestClass { Foo = 789, Bar = "hello" });

			LogEvent logEvent = logger.Events[0];
			logEvent.Time = new DateTime(2024, 1, 1);

			Assert.AreEqual("Test TestClass(123,456) {\"foo\":789,\"bar\":\"hello\"}", logEvent.ToString());
		}

		[TestMethod]
		public void RoundTripTests()
		{
			DateTime time = new(2024, 1, 1);
			LogEvent ev = new(time, LogLevel.Information, default, "hello", null, [KeyValuePair.Create<string, object?>("foo", new LogValue(new Utf8String("type"), "text"))], null);

			JsonLogEvent jev = JsonLogEvent.Parse(ev.ToJsonBytes());
			string jevs = jev.ToString();
			Assert.AreEqual("{\"time\":\"2024-01-01T00:00:00\",\"level\":\"Information\",\"message\":\"hello\",\"properties\":{\"foo\":{\"$type\":\"type\",\"$text\":\"text\"}}}", jevs);

			LogEvent ev2 = LogEvent.Read(jev.Data.Span);
			Assert.AreEqual(ev.ToString(), ev2.ToString());

			JsonLogEvent jev2 = JsonLogEvent.Parse(ev2.ToJsonBytes());
			string jevs2 = jev2.ToString();
			Assert.AreEqual(jevs, jevs2);
		}
		
		[TestMethod]
		[DataRow(5)]   // splits inside "message" (top-level property name)
		[DataRow(42)]  // splits inside "properties" (top-level property name)
		[DataRow(52)]  // splits inside "Foo" (nested property in ReadProperties)
		[DataRow(60)]  // splits inside "$type" (nested property in ReadObjectPropertyValue)
		[DataRow(77)]  // splits inside "$text" (nested property in ReadObjectPropertyValue)
		[DataRow(92)]  // splits inside "count" (nested property in ReadProperties)
		public void ReadMultiSegment(int splitPoint)
		{
			string json = "{\"message\":\"hello\",\"format\":\"{Foo}\",\"properties\":{\"Foo\":{\"$type\":\"Symbol\",\"$text\":\"bar\"},\"count\":42}}";
			byte[] bytes = Encoding.UTF8.GetBytes(json);

			// Reference: deserialize from contiguous span (known good)
			LogEvent expected = LogEvent.Read(bytes);

			// Test: deserialize from multi-segment sequence (exercises TryReadNextPropertyName across segment boundaries)
			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);
			Utf8JsonReader reader = new(sequence);
			reader.Read();
			LogEvent actual = LogEvent.Read(ref reader);

			Assert.AreEqual(Encoding.UTF8.GetString(expected.ToJsonBytes()), Encoding.UTF8.GetString(actual.ToJsonBytes()));
		}

		[TestMethod]
		[DataRow(63)]  // splits \n escape in "Foo" value (between \ and n)
		[DataRow(83)]  // splits first \" escape in "Bar" value (between \ and ")
		[DataRow(87)]  // splits second \" escape in "Bar" value (between \ and ")
		public void ReadMultiSegmentEscapedStrings(int splitPoint)
		{
			// String values with JSON escape sequences (\n, \") — exercises reader.GetString()! at line 352
			string json = """{"message":"hello","format":"{Foo}","properties":{"Foo":"line1\nline2","Bar":"say \"hi\""}}""";
			byte[] bytes = Encoding.UTF8.GetBytes(json);

			LogEvent expected = LogEvent.Read(bytes);

			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);
			Utf8JsonReader reader = new(sequence);
			reader.Read();
			LogEvent actual = LogEvent.Read(ref reader);

			Assert.AreEqual(Encoding.UTF8.GetString(expected.ToJsonBytes()), Encoding.UTF8.GetString(actual.ToJsonBytes()));
		}

		[TestMethod]
		[DataRow(59)]  // splits inside "$type" property name
		[DataRow(75)]  // splits inside "$text" property name
		[DataRow(87)]  // splits \\ escape in $text value (between first \ and second \)
		[DataRow(91)]  // splits second \\ escape in $text value
		public void ReadMultiSegmentEscapedObjectValue(int splitPoint)
		{
			// $text with backslash escapes — exercises ReadObjectPropertyValue + TryReadNextPropertyName
			string json = """{"message":"test","format":"{Msg}","properties":{"Msg":{"$type":"Source","$text":"path\\to\\file"}}}""";
			byte[] bytes = Encoding.UTF8.GetBytes(json);

			LogEvent expected = LogEvent.Read(bytes);

			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);
			Utf8JsonReader reader = new(sequence);
			reader.Read();
			LogEvent actual = LogEvent.Read(ref reader);

			Assert.AreEqual(Encoding.UTF8.GetString(expected.ToJsonBytes()), Encoding.UTF8.GetString(actual.ToJsonBytes()));
		}

		[TestMethod]
		[DataRow(52)]  // splits at decimal point in 3.14159 (between 3 and .)
		[DataRow(54)]  // splits inside decimal digits (between 4 and 1)
		public void ReadMultiSegmentNumberDouble(int splitPoint)
		{
			// Non-integer number exercises the TryGetDouble path (line 358) across segment boundaries
			string json = """{"message":"test","format":"{X}","properties":{"X":3.14159}}""";
			byte[] bytes = Encoding.UTF8.GetBytes(json);

			LogEvent expected = LogEvent.Read(bytes);

			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);
			Utf8JsonReader reader = new(sequence);
			reader.Read();
			LogEvent actual = LogEvent.Read(ref reader);

			Assert.AreEqual(Encoding.UTF8.GetString(expected.ToJsonBytes()), Encoding.UTF8.GetString(actual.ToJsonBytes()));
		}
	}
}
