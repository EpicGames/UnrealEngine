// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Text.Json;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class LogValueTests
	{
		[LogValueType]
		class TaggedType
		{
			public override string ToString() => "hello";
		}

		static string RenderLogValue(object value)
		{
			ArrayBufferWriter<byte> buffer = new();
			using (Utf8JsonWriter writer = new(buffer))
			{
				LogValueFormatter.Format(value, writer);
			}
			return Encoding.UTF8.GetString(buffer.WrittenSpan);
		}

		[TestMethod]
		public void TaggedTypeTest()
		{
			string value = RenderLogValue(new TaggedType());
			Assert.AreEqual("{\"$type\":\"TaggedType\",\"$text\":\"hello\"}", value);
		}

		[TestMethod]
		public void LinkTest()
		{
			string value = RenderLogValue(LogValue.Link(new Uri("https://epicgames.com/"), "Epic Games"));
			Assert.AreEqual("{\"$type\":\"Link\",\"$text\":\"Epic Games\",\"target\":\"https://epicgames.com/\"}", value);
		}

		[TestMethod]
		public void SourceFile()
		{
			FileReference file = new("foo.txt");
			string value = RenderLogValue(LogValue.SourceFile(file, "foo"));
			Assert.AreEqual(value, "{\"$type\":\"SourceFile\",\"$text\":\"foo\",\"file\":\"" + file.FullName.Replace("\\", "\\\\", StringComparison.Ordinal) + "\"}");
		}
	}
}
