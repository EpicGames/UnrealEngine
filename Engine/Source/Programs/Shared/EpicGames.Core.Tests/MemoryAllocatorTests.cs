// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class MemoryAllocatorTests
	{
		[TestMethod]
		public void TestGlobalAllocator()
		{
			GlobalHeapAllocator allocator = new();

			using IMemoryOwner<byte> handle = allocator.Alloc(1024, null);
			handle.Memory.Span[0] = 123;
		}

		[TestMethod]
		public void TestVirtualAllocator()
		{
			VirtualMemoryAllocator allocator = new();

			using IMemoryOwner<byte> handle = allocator.Alloc(1024, null);
			handle.Memory.Span[0] = 123;
		}
	}
}
