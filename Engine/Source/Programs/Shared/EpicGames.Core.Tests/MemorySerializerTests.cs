// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class MemorySerializerTests
	{
		[TestMethod]
		public void SignedIntegers()
		{
			byte[] data;
			{
				ByteArrayBuilder writer = new();
				writer.WriteInt32(12345);
				writer.WriteInt32(-12345);
				writer.WriteInt8(SByte.MinValue);
				writer.WriteInt8(SByte.MaxValue);
				writer.WriteInt16(Int16.MinValue);
				writer.WriteInt16(Int16.MaxValue);
				writer.WriteInt32(Int32.MinValue);
				writer.WriteInt32(Int32.MaxValue);
				writer.WriteInt64(Int64.MinValue);
				writer.WriteInt64(Int64.MaxValue);
				data = writer.ToByteArray();
			}
			{
				MemoryReader reader = new(data);
				Assert.AreEqual(12345, reader.ReadInt32());
				Assert.AreEqual(-12345, reader.ReadInt32());
				Assert.AreEqual(SByte.MinValue, reader.ReadInt8());
				Assert.AreEqual(SByte.MaxValue, reader.ReadInt8());
				Assert.AreEqual(Int16.MinValue, reader.ReadInt16());
				Assert.AreEqual(Int16.MaxValue, reader.ReadInt16());
				Assert.AreEqual(Int32.MinValue, reader.ReadInt32());
				Assert.AreEqual(Int32.MaxValue, reader.ReadInt32());
				Assert.AreEqual(Int64.MinValue, reader.ReadInt64());
				Assert.AreEqual(Int64.MaxValue, reader.ReadInt64());
				reader.CheckEmpty();
			}
		}

		[TestMethod]
		public void UnsignedIntegers()
		{
			byte[] data;
			{
				ByteArrayBuilder writer = new();
				writer.WriteUInt32(12345);
				writer.WriteUInt8(Byte.MaxValue);
				writer.WriteUInt16(UInt16.MaxValue);
				writer.WriteUInt32(UInt32.MaxValue);
				writer.WriteUInt64(UInt64.MaxValue);
				data = writer.ToByteArray();
			}
			{
				MemoryReader reader = new(data);
				Assert.AreEqual(12345U, reader.ReadUInt32());
				Assert.AreEqual(Byte.MaxValue, reader.ReadUInt8());
				Assert.AreEqual(UInt16.MaxValue, reader.ReadUInt16());
				Assert.AreEqual(UInt32.MaxValue, reader.ReadUInt32());
				Assert.AreEqual(UInt64.MaxValue, reader.ReadUInt64());
				reader.CheckEmpty();
			}
		}

		[TestMethod]
		public void TestMemory()
		{
			byte[] data;
			{
				ByteArrayBuilder writer = new();
				writer.WriteVariableLengthBytes([1, 2, 3]);
				writer.WriteVariableLengthBytes([4, 5, 6]);
				data = writer.ToByteArray();
			}
			{
				MemoryReader reader = new(data);
				Assert.IsTrue(reader.ReadVariableLengthBytes().Span.SequenceEqual(new byte[] { 1, 2, 3 }));
				Assert.IsTrue(reader.ReadVariableLengthBytes().Span.SequenceEqual(new byte[] { 4, 5, 6 }));
				reader.CheckEmpty();
			}
		}

		[TestMethod]
		public void TestStrings()
		{
			byte[] data;
			{
				ByteArrayBuilder writer = new();
				writer.WriteString("hello", Encoding.UTF8);
				writer.WriteString("there", Encoding.Unicode);
				writer.WriteString("world", Encoding.UTF32);
				data = writer.ToByteArray();
			}
			{
				MemoryReader reader = new(data);
				Assert.AreEqual("hello", reader.ReadString(Encoding.UTF8));
				Assert.AreEqual("there", reader.ReadString(Encoding.Unicode));
				Assert.AreEqual("world", reader.ReadString(Encoding.UTF32));
				reader.CheckEmpty();
			}
		}
	}
}
