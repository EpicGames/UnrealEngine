// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Threading;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class MemoryWriterTests
	{
		class TestAllocator : IMemoryAllocator<byte>
		{
			class Allocation(MemoryWriterTests.TestAllocator allocator, Memory<byte> memory) : IMemoryOwner<byte>
			{
				TestAllocator? _allocator = allocator;

				public Memory<byte> Memory => memory;

				public void Dispose()
				{
					if (_allocator != null)
					{
						Interlocked.Decrement(ref _allocator._allocatedCount);
						Interlocked.Add(ref _allocator._allocatedSize, -memory.Length);
						_allocator = null;
					}
				}
			}

			int _allocatedSize;
			int _allocatedCount;

			public int AllocatedSize => _allocatedSize;
			public int AllocatedCount => _allocatedCount;

			public IMemoryOwner<byte> Alloc(int size, object? tag = null)
			{
				Interlocked.Increment(ref _allocatedCount);
				Interlocked.Add(ref _allocatedSize, size);
				return new Allocation(this, new byte[size]);
			}
		}

		[TestMethod]
		public void RefCountedMemoryWriter()
		{
			TestAllocator allocator = new();
			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);

			using (RefCountedMemoryWriter writer = new(allocator, 4096))
			{
				writer.WriteString("Hello world");
				Assert.IsTrue(allocator.AllocatedSize > 0);
				Assert.AreEqual(1, allocator.AllocatedCount);
			}

			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);
		}

		[TestMethod]
		public void RefCountedMemoryWriterSequence()
		{
			TestAllocator allocator = new();
			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);

			IRefCountedHandle<ReadOnlyMemory<byte>> memory;
			IRefCountedHandle<ReadOnlySequence<byte>> sequence;

			using (RefCountedMemoryWriter writer = new(allocator, 10))
			{
				writer.WriteString("This should be written to a single buffer and the original buffer should be freed");
				int allocatedSize1 = allocator.AllocatedSize;
				Assert.IsTrue(allocatedSize1 > 0);
				Assert.AreEqual(1, allocator.AllocatedCount);

				writer.WriteString("This should be written to a separate buffer");
				int allocatedSize2 = allocator.AllocatedSize;
				Assert.IsTrue(allocatedSize2 > allocatedSize1);
				Assert.AreEqual(2, allocator.AllocatedCount);

				sequence = writer.AsRefCountedSequence();
				Assert.AreEqual(allocatedSize2, allocator.AllocatedSize);
				Assert.AreEqual(2, allocator.AllocatedCount);

				memory = writer.AsRefCountedMemory();
				Assert.IsTrue(allocator.AllocatedSize > allocatedSize2);
				Assert.AreEqual(3, allocator.AllocatedCount);
			}

			Assert.AreEqual(3, allocator.AllocatedCount);
			memory.Dispose();

			Assert.AreEqual(2, allocator.AllocatedCount);
			sequence.Dispose();

			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);
		}

		[TestMethod]
		public void RefCountedMemoryWriterSequence2()
		{
			TestAllocator allocator = new();
			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);

			int allocatedSize1;
			IRefCountedHandle<ReadOnlySequence<byte>> sequence;

			using (RefCountedMemoryWriter writer = new(allocator, 10))
			{
				writer.WriteString("This should be written to a single buffer and the original buffer should be freed");
				allocatedSize1 = allocator.AllocatedSize;
				Assert.IsTrue(allocatedSize1 > 0);
				Assert.AreEqual(1, allocator.AllocatedCount);

				writer.WriteString("This should be written to a separate buffer");
				int allocatedSize2 = allocator.AllocatedSize;
				Assert.IsTrue(allocatedSize2 > allocatedSize1);
				Assert.AreEqual(2, allocator.AllocatedCount);

				sequence = writer.AsRefCountedSequence(6, 6); // one byte size field, followed by string above
				Assert.AreEqual(allocatedSize2, allocator.AllocatedSize);
				Assert.AreEqual(2, allocator.AllocatedCount);
			}

			Assert.AreEqual(allocatedSize1, allocator.AllocatedSize);
			Assert.AreEqual(1, allocator.AllocatedCount);
			Assert.AreEqual("should", Encoding.UTF8.GetString(sequence.Target.FirstSpan));

			sequence.Dispose();

			Assert.AreEqual(0, allocator.AllocatedSize);
			Assert.AreEqual(0, allocator.AllocatedCount);
		}
	}
}
