// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class SemVerTests
	{
		[TestMethod]
		public void PrecedenceTests()
		{
			Assert.IsFalse(SemVer.Parse("1.0.0") < SemVer.Parse("1.0.0"));
			Assert.IsFalse(SemVer.Parse("1.0.0") > SemVer.Parse("1.0.0"));
			Assert.IsTrue(SemVer.Parse("1.0.0") <= SemVer.Parse("1.0.0"));
			Assert.IsTrue(SemVer.Parse("1.0.0") >= SemVer.Parse("1.0.0"));
			Assert.IsTrue(SemVer.Parse("1.0.0") == SemVer.Parse("1.0.0"));

			Assert.IsTrue(SemVer.Parse("1.0.0") < SemVer.Parse("2.0.0"));
			Assert.IsTrue(SemVer.Parse("1.0.0") < SemVer.Parse("1.2.0"));
			Assert.IsTrue(SemVer.Parse("1.0.0") < SemVer.Parse("1.0.2"));
		}

		[TestMethod]
		public void PrereleaseTests()
		{
			Assert.IsTrue(SemVer.Compare(SemVer.Parse("5.1.0-19607491"), SemVer.Parse("5.1.0-19607492")) < 0);
			Assert.IsTrue(SemVer.Compare(SemVer.Parse("5.1.0-19607492"), SemVer.Parse("5.1.0-19607492")) == 0);
			Assert.IsTrue(SemVer.Compare(SemVer.Parse("5.1.0-19607493"), SemVer.Parse("5.1.0-19607492")) > 0);

			Assert.IsTrue(SemVer.Parse("1.0.0-123") < SemVer.Parse("1.0.0-124"));
			Assert.IsTrue(SemVer.Parse("1.0.0-124") > SemVer.Parse("1.0.0-123"));

			Assert.IsTrue(SemVer.Parse("1.0.0-1230") > SemVer.Parse("1.0.0-122"));
			Assert.IsTrue(SemVer.Parse("1.0.0-122") < SemVer.Parse("1.0.0-1230"));

			Assert.IsTrue(SemVer.Parse("1.0.0-alpha") < SemVer.Parse("1.0.0-alpha.1"));
			Assert.IsTrue(SemVer.Parse("1.0.0-alpha.1") < SemVer.Parse("1.0.0-alpha.beta"));
			Assert.IsTrue(SemVer.Parse("1.0.0-alpha.beta") < SemVer.Parse("1.0.0-beta"));
			Assert.IsTrue(SemVer.Parse("1.0.0-beta") < SemVer.Parse("1.0.0-beta.2"));
			Assert.IsTrue(SemVer.Parse("1.0.0-beta.2") < SemVer.Parse("1.0.0-beta.11"));
			Assert.IsTrue(SemVer.Parse("1.0.0-beta.11") < SemVer.Parse("1.0.0-rc.1"));
			Assert.IsTrue(SemVer.Parse("1.0.0-rc.1") < SemVer.Parse("1.0.0"));

			Assert.IsFalse(SemVer.Parse("1.0.0-alpha") > SemVer.Parse("1.0.0-alpha.1"));
			Assert.IsFalse(SemVer.Parse("1.0.0-alpha.1") > SemVer.Parse("1.0.0-alpha.beta"));
			Assert.IsFalse(SemVer.Parse("1.0.0-alpha.beta") > SemVer.Parse("1.0.0-beta"));
			Assert.IsFalse(SemVer.Parse("1.0.0-beta") > SemVer.Parse("1.0.0-beta.2"));
			Assert.IsFalse(SemVer.Parse("1.0.0-beta.2") > SemVer.Parse("1.0.0-beta.11"));
			Assert.IsFalse(SemVer.Parse("1.0.0-beta.11") > SemVer.Parse("1.0.0-rc.1"));
			Assert.IsFalse(SemVer.Parse("1.0.0-rc.1") > SemVer.Parse("1.0.0"));
		}
	}
}
