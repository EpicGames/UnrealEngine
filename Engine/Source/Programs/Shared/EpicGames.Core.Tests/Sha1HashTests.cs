// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Text.Json;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class Sha1HashTests
	{
		static readonly Sha1Hash s_testHash = Sha1Hash.Compute(Encoding.ASCII.GetBytes("test data for sha1"));

		[TestMethod]
		[DataRow(5)]
		[DataRow(22)]
		[DataRow(30)]
		public void JsonConverterMultiSegment(int splitPoint)
		{
			string json = $"\"{s_testHash}\"";
			byte[] bytes = Encoding.UTF8.GetBytes(json);
			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			Sha1Hash result = JsonSerializer.Deserialize<Sha1Hash>(ref reader);
			Assert.AreEqual(s_testHash, result);
		}
	}
}
