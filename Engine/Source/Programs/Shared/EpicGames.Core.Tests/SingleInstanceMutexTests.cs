// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public class SingleInstanceMutexTests
{
#pragma warning disable CA1849 // Intentionally testing the synchronous code paths.
	[TestMethod]
	public async Task AcquiringGlobalMutexSynchronouslyBlocksCorrectlyAsync()
	{
		const string MutexName = "EpicGames.Build.Tests.UnitTestMutex0";
		IDisposable? mutex1 = SingleInstanceMutex.Acquire(MutexName);

		try
		{
			Task mutex2Task = Task.Run(() =>
			{
				using IDisposable mutex2 = SingleInstanceMutex.Acquire(MutexName);
			}, TestContext.CancellationToken);

			// We should get the delay task back because the mutex 2 task should be blocked acquiring the mutex.
			Task delayTask = Task.Delay(200, TestContext.CancellationToken);
			Task finishedTask = await Task.WhenAny(mutex2Task, delayTask);

			Assert.AreSame(delayTask, finishedTask);

			// Dispose of the first mutex so the second one can acquire it.
			mutex1.Dispose();
			mutex1 = null;

			delayTask = Task.Delay(200, TestContext.CancellationToken);
			finishedTask = await Task.WhenAny(mutex2Task, delayTask);
			await finishedTask;

			// We should get the mutex 2 task as it should have been able to acquire the mutex and free it again.
			Assert.AreSame(mutex2Task, finishedTask);
		}
		finally
		{
			mutex1?.Dispose();
		}
	}
#pragma warning restore CA1849

	[TestMethod]
	public async Task AcquiringGlobalMutexAsynchronouslyBlocksCorrectlyAsync()
	{
		const string MutexName = "EpicGames.Build.Tests.UnitTestMutex1";
		IDisposable? mutex1 = await SingleInstanceMutex.AcquireAsync(MutexName, TestContext.CancellationToken);

		try
		{
			Task<IDisposable> mutex2Task = SingleInstanceMutex.AcquireAsync(MutexName, TestContext.CancellationToken);

			// We should get the delay task back because the mutex 2 task should be blocked acquiring the mutex.
			Task delayTask = Task.Delay(200, TestContext.CancellationToken);
			Task finishedTask = await Task.WhenAny(mutex2Task, delayTask);

			Assert.AreSame(delayTask, finishedTask);

			// Dispose of the first mutex so the second one can acquire it.
			mutex1.Dispose();
			mutex1 = null;

			delayTask = Task.Delay(200, TestContext.CancellationToken);
			finishedTask = await Task.WhenAny(mutex2Task, delayTask);
			await finishedTask;

			// We should get the mutex 2 task as it should have been able to acquire the mutex and free it again.
			Assert.AreSame(mutex2Task, finishedTask);

			(await mutex2Task).Dispose();
		}
		finally
		{
			mutex1?.Dispose();
		}
	}

	[TestMethod]
	public async Task AcquiringGlobalMutexAsynchronouslyCancelsCorrectlyAsync()
	{
		const string MutexName = "EpicGames.Build.Tests.UnitTestMutex2";
		IDisposable? mutex1 = await SingleInstanceMutex.AcquireAsync(MutexName, TestContext.CancellationToken);

		try
		{
			using (CancellationTokenSource cancellationTokenSource = new())
			{
				Task mutex2Task = SingleInstanceMutex.AcquireAsync(MutexName, cancellationTokenSource.Token);

				// We should get the delay task back because the mutex 2 task should be blocked acquiring the mutex.
				Task delayTask = Task.Delay(200, TestContext.CancellationToken);
				Task finishedTask = await Task.WhenAny(mutex2Task, delayTask);

				Assert.AreSame(delayTask, finishedTask);

				await cancellationTokenSource.CancelAsync();

				await Assert.ThrowsAsync<OperationCanceledException>(async () => await mutex2Task);

				// Dispose of the first mutex.
				mutex1.Dispose();
				mutex1 = null;

				// Prove that the task was properly cancelled and we can acquire the mutex now.
				mutex1 = await SingleInstanceMutex.AcquireAsync(MutexName, TestContext.CancellationToken);
			}
		}
		finally
		{
			mutex1?.Dispose();
		}
	}

	[TestMethod]
	public void TryAcquiringGlobalMutex()
	{
		const string MutexName = "EpicGames.Build.Tests.UnitTestMutex3";
		IDisposable? mutex1 = null;
		IDisposable? mutex2 = null;

		try
		{
			bool didAcquire = SingleInstanceMutex.TryAcquire(MutexName, out mutex1);
			Assert.IsTrue(didAcquire);
			Assert.IsNotNull(mutex1);

			didAcquire = SingleInstanceMutex.TryAcquire(MutexName, out mutex2);
			Assert.IsFalse(didAcquire);
			Assert.IsNull(mutex2);
		}
		finally
		{
			mutex1?.Dispose();
			mutex2?.Dispose();
		}
	}

	public TestContext TestContext { get; set; }
}
