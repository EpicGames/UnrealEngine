// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Diagnostics;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public class SingleThreadTaskSchedulerTests
{
	[TestMethod]
	public async Task TasksAreExecutedOnTheSameThreadInOrderAsync()
	{
		using SingleThreadTaskScheduler scheduler = new();

		const int NumTasksToRun = 10;
		ConcurrentQueue<int> threadIDs = [];
		ConcurrentQueue<int> taskIDs = [];
		ConcurrentQueue<long> timestamps = [];
		Task[] tasks = new Task[NumTasksToRun];

		for (int i = 0; i < NumTasksToRun; ++i)
		{
			int taskID = i;
			tasks[i] = Task.Factory.StartNew(() =>
				{
					threadIDs.Enqueue(Environment.CurrentManagedThreadId);
					taskIDs.Enqueue(taskID);
					timestamps.Enqueue(Stopwatch.GetTimestamp());
					// Sleep for just long enough to not be instantaneous.
					Thread.Sleep(1);
					timestamps.Enqueue(Stopwatch.GetTimestamp());
				},
				CancellationToken.None,
				TaskCreationOptions.None,
				scheduler);
		}

		await Task.WhenAll(tasks);

		// Everything should have ran on the same thread.
		Assert.AreEqual(1, threadIDs.Distinct().Count());
		// The tasks should have executed in the order they were started.
		CollectionAssert.AreEqual(Enumerable.Range(0, NumTasksToRun).ToList(), taskIDs);
		// The timestamps for each task shouldn't overlap at all.
		CollectionAssert.AreEqual(timestamps.OrderBy(x => x).ToList(), timestamps);
	}
}
