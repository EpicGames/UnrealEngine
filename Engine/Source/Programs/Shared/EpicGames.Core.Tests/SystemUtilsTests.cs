// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests;

[TestClass]
public class SystemUtilsTests
{
	#region GetLogicalProcessorCount

	[TestMethod]
	public void GetLogicalProcessorCount_ReturnsPositiveValue()
	{
		int count = SystemUtils.GetLogicalProcessorCount();
		Assert.IsGreaterThan(0, count, $"Expected logical processor count > 0, got {count}");
	}

	[TestMethod]
	public void GetLogicalProcessorCount_AtLeastEnvironmentProcessorCount()
	{
		if (!OperatingSystem.IsWindows())
		{
			return; // On Linux, lscpu respects cgroup CPU limits and may report fewer cores
		}

		// SystemUtils handles >64-core machines via group enumeration; it should never return
		// fewer cores than the clamped Environment.ProcessorCount value.
		int count = SystemUtils.GetLogicalProcessorCount();
		Assert.IsGreaterThanOrEqualTo(Environment.ProcessorCount, count, $"Expected >= Environment.ProcessorCount ({Environment.ProcessorCount}), got {count}");
	}

	#endregion

	#region GetPhysicalProcessorCount

	[TestMethod]
	public void GetPhysicalProcessorCount_ReturnsValidSentinelOrPositiveValue()
	{
		int count = SystemUtils.GetPhysicalProcessorCount();
		Assert.IsGreaterThan(0, count, $"Expected positive count, got {count}");
	}

	[TestMethod]
	public void GetPhysicalProcessorCount_DoesNotExceedLogicalCount()
	{
		int physical = SystemUtils.GetPhysicalProcessorCount();
		int logical = SystemUtils.GetLogicalProcessorCount();
		Assert.IsLessThanOrEqualTo(logical, physical, $"Physical core count ({physical}) must not exceed logical core count ({logical})");
	}

	#endregion

	#region IsAsymmetricalProcessor

	[TestMethod]
	public void IsAsymmetricalProcessor_FalseWhenPhysicalEqualsLogical()
	{
		int physical = SystemUtils.GetPhysicalProcessorCount();
		int logical = SystemUtils.GetLogicalProcessorCount();
		if (physical == logical)
		{
			Assert.IsFalse(SystemUtils.IsAsymmetricalProcessor(), $"Expected false when physical ({physical}) == logical ({logical})");
		}
	}

	[TestMethod]
	public void IsAsymmetricalProcessor_FalseWhenPhysicalIsHalfLogical()
	{
		int physical = SystemUtils.GetPhysicalProcessorCount();
		int logical = SystemUtils.GetLogicalProcessorCount();
		if (logical == physical * 2)
		{
			Assert.IsFalse(SystemUtils.IsAsymmetricalProcessor(), $"Expected false for symmetric HT setup: physical={physical}, logical={logical}");
		}
	}

	#endregion

	#region GetTotalSystemMemoryBytes

	[TestMethod]
	public void GetTotalSystemMemoryBytes_ReturnsPositiveValue()
	{
		long bytes = SystemUtils.GetTotalSystemMemoryBytes();
		Assert.IsGreaterThan(0, bytes, $"Expected total system memory > 0, got {bytes}");
	}

	#endregion

	#region GetAvailableMemoryBytes

	[TestMethod]
	public void GetAvailableMemoryBytes_ReturnsValidValue()
	{
		GC.Collect();
		long bytes = SystemUtils.GetAvailableMemoryBytes();
		Assert.IsGreaterThan(0, bytes, $"Expected available memory > 0 after GC.Collect(), got {bytes}");
	}

	[TestMethod]
	public void GetAvailableMemoryBytes_NotGreaterThanTotal()
	{
		GC.Collect();
		long available = SystemUtils.GetAvailableMemoryBytes();
		long total = SystemUtils.GetTotalSystemMemoryBytes();
		Assert.IsLessThanOrEqualTo(total, available, $"Available ({available}) must not exceed total ({total})");
	}

	#endregion

	#region GetFreeMemoryBytes

	[TestMethod]
	public void GetFreeMemoryBytes_ReturnsValidValue()
	{
		GC.Collect();
		long bytes = SystemUtils.GetFreeMemoryBytes();
		Assert.IsGreaterThanOrEqualTo(0, bytes, $"Expected free memory >= 0 after GC.Collect(), got {bytes}");
	}

	[TestMethod]
	public void GetFreeMemoryBytes_NotGreaterThanTotal()
	{
		GC.Collect();
		long free = SystemUtils.GetFreeMemoryBytes();
		long total = SystemUtils.GetTotalSystemMemoryBytes();
		Assert.IsLessThanOrEqualTo(total, free, $"Free ({free}) must not exceed total ({total})");
	}

	#endregion

	#region GetCurrentCommittedMemoryBytes

	[TestMethod]
	public void GetCurrentCommittedMemoryBytes_ReturnsPositiveValue()
	{
		long bytes = SystemUtils.GetCurrentCommittedMemoryBytes();
		Assert.IsGreaterThan(0, bytes, $"Expected current committed memory > 0, got {bytes}");
	}

	[TestMethod]
	public void GetCurrentCommittedMemoryBytes_NotGreaterThanMax()
	{
		long current = SystemUtils.GetCurrentCommittedMemoryBytes();
		long max = SystemUtils.GetMaxCommittedMemoryBytes();
		if (current >= 0 && max >= 0)
		{
			Assert.IsLessThanOrEqualTo(max, current, $"Current committed ({current}) must not exceed max committed ({max})");
		}
	}

	#endregion

	#region GetMaxCommittedMemoryBytes

	[TestMethod]
	public void GetMaxCommittedMemoryBytes_ReturnsPositiveValue()
	{
		long bytes = SystemUtils.GetMaxCommittedMemoryBytes();
		Assert.IsGreaterThan(0, bytes, $"Expected max committed memory > 0, got {bytes}");
	}

	[TestMethod]
	public void GetMaxCommittedMemoryBytes_AtLeastTotalSystemMemory()
	{
		if (!OperatingSystem.IsWindows())
		{
			return; // On Linux, CommitLimit depends on overcommit_ratio and swap, may be < physical RAM
		}

		long max = SystemUtils.GetMaxCommittedMemoryBytes();
		long total = SystemUtils.GetTotalSystemMemoryBytes();
		if (max >= 0 && total > 0)
		{
			// Commit limit should be at least as large as physical RAM (it includes the page file)
			Assert.IsGreaterThanOrEqualTo(total, max, $"Max committed ({max}) should be >= total physical memory ({total})");
		}
	}

	#endregion

	#region GetTotalCpuUtilization

	[TestMethod]
	public void GetTotalCpuUtilization_ReturnsValidValue()
	{
		if (OperatingSystem.IsWindows())
		{
			bool success = SystemUtils.GetTotalCpuUtilization(out float utilization);
			Assert.IsTrue(success, "Expected GetTotalCpuUtilization to return true on Windows");
			Assert.IsGreaterThanOrEqualTo(0.0f, utilization, $"Expected utilization in [0, 100], got {utilization}");
			Assert.IsLessThanOrEqualTo(100.0f, utilization, $"Expected utilization in [0, 100], got {utilization}");
		}
	}

	#endregion
}
