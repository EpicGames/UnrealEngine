// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Core.Tests
{
	[TestClass]
	public class VarIntTests
	{
		[TestMethod]
		public void TestVarInt()
		{
			byte[] buffer = new byte[20];

			int length = VarInt.WriteUnsigned(buffer, -1);
			Assert.AreEqual(9, length);
			Assert.AreEqual(9, VarInt.MeasureUnsigned(-1));

			Assert.AreEqual(9, VarInt.Measure(buffer));
			int value = (int)(long)VarInt.ReadUnsigned(buffer, out int bytesRead);
			Assert.AreEqual(9, bytesRead);

			Assert.AreEqual(-1, value);
		}
	}
}
