// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Security.AccessControl;
using System.Security.Principal;

namespace EpicGames.Core;

/// <summary>
/// Exception for AppContainer
/// </summary>
public class AppContainerException : Exception
{
	/// <inheritdoc/>
	public AppContainerException(string? message) : base(message)
	{
	}

	/// <inheritdoc/>
	public AppContainerException(string? message, Exception? innerException) : base(message, innerException)
	{
	}
}

/// <summary>
/// Manages a setup and initialization of an AppContainer on Windows
/// </summary>
[SupportedOSPlatform("windows")]
public sealed class AppContainer : IDisposable
{
	#region P/Invoke
	private enum HRESULT : uint
	{
		S_FALSE = 0x0001,
		S_OK = 0x0000,
		E_INVALIDARG = 0x80070057,
	}

	private enum PROC_THREAD_ATTRIBUTES
	{
		PROC_THREAD_ATTRIBUTE_SECURITY_CAPABILITIES = 0x00020009,
	}

	private enum Win32Error : uint
	{
		ERROR_ALREADY_EXISTS = 0x000000B7
	}

	[StructLayout(LayoutKind.Sequential)]
	[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Win32 API naming convention")]
	private struct SID_AND_ATTRIBUTES
	{
		public IntPtr Sid;
		public int Attributes;
	}

	[StructLayout(LayoutKind.Sequential)]
	[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Win32 API naming convention")]
	private struct SECURITY_CAPABILITIES
	{
		public IntPtr AppContainerSid;
		public IntPtr Capabilities;
		public uint CapabilityCount;
		public uint Reserved;
	}

	[DllImport("advapi32.dll")]
	private static extern IntPtr FreeSid(IntPtr pSid);

	[DllImport("advapi32.dll", CharSet = CharSet.Auto, SetLastError = true)]
	private static extern bool ConvertSidToStringSid(IntPtr pSid, out IntPtr ptrSid);

	[DllImport("kernel32.dll", SetLastError = true)]
	private static extern bool DeleteProcThreadAttributeList(IntPtr lpAttributeList);

	[DllImport("kernel32.dll", SetLastError = true)]
	static extern IntPtr LocalFree(IntPtr hMem);

	[DllImport("kernel32.dll", SetLastError = true)]
	[return: MarshalAs(UnmanagedType.Bool)]
	private static extern bool UpdateProcThreadAttribute(
		IntPtr lpAttributeList,
		uint dwFlags,
		IntPtr attribute,
		IntPtr lpValue,
		IntPtr cbSize,
		IntPtr lpPreviousValue,
		IntPtr lpReturnSize);

	[DllImport("kernel32.dll", SetLastError = true)]
	[return: MarshalAs(UnmanagedType.Bool)]
	private static extern bool InitializeProcThreadAttributeList(
		IntPtr lpAttributeList,
		int dwAttributeCount,
		int dwFlags,
		ref IntPtr lpSize);

	[DllImport("userenv.dll", SetLastError = false, ExactSpelling = true)]
	private static extern HRESULT CreateAppContainerProfile(
		[MarshalAs(UnmanagedType.LPWStr)] string pszAppContainerName,
		[MarshalAs(UnmanagedType.LPWStr)] string pszDisplayName,
		[MarshalAs(UnmanagedType.LPWStr)] string pszDescription,
		[In] SID_AND_ATTRIBUTES[] pCapabilities,
		uint dwCapabilityCount,
		out IntPtr ppSidAppContainerSid);

	[DllImport("userenv.dll", SetLastError = false, ExactSpelling = true)]
	private static extern HRESULT GetAppContainerFolderPath(
		[MarshalAs(UnmanagedType.LPWStr)] string pszAppContainerSid,
		[MarshalAs(UnmanagedType.LPWStr)] out string ppszPath);

	[DllImport("userenv.dll", SetLastError = false, ExactSpelling = true)]
	private static extern HRESULT DeleteAppContainerProfile([MarshalAs(UnmanagedType.LPWStr)] string pszAppContainerName);

	[DllImport("userenv.dll", SetLastError = false, ExactSpelling = true)]
	private static extern HRESULT DeriveAppContainerSidFromAppContainerName([MarshalAs(UnmanagedType.LPWStr)] string pszAppContainerName, out IntPtr ppsidAppContainerSid);

	private static string ManagedConvertSidToStringSid(IntPtr sid)
	{
		IntPtr sidStringPtr = IntPtr.Zero;
		try
		{
			if (!ConvertSidToStringSid(sid, out sidStringPtr))
			{
				throw new AppContainerException("Failed converting SID to string SID", new Win32Exception());
			}

			string? sidString = Marshal.PtrToStringAuto(sidStringPtr);
			return sidString ?? throw new AppContainerException("Failed converting string pointer");
		}
		finally
		{
			if (sidStringPtr != IntPtr.Zero)
			{
				LocalFree(sidStringPtr);
			}
		}
	}

	private static HRESULT HResultFromWin32(uint errorCode)
	{
		const int FacilityWin32 = 7;
		if (errorCode <= 0)
		{
			return (HRESULT)errorCode;
		}
		return (HRESULT)((errorCode & 0x0000FFFF) | (FacilityWin32 << 16) | 0x80000000);
	}

	#endregion

	/// <summary>
	/// Name of the container
	/// </summary>
	public string ContainerName { get; }

	private readonly IntPtr _sid;
	private IntPtr _attribList;
	private bool _isDisposed;

	private AppContainer(string containerName, IntPtr sid)
	{
		ContainerName = containerName;
		_sid = sid;
	}

	/// <inheritdoc/>
	public void Dispose()
	{
		if (!_isDisposed)
		{
			if (_sid != IntPtr.Zero)
			{
				FreeSid(_sid);
			}

			if (_attribList != IntPtr.Zero)
			{
				DeleteProcThreadAttributeList(_attribList);
			}

			GC.SuppressFinalize(this);
			_isDisposed = true;
		}
	}

	/// <summary>
	/// Create an AppContainer
	/// </summary>
	/// <param name="containerName">Unique container name (see CreateAppContainerProfile call for details)</param>
	/// <param name="displayName">Display name</param>
	/// <param name="description">Description</param>
	/// <param name="errorIfExists">Throw an exception if container already exists, do not attempt to get it</param>
	/// <returns>An initialized AppContainer</returns>
	/// <exception cref="AppContainerException"></exception>
	public static AppContainer Create(string containerName, string displayName, string description, bool errorIfExists = false)
	{
		HRESULT res = CreateAppContainerProfile(containerName, displayName, description, null!, 0, out IntPtr sid);
		if (!errorIfExists && res == HResultFromWin32((uint)Win32Error.ERROR_ALREADY_EXISTS))
		{
			res = DeriveAppContainerSidFromAppContainerName(containerName, out sid);
		}

		if (res != HRESULT.S_OK)
		{
			FreeSid(sid);
			throw new AppContainerException($"Unable to create AppContainer profile '{containerName}. Error: {res}");
		}

		return new AppContainer(containerName, sid);
	}

	/// <summary>
	/// Delete an AppContainer
	/// </summary>
	/// <param name="containerName">Name</param>
	/// <exception cref="AppContainerException">If deletion fails</exception>
	public static void Delete(string containerName)
	{
		HRESULT res = DeleteAppContainerProfile(containerName);
		if (res != HRESULT.S_OK)
		{
			throw new AppContainerException($"Unable to delete AppContainer profile '{containerName}. Error: {res}");
		}
	}

	/// <summary>
	/// Get home directory path for the AppContainer
	/// </summary>
	/// <returns>Path to directory</returns>
	/// <exception cref="Exception"></exception>
	public DirectoryInfo GetFolderPath()
	{
		string sidString = ManagedConvertSidToStringSid(_sid);
		HRESULT res = GetAppContainerFolderPath(sidString, out string folderPath);
		if (res != HRESULT.S_OK)
		{
			throw new Exception("Unable to get AppContainer folder path. Error: " + res, new Win32Exception());
		}

		return new DirectoryInfo(folderPath);
	}

	/// <summary>
	/// Grant permissions to a directory for current AppContainer
	/// </summary>
	/// <param name="dirInfo">Directory to modify</param>
	/// <param name="rights">Permissions to grant</param>
	public void AddDirectoryAccess(DirectoryInfo dirInfo, FileSystemRights rights)
	{
		AccessControlType controlType = AccessControlType.Allow;
		DirectorySecurity dirSecurity = dirInfo.GetAccessControl();
		SecurityIdentifier sid = new(_sid);
		const InheritanceFlags InheritanceFlags = InheritanceFlags.ContainerInherit | InheritanceFlags.ObjectInherit;

		dirSecurity.AddAccessRule(new FileSystemAccessRule(sid, rights, InheritanceFlags, PropagationFlags.None, controlType));
		dirInfo.SetAccessControl(dirSecurity);
	}

	/// <summary>
	/// Get attribute list for use in STARTUPINFOEX struct given to CreateProcess
	/// </summary>
	/// <returns>An attribute list pointer</returns>
	/// <exception cref="Exception"></exception>
	internal IntPtr GetAttributeList()
	{
		if (_attribList != IntPtr.Zero)
		{
			return _attribList;
		}

		SECURITY_CAPABILITIES securityCapabilities = new() { AppContainerSid = _sid, Capabilities = IntPtr.Zero, CapabilityCount = 0, Reserved = 0 };
		IntPtr lpSize = IntPtr.Zero;

		bool isSuccess = InitializeProcThreadAttributeList(IntPtr.Zero, 1, 0, ref lpSize);
		if (isSuccess || lpSize == IntPtr.Zero)
		{
			throw new Exception("Unable to initialize thread attribute list size", new Win32Exception());
		}

		IntPtr attribList = Marshal.AllocHGlobal(lpSize);
		isSuccess = InitializeProcThreadAttributeList(attribList, 1, 0, ref lpSize);
		if (!isSuccess)
		{
			throw new Exception("Unable to initialize thread attribute list", new Win32Exception());
		}

		IntPtr scPtr = Marshal.AllocHGlobal(Marshal.SizeOf(securityCapabilities));
		Marshal.StructureToPtr(securityCapabilities, scPtr, false);

		isSuccess = UpdateProcThreadAttribute(
			attribList,
			0,
			(IntPtr)PROC_THREAD_ATTRIBUTES.PROC_THREAD_ATTRIBUTE_SECURITY_CAPABILITIES,
			scPtr,
			(IntPtr)Marshal.SizeOf(securityCapabilities),
			IntPtr.Zero,
			IntPtr.Zero);

		if (!isSuccess)
		{
			throw new Exception("Unable to update thread attribute list", new Win32Exception());
		}

		_attribList = attribList;
		return attribList;
	}
}

