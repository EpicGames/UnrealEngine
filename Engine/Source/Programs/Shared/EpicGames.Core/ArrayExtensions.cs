// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for arrays.
	/// </summary>
	public static class ArrayExtensions
	{
		/// <summary>
		/// Converts an array of elements to a different type
		/// </summary>
		/// <typeparam name="TIn">Type of the input elements</typeparam>
		/// <typeparam name="TOut">Type of the output elements</typeparam>
		/// <param name="input">The array to convert</param>
		/// <param name="func">Conversion function for each element</param>
		/// <returns>Array of elements of the output type</returns>
		public static TOut[] ConvertAll<TIn, TOut>(this TIn[] input, Func<TIn, TOut> func)
		{
			TOut[] output = new TOut[input.Length];
			for (int idx = 0; idx < input.Length; idx++)
			{
				output[idx] = func(input[idx]);
			}
			return output;
		}
	}
}
