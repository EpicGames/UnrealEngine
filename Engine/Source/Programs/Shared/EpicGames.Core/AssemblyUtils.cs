// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Reflection;
using System.Runtime.Loader;

namespace EpicGames.Core
{
	/// <summary>
	/// Utilities for dealing with assembly loading
	/// </summary>
	public static class AssemblyUtils
	{
		/// <summary>
		/// Gets the original location (path and filename) of an assembly.
		/// This method is using Assembly.Location property to properly resolve original
		/// assembly path in case shadow copying is enabled.
		/// </summary>
		/// <returns>Absolute path and filename to the assembly.</returns>
		public static string GetOriginalLocation(this Assembly thisAssembly)
		{
			return new Uri(thisAssembly.Location).LocalPath;
		}

		/// <summary>
		/// Version info of the executable which runs this code.
		/// </summary>
		public static FileVersionInfo ExecutableVersion => FileVersionInfo.GetVersionInfo(Assembly.GetEntryAssembly()!.GetOriginalLocation());

		/// <summary>
		/// Installs an assembly resolver, which will load *any* assembly which exists recursively within the supplied folder.
		/// </summary>
		/// <param name="rootDirectory">The directory to enumerate.</param>
		public static void InstallRecursiveAssemblyResolver(string rootDirectory)
		{
			if (Directory.Exists(rootDirectory))
			{
				// Initialize our cache of assemblies by enumerating all files in the given folder.
				foreach (string discoveredAssembly in Directory.EnumerateFiles(rootDirectory, "*.dll", SearchOption.AllDirectories))
				{
					AddFileToAssemblyCache(discoveredAssembly);
				}
			}
		}

		/// <summary>
		/// Adds a file to the assembly cache.
		/// </summary>
		/// <param name="assemblyPath"></param>
		public static void AddFileToAssemblyCache(string assemblyPath)
		{
			// Ignore any reference assemblies
			string? directory = Path.GetFileName(Path.GetDirectoryName(assemblyPath));
			if (!String.IsNullOrEmpty(directory) && (directory == "ref" | directory == "refint"))
			{
				return;
			}

			string assemblyName = Path.GetFileNameWithoutExtension(assemblyPath);
			DateTime assemblyLastWriteTime = File.GetLastWriteTimeUtc(assemblyPath);
			lock (s_assemblyLocationCache)
			{
				if (!s_assemblyLocationCache.TryAdd(assemblyName, assemblyPath))
				{
					// We already have this assembly in our cache. Only replace it if the discovered file is newer (to avoid stale assemblies breaking stuff).
					if (assemblyLastWriteTime > s_assemblyWriteTimes[assemblyName])
					{
						s_assemblyLocationCache[assemblyName] = assemblyPath;
						s_assemblyWriteTimes[assemblyName] = assemblyLastWriteTime;
					}
				}
				else
				{
					s_assemblyWriteTimes.Add(assemblyName, assemblyLastWriteTime);
				}
			}

			AddAssemblyCacheResolverIfNecessary();
		}

		private static void AddAssemblyCacheResolverIfNecessary()
		{
			if (!s_addedToAssemblyResolver)
			{
				AppDomain.CurrentDomain.AssemblyResolve += (sender, args) =>
				{
					// Name is fully qualified assembly definition - e.g. "p4dn, Version=1.0.0.0, Culture=neutral, PublicKeyToken=ff968dc1933aba6f"
					string assemblyName = args.Name!.Split(',')[0];
					lock (s_assemblyLocationCache)
					{
						if (s_assemblyLocationCache.TryGetValue(assemblyName, out string? assemblyLocation))
						{
							// We have this assembly in our folder.
							if (File.Exists(assemblyLocation))
							{
								// The assembly still exists, so load it into the default, shared context, so we only get one version of the assembly.
								return AssemblyLoadContext.Default.LoadFromAssemblyPath(assemblyLocation);
							}
							else
							{
								// The assembly no longer exists on disk, so remove it from our cache.
								s_assemblyLocationCache.Remove(assemblyName);
							}
						}
					}

					return null;
				};

				s_addedToAssemblyResolver = true;
			}
		}

		// Map of assembly name to path on disk
		private static readonly Dictionary<string, string> s_assemblyLocationCache = new(StringComparer.OrdinalIgnoreCase);
		// Track last modified date of each assembly, so we can ensure we always reference the latest one in the case of stale assemblies on disk.
		private static readonly Dictionary<string, DateTime> s_assemblyWriteTimes = new(StringComparer.OrdinalIgnoreCase);
		// Flag used to make sure we don't redundantly add resolvers
		private static bool s_addedToAssemblyResolver = false;

	}
}
