// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper to allow awaiting an event being signalled, similar to an AutoResetEvent.
	/// </summary>
	public class AsyncEvent
	{
		/// <summary>
		/// Completion source to wait on
		/// </summary>
		TaskCompletionSource<bool> _source = new(TaskCreationOptions.RunContinuationsAsynchronously);

		/// <summary>
		/// Legacy name for <see cref="Pulse"/>.
		/// </summary>
		public void Set() => Pulse();

		/// <summary>
		/// Pulse the event, allowing any captured copies of the task to continue.
		/// </summary>
		public void Pulse()
		{
			for (; ; )
			{
				TaskCompletionSource<bool> prevSource = _source;
				if (prevSource.Task.IsCompleted)
				{
					// The task source is latched or has been set from another thread. Either way behaves identically wrt the contract of this method.
					break;
				}
				if (Interlocked.CompareExchange(ref _source, new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously), prevSource) == prevSource)
				{
					prevSource.TrySetResult(true);
					break;
				}
			}
		}

		/// <summary>
		/// Resets an event which is latched to the set state.
		/// </summary>
		public void Reset()
		{
			for (; ; )
			{
				TaskCompletionSource<bool> prevSource = _source;
				if (!prevSource.Task.IsCompleted)
				{
					break;
				}
				else if (Interlocked.CompareExchange(ref _source, new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously), prevSource) == prevSource)
				{
					break;
				}
			}
		}

		/// <summary>
		/// Move the task to completed without returning it to an unset state. A call to Reset() is required to clear it.
		/// </summary>
		public void Latch()
		{
			_source.TrySetResult(true);
		}

		/// <summary>
		/// Determines if this event is set
		/// </summary>
		/// <returns>True if the event is set</returns>
		public bool IsSet()
		{
			return _source.Task.IsCompleted;
		}

		/// <summary>
		/// Waits for this event to be set
		/// </summary>
		public Task Task => _source.Task;
	}
}
