// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.ExceptionServices;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Manages a set of async tasks forming a pipeline, which can be cancelled and awaited as a complete unit. The first exception thrown by an individual task is captured and reported back to the main thread.
	/// </summary>
	public sealed class AsyncPipeline : IAsyncDisposable
	{
		readonly List<Task> _tasks;
		readonly CancellationToken _cancellationToken;
		CancellationTokenSource _cancellationSource;
		ExceptionDispatchInfo? _exceptionDispatchInfo;

		/// <summary>
		/// Tests whether the pipeline has failed
		/// </summary>
		public bool IsFaulted => _exceptionDispatchInfo != null;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the pipeline</param>
		public AsyncPipeline(CancellationToken cancellationToken)
		{
			_tasks = [];
			_cancellationToken = cancellationToken;
			_cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_cancellationSource != null)
			{
				await _cancellationSource.CancelAsync();
			}

			if (_tasks.Count > 0)
			{
				try
				{
					await Task.WhenAll(_tasks);
				}
				catch (OperationCanceledException)
				{
					// Ignore during dispose
				}
				_tasks.Clear();
			}

			if (_cancellationSource != null)
			{
				_cancellationSource.Dispose();
				_cancellationSource = null!;
			}
		}

		/// <summary>
		/// Adds a new task to the pipeline.
		/// </summary>
		/// <param name="taskFunc">Method to execute</param>
		public Task AddTask(Func<CancellationToken, Task> taskFunc)
		{
			Task task = Task.Run(() => RunGuardedAsync(taskFunc), _cancellationSource.Token);
			_tasks.Add(task);
			return task;
		}

		/// <summary>
		/// Adds several tasks to the pipeline.
		/// </summary>
		/// <param name="count">Number of tasks to run</param>
		/// <param name="taskFunc">Method to execute</param>
		public Task[] AddTasks(int count, Func<CancellationToken, Task> taskFunc)
		{
			Task[] tasks = new Task[count];
			for (int idx = 0; idx < count; idx++)
			{
				tasks[idx] = AddTask(taskFunc);
			}
			return tasks;
		}

		async Task RunGuardedAsync(Func<CancellationToken, Task> taskFunc)
		{
			try
			{
				await taskFunc(_cancellationSource.Token);
			}
			catch (OperationCanceledException)
			{
				// Ignore
			}
			catch (Exception ex)
			{
				if (_exceptionDispatchInfo == null)
				{
					ExceptionDispatchInfo dispatchInfo = ExceptionDispatchInfo.Capture(ex);
					Interlocked.CompareExchange(ref _exceptionDispatchInfo, dispatchInfo, null);
				}
				await _cancellationSource.CancelAsync();
			}
		}

		/// <summary>
		/// Waits for all tasks to complete, and throws any exceptions 
		/// </summary>
		public async Task WaitForCompletionAsync()
		{
			await Task.WhenAll(_tasks);

			_cancellationToken.ThrowIfCancellationRequested();

			_exceptionDispatchInfo?.Throw();
		}
	}

	/// <summary>
	/// Extension methods for async pipelines
	/// </summary>
	public static class AsyncPipelineExtensions
	{
		/// <summary>
		/// Adds a worker to process items from a channel
		/// </summary>
		/// <typeparam name="T">Item type</typeparam>
		/// <param name="pipeline">Pipeline to add the worker to</param>
		/// <param name="reader">Source for the items</param>
		/// <param name="taskFunc">Action to execute for each item</param>
		public static Task AddTask<T>(this AsyncPipeline pipeline, ChannelReader<T> reader, Func<T, CancellationToken, ValueTask> taskFunc)
			=> pipeline.AddTask(ctx => ProcessItemsAsync(reader, taskFunc, ctx));

		/// <summary>
		/// Adds a worker to process items from a channel
		/// </summary>
		/// <typeparam name="TInput">Input item type</typeparam>
		/// <typeparam name="TOutput">Output item type</typeparam>
		/// <param name="pipeline">Pipeline to add the worker to</param>
		/// <param name="reader">Reader for input items</param>
		/// <param name="writer">Writer for output items</param>
		/// <param name="taskFunc">Action to execute for each item</param>
		public static Task AddTask<TInput, TOutput>(this AsyncPipeline pipeline, ChannelReader<TInput> reader, ChannelWriter<TOutput> writer, Func<TInput, CancellationToken, ValueTask<TOutput>> taskFunc)
			=> pipeline.AddTask(ctx => ProcessItemsAsync(reader, writer, taskFunc, ctx));

		/// <summary>
		/// Adds a worker to process items from a channel
		/// </summary>
		/// <typeparam name="T">Input item type</typeparam>
		/// <param name="pipeline">Pipeline to add the worker to</param>
		/// <param name="count">Number of workers to add</param>
		/// <param name="reader">Reader for input items</param>
		/// <param name="taskFunc">Action to execute for each item</param>
		public static Task[] AddTasks<T>(this AsyncPipeline pipeline, int count, ChannelReader<T> reader, Func<T, CancellationToken, ValueTask> taskFunc)
		{
			Task[] tasks = new Task[count];
			for (int idx = 0; idx < count; idx++)
			{
				tasks[idx] = AddTask(pipeline, reader, taskFunc);
			}
			return tasks;
		}

		/// <summary>
		/// Adds a worker to process items from a channel
		/// </summary>
		/// <typeparam name="TInput">Input item type</typeparam>
		/// <typeparam name="TOutput">Output item type</typeparam>
		/// <param name="pipeline">Pipeline to add the worker to</param>
		/// <param name="count">Number of workers to add</param>
		/// <param name="reader">Reader for input items</param>
		/// <param name="writer">Writer for output items</param>
		/// <param name="taskFunc">Action to execute for each item</param>
		public static Task[] AddTasks<TInput, TOutput>(this AsyncPipeline pipeline, int count, ChannelReader<TInput> reader, ChannelWriter<TOutput> writer, Func<TInput, CancellationToken, ValueTask<TOutput>> taskFunc)
		{
			Task[] tasks = new Task[count];
			for (int idx = 0; idx < count; idx++)
			{
				tasks[idx] = AddTask(pipeline, reader, writer, taskFunc);
			}
			return tasks;
		}

		static async Task ProcessItemsAsync<T>(ChannelReader<T> reader, Func<T, CancellationToken, ValueTask> taskFunc, CancellationToken cancellationToken)
		{
			while (await reader.WaitToReadAsync(cancellationToken))
			{
				T? item;
				if (reader.TryRead(out item))
				{
					await taskFunc(item, cancellationToken);
				}
			}
		}

		static async Task ProcessItemsAsync<TInput, TOutput>(ChannelReader<TInput> reader, ChannelWriter<TOutput> writer, Func<TInput, CancellationToken, ValueTask<TOutput>> taskFunc, CancellationToken cancellationToken)
		{
			while (await reader.WaitToReadAsync(cancellationToken))
			{
				TInput? input;
				if (reader.TryRead(out input))
				{
					TOutput output = await taskFunc(input, cancellationToken);
					await writer.WriteAsync(output, cancellationToken);
				}
			}
		}
	}
}
