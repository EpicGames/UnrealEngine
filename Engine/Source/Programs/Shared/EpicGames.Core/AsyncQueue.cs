// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Manages a set of tasks run in the background on a fixed number of tasks. Similar to <see cref="AsyncThreadPoolWorkQueue"/>.
	/// </summary>
	public sealed class AsyncQueue : IAsyncDisposable
	{
		readonly List<Task> _workerTasks = [];
		readonly Channel<Func<CancellationToken, Task>> _queuedTasks = Channel.CreateUnbounded<Func<CancellationToken, Task>>();
		readonly CancellationTokenSource _cancellationSource = new();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="numWorkers">Number of concurrent workers executing tasks</param>
		public AsyncQueue(int numWorkers)
		{
			for (int idx = 0; idx < numWorkers; idx++)
			{
				_workerTasks.Add(Task.Run(() => RunWorkerAsync(), _cancellationSource.Token));
			}
		}

		/// <summary>
		/// Finish executing all the tasks and wait for them to complete
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			_queuedTasks.Writer.TryComplete();
			await Task.WhenAll(_workerTasks).WaitAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _cancellationSource.CancelAsync();
			await StopAsync(CancellationToken.None).ConfigureAwait(false);

			_cancellationSource.Dispose();
		}

		/// <summary>
		/// Enqueue a task to be executed
		/// </summary>
		/// <param name="task"></param>
		public void Enqueue(Func<CancellationToken, Task> task)
			=> _queuedTasks.Writer.TryWrite(task);

		/// <summary>
		/// Worker executing tasks. Any exception encountered will be bubbled up as all calls are awaited.
		/// </summary>
		private async Task RunWorkerAsync()
		{
			try
			{
				await foreach (Func<CancellationToken, Task> task in _queuedTasks.Reader.ReadAllAsync(_cancellationSource.Token))
				{
					try
					{
						await task(_cancellationSource.Token);
					}
					catch
					{
					}
				}
			}
			catch (OperationCanceledException)
			{
			}
		}
	}
}
