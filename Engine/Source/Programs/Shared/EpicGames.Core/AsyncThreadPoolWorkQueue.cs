// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;

namespace EpicGames.Core;

/// <summary>
/// Allows queuing a large number of async tasks to the default thread pool
/// In contrast to <see cref="ThreadPoolWorkQueue" />, the number of concurrent workers can be set 
/// </summary>
public sealed class AsyncThreadPoolWorkQueue : IDisposable
{
	/// <summary>
	/// Object used for controlling access to updating _lockObject
	/// </summary>
	private readonly Lock _lockObject = new();

	/// <summary>
	/// Event which indicates whether the queue is empty.
	/// </summary>
	private readonly ManualResetEvent _emptyEvent = new(true);

	/// <summary>
	/// Number of concurrent workers that will process tasks
	/// </summary>
	private readonly int _numWorkers;

	/// <summary>
	/// Number of tasks remaining in the queue. This is updated in an atomic way.
	/// </summary>
	private int _numOutstandingTasks;

	/// <summary>
	/// Channel used as the queue for tasks
	/// </summary>
	readonly Channel<Func<CancellationToken, Task>> _tasks = Channel.CreateUnbounded<Func<CancellationToken, Task>>();

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="numWorkers">Number of concurrent workers executing tasks</param>
	public AsyncThreadPoolWorkQueue(int numWorkers)
	{
		_numWorkers = numWorkers;
	}

	/// <summary>
	/// Dispose
	/// </summary>
	public void Dispose()
	{
		_emptyEvent.Dispose();
	}

	/// <summary>
	/// Enqueue a task to be executed
	/// </summary>
	/// <param name="task"></param>
	public async Task EnqueueAsync(Func<CancellationToken, Task> task)
	{
		if (Interlocked.Increment(ref _numOutstandingTasks) == 1)
		{
			SetEventState();
		}
		await _tasks.Writer.WriteAsync(task);
	}

	/// <summary>
	/// Execute all enqueued tasks
	/// In contrast to Parallel.ForEachAsync, new tasks can be enqueued during execution.
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	public async Task ExecuteAsync(CancellationToken cancellationToken = default)
	{
		if (_numOutstandingTasks < 1)
		{
			return;
		}

		List<Task> tasks = [];
		for (int i = 0; i < _numWorkers; i++)
		{
			tasks.Add(Task.Run(() => WorkerAsync(cancellationToken), cancellationToken));
		}

		Task queueCompletionTask = Task.Run(async () =>
		{
			await _emptyEvent.WaitOneAsync(cancellationToken: cancellationToken);
			_tasks.Writer.Complete();
		}, cancellationToken);

		tasks.Add(queueCompletionTask);
		await Task.WhenAll(tasks);
	}

	/// <summary>
	/// Worker executing tasks. Any exception encountered will be bubbled up as all calls are awaited.
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	private async Task WorkerAsync(CancellationToken cancellationToken)
	{
		await foreach (Func<CancellationToken, Task> task in _tasks.Reader.ReadAllAsync(cancellationToken))
		{
			try
			{
				await task(cancellationToken);
			}
			finally
			{
				if (Interlocked.Decrement(ref _numOutstandingTasks) == 0)
				{
					SetEventState();
				}
			}
		}
	}

	/// <summary>
	/// When the number of outstanding tasks transitions between 0 and 1 and vice-versa, we need to set the event 
	/// based on the actual counter. While the adjustment of the counter happens with interlocked instructions,
	/// the setting of the event must be done under a lock to serialize the set/reset
	/// </summary>
	private void SetEventState()
	{
		lock (_lockObject)
		{
			if (_numOutstandingTasks > 0)
			{
				_emptyEvent?.Reset();
			}
			else
			{
				_emptyEvent?.Set();
			}
		}
	}
}
