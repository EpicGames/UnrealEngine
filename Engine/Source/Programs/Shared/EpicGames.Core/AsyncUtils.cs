// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility functions for manipulating async tasks
	/// </summary>
	public static class AsyncUtils
	{
		/// <summary>
		/// Converts a cancellation token to a waitable task
		/// </summary>
		/// <param name="token">Cancellation token</param>
		/// <returns></returns>
		public static Task AsTask(this CancellationToken token)
		{
			return Task.Delay(-1, token).ContinueWith(x => { }, TaskScheduler.Default);
		}

		/// <summary>
		/// Converts a cancellation token to a waitable task
		/// </summary>
		/// <param name="token">Cancellation token</param>
		/// <returns></returns>
		public static Task<T> AsTask<T>(this CancellationToken token)
		{
			return Task.Delay(-1, token).ContinueWith(_ => Task.FromCanceled<T>(token), TaskScheduler.Default).Unwrap();
		}

		/// <summary>
		/// Waits for a task to complete, ignoring any cancellation exceptions
		/// </summary>
		/// <param name="task">Task to wait for</param>
		public static async Task IgnoreCanceledExceptionsAsync(this Task task)
		{
			try
			{
				await task.ConfigureAwait(false);
			}
			catch (OperationCanceledException)
			{
			}
		}

		/// <summary>
		/// Returns a task that will be abandoned if a cancellation token is activated. This differs from the normal cancellation pattern in that the task will run to completion, but waiting for it can be cancelled.
		/// </summary>
		/// <param name="task">Task to wait for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Wrapped task</returns>
		public static async Task<T> AbandonOnCancelAsync<T>(this Task<T> task, CancellationToken cancellationToken)
		{
			if (cancellationToken.CanBeCanceled)
			{
				await await Task.WhenAny(task, Task.Delay(-1, cancellationToken)); // Double await to ensure cancellation exception is rethrown if returned
			}
			return await task;
		}

		/// <summary>
		/// Attempts to get the result of a task, if it has finished
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="task"></param>
		/// <param name="result"></param>
		/// <returns></returns>
		public static bool TryGetResult<T>(this Task<T> task, out T result)
		{
			if (task.IsCompleted)
			{
				result = task.Result;
				return true;
			}
			else
			{
				result = default!;
				return false;
			}
		}

		/// <summary>
		/// Waits for a time period to elapse or the task to be cancelled, without throwing an cancellation exception
		/// </summary>
		/// <param name="time">Time to wait</param>
		/// <param name="token">Cancellation token</param>
		/// <returns></returns>
		public static Task DelayNoThrow(TimeSpan time, CancellationToken token)
		{
			return Task.Delay(time, token).ContinueWith(x => { }, CancellationToken.None, TaskContinuationOptions.None, TaskScheduler.Default);
		}

		/// <summary>
		/// Removes all the complete tasks from a list, allowing each to throw exceptions as necessary
		/// </summary>
		/// <param name="tasks">List of tasks to remove tasks from</param>
		public static void RemoveCompleteTasks(this List<Task> tasks)
		{
			List<Exception> exceptions = [];

			int outIdx = 0;
			for (int idx = 0; idx < tasks.Count; idx++)
			{
				if (tasks[idx].IsCompleted)
				{
					AggregateException? exception = tasks[idx].Exception;
					if (exception != null)
					{
						exceptions.AddRange(exception.InnerExceptions);
					}
				}
				else
				{
					if (idx != outIdx)
					{
						tasks[outIdx] = tasks[idx];
					}
					outIdx++;
				}
			}
			tasks.RemoveRange(outIdx, tasks.Count - outIdx);

			if (exceptions.Count > 0)
			{
				throw new AggregateException(exceptions);
			}
		}

		/// <summary>
		/// Removes all the complete tasks from a list, allowing each to throw exceptions as necessary
		/// </summary>
		/// <param name="tasks">List of tasks to remove tasks from</param>
		/// <returns>Return values from the completed tasks</returns>
		public static List<T> RemoveCompleteTasks<T>(this List<Task<T>> tasks)
		{
			List<T> results = [];

			int outIdx = 0;
			for (int idx = 0; idx < tasks.Count; idx++)
			{
				if (tasks[idx].IsCompleted)
				{
					results.Add(tasks[idx].Result);
				}
				else if (idx != outIdx)
				{
					tasks[outIdx++] = tasks[idx];
				}
			}
			tasks.RemoveRange(outIdx, tasks.Count - outIdx);

			return results;
		}

		/// <summary>
		/// Starts prefetching the next item from an async enumerator while the current one is being processes
		/// </summary>
		/// <typeparam name="T">Value type</typeparam>
		/// <param name="source">Sequence to enumerate</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async IAsyncEnumerable<T> PrefetchAsync<T>(this IAsyncEnumerable<T> source, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			await using IAsyncEnumerator<T> enumerator = source.GetAsyncEnumerator(cancellationToken);
			if (await enumerator.MoveNextAsync())
			{
				T value = enumerator.Current;

				for (; ; )
				{
					cancellationToken.ThrowIfCancellationRequested();

					Task<bool> task = enumerator.MoveNextAsync().AsTask();
					try
					{
						yield return value;
					}
					finally
					{
						await task; // Async state machine throws a NotSupportedException if disposed before awaiting this task
					}

					if (!await task)
					{
						break;
					}

					value = enumerator.Current;
				}
			}
		}

		/// <summary>
		/// Starts prefetching a number of items from an async enumerator while the current one is being processes
		/// </summary>
		/// <typeparam name="T">Value type</typeparam>
		/// <param name="source">Sequence to enumerate</param>
		/// <param name="count">Number of items to prefetch</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static IAsyncEnumerable<T> Prefetch<T>(this IAsyncEnumerable<T> source, int count, CancellationToken cancellationToken = default)
		{
			if (count == 0)
			{
				return source;
			}
			else
			{
				return Prefetch(source, count - 1, cancellationToken);
			}
		}

		/// <summary>
		/// Waits for a native wait handle to be signaled
		/// </summary>
		/// <param name="handle">Handle to wait for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WaitOneAsync(this WaitHandle handle, CancellationToken cancellationToken = default) => handle.WaitOneAsync(-1, cancellationToken);

		/// <summary>
		/// Waits for a native wait handle to be signaled
		/// </summary>
		/// <param name="handle">Handle to wait for</param>
		/// <param name="timeoutMs">Timeout for the wait</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task WaitOneAsync(this WaitHandle handle, int timeoutMs, CancellationToken cancellationToken = default)
		{
			TaskCompletionSource<bool> completionSource = new();

			RegisteredWaitHandle waitHandle = ThreadPool.RegisterWaitForSingleObject(handle, (state, timedOut) => ((TaskCompletionSource<bool>)state!).TrySetResult(!timedOut), completionSource, timeoutMs, true);
			try
			{
				using IDisposable registration = cancellationToken.Register(x => ((TaskCompletionSource<bool>)x!).SetCanceled(), completionSource);
				await completionSource.Task;
			}
			finally
			{
				waitHandle.Unregister(null);
			}
		}
	}
}
