// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Runs a task in the background and allows stopping it on demand
	/// </summary>
	public sealed class BackgroundTask : IAsyncDisposable
	{
		readonly Func<CancellationToken, Task> _runTask;
		Task? _task;
		readonly CancellationTokenSource _cancellationTokenSource = new();

		/// <summary>
		/// Accessor for the underlying task
		/// </summary>
		public Task? Task => _task;

		/// <summary>
		/// Constructor. Note that the task does not start until <see cref="Start"/> is called.
		/// </summary>
		/// <param name="runTask"></param>
		public BackgroundTask(Func<CancellationToken, Task> runTask)
		{
			_runTask = runTask;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await StopAsync().ConfigureAwait(false);
			_cancellationTokenSource.Dispose();
		}

		/// <summary>
		/// Creates and starts a new background task instance
		/// </summary>
		/// <param name="runTask"></param>
		/// <returns></returns>
		public static BackgroundTask StartNew(Func<CancellationToken, Task> runTask)
		{
			BackgroundTask task = new(runTask);
			task.Start();
			return task;
		}

		/// <summary>
		/// Creates and starts a new background task instance
		/// </summary>
		/// <param name="runTask"></param>
		/// <returns>New background task instance</returns>
		public static BackgroundTask<T> StartNew<T>(Func<CancellationToken, Task<T>> runTask)
		{
			BackgroundTask<T> task = new(runTask);
			task.Start();
			return task;
		}

		/// <summary>
		/// Starts the task
		/// </summary>
		public void Start()
		{
			if (_task != null)
			{
				throw new InvalidOperationException("Background task is already running");
			}
			_task = Task.Run(() => _runTask(_cancellationTokenSource.Token), _cancellationTokenSource.Token);
		}

		/// <summary>
		/// Signals the cancellation token and waits for the task to finish
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task StopAsync(CancellationToken cancellationToken = default)
		{
			if (_task != null && !_task.IsCompleted)
			{
				try
				{
					await _cancellationTokenSource.CancelAsync();
					await _task.WaitAsync(cancellationToken).ConfigureAwait(false);
				}
				catch (OperationCanceledException)
				{
				}
			}
		}

		/// <summary>
		/// Waits for the task to complete
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task WaitAsync(CancellationToken cancellationToken)
		{
			if (_task == null)
			{
				throw new InvalidOperationException("Task has not been started.");
			}
			await _task.WaitAsync(cancellationToken);
		}
	}

	/// <summary>
	/// Runs a task in the background and allows stopping it on demand
	/// </summary>
	public sealed class BackgroundTask<T> : IAsyncDisposable
	{
		readonly Func<CancellationToken, Task<T>> _runTask;
		Task<T>? _task;
		readonly CancellationTokenSource _cancellationTokenSource = new();

		/// <summary>
		/// Accessor for the underlying task
		/// </summary>
		public Task<T>? Task => _task;

		/// <summary>
		/// Constructor. Note that the task does not start until <see cref="Start"/> is called.
		/// </summary>
		/// <param name="runTask"></param>
		public BackgroundTask(Func<CancellationToken, Task<T>> runTask)
		{
			_runTask = runTask;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await StopAsync();
			_cancellationTokenSource.Dispose();
		}

		/// <summary>
		/// Starts the task
		/// </summary>
		public void Start()
		{
			if (_task != null)
			{
				throw new InvalidOperationException("Background task is already running");
			}
			_task = System.Threading.Tasks.Task.Run(() => _runTask(_cancellationTokenSource.Token), _cancellationTokenSource.Token);
		}

		/// <summary>
		/// Signals the cancellation token and waits for the task to finish
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task StopAsync(CancellationToken cancellationToken = default)
		{
			if (_task != null && !_task.IsCompleted)
			{
				try
				{
					await _cancellationTokenSource.CancelAsync();
					await _task.WaitAsync(cancellationToken);
				}
				catch (OperationCanceledException)
				{
				}
			}
		}

		/// <summary>
		/// Waits for the task to complete
		/// </summary>
		public Task<T> WaitAsync(CancellationToken cancellationToken = default)
		{
			if (_task == null)
			{
				throw new InvalidOperationException("Task has not been started.");
			}
			return _task.WaitAsync(cancellationToken);
		}
	}
}
