// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq.Expressions;
using System.Reflection;
using System.Reflection.Emit;

namespace EpicGames.Core
{
	/// <summary>
	/// Common functionality for dealing with generic type serialization in binary archives
	/// </summary>
	public static class BinaryArchive
	{
		class Instantiator<T> where T : class, new()
		{
			public static T Instance = new();
		}

		sealed class TypeSerializer
		{
			public MethodInfo ReadMethodInfo { get; }
			public Func<BinaryArchiveReader, object> ReadMethod { get; }
			public MethodInfo WriteMethodInfo { get; }
			public Action<BinaryArchiveWriter, object> WriteMethod { get; }

			public TypeSerializer(MethodInfo readMethodInfo, MethodInfo writeMethodInfo)
			{
				ReadMethodInfo = readMethodInfo;
				WriteMethodInfo = writeMethodInfo;

				Type type = readMethodInfo.ReturnType;
				ReadMethod = CreateBoxedReadMethod(type, readMethodInfo);
				WriteMethod = CreateBoxedWriteMethod(type, writeMethodInfo);
			}

			static Func<BinaryArchiveReader, object> CreateBoxedReadMethod(Type type, MethodInfo methodInfo)
			{
				DynamicMethod readerMethod = new($"Boxed_{methodInfo.Name}", typeof(object), [typeof(BinaryArchiveReader)]);

				ILGenerator generator = readerMethod.GetILGenerator(64);
				generator.Emit(OpCodes.Ldarg_0);
				generator.EmitCall(OpCodes.Call, methodInfo, null);
				if (!type.IsClass)
				{
					generator.Emit(OpCodes.Box);
				}
				generator.Emit(OpCodes.Ret);

				return (Func<BinaryArchiveReader, object>)readerMethod.CreateDelegate(typeof(Func<BinaryArchiveReader, object>));
			}

			static Action<BinaryArchiveWriter, object> CreateBoxedWriteMethod(Type type, MethodInfo methodInfo)
			{
				DynamicMethod writerMethod = new($"Boxed_{methodInfo.Name}", typeof(void), [typeof(BinaryArchiveWriter), typeof(object)]);

				ILGenerator generator = writerMethod.GetILGenerator(64);
				generator.Emit(OpCodes.Ldarg_0);
				generator.Emit(OpCodes.Ldarg_1);
				generator.Emit(OpCodes.Unbox_Any, type);
				generator.EmitCall(OpCodes.Call, methodInfo, null);
				generator.Emit(OpCodes.Ret);

				return (Action<BinaryArchiveWriter, object>)writerMethod.CreateDelegate(typeof(Action<BinaryArchiveWriter, object>));
			}

			public static TypeSerializer Create<T>(Expression<Func<BinaryArchiveReader, T>> readerExpr, Expression<Action<BinaryArchiveWriter, T>> writerExpr)
			{
				MethodInfo readMethod = ((MethodCallExpression)readerExpr.Body).Method;
				MethodInfo writeMethod = ((MethodCallExpression)writerExpr.Body).Method;
				return new TypeSerializer(readMethod, writeMethod);
			}
		}

		static readonly ConcurrentDictionary<Type, TypeSerializer> s_typeToSerializerInfo = new()
		{
			[typeof(byte)] = TypeSerializer.Create(reader => reader.ReadByte(), (writer, value) => writer.WriteByte(value)),
			[typeof(sbyte)] = TypeSerializer.Create(reader => reader.ReadSignedByte(), (writer, value) => writer.WriteSignedByte(value)),
			[typeof(short)] = TypeSerializer.Create(reader => reader.ReadShort(), (writer, value) => writer.WriteShort(value)),
			[typeof(ushort)] = TypeSerializer.Create(reader => reader.ReadUnsignedShort(), (writer, value) => writer.WriteUnsignedShort(value)),
			[typeof(int)] = TypeSerializer.Create(reader => reader.ReadInt(), (writer, value) => writer.WriteInt(value)),
			[typeof(uint)] = TypeSerializer.Create(reader => reader.ReadUnsignedInt(), (writer, value) => writer.WriteUnsignedInt(value)),
			[typeof(long)] = TypeSerializer.Create(reader => reader.ReadLong(), (writer, value) => writer.WriteLong(value)),
			[typeof(ulong)] = TypeSerializer.Create(reader => reader.ReadUnsignedLong(), (writer, value) => writer.WriteUnsignedLong(value)),
			[typeof(string)] = TypeSerializer.Create(reader => reader.ReadString(), (writer, value) => writer.WriteString(value))
		};

		static readonly ConcurrentDictionary<Type, ContentHash> s_typeToDigest = new();
		static readonly ConcurrentDictionary<ContentHash, Type> s_digestToType = new();

		static MethodInfo GetMethodInfo(Expression<Action> expr)
		{
			return ((MethodCallExpression)expr.Body).Method;
		}

		static MethodInfo GetMethodInfo<T>(Expression<Action<T>> expr)
		{
			return ((MethodCallExpression)expr.Body).Method;
		}

		static MethodInfo GetGenericMethodInfo<T, T1>(Expression<Action<T, T1>> expr)
		{
			return ((MethodCallExpression)expr.Body).Method.GetGenericMethodDefinition();
		}

		static readonly MethodInfo s_readBoolMethodInfo = GetMethodInfo<BinaryArchiveReader>(x => x.ReadBool());
		static readonly MethodInfo s_readArrayMethodInfo = GetGenericMethodInfo<BinaryArchiveReader, Func<int>>((x, y) => x.ReadArray(y));
		static readonly MethodInfo s_readPolymorphicObjectMethodInfo = GetMethodInfo(() => ReadObject(null!));
		static readonly MethodInfo s_writeBoolMethodInfo = GetMethodInfo<BinaryArchiveWriter>(x => x.WriteBool(false));
		static readonly MethodInfo s_writeArrayMethodInfo = GetGenericMethodInfo<BinaryArchiveWriter, Action<int>>((x, y) => x.WriteArray(null!, y));
		static readonly MethodInfo s_writePolymorphicObjectMethodInfo = GetMethodInfo(() => WritePolymorphicObject(null!, null!));

		/// <summary>
		/// Writes an arbitrary type to the given archive. May be an object or value type.
		/// </summary>
		/// <typeparam name="T">The type to write</typeparam>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void Write<T>(this BinaryArchiveWriter writer, T value)
		{
			Type type = typeof(T);
			if (type.IsClass && !type.IsSealed)
			{
				writer.WriteObjectReference(value!, () => WriteNewPolymorphicObject(writer, value!));
			}
			else
			{
				FindOrAddSerializerInfo(type).WriteMethod(writer, value!);
			}
		}

		/// <summary>
		/// Registers the given type. Allows serializing/deserializing it through calls to ReadType/WriteType.
		/// </summary>
		/// <param name="type">Type to register</param>
		public static void RegisterType(Type type)
		{
			ContentHash digest = ContentHash.MD5($"{type.Assembly.FullName}\n{type.FullName}");
			s_typeToDigest.TryAdd(type, digest);
			s_digestToType.TryAdd(digest, type);
		}

		/// <summary>
		/// Registers all types in the given assembly with the <see cref="BinarySerializableAttribute"/> attribute for serialization
		/// </summary>
		/// <param name="assembly">Assembly to search in</param>
		public static void RegisterTypes(Assembly assembly)
		{
			foreach (Type type in assembly.GetTypes())
			{
				if (type.GetCustomAttribute<BinarySerializableAttribute>() != null)
				{
					RegisterType(type);
				}
			}
		}

		/// <summary>
		/// Reads a tyoe from the archive
		/// </summary>
		/// <param name="reader">Reader to deserializer the type from</param>
		/// <returns>The matching type</returns>
		public static Type? ReadType(this BinaryArchiveReader reader)
		{
			return reader.ReadObjectReference(() => s_digestToType[reader.ReadContentHash()!]);
		}

		/// <summary>
		/// Writes a type to an archive
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="type">The type to serialize</param>
		public static void WriteType(this BinaryArchiveWriter writer, Type type)
		{
			writer.WriteObjectReference(type, () => writer.WriteContentHash(s_typeToDigest[type]));
		}

		static object? ReadNewPolymorphicObject(BinaryArchiveReader reader)
		{
			Type? finalType = reader.ReadType();
			if (finalType == null)
			{
				return null;
			}

			TypeSerializer info = FindOrAddSerializerInfo(finalType);
			return info.ReadMethod(reader)!;
		}

		/// <summary>
		/// Reads an object from the archive
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The deserialized object</returns>
		public static object? ReadObject(this BinaryArchiveReader reader)
		{
			return reader.ReadUntypedObjectReference(() => ReadNewPolymorphicObject(reader));
		}

		static void WriteNewPolymorphicObject(BinaryArchiveWriter writer, object value)
		{
			Type actualType = value.GetType();
			writer.WriteType(actualType);

			TypeSerializer info = FindOrAddSerializerInfo(actualType);
			info.WriteMethod(writer, value);
		}

		static void WritePolymorphicObject(this BinaryArchiveWriter writer, object value)
		{
			writer.WriteObjectReference(value, () => WriteNewPolymorphicObject(writer, value));
		}

		/// <summary>
		/// Finds an existing serializer for the given type, or creates one
		/// </summary>
		/// <param name="type">Type to create a serializer for</param>
		/// <param name="converterType">Type of the converter to use</param>
		/// <returns>Instance of the type serializer</returns>
		static TypeSerializer FindOrAddSerializerInfo(Type type, Type? converterType = null)
		{
			Type serializerKey = converterType ?? type;

			// Get the serializer info
			if (!s_typeToSerializerInfo.TryGetValue(serializerKey, out TypeSerializer? serializerInfo))
			{
				lock (s_typeToSerializerInfo)
				{
					serializerInfo = CreateTypeSerializer(type, converterType);
					s_typeToSerializerInfo[serializerKey] = serializerInfo;
				}
			}

			return serializerInfo;
		}

		/// <summary>
		/// Creates a serializer for the given type
		/// </summary>
		/// <param name="type">Type to create a serializer from</param>
		/// <param name="converterType">Converter for the type</param>
		/// <returns>New instance of a type serializer</returns>
		static TypeSerializer CreateTypeSerializer(Type type, Type? converterType = null)
		{
			// If there's a specific converter defined, generate serialization methods using that
			if (converterType != null)
			{
				// Make sure the converter is derived from IBinaryConverter
				Type interfaceType = typeof(IBinaryConverter<>).MakeGenericType(type);
				if (!interfaceType.IsAssignableFrom(converterType))
				{
					throw new NotImplementedException($"Converter does not implement IBinaryArchiveConverter<{type.Name}>");
				}

				// Instantiate the converter, and store it in a static variable
				Type instantiatorType = typeof(Instantiator<>).MakeGenericType(converterType);
				FieldInfo converterField = instantiatorType.GetField(nameof(Instantiator<object>.Instance))!;

				// Get the mapping between interface methods and instance methods
				InterfaceMapping interfaceMapping = converterType.GetInterfaceMap(interfaceType);

				// Create a reader method
				DynamicMethod readerMethod = new($"BinaryArchiveReader_Dynamic_{type.Name}_With_{converterType.Name}", type, [typeof(BinaryArchiveReader)]);
				int readerIdx = Array.FindIndex(interfaceMapping.InterfaceMethods, x => x.Name.Equals(nameof(IBinaryConverter<object>.Read), StringComparison.Ordinal));
				GenerateConverterReaderForwardingMethod(readerMethod, converterField, interfaceMapping.TargetMethods[readerIdx]);

				// Create a writer method
				DynamicMethod writerMethod = new($"BinaryArchiveWriter_Dynamic_{type.Name}_With_{converterType.Name}", typeof(void), [typeof(BinaryArchiveWriter), typeof(object)]);
				int writerIdx = Array.FindIndex(interfaceMapping.InterfaceMethods, x => x.Name.Equals(nameof(IBinaryConverter<object>.Write), StringComparison.Ordinal));
				GenerateConverterWriterForwardingMethod(writerMethod, converterField, interfaceMapping.TargetMethods[writerIdx]);

				return new TypeSerializer(readerMethod, writerMethod);
			}

			// Get the default converter type
			if (BinaryConverter.TryGetConverterType(type, out Type? defaultConverterType))
			{
				return FindOrAddSerializerInfo(type, defaultConverterType);
			}

			// Check if it's an array
			if (type.IsArray)
			{
				Type elementType = type.GetElementType()!;

				MethodInfo readerMethod = s_readArrayMethodInfo.MakeGenericMethod(elementType);
				MethodInfo writerMethod = s_writeArrayMethodInfo.MakeGenericMethod(elementType);

				return new TypeSerializer(readerMethod, writerMethod);
			}

			// Check if it's a class
			if (type.IsClass)
			{
				// Get all the class properties
				List<PropertyInfo> properties = [];
				foreach (PropertyInfo property in type.GetProperties(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance))
				{
					MethodInfo? getMethod = property.GetGetMethod();
					MethodInfo? setMethod = property.GetSetMethod();
					if (getMethod != null && setMethod != null && property.GetCustomAttribute<BinaryIgnoreAttribute>() == null)
					{
						properties.Add(property);
					}
				}

				// Find the type constructor
				ConstructorInfo? constructorInfo = type.GetConstructor(BindingFlags.Instance | BindingFlags.Public | BindingFlags.NonPublic, null, [], null)
					?? throw new NotImplementedException($"Type '{type.Name}' does not have a parameterless constructor");

				// Create the methods
				DynamicMethod writerMethod = new($"BinaryArchiveWriter_Dynamic_{type.Name}", typeof(void), [typeof(BinaryArchiveWriter), type]);
				GenerateClassWriterMethod(writerMethod, properties);

				DynamicMethod readerMethod = new($"BinaryArchiveReader_Dynamic_{type.Name}", type, [typeof(BinaryArchiveReader)]);
				GenerateClassReaderMethod(readerMethod, constructorInfo, properties);

				return new TypeSerializer(readerMethod, writerMethod);
			}

			throw new NotImplementedException($"Unable to create a serializer for {type.Name}");
		}

		static void GenerateClassWriterMethod(DynamicMethod writerMethod, List<PropertyInfo> properties)
		{
			// Get the IL generator
			ILGenerator generator = writerMethod.GetILGenerator(256 + (properties.Count * 24));

			// Check if the argument is null, and write 'false' if it is, then early out.
			Label nonNullInstance = generator.DefineLabel();
			generator.Emit(OpCodes.Ldarg_1);                    // [0] = Arg1 (Instance)
			generator.Emit(OpCodes.Brtrue, nonNullInstance);    // [EMPTY] Instance != null -> NonNullInstance

			generator.Emit(OpCodes.Ldarg_0);                    // [0] = Arg0 (Writer)
			generator.Emit(OpCodes.Ldc_I4_0);                   // [1] = false
			generator.EmitCall(OpCodes.Call, s_writeBoolMethodInfo, null);  // [EMPTY] WriteBool(Write, false)

			generator.Emit(OpCodes.Ret);                        // [EMPTY] Return

			// Otherwise write true.
			generator.MarkLabel(nonNullInstance);

			generator.Emit(OpCodes.Ldarg_0);                    // [0] = Arg0 (Writer)
			generator.Emit(OpCodes.Ldc_I4_1);                   // [1] = true
			generator.EmitCall(OpCodes.Call, s_writeBoolMethodInfo, null);  // [EMPTY] WriteBool(Writer, true)

			// Write all the properties
			foreach (PropertyInfo property in properties)
			{
				BinaryConverterAttribute? converter = property.GetCustomAttribute<BinaryConverterAttribute>();

				MethodInfo? writePropertyMethod;
				if (property.PropertyType.IsClass && !property.PropertyType.IsSealed && converter == null)
				{
					writePropertyMethod = s_writePolymorphicObjectMethodInfo;
				}
				else
				{
					writePropertyMethod = FindOrAddSerializerInfo(property.PropertyType, converter?.Type).WriteMethodInfo;
				}

				generator.Emit(OpCodes.Ldarg_0);                                            // [0] = Arg0 (BinaryArchiveWriter)
				generator.Emit(OpCodes.Ldarg_1);                                            // [1] = Arg1 (Instance)
				generator.EmitCall(OpCodes.Call, property.GetGetMethod(true)!, null);       // [1] = GetMethod(Instance)
				generator.EmitCall(OpCodes.Call, writePropertyMethod, null);                // [EMPTY] Write(BinaryArchiveWriter, GetMethod(Instance));
			}

			// Return
			generator.Emit(OpCodes.Ret);
		}

		static void GenerateClassReaderMethod(DynamicMethod readerMethod, ConstructorInfo constructorInfo, List<PropertyInfo> properties)
		{
			// Get the IL generator
			ILGenerator generator = readerMethod.GetILGenerator(256 + (properties.Count * 24));

			// Check if it was a null instance
			generator.Emit(OpCodes.Ldarg_0);                    // [0] = Arg1 (Reader)
			generator.EmitCall(OpCodes.Call, s_readBoolMethodInfo, null);   // [0] = Reader.ReadBool()

			Label nonNullInstance = generator.DefineLabel();
			generator.Emit(OpCodes.Brtrue, nonNullInstance);    // Bool != null -> NonNullInstance
			generator.Emit(OpCodes.Ldnull);                     // [0] = null
			generator.Emit(OpCodes.Ret);                        // return

			// Construct an instance
			generator.MarkLabel(nonNullInstance);
			generator.Emit(OpCodes.Newobj, constructorInfo);    // [0] = new Type()

			// Read all the properties
			foreach (PropertyInfo property in properties)
			{
				BinaryConverterAttribute? converter = property.GetCustomAttribute<BinaryConverterAttribute>();

				MethodInfo? readPropertyMethod;
				if (property.PropertyType.IsClass && !property.PropertyType.IsSealed && converter == null)
				{
					readPropertyMethod = s_readPolymorphicObjectMethodInfo;
				}
				else
				{
					readPropertyMethod = FindOrAddSerializerInfo(property.PropertyType, converter?.Type).ReadMethodInfo;
				}

				generator.Emit(OpCodes.Dup);                                                // [1] = new Type()

				generator.Emit(OpCodes.Ldarg_0);                                            // [2] = Arg1 (Reader)
				generator.Emit(OpCodes.Call, readPropertyMethod);                           // [2] = Reader.ReadMethod() or ReadMethod(Reader)

				generator.Emit(OpCodes.Call, property.GetSetMethod(true)!);                 // SetMethod(new Type(), ReadMethod(Reader))
			}

			// Return the instance
			generator.Emit(OpCodes.Ret);                        // [0] = new Type()
		}

		static void GenerateConverterReaderForwardingMethod(DynamicMethod readerMethod, FieldInfo converterField, MethodInfo targetMethod)
		{
			ILGenerator generator = readerMethod.GetILGenerator(64);
			generator.Emit(OpCodes.Ldsfld, converterField);
			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Call, targetMethod);
			generator.Emit(OpCodes.Ret);
		}

		static void GenerateConverterWriterForwardingMethod(DynamicMethod writerMethod, FieldInfo converterField, MethodInfo targetMethod)
		{
			ILGenerator generator = writerMethod.GetILGenerator(64);
			generator.Emit(OpCodes.Ldsfld, converterField);
			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_1);
			generator.Emit(OpCodes.Call, targetMethod);
			generator.Emit(OpCodes.Ret);
		}
	}
}
