// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Instructs the serializer to ignore this property during serialization
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class BinaryIgnoreAttribute : Attribute
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public BinaryIgnoreAttribute()
		{
		}
	}

	/// <summary>
	/// Marks this class as supporting binary serialization. Used to automatically register types via BinaryArchive.RegisterTypes().
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class BinarySerializableAttribute : Attribute
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public BinarySerializableAttribute()
		{
		}
	}

	/// <summary>
	/// Instructs the serializer to use a specific converter for this property or class
	/// </summary>
	[AttributeUsage(AttributeTargets.Property | AttributeTargets.Class)]
	public sealed class BinaryConverterAttribute : Attribute
	{
		/// <summary>
		/// The serializer type
		/// </summary>
		public Type Type { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The serializer type</param>
		public BinaryConverterAttribute(Type type)
		{
			Type = type;
		}
	}
}
