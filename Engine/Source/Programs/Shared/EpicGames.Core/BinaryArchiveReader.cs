// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Runtime.CompilerServices;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Reads data from a binary output stream. Similar to the NET Framework BinaryReader class, but supports fast serialization of object graphs and container types, and supports nullable objects.
	/// Significantly faster than BinaryReader due to the expectation that the whole stream is in memory before deserialization.
	/// </summary>
	public sealed class BinaryArchiveReader : IDisposable
	{
		/// <summary>
		/// The input stream.
		/// </summary>
		Stream? _stream;

		/// <summary>
		/// The input buffer
		/// </summary>
		byte[]? _buffer;

		/// <summary>
		/// Current position within the buffer
		/// </summary>
		int _bufferPos;

		/// <summary>
		/// List of previously serialized objects
		/// </summary>
		readonly List<object?> _objects = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		public BinaryArchiveReader(byte[] buffer)
		{
			_buffer = buffer;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileName">File to read from</param>
		public BinaryArchiveReader(FileReference fileName)
		{
			_buffer = FileReference.ReadAllBytes(fileName);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		public BinaryArchiveReader(Stream stream)
		{
			_buffer = new byte[stream.Length];
			stream.ReadExactly(_buffer, 0, _buffer.Length);
		}

		/// <summary>
		/// Dispose of the stream owned by this reader
		/// </summary>
		public void Dispose()
		{
			_stream?.Dispose();
			_stream = null;

			_buffer = null;
		}

		/// <summary>
		/// Reads a bool from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public bool ReadBool()
		{
			return ReadByte() != 0;
		}

		/// <summary>
		/// Reads a single byte from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public byte ReadByte()
		{
			byte value = _buffer![_bufferPos];
			_bufferPos++;
			return value;
		}

		/// <summary>
		/// Reads a single signed byte from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public sbyte ReadSignedByte()
		{
			return (sbyte)ReadByte();
		}

		/// <summary>
		/// Reads a single short from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public short ReadShort()
		{
			return (short)ReadUnsignedShort();
		}

		/// <summary>
		/// Reads a single unsigned short from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public ushort ReadUnsignedShort()
		{
			ushort value = (ushort)(_buffer![_bufferPos + 0] | (_buffer[_bufferPos + 1] << 8));
			_bufferPos += 2;
			return value;
		}

		/// <summary>
		/// Reads a single int from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		public int ReadInt()
		{
			return (int)ReadUnsignedInt();
		}

		/// <summary>
		/// Reads a single unsigned int from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		public uint ReadUnsignedInt()
		{
			uint value = (uint)(_buffer![_bufferPos + 0] | (_buffer[_bufferPos + 1] << 8) | (_buffer[_bufferPos + 2] << 16) | (_buffer[_bufferPos + 3] << 24));
			_bufferPos += 4;
			return value;
		}

		/// <summary>
		/// Reads a single long from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public long ReadLong()
		{
			return (long)ReadUnsignedLong();
		}

		/// <summary>
		/// Reads a single unsigned long from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public ulong ReadUnsignedLong()
		{
			ulong value = (ulong)ReadUnsignedInt();
			value |= (ulong)ReadUnsignedInt() << 32;
			return value;
		}

		/// <summary>
		/// Reads a double (64 bit floating point value) from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public double ReadDouble()
		{
			return BitConverter.Int64BitsToDouble(ReadLong());
		}

		/// <summary>
		/// Reads a string from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public string? ReadString()
		{
			// ReadPrimitiveArray has been inlined here to avoid the transient byte array allocation
			int length = ReadInt();
			if (length < 0)
			{
				return null;
			}
			else
			{
				int offset = _bufferPos;
				_bufferPos += length;
				return Encoding.UTF8.GetString(_buffer!, offset, length);
			}
		}

		/// <summary>
		/// Reads a LogEvent from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public LogEvent ReadLogEvent()
		{
			return LogEvent.Read(ReadByteArray());
		}

		/// <summary>
		/// Reads a byte array from the stream
		/// </summary>
		/// <returns>The data that was read</returns>
		public byte[]? ReadByteArray()
		{
			return ReadPrimitiveArray<byte>(sizeof(byte));
		}

		/// <summary>
		/// Reads a short array from the stream
		/// </summary>
		/// <returns>The data that was read</returns>
		public short[]? ReadShortArray()
		{
			return ReadPrimitiveArray<short>(sizeof(short));
		}

		/// <summary>
		/// Reads an int array from the stream
		/// </summary>
		/// <returns>The data that was read</returns>
		public int[]? ReadIntArray()
		{
			return ReadPrimitiveArray<int>(sizeof(int));
		}

		/// <summary>
		/// Reads an array of primitive types from the stream
		/// </summary>
		/// <param name="elementSize">Size of a single element</param>
		/// <returns>The data that was read</returns>
		private T[]? ReadPrimitiveArray<T>(int elementSize) where T : struct
		{
			int length = ReadInt();
			if (length < 0)
			{
				return null;
			}
			else
			{
				T[] result = new T[length];
				ReadBulkData(result, length * elementSize);
				return result;
			}
		}

		/// <summary>
		/// Reads a byte array from the stream
		/// </summary>
		/// <param name="length">Length of the array to read</param>
		/// <returns>The data that was read</returns>
		public byte[] ReadFixedSizeByteArray(int length)
		{
			return ReadFixedSizePrimitiveArray<byte>(sizeof(byte), length);
		}

		/// <summary>
		/// Reads a short array from the stream
		/// </summary>
		/// <param name="length">Length of the array to read</param>
		/// <returns>The data that was read</returns>
		public short[] ReadFixedSizeShortArray(int length)
		{
			return ReadFixedSizePrimitiveArray<short>(sizeof(short), length);
		}

		/// <summary>
		/// Reads an int array from the stream
		/// </summary>
		/// <param name="length">Length of the array to read</param>
		/// <returns>The data that was read</returns>
		public int[] ReadFixedSizeIntArray(int length)
		{
			return ReadFixedSizePrimitiveArray<int>(sizeof(int), length);
		}

		/// <summary>
		/// Reads an array of primitive types from the stream
		/// </summary>
		/// <param name="elementSize">Size of a single element</param>
		/// <param name="elementCount">Number of elements to read</param>
		/// <returns>The data that was read</returns>
		private T[] ReadFixedSizePrimitiveArray<T>(int elementSize, int elementCount) where T : struct
		{
			T[] result = new T[elementCount];
			ReadBulkData(result, elementSize * elementCount);
			return result;
		}

		/// <summary>
		/// Reads bulk data from the stream into the given buffer
		/// </summary>
		/// <param name="data">Array which receives the data that was read</param>
		/// <param name="size">Size of data to read</param>
		private void ReadBulkData(Array data, int size)
		{
			Buffer.BlockCopy(_buffer!, _bufferPos, data, 0, size);
			_bufferPos += size;
		}

		/// <summary>
		/// Reads an array of items
		/// </summary>
		/// <returns>New array</returns>
		public T[]? ReadArray<T>(Func<T> readElement)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				T[] result = new T[count];
				for (int idx = 0; idx < count; idx++)
				{
					result[idx] = readElement();
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a list of items
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="readElement">Delegate used to read a single element</param>
		/// <returns>List of items</returns>
		public List<T>? ReadList<T>(Func<T> readElement)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				List<T> result = new(count);
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readElement());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a hashset of items
		/// </summary>
		/// <typeparam name="T">The element type for the set</typeparam>
		/// <param name="readElement">Delegate used to read a single element</param>
		/// <returns>Set of items</returns>
		public HashSet<T>? ReadHashSet<T>(Func<T> readElement)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				HashSet<T> result = [];
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readElement());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a hashset of items
		/// </summary>
		/// <typeparam name="T">The element type for the set</typeparam>
		/// <param name="readElement">Delegate used to read a single element</param>
		/// <param name="comparer">Comparison function for the set</param>
		/// <returns>Set of items</returns>
		public HashSet<T>? ReadHashSet<T>(Func<T> readElement, IEqualityComparer<T> comparer)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				HashSet<T> result = new(comparer);
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readElement());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a sortedset of items
		/// </summary>
		/// <typeparam name="T">The element type for the sortedset</typeparam>
		/// <param name="readElement">Delegate used to read a single element</param>
		/// <returns>SortedSet of items</returns>
		public SortedSet<T>? ReadSortedSet<T>(Func<T> readElement)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				SortedSet<T> result = [];
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readElement());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a sortedset of items
		/// </summary>
		/// <typeparam name="T">The element type for the sortedset</typeparam>
		/// <param name="readElement">Delegate used to read a single element</param>
		/// <param name="comparer">Comparison function for the set</param>
		/// <returns>SortedSet of items</returns>
		public SortedSet<T>? ReadSortedSet<T>(Func<T> readElement, IComparer<T> comparer)
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				SortedSet<T> result = new(comparer);
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readElement());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a dictionary of items
		/// </summary>
		/// <typeparam name="TK">Type of the dictionary key</typeparam>
		/// <typeparam name="TV">Type of the dictionary value</typeparam>
		/// <param name="readKey">Delegate used to read a single key</param>
		/// <param name="readValue">Delegate used to read a single value</param>
		/// <returns>New dictionary instance</returns>
		public Dictionary<TK, TV>? ReadDictionary<TK, TV>(Func<TK> readKey, Func<TV> readValue) where TK : notnull
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				Dictionary<TK, TV> result = new(count);
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readKey(), readValue());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a dictionary of items
		/// </summary>
		/// <typeparam name="TK">Type of the dictionary key</typeparam>
		/// <typeparam name="TV">Type of the dictionary value</typeparam>
		/// <param name="readKey">Delegate used to read a single key</param>
		/// <param name="readValue">Delegate used to read a single value</param>
		/// <param name="comparer">Comparison function for keys in the dictionary</param>
		/// <returns>New dictionary instance</returns>
		public Dictionary<TK, TV>? ReadDictionary<TK, TV>(Func<TK> readKey, Func<TV> readValue, IEqualityComparer<TK> comparer) where TK : notnull
		{
			int count = ReadInt();
			if (count < 0)
			{
				return null;
			}
			else
			{
				Dictionary<TK, TV> result = new(count, comparer);
				for (int idx = 0; idx < count; idx++)
				{
					result.Add(readKey(), readValue());
				}
				return result;
			}
		}

		/// <summary>
		/// Reads a nullable object from the archive
		/// </summary>
		/// <typeparam name="T">The nullable type</typeparam>
		/// <param name="readValue">Delegate used to read a value</param>
		public Nullable<T> ReadNullable<T>(Func<T> readValue) where T : struct
		{
			if (ReadBool())
			{
				return new Nullable<T>(readValue());
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Reads an object, which may be null, from the archive. Does not handle de-duplicating object references. 
		/// </summary>
		/// <typeparam name="T">Type of the object to read</typeparam>
		/// <param name="read">Delegate used to read the object</param>
		/// <returns>The object instance</returns>
		public T? ReadOptionalObject<T>(Func<T> read) where T : class
		{
			if (ReadBool())
			{
				return read();
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Reads an object reference from the stream. Each referenced object will only be serialized once using the supplied delegates. Reading an object instance is
		/// done in two phases; first the object is created and its reference stored in the unique object list, then the object contents are read. This allows the object to 
		/// serialize a reference to itself.
		/// </summary>
		/// <typeparam name="T">Type of the object to read.</typeparam>
		/// <param name="createObject">Delegate used to create an object instance</param>
		/// <param name="readObject">Delegate used to read an object instance</param>
		/// <returns>Object instance</returns>
		public T? ReadObjectReference<T>(Func<T> createObject, Action<T> readObject) where T : class
		{
			int index = ReadInt();
			if (index < 0)
			{
				return null;
			}
			else
			{
				if (index == _objects.Count)
				{
					T obj = createObject();
					_objects.Add(obj);
					readObject(obj);
				}
				return (T?)_objects[index];
			}
		}

		/// <summary>
		/// Reads an object reference from the stream. Each object will only be serialized once using the supplied delegate; subsequent reads reference the original.
		/// Since the reader only receives the object reference when the CreateObject delegate returns, it is not possible for the object to serialize a reference to itself.
		/// </summary>
		/// <param name="readObject">Delegate used to create an object instance. The object may not reference itself recursively.</param>
		/// <returns>Object instance</returns>
		public object? ReadUntypedObjectReference(Func<object?> readObject)
		{
			int index = ReadInt();
			if (index < 0)
			{
				return null;
			}
			else
			{
				// Temporarily add the reader to the object list, so we can detect invalid recursive references. 
				if (index == _objects.Count)
				{
					_objects.Add(null);
					_objects[index] = readObject();
				}
				if (_objects[index] == null)
				{
					throw new InvalidOperationException("Attempt to serialize reference to object recursively.");
				}
				return _objects[index];
			}
		}

		/// <summary>
		/// Reads an object reference from the stream. Each object will only be serialized once using the supplied delegate; subsequent reads reference the original.
		/// Since the reader only receives the object reference when the CreateObject delegate returns, it is not possible for the object to serialize a reference to itself.
		/// </summary>
		/// <param name="readObject">Delegate used to create an object instance. The object may not reference itself recursively.</param>
		/// <returns>Object instance</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		public object? ReadUntypedObjectReference(Func<BinaryArchiveReader, object?> readObject)
		{
			int index = ReadInt();
			if (index < 0)
			{
				return null;
			}
			else
			{
				// Temporarily add the reader to the object list, so we can detect invalid recursive references. 
				if (index == _objects.Count)
				{
					_objects.Add(null);
					_objects[index] = readObject(this);
				}
				if (_objects[index] == null)
				{
					throw new InvalidOperationException("Attempt to serialize reference to object recursively.");
				}
				return _objects[index];
			}
		}

		/// <summary>
		/// Reads an object reference from the stream. Each object will only be serialized once using the supplied delegate; subsequent reads reference the original.
		/// Since the reader only receives the object reference when the CreateObject delegate returns, it is not possible for the object to serialize a reference to itself.
		/// </summary>
		/// <typeparam name="T">Type of the object to read.</typeparam>
		/// <param name="readObject">Delegate used to create an object instance. The object may not reference itself recursively.</param>
		/// <returns>Object instance</returns>
		public T? ReadObjectReference<T>(Func<T> readObject) where T : class => (T?)ReadUntypedObjectReference(readObject);

		/// <summary>
		/// Reads an object reference from the stream. Each object will only be serialized once using the supplied delegate; subsequent reads reference the original.
		/// Since the reader only receives the object reference when the CreateObject delegate returns, it is not possible for the object to serialize a reference to itself.
		/// </summary>
		/// <typeparam name="T">Type of the object to read.</typeparam>
		/// <param name="readObject">Delegate used to create an object instance. The object may not reference itself recursively.</param>
		/// <returns>Object instance</returns>
		[MethodImpl(MethodImplOptions.AggressiveInlining)]
		public T? ReadObjectReference<T>(Func<BinaryArchiveReader, T> readObject) where T : class => (T?)ReadUntypedObjectReference(readObject);

		/// <summary>
		/// Helper method for validating that deserialized objects are not null
		/// </summary>
		/// <typeparam name="T">Type of the deserialized object</typeparam>
		/// <param name="param">The object instance</param>
		/// <returns>The object instance</returns>
		[return: NotNull]
		public static T NotNull<T>(T? param) where T : class
		{
			if (param == null)
			{
				throw new InvalidDataException("Object stored in archive is not allowed to be null.");
			}
			return param;
		}

		/// <summary>
		/// Helper method for validating that deserialized objects are not null
		/// </summary>
		/// <typeparam name="T">Type of the deserialized object</typeparam>
		/// <param name="param">The object instance</param>
		/// <returns>The object instance</returns>
		public static T NotNullStruct<T>(T? param) where T : struct
		{
			if (param == null)
			{
				throw new InvalidDataException("Object stored in archive is not allowed to be null.");
			}
			return param.Value;
		}
	}
}
