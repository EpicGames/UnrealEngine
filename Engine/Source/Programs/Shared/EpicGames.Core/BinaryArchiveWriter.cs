// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Runtime.CompilerServices;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Writes data to a binary output stream. Similar to the NET Framework BinaryWriter class, but supports fast serialization of object graphs and container types, and supports nullable objects.
	/// </summary>
	public sealed class BinaryArchiveWriter : IDisposable
	{
		/// <summary>
		/// Comparer which tests for reference equality between two objects
		/// </summary>
		class ReferenceComparer : IEqualityComparer<object>
		{
			bool IEqualityComparer<object>.Equals(object? a, object? b)
			{
				return a == b;
			}

			int IEqualityComparer<object>.GetHashCode(object x)
			{
				return RuntimeHelpers.GetHashCode(x);
			}
		}

		/// <summary>
		/// Instance of the ReferenceComparer class which can be shared by all archive writers
		/// </summary>
		static readonly ReferenceComparer s_referenceComparerInstance = new();

		/// <summary>
		/// The output stream being written to
		/// </summary>
		Stream? _stream;

		/// <summary>
		/// Buffer for data to be written to the stream
		/// </summary>
		byte[] _buffer;

		/// <summary>
		/// Current position within the output buffer
		/// </summary>
		int _bufferPos;

		/// <summary>
		/// Map of object instance to unique id
		/// </summary>
		readonly Dictionary<object, int> _objectToUniqueId = new(s_referenceComparerInstance);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="stream">The output stream</param>
		public BinaryArchiveWriter(Stream stream)
		{
			_stream = stream;
			_buffer = new byte[4096];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileName">File to write to</param>
		public BinaryArchiveWriter(FileReference fileName)
			: this(File.Open(fileName.FullName, FileMode.Create, FileAccess.Write, FileShare.Read))
		{
		}

		/// <summary>
		/// Flushes this stream, and disposes the stream
		/// </summary>
		public void Dispose()
		{
			Flush();
			_stream?.Dispose();
			_stream = null;
		}

		/// <summary>
		/// Writes all buffered data to disk
		/// </summary>
		public void Flush()
		{
			if (_bufferPos > 0)
			{
				_stream!.Write(_buffer, 0, _bufferPos);
				_bufferPos = 0;
			}
		}

		/// <summary>
		/// Ensures there is a minimum amount of space in the output buffer
		/// </summary>
		/// <param name="numBytes">Minimum amount of space required in the output buffer</param>
		private void EnsureSpace(int numBytes)
		{
			if (_bufferPos + numBytes > _buffer.Length)
			{
				Flush();
				if (numBytes > _buffer.Length)
				{
					_buffer = new byte[numBytes];
				}
			}
		}

		/// <summary>
		/// Writes a bool to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteBool(bool value)
		{
			WriteByte(value ? (byte)1 : (byte)0);
		}

		/// <summary>
		/// Writes a single byte to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteByte(byte value)
		{
			EnsureSpace(1);

			_buffer[_bufferPos] = value;

			_bufferPos++;
		}

		/// <summary>
		/// Writes a single signed byte to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteSignedByte(sbyte value)
		{
			WriteByte((byte)value);
		}

		/// <summary>
		/// Writes a single short to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteShort(short value)
		{
			WriteUnsignedShort((ushort)value);
		}

		/// <summary>
		/// Writes a single unsigned short to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUnsignedShort(ushort value)
		{
			EnsureSpace(2);

			_buffer[_bufferPos + 0] = (byte)value;
			_buffer[_bufferPos + 1] = (byte)(value >> 8);

			_bufferPos += 2;
		}

		/// <summary>
		/// Writes a single int to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteInt(int value)
		{
			WriteUnsignedInt((uint)value);
		}

		/// <summary>
		/// Writes a single unsigned int to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUnsignedInt(uint value)
		{
			EnsureSpace(4);

			_buffer[_bufferPos + 0] = (byte)value;
			_buffer[_bufferPos + 1] = (byte)(value >> 8);
			_buffer[_bufferPos + 2] = (byte)(value >> 16);
			_buffer[_bufferPos + 3] = (byte)(value >> 24);

			_bufferPos += 4;
		}

		/// <summary>
		/// Writes a single long to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteLong(long value)
		{
			WriteUnsignedLong((ulong)value);
		}

		/// <summary>
		/// Writes a single unsigned long to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUnsignedLong(ulong value)
		{
			EnsureSpace(8);

			_buffer[_bufferPos + 0] = (byte)value;
			_buffer[_bufferPos + 1] = (byte)(value >> 8);
			_buffer[_bufferPos + 2] = (byte)(value >> 16);
			_buffer[_bufferPos + 3] = (byte)(value >> 24);
			_buffer[_bufferPos + 4] = (byte)(value >> 32);
			_buffer[_bufferPos + 5] = (byte)(value >> 40);
			_buffer[_bufferPos + 6] = (byte)(value >> 48);
			_buffer[_bufferPos + 7] = (byte)(value >> 56);

			_bufferPos += 8;
		}

		/// <summary>
		/// Writes a double (64 bit floating point value) to the stream
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteDouble(double value)
		{
			WriteLong(BitConverter.DoubleToInt64Bits(value));
		}

		/// <summary>
		/// Writes a string to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteString(string? value)
		{
			byte[]? bytes;
			if (value == null)
			{
				bytes = null;
			}
			else
			{
				bytes = Encoding.UTF8.GetBytes(value);
			}
			WriteByteArray(bytes);
		}

		/// <summary>
		/// Writes a LogEvent to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteLogEvent(LogEvent value)
		{
			WriteByteArray(value.ToJsonBytes());
		}

		/// <summary>
		/// Writes an array of bytes to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteByteArray(byte[]? data)
		{
			WritePrimitiveArray(data, sizeof(byte));
		}

		/// <summary>
		/// Writes an array of shorts to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteShortArray(short[]? data)
		{
			WritePrimitiveArray(data, sizeof(short));
		}

		/// <summary>
		/// Writes an array of ints to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteIntArray(int[]? data)
		{
			WritePrimitiveArray(data, sizeof(int));
		}

		/// <summary>
		/// Writes an array of primitive types to the output.
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		/// <param name="elementSize">Size of each element</param>
		private void WritePrimitiveArray<T>(T[]? data, int elementSize) where T : struct
		{
			if (data == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(data.Length);
				WriteBulkData(data, data.Length * elementSize);
			}
		}

		/// <summary>
		/// Writes an array of bytes to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteFixedSizeByteArray(byte[] data)
		{
			WriteFixedSizePrimitiveArray(data, sizeof(byte));
		}

		/// <summary>
		/// Writes an array of shorts to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteFixedSizeShortArray(short[] data)
		{
			WriteFixedSizePrimitiveArray(data, sizeof(short));
		}

		/// <summary>
		/// Writes an array of ints to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteFixedSizeIntArray(int[] data)
		{
			WriteFixedSizePrimitiveArray(data, sizeof(int));
		}

		/// <summary>
		/// Writes an array of primitive types to the output.
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		/// <param name="elementSize">Size of each element</param>
		private void WriteFixedSizePrimitiveArray<T>(T[] data, int elementSize) where T : struct
		{
			WriteBulkData(data, data.Length * elementSize);
		}

		/// <summary>
		/// Writes primitive data from the given array to the output buffer.
		/// </summary>
		/// <param name="data">Data to write.</param>
		/// <param name="size">Size of the data, in bytes</param>
		private void WriteBulkData(Array data, int size)
		{
			if (size > 0)
			{
				for (int pos = 0; ;)
				{
					int copySize = Math.Min(size - pos, _buffer.Length - _bufferPos);

					Buffer.BlockCopy(data, pos, _buffer, _bufferPos, copySize);
					_bufferPos += copySize;
					pos += copySize;

					if (pos == size)
					{
						break;
					}

					Flush();
				}
			}
		}

		/// <summary>
		/// Write an array of items to the archive
		/// </summary>
		/// <typeparam name="T">Type of the element</typeparam>
		/// <param name="items">Array of items</param>
		/// <param name="writeElement">Writes an individual element to the archive</param>
		public void WriteArray<T>(T[]? items, Action<T> writeElement)
		{
			if (items == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(items.Length);
				for (int idx = 0; idx < items.Length; idx++)
				{
					writeElement(items[idx]);
				}
			}
		}

		/// <summary>
		/// Write a list of items to the archive
		/// </summary>
		/// <typeparam name="T">Type of the element</typeparam>
		/// <param name="items">List of items</param>
		/// <param name="writeElement">Writes an individual element to the archive</param>
		public void WriteList<T>(IReadOnlyList<T>? items, Action<T> writeElement)
		{
			if (items == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(items.Count);
				for (int idx = 0; idx < items.Count; idx++)
				{
					writeElement(items[idx]);
				}
			}
		}

		/// <summary>
		/// Writes a hashset of items
		/// </summary>
		/// <typeparam name="T">The element type for the set</typeparam>
		/// <param name="set">The set to write</param>
		/// <param name="writeElement">Delegate used to read a single element</param>
		public void WriteHashSet<T>(HashSet<T> set, Action<T> writeElement)
		{
			if (set == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(set.Count);
				foreach (T element in set)
				{
					writeElement(element);
				}
			}
		}

		/// <summary>
		/// Writes a sortedset of items
		/// </summary>
		/// <typeparam name="T">The element type for the sortedset</typeparam>
		/// <param name="set">The set to write</param>
		/// <param name="writeElement">Delegate used to read a single element</param>
		public void WriteSortedSet<T>(SortedSet<T> set, Action<T> writeElement)
		{
			if (set == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(set.Count);
				foreach (T element in set)
				{
					writeElement(element);
				}
			}
		}

		/// <summary>
		/// Writes a dictionary of items
		/// </summary>
		/// <typeparam name="TK">Type of the dictionary key</typeparam>
		/// <typeparam name="TV">Type of the dictionary value</typeparam>
		/// <param name="dictionary">The dictionary to write</param>
		/// <param name="writeKey">Delegate used to read a single key</param>
		/// <param name="writeValue">Delegate used to read a single value</param>
		public void WriteDictionary<TK, TV>(IReadOnlyDictionary<TK, TV> dictionary, Action<TK> writeKey, Action<TV> writeValue) where TK : notnull
		{
			if (dictionary == null)
			{
				WriteInt(-1);
			}
			else
			{
				WriteInt(dictionary.Count);
				foreach (KeyValuePair<TK, TV> pair in dictionary)
				{
					writeKey(pair.Key);
					writeValue(pair.Value);
				}
			}
		}

		/// <summary>
		/// Writes a nullable object to the archive
		/// </summary>
		/// <typeparam name="T">The nullable type</typeparam>
		/// <param name="item">Item to write</param>
		/// <param name="writeValue">Delegate used to write a value</param>
		public void WriteNullable<T>(Nullable<T> item, Action<T> writeValue) where T : struct
		{
			if (item.HasValue)
			{
				WriteBool(true);
				writeValue(item.Value);
			}
			else
			{
				WriteBool(false);
			}
		}

		/// <summary>
		/// Writes an object to the output, checking whether it is null or not. Does not preserve object references; each object written is duplicated.
		/// </summary>
		/// <typeparam name="T">Type of the object to serialize</typeparam>
		/// <param name="obj">Reference to check for null before serializing</param>
		/// <param name="writeObject">Delegate used to write the object</param>
		public void WriteOptionalObject<T>(T obj, Action writeObject) where T : class
		{
			if (obj == null)
			{
				WriteBool(false);
			}
			else
			{
				WriteBool(true);
				writeObject();
			}
		}

		/// <summary>
		/// Writes a null object reference to the output.
		/// </summary>
		public void WriteNullObjectReference()
		{
			WriteInt(-1);
		}

		/// <summary>
		/// Writes an object to the output. If the specific instance has already been written, preserves the reference to that.
		/// </summary>
		/// <param name="obj">The object to serialize</param>
		/// <param name="writeObject">Delegate used to write the object</param>
		public void WriteObjectReference(object? obj, Action writeObject)
		{
			if (WriteObjectReferenceUniqueId(obj))
			{
				writeObject();
			}
		}

		/// <summary>
		/// Writes an object to the output. If the specific instance has already been written, preserves the reference to that.
		/// </summary>
		/// <param name="obj">The object to serialize</param>
		/// <param name="writeObject">Delegate used to write the object</param>
		public void WriteObjectReference<T>(T obj, Action writeObject) where T : class?
		{
			WriteObjectReference((object?)obj, writeObject);
		}

		/// <summary>
		/// Writes an object to the output. If the specific instance has already been written, preserves the reference to that.
		/// </summary>
		/// <param name="obj">The object to serialize</param>
		/// <param name="writeObject">Delegate used to write the object</param>
		public void WriteObjectReference<T>(T obj, Action<BinaryArchiveWriter, T> writeObject) where T : class?
		{
			if (WriteObjectReferenceUniqueId(obj))
			{
				writeObject(this, obj);
			}
		}

		/// <summary>
		/// Writes a unique id for the given object.
		/// </summary>
		/// <param name="obj">The object to serialize</param>
		/// <returns>False if the object has already been written.  True if this is the first time the object has been referenced.</returns>
		private bool WriteObjectReferenceUniqueId(object? obj)
		{
			if (obj == null)
			{
				WriteInt(-1);
				return false;
			}
			else
			{
				if (_objectToUniqueId.TryGetValue(obj, out int index))
				{
					WriteInt(index);
					return false;
				}
				else
				{
					WriteInt(_objectToUniqueId.Count);
					_objectToUniqueId.Add(obj, _objectToUniqueId.Count);
					return true;
				}
			}
		}
	}
}
