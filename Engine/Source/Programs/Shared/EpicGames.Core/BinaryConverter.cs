// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Reflection;

namespace EpicGames.Core
{
	/// <summary>
	/// Base interface for a binary converter. Implementations must derive from <see cref="IBinaryConverter{TValue}"/> instead
	/// </summary>
	public interface IBinaryConverter
	{
		/// <summary>
		/// Converter version number
		/// </summary>
		int Version { get; }
	}

	/// <summary>
	/// Interface for converting serializing object to and from a binary format
	/// </summary>
	/// <typeparam name="TValue">The value to serialize</typeparam>
	public interface IBinaryConverter<TValue> : IBinaryConverter
	{
		/// <summary>
		/// Reads a value from the archive
		/// </summary>
		/// <param name="reader">The archive reader</param>
		/// <returns>New instance of the value</returns>
		TValue Read(BinaryArchiveReader reader);

		/// <summary>
		/// Writes a value to the archive
		/// </summary>
		/// <param name="writer">The archive writer</param>
		/// <param name="value">The value to write</param>
		void Write(BinaryArchiveWriter writer, TValue value);
	}

	/// <summary>
	/// Registration of IBinaryConverter instances
	/// </summary>
	public static class BinaryConverter
	{
		/// <summary>
		/// Map from type to the converter type
		/// </summary>
		static readonly ConcurrentDictionary<Type, Type> s_typeToConverterType = new();

		/// <summary>
		/// Explicitly register the converter for a type. If Type is a generic type, the converter should also be a generic type with the same type arguments.
		/// </summary>
		/// <param name="type">Type to register a converter for</param>
		/// <param name="converterType">The converter type</param>
		public static void RegisterConverter(Type type, Type converterType)
		{
			if (!s_typeToConverterType.TryAdd(type, converterType))
			{
				throw new Exception($"Type '{type.Name}' already has a registered converter ({s_typeToConverterType[type].Name})");
			}
		}

		/// <summary>
		/// Attempts to get the converter for a particular type
		/// </summary>
		/// <param name="type">The type to use</param>
		/// <param name="converterType">The converter type</param>
		/// <returns>True if a converter was found</returns>
		public static bool TryGetConverterType(Type type, out Type? converterType)
		{
			if (s_typeToConverterType.TryGetValue(type, out Type? customConverterType))
			{
				converterType = customConverterType;
				return true;
			}

			BinaryConverterAttribute? converterAttribute = type.GetCustomAttribute<BinaryConverterAttribute>();
			if (converterAttribute != null)
			{
				converterType = converterAttribute.Type;
				return true;
			}

			if (type.IsGenericType)
			{
				BinaryConverterAttribute? genericConverterAttribute = type.GetGenericTypeDefinition().GetCustomAttribute<BinaryConverterAttribute>();
				if (genericConverterAttribute != null)
				{
					converterType = genericConverterAttribute.Type.MakeGenericType(type.GetGenericArguments());
					return true;
				}
			}

			converterType = null;
			return false;
		}
	}
}
