// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Runtime.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility functions for serializing using the DataContractSerializer xml (previously BinaryFormatter which is no longer supported)
	/// </summary>
	public static class BinaryFormatterUtils
	{
		/// <summary>
		/// Load an object from a file on disk, using DataContractSerializer
		/// </summary>
		/// <param name="location">File to read from</param>
		/// <returns>Instance of the object that was read from disk</returns>
		public static T Load<T>(FileReference location)
		{
			using FileStream stream = new(location.FullName, FileMode.Open, FileAccess.Read);
			DataContractSerializer serializer = new(typeof(T));
			return (T?)serializer.ReadObject(stream) ?? throw new SerializationException($"Unable to deserialize {location} as {typeof(T)}");
		}

		/// <summary>
		/// Saves a file to disk, using DataContractSerializer
		/// </summary>
		/// <param name="location">File to write to</param>
		/// <param name="obj">Object to serialize</param>
		public static void Save(FileReference location, object obj)
		{
			DirectoryReference.CreateDirectory(location.Directory);
			using FileStream stream = new(location.FullName, FileMode.Create, FileAccess.Write);
			DataContractSerializer serializer = new(obj.GetType());
			serializer.WriteObject(stream, obj);
		}

		/// <summary>
		/// Saves a file to disk using DataContractSerializer, without updating the timestamp if it hasn't changed
		/// </summary>
		/// <param name="location">File to write to</param>
		/// <param name="obj">Object to serialize</param>
		public static void SaveIfDifferent(FileReference location, object obj)
		{
			byte[] contents;
			using (MemoryStream stream = new())
			{
				DataContractSerializer serializer = new(obj.GetType());
				serializer.WriteObject(stream, obj);
				contents = stream.ToArray();
			}

			DirectoryReference.CreateDirectory(location.Directory);
			FileReference.WriteAllBytesIfDifferent(location, contents);
		}
	}
}
