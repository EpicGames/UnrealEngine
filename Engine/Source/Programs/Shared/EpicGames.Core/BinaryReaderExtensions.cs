// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for <see cref="BinaryReader"/>
	/// </summary>
	public static class BinaryReaderExtensions
	{
		/// <summary>
		/// Read an objects from a binary reader
		/// </summary>
		/// <typeparam name="T">The element type for the array</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <returns>Object instance.</returns>
		public static T ReadObject<T>(this BinaryReader reader) where T : class, IBinarySerializable
		{
			return (T)Activator.CreateInstance(typeof(T), reader)!;
		}

		/// <summary>
		/// Read an array of strings from a binary reader
		/// </summary>
		/// <param name="reader">Reader to read data from</param>
		/// <returns>Array of strings, as serialized. May be null.</returns>
		public static string[]? ReadStringArray(this BinaryReader reader)
		{
			return reader.ReadArray(() => reader.ReadString());
		}

		/// <summary>
		/// Read an array of objects from a binary reader
		/// </summary>
		/// <typeparam name="T">The element type for the array</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <returns>Array of objects, as serialized. May be null.</returns>
		public static T[]? ReadArray<T>(this BinaryReader reader) where T : class, IBinarySerializable
		{
			return ReadArray<T>(reader, () => reader.ReadObject<T>());
		}

		/// <summary>
		/// Read an array of objects from a binary reader
		/// </summary>
		/// <typeparam name="T">The element type for the array</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <param name="readElement">Delegate to call to serialize each element</param>
		/// <returns>Array of objects, as serialized. May be null.</returns>
		public static T[]? ReadArray<T>(this BinaryReader reader, Func<T> readElement)
		{
			int numItems = reader.ReadInt32();
			if (numItems < 0)
			{
				return null;
			}

			T[] items = new T[numItems];
			for (int idx = 0; idx < numItems; idx++)
			{
				items[idx] = readElement();
			}
			return items;
		}

		/// <summary>
		/// Read a list of strings from a binary reader
		/// </summary>
		/// <param name="reader">Reader to read data from</param>
		/// <returns>Array of strings, as serialized. May be null.</returns>
		public static List<string>? ReadStringList(this BinaryReader reader)
		{
			return reader.ReadList(() => reader.ReadString());
		}

		/// <summary>
		/// Read a list of objects from a binary reader
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <returns>List of objects, as serialized. May be null.</returns>
		public static List<T>? ReadList<T>(this BinaryReader reader) where T : class, IBinarySerializable
		{
			return ReadList<T>(reader, () => reader.ReadObject<T>());
		}

		/// <summary>
		/// Read a list of objects from a binary reader
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <param name="readElement">Delegate to call to serialize each element</param>
		/// <returns>List of objects, as serialized. May be null.</returns>
		public static List<T>? ReadList<T>(this BinaryReader reader, Func<T> readElement)
		{
			int numItems = reader.ReadInt32();
			if (numItems < 0)
			{
				return null;
			}

			List<T> items = new(numItems);
			for (int idx = 0; idx < numItems; idx++)
			{
				items.Add(readElement());
			}
			return items;
		}

		/// <summary>
		/// Read a list of objects from a binary reader
		/// </summary>
		/// <typeparam name="TKey">Dictionary key type</typeparam>
		/// <typeparam name="TValue">Dictionary value type</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <param name="readKey">Delegate to call to serialize each key</param>
		/// <param name="readValue">Delegate to call to serialize each value</param>
		/// <returns>List of objects, as serialized. May be null.</returns>
		public static Dictionary<TKey, TValue>? ReadDictionary<TKey, TValue>(this BinaryReader reader, Func<TKey> readKey, Func<TValue> readValue) where TKey : notnull
		{
			int numItems = reader.ReadInt32();
			if (numItems < 0)
			{
				return null;
			}

			Dictionary<TKey, TValue> items = new(numItems);
			for (int idx = 0; idx < numItems; idx++)
			{
				TKey key = readKey();
				TValue value = readValue();
				items.Add(key, value);
			}
			return items;
		}

		/// <summary>
		/// Read a nullable object from a binary reader
		/// </summary>
		/// <typeparam name="T">Type of the object</typeparam>
		/// <param name="reader">Reader to read data from</param>
		/// <param name="readItem">Function to read the payload, if non-null</param>
		/// <returns>Object instance or null</returns>
		public static T? ReadNullable<T>(this BinaryReader reader, Func<T> readItem) where T : class
		{
			if (reader.ReadBoolean())
			{
				return readItem();
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Reads a value of a specific type from a binary reader
		/// </summary>
		/// <param name="reader">Reader for input data</param>
		/// <param name="objectType">Type of value to read</param>
		/// <returns>The value read from the stream</returns>
		public static object? ReadObject(this BinaryReader reader, Type objectType)
		{
			if (objectType == typeof(string))
			{
				return reader.ReadString();
			}
			else if (objectType == typeof(bool))
			{
				return reader.ReadBoolean();
			}
			else if (objectType == typeof(int))
			{
				return reader.ReadInt32();
			}
			else if (objectType == typeof(float))
			{
				return reader.ReadSingle();
			}
			else if (objectType == typeof(double))
			{
				return reader.ReadDouble();
			}
			else if (objectType == typeof(string[]))
			{
				return reader.ReadStringArray();
			}
			else if (objectType == typeof(bool?))
			{
				int value = reader.ReadInt32();
				return (value == -1) ? (bool?)null : (value == 0) ? (bool?)false : (bool?)true;
			}
			else if (objectType == typeof(FileReference))
			{
				return reader.ReadFileReference();
			}
			else if (objectType.IsEnum)
			{
				return Enum.ToObject(objectType, reader.ReadInt32());
			}
			else if (objectType.IsArray && objectType.GetElementType()!.IsEnum)
			{
				Type elementType = objectType.GetElementType()!;
				int[]? enumValuesAsInts = reader.ReadArray(reader.ReadInt32);
				if (enumValuesAsInts is null)
				{
					return null;
				}
				Array array = Array.CreateInstance(elementType, enumValuesAsInts.Length);
				enumValuesAsInts.CopyTo(array, 0);
				return array;
			}
			else
			{
				throw new Exception(String.Format("Reading binary objects of type '{0}' is not currently supported.", objectType.Name));
			}
		}
	}
}
