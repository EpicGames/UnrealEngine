// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface that can be implemented by objects to support BinaryWriter container functionality. Also implies that
	/// derived objects will have a constructor taking a single BinaryReader argument (similar to how the system ISerializable
	/// interface assumes a specific constructor)
	/// </summary>
	public interface IBinarySerializable
	{
		/// <summary>
		/// Write an item
		/// </summary>
		/// <param name="writer"></param>
		void Write(BinaryWriter writer);
	}

	/// <summary>
	/// Extension methods for BinaryWriter class
	/// </summary>
	public static class BinaryWriterExtensions
	{
		/// <summary>
		/// Writes an item implementing the IBinarySerializable interface to a binary writer. Included for symmetry with standard Writer.Write(X) calls.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="item">The item to write</param>
		public static void Write(this BinaryWriter writer, IBinarySerializable item)
		{
			item.Write(writer);
		}

		/// <summary>
		/// Writes an array of strings to a binary writer.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">Array of items</param>
		public static void Write(this BinaryWriter writer, string[] items)
		{
			Write(writer, items, item => writer.Write(item));
		}

		/// <summary>
		/// Writes an array to a binary writer.
		/// </summary>
		/// <typeparam name="T">The array element type</typeparam>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">Array of items</param>
		public static void Write<T>(this BinaryWriter writer, T[] items) where T : class, IBinarySerializable
		{
			Write(writer, items, item => writer.Write(item));
		}

		/// <summary>
		/// Writes an array to a binary writer.
		/// </summary>
		/// <typeparam name="T">The array element type</typeparam>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">Array of items</param>
		/// <param name="writeElement">Delegate to call to serialize each element</param>
		public static void Write<T>(this BinaryWriter writer, T[] items, Action<T> writeElement)
		{
			if (items == null)
			{
				writer.Write(-1);
			}
			else
			{
				writer.Write(items.Length);
				for (int idx = 0; idx < items.Length; idx++)
				{
					writeElement(items[idx]);
				}
			}
		}

		/// <summary>
		/// Writes a list of strings to a binary writer.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">Array of items</param>
		public static void Write(this BinaryWriter writer, List<string> items)
		{
			Write(writer, items, item => writer.Write(item));
		}

		/// <summary>
		/// Writes a list to a binary writer.
		/// </summary>
		/// <typeparam name="T">The array element type</typeparam>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">Array of items</param>
		public static void Write<T>(this BinaryWriter writer, List<T> items) where T : class, IBinarySerializable
		{
			Write(writer, items, item => writer.Write(item));
		}

		/// <summary>
		/// Writes a list to a binary writer.
		/// </summary>
		/// <typeparam name="T">The list element type</typeparam>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="items">List of items</param>
		/// <param name="writeElement">Delegate to call to serialize each element</param>
		public static void Write<T>(this BinaryWriter writer, List<T> items, Action<T> writeElement)
		{
			if (items == null)
			{
				writer.Write(-1);
			}
			else
			{
				writer.Write(items.Count);
				for (int idx = 0; idx < items.Count; idx++)
				{
					writeElement(items[idx]);
				}
			}
		}

		/// <summary>
		/// Write a dictionary to a binary writer
		/// </summary>
		/// <typeparam name="TKey">The key type for the dictionary</typeparam>
		/// <typeparam name="TValue">The value type for the dictionary</typeparam>
		/// <param name="writer">Writer to write data to</param>
		/// <param name="items">List of items to be written</param>
		/// <param name="writeKey">Delegate to call to serialize each key</param>
		/// <param name="writeValue">Delegate to call to serialize each value</param>
		/// <returns>Dictionary of objects, as serialized. May be null.</returns>
		public static void Write<TKey, TValue>(this BinaryWriter writer, Dictionary<TKey, TValue> items, Action<TKey> writeKey, Action<TValue> writeValue) where TKey : notnull
		{
			if (items == null)
			{
				writer.Write(-1);
			}
			else
			{
				writer.Write(items.Count);
				foreach (KeyValuePair<TKey, TValue> item in items)
				{
					writeKey(item.Key);
					writeValue(item.Value);
				}
			}
		}

		/// <summary>
		/// Read a nullable object from a binary reader
		/// </summary>
		/// <typeparam name="T">Type of the object</typeparam>
		/// <param name="writer">Reader to read data from</param>
		/// <param name="item">Item to write</param>
		/// <param name="writeItem">Function to read the payload, if non-null</param>
		/// <returns>Object instance or null</returns>
		public static void WriteNullable<T>(this BinaryWriter writer, T item, Action writeItem) where T : class
		{
			if (item == null)
			{
				writer.Write(false);
			}
			else
			{
				writer.Write(true);
				writeItem();
			}
		}

		/// <summary>
		/// Writes a value of a specific type to a binary writer
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="fieldType">Type of value to write</param>
		/// <param name="value">The value to output</param>
		public static void Write(this BinaryWriter writer, Type fieldType, object value)
		{
			if (fieldType == typeof(string))
			{
				writer.Write((string)value);
			}
			else if (fieldType == typeof(bool))
			{
				writer.Write((bool)value);
			}
			else if (fieldType == typeof(int))
			{
				writer.Write((int)value);
			}
			else if (fieldType == typeof(float))
			{
				writer.Write((float)value);
			}
			else if (fieldType == typeof(double))
			{
				writer.Write((double)value);
			}
			else if (fieldType.IsEnum)
			{
				writer.Write((int)value);
			}
			else if (fieldType == typeof(string[]))
			{
				writer.Write((string[])value);
			}
			else if (fieldType == typeof(bool?))
			{
				bool? nullableValue = (bool?)value;
				writer.Write(nullableValue.HasValue ? nullableValue.Value ? 1 : 0 : -1);
			}
			else if (fieldType == typeof(FileReference))
			{
				writer.Write((FileReference)value);
			}
			else if (fieldType.IsArray && fieldType.GetElementType()!.IsEnum)
			{
				List<int> arrayEnumValuesAsInts = [];
				Array arrayValues = (Array)value;
				foreach (object arrayValue in arrayValues)
				{
					arrayEnumValuesAsInts.Add((int)arrayValue);
				}
				writer.Write(arrayEnumValuesAsInts, writer.Write);
			}
			else
			{
				throw new Exception(String.Format("Unsupported type '{0}' for binary serialization", fieldType.Name));
			}
		}
	}
}
