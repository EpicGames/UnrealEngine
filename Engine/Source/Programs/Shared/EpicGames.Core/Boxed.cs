// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Core
{
	/// <summary>
	/// Explicitly boxes a value type, allowing it to be passed by reference to (and modified in) an async method.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class Boxed<T>
	{
		/// <summary>
		/// The struct value
		/// </summary>
		public T Value { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="value">The value to copy</param>
		public Boxed(T value)
		{
			Value = value;
		}

		/// <inheritdoc/>
		public override string? ToString()
		{
			return Value?.ToString();
		}

		/// <summary>
		/// Implicit conversion of a boxed value to value type
		/// </summary>
		/// <param name="boxed"></param>
		public static implicit operator T(Boxed<T> boxed)
		{
			return boxed.Value;
		}

		/// <summary>
		/// Implicit conversion of a boxed value to value type
		/// </summary>
		/// <param name="value"></param>
		public static implicit operator Boxed<T>(T value)
		{
			return new Boxed<T>(value);
		}
	}
}
