// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;

namespace EpicGames.Core
{
	/// <summary>
	/// Rolling hash by cyclic polynomial
	/// </summary>
	public class BuzHash
	{
		static readonly uint[] s_table =
		[
			0x458be752, 0xc10748cc, 0xfbbcdbb8, 0x6ded5b68, 0xb10a82b5, 0x20d75648, 0xdfc5665f, 0xa8428801, 0x7ebf5191, 0x841135c7, 0x65cc53b3,
			0x280a597c, 0x16f60255, 0xc78cbc3e, 0x294415f5, 0xb938d494, 0xec85c4e6, 0xb7d33edc, 0xe549b544, 0xfdeda5aa, 0x882bf287, 0x3116737c,
			0x05569956, 0xe8cc1f68, 0x0806ac5e, 0x22a14443, 0x15297e10, 0x50d090e7, 0x4ba60f6f, 0xefd9f1a7, 0x5c5c885c, 0x82482f93, 0x9bfd7c64,
			0x0b3e7276, 0xf2688e77, 0x8fad8abc, 0xb0509568, 0xf1ada29f, 0xa53efdfe, 0xcb2b1d00, 0xf2a9e986, 0x6463432b, 0x95094051, 0x5a223ad2,
			0x9be8401b, 0x61e579cb, 0x1a556a14, 0x5840fdc2, 0x9261ddf6, 0xcde002bb, 0x52432bb0, 0xbf17373e, 0x7b7c222f, 0x2955ed16, 0x9f10ca59,
			0xe840c4c9, 0xccabd806, 0x14543f34, 0x1462417a, 0x0d4a1f9c, 0x087ed925, 0xd7f8f24c, 0x7338c425, 0xcf86c8f5, 0xb19165cd, 0x9891c393,
			0x325384ac, 0x0308459d, 0x86141d7e, 0xc922116a, 0xe2ffa6b6, 0x53f52aed, 0x2cd86197, 0xf5b9f498, 0xbf319c8f, 0xe0411fae, 0x977eb18c,
			0xd8770976, 0x9833466a, 0xc674df7f, 0x8c297d45, 0x8ca48d26, 0xc49ed8e2, 0x7344f874, 0x556f79c7, 0x6b25eaed, 0xa03e2b42, 0xf68f66a4,
			0x8e8b09a2, 0xf2e0e62a, 0x0d3a9806, 0x9729e493, 0x8c72b0fc, 0x160b94f6, 0x450e4d3d, 0x7a320e85, 0xbef8f0e1, 0x21d73653, 0x4e3d977a,
			0x1e7b3929, 0x1cc6c719, 0xbe478d53, 0x8d752809, 0xe6d8c2c6, 0x275f0892, 0xc8acc273, 0x4cc21580, 0xecc4a617, 0xf5f7be70, 0xe795248a,
			0x375a2fe9, 0x425570b6, 0x8898dcf8, 0xdc2d97c4, 0x0106114b, 0x364dc22f, 0x1e0cad1f, 0xbe63803c, 0x5f69fac2, 0x4d5afa6f, 0x1bc0dfb5,
			0xfb273589, 0x0ea47f7b, 0x3c1c2b50, 0x21b2a932, 0x6b1223fd, 0x2fe706a8, 0xf9bd6ce2, 0xa268e64e, 0xe987f486, 0x3eacf563, 0x1ca2018c,
			0x65e18228, 0x2207360a, 0x57cf1715, 0x34c37d2b, 0x1f8f3cde, 0x93b657cf, 0x31a019fd, 0xe69eb729, 0x8bca7b9b, 0x4c9d5bed, 0x277ebeaf,
			0xe0d8f8ae, 0xd150821c, 0x31381871, 0xafc3f1b0, 0x927db328, 0xe95effac, 0x305a47bd, 0x426ba35b, 0x1233af3f, 0x686a5b83, 0x50e072e5,
			0xd9d3bb2a, 0x8befc475, 0x487f0de6, 0xc88dff89, 0xbd664d5e, 0x971b5d18, 0x63b14847, 0xd7d3c1ce, 0x7f583cf3, 0x72cbcb09, 0xc0d0a81c,
			0x7fa3429b, 0xe9158a1b, 0x225ea19a, 0xd8ca9ea3, 0xc763b282, 0xbb0c6341, 0x020b8293, 0xd4cd299d, 0x58cfa7f8, 0x91b4ee53, 0x37e4d140,
			0x95ec764c, 0x30f76b06, 0x5ee68d24, 0x679c8661, 0xa41979c2, 0xf2b61284, 0x4fac1475, 0x0adb49f9, 0x19727a23, 0x15a7e374, 0xc43a18d5,
			0x3fb1aa73, 0x342fc615, 0x924c0793, 0xbee2d7f0, 0x8a279de9, 0x4aa2d70c, 0xe24dd37f, 0xbe862c0b, 0x177c22c2, 0x5388e5ee, 0xcd8a7510,
			0xf901b4fd, 0xdbc13dbc, 0x6c0bae5b, 0x64efe8c7, 0x48b02079, 0x80331a49, 0xca3d8ae6, 0xf3546190, 0xfed7108b, 0xc49b941b, 0x32baf4a9,
			0xeb833a4a, 0x88a3f1a5, 0x3a91ce0a, 0x3cc27da1, 0x7112e684, 0x4a3096b1, 0x3794574c, 0xa3c8b6f3, 0x1d213941, 0x6e0a2e00, 0x233479f1,
			0x0f4cd82f, 0x6093edd2, 0x5d7d209e, 0x464fe319, 0xd4dcac9e, 0x0db845cb, 0xfb5e4bc3, 0xe0256ce1, 0x09fb4ed1, 0x0914be1e, 0xa5bdb2c3,
			0xc6eb57bb, 0x30320350, 0x3f397e91, 0xa67791bc, 0x86bc0e2c, 0xefa0a7e2, 0xe9ff7543, 0xe733612c, 0xd185897b, 0x329e5388, 0x91dd236b,
			0x2ecb0d93, 0xf4d82a3d, 0x35b5c03f, 0xe4e606f0, 0x05b21843, 0x37b45964, 0x5eff22f4, 0x6027f4cc, 0x77178b3c, 0xae507131, 0x7bf7cabc,
			0xf9c18d66, 0x593ade65, 0xd95ddf11,
		];

		uint _state;

		/// <summary>
		/// Constructor
		/// </summary>
		public BuzHash()
			: this(0)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="state"></param>
		public BuzHash(uint state)
		{
			_state = state;
		}

		/// <summary>
		/// Resets the state of the 
		/// </summary>
		public void Reset()
		{
			_state = 0;
		}

		/// <summary>
		/// Add a byte to the window and update the hash
		/// </summary>
		/// <param name="value">New value to append to the window</param>
		public void Add(byte value)
		{
			_state = Add(_state, value);
		}

		/// <summary>
		/// Add a byte to the window and update the hash
		/// </summary>
		/// <param name="span">New data to append to the window</param>
		public void Add(ReadOnlySpan<byte> span)
		{
			_state = Add(_state, span);
		}

		/// <summary>
		/// Static method for updating a hash given a particular byte value
		/// </summary>
		/// <param name="state">The current hash value</param>
		/// <param name="value">Byte to add to the hash</param>
		/// <returns>New hash value</returns>
		public static uint Add(uint state, byte value)
		{
			return Rol32(state, 1) ^ s_table[value];
		}

		/// <summary>
		/// Static method for appending a range of data to a hash value
		/// </summary>
		/// <param name="state">The current hash value</param>
		/// <param name="data">Data to append to the hash</param>
		/// <returns>New hash value</returns>
		public static unsafe uint Add(uint state, ReadOnlySpan<byte> data)
		{
			fixed (uint* table = s_table)
			fixed (byte* dataPtr = data)
			{
				for (int idx = 0; idx < data.Length; idx++)
				{
					state = Rol32(state, 1) ^ table[dataPtr[idx]];
				}
			}
			return state;
		}

		/// <summary>
		/// Removes a byte from the start of the hash window
		/// </summary>
		/// <param name="value">Value at the start of the window</param>
		/// <param name="count">Number of items in the hash</param>
		public void Sub(byte value, int count)
		{
			_state ^= Rol32(s_table[value], (int)(count - 1));
		}

		/// <summary>
		/// Removes a byte from the start of the hash window
		/// </summary>
		/// <param name="state">Current state</param>
		/// <param name="value">Value at the start of the window</param>
		/// <param name="count">Size of the window</param>
		public static uint Sub(uint state, byte value, int count)
		{
			return state ^ Rol32(s_table[value], (int)(count - 1));
		}

		/// <summary>
		/// Static method for appending a range of data to a hash value
		/// </summary>
		/// <param name="state">The current hash value</param>
		/// <param name="data">Data to append to the hash</param>
		/// <param name="count">Size of the window</param>
		/// <returns>New hash value</returns>
		public static unsafe uint Sub(uint state, ReadOnlySpan<byte> data, int count)
		{
			fixed (uint* table = s_table)
			fixed (byte* dataPtr = data)
			{
				for (int idx = 0; idx < data.Length; idx++)
				{
					state ^= Rol32(s_table[dataPtr[idx]], --count);
				}
			}
			return state;
		}

		/// <summary>
		/// Update the hash with the given data
		/// </summary>
		/// <param name="data"></param>
		public void Update(ReadOnlySpan<byte> data)
		{
			for (int idx = 0; idx < data.Length; ++idx)
			{
				Add(data[idx]);
			}
		}

		/// <summary>
		/// Consume data from a source until the hash is below a certain threshold 
		/// </summary>
		/// <param name="prev">Previous data added to the hash</param>
		/// <param name="next">Next data to be added to the hash</param>
		/// <param name="windowSize">Size of the window over which the rolling hash is computed</param>
		/// <param name="threshold">Threshold for hash below which to terminate the scan</param>
		/// <param name="state">The current hash value</param>
		/// <returns>Number of bytes consumed from next</returns>
#pragma warning disable CA1045 // Do not pass types by reference
		public static unsafe int Update(ReadOnlySpan<byte> prev, ReadOnlySpan<byte> next, int windowSize, uint threshold, ref uint state)
#pragma warning restore CA1045 // Do not pass types by reference
		{
			Debug.Assert(prev.Length == next.Length);

			fixed (uint* table = s_table)
			fixed (byte* prevPtr = prev)
			fixed (byte* nextPtr = next)
			{
				uint hash = state;
				for (int idx = 0; idx < prev.Length; idx++)
				{
					if (hash < threshold)
					{
						state = hash;
						return idx;
					}
					hash ^= Rol32(table[prevPtr[idx]], windowSize - 1);
					hash = Rol32(hash, 1) ^ table[nextPtr[idx]];
				}
				state = hash;
			}

			return -1;
		}

		/// <summary>
		/// Finds split points from a buffer.
		/// </summary>
		/// <param name="span">Memory to find chunk points for</param>
		/// <param name="minSize">Minimum size for each chunk</param>
		/// <param name="maxSize">Maximum size for each chunk</param>
		/// <param name="targetSize">Target size for each chunk</param>
		/// <returns>List of chunk offsets and hashes</returns>
		public static List<(int, uint)> FindSplitPoints(ReadOnlySpan<byte> span, int minSize, int maxSize, int targetSize)
		{
			List<(int, uint)> chunkPoints = [];

			int offset = 0;
			for (; ; )
			{
				int length = FindChunkLength(span[offset..], minSize, maxSize, targetSize, out uint rollingHash);

				offset += length;
				if (offset >= span.Length)
				{
					break;
				}

				chunkPoints.Add((offset, rollingHash));
			}

			return chunkPoints;
		}

		/// <summary>
		/// Finds candidate chunk points from a buffer, ignoring minimum/maximum chunk sizes.
		/// Can be used to precalculate chunking points when hashing in parallel.
		/// </summary>
		/// <param name="span">Memory to find chunk points for</param>
		/// <param name="windowSize">Size of the hash window</param>
		/// <param name="threshold">Threshold below which to add a chunk point</param>
		/// <returns>List of chunk offsets and hashes</returns>
		public static unsafe List<(int, uint)> FindCandidateSplitPoints(ReadOnlySpan<byte> span, int windowSize, uint threshold)
		{
			List<(int, uint)> offsets = [];
			if (span.Length < windowSize)
			{
				return offsets;
			}

			uint state = Add(0, span[0..windowSize]);

			ReadOnlySpan<byte> prev = span.Slice(0, span.Length - windowSize);
			ReadOnlySpan<byte> next = span.Slice(windowSize);

			fixed (uint* table = s_table)
			fixed (byte* prevPtr = prev)
			fixed (byte* nextPtr = next)
			{
				uint hash = state;
				for (int idx = 0; idx < prev.Length; idx++)
				{
					if (hash < threshold)
					{
						offsets.Add((idx + windowSize, hash));
					}
					hash ^= Rol32(table[prevPtr[idx]], windowSize - 1);
					hash = Rol32(hash, 1) ^ table[nextPtr[idx]];
				}
				state = hash;
			}

			return offsets;
		}

		/// <summary>
		/// Gets the current hash value
		/// </summary>
		/// <returns>The current hash value</returns>
		public uint Get()
		{
			return _state;
		}

		static uint Rol32(uint v, int n)
		{
			n &= 31;
			return ((v) << (n)) | ((v) >> (32 - n));
		}

		/// <summary>
		/// Gets the threshold below which hash values should statistically occur on average after the given number of entries
		/// </summary>
		/// <param name="targetSize"></param>
		/// <returns></returns>
		public static uint GetThreshold(int targetSize)
		{
			return (uint)((1L << 32) / targetSize);
		}

		/// <summary>
		/// Gets the length of a chunk up to the next content-defined boundary
		/// </summary>
		/// <param name="inputData">Input data to find a boundary in</param>
		/// <param name="minSize">Minimum size for a chunk</param>
		/// <param name="maxSize">Maximum size for a chunk</param>
		/// <param name="targetSize">Desired average size for a chunk</param>
		/// <param name="rollingHash">Receives the rolling hash for the chunk</param>
		/// <returns></returns>
		public static int FindChunkLength(ReadOnlySpan<byte> inputData, int minSize, int maxSize, int targetSize, out uint rollingHash)
		{
			// If the target option sizes are fixed, just chunk the data along fixed boundaries
			if (minSize == targetSize && maxSize == targetSize)
			{
				rollingHash = 0;
				return Math.Min(inputData.Length, maxSize);
			}

			// Cap the append data span to the maximum amount we can add
			int maxLength = maxSize;
			if (maxLength < inputData.Length)
			{
				inputData = inputData.Slice(0, maxLength);
			}

			int windowSize = minSize;

			// Fast path for appending data to the buffer up to the chunk window size
			int length = Math.Min(windowSize, inputData.Length);
			rollingHash = Add(0, inputData.Slice(0, length));

			// Get the threshold for the rolling hash to split the output
			uint rollingHashThreshold = (uint)((1L << 32) / (targetSize - minSize));

			// Step through the rest of the data which is completely contained in appendData.
			if (length < inputData.Length)
			{
				Debug.Assert(length >= windowSize);

				ReadOnlySpan<byte> tailSpan = inputData.Slice(length - windowSize, inputData.Length - windowSize);
				ReadOnlySpan<byte> headSpan = inputData.Slice(length);

				int count = Update(tailSpan, headSpan, minSize, rollingHashThreshold, ref rollingHash);
				if (count != -1)
				{
					length += count;
					return length;
				}

				length += headSpan.Length;
			}

			return length;
		}
	}
}
