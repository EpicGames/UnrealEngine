// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Class for building byte sequences, similar to StringBuilder. Allocates memory in chunks to avoid copying data.
	/// </summary>
	public class ByteArrayBuilder : IMemoryWriter
	{
		class Chunk
		{
			public readonly int RunningIndex;
			public readonly byte[] Data;
			public int Length { get; set; }

			public Chunk(int runningIndex, int size)
			{
				RunningIndex = runningIndex;
				Data = new byte[size];
			}

			public ReadOnlySpan<byte> WrittenSpan => Data.AsSpan(0, Length);
			public ReadOnlyMemory<byte> WrittenMemory => Data.AsMemory(0, Length);
		}

		readonly List<Chunk> _chunks = [];
		readonly int _chunkSize;
		Chunk _currentChunk;

		/// <summary>
		/// Length of the current sequence
		/// </summary>
		public int Length { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="chunkSize"></param>
		public ByteArrayBuilder(int chunkSize)
			: this(chunkSize, chunkSize)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="initialSize">Size of the initial chunk</param>
		/// <param name="chunkSize">Default size for subsequent chunks</param>
		public ByteArrayBuilder(int initialSize = 4096, int chunkSize = 4096)
		{
			_currentChunk = new Chunk(0, initialSize);
			_chunks.Add(_currentChunk);
			_chunkSize = chunkSize;
		}

		/// <summary>
		/// Clear the current builder
		/// </summary>
		public void Clear()
		{
			_currentChunk = _chunks[0];
			_currentChunk.Length = 0;
			_chunks.RemoveRange(1, _chunks.Count - 1);
			Length = 0;
		}

		/// <inheritdoc/>
		public Span<byte> GetSpan(int sizeHint) => GetMemory(sizeHint).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int sizeHint)
		{
			int requiredSize = _currentChunk.Length + Math.Max(sizeHint, 1);
			if (requiredSize > _currentChunk.Data.Length)
			{
				_currentChunk = new Chunk(_currentChunk.RunningIndex + _currentChunk.Length, Math.Max(sizeHint, _chunkSize));
				_chunks.Add(_currentChunk);
			}
			return _currentChunk.Data.AsMemory(_currentChunk.Length);
		}

		/// <inheritdoc/>
		public void Advance(int length)
		{
			if (length < 0 || _currentChunk.Length + length > _currentChunk.Data.Length)
			{
				throw new ArgumentException("Length exceeds possible size of data written to output buffer", nameof(length));
			}

			_currentChunk.Length += length;
			Length += length;
		}

		/// <summary>
		/// Appends data in this builder to the given sequence
		/// </summary>
		/// <param name="builder">Sequence builder</param>
		public void AppendTo(ReadOnlySequenceBuilder<byte> builder)
		{
			foreach (Chunk chunk in _chunks)
			{
				builder.Append(chunk.WrittenMemory);
			}
		}

		/// <summary>
		/// Gets a contiguous, non-owned buffer containing the current data. Will merge together current chunks if necessary.
		/// </summary>
		/// <returns></returns>
		public ReadOnlyMemory<byte> AsMemory()
		{
			if (_chunks.Count == 1)
			{
				return _chunks[0].WrittenMemory;
			}
			else
			{
				return ToByteArray();
			}
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far
		/// </summary>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence()
		{
			ReadOnlySequenceBuilder<byte> builder = new();
			AppendTo(builder);
			return builder.Construct();
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far, starting at the given offset
		/// </summary>
		/// <param name="offset">Offset to start from</param>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence(int offset)
		{
			// TODO: could do a binary search for offset and work forwards from there
			return AsSequence().Slice(offset);
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far, starting at the given offset and 
		/// </summary>
		/// <param name="offset">Offset to start from</param>
		/// <param name="length">Length of the sequence to return</param>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence(int offset, int length)
		{
			// TODO: could do a binary search for offset and work fowards from there
			return AsSequence().Slice(offset, length);
		}

		/// <summary>
		/// Copies the data to the given output span
		/// </summary>
		/// <param name="span">Span to write to</param>
		public void CopyTo(Span<byte> span)
		{
			foreach (Chunk chunk in _chunks)
			{
				Span<byte> output = span.Slice(chunk.RunningIndex);
				chunk.WrittenSpan.CopyTo(output);
			}
		}

		/// <summary>
		/// Create a byte array from the sequence
		/// </summary>
		/// <returns>Byte array containing the current buffer data</returns>
		public byte[] ToByteArray()
		{
			byte[] data = new byte[Length];
			CopyTo(data);
			return data;
		}
	}
}
