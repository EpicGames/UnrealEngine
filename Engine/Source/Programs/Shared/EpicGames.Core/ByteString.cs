// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Wraps a sequence of bytes that can be manipulated with value-like semantics
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(ByteStringJsonConverter))]
	[TypeConverter(typeof(ByteStringTypeConverter))]
	public readonly struct ByteString : IEquatable<ByteString>, IComparable<ByteString>
	{
		/// <summary>
		/// Underlying data for this string
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Accessor for the data underlying this string
		/// </summary>
		public ReadOnlySpan<byte> Span => Data.Span;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data"></param>
		public ByteString(ReadOnlyMemory<byte> data) => Data = data;

		/// <summary>
		/// Parses a byte string from a hexadecimal string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>New byte string instance</returns>
		public static ByteString Parse(ReadOnlySpan<byte> text) => new(StringUtils.ParseHexString(text));

		/// <summary>
		/// Parses a byte string from a hexadecimal string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>New byte string instance</returns>
		public static ByteString Parse(Utf8String text) => Parse(text.Span);

		/// <summary>
		/// Parses a byte string from a hexadecimal string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>New byte string instance</returns>
		public static ByteString Parse(string text) => new(StringUtils.ParseHexString(text));

		/// <summary>
		/// Attempts to parse a hexadecimal string as a byte string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="byteString">On success, receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParse(ReadOnlySpan<byte> text, out ByteString byteString)
		{
			byte[]? bytes;
			if (StringUtils.TryParseHexString(text, out bytes))
			{
				byteString = new ByteString(bytes);
				return true;
			}
			else
			{
				byteString = default;
				return false;
			}
		}

		/// <summary>
		/// Attempts to parse a hexadecimal string as a byte string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="byteString">On success, receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParse(Utf8String text, out ByteString byteString) => TryParse(text.Span, out byteString);

		/// <summary>
		/// Attempts to parse a hexadecimal string as a byte string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="byteString">On success, receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParse(string text, out ByteString byteString)
		{
			byte[]? bytes;
			if (StringUtils.TryParseHexString(text, out bytes))
			{
				byteString = new ByteString(bytes);
				return true;
			}
			else
			{
				byteString = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public int CompareTo(ByteString other) => Data.Span.SequenceCompareTo(other.Data.Span);

		/// <inheritdoc/>
		public override bool Equals([NotNullWhen(true)] object? obj) => obj is ByteString byteString && Equals(byteString);

		/// <inheritdoc/>
		public bool Equals(ByteString other) => Data.Span.SequenceEqual(other.Data.Span);

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			HashCode hashCode = new();
			hashCode.AddBytes(Data.Span);
			return hashCode.ToHashCode();
		}

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(Data.Span);

		/// <summary>
		/// Creates a Utf8 string representing the bytes in this string
		/// </summary>
		public Utf8String ToUtf8String() => StringUtils.FormatUtf8HexString(Data.Span);

#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member
		public static bool operator ==(ByteString left, ByteString right) => left.Equals(right);
		public static bool operator !=(ByteString left, ByteString right) => !(left == right);
		public static bool operator <(ByteString left, ByteString right) => left.CompareTo(right) < 0;
		public static bool operator <=(ByteString left, ByteString right) => left.CompareTo(right) <= 0;
		public static bool operator >(ByteString left, ByteString right) => left.CompareTo(right) > 0;
		public static bool operator >=(ByteString left, ByteString right) => left.CompareTo(right) >= 0;
#pragma warning restore CS1591 // Missing XML comment for publicly visible type or member
	}

	/// <summary>
	/// Type converter for ClusterId to and from JSON
	/// </summary>
	sealed class ByteStringJsonConverter : JsonConverter<ByteString>
	{
		/// <inheritdoc/>
		public override ByteString Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => ByteString.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ByteString value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToUtf8String().Span);
	}

	/// <summary>
	/// Type converter from strings to ClusterId objects
	/// </summary>
	sealed class ByteStringTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type? sourceType)
		{
			return
				sourceType == typeof(byte[]) ||
				sourceType == typeof(Memory<byte>) ||
				sourceType == typeof(ReadOnlyMemory<byte>) ||
				sourceType == typeof(string) ||
				sourceType == typeof(Utf8String);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object? value)
		{
			switch (value)
			{
				case null:
					return null;
				case byte[] bytes:
					return new ByteString(bytes);
				case Memory<byte> bytes:
					return new ByteString(bytes);
				case ReadOnlyMemory<byte> bytes:
					return new ByteString(bytes);
				case string stringValue:
					return ByteString.Parse(stringValue);
				case Utf8String stringValue:
					return ByteString.Parse(stringValue);
				default:
					throw new InvalidOperationException();
			}
		}
	}
}
