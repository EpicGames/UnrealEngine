// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper class to create a waitable task out of a cancellation token
	/// </summary>
	public sealed class CancellationTask : IDisposable
	{
		/// <summary>
		/// Completion source for the task
		/// </summary>
		readonly TaskCompletionSource<bool> _completionSource;

		/// <summary>
		/// Registration handle with the cancellation token
		/// </summary>
		readonly CancellationTokenRegistration _registration;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="token">The cancellation token to register with</param>
		public CancellationTask(CancellationToken token)
		{
			_completionSource = new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously);
			_registration = token.Register(() => _completionSource.TrySetResult(true));
		}

		/// <summary>
		/// The task that can be waited on
		/// </summary>
		public Task Task => _completionSource.Task;

		/// <summary>
		/// Dispose of any allocated resources
		/// </summary>
		public void Dispose()
		{
			_registration.Dispose();
		}
	}
}
