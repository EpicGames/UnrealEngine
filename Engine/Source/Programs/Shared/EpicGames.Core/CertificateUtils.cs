// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Security.Cryptography;
using System.Security.Cryptography.X509Certificates;
using System.Text.RegularExpressions;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for certificates
	/// </summary>
	public static class CertificateUtils
	{
		/// <summary>
		/// Create a self signed cert for the agent
		/// </summary>
		/// <param name="dnsName"></param>
		/// <param name="friendlyName">Friendly name for the certificate</param>
		/// <returns></returns>
		public static byte[] CreateSelfSignedCert(string dnsName, string friendlyName)
		{
			dnsName = Regex.Replace(dnsName, @"^[a-zA-Z]+://", "");
			dnsName = Regex.Replace(dnsName, "/.*$", "");

			using (RSA algorithm = RSA.Create(2048))
			{
				X500DistinguishedName distinguishedName = new($"CN={dnsName}");
				CertificateRequest request = new(distinguishedName, algorithm, HashAlgorithmName.SHA256, RSASignaturePadding.Pkcs1);

				request.CertificateExtensions.Add(new X509BasicConstraintsExtension(false, false, 0, true));
				request.CertificateExtensions.Add(new X509KeyUsageExtension(X509KeyUsageFlags.KeyEncipherment | X509KeyUsageFlags.DigitalSignature, true));
				request.CertificateExtensions.Add(new X509EnhancedKeyUsageExtension([new Oid("1.3.6.1.5.5.7.3.1")], true));

				SubjectAlternativeNameBuilder alternativeNameBuilder = new();
				alternativeNameBuilder.AddDnsName(dnsName);
				request.CertificateExtensions.Add(alternativeNameBuilder.Build(true));

				// NB: MacOS requires 825 days or fewer (https://support.apple.com/en-us/HT210176)
				using (X509Certificate2 certificate = request.CreateSelfSigned(new DateTimeOffset(DateTime.UtcNow.AddDays(-1)), new DateTimeOffset(DateTime.UtcNow.AddDays(800))))
				{
					if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						certificate.FriendlyName = friendlyName;
					}
					byte[] privateCertData = certificate.Export(X509ContentType.Pkcs12); // Note: Need to reimport this to use immediately, otherwise key is ephemeral
					return privateCertData;
				}
			}
		}
	}
}
