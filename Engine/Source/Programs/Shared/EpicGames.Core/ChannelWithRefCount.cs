// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Channels;

namespace EpicGames.Core
{
	/// <summary>
	/// Implements a <see cref="Channel{T}"/> which signals completion according to a reference count.
	/// </summary>
	public class ChannelWithRefCount<T> : Channel<T>
	{
		int _count = 1;

		/// <summary>
		/// Called when the channel is completed
		/// </summary>
		public event Action? OnComplete;

		/// <summary>
		/// Constructor
		/// </summary>
		internal ChannelWithRefCount(Channel<T> inner)
		{
			Reader = inner.Reader;
			Writer = inner.Writer;
		}

		/// <summary>
		/// Adds a new reference to the channel
		/// </summary>
		public ChannelWithRefCount<T> AddRef()
		{
			Interlocked.Increment(ref _count);
			return this;
		}

		/// <summary>
		/// Releases a reference to the channel
		/// </summary>
		public void Release()
		{
			if (Interlocked.Decrement(ref _count) == 0)
			{
				OnComplete?.Invoke();
				Writer.Complete();
			}
		}
	}

	/// <summary>
	/// Extension methods for <see cref="Channel{T}"/>
	/// </summary>
	public static class ChannelExtensions
	{
		/// <summary>
		/// Creates a channel whose completion state is controlled by a reference count
		/// </summary>
		public static ChannelWithRefCount<T> WithRefCount<T>(this Channel<T> channel)
			=> new(channel);
	}
}
