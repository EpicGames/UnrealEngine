// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core;

/// <summary>
/// Base interface for a scheduled event
/// </summary>
public interface ITicker : IAsyncDisposable
{
	/// <summary>
	/// Start the ticker
	/// </summary>
	Task StartAsync();

	/// <summary>
	/// Stop the ticker
	/// </summary>
	Task StopAsync();
}

/// <summary>
/// Interface representing time and scheduling events which is pluggable during testing. In normal use, the Clock implementation below is used. 
/// </summary>
public interface IClock
{
	/// <summary>
	/// Return time expressed as the Coordinated Universal Time (UTC)
	/// </summary>
	DateTime UtcNow { get; }

	/// <summary>
	/// Time zone for schedules etc...
	/// </summary>
	TimeZoneInfo TimeZone { get; }

	/// <summary>
	/// Create an event that will trigger after the given time
	/// </summary>
	/// <param name="name">Name of the event</param>
	/// <param name="interval">Time after which the event will trigger</param>
	/// <param name="tickAsync">Callback for the tick. Returns the time interval until the next tick, or null to cancel the tick.</param>
	/// <param name="logger">Logger for error messages</param>
	/// <returns>Handle to the event</returns>
	ITicker AddTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger);

	/// <summary>
	/// Create a ticker shared between all server processes.
	/// Callback can be run inside any available process but will still only be called once per tick.
	/// </summary>
	/// <param name="name">Name of the event</param>
	/// <param name="interval">Time after which the event will trigger</param>
	/// <param name="tickAsync">Callback for the tick. Returns the time interval until the next tick, or null to cancel the tick.</param>
	/// <param name="logger">Logger for error messages</param>
	/// <returns>New ticker instance</returns>
	ITicker AddSharedTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger);
}

/// <summary>
/// Placeholder interface for ITicker
/// </summary>
public sealed class NullTicker : ITicker
{
	/// <inheritdoc/>
	public ValueTask DisposeAsync() => new();

	/// <inheritdoc/>
	public Task StartAsync() => Task.CompletedTask;

	/// <inheritdoc/>
	public Task StopAsync() => Task.CompletedTask;
}

/// <summary>
/// A default implementation of IClock for normal production use
/// </summary>
public class DefaultClock : IClock
{
	/// <inheritdoc/>
	public DateTime UtcNow => DateTime.UtcNow;

	/// <inheritdoc/>
	public TimeZoneInfo TimeZone => TimeZoneInfo.Local;

	/// <inheritdoc/>
	public ITicker AddTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger)
	{
		return new NullTicker();
	}

	/// <inheritdoc/>
	public ITicker AddSharedTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
	{
		return new NullTicker();
	}
}

/// <summary>
/// A stub implementation of IClock. Intended for testing to override time.
/// </summary>
public class StubClock : IClock
{
	private DateTime _utcNow = DateTime.UtcNow;

	/// <inheritdoc/>
	public DateTime UtcNow
	{
		get => _utcNow;
		set => _utcNow = value.ToUniversalTime();
	}

	/// <inheritdoc/>
	public TimeZoneInfo TimeZone { get; set; } = TimeZoneInfo.Local;

	/// <inheritdoc/>
	public ITicker AddTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger)
	{
		throw new NotImplementedException("Not available in stub implementation");
	}

	/// <inheritdoc/>
	public ITicker AddSharedTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
	{
		throw new NotImplementedException("Not available in stub implementation");
	}

	/// <summary>
	/// Advance the time
	/// </summary>
	/// <param name="delta"></param>
	public void Advance(TimeSpan delta)
	{
		_utcNow += delta;
	}
}

/// <summary>
/// Extension methods for <see cref="IClock"/>
/// </summary>
public static class ClockExtensions
{
	/// <summary>
	/// Create an event that will trigger after the given time
	/// </summary>
	/// <param name="clock">Clock to schedule the event on</param>
	/// <param name="name">Name of the ticker</param>
	/// <param name="interval">Interval for the callback</param>
	/// <param name="tickAsync">Trigger callback</param>
	/// <param name="logger">Logger for any error messages</param>
	/// <returns>Handle to the event</returns>
	public static ITicker AddTicker(this IClock clock, string name, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
	{
		async ValueTask<TimeSpan?> WrappedTrigger(CancellationToken token)
		{
			Stopwatch timer = Stopwatch.StartNew();
			await tickAsync(token);
			return interval - timer.Elapsed;
		}

		return clock.AddTicker(name, interval, WrappedTrigger, logger);
	}

	/// <summary>
	/// Create an event that will trigger after the given time
	/// </summary>
	/// <param name="clock">Clock to schedule the event on</param>
	/// <param name="interval">Time after which the event will trigger</param>
	/// <param name="tickAsync">Callback for the tick. Returns the time interval until the next tick, or null to cancel the tick.</param>
	/// <param name="logger">Logger for error messages</param>
	/// <returns>Handle to the event</returns>
	public static ITicker AddTicker<T>(this IClock clock, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger)
		=> clock.AddTicker(typeof(T).Name, interval, tickAsync, logger);

	/// <summary>
	/// Create an event that will trigger after the given time
	/// </summary>
	/// <param name="clock">Clock to schedule the event on</param>
	/// <param name="interval">Interval for the callback</param>
	/// <param name="tickAsync">Trigger callback</param>
	/// <param name="logger">Logger for any error messages</param>
	/// <returns>Handle to the event</returns>
	public static ITicker AddTicker<T>(this IClock clock, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
		=> clock.AddTicker(typeof(T).Name, interval, tickAsync, logger);

	/// <summary>
	/// Create a ticker shared between all server pods
	/// </summary>
	/// <param name="clock">Clock to schedule the event on</param>
	/// <param name="interval">Time after which the event will trigger</param>
	/// <param name="tickAsync">Callback for the tick. Returns the time interval until the next tick, or null to cancel the tick.</param>
	/// <param name="logger">Logger for error messages</param>
	/// <returns>New ticker instance</returns>
	public static ITicker AddSharedTicker<T>(this IClock clock, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
		=> clock.AddSharedTicker(typeof(T).Name, interval, tickAsync, logger);
}
