// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Threading.Tasks;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for commands
	/// </summary>
	public interface ICommand
	{
		/// <summary>
		/// Configure this object with the given command line arguments
		/// </summary>
		/// <param name="arguments">Command line arguments</param>
		/// <param name="logger">Logger for output</param>
		void Configure(CommandLineArguments arguments, ILogger logger);

		/// <summary>
		/// Gets all command line parameters to show in help for this command
		/// </summary>
		/// <param name="arguments">The command line arguments</param>
		/// <returns>List of name/description pairs</returns>
		List<KeyValuePair<string, string>> GetParameters(CommandLineArguments arguments);

		/// <summary>
		/// Execute this command
		/// </summary>
		/// <param name="logger">The logger to use for this command</param>
		/// <returns>Exit code</returns>
		Task<int> ExecuteAsync(ILogger logger);
	}

	/// <summary>
	/// Interface describing a command that can be exectued
	/// </summary>
	public interface ICommandFactory
	{
		/// <summary>
		/// Names for this command
		/// </summary>
		public IReadOnlyList<string> Names { get; }

		/// <summary>
		/// Short description for the mode. Will be displayed in the help text.
		/// </summary>
		public string Description { get; }

		/// <summary>
		/// Whether to include this command in help text
		/// </summary>
		public bool Advertise { get; }

		/// <summary>
		/// Whether the command accepts positional arguments
		/// </summary>
		public bool AcceptsPositionalArguments { get; }

		/// <summary>
		/// Create a command instance
		/// </summary>
		public ICommand CreateInstance(IServiceProvider serviceProvider);
	}

#pragma warning disable CA1019 // Define accessors for attribute arguments
	/// <summary>
	/// Attribute used to specify names of program modes, and help text
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class CommandAttribute : Attribute
	{
		/// <summary>
		/// Names for this command
		/// </summary>
		public string[] Names { get; }

		/// <summary>
		/// Short description for the mode. Will be displayed in the help text.
		/// </summary>
		public string Description { get; }

		/// <summary>
		/// Whether to include the command in help listings
		/// </summary>
		public bool Advertise { get; set; } = true;

		/// <summary>
		/// Whether the command takes positional arguments
		/// </summary>
		public bool AcceptsPositionalArguments { get; set; } = false;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the mode</param>
		/// <param name="description">Short description for display in the help text</param>
		public CommandAttribute(string name, string description)
		{
			Names = [name];
			Description = description;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="category">Category for this command</param>
		/// <param name="name">Name of the mode</param>
		/// <param name="description">Short description for display in the help text</param>
		public CommandAttribute(string category, string name, string description)
		{
			Names = [category, name];
			Description = description;
		}
	}
#pragma warning restore CA1019 // Define accessors for attribute arguments

	/// <summary>
	/// Base class for all commands that can be executed by HordeAgent
	/// </summary>
	public abstract class Command : ICommand
	{
		/// <inheritdoc/>
		public virtual void Configure(CommandLineArguments arguments, ILogger logger)
		{
			arguments.ApplyTo(this, logger);
		}

		/// <inheritdoc/>
		public virtual List<KeyValuePair<string, string>> GetParameters(CommandLineArguments arguments)
		{
			return CommandLineArguments.GetParameters(GetType());
		}

		/// <inheritdoc/>
		public abstract Task<int> ExecuteAsync(ILogger logger);
	}

	/// <summary>
	/// Default implementation of a command factory
	/// </summary>
	class CommandFactory : ICommandFactory
	{
		public IReadOnlyList<string> Names { get; }
		public string Description { get; }
		public bool Advertise { get; }
		public bool AcceptsPositionalArguments { get; }

		public Type Type { get; }

		public CommandFactory(string[] names, string description, bool advertise, bool acceptsPositionalArguments, Type type)
		{
			Names = names;
			Description = description;
			Advertise = advertise;
			AcceptsPositionalArguments = acceptsPositionalArguments;
			Type = type;
		}

		public ICommand CreateInstance(IServiceProvider serviceProvider) => (ICommand)serviceProvider.GetRequiredService(Type);
		public override string ToString() => String.Join(" ", Names);
	}

	/// <summary>
	/// Entry point for dispatching commands
	/// </summary>
	public static class CommandHost
	{
		/// <summary>
		/// Adds services for executing the 
		/// </summary>
		/// <param name="services"></param>
		/// <param name="assembly"></param>
		public static void AddCommandsFromAssembly(this IServiceCollection services, Assembly assembly)
		{
			List<(CommandAttribute, Type)> commands = [];
			foreach (Type type in assembly.GetTypes())
			{
				if (typeof(ICommand).IsAssignableFrom(type) && !type.IsAbstract)
				{
					CommandAttribute? attribute = type.GetCustomAttribute<CommandAttribute>();
					if (attribute != null)
					{
						services.AddTransient(type);
						services.AddTransient(typeof(ICommandFactory), sp => new CommandFactory(attribute.Names, attribute.Description, attribute.Advertise, attribute.AcceptsPositionalArguments, type));
					}
				}
			}
		}

		/// <summary>
		/// Entry point for executing registered command types in a particular assembly
		/// </summary>
		/// <param name="args">Command line arguments</param>
		/// <param name="serviceProvider">The service provider for the application</param>
		/// <param name="defaultCommandType">The default command type</param>
		/// <param name="toolDescription">Description of the tool, to print at the top of help text.</param>
		/// <returns>Return code from the command</returns>
		public static async Task<int> RunAsync(CommandLineArguments args, IServiceProvider serviceProvider, Type? defaultCommandType, string? toolDescription = null)
		{
			// Find all the command types
			List<ICommandFactory> commandFactories = [.. serviceProvider.GetServices<ICommandFactory>()];

			// Check if there's a matching command
			ICommand? command = null;
			ICommandFactory? commandFactory = null;

			// Parse the positional arguments for the command name
			string[] positionalArgs = args.GetPositionalArguments();
			if (positionalArgs.Length == 0)
			{
				if (defaultCommandType == null || args.HasOption("-Help"))
				{
					if (toolDescription != null)
					{
						foreach (string line in toolDescription.Split('\n'))
						{
							Console.WriteLine(line);
						}
						Console.WriteLine("");
					}

					Console.WriteLine("Usage:");
					Console.WriteLine("    [Command] [-Option1] [-Option2]...");
					Console.WriteLine("");
					Console.WriteLine("Commands:");

					PrintCommands(commandFactories);

					Console.WriteLine("");
					Console.WriteLine("Specify \"<CommandName> -Help\" for command-specific help");
					return 0;
				}
				else
				{
					command = (ICommand)serviceProvider.GetService(defaultCommandType)!;
				}
			}
			else
			{
				foreach (ICommandFactory factory in commandFactories)
				{
					IEnumerable<string> positionalArgsEnum = positionalArgs;
					if (factory.AcceptsPositionalArguments && factory.Names.Count < positionalArgs.Length)
					{
						positionalArgsEnum = positionalArgsEnum.Take(factory.Names.Count);
					}
					if (factory.Names.SequenceEqual(positionalArgsEnum, StringComparer.OrdinalIgnoreCase))
					{
						args.RemovePositionalArguments(factory.Names.Count);
						command = factory.CreateInstance(serviceProvider);
						commandFactory = factory;
						break;
					}
				}
				if (command == null)
				{
					ConsoleUtils.WriteError($"Invalid command '{String.Join(" ", positionalArgs)}'");
					Console.WriteLine("");
					Console.WriteLine("Available commands:");

					PrintCommands(commandFactories);
					return 1;
				}
			}

			// If the help flag is specified, print the help info and exit immediately
			if (args.HasOption("-Help"))
			{
				if (commandFactory == null)
				{
					HelpUtils.PrintHelp(null, null, command.GetParameters(args));
				}
				else
				{
					HelpUtils.PrintHelp($"Command: {String.Join(" ", commandFactory.Names)}", commandFactory.Description, command.GetParameters(args));
				}
				return 1;
			}

			// Configure the command
			ILogger logger = serviceProvider.GetRequiredService<ILogger<Command>>();
			try
			{
				command.Configure(args, logger);
				args.CheckAllArgumentsUsed(logger);
			}
			catch (CommandLineArgumentException ex)
			{
				ConsoleUtils.WriteError(ex.Message);
				Console.WriteLine("");
				Console.WriteLine("Valid parameters:");

				HelpUtils.PrintTable(command.GetParameters(args), 4, 24);
				return 1;
			}

			// Execute all the commands
			try
			{
				return await command.ExecuteAsync(logger);
			}
			catch (FatalErrorException ex)
			{
				logger.LogCritical(ex, "Fatal error: {Message}", ex.Message);
				return ex.ExitCode;
			}
			catch (Exception ex)
			{
				logger.LogCritical(ex, "Fatal error: {ExceptionInfo}", ex.ToString());
				return 1;
			}
		}

		/// <summary>
		/// Print a formatted list of all the available commands
		/// </summary>
		/// <param name="attributes">List of command attributes</param>
		static void PrintCommands(IEnumerable<ICommandFactory> attributes)
		{
			List<KeyValuePair<string, string>> commands = [];
			foreach (ICommandFactory attribute in attributes)
			{
				if (attribute.Advertise)
				{
					commands.Add(new KeyValuePair<string, string>(String.Join(" ", attribute.Names), attribute.Description));
				}
			}
			HelpUtils.PrintTable([.. commands.OrderBy(x => x.Key)], 4, 20);
		}
	}
}
