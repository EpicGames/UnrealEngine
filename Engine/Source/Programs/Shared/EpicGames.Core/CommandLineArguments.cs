// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using Microsoft.Extensions.Logging;

#pragma warning disable CA1710 // Identifiers should have correct suffix

namespace EpicGames.Core
{
	/// <summary>
	/// Helper class to visualize an argument list
	/// </summary>
	class CommandLineArgumentListView
	{
		/// <summary>
		/// The list of arguments
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
		public string[] Arguments { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="argumentList">The argument list to proxy</param>
		public CommandLineArgumentListView(CommandLineArguments argumentList)
		{
			Arguments = argumentList.GetRawArray();
		}
	}

	/// <summary>
	/// Exception thrown for invalid command line arguments
	/// </summary>
	public class CommandLineArgumentException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message to display for this exception</param>
		public CommandLineArgumentException(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message to display for this exception</param>
		/// <param name="innerException">The inner exception</param>
		public CommandLineArgumentException(string message, Exception innerException)
			: base(message, innerException)
		{
		}

		/// <summary>
		/// Converts this exception to a string
		/// </summary>
		/// <returns>Exception message</returns>
		public override string ToString()
		{
			return Message;
		}
	}

	/// <summary>
	/// Stores a list of command line arguments, allowing efficient ad-hoc queries of particular options (eg. "-Flag") and retreival of typed values (eg. "-Foo=Bar"), as
	/// well as attribute-driven application to fields with the [CommandLine] attribute applied.
	/// 
	/// Also tracks which arguments have been retrieved, allowing the display of diagnostic messages for invalid arguments.
	/// </summary>
	[DebuggerDisplay("Count = {Count}")]
	[DebuggerTypeProxy(typeof(CommandLineArgumentListView))]
	[JsonConverter(typeof(CommandLineArgumentsJsonConverter))]
	public class CommandLineArguments : IReadOnlyList<string>, IReadOnlyCollection<string>, IEnumerable<string>, IEnumerable
	{
		/// <summary>
		/// Information about a property or field that can receive an argument
		/// </summary>
		class ArgumentTarget
		{
			public MemberInfo Member { get; }
			public Type ValueType { get; }
			public Action<object?, object?> SetValue { get; }
			public Func<object?, object?> GetValue { get; }
			public CommandLineAttribute[] Attributes { get; }

			public ArgumentTarget(MemberInfo member, Type valueType, Action<object?, object?> setValue, Func<object?, object?> getValue, CommandLineAttribute[] attributes)
			{
				Member = member;
				ValueType = valueType;
				SetValue = setValue;
				GetValue = getValue;
				Attributes = attributes;
			}
		}

		/// <summary>
		/// The raw array of arguments
		/// </summary>
		readonly string[] _arguments;

		/// <summary>
		/// Bitmask indicating which arguments are flags rather than values
		/// </summary>
		readonly BitArray _flagArguments;

		/// <summary>
		/// Bitmask indicating which arguments have been used, via calls to GetOption(), GetValues() etc...
		/// </summary>
		readonly BitArray _usedArguments;

		/// <summary>
		/// Dictionary of argument names (or prefixes, in the case of "-Foo=" style arguments) to their index into the arguments array.
		/// </summary>
		readonly Dictionary<string, int> _argumentToFirstIndex;

		/// <summary>
		/// For each argument which is seen more than once, keeps a list of indices for the second and subsequent arguments.
		/// </summary>
		readonly int[] _nextArgumentIndex;

		/// <summary>
		/// List of positional arguments
		/// </summary>
		readonly List<int> _positionalArgumentIndices = [];

		/// <summary>
		/// Array of characters that separate argument names from values
		/// </summary>
		static readonly char[] s_valueSeparators = ['=', ':'];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="arguments">The raw list of arguments</param>
		public CommandLineArguments(string[] arguments)
		{
			_arguments = arguments;
			_flagArguments = new BitArray(arguments.Length);
			_usedArguments = new BitArray(arguments.Length);

			// Clear the linked list of identical arguments
			_nextArgumentIndex = new int[arguments.Length];
			for (int idx = 0; idx < arguments.Length; idx++)
			{
				_nextArgumentIndex[idx] = -1;
			}

			// Temporarily store the index of the last matching argument
			int[] lastArgumentIndex = new int[arguments.Length];

			// Parse the argument array and build a lookup
			_argumentToFirstIndex = new Dictionary<string, int>(arguments.Length, StringComparer.OrdinalIgnoreCase);
			for (int idx = 0; idx < arguments.Length; idx++)
			{
				if (arguments[idx].Equals("--", StringComparison.Ordinal))
				{
					// End of option arguments
					MarkAsUsed(idx++);
					for (; idx < arguments.Length; idx++)
					{
						_positionalArgumentIndices.Add(idx);
					}
					break;
				}
				else if (arguments[idx].StartsWith('-'))
				{
					// Option argument
					int separatorIdx = arguments[idx].IndexOfAny(s_valueSeparators);
					if (separatorIdx == -1)
					{
						// Ignore duplicate -Option flags; they are harmless.
						if (!_argumentToFirstIndex.TryAdd(arguments[idx], idx))
						{
							_usedArguments.Set(idx, true);
						}

						// Mark this argument as a flag
						_flagArguments.Set(idx, true);
					}
					else
					{
						// Just take the part up to and including the separator character
						string prefix = arguments[idx].Substring(0, separatorIdx + 1);

						// Add the prefix to the argument lookup, or update the appropriate matching argument list if it's been seen before
						int existingArgumentIndex;
						if (_argumentToFirstIndex.TryGetValue(prefix, out existingArgumentIndex))
						{
							_nextArgumentIndex[lastArgumentIndex[existingArgumentIndex]] = idx;
							lastArgumentIndex[existingArgumentIndex] = idx;
						}
						else
						{
							_argumentToFirstIndex.Add(prefix, idx);
							lastArgumentIndex[idx] = idx;
						}
					}
				}
				else
				{
					// Positional argument
					_positionalArgumentIndices.Add(idx);
				}
			}
		}

		/// <summary>
		/// The number of arguments in this list
		/// </summary>
		public int Count => _arguments.Length;

		/// <summary>
		/// Access an argument by index
		/// </summary>
		/// <param name="index">Index of the argument</param>
		/// <returns>The argument at the given index</returns>
		public string this[int index] => _arguments[index];

		/// <summary>
		/// Determines if an argument has been used
		/// </summary>
		/// <param name="index">Index of the argument</param>
		/// <returns>True if the argument has been used, false otherwise</returns>
		public bool HasBeenUsed(int index)
		{
			return _usedArguments.Get(index);
		}

		/// <summary>
		/// Marks an argument as having been used
		/// </summary>
		/// <param name="index">Index of the argument to mark as used</param>
		public void MarkAsUsed(int index)
		{
			_usedArguments.Set(index, true);
		}

		/// <summary>
		/// Marks an argument as not having been used
		/// </summary>
		/// <param name="index">Index of the argument to mark as being unused</param>
		public void MarkAsUnused(int index)
		{
			_usedArguments.Set(index, false);
		}

		/// <summary>
		/// Checks if the given option (eg. "-Foo") was specified on the command line.
		/// </summary>
		/// <param name="option">The option to look for</param>
		/// <returns>True if the option was found, false otherwise.</returns>
		public bool HasOption(string option)
		{
			int index;
			if (_argumentToFirstIndex.TryGetValue(option, out index))
			{
				_usedArguments.Set(index, true);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Checks for an argument prefixed with the given string is present.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>True if an argument with the given prefix was specified</returns>
		public bool HasValue(string prefix)
		{
			CheckValidPrefix(prefix);
			return _argumentToFirstIndex.ContainsKey(prefix);
		}

		/// <summary>
		/// Gets the positional argument at the given index
		/// </summary>
		/// <returns>Number of positional arguments</returns>
		public int GetPositionalArgumentCount()
		{
			return _positionalArgumentIndices.Count;
		}

		/// <summary>
		/// Gets the index of the numbered positional argument
		/// </summary>
		/// <param name="num">Number of the positional argument</param>
		/// <returns>Index of the positional argument</returns>
		public int GetPositionalArgumentIndex(int num)
		{
			return _positionalArgumentIndices[num];
		}

		/// <summary>
		/// Attempts to read the next unused positional argument
		/// </summary>
		/// <param name="argument">Receives the argument that was read, on success</param>
		/// <returns>True if an argument was read</returns>
		public bool TryGetPositionalArgument([NotNullWhen(true)] out string? argument)
		{
			for (int idx = 0; idx < _positionalArgumentIndices.Count; idx++)
			{
				int index = _positionalArgumentIndices[idx];
				if (!HasBeenUsed(index))
				{
					MarkAsUsed(index);
					argument = _arguments[index];
					return true;
				}
			}

			argument = null;
			return false;
		}

		/// <summary>
		/// Returns all the positional arguments, and marks them as used
		/// </summary>
		/// <returns>Array of positional arguments</returns>
		public string[] GetPositionalArguments()
		{
			string[] positionalArguments = new string[_positionalArgumentIndices.Count];
			for (int idx = 0; idx < positionalArguments.Length; idx++)
			{
				int index = _positionalArgumentIndices[idx];
				MarkAsUsed(index);
				positionalArguments[idx] = _arguments[index];
			}
			return positionalArguments;
		}

		/// <summary>
		/// Consume positional arguments from command line
		/// </summary>
		/// <param name="count">Number of arguments to remove</param>
		public void RemovePositionalArguments(int count)
		{
			_positionalArgumentIndices.RemoveRange(0, count);
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix. Throws an exception if the argument was not specified.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Value of the argument</returns>
		public string GetString(string prefix)
		{
			string? value;
			if (!TryGetValue(prefix, out value))
			{
				throw new CommandLineArgumentException(String.Format("Missing '{0}...' argument", prefix));
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix. Throws an exception if the argument was not specified.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Value of the argument</returns>
		public int GetInteger(string prefix)
		{
			int value;
			if (!TryGetValue(prefix, out value))
			{
				throw new CommandLineArgumentException(String.Format("Missing '{0}...' argument", prefix));
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix. Throws an exception if the argument was not specified.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Value of the argument</returns>
		public FileReference GetFileReference(string prefix)
		{
			FileReference? value;
			if (!TryGetValue(prefix, out value))
			{
				throw new CommandLineArgumentException(String.Format("Missing '{0}...' argument", prefix));
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix. Throws an exception if the argument was not specified.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Value of the argument</returns>
		public DirectoryReference GetDirectoryReference(string prefix)
		{
			DirectoryReference? value;
			if (!TryGetValue(prefix, out value))
			{
				throw new CommandLineArgumentException(String.Format("Missing '{0}...' argument", prefix));
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix. Throws an exception if the argument was not specified.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Value of the argument</returns>
		public T GetEnum<T>(string prefix) where T : struct
		{
			T value;
			if (!TryGetValue(prefix, out value))
			{
				throw new CommandLineArgumentException(String.Format("Missing '{0}...' argument", prefix));
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix, or a default value.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="defaultValue">Default value for the argument</param>
		/// <returns>Value of the argument</returns>
		[return: NotNullIfNotNull(nameof(defaultValue))]
		public string? GetStringOrDefault(string prefix, string? defaultValue)
		{
			string? value;
			if (!TryGetValue(prefix, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix, or a default value.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="defaultValue">Default value for the argument</param>
		/// <returns>Value of the argument</returns>
		public int GetIntegerOrDefault(string prefix, int defaultValue)
		{
			int value;
			if (!TryGetValue(prefix, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix, or a default value.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="defaultValue">Default value for the argument</param>
		/// <returns>Value of the argument</returns>
		[return: NotNullIfNotNull(nameof(defaultValue))]
		public FileReference? GetFileReferenceOrDefault(string prefix, FileReference? defaultValue)
		{
			FileReference? value;
			if (!TryGetValue(prefix, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix, or a default value.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="defaultValue">Default value for the argument</param>
		/// <returns>Value of the argument</returns>
		[return: NotNullIfNotNull(nameof(defaultValue))]
		public DirectoryReference? GetDirectoryReferenceOrDefault(string prefix, DirectoryReference? defaultValue)
		{
			DirectoryReference? value;
			if (!TryGetValue(prefix, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Gets the value specified by an argument with the given prefix, or a default value.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="defaultValue">Default value for the argument</param>
		/// <returns>Value of the argument</returns>
		public T GetEnumOrDefault<T>(string prefix, T defaultValue) where T : struct
		{
			T value;
			if (!TryGetValue(prefix, out value))
			{
				value = defaultValue;
			}
			return value;
		}

		/// <summary>
		/// Tries to gets the value specified by an argument with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="value">Value of the argument, if found</param>
		/// <returns>True if the argument was found (and Value was set), false otherwise.</returns>
		public bool TryGetValue(string prefix, [NotNullWhen(true)] out string? value)
		{
			CheckValidPrefix(prefix);

			int index;
			if (!_argumentToFirstIndex.TryGetValue(prefix, out index))
			{
				value = null;
				return false;
			}

			if (_nextArgumentIndex[index] != -1)
			{
				throw new CommandLineArgumentException(String.Format("Multiple {0}... arguments are specified", prefix));
			}

			_usedArguments.Set(index, true);
			value = _arguments[index].Substring(prefix.Length);
			return true;
		}

		/// <summary>
		/// Tries to gets the value specified by an argument with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="value">Value of the argument, if found</param>
		/// <returns>True if the argument was found (and Value was set), false otherwise.</returns>
		public bool TryGetValue(string prefix, out int value)
		{
			// Try to get the string value of this argument
			string? stringValue;
			if (!TryGetValue(prefix, out stringValue))
			{
				value = 0;
				return false;
			}

			// Try to parse it. If it fails, throw an exception.
			try
			{
				value = Int32.Parse(stringValue);
				return true;
			}
			catch (Exception ex)
			{
				throw new CommandLineArgumentException(String.Format("The argument '{0}{1}' does not specify a valid integer", prefix, stringValue), ex);
			}
		}

		/// <summary>
		/// Tries to gets the value specified by an argument with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="value">Value of the argument, if found</param>
		/// <returns>True if the argument was found (and Value was set), false otherwise.</returns>
		public bool TryGetValue(string prefix, [NotNullWhen(true)] out FileReference? value)
		{
			// Try to get the string value of this argument
			string? stringValue;
			if (!TryGetValue(prefix, out stringValue))
			{
				value = null;
				return false;
			}

			// Try to parse it. If it fails, throw an exception.
			try
			{
				value = new FileReference(stringValue);
				return true;
			}
			catch (Exception ex)
			{
				throw new CommandLineArgumentException(String.Format("The argument '{0}{1}' does not specify a valid file name", prefix, stringValue), ex);
			}
		}

		/// <summary>
		/// Tries to gets the value specified by an argument with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="value">Value of the argument, if found</param>
		/// <returns>True if the argument was found (and Value was set), false otherwise.</returns>
		public bool TryGetValue(string prefix, [NotNullWhen(true)] out DirectoryReference? value)
		{
			// Try to get the string value of this argument
			string? stringValue;
			if (!TryGetValue(prefix, out stringValue))
			{
				value = null;
				return false;
			}

			// Try to parse it. If it fails, throw an exception.
			try
			{
				value = new DirectoryReference(stringValue);
				return true;
			}
			catch (Exception ex)
			{
				throw new CommandLineArgumentException(String.Format("The argument '{0}{1}' does not specify a valid directory name", prefix, stringValue), ex);
			}
		}

		/// <summary>
		/// Tries to gets the value specified by an argument with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="value">Value of the argument, if found</param>
		/// <returns>True if the argument was found (and Value was set), false otherwise.</returns>
		public bool TryGetValue<T>(string prefix, out T value) where T : struct
		{
			// Try to get the string value of this argument
			string? stringValue;
			if (!TryGetValue(prefix, out stringValue))
			{
				value = new T();
				return false;
			}

			// Try to parse it. If it fails, throw an exception.
			try
			{
				value = (T)Enum.Parse(typeof(T), stringValue, true);
				return true;
			}
			catch (Exception ex)
			{
				throw new CommandLineArgumentException(String.Format("The argument '{0}{1}' does not specify a valid {2}", prefix, stringValue, typeof(T).Name), ex);
			}
		}

		/// <summary>
		/// Returns all arguments with the given prefix.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <returns>Sequence of values for the given prefix.</returns>
		public IEnumerable<string> GetValues(string prefix)
		{
			CheckValidPrefix(prefix);

			int index;
			if (_argumentToFirstIndex.TryGetValue(prefix, out index))
			{
				for (; index != -1; index = _nextArgumentIndex[index])
				{
					_usedArguments.Set(index, true);
					yield return _arguments[index].Substring(prefix.Length);
				}
			}
		}

		/// <summary>
		/// Returns all arguments with the given prefix, allowing multiple arguments to be specified in a single argument with a separator character.
		/// </summary>
		/// <param name="prefix">The argument prefix (eg. "-Foo="). Must end with an '=' character.</param>
		/// <param name="separator">The separator character (eg. '+')</param>
		/// <returns>Sequence of values for the given prefix.</returns>
		public IEnumerable<string> GetValues(string prefix, char separator)
		{
			foreach (string value in GetValues(prefix))
			{
				foreach (string splitValue in value.Split(separator))
				{
					yield return splitValue;
				}
			}
		}

		/// <summary>
		/// Gets the prefix for a particular argument
		/// </summary>
		/// <param name="target">The target hosting the attribute</param>
		/// <param name="attribute">The attribute instance</param>
		/// <returns>Prefix for this argument</returns>
		private static string GetArgumentPrefix(ArgumentTarget target, CommandLineAttribute attribute)
		{
			// Get the inner field type, unwrapping nullable types
			Type valueType = target.ValueType;
			if (valueType.IsGenericType && valueType.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				valueType = valueType.GetGenericArguments()[0];
			}

			string? prefix = attribute.Prefix;
			if (prefix == null)
			{
				if (valueType == typeof(bool))
				{
					prefix = String.Format("-{0}", target.Member.Name);
				}
				else
				{
					prefix = String.Format("-{0}=", target.Member.Name);
				}
			}
			else
			{
				if (valueType != typeof(bool) && attribute.Value == null && !prefix.EndsWith('=') && !prefix.EndsWith(':'))
				{
					prefix += "=";
				}
			}
			return prefix;
		}

		/// <summary>
		/// Applies these arguments to fields with the [CommandLine] attribute in the given object.
		/// </summary>
		/// <param name="targetObject">The object to configure</param>
		public void ApplyTo(object targetObject)
		{
			ApplyTo(targetObject, Log.Logger);
		}

		/// <summary>
		/// Applies these arguments to fields with the [CommandLine] attribute in the given object.
		/// </summary>
		/// <param name="targetObject">The object to configure</param>
		/// <param name="logger">Sink for error/warning messages</param>
		public void ApplyTo(object targetObject, ILogger logger)
		{
			List<string> missingArguments = [];

			// Build a mapping from name to field and attribute for this object
			List<ArgumentTarget> targets = GetArgumentTargetsForType(targetObject.GetType());
			foreach (ArgumentTarget target in targets)
			{
				// If any attribute is required, keep track of it so we can include an error for it
				string? requiredPrefix = null;

				// Keep track of whether a value has already been assigned to this field
				string? assignedArgument = null;

				// Loop through all the attributes for different command line options that can modify it
				foreach (CommandLineAttribute attribute in target.Attributes)
				{
					// Add in any positional arguments
					if (attribute.Positional)
					{
						foreach (int argIdx in _positionalArgumentIndices)
						{
							string arg = _arguments[argIdx];
							if (ApplyArgument(targetObject, target, arg, arg, assignedArgument, logger))
							{
								MarkAsUsed(argIdx);
								assignedArgument = arg;
							}
						}
					}

					// Get the appropriate prefix for this attribute
					string prefix = GetArgumentPrefix(target, attribute);

					// Get the value with the correct prefix
					int firstIndex;
					if (_argumentToFirstIndex.TryGetValue(prefix, out firstIndex))
					{
						for (int index = firstIndex; index != -1; index = _nextArgumentIndex[index])
						{
							// Get the argument text
							string argument = _arguments[index];

							// Get the text for this value
							string valueText;
							if (attribute.Value != null)
							{
								valueText = attribute.Value;
							}
							else if (_flagArguments.Get(index))
							{
								valueText = "true";
							}
							else
							{
								valueText = argument.Substring(prefix.Length);
							}

							// Apply the value to the field
							if (attribute.ListSeparator == 0)
							{
								if (ApplyArgument(targetObject, target, argument, valueText, assignedArgument, logger))
								{
									assignedArgument = argument;
								}
							}
							else
							{
								foreach (string itemValueText in valueText.Split(attribute.ListSeparator))
								{
									if (ApplyArgument(targetObject, target, argument, itemValueText, assignedArgument, logger))
									{
										assignedArgument = argument;
									}
								}
							}

							// Mark this argument as used
							if (attribute.MarkUsed)
							{
								_usedArguments.Set(index, true);
							}
						}
					}

					// If this attribute is marked as required, keep track of it so we can warn if the field is not assigned to
					if (attribute.Required && requiredPrefix == null)
					{
						requiredPrefix = prefix;
					}
				}

				// Make sure that this field has been assigned to
				if (assignedArgument == null && requiredPrefix != null)
				{
					missingArguments.Add(requiredPrefix);
				}
			}

			// If any arguments were missing, print an error about them
			if (missingArguments.Count > 0)
			{
				if (missingArguments.Count == 1)
				{
					throw new CommandLineArgumentException(String.Format("Missing {0} argument", missingArguments[0].Replace("=", "=...", StringComparison.Ordinal)));
				}
				else
				{
					throw new CommandLineArgumentException(String.Format("Missing {0} arguments", StringUtils.FormatList(missingArguments.Select(x => x.Replace("=", "=...", StringComparison.Ordinal)))));
				}
			}
		}

		/// <summary>
		/// Applies these arguments to fields with the [CommandLine] attribute in the given object.
		/// </summary>
		/// <param name="logger">Sink for error/warning messages</param>
		public T ApplyTo<T>(ILogger logger) where T : new()
		{
			T obj = new();
			ApplyTo(obj, logger);
			return obj;
		}

		private static readonly ConcurrentDictionary<Type, Lazy<List<ArgumentTarget>>> s_cachedArgumentTargetsForType = [];
		static List<ArgumentTarget> GetArgumentTargetsForType(Type? targetType)
		{
			if (targetType == null || targetType == typeof(object))
			{
				return [];
			}

			return s_cachedArgumentTargetsForType.GetOrAdd(targetType, static targetType => new Lazy<List<ArgumentTarget>>(() =>
			{
				List<ArgumentTarget> targets = [];
				foreach (FieldInfo fieldInfo in targetType.GetFields(BindingFlags.Instance | BindingFlags.GetField | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly))
				{
					IEnumerable<CommandLineAttribute> attributes = fieldInfo.GetCustomAttributes<CommandLineAttribute>();
					if (attributes.Any())
					{
						targets.Add(new ArgumentTarget(fieldInfo, fieldInfo.FieldType, fieldInfo.SetValue, fieldInfo.GetValue, [.. attributes]));
					}
				}
				foreach (PropertyInfo propertyInfo in targetType.GetProperties(BindingFlags.Instance | BindingFlags.GetProperty | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly))
				{
					IEnumerable<CommandLineAttribute> attributes = propertyInfo.GetCustomAttributes<CommandLineAttribute>();
					if (attributes.Any())
					{
						targets.Add(new ArgumentTarget(propertyInfo, propertyInfo.PropertyType, propertyInfo.SetValue, propertyInfo.GetValue, [.. attributes]));
					}
				}

				List<ArgumentTarget> baseTargets = GetArgumentTargetsForType(targetType.BaseType);
				if (baseTargets.Count > 0)
				{
					if (targets.Count > 0)
					{
						targets.AddRange(baseTargets);
					}
					else
					{
						targets = baseTargets;
					}
				}
				return targets;
			})).Value;
		}

		/// <summary>
		/// Gets help text for the arguments of a given type
		/// </summary>
		/// <param name="type">The type to find parameters for</param>
		/// <returns>List of parameters</returns>
		public static List<KeyValuePair<string, string>> GetParameters(Type type)
		{
			List<KeyValuePair<string, string>> parameters = [];

			List<ArgumentTarget> targets = GetArgumentTargetsForType(type);
			foreach (ArgumentTarget target in targets)
			{
				StringBuilder descriptionBuilder = new();
				foreach (DescriptionAttribute attribute in target.Member.GetCustomAttributes<DescriptionAttribute>())
				{
					if (descriptionBuilder.Length > 0)
					{
						descriptionBuilder.Append('\n');
					}
					descriptionBuilder.Append(attribute.Description);
				}
				foreach (CommandLineAttribute attribute in target.Member.GetCustomAttributes<CommandLineAttribute>().Where(x => x.Description != null))
				{
					if (descriptionBuilder.Length > 0)
					{
						descriptionBuilder.Append('\n');
					}
					descriptionBuilder.Append(attribute.Description);
				}

				string description = descriptionBuilder.ToString();
				if (description.Length == 0)
				{
					description = "No description available.";
				}

				foreach (CommandLineAttribute attribute in target.Attributes)
				{
					string prefix = GetArgumentPrefix(target, attribute);
					if (prefix.EndsWith('='))
					{
						prefix += "...";
					}
					parameters.Add(new KeyValuePair<string, string>(prefix, description));
				}
			}
			return parameters;
		}

		/// <summary>
		/// Quotes a command line argument, if necessary
		/// </summary>
		/// <param name="argument">The argument that may need quoting</param>
		/// <returns>Argument which is safe to pass on the command line</returns>
		public static string Quote(string argument)
		{
			// See if the entire string is quoted correctly
			bool bInQuotes = false;
			for (int idx = 0; ; idx++)
			{
				if (idx == argument.Length)
				{
					return argument;
				}
				else if (argument[idx] == '\"')
				{
					bInQuotes ^= true;
				}
				else if (argument[idx] == ' ')
				{
					break;
				}
			}

			// Try to insert a quote after the argument string
			if (argument[0] == '-')
			{
				for (int idx = 1; idx < argument.Length && argument[idx] != ' '; idx++)
				{
					if (argument[idx] == '=')
					{
						return String.Format("{0}=\"{1}\"", argument.Substring(0, idx), argument.Substring(idx + 1).Replace("\"", "\\\"", StringComparison.Ordinal));
					}
				}
			}

			// Quote the whole thing
			return "\"" + argument.Replace("\"", "\\\"", StringComparison.Ordinal) + "\"";
		}

		/// <summary>
		/// Joins the given arguments into a command line
		/// </summary>
		/// <param name="arguments">List of command line arguments</param>
		/// <returns>Joined command line</returns>
		public static string Join(IEnumerable<string> arguments)
		{
			StringBuilder result = new();
			foreach (string argument in arguments)
			{
				if (result.Length > 0)
				{
					result.Append(' ');
				}
				result.Append(Quote(argument));
			}
			return result.ToString();
		}

		/// <summary>
		/// Splits a command line into individual arguments
		/// </summary>
		/// <param name="commandLine">The command line text</param>
		/// <returns>Array of arguments</returns>
		public static string[] Split(string commandLine)
		{
			StringBuilder argument = new();

			List<string> arguments = [];
			// First do a pass leaving all quotes in the arguments, they will be removed later
			for (int idx = 0; idx < commandLine.Length; idx++)
			{
				if (!Char.IsWhiteSpace(commandLine[idx]))
				{
					argument.Clear();
					for (bool bInQuotes = false; idx < commandLine.Length; idx++)
					{
						if (commandLine[idx] == '\"')
						{
							bInQuotes ^= true;
						}
						else if (!bInQuotes && Char.IsWhiteSpace(commandLine[idx]))
						{
							break;
						}
						argument.Append(commandLine[idx]);
					}
					arguments.Add(argument.ToString());
				}
			}

			// Remove quotes from arguments except where only the value is quoted in -Define (-Define:KEY="VALUE")
			for (int idx = 0; idx < arguments.Count; idx++)
			{
				string arg = arguments[idx];
				if (arg.StartsWith("-Define:", StringComparison.OrdinalIgnoreCase) && !arg.StartsWith("-Define:\"", StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}
				arguments[idx] = arg.Replace("\"", "", StringComparison.OrdinalIgnoreCase);
			}
			return [.. arguments];
		}

		/// <summary>
		/// Appends the given arguments to the current argument list
		/// </summary>
		/// <param name="appendArguments">The arguments to add</param>
		/// <returns>New argument list</returns>
		public CommandLineArguments Append(IEnumerable<string> appendArguments)
		{
			CommandLineArguments newArguments = new([.. Enumerable.Concat(_arguments, appendArguments)]);
			for (int idx = 0; idx < _arguments.Length; idx++)
			{
				if (HasBeenUsed(idx))
				{
					newArguments.MarkAsUsed(idx);
				}
			}
			return newArguments;
		}

		/// <summary>
		/// Retrieves all arguments with the given prefix, and returns the remaining a list of strings
		/// </summary>
		/// <param name="prefix">Prefix for the arguments to remove</param>
		/// <param name="values">Receives a list of values with the given prefix</param>
		/// <returns>New argument list</returns>
		public CommandLineArguments Remove(string prefix, out List<string> values)
		{
			values = [];

			// Split the arguments into the values array and an array of new arguments
			int[] newArgumentIndex = new int[_arguments.Length];
			List<string> newArgumentList = new(_arguments.Length);
			for (int idx = 0; idx < _arguments.Length; idx++)
			{
				string argument = _arguments[idx];
				if (argument.StartsWith(prefix, StringComparison.OrdinalIgnoreCase))
				{
					newArgumentIndex[idx] = -1;
					values.Add(argument.Substring(prefix.Length));
				}
				else
				{
					newArgumentIndex[idx] = newArgumentList.Count;
					newArgumentList.Add(argument);
				}
			}

			// Create the new argument list, and mark the same arguments as used
			CommandLineArguments newArguments = new([.. newArgumentList]);
			for (int idx = 0; idx < _arguments.Length; idx++)
			{
				if (HasBeenUsed(idx) && newArgumentIndex[idx] != -1)
				{
					newArguments.MarkAsUsed(newArgumentIndex[idx]);
				}
			}
			return newArguments;
		}

		/// <summary>
		/// Checks that there are no unused arguments (and warns if there are)
		/// </summary>
		public void CheckAllArgumentsUsed()
		{
			CheckAllArgumentsUsed(Log.Logger);
		}

		/// <summary>
		/// Checks that there are no unused arguments (and warns if there are)
		/// </summary>
		public void CheckAllArgumentsUsed(ILogger logger)
		{
			// Find all the unused arguments
			List<string> remainingArguments = [];
			for (int idx = 0; idx < _arguments.Length; idx++)
			{
				if (!_usedArguments[idx])
				{
					remainingArguments.Add(_arguments[idx]);
				}
			}

			// Output a warning
			if (remainingArguments.Count > 0)
			{
				if (remainingArguments.Count == 1)
				{
					logger.LogWarning("Invalid argument: {Argument}", remainingArguments[0]);
				}
				else
				{
					logger.LogWarning("Invalid arguments:\n{Arguments}", String.Join("\n", remainingArguments));
				}
			}
		}

		/// <summary>
		/// Checks to see if any arguments are used
		/// </summary>
		/// <returns></returns>
		public bool AreAnyArgumentsUsed()
		{
			return _usedArguments.Cast<bool>().Any(b => b);
		}

		/// <summary>
		/// Checks to see if any arguments are used
		/// </summary>
		/// <returns></returns>
		public IEnumerable<string> GetUnusedArguments()
		{
			for (int idx = 0; idx < _arguments.Length; idx++)
			{
				if (!_usedArguments[idx])
				{
					yield return _arguments[idx];
				}
			}
		}

		/// <summary>
		/// Count the number of value (non-flag) arguments on the command line
		/// </summary>
		/// <returns></returns>
		public int CountValueArguments()
		{
			return _flagArguments.Cast<bool>().Count(b => !b);
		}

		/// <summary>
		/// Checks that a given string is a valid argument prefix
		/// </summary>
		/// <param name="prefix">The prefix to check</param>
		private static void CheckValidPrefix(string prefix)
		{
			if (prefix.Length == 0)
			{
				throw new ArgumentException("Argument prefix cannot be empty.");
			}
			else if (prefix[0] != '-')
			{
				throw new ArgumentException("Argument prefix must begin with a hyphen.");
			}
			else if (!s_valueSeparators.Contains(prefix[^1]))
			{
				throw new ArgumentException(String.Format("Argument prefix must end with '{0}'", String.Join("' or '", s_valueSeparators)));
			}
		}

		/// <summary>
		/// Parses and assigns a value to a field
		/// </summary>
		/// <param name="targetObject">The target object to assign values to</param>
		/// <param name="target">The target to assign the value to</param>
		/// <param name="argumentText">The full argument text</param>
		/// <param name="valueText">Argument text</param>
		/// <param name="previousArgumentText">The previous text used to configure this field</param>
		/// <param name="logger">Logger for error/warning messages</param>
		/// <returns>True if the value was assigned to the field, false otherwise</returns>
		private static bool ApplyArgument(object targetObject, ArgumentTarget target, string argumentText, string valueText, string? previousArgumentText, ILogger logger)
		{
			Type valueType = target.ValueType;

			// Check if the field type implements ICollection<>. If so, we can take multiple values.
			Type? collectionType = null;
			foreach (Type interfaceType in valueType.GetInterfaces())
			{
				if (interfaceType.IsGenericType && interfaceType.GetGenericTypeDefinition() == typeof(ICollection<>))
				{
					valueType = interfaceType.GetGenericArguments()[0];
					collectionType = interfaceType;
					break;
				}
			}

			// Try to parse the value
			object? value;
			if (!TryParseValue(valueType, valueText, out value))
			{
				logger.LogWarning("Unable to parse value for argument '{Argument}'.", argumentText);
				return false;
			}

			// Try to assign values to the target field
			if (collectionType == null)
			{
				// Check if this field has already been assigned to. Output a warning if the previous value is in conflict with the new one.
				if (previousArgumentText != null)
				{
					object? previousValue = target.GetValue(targetObject);
					if (!Equals(previousValue, value))
					{
						logger.LogWarning("Argument '{Argument}' conflicts with '{PrevArgument}'; ignoring.", argumentText, previousArgumentText);
					}
					return false;
				}

				// Set the value on the target object
				target.SetValue(targetObject, value);
				return true;
			}
			else
			{
				// Call the 'Add' method on the collection
				collectionType.InvokeMember("Add", BindingFlags.InvokeMethod, null, target.GetValue(targetObject), [value]);
				return true;
			}
		}

		/// <summary>
		/// Attempts to parse the given string to a value
		/// </summary>
		/// <param name="fieldType">Type of the field to convert to</param>
		/// <param name="text">The value text</param>
		/// <param name="value">On success, contains the parsed object</param>
		/// <returns>True if the text could be parsed, false otherwise</returns>
		private static bool TryParseValue(Type fieldType, string text, [NotNullWhen(true)] out object? value)
		{
			if (fieldType.IsGenericType && fieldType.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				// Try to parse the inner type instead
				return TryParseValue(fieldType.GetGenericArguments()[0], text, out value);
			}
			else if (fieldType.IsEnum)
			{
				// Special handling for enums; parse the value ignoring case.
				try
				{
					value = Enum.Parse(fieldType, text, true);
					return true;
				}
				catch (ArgumentException)
				{
					value = null;
					return false;
				}
			}
			else if (fieldType == typeof(FileReference))
			{
				// Construct a file reference from the string
				try
				{
					value = new FileReference(text);
					return true;
				}
				catch
				{
					value = null;
					return false;
				}
			}
			else if (fieldType == typeof(DirectoryReference))
			{
				// Construct a file reference from the string
				try
				{
					value = new DirectoryReference(text);
					return true;
				}
				catch
				{
					value = null;
					return false;
				}
			}
			else if (fieldType == typeof(TimeSpan))
			{
				// Construct a time span form the string
				double floatValue;
				if (text.EndsWith("h", StringComparison.OrdinalIgnoreCase) && Double.TryParse(text.Substring(0, text.Length - 1), out floatValue))
				{
					value = TimeSpan.FromHours(floatValue);
					return true;
				}
				else if (text.EndsWith("m", StringComparison.OrdinalIgnoreCase) && Double.TryParse(text.Substring(0, text.Length - 1), out floatValue))
				{
					value = TimeSpan.FromMinutes(floatValue);
					return true;
				}
				else if (text.EndsWith("s", StringComparison.OrdinalIgnoreCase) && Double.TryParse(text.Substring(0, text.Length - 1), out floatValue))
				{
					value = TimeSpan.FromSeconds(floatValue);
					return true;
				}

				TimeSpan timeSpanValue;
				if (TimeSpan.TryParse(text, out timeSpanValue))
				{
					value = timeSpanValue;
					return true;
				}
				else
				{
					value = null;
					return false;
				}
			}
			else
			{
				// First check for a TypeConverter
				TypeConverter typeConverter = TypeDescriptor.GetConverter(fieldType);
				if (typeConverter.CanConvertFrom(typeof(string)))
				{
					value = typeConverter.ConvertFrom(text)!;
					return true;
				}

				// Otherwise let the framework convert between types
				try
				{
					value = Convert.ChangeType(text, fieldType);
					return true;
				}
				catch (InvalidCastException)
				{
					value = null;
					return false;
				}
			}
		}

		/// <summary>
		/// Obtains an enumerator for the argument list
		/// </summary>
		/// <returns>IEnumerator interface</returns>
		IEnumerator IEnumerable.GetEnumerator()
		{
			return _arguments.GetEnumerator();
		}

		/// <summary>
		/// Obtains an enumerator for the argument list
		/// </summary>
		/// <returns>Generic IEnumerator interface</returns>
		public IEnumerator<string> GetEnumerator()
		{
			return ((IEnumerable<string>)_arguments).GetEnumerator();
		}

		/// <summary>
		/// Gets the raw argument array
		/// </summary>
		/// <returns>Array of arguments</returns>
		public string[] GetRawArray()
		{
			return _arguments;
		}

		/// <summary>
		/// Takes a command line argument and adds quotes if necessary
		/// </summary>
		/// <param name="commandLine"></param>
		/// <param name="argument">The command line argument</param>
		/// <returns>The command line argument with quotes inserted to escape it if necessary</returns>
		public static void Append(StringBuilder commandLine, string argument)
		{
			if (commandLine.Length > 0)
			{
				commandLine.Append(' ');
			}

			int spaceIdx = argument.IndexOf(' ', StringComparison.Ordinal);
			if (spaceIdx == -1)
			{
				commandLine.Append(argument);
			}
			else
			{
				int equalsIdx = argument.IndexOf('=', StringComparison.Ordinal);
				if (equalsIdx == -1 || equalsIdx > spaceIdx)
				{
					commandLine.AppendFormat("\"{0}\"", argument);
				}
				else
				{
					commandLine.AppendFormat("{0}\"{1}\"", argument.Substring(0, equalsIdx + 1), argument.Substring(equalsIdx + 1));
				}
			}
		}

		/// <summary>
		/// Converts this string to 
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			StringBuilder result = new();
			foreach (string argument in _arguments)
			{
				Append(result, argument);
			}
			return result.ToString();
		}
	}

	/// <summary>
	/// Serializes command-line arguments to and from an array of strings.
	/// </summary>
	public class CommandLineArgumentsJsonConverter : JsonConverter<CommandLineArguments>
	{
		/// <inheritdoc />
		public override CommandLineArguments? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			List<string> arguments = [];
			if (reader.TokenType != JsonTokenType.StartArray)
			{
				throw new JsonException();
			}
			while (reader.Read())
			{
				if (reader.TokenType == JsonTokenType.EndArray)
				{
					return new CommandLineArguments([.. arguments]);
				}

				if (reader.TokenType != JsonTokenType.String)
				{
					throw new JsonException();
				}

				string argument = reader.GetString()!;
				arguments.Add(argument);
			}

			throw new JsonException();
		}

		/// <inheritdoc />
		public override void Write(Utf8JsonWriter writer, CommandLineArguments value, JsonSerializerOptions options)
		{
			writer.WriteStartArray();
			for (int i = 0; i < value.Count; ++i)
			{
				writer.WriteStringValue(value[i]);
			}
			writer.WriteEndArray();
		}
	}
}
