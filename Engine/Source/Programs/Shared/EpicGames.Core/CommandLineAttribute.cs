// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Attribute to indicate the name of a command line argument
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	public sealed class CommandLineAttribute : Attribute
	{
		/// <summary>
		/// Prefix for the option, with a leading '-' and trailing '=' character if a value is expected.
		/// </summary>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public string? Prefix { get; set; }
#pragma warning restore CA1019 // Define accessors for attribute arguments

		/// <summary>
		/// Specifies a fixed value for this argument. Specifying an alternate value is not permitted.
		/// </summary>
		public string? Value { get; set; } = null;

		/// <summary>
		/// Whether this argument is required
		/// </summary>
		public bool Required { get; set; }

		/// <summary>
		/// Whether this argument is required
		/// </summary>
		public bool MarkUsed { get; set; } = true;

		/// <summary>
		/// Accepts positional arguments (ie. without a preceding hyphen)
		/// </summary>
		public bool Positional { get; set; }

		/// <summary>
		/// For collection types, specifies the separator character between multiple arguments
		/// </summary>
		public char ListSeparator { get; set; } = '\0';

		/// <summary>
		/// Description of the operation.
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="prefix">Prefix for this argument</param>
		public CommandLineAttribute(string? prefix = null)
		{
			Prefix = prefix;

			if (prefix?.StartsWith('-') == false)
			{
				throw new Exception($"Command-line argument '{Prefix}' must begin with a '-' character");
			}
		}
	}
}
