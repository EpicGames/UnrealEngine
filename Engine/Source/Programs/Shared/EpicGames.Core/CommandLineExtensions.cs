// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Extensions to StringBuilder for building command lines
	/// </summary>
	public static class CommandLineExtensions
	{
		/// <summary>
		/// Determines if the given argument needs to be escaped
		/// </summary>
		/// <param name="argument">The argument to check</param>
		/// <returns>True if the argument needs to be escaped</returns>
		static bool NeedsEscaping(string argument)
		{
			return argument.Contains(' ', StringComparison.Ordinal) || argument.Contains('\"', StringComparison.Ordinal);
		}

		/// <summary>
		/// Appends command line argument with a prefixed space. The argument may contain spaces or quotes.
		/// </summary>
		/// <param name="builder">The command line to append to</param>
		/// <param name="argument">The argument to append</param>
		public static void AppendArgument(this StringBuilder builder, string argument)
		{
			if (builder.Length > 0)
			{
				builder.Append(' ');
			}

			int equalsIdx = argument.IndexOf('=', StringComparison.Ordinal);
			if (equalsIdx != -1)
			{
				string name = argument.Substring(0, equalsIdx + 1);
				if (!NeedsEscaping(name))
				{
					builder.Append(name);
					argument = argument.Substring(equalsIdx + 1);
				}
			}

			AppendCommandLineArgumentWithoutSpace(builder, argument);
		}

		/// <summary>
		/// Appends command line argument with a prefixed space. The argument may contain spaces or quotes.
		/// </summary>
		/// <param name="builder">The command line to append to</param>
		/// <param name="name">Name of the argument (eg. -Foo=)</param>
		/// <param name="value">Value of the argument</param>
		public static void AppendArgument(this StringBuilder builder, string name, string value)
		{
			if (builder.Length > 0)
			{
				builder.Append(' ');
			}

			if (NeedsEscaping(name))
			{
				AppendCommandLineArgumentWithoutSpace(builder, name + value);
			}
			else
			{
				builder.Append(name);
				AppendCommandLineArgumentWithoutSpace(builder, value);
			}
		}

		/// <summary>
		/// Appends an escaped command line argument. The argument may contain spaces or quotes, and is escaped according to the rules in
		/// https://docs.microsoft.com/en-us/windows/win32/api/shellapi/nf-shellapi-commandlinetoargvw.
		/// </summary>
		/// <param name="builder">The builder to append to</param>
		/// <param name="argument">The argument to escape</param>
		public static void AppendCommandLineArgumentWithoutSpace(this StringBuilder builder, string argument)
		{
			if (!NeedsEscaping(argument))
			{
				// No escaping necessary if the argument doesn't contain any special characters
				builder.Append(argument);
			}
			else
			{
				// Escape the whole string following the rules on the CommandLineToArgV MSDN page. 
				builder.Append('\"');
				for (int idx = 0; idx < argument.Length; idx++)
				{
					char character = argument[idx];
					if (character == '\"')
					{
						// Escape a single quotation mark
						builder.Append("\\\"");
					}
					else if (character == '\\')
					{
						// Special handling for slashes which may be followed by a quotation mark, as dictated by CommandLineToArgV
						int startIdx = idx;
						for (; ; )
						{
							int nextIdx = idx + 1;
							if (nextIdx == argument.Length)
							{
								// Will have a trailing quotation mark toggling 'in quotes' mode (2n)
								builder.Append('\\', (nextIdx - startIdx) * 2);
								break;
							}
							else if (argument[nextIdx] == '\"')
							{
								// Needs to have a trailing quotation mark, so need to escape each backslash plus the quotation mark (2n+1)
								builder.Append('\\', (nextIdx - startIdx) * 2 + 1);
								break;
							}
							else if (argument[nextIdx] != '\\')
							{
								// No trailing quote; can just pass through verbatim
								builder.Append('\\', (nextIdx - startIdx));
								break;
							}
							idx = nextIdx;
						}
					}
					else
					{
						// Regular character
						builder.Append(character);
					}
				}
				builder.Append('\"');
			}
		}
	}
}
