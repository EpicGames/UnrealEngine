// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Builds a condensed graph for a directed graph using Kosaraju-Sharir's algorithm.
	/// Each strongly connected component (SCC) in the input is collapsed into a single node,
	/// producing a directed acyclic graph (DAG). Vertices that are not part of any cycle form
	/// their own single-element SCCs.
	///
	/// Build() returns false if the input graph is already a DAG (no cycles found), or true if
	/// cycles were detected and the condensed differs from the input. In both cases GetSCCs()
	/// and GetNodeToSCCIndex() are fully populated. GetCondensedEdges() is only populated when
	/// Build() returns true (matching the C++ TryConstructcondensedGraph contract).
	///
	/// The output SCCs are ordered so that every edge in the condensed graph points from a
	/// lower SCC index to a higher one (root-to-leaf topological order).
	/// </summary>
	/// <typeparam name="T">Type of node objects</typeparam>
	public class CondensedGraphBuilder<T> where T : notnull
	{
		/// <summary>
		/// Functor returning a node's name. Used for diagnostic logging.
		/// </summary>
		private Func<T, string>? _nodeToString = null;

		/// <summary>
		/// List of all graph nodes in insertion order.
		/// </summary>
		private readonly List<GraphNode> _nodes = [];

		/// <summary>
		/// Maps each user-provided node to its internal GraphNode.
		/// </summary>
		private readonly Dictionary<T, GraphNode> _nodeMap = [];

		/// <summary>
		/// Output: list of strongly connected components. Each SCC is an array of original nodes.
		/// Ordered root-to-leaf: condensed edges always point from lower to higher SCC index.
		/// Always populated after Build().
		/// </summary>
		private readonly List<T[]> _resultSCCs = [];

		/// <summary>
		/// Output: maps each original node to the index of its SCC in ResultSCCs.
		/// Always populated after Build().
		/// </summary>
		private readonly Dictionary<T, int> _resultNodeToSCCIndex = [];

		/// <summary>
		/// Output: adjacency list of the condensed graph. ResultCondensedEdges[i] holds the
		/// SCC indices that SCC i has outgoing edges to. Only populated when Build() returns true.
		/// </summary>
		private readonly List<int[]> _resultCondensedEdges = [];

		/// <summary>
		/// Function to go from a node to the string output. Can be used for logging.
		/// </summary>
		public Func<T, string>? NodeToString { get => _nodeToString; set => _nodeToString = value; }

		/// <summary>
		/// Initializes the builder and constructs the internal graph from the provided edges.
		/// All nodes referenced by edges (both source and destination) are included.
		/// Self-edges are discarded.
		/// </summary>
		/// <param name="edges">Directed edges as (source, destination) tuples</param>
		public CondensedGraphBuilder(List<Tuple<T, T>> edges)
		{
			CreateGraph(edges);
		}

		/// <summary>
		/// Runs Kosaraju-Sharir's algorithm to compute the condensed graph.
		/// </summary>
		/// <returns>
		/// True if cycles were found and the condensed differs from the input graph.
		/// False if the input graph is already a DAG.
		/// </returns>
		public bool Build()
		{
			_resultSCCs.Clear();
			_resultNodeToSCCIndex.Clear();
			_resultCondensedEdges.Clear();

			int numVertices = _nodes.Count;
			if (numVertices == 0)
			{
				return false;
			}

			// Phase 1 — Iterative DFS on the original graph.
			// Records each vertex in post-order (after all outgoing edges are exhausted).
			// Any back-edge to an InProgress vertex (other than a self-edge) indicates a cycle.
			List<int> postOrder = new(numVertices);
			bool hasCycle = false;

			{
				VisitStatus[] status = new VisitStatus[numVertices]; // default: NotVisited
				List<(int Vertex, int NextEdge)> stack = new(numVertices);

				for (int root = 0; root < numVertices; ++root)
				{
					if (status[root] != VisitStatus.NotVisited)
					{
						continue;
					}
					status[root] = VisitStatus.InProgress;
					stack.Add((root, 0));

					while (stack.Count > 0)
					{
						(int vertex, int nextEdge) = stack[stack.Count - 1];
						List<GraphNode> edges = _nodes[vertex]._links;
						bool pushed = false;

						while (nextEdge < edges.Count)
						{
							int edgeVertex = edges[nextEdge]._index;
							++nextEdge;

							if (status[edgeVertex] == VisitStatus.Visited)
							{
								// Already fully processed; nothing to do.
							}
							else if (status[edgeVertex] == VisitStatus.NotVisited)
							{
								status[edgeVertex] = VisitStatus.InProgress;
								stack[stack.Count - 1] = (vertex, nextEdge);
								stack.Add((edgeVertex, 0));
								pushed = true;
								break;
							}
							else // InProgress — back edge detected
							{
								// Self-edges are not counted as cycles; they are discarded during CreateGraph.
								if (edgeVertex != vertex)
								{
									hasCycle = true;
								}
							}
						}

						if (!pushed)
						{
							postOrder.Add(vertex);
							status[vertex] = VisitStatus.Visited;
							stack.RemoveAt(stack.Count - 1);
						}
					}
				}
			}

			// No cycles found: input is already a DAG.
			// Populate SCCs as an identity mapping in reverse post-order (root-to-leaf).
			// ResultCondensedEdges is left empty per the C++ TryConstructcondensedGraph contract.
			if (!hasCycle)
			{
				_resultSCCs.Capacity = numVertices;
				_resultNodeToSCCIndex.EnsureCapacity(numVertices);

				for (int sCCIndex = 0; sCCIndex < numVertices; ++sCCIndex)
				{
					T nodeData = _nodes[postOrder[numVertices - 1 - sCCIndex]]._data;
					_resultSCCs.Add([nodeData]);
					_resultNodeToSCCIndex[nodeData] = sCCIndex;
				}

				return false;
			}

			// Phase 2 — Iterative DFS on the transpose graph in reverse post-order.
			// Each unassigned vertex seeds a new SCC. All vertices reachable via TransposeLinks
			// (i.e., all vertices that can reach the seed in the original graph) join the same SCC.
			// Processing in reverse post-order ensures SCCs are discovered root-to-leaf.
			{
				bool[] assigned = new bool[numVertices];
				List<(int Vertex, int NextEdge)> stack = new(numVertices);

				for (int i = numVertices - 1; i >= 0; --i)
				{
					int root = postOrder[i];
					if (assigned[root])
					{
						continue;
					}

					List<int> sCCVertices = [];
					assigned[root] = true;
					sCCVertices.Add(root);
					stack.Add((root, 0));

					while (stack.Count > 0)
					{
						(int vertex, int nextEdge) = stack[stack.Count - 1];
						List<GraphNode> transposeEdges = _nodes[vertex]._transposeLinks;
						bool pushed = false;

						while (nextEdge < transposeEdges.Count)
						{
							int edgeVertex = transposeEdges[nextEdge]._index;
							++nextEdge;

							if (!assigned[edgeVertex])
							{
								assigned[edgeVertex] = true;
								sCCVertices.Add(edgeVertex);
								stack[stack.Count - 1] = (vertex, nextEdge);
								stack.Add((edgeVertex, 0));
								pushed = true;
								break;
							}
						}

						if (!pushed)
						{
							stack.RemoveAt(stack.Count - 1);
						}
					}

					int sCCIndex = _resultSCCs.Count;
					T[] sCCData = new T[sCCVertices.Count];
					for (int j = 0; j < sCCVertices.Count; ++j)
					{
						T nodeData = _nodes[sCCVertices[j]]._data;
						sCCData[j] = nodeData;
						_resultNodeToSCCIndex[nodeData] = sCCIndex;
					}
					_resultSCCs.Add(sCCData);
				}
			}

			// Phase 3 — Build condensed graph edges.
			// For each SCC, union the outgoing edges of all its member nodes into the condensed
			// edge list, mapping each target node to its SCC index and deduplicating with a
			// boolean scratch array that is spot-cleared per iteration.
			{
				int numSCCs = _resultSCCs.Count;
				bool[] edgeAdded = new bool[numSCCs];

				for (int sCCIndex = 0; sCCIndex < numSCCs; ++sCCIndex)
				{
					List<int> sCCEdges = [];

					foreach (T inNode in _resultSCCs[sCCIndex])
					{
						foreach (GraphNode neighbor in _nodeMap[inNode]._links)
						{
							int neighborSCC = _resultNodeToSCCIndex[neighbor._data];
							// NeighborSCC != SCCIndex also removes condensed self-edges
							if (neighborSCC != sCCIndex && !edgeAdded[neighborSCC])
							{
								edgeAdded[neighborSCC] = true;
								sCCEdges.Add(neighborSCC);
							}
						}
					}

					// Clear only the entries we touched — faster than resetting the whole array.
					foreach (int edgeSCC in sCCEdges)
					{
						edgeAdded[edgeSCC] = false;
					}

					_resultCondensedEdges.Add([.. sCCEdges]);
				}
			}

			return true;
		}

		/// <summary>
		/// Returns the list of strongly connected components (valid after calling Build()).
		/// Each SCC is an array of original nodes. Ordered root-to-leaf in the condensed graph.
		/// Always populated, even when Build() returns false.
		/// </summary>
		public List<T[]> GetSCCs() => _resultSCCs;

		/// <summary>
		/// Returns a mapping from each original node to the index of its SCC in GetSCCs()
		/// (valid after calling Build()). Always populated, even when Build() returns false.
		/// </summary>
		public IReadOnlyDictionary<T, int> GetNodeToSCCIndex() => _resultNodeToSCCIndex;

		/// <summary>
		/// Returns the adjacency list of the condensed graph (valid after calling Build()).
		/// GetCondensedEdges()[i] lists the SCC indices that SCC i has outgoing edges to.
		/// Only populated when Build() returns true; empty when Build() returns false.
		/// </summary>
		public List<int[]> GetCondensedEdges() => _resultCondensedEdges;

		/// <summary>
		/// Constructs the internal graph from the provided edge list.
		/// Nodes are added in the order they first appear. Self-edges are discarded.
		/// </summary>
		private void CreateGraph(List<Tuple<T, T>> edges)
		{
			// First pass: register all nodes in insertion order.
			foreach (Tuple<T, T> edge in edges)
			{
				if (!_nodeMap.ContainsKey(edge.Item1))
				{
					GraphNode node = new(edge.Item1, _nodes.Count);
					_nodes.Add(node);
					_nodeMap[edge.Item1] = node;
				}
				if (!_nodeMap.ContainsKey(edge.Item2))
				{
					GraphNode node = new(edge.Item2, _nodes.Count);
					_nodes.Add(node);
					_nodeMap[edge.Item2] = node;
				}
			}

			// Second pass: add edges, discarding self-edges.
			foreach (Tuple<T, T> edge in edges)
			{
				GraphNode src = _nodeMap[edge.Item1];
				GraphNode dst = _nodeMap[edge.Item2];
				if (src == dst)
				{
					continue;
				}
				src._links.Add(dst);
				dst._transposeLinks.Add(src);
			}
		}

		private enum VisitStatus : byte
		{
			NotVisited = 0,
			InProgress = 1,
			Visited    = 2,
		}

		private class GraphNode
		{
			/// <summary>User-provided node object.</summary>
			public T _data;

			/// <summary>Position in the Nodes list; used as an array index during Build().</summary>
			public int _index;

			/// <summary>Outgoing edges in the original graph.</summary>
			public List<GraphNode> _links = [];

			/// <summary>Incoming edges in the original graph (outgoing edges in the transpose graph).</summary>
			public List<GraphNode> _transposeLinks = [];

			public GraphNode(T data, int index)
			{
				_data  = data;
				_index = index;
			}
		}
	}
}
