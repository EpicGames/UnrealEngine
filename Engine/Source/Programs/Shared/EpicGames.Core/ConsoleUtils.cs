// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for writing to the console
	/// </summary>
	public static class ConsoleUtils
	{
		/// <summary>
		/// Gets the width of the window for formatting purposes
		/// </summary>
		public static int WindowWidth
		{
			get
			{
				// Get the window width, using a default value if there's no console attached to this process.
				int newWindowWidth;
				try
				{
					newWindowWidth = Console.WindowWidth;
				}
				catch
				{
					newWindowWidth = 240;
				}

				if (newWindowWidth <= 0)
				{
					newWindowWidth = 240;
				}

				return newWindowWidth;
			}
		}

		/// <summary>
		/// Writes the given text to the console as a sequence of word-wrapped lines
		/// </summary>
		/// <param name="text">The text to write to the console</param>
		public static void WriteLineWithWordWrap(string text)
		{
			WriteLineWithWordWrap(text, 0, 0);
		}

		/// <summary>
		/// Writes the given text to the console as a sequence of word-wrapped lines
		/// </summary>
		/// <param name="text">The text to write to the console</param>
		/// <param name="initialIndent">Indent for the first line</param>
		/// <param name="hangingIndent">Indent for lines after the first</param>
		public static void WriteLineWithWordWrap(string text, int initialIndent, int hangingIndent)
		{
			foreach (string line in StringUtils.WordWrap(text, initialIndent, hangingIndent, WindowWidth))
			{
				Console.WriteLine(line);
			}
		}

		/// <summary>
		/// Writes an colored warning message to the console
		/// </summary>
		/// <param name="text">The message to output</param>
		public static void WriteWarning(string text)
		{
			Console.ForegroundColor = ConsoleColor.Yellow;
			Console.WriteLine(text);
			Console.ResetColor();
		}

		/// <summary>
		/// Writes an colored error message to the console
		/// </summary>
		/// <param name="text">The message to output</param>
		public static void WriteError(string text)
		{
			Console.ForegroundColor = ConsoleColor.Red;
			Console.WriteLine(text);
			Console.ResetColor();
		}
	}
}
