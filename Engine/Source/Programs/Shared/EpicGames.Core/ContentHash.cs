// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Security.Cryptography;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;

#pragma warning disable CA5351 // Do Not Use Broken Cryptographic Algorithms
#pragma warning disable CA5350 // Do Not Use Weak Cryptographic Algorithms
#pragma warning disable CA1819 // Properties should not return arrays

namespace EpicGames.Core
{
	/// <summary>
	/// Stores the hash value for a piece of content as a byte array, allowing it to be used as a dictionary key
	/// </summary>
	[JsonConverter(typeof(ContentHashJsonConverter))]
	public class ContentHash : IEquatable<ContentHash>
	{
		/// <summary>
		/// Length of an MD5 hash
		/// </summary>
		public const int LengthMD5 = 16;

		/// <summary>
		/// Length of a SHA1 hash
		/// </summary>
		public const int LengthSHA1 = 20;

		/// <summary>
		/// Retrieves an empty content hash
		/// </summary>
		public static ContentHash Empty { get; } = new ContentHash([]);

		/// <summary>
		/// The bytes compromising this hash
		/// </summary>
		public byte[] Bytes
		{
			get;
			private set;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="bytes">The hash data</param>
		public ContentHash(byte[] bytes)
		{
			Bytes = bytes;
		}

		/// <summary>
		/// Compares two content hashes for equality
		/// </summary>
		/// <param name="other">The object to compare against</param>
		/// <returns>True if the hashes are equal, false otherwise</returns>
		public override bool Equals(object? other)
		{
			return Equals(other as ContentHash);
		}

		/// <summary>
		/// Compares two content hashes for equality
		/// </summary>
		/// <param name="other">The hash to compare against</param>
		/// <returns>True if the hashes are equal, false otherwise</returns>
		public bool Equals(ContentHash? other)
		{
			if (other is null)
			{
				return false;
			}
			else
			{
				return Bytes.SequenceEqual(other.Bytes);
			}
		}

		/// <summary>
		/// Compares two content hash objects for equality
		/// </summary>
		/// <param name="a">The first hash to compare</param>
		/// <param name="b">The second has to compare</param>
		/// <returns>True if the objects are equal, false otherwise</returns>
		public static bool operator ==(ContentHash? a, ContentHash? b)
		{
			if (a is null)
			{
				return b is null;
			}
			else
			{
				return a.Equals(b);
			}
		}

		/// <summary>
		/// Compares two content hash objects for inequality
		/// </summary>
		/// <param name="a">The first hash to compare</param>
		/// <param name="b">The second has to compare</param>
		/// <returns>True if the objects are not equal, false otherwise</returns>
		public static bool operator !=(ContentHash? a, ContentHash? b)
		{
			return !(a == b);
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <param name="algorithm">Algorithm to use to create the hash</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static ContentHash Compute(byte[] data, HashAlgorithm algorithm)
		{
			return new ContentHash(algorithm.ComputeHash(data));
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <param name="offset">Offset to the start of the data</param>
		/// <param name="count">Length of the data</param>
		/// <param name="algorithm">Algorithm to use to create the hash</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static ContentHash Compute(byte[] data, int offset, int count, HashAlgorithm algorithm)
		{
			return new ContentHash(algorithm.ComputeHash(data, offset, count));
		}

		/// <summary>
		/// Creates a content hash for a string, using a given algorithm.
		/// </summary>
		/// <param name="text">Text to compute a hash for</param>
		/// <param name="algorithm">Algorithm to use to create the hash</param>
		/// <returns>New content hash instance containing the hash of the text</returns>
		public static ContentHash Compute(string text, HashAlgorithm algorithm)
		{
			return new ContentHash(algorithm.ComputeHash(Encoding.Unicode.GetBytes(text)));
		}

		/// <summary>
		/// Creates a content hash for a file, using a given algorithm.
		/// </summary>
		/// <param name="location">File to compute a hash for</param>
		/// <param name="algorithm">Algorithm to use to create the hash</param>
		/// <returns>New content hash instance containing the hash of the file</returns>
		public static ContentHash Compute(FileReference location, HashAlgorithm algorithm)
		{
			using (FileStream stream = FileReference.Open(location, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				return new ContentHash(algorithm.ComputeHash(stream));
			}
		}

		/// <summary>
		/// Creates a content hash for a block of data using MD5
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static ContentHash MD5(byte[] data)
		{
			using (MD5 algorithm = System.Security.Cryptography.MD5.Create())
			{
				return Compute(data, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a block of data using MD5
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <param name="offset">Offset to the start of the data</param>
		/// <param name="count">Length of the data</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static ContentHash MD5(byte[] data, int offset, int count)
		{
			using (MD5 algorithm = System.Security.Cryptography.MD5.Create())
			{
				return Compute(data, offset, count, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a string using MD5.
		/// </summary>
		/// <param name="text">Text to compute a hash for</param>
		/// <returns>New content hash instance containing the hash of the text</returns>
		public static ContentHash MD5(string text)
		{
			using (MD5 algorithm = System.Security.Cryptography.MD5.Create())
			{
				return Compute(text, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a file, using a given algorithm.
		/// </summary>
		/// <param name="location">File to compute a hash for</param>
		/// <returns>New content hash instance containing the hash of the file</returns>
		public static ContentHash MD5(FileReference location)
		{
			using (MD5 algorithm = System.Security.Cryptography.MD5.Create())
			{
				return Compute(location, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a block of data using SHA1
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static ContentHash SHA1(byte[] data)
		{
			using (SHA1 algorithm = System.Security.Cryptography.SHA1.Create())
			{
				return Compute(data, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a string using SHA1.
		/// </summary>
		/// <param name="text">Text to compute a hash for</param>
		/// <returns>New content hash instance containing the hash of the text</returns>
		public static ContentHash SHA1(string text)
		{
			using (SHA1 algorithm = System.Security.Cryptography.SHA1.Create())
			{
				return Compute(text, algorithm);
			}
		}

		/// <summary>
		/// Creates a content hash for a file using SHA1.
		/// </summary>
		/// <param name="location">File to compute a hash for</param>
		/// <returns>New content hash instance containing the hash of the file</returns>
		public static ContentHash SHA1(FileReference location)
		{
			using (SHA1 algorithm = System.Security.Cryptography.SHA1.Create())
			{
				return Compute(location, algorithm);
			}
		}

		/// <summary>
		/// Parse a hash from a string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>Value of the hash</returns>
		public static ContentHash Parse(string text)
		{
			ContentHash? hash;
			if (!TryParse(text, out hash))
			{
				throw new ArgumentException(String.Format("'{0}' is not a valid content hash", text));
			}
			return hash;
		}

		/// <summary>
		/// Parse a hash from a string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="hash"></param>
		/// <returns>Value of the hash</returns>
		public static bool TryParse(string text, [NotNullWhen(true)] out ContentHash? hash)
		{
			byte[]? bytes;
			if (StringUtils.TryParseHexString(text, out bytes))
			{
				hash = new ContentHash(bytes);
				return true;
			}
			else
			{
				hash = null;
				return false;
			}
		}

		/// <summary>
		/// Computes a hash code for this digest
		/// </summary>
		/// <returns>Integer value to use as a hash code</returns>
		public override int GetHashCode()
		{
			int hashCode = Bytes[0];
			for (int idx = 1; idx < Bytes.Length; idx++)
			{
				hashCode = (hashCode * 31) + Bytes[idx];
			}
			return hashCode;
		}

		/// <summary>
		/// Formats this hash as a string
		/// </summary>
		/// <returns>The hashed value</returns>
		public override string ToString()
		{
			return StringUtils.FormatHexString(Bytes);
		}
	}

	/// <summary>
	/// Converts <see cref="ContentHash"/> values to and from JSON
	/// </summary>
	public class ContentHashJsonConverter : JsonConverter<ContentHash>
	{
		/// <inheritdoc/>
		public override ContentHash Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return ContentHash.Parse(reader.GetString()!);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ContentHash value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}

	/// <summary>
	/// Utility methods for serializing ContentHash objects
	/// </summary>
	public static class ContentHashExtensionMethods
	{
		/// <summary>
		/// Writes a ContentHash to a binary archive
		/// </summary>
		/// <param name="writer">The writer to output data to</param>
		/// <param name="hash">The hash to write</param>
		public static void WriteContentHash(this BinaryArchiveWriter writer, ContentHash? hash)
		{
			if (hash is null)
			{
				writer.WriteByteArray(null);
			}
			else
			{
				writer.WriteByteArray(hash.Bytes);
			}
		}

		/// <summary>
		/// Reads a ContentHash from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New hash instance</returns>
		public static ContentHash? ReadContentHash(this BinaryArchiveReader reader)
		{
			byte[]? data = reader.ReadByteArray();
			if (data == null)
			{
				return null;
			}
			else
			{
				return new ContentHash(data);
			}
		}
	}
}
