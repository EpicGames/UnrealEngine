// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

#pragma warning disable CA1715 // Identifiers should have correct prefix

namespace EpicGames.Core
{

	/// <summary>
	/// This interface exists to allow the CsProjBuilder in EpicGames.MsBuild to call back out
	/// into EpicGames.Build.  It is EXTREMELY important that any type definitions that must be 
	/// referenced or implemented in EpicGames.Build for use by EpicGame.MsBuild *NOT* be defined
	/// in EpicGames.MsBuild.  If they are, there is a strong chance of running into an issue 
	/// gathering types (Assembly.GetTypes()) on EpicGames.Build.  
	/// </summary>
	public interface CsProjBuildHook : IMsBuildRegistration
	{
		/// <summary>
		/// Test the cache for a given file to get the last write time of the given file 
		/// </summary>
		/// <param name="basePath">Base path of the file.</param>
		/// <param name="relativeFilePath">Relative path of the file</param>
		/// <returns>Last write time of the file.</returns>
		DateTime GetLastWriteTime(DirectoryReference basePath, string relativeFilePath);

		/// <summary>
		/// Test the cache for a given file to get the last write time of the given file 
		/// </summary>
		/// <param name="basePath">Base path of the file.</param>
		/// <param name="relativeFilePath">Relative path of the file</param>
		/// <returns>Last write time of the file.</returns>
		DateTime GetLastWriteTime(string basePath, string relativeFilePath);

		/// <summary>
		/// Return the build record directory for the given base path (i.e. engine dir or project dir)
		/// </summary>
		/// <param name="basePath">The base path for the directory</param>
		/// <returns>Directory for the build records</returns>
		DirectoryReference GetBuildRecordDirectory(DirectoryReference basePath);

		/// <summary>
		/// Validate the given build records for the project
		/// </summary>
		/// <param name="buildRecords">Build records being validated.  This also includes build records for dependencies.</param>
		/// <param name="projectPath">Path of the project</param>
		void ValidateRecursively(Dictionary<FileReference, CsProjBuildRecordEntry> buildRecords, FileReference projectPath);

		/// <summary>
		/// (Optional) Sets of existing build records that are valid
		/// </summary>
		/// <param name="records">The records to set</param>
		void SetValidBuildRecords(IReadOnlyDictionary<FileReference, CsProjBuildRecordEntry> records);

		/// <summary>
		/// (Optional) Collection of existing build records for projects already been registered as valid, which could be skipped when building
		/// </summary>
		IReadOnlyDictionary<FileReference, CsProjBuildRecordEntry> ValidBuildRecords { get; }

		/// <summary>
		/// Unreal engine directory
		/// </summary>
		DirectoryReference EngineDirectory { get; }
	}
}
