// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

#pragma warning disable CA2227 // Collection properties should be read only
#pragma warning disable CA1034 // Nested types should not be visible

namespace EpicGames.Core
{

	/// <summary>
	/// The list of references projects needs to be both the path to the project and the
	/// target build time from that project.
	/// </summary>
	public class CsProjBuildRecordRef
	{
		/// <summary>
		/// Name of the referenced project
		/// </summary>
		public string ProjectPath { get; set; } = String.Empty;

		/// <summary>
		/// The time that the target assembly was built (read from the file after the build)
		/// </summary>
		public DateTime TargetBuildTime { get; set; } = DateTime.Now;
	}

	/// <summary>
	/// Acceleration structure:
	/// used to encapsulate a full set of dependencies for an msbuild project - explicit and globbed
	/// These files are written to Intermediate/ScriptModules
	/// </summary>
	public class CsProjBuildRecord
	{
		/// <summary>
		/// Version number making it possible to quickly invalidate written records.
		/// </summary>
		public static readonly int CurrentVersion = 7;

		/// <summary>
		/// Build configuration for valid written records
		/// </summary>
		public static readonly string CurrentConfiguration
#if DEBUG
		= "Debug";
#else
		= "Development";
#endif

		/// <summary>
		/// Version number for this record
		/// </summary>
		public int Version { get; set; } = -1;

		/// <summary>
		/// .NET runtime version when the target assembly was built
		/// </summary>
		public Version RuntimeVersion { get; set; } = Environment.Version;

		/// <summary>
		/// Path to the .csproj project file, relative to the location of the build record .json file
		/// </summary>
		public string? ProjectPath { get; set; }

		/// <summary>
		/// The time that the target assembly was built (read from the file after the build)
		/// </summary>
		public DateTime TargetBuildTime { get; set; }

		/// <summary>
		/// The build configuration for the target assembly when built
		/// </summary>
		public string TargetConfiguration { get; set; } = CurrentConfiguration;

		// all following paths are relative to the project directory, the directory containing ProjectPath 

		/// <summary>
		/// assembly (dll) location
		/// </summary>
		public string? TargetPath { get; set; }

		/// <summary>
		/// Paths to referenced projects and the target build time in that project
		/// </summary>
		public List<CsProjBuildRecordRef> ProjectReferencesAndTimes { get; set; } = [];

		/// <summary>
		/// file dependencies. Any globs are resolved
		/// </summary>
		public HashSet<string> Dependencies { get; set; } = [];

		/// <summary>
		/// A glob pattern
		/// </summary>
		public class Glob
		{
			/// <summary>
			/// Type of item
			/// </summary>
			public string? ItemType { get; set; }

			/// <summary>
			/// Paths to include
			/// </summary>
			public List<string>? Include { get; set; }

			/// <summary>
			/// Paths to exclude
			/// </summary>
			public List<string>? Exclude { get; set; }

			/// <summary>
			/// 
			/// </summary>
			public List<string>? Remove { get; set; }
		}

		/// <summary>
		/// List of globs
		/// </summary>
		public List<Glob> Globs { get; set; } = [];
	}

	/// <summary>
	/// Represents the status of a build record.  When read from disk, the status will start
	/// as being unknown.  Then through the validation process, it will either become valid
	/// or invalid.
	/// </summary>
	public enum CsProjBuildRecordStatus
	{
		/// <summary>
		/// Status of the build record is unknown
		/// </summary>
		Unknown,

		/// <summary>
		/// The build record is valid and doesn't require rebuild
		/// </summary>
		Valid,

		/// <summary>
		/// The build record is invalid
		/// </summary>
		Invalid,
	};

	/// <summary>
	/// Represents a runtime build record entry that isn't persisted.  Aside from the build record, it 
	/// contains the project file and build record file reference.  Also the "need to build" status
	/// of the record.
	/// </summary>
	public class CsProjBuildRecordEntry
	{

		/// <summary>
		/// Location of the csproj file
		/// </summary>
		public FileReference ProjectFile { get; }

		/// <summary>
		/// Location of the build record
		/// </summary>
		public FileReference BuildRecordFile { get; }

		/// <summary>
		/// The persisted build record
		/// </summary>
		public CsProjBuildRecord BuildRecord { get; }

		/// <summary>
		/// Status of the build record
		/// </summary>
		public CsProjBuildRecordStatus Status { get; set; } = CsProjBuildRecordStatus.Unknown;

		/// <summary>
		/// Construct a new entry
		/// </summary>
		/// <param name="projectFile">Project file reference</param>
		/// <param name="buildRecordFile">Build record file reference</param>
		/// <param name="buildRecord">Persisted build record</param>
		public CsProjBuildRecordEntry(FileReference projectFile, FileReference buildRecordFile, CsProjBuildRecord buildRecord)
		{
			ProjectFile = projectFile;
			BuildRecordFile = buildRecordFile;
			BuildRecord = buildRecord;
		}
	}
}
