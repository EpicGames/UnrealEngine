// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Security.Cryptography;
using System.Text;

#pragma warning disable CA1000 // Do not declare static members on generic types
#pragma warning disable CA5351 // Do Not Use Broken Cryptographic Algorithms
#pragma warning disable CA5350 // Do Not Use Weak Cryptographic Algorithms

namespace EpicGames.Core
{
	/// <summary>
	/// Struct representing a weakly typed hash value. Counterpart to <see cref="Digest{T}"/> - a strongly typed digest.
	/// </summary>
	public readonly struct Digest : IEquatable<Digest>
	{
		/// <summary>
		/// Memory storing the digest data
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Accessor for the span of memory storing the data
		/// </summary>
		public ReadOnlySpan<byte> Span => Memory.Span;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">Memory to construct from</param>
		public Digest(ReadOnlyMemory<byte> memory)
		{
			Memory = memory;
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Digest<T> Compute<T>(byte[] data) where T : DigestTraits, new()
		{
			using HashAlgorithm algorithm = Digest<T>.Traits.CreateAlgorithm();
			return algorithm.ComputeHash(data);
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="text">Text to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Digest<T> Compute<T>(string text) where T : DigestTraits, new()
		{
			return Compute<T>(Encoding.UTF8.GetBytes(text));
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Digest<T> Compute<T>(ReadOnlySpan<byte> data) where T : DigestTraits, new()
		{
			byte[] value = new byte[Digest<T>.Traits.Length];
			using (HashAlgorithm algorithm = Digest<T>.Traits.CreateAlgorithm())
			{
				if (!algorithm.TryComputeHash(data, value, out int written) || written != value.Length)
				{
					throw new InvalidOperationException("Unable to compute hash for buffer");
				}
			}
			return new Digest<T>(value);
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Digest Parse(string text)
		{
			return new Digest(StringUtils.ParseHexString(text));
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Digest<T> Parse<T>(string text) where T : DigestTraits, new()
		{
			return new Digest<T>(StringUtils.ParseHexString(text));
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is Digest digest) && Equals(digest);

		/// <inheritdoc/>
		public bool Equals(Digest other) => other.Span.SequenceEqual(Span);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32LittleEndian(Span);

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(Memory.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator ==(Digest a, Digest b)
		{
			return a.Memory.Span.SequenceEqual(b.Memory.Span);
		}

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator !=(Digest a, Digest b)
		{
			return !(a == b);
		}

		/// <summary>
		/// Implicit conversion operator from memory objects
		/// </summary>
		/// <param name="memory"></param>
		public static implicit operator Digest(ReadOnlyMemory<byte> memory)
		{
			return new Digest(memory);
		}

		/// <summary>
		/// Implicit conversion operator from byte arrays
		/// </summary>
		/// <param name="memory"></param>
		public static implicit operator Digest(byte[] memory)
		{
			return new Digest(memory);
		}
	}

	/// <summary>
	/// Traits for a hashing algorithm
	/// </summary>
	public abstract class DigestTraits
	{
		/// <summary>
		/// Length of the produced hash
		/// </summary>
		public int Length { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="length"></param>
		protected DigestTraits(int length)
		{
			Length = length;
		}

		/// <summary>
		/// Creates a HashAlgorithm object
		/// </summary>
		/// <returns></returns>
		public abstract HashAlgorithm CreateAlgorithm();
	}

	/// <summary>
	/// Traits for the MD5 hash algorithm
	/// </summary>
	public class Md5 : DigestTraits
	{
		/// <summary>
		/// Length of the produced digest
		/// </summary>
		public new const int Length = 16;

		/// <summary>
		/// Constructor
		/// </summary>
		public Md5()
			: base(Length)
		{
		}

		/// <inheritdoc/>
		public override HashAlgorithm CreateAlgorithm() => MD5.Create();
	}

	/// <summary>
	/// Traits for the SHA1 hash algorithm
	/// </summary>
	public class Sha1 : DigestTraits
	{
		/// <summary>
		/// Length of the produced digest
		/// </summary>
		public new const int Length = 20;

		/// <summary>
		/// Constructor
		/// </summary>
		public Sha1()
			: base(Length)
		{
		}

		/// <inheritdoc/>
		public override HashAlgorithm CreateAlgorithm() => SHA1.Create();
	}

	/// <summary>
	/// Traits for the SHA1 hash algorithm
	/// </summary>
	public class Sha256 : DigestTraits
	{
		/// <summary>
		/// Length of the produced digest
		/// </summary>
		public new const int Length = 32;

		/// <summary>
		/// Constructor
		/// </summary>
		public Sha256()
			: base(Length)
		{
		}

		/// <inheritdoc/>
		public override HashAlgorithm CreateAlgorithm() => SHA256.Create();
	}

	/// <summary>
	/// Generic HashValue implementation
	/// </summary>
	public readonly struct Digest<T> : IEquatable<Digest<T>> where T : DigestTraits, new()
	{
		/// <summary>
		/// Traits instance
		/// </summary>
		public static T Traits { get; } = new T();

		/// <summary>
		/// Length of a hash value
		/// </summary>
		public static int Length => Traits.Length;

		/// <summary>
		/// Zero digest value
		/// </summary>
		public static Digest<T> Zero => new(new byte[Traits.Length]);

		/// <summary>
		/// Memory storing the digest data
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Accessor for the span of memory storing the data
		/// </summary>
		public ReadOnlySpan<byte> Span => Memory.Span;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">Memory to construct from</param>
		public Digest(ReadOnlyMemory<byte> memory)
		{
			Memory = memory;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is Digest<T> hash) && Equals(hash);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32LittleEndian(Span);

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(Memory.Span);

		/// <inheritdoc/>
		public bool Equals(Digest<T> other) => other.Span.SequenceEqual(Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator ==(Digest<T> a, Digest<T> b) => a.Span.SequenceEqual(b.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator !=(Digest<T> a, Digest<T> b) => !a.Span.SequenceEqual(b.Span);

		/// <summary>
		/// Implicit conversion operator from memory objects
		/// </summary>
		/// <param name="memory"></param>
		public static implicit operator Digest<T>(ReadOnlyMemory<byte> memory)
		{
			return new Digest<T>(memory);
		}

		/// <summary>
		/// Implicit conversion operator from byte arrays
		/// </summary>
		/// <param name="memory"></param>
		public static implicit operator Digest<T>(byte[] memory)
		{
			return new Digest<T>(memory);
		}
	}

	/// <summary>
	/// Extension methods for dealing with digests
	/// </summary>
	public static class DigestExtensions
	{
		/// <summary>
		/// Read a digest from a memory reader
		/// </summary>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static Digest ReadDigest(this MemoryReader reader)
		{
			return new Digest(reader.ReadVariableLengthBytesWithInt32Length());
		}

		/// <summary>
		/// Read a strongly-typed digest from a memory reader
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static Digest<T> ReadDigest<T>(this MemoryReader reader) where T : DigestTraits, new()
		{
			return new Digest<T>(reader.ReadFixedLengthBytes(Digest<T>.Traits.Length));
		}

		/// <summary>
		/// Write a digest to a memory writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="digest"></param>
		public static void WriteDigest(this MemoryWriter writer, Digest digest)
		{
			writer.WriteVariableLengthBytesWithInt32Length(digest.Span);
		}

		/// <summary>
		/// Write a strongly typed digest to a memory writer
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="writer"></param>
		/// <param name="digest"></param>
		public static void WriteDigest<T>(this MemoryWriter writer, Digest<T> digest) where T : DigestTraits, new()
		{
			writer.WriteFixedLengthBytes(digest.Span);
		}
	}
}
