// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Representation of an absolute directory path. Allows fast hashing and comparisons.
	/// </summary>
	[Serializable]
	[TypeConverter(typeof(DirectoryReferenceTypeConverter))]
	[JsonConverter(typeof(DirectoryReferenceJsonConverter))]
	public class DirectoryReference : FileSystemReference, IEquatable<DirectoryReference>, IComparable<DirectoryReference>
	{
		/// <summary>
		/// Special value used to invoke the non-sanitizing constructor overload
		/// </summary>
		public enum Sanitize
		{
			/// <summary>
			/// Dummy value
			/// </summary>
			None
		}

		/// <summary>
		/// Default constructor.
		/// </summary>
		/// <param name="inPath">Path to this directory.</param>
		public DirectoryReference(string inPath)
			: base(FixTrailingPathSeparator(Path.GetFullPath(inPath)))
		{
		}

		/// <summary>
		/// Construct a DirectoryReference from a DirectoryInfo object.
		/// </summary>
		/// <param name="inInfo">Path to this file</param>
		public DirectoryReference(DirectoryInfo inInfo)
			: base(FixTrailingPathSeparator(inInfo.FullName))
		{
		}

		/// <summary>
		/// Constructor for creating a directory object directly from two strings.
		/// </summary>
		/// <param name="fullName">The full, sanitized path name</param>
		/// <param name="sanitize">Sanitize argument. Ignored.</param>
		public DirectoryReference(string fullName, Sanitize sanitize)
			: base(fullName)
		{
			_ = sanitize;
		}

		/// <summary>
		/// Ensures that the correct trailing path separator is appended. On Windows, the root directory (eg. C:\) always has a trailing path separator, but no other
		/// path does.
		/// </summary>
		/// <param name="dirName">Absolute path to the directory</param>
		/// <returns>Path to the directory, with the correct trailing path separator</returns>
		private static string FixTrailingPathSeparator(string dirName)
		{
			if (dirName.Length == 2 && dirName[1] == ':')
			{
				return dirName + Path.DirectorySeparatorChar;
			}
			else if (dirName.Length == 3 && dirName[1] == ':' && dirName[2] == Path.DirectorySeparatorChar)
			{
				return dirName;
			}
			else if (dirName.Length > 1 && dirName[^1] == Path.DirectorySeparatorChar)
			{
				return dirName.TrimEnd(Path.DirectorySeparatorChar);
			}
			else
			{
				return dirName;
			}
		}

		/// <summary>
		/// Gets the top level directory name
		/// </summary>
		/// <returns>The name of the directory</returns>
		public string GetDirectoryName() => Path.GetFileName(FullName);

		/// <summary>
		/// Gets the directory containing this object
		/// </summary>
		/// <returns>A new directory object representing the directory containing this object</returns>
		public DirectoryReference? ParentDirectory
		{
			get
			{
				if (IsRootDirectory())
				{
					return null;
				}

				int parentLength = FullName.LastIndexOf(Path.DirectorySeparatorChar);
				if (parentLength == 2 && FullName[1] == ':')
				{
					parentLength++;
				}

				if (parentLength == 0 && FullName[0] == Path.DirectorySeparatorChar)
				{
					// we have reached the root
					parentLength = 1;
				}

				return new(FullName.Substring(0, parentLength), Sanitize.None);
			}
		}

		/// <summary>
		/// Add a directory to the PATH environment variable
		/// </summary>
		/// <param name="path">The path to add</param>
		public static void AddDirectoryToPath(DirectoryReference path)
		{
			string pathEnvironmentVariable = Environment.GetEnvironmentVariable("PATH") ?? "";
			if (!pathEnvironmentVariable.Split(Path.PathSeparator).Any(x => String.Equals(x, path.FullName, StringComparison.Ordinal)))
			{
				pathEnvironmentVariable = $"{path.FullName}{Path.PathSeparator}{pathEnvironmentVariable}";
				Environment.SetEnvironmentVariable("PATH", pathEnvironmentVariable);
			}
		}

		/// <summary>
		/// Returns the path of the current user's temporary folder
		/// </summary>
		/// <returns>The path to the temporary folder</returns>
		public static DirectoryReference GetTempPath() => new(Path.GetTempPath());

		/// <summary>
		/// Gets the path for a special folder
		/// </summary>
		/// <param name="folder">The folder to receive the path for</param>
		/// <returns>Directory reference for the given folder, or null if it is not available</returns>
		public static DirectoryReference? GetSpecialFolder(Environment.SpecialFolder folder)
		{
			string folderPath = Environment.GetFolderPath(folder);
			return String.IsNullOrEmpty(folderPath) ? null : new(folderPath);
		}

		/// <summary>
		/// Determines whether this path represents a root directory in the filesystem
		/// </summary>
		/// <returns>True if this path is a root directory, false otherwise</returns>
		public bool IsRootDirectory() => FullName[^1] == Path.DirectorySeparatorChar;

		/// <summary>
		/// Combine several fragments with a base directory, to form a new directory name
		/// </summary>
		/// <param name="baseDirectory">The base directory</param>
		/// <param name="fragments">Fragments to combine with the base directory</param>
		/// <returns>The new directory name</returns>
		public static DirectoryReference Combine(DirectoryReference baseDirectory, params string[] fragments) => fragments.Length == 0 ? baseDirectory : new(CombineStrings(baseDirectory, fragments), Sanitize.None);

		/// <summary>
		/// Compares two filesystem object names for equality. Uses the canonical name representation, not the display name representation.
		/// </summary>
		/// <param name="a">First object to compare.</param>
		/// <param name="b">Second object to compare.</param>
		/// <returns>True if the names represent the same object, false otherwise</returns>
		public static bool operator ==(DirectoryReference? a, DirectoryReference? b)
		{
			if (a is null)
			{
				return b is null;
			}
			else if (b is null)
			{
				return false;
			}
			else
			{
				return a.FullName.Equals(b.FullName, Comparison);
			}
		}

		/// <summary>
		/// Compares two filesystem object names for inequality. Uses the canonical name representation, not the display name representation.
		/// </summary>
		/// <param name="a">First object to compare.</param>
		/// <param name="b">Second object to compare.</param>
		/// <returns>False if the names represent the same object, true otherwise</returns>
		public static bool operator !=(DirectoryReference? a, DirectoryReference? b) => !(a == b);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is DirectoryReference dir) && dir == this;

		/// <summary>
		/// Compares against another object for equality.
		/// </summary>
		/// <param name="obj">other instance to compare.</param>
		/// <returns>True if the names represent the same object, false otherwise</returns>
		public bool Equals(DirectoryReference? obj) => obj == this;

		/// <inheritdoc/>
		public override int GetHashCode() => base.GetHashCode();

		/// <inheritdoc/>
		public int CompareTo(DirectoryReference? other) => Comparer.Compare(FullName, other?.FullName);

		/// <inheritdoc/>
		public static bool operator <(DirectoryReference left, DirectoryReference right) => left is null ? right is not null : left.CompareTo(right) < 0;

		/// <inheritdoc/>
		public static bool operator <=(DirectoryReference left, DirectoryReference right) => left is null || left.CompareTo(right) <= 0;

		/// <inheritdoc/>
		public static bool operator >(DirectoryReference left, DirectoryReference right) => left is not null && left.CompareTo(right) > 0;

		/// <inheritdoc/>
		public static bool operator >=(DirectoryReference left, DirectoryReference right) => left is null ? right is null : left.CompareTo(right) >= 0;

		/// <summary>
		/// Moves a directory from one location to another
		/// </summary>
		/// <param name="sourceDir">The source directory</param>
		/// <param name="destinationDir">The destination directory</param>
		public static void Move(DirectoryReference sourceDir, DirectoryReference destinationDir)
			=> Directory.Move(sourceDir.FullName, destinationDir.FullName);

		/// <summary>
		/// Copies a directory from one location to another
		/// </summary>
		/// <param name="sourceDir">The source directory</param>
		/// <param name="destinationDir">The destination directory</param>
		public static void Copy(DirectoryReference sourceDir, DirectoryReference destinationDir)
		{
			CreateDirectory(destinationDir);

			foreach (string file in Directory.GetFiles(sourceDir.FullName))
			{
				File.Copy(file, Path.Combine(destinationDir.FullName, Path.GetFileName(file)), true);
			}

			foreach (string directory in Directory.GetDirectories(sourceDir.FullName))
			{
				Copy(new(directory), Combine(destinationDir, Path.GetFileName(directory)));
			}
		}

		/// <summary>
		/// Calculates the total size in bytes of all files in a directory and its subdirectories
		/// </summary>
		/// <param name="dirPath">Path to directory</param>
		/// <returns>The total size in bytes of all accessible files in the directory and its subdirectories</returns>
		public static long GetTotalSize(DirectoryReference dirPath)
		{
			long size = 0;
			foreach (FileReference file in EnumerateFiles(dirPath, "*", SearchOption.AllDirectories))
			{
				try
				{
					size += file.ToFileInfo().Length;
				}
				catch (UnauthorizedAccessException) { }
				catch (FileNotFoundException) { }
			}
			return size;
		}

		/// <summary>
		/// Helper function to create a remote directory reference. Unlike normal DirectoryReference objects, these aren't converted to a full path in the local filesystem.
		/// </summary>
		/// <param name="absolutePath">The absolute path in the remote file system</param>
		/// <returns>New directory reference</returns>
		public static DirectoryReference MakeRemote(string absolutePath) => new(absolutePath, Sanitize.None);

		/// <summary>
		/// Gets the parent directory for a file, or returns null if it's null.
		/// </summary>
		/// <param name="file">The file to create a directory reference for</param>
		/// <returns>The directory containing the file  </returns>
		[return: NotNullIfNotNull(nameof(file))]
		public static DirectoryReference? FromFile(FileReference? file) => file?.Directory;

		/// <summary>
		/// Create a DirectoryReference from a string. If the string is null or whitespace, returns a null DirectoryReference.
		/// </summary>
		/// <param name="directoryName">Path for the new object</param>
		/// <returns>Returns a FileReference representing the given string, or null.</returns>
		public static DirectoryReference? FromString(string? directoryName) => String.IsNullOrWhiteSpace(directoryName) ? null : new(directoryName);

		/// <summary>
		/// Finds the correct case to match the location of this file on disk. Uses the given case for parts of the path that do not exist.
		/// </summary>
		/// <param name="location">The path to find the correct case for</param>
		/// <returns>Location of the file with the correct case</returns>
		public static DirectoryReference FindCorrectCase(DirectoryReference location) => new(DirectoryUtils.FindCorrectCase(location.ToDirectoryInfo()));

		/// <summary>
		/// Constructs a DirectoryInfo object from this reference
		/// </summary>
		/// <returns>New DirectoryInfo object</returns>
		public DirectoryInfo ToDirectoryInfo() => new(FullName);

		#region System.IO.Directory Wrapper Methods

		/// <summary>
		/// Finds the current directory
		/// </summary>
		/// <returns>The current directory</returns>
		public static DirectoryReference GetCurrentDirectory() => new(Directory.GetCurrentDirectory());

		/// <summary>
		/// Creates a directory
		/// </summary>
		/// <param name="location">Location of the directory</param>
		public static void CreateDirectory(DirectoryReference location) => Directory.CreateDirectory(location.FullName);

		/// <summary>
		/// Deletes a directory
		/// </summary>
		/// <param name="location">Location of the directory</param>
		public static void Delete(DirectoryReference location) => Directory.Delete(location.FullName);

		/// <summary>
		/// Deletes a directory
		/// </summary>
		/// <param name="location">Location of the directory</param>
		/// <param name="bRecursive">Whether to remove directories recursively</param>
		public static void Delete(DirectoryReference location, bool bRecursive) => Directory.Delete(location.FullName, bRecursive);

		/// <summary>
		/// Checks whether the directory exists
		/// </summary>
		/// <param name="location">Location of the directory</param>
		/// <returns>True if this directory exists</returns>
		public static bool Exists(DirectoryReference location) => Directory.Exists(location.FullName);

		/// <summary>
		/// Enumerate files from a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <returns>Sequence of file references</returns>
		public static IEnumerable<FileReference> EnumerateFiles(DirectoryReference baseDir) => Directory.EnumerateFiles(baseDir.FullName).Select(fileName => new FileReference(fileName, FileReference.Sanitize.None));

		/// <summary>
		/// Enumerate files from a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <param name="pattern">Pattern for matching files</param>
		/// <returns>Sequence of file references</returns>
		public static IEnumerable<FileReference> EnumerateFiles(DirectoryReference baseDir, string pattern) => Directory.EnumerateFiles(baseDir.FullName, pattern).Select(fileName => new FileReference(fileName, FileReference.Sanitize.None));

		/// <summary>
		/// Enumerate files from a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <param name="pattern">Pattern for matching files</param>
		/// <param name="option">Options for the search</param>
		/// <returns>Sequence of file references</returns>
		public static IEnumerable<FileReference> EnumerateFiles(DirectoryReference baseDir, string pattern, SearchOption option) => Directory.EnumerateFiles(baseDir.FullName, pattern, option).Select(fileName => new FileReference(fileName, FileReference.Sanitize.None));

		/// <summary>
		/// Enumerate subdirectories in a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <returns>Sequence of directory references</returns>
		public static IEnumerable<DirectoryReference> EnumerateDirectories(DirectoryReference baseDir) => Directory.EnumerateDirectories(baseDir.FullName).Select(directoryName => new DirectoryReference(directoryName, Sanitize.None));

		/// <summary>
		/// Enumerate subdirectories in a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <param name="pattern">Pattern for matching directories</param>
		/// <returns>Sequence of directory references</returns>
		public static IEnumerable<DirectoryReference> EnumerateDirectories(DirectoryReference baseDir, string pattern) => Directory.EnumerateDirectories(baseDir.FullName, pattern).Select(directoryName => new DirectoryReference(directoryName, Sanitize.None));

		/// <summary>
		/// Enumerate subdirectories in a given directory
		/// </summary>
		/// <param name="baseDir">Base directory to search in</param>
		/// <param name="pattern">Pattern for matching files</param>
		/// <param name="option">Options for the search</param>
		/// <returns>Sequence of directory references</returns>
		public static IEnumerable<DirectoryReference> EnumerateDirectories(DirectoryReference baseDir, string pattern, SearchOption option) => Directory.EnumerateDirectories(baseDir.FullName, pattern, option).Select(directoryName => new DirectoryReference(directoryName, Sanitize.None));

		/// <summary>
		/// Sets the current directory
		/// </summary>
		/// <param name="location">Location of the new current directory</param>
		public static void SetCurrentDirectory(DirectoryReference location)
		{
			// If the new working directory only differs by text case on Windows, Directory.SetCurrentDirectory() will not actually update the working directory.
			// To work around this, set the working directory to an entirely different path first.
			if (RuntimePlatform.Current == RuntimePlatform.Type.Windows && Directory.GetCurrentDirectory().Equals(location.FullName, StringComparison.OrdinalIgnoreCase))
			{
				Directory.SetCurrentDirectory(Path.GetTempPath());
			}
			Directory.SetCurrentDirectory(location.FullName);
		}

		#endregion
	}

	/// <summary>
	/// Type converter to/from strings
	/// </summary>
	class DirectoryReferenceTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || base.CanConvertFrom(context, sourceType);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string stringValue)
			{
				return new DirectoryReference(stringValue);
			}
			return base.ConvertFrom(context, culture, value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			return destinationType == typeof(string) || base.CanConvertTo(context, destinationType);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}

	internal class DirectoryReferenceJsonConverter : JsonConverter<DirectoryReference>
	{
		public override DirectoryReference? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? fullName = reader.GetString();
			return fullName is null ? null : new(fullName);
		}

		public override DirectoryReference ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.PropertyName)
			{
				throw new JsonException("DirectoryReference keys must be represented as JSON strings");
			}
			return new(reader.GetString()!);
		}

		public override void Write(Utf8JsonWriter writer, DirectoryReference value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.FullName);
		}

		public override void WriteAsPropertyName(Utf8JsonWriter writer, [DisallowNull] DirectoryReference value, JsonSerializerOptions options)
		{
			writer.WritePropertyName(value.FullName);
		}
	}

	/// <summary>
	/// Extension methods for passing DirectoryReference arguments
	/// </summary>
	public static class DirectoryReferenceExtensionMethods
	{
		/// <summary>
		/// Manually serialize a file reference to a binary stream.
		/// </summary>
		/// <param name="writer">Binary writer to write to</param>
		/// <param name="directory">The directory reference to write</param>
		public static void Write(this BinaryWriter writer, DirectoryReference directory) => writer.Write((directory == null) ? String.Empty : directory.FullName);

		/// <summary>
		/// Manually deserialize a directory reference from a binary stream.
		/// </summary>
		/// <param name="reader">Binary reader to read from</param>
		/// <returns>New DirectoryReference object</returns>
		public static DirectoryReference? ReadDirectoryReference(this BinaryReader reader)
		{
			string fullName = reader.ReadString();
			return (fullName.Length == 0) ? null : new(fullName, DirectoryReference.Sanitize.None);
		}

		/// <summary>
		/// Manually deserialize a directory reference from a binary stream.
		/// </summary>
		/// <param name="reader">Binary reader to read from</param>
		/// <returns>New DirectoryReference object</returns>
		public static DirectoryReference ReadDirectoryReferenceNotNull(this BinaryReader reader) => BinaryArchiveReader.NotNull(ReadDirectoryReference(reader));

		/// <summary>
		/// Writes a directory reference  to a binary archive
		/// </summary>
		/// <param name="writer">The writer to output data to</param>
		/// <param name="directory">The item to write</param>
		public static void WriteDirectoryReference(this BinaryArchiveWriter writer, DirectoryReference? directory) => writer.WriteString(directory?.FullName);

		/// <summary>
		/// Reads a directory reference from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New directory reference instance</returns>
		public static DirectoryReference? ReadDirectoryReference(this BinaryArchiveReader reader)
		{
			string? fullName = reader.ReadString();
			return fullName == null ? null : new(fullName, DirectoryReference.Sanitize.None);
		}

		/// <summary>
		/// Reads a directory reference from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New directory reference instance</returns>
		public static DirectoryReference ReadDirectoryReferenceNotNull(this BinaryArchiveReader reader) => BinaryArchiveReader.NotNull(ReadDirectoryReference(reader));
	}
}
