// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility functions for manipulating directories
	/// </summary>
	public static class DirectoryUtils
	{
		/// <summary>
		/// Finds the on-disk case of a a directory
		/// </summary>
		/// <param name="info">DirectoryInfo instance describing the directory</param>
		/// <returns>New DirectoryInfo instance that represents the directory with the correct case</returns>
		public static DirectoryInfo FindCorrectCase(DirectoryInfo info)
		{
			DirectoryInfo? parentInfo = info.Parent;
			if (parentInfo == null)
			{
				string fullName = info.FullName;
				if (fullName.Length >= 2 && (fullName[0] >= 'a' && fullName[0] <= 'z') && fullName[1] == ':')
				{
					return new DirectoryInfo(Char.ToUpper(fullName[0]) + fullName.Substring(1));
				}
				else
				{
					return info;
				}
			}
			else
			{
				parentInfo = FindCorrectCase(parentInfo);
				if (info.Exists)
				{
					try
					{
						foreach (DirectoryInfo childInfo in parentInfo.EnumerateDirectories())
						{
							if (String.Equals(childInfo.Name, info.Name, FileSystemReference.Comparison))
							{
								return childInfo;
							}
						}
					}
					catch (Exception)
					{
						// System.Security.SecurityException is expected here if we try to enumerate a directory
						// we don't have permission to read, such as another user's mounted volume.
					}
				}

				return new DirectoryInfo(Path.Combine(parentInfo.FullName, info.Name));
			}
		}
	}
}
