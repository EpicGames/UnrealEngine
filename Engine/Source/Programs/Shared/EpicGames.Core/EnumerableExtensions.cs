// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for <see cref="IEnumerable{T}"/>
	/// </summary>
	public static class EnumerableExtensions
	{
		/// <summary>
		/// Dispose all elements in a sequence
		/// </summary>
		/// <param name="sequence">Sequence of elements to dispose</param>
		public static void DisposeAll(this IEnumerable<IDisposable> sequence)
		{
			foreach (IDisposable element in sequence)
			{
				element.Dispose();
			}
		}

		/// <summary>
		/// Split the sequence into batches of at most the given size
		/// </summary>
		/// <typeparam name="TElement">The element type</typeparam>
		/// <param name="sequence">Sequence to split into batches</param>
		/// <param name="batchSize">Maximum size of each batch</param>
		/// <returns>Sequence of batches</returns>
		public static IEnumerable<IReadOnlyList<TElement>> Batch<TElement>(this IEnumerable<TElement> sequence, int batchSize)
		{
			List<TElement> elements = new(batchSize);
			foreach (TElement element in sequence)
			{
				elements.Add(element);
				if (elements.Count == batchSize)
				{
					yield return elements;
					elements.Clear();
				}
			}
			if (elements.Count > 0)
			{
				yield return elements;
			}
		}

		/// <summary>
		/// Finds the minimum element by a given field
		/// </summary>
		/// <typeparam name="TElement"></typeparam>
		/// <param name="sequence"></param>
		/// <param name="selector"></param>
		/// <returns></returns>
		public static TElement MinBy<TElement>(this IEnumerable<TElement> sequence, Func<TElement, int> selector)
		{
			IEnumerator<TElement> enumerator = sequence.GetEnumerator();
			if (!enumerator.MoveNext())
			{
				throw new Exception("Collection is empty");
			}

			TElement minElement = enumerator.Current;

			int minValue = selector(minElement);
			while (enumerator.MoveNext())
			{
				int value = selector(enumerator.Current);
				if (value < minValue)
				{
					minElement = enumerator.Current;
				}
			}

			return minElement;
		}

		/// <summary>
		/// Finds the maximum element by a given field
		/// </summary>
		/// <typeparam name="TElement"></typeparam>
		/// <param name="sequence"></param>
		/// <param name="selector"></param>
		/// <returns></returns>
		public static TElement MaxBy<TElement>(this IEnumerable<TElement> sequence, Func<TElement, int> selector)
		{
			IEnumerator<TElement> enumerator = sequence.GetEnumerator();
			if (!enumerator.MoveNext())
			{
				throw new Exception("Collection is empty");
			}

			TElement maxElement = enumerator.Current;

			int maxValue = selector(maxElement);
			while (enumerator.MoveNext())
			{
				int value = selector(enumerator.Current);
				if (value > maxValue)
				{
					maxElement = enumerator.Current;
				}
			}

			return maxElement;
		}

		/// <summary>
		/// Zips two dictionaries, returning a sequence of keys with the old and new values (or null if they have been removed).
		/// </summary>
		/// <typeparam name="TKey">Common key between the two dictionaries</typeparam>
		/// <typeparam name="TOldValue">Type of values in the first dictionary</typeparam>
		/// <typeparam name="TNewValue">Type of values in the second dictionary</typeparam>
		/// <param name="oldDictionary">The first dictionary</param>
		/// <param name="newDictionary">The second dictionary</param>
		/// <returns>Sequence of key/value pairs from each dictionary</returns>
		public static IEnumerable<(TKey, TOldValue?, TNewValue?)> Zip<TKey, TOldValue, TNewValue>(this IReadOnlyDictionary<TKey, TOldValue>? oldDictionary, IReadOnlyDictionary<TKey, TNewValue>? newDictionary)
			where TKey : notnull
			where TOldValue : class
			where TNewValue : class
		{
			if (newDictionary != null)
			{
				foreach ((TKey key, TNewValue newValue) in newDictionary)
				{
					TOldValue? oldValue;
					if (oldDictionary == null || !oldDictionary.TryGetValue(key, out oldValue))
					{
						yield return (key, null, newValue);
					}
					else
					{
						yield return (key, oldValue, newValue);
					}
				}
			}
			if (oldDictionary != null)
			{
				foreach ((TKey key, TOldValue oldValue) in oldDictionary)
				{
					if (newDictionary == null || !newDictionary.ContainsKey(key))
					{
						yield return (key, oldValue, null);
					}
				}
			}
		}
	}
}
