<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="JetBrains.Annotations" Version="2024.3.0" />
    <PackageReference Include="Microsoft.Extensions.Configuration" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http.Polly" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging.Console" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.ObjectPool" Version="10.0.6" />
    <PackageReference Include="OpenTracing" Version="0.12.1" />
  </ItemGroup>
</Project>
