// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Methods for adding context information to exceptions
	/// </summary>
	public static class ExceptionUtils
	{
		/// <summary>
		/// Unique key name for adding context to exceptions thrown inside Epic apps
		/// </summary>
		const string ContextEntryName = "EpicGames.Context";

		/// <summary>
		/// Adds a context message to a list stored on the given exception. Intended to be used in situations where supplying additional context
		/// for an exception is valuable, but wrapping it would remove information.
		/// </summary>
		/// <param name="ex">The exception that was thrown</param>
		/// <param name="message">Message to append to the context list</param>
		public static void AddContext(Exception ex, string message)
		{
			List<string>? messages = ex.Data[ContextEntryName] as List<string>;
			if (messages == null)
			{
				messages = [];
				ex.Data[ContextEntryName] = messages;
			}
			messages.Add(message);
		}

		/// <summary>
		/// Adds a context message to a list stored on the given exception. Intended to be used in situations where supplying additional context
		/// for an exception is valuable, but wrapping it would remove information.
		/// </summary>
		/// <param name="ex">The exception that was thrown</param>
		/// <param name="format">Formatting string for </param>
		/// <param name="args">Message to append to the context list</param>
		public static void AddContext(Exception ex, string format, params object[] args)
		{
			AddContext(ex, String.Format(format, args));
		}

		/// <summary>
		/// Enumerates the context lines from the given exception
		/// </summary>
		/// <param name="ex">The exception to retrieve context from</param>
		/// <returns>Sequence of context lines</returns>
		public static IEnumerable<string> GetContext(Exception ex)
		{
			List<string>? messages = ex.Data[ContextEntryName] as List<string>;
			if (messages != null)
			{
				foreach (string message in messages)
				{
					yield return message;
				}
			}
		}

		/// <summary>
		/// Formats an exception for display in the log, including additional lines of context that were attached to it.
		/// </summary>
		/// <param name="ex">The exception to format</param>
		/// <returns>String containing the exception information. May be multiple lines.</returns>
		public static string FormatException(Exception ex)
		{
			StringBuilder errorMessage = new();
			if (ex is AggregateException exAgg)
			{
				Exception? innerException = exAgg.InnerException;
				if (innerException != null)
				{
					errorMessage.Append(innerException.ToString());
					foreach (string line in GetContext(innerException))
					{
						errorMessage.AppendFormat("\n  {0}", line);
					}
				}
			}
			else
			{
				errorMessage.Append(ex.ToString());
			}
			foreach (string line in GetContext(ex))
			{
				errorMessage.AppendFormat("\n{0}", line);
			}
			return errorMessage.Replace("\r\n", "\n").ToString();
		}

		/// <summary>
		/// Formats a detailed information about where an exception occurs, including any inner exceptions
		/// </summary>
		/// <param name="ex">The exception to format</param>
		/// <returns>String containing the exception information. May be multiple lines.</returns>
		public static string FormatExceptionDetails(Exception ex)
		{
			List<Exception> exceptionStack = [];
			for (Exception? currentEx = ex; currentEx != null; currentEx = currentEx.InnerException)
			{
				exceptionStack.Add(currentEx);
			}

			StringBuilder message = new();
			for (int idx = exceptionStack.Count - 1; idx >= 0; idx--)
			{
				Exception currentEx = exceptionStack[idx];
				message.AppendFormat("{0}{1}: {2}\n{3}", (idx == exceptionStack.Count - 1) ? "" : "Wrapped by ", currentEx.GetType().Name, currentEx.Message, currentEx.StackTrace);

				if (currentEx.Data.Count > 0)
				{
					foreach (object? key in currentEx.Data.Keys)
					{
						if (key == null)
						{
							continue;
						}

						object? value = currentEx.Data[key];
						if (value == null)
						{
							continue;
						}

						string valueString;
						if (value is List<string> valueList)
						{
							valueString = String.Format("({0})", String.Join(", ", valueList.Select(x => String.Format("\"{0}\"", x))));
						}
						else
						{
							valueString = value.ToString() ?? String.Empty;
						}

						message.AppendFormat("   data: {0} = {1}", key, valueString);
					}
				}
			}
			return message.Replace("\r\n", "\n").ToString();
		}
	}
}
