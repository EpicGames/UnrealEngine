// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Base class for user-facing fatal errors. These errors are shown to the user prior to termination, without a callstack, and may dictate the program exit code.
	/// </summary>
	public class FatalErrorException : Exception
	{
		/// <summary>
		/// Exit code for the process
		/// </summary>
		public int ExitCode { get; set; } = 1;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">The error message to display.</param>
		public FatalErrorException(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="innerException">An inner exception to wrap</param>
		/// <param name="message">The error message to display.</param>
		public FatalErrorException(Exception innerException, string message)
			: base(message, innerException)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">Formatting string for the error message</param>
		/// <param name="arguments">Arguments for the formatting string</param>
		public FatalErrorException(string format, params object[] arguments)
			: base(String.Format(format, arguments))
		{
		}

		/// <summary>
		/// Constructor which wraps another exception
		/// </summary>
		/// <param name="innerException">The inner exception being wrapped</param>
		/// <param name="format">Format for the message string</param>
		/// <param name="arguments">Format arguments</param>
		public FatalErrorException(Exception innerException, string format, params object[] arguments)
			: base(String.Format(format, arguments), innerException)
		{
		}

		/// <summary>
		/// Returns the string representing the exception. Our build exceptions do not show the callstack since they are used to report known error conditions.
		/// </summary>
		/// <returns>Message for the exception</returns>
		public override string ToString()
		{
			return Message;
		}
	}
}
