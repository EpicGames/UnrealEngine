// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;

namespace EpicGames.Core
{
	/// <summary>
	/// Indicates whether files which match a pattern should be included or excluded
	/// </summary>
	public enum FileFilterType
	{
		/// <summary>
		/// Include files matching this pattern
		/// </summary>
		Include,

		/// <summary>
		/// Exclude files matching this pattern
		/// </summary>
		Exclude,
	}

	/// <summary>
	/// Stores a set of rules, similar to p4 path specifications, which can be used to efficiently filter files in or out of a set.
	/// The rules are merged into a tree with nodes for each path fragments and rule numbers in each leaf, allowing tree traversal in an arbitrary order 
	/// to how they are applied.
	/// </summary>
	[TypeConverter(typeof(FileFilterConverter))]
	public class FileFilter
	{
		/// <summary>
		/// Root node for the tree.
		/// </summary>
		readonly FileFilterNode _rootNode;

		/// <summary>
		/// The default node, which will match any path
		/// </summary>
		readonly FileFilterNode _defaultNode;

		/// <summary>
		/// Terminating nodes for each rule added to the filter
		/// </summary>
		readonly List<FileFilterNode> _rules = [];

		/// <summary>
		/// Default constructor
		/// </summary>
		public FileFilter(FileFilterType defaultType = FileFilterType.Exclude)
		{
			_rootNode = new FileFilterNode(null, "");

			_defaultNode = new FileFilterNode(_rootNode, "...");
			_defaultNode.Type = defaultType;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">Filter to copy from</param>
		public FileFilter(FileFilter other)
			: this(other._defaultNode.Type)
		{
			foreach (FileFilterNode otherRule in other._rules)
			{
				AddRule(otherRule.ToString(), otherRule.Type);
			}
		}

		/// <summary>
		/// Constructs a file filter from a p4-style filespec. Exclude lines are prefixed with a - character.
		/// </summary>
		/// <param name="lines">Lines to parse rules from</param>
		/// <param name="defaultType">Default behavior for lines not matched by the filter</param>
		public FileFilter(IEnumerable<string> lines, FileFilterType defaultType = FileFilterType.Exclude)
			: this(defaultType)
		{
			foreach (string line in lines)
			{
				AddRule(line);
			}
		}

		/// <summary>
		/// Parse a filter pattern, determining the default rule type from the initial rule
		/// </summary>
		/// <param name="text">Sequence of patterns to parse. Entries may be separated by semicolons.</param>
		public static FileFilter? Parse(string? text)
		{
			FileFilter? filter = null;
			if (text != null)
			{
				string[] rules = text.Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);
				filter = Parse(rules);
			}
			return filter;
		}

		/// <summary>
		/// Parse a filter pattern, determining the default rule type from the initial rule
		/// </summary>
		/// <param name="rules">Array of rules</param>
		public static FileFilter? Parse(IReadOnlyList<string>? rules)
		{
			FileFilter? filter = null;
			if (rules != null && rules.Count > 0)
			{
				FileFilterType defaultType = rules[0].StartsWith('-') ? FileFilterType.Include : FileFilterType.Exclude;
				filter = new FileFilter(defaultType);
				filter.AddRules(rules);
			}
			return filter;
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter
		/// </summary>
		/// <param name="rule">Pattern to match. See CreateRegex() for details.</param>
		public void AddRule(string rule)
		{
			if (rule.StartsWith('-'))
			{
				Exclude(rule.Substring(1).TrimStart());
			}
			else
			{
				Include(rule);
			}
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter. The rule may be 
		/// </summary>
		/// <param name="rule">Pattern to match. See CreateRegex() for details.</param>
		/// <param name="allowTags">If the rule starts with a list of tags {a,b,c}, specifies a list of tags to match against</param>
		public void AddRule(string rule, params string[] allowTags)
		{
			string cleanRule = rule.Trim();
			if (cleanRule.StartsWith('{'))
			{
				// Find the end of the condition
				int conditionEnd = cleanRule.IndexOf('}', StringComparison.Ordinal);
				if (conditionEnd == -1)
				{
					throw new Exception(String.Format("Missing closing parenthesis in rule: {0}", cleanRule));
				}

				// Check there's a matching tag
				string[] ruleTags = [.. cleanRule.Substring(1, conditionEnd - 1).Split(',').Select(x => x.Trim())];
				if (!ruleTags.Any(x => allowTags.Contains(x)))
				{
					return;
				}

				// Strip the condition from the rule
				cleanRule = cleanRule.Substring(conditionEnd + 1).TrimStart();
			}
			AddRule(cleanRule);
		}

		/// <summary>
		/// Adds several rules to the filter
		/// </summary>
		/// <param name="rules">List of patterns to match.</param>
		public void AddRules(IEnumerable<string> rules)
		{
			foreach (string rule in rules)
			{
				AddRule(rule);
			}
		}

		/// <summary>
		/// Adds several rules to the filter
		/// </summary>
		/// <param name="rules">List of patterns to match.</param>
		/// <param name="type">Whether the rules are include or exclude rules</param>
		public void AddRules(IEnumerable<string> rules, FileFilterType type)
		{
			foreach (string rule in rules)
			{
				AddRule(rule, type);
			}
		}

		/// <summary>
		/// Adds several rules in the given lines. Rules may be prefixed with conditions of the syntax {Key=Value, Key2=Value2}, which
		/// will be evaluated using variables in the given dictionary before being added.
		/// </summary>
		/// <param name="rules"></param>
		/// <param name="tags">Lookup for variables to test against</param>
		public void AddRules(IEnumerable<string> rules, params string[] tags)
		{
			foreach (string rule in rules)
			{
				AddRule(rule, tags);
			}
		}

		/// <summary>
		/// Adds a rule which matches a filename relative to a given base directory.
		/// </summary>
		/// <param name="file">The filename to add a rule for</param>
		/// <param name="baseDirectory">Base directory for the rule</param>
		/// <param name="type">Whether to add an include or exclude rule</param>
		public void AddRuleForFile(FileReference file, DirectoryReference baseDirectory, FileFilterType type)
		{
			AddRule("/" + file.MakeRelativeTo(baseDirectory), type);
		}

		/// <summary>
		/// Adds rules for files which match the given names
		/// </summary>
		/// <param name="files">The filenames to rules for</param>
		/// <param name="baseDirectory">Base directory for the rules</param>
		/// <param name="type">Whether to add an include or exclude rule</param>
		public void AddRuleForFiles(IEnumerable<FileReference> files, DirectoryReference baseDirectory, FileFilterType type)
		{
			foreach (FileReference file in files)
			{
				AddRuleForFile(file, baseDirectory, type);
			}
		}

		/// <summary>
		/// Reads a configuration file split into sections
		/// </summary>
		/// <param name="fileName"></param>
		/// <param name="sectionName"></param>
		/// <param name="allowTags"></param>
		public void ReadRulesFromFile(FileReference fileName, string sectionName, params string[] allowTags)
		{
			bool bInSection = false;
			foreach (string line in File.ReadAllLines(fileName.FullName))
			{
				string trimLine = line.Trim();
				if (!trimLine.StartsWith(';') && trimLine.Length > 0)
				{
					if (trimLine.StartsWith('['))
					{
						bInSection = (trimLine == "[" + sectionName + "]");
					}
					else if (bInSection)
					{
						AddRule(line, allowTags);
					}
				}
			}
		}

		/// <summary>
		/// Adds an include rule to the filter
		/// </summary>
		/// <param name="pattern">Pattern to match. See CreateRegex() for details.</param>
		public void Include(string pattern)
		{
			AddRule(pattern, FileFilterType.Include);
		}

		/// <summary>
		/// Adds several exclude rules to the filter
		/// </summary>
		/// <param name="patterns">Patterns to match. See CreateRegex() for details.</param>
		public void Include(IEnumerable<string> patterns)
		{
			foreach (string pattern in patterns)
			{
				Include(pattern);
			}
		}

		/// <summary>
		/// Adds an exclude rule to the filter
		/// </summary>
		/// <param name="pattern">Mask to match. See CreateRegex() for details.</param>
		public void Exclude(string pattern)
		{
			AddRule(pattern, FileFilterType.Exclude);
		}

		/// <summary>
		/// Adds several exclude rules to the filter
		/// </summary>
		/// <param name="patterns">Patterns to match. See CreateRegex() for details.</param>
		public void Exclude(IEnumerable<string> patterns)
		{
			foreach (string pattern in patterns)
			{
				Exclude(pattern);
			}
		}

		/// <summary>
		/// Normalizes a path to use forward slashes as a path separator
		/// </summary>
		/// <param name="path">Path to process</param>
		/// <returns>Normalized path</returns>
		public static string NormalizePath(string path)
		{
			Match match = Regex.Match(path, @"^\\\\[^\\/]+\\[^\\/]+");
			if (match.Success)
			{
				return match.Groups[0].Value + path.Substring(match.Groups[0].Length).Replace('\\', '/');
			}
			else
			{
				return path.Replace('\\', '/');
			}
		}

		/// <summary>
		/// Adds an include or exclude rule to the filter
		/// </summary>
		/// <param name="pattern">The pattern which the rule should match</param>
		/// <param name="type">Whether to include or exclude files matching this rule</param>
		public void AddRule(string pattern, FileFilterType type)
		{
			string normalizedPattern = NormalizePath(pattern);

			// Remove the slash from the start of the pattern. Any exclude pattern that doesn't contain a directory separator is assumed to apply to any directory (eg. *.cpp), otherwise it's 
			// taken relative to the root.
			if (normalizedPattern.StartsWith('/'))
			{
				normalizedPattern = normalizedPattern.Substring(1);
			}
			else if (!normalizedPattern.Contains('/', StringComparison.Ordinal) && !normalizedPattern.StartsWith("...", StringComparison.Ordinal))
			{
				normalizedPattern = ".../" + normalizedPattern;
			}

			// All directories indicate a wildcard match
			if (normalizedPattern.EndsWith('/'))
			{
				normalizedPattern += "...";
			}

			// Replace any directory wildcards mid-string
			for (int idx = normalizedPattern.IndexOf("...", StringComparison.Ordinal); idx != -1; idx = normalizedPattern.IndexOf("...", idx, StringComparison.Ordinal))
			{
				if (idx > 0 && normalizedPattern[idx - 1] != '/')
				{
					normalizedPattern = normalizedPattern.Insert(idx, "*/");
					idx++;
				}

				idx += 3;

				if (idx < normalizedPattern.Length && normalizedPattern[idx] != '/')
				{
					normalizedPattern = normalizedPattern.Insert(idx, "/*");
					idx += 2;
				}
			}

			// Split the pattern into fragments
			string[] branchPatterns = normalizedPattern.Split('/');

			// Add it into the tree
			FileFilterNode lastNode = _rootNode;
			foreach (string branchPattern in branchPatterns)
			{
				FileFilterNode? nextNode = lastNode.Branches.FirstOrDefault(x => x.Pattern == branchPattern);
				if (nextNode == null)
				{
					nextNode = new FileFilterNode(lastNode, branchPattern);
					lastNode.Branches.Add(nextNode);
				}
				lastNode = nextNode;
			}

			// We've reached the end of the pattern, so mark it as a leaf node
			_rules.Add(lastNode);
			lastNode.RuleNumber = _rules.Count - 1;
			lastNode.Type = type;

			// Update the maximums along that path
			for (FileFilterNode? updateNode = lastNode; updateNode != null; updateNode = updateNode.Parent)
			{
				if (type == FileFilterType.Include)
				{
					updateNode.MaxIncludeRuleNumber = lastNode.RuleNumber;
				}
				else
				{
					updateNode.MaxExcludeRuleNumber = lastNode.RuleNumber;
				}
			}
		}

		/// <summary>
		/// Determines whether the given file matches the filter
		/// </summary>
		/// <param name="fileName">File to match</param>
		/// <returns>True if the file passes the filter</returns>
		public bool Matches(string fileName)
		{
			string[] tokens = NormalizePath(fileName).TrimStart('/').Split('/');

			FileFilterNode matchingNode = FindMatchingNode(_rootNode, tokens, 0, _defaultNode);

			return matchingNode.Type == FileFilterType.Include;
		}

		/// <summary>
		/// Determines whether it's possible for anything within the given folder name to match the filter. Useful to early out of recursive file searches.
		/// </summary>
		/// <param name="folderName">File to match</param>
		/// <returns>True if the file passes the filter</returns>
		public bool PossiblyMatches(string folderName)
		{
			string[] tokens = NormalizePath(folderName).Trim('/').Split('/');

			FileFilterNode matchingNode = FindMatchingNode(_rootNode, [.. tokens.Union([""])], 0, _defaultNode);

			return matchingNode.Type == FileFilterType.Include || HighestPossibleIncludeMatch(_rootNode, tokens, 0, matchingNode.RuleNumber) > matchingNode.RuleNumber;
		}

		/// <summary>
		/// Applies the filter to each element in a sequence, and returns the list of files that match
		/// </summary>
		/// <param name="fileNames">List of filenames</param>
		/// <returns>List of filenames which match the filter</returns>
		public IEnumerable<string> ApplyTo(IEnumerable<string> fileNames)
		{
			return fileNames.Where(x => Matches(x));
		}

		/// <summary>
		/// Applies the filter to each element in a sequence, and returns the list of files that match
		/// </summary>
		/// <param name="baseDirectory">The base directory to match files against</param>
		/// <param name="fileNames">List of filenames</param>
		/// <returns>List of filenames which match the filter</returns>
		public IEnumerable<FileReference> ApplyTo(DirectoryReference baseDirectory, IEnumerable<FileReference> fileNames)
		{
			return fileNames.Where(x => Matches(x.MakeRelativeTo(baseDirectory)));
		}

		/// <summary>
		/// Finds a list of files within a given directory which match the filter.
		/// </summary>
		/// <param name="directoryName">File to match</param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks in the output</param>
		/// <returns>List of files that pass the filter</returns>
		public List<FileReference> ApplyToDirectory(DirectoryReference directoryName, bool bIgnoreSymlinks)
			=> ApplyToDirectory(directoryName.ToDirectoryInfo(), bIgnoreSymlinks).ConvertAll(x => new FileReference(x));

		/// <summary>
		/// Finds a list of files within a given directory which match the filter.
		/// </summary>
		/// <param name="directoryInfo">File to match</param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks in the output</param>
		/// <returns>List of files that pass the filter</returns>
		public List<FileInfo> ApplyToDirectory(DirectoryInfo directoryInfo, bool bIgnoreSymlinks)
		{
			List<FileInfo> matchingFileNames = [];
			if (directoryInfo.Exists)
			{
				FindMatchesFromDirectory(directoryInfo, "", bIgnoreSymlinks, matchingFileNames);
			}
			return matchingFileNames;
		}

		/// <summary>
		/// Finds a list of files within a given directory which match the filter.
		/// </summary>
		/// <param name="directoryName">File to match</param>
		/// <param name="prefixPath"></param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks in the output</param>
		/// <returns>List of files that pass the filter</returns>
		public List<FileReference> ApplyToDirectory(DirectoryReference directoryName, string prefixPath, bool bIgnoreSymlinks)
			=> ApplyToDirectory(directoryName.ToDirectoryInfo(), prefixPath, bIgnoreSymlinks).ConvertAll(x => new FileReference(x));

		/// <summary>
		/// Finds a list of files within a given directory which match the filter.
		/// </summary>
		/// <param name="directoryInfo">File to match</param>
		/// <param name="prefixPath"></param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks in the output</param>
		/// <returns>List of files that pass the filter</returns>
		public List<FileInfo> ApplyToDirectory(DirectoryInfo directoryInfo, string prefixPath, bool bIgnoreSymlinks)
		{
			List<FileInfo> matchingFileNames = [];
			if (directoryInfo.Exists)
			{
				FindMatchesFromDirectory(directoryInfo, NormalizePath(prefixPath).TrimEnd('/') + "/", bIgnoreSymlinks, matchingFileNames);
			}
			return matchingFileNames;
		}

		/// <summary>
		/// Checks whether the given pattern contains a supported wildcard. Useful for distinguishing explicit file references from opportunistic file references.
		/// </summary>
		/// <param name="pattern">The pattern to check</param>
		/// <returns>True if the pattern contains a wildcard (?, *, ...), false otherwise.</returns>
		public static int FindWildcardIndex(string pattern)
		{
			int result = -1;
			for (int idx = 0; idx < pattern.Length; idx++)
			{
				if (pattern[idx] == '?' || pattern[idx] == '*' || (pattern[idx] == '.' && idx + 2 < pattern.Length && pattern[idx + 1] == '.' && pattern[idx + 2] == '.'))
				{
					result = idx;
					break;
				}
			}
			return result;
		}

		/// <summary>
		/// Resolve an individual wildcard to a set of files
		/// </summary>
		/// <param name="pattern">The pattern to resolve</param>
		/// <returns>List of files matching the wildcard</returns>
		public static List<FileReference> ResolveWildcard(string pattern)
		{
			for (int idx = FindWildcardIndex(pattern); idx > 0; idx--)
			{
				if (pattern[idx] == '/' || pattern[idx] == '\\')
				{
					return ResolveWildcard(new DirectoryReference(pattern.Substring(0, idx)), pattern.Substring(idx + 1));
				}
			}
			return ResolveWildcard(DirectoryReference.GetCurrentDirectory(), pattern);
		}

		/// <summary>
		/// Resolve an individual wildcard to a set of files
		/// </summary>
		/// <param name="baseDir">Base directory for wildcards</param>
		/// <param name="pattern">The pattern to resolve</param>
		/// <returns>List of files matching the wildcard</returns>
		public static List<FileReference> ResolveWildcard(DirectoryReference baseDir, string pattern)
		{
			FileFilter filter = new(FileFilterType.Exclude);
			filter.AddRule(pattern);
			return filter.ApplyToDirectory(baseDir, true);
		}

		/// <summary>
		/// Finds a list of files within a given directory which match the filter.
		/// </summary>
		/// <param name="currentDirectory">The current directory</param>
		/// <param name="namePrefix">Current relative path prefix in the traversed directory tree</param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks in the output</param>
		/// <param name="matchingFileNames">Receives a list of matching files</param>
		/// <returns>True if the file passes the filter</returns>
		void FindMatchesFromDirectory(DirectoryInfo currentDirectory, string namePrefix, bool bIgnoreSymlinks, List<FileInfo> matchingFileNames)
		{
			foreach (FileInfo nextFile in currentDirectory.EnumerateFiles())
			{
				string fileName = namePrefix + nextFile.Name;
				if (Matches(fileName) && (!bIgnoreSymlinks || !nextFile.Attributes.HasFlag(FileAttributes.ReparsePoint)))
				{
					matchingFileNames.Add(nextFile);
				}
			}
			foreach (DirectoryInfo nextDirectory in currentDirectory.EnumerateDirectories())
			{
				string nextNamePrefix = namePrefix + nextDirectory.Name;
				if (PossiblyMatches(nextNamePrefix))
				{
					FindMatchesFromDirectory(nextDirectory, nextNamePrefix + "/", bIgnoreSymlinks, matchingFileNames);
				}
			}
		}

		/// <summary>
		/// Finds the node which matches a given list of tokens.
		/// </summary>
		/// <param name="currentNode"></param>
		/// <param name="tokens"></param>
		/// <param name="tokenIdx"></param>
		/// <param name="currentBestNode"></param>
		/// <returns></returns>
		static FileFilterNode FindMatchingNode(FileFilterNode currentNode, string[] tokens, int tokenIdx, FileFilterNode currentBestNode)
		{
			// If we've matched all the input tokens, check if this rule is better than any other we've seen
			if (tokenIdx == tokens.Length)
			{
				return (currentNode.RuleNumber > currentBestNode.RuleNumber) ? currentNode : currentBestNode;
			}

			// If there is no rule under the current node which is better than the current best node, early out
			if (currentNode.MaxIncludeRuleNumber <= currentBestNode.RuleNumber && currentNode.MaxExcludeRuleNumber <= currentBestNode.RuleNumber)
			{
				return currentBestNode;
			}

			// Test all the branches for one that matches
			FileFilterNode bestNode = currentBestNode;
			foreach (FileFilterNode branch in currentNode.Branches)
			{
				if (branch.Pattern == "...")
				{
					for (int nextTokenIdx = tokens.Length; nextTokenIdx >= tokenIdx; nextTokenIdx--)
					{
						bestNode = FindMatchingNode(branch, tokens, nextTokenIdx, bestNode);
					}
				}
				else
				{
					if (branch.IsMatch(tokens[tokenIdx]))
					{
						bestNode = FindMatchingNode(branch, tokens, tokenIdx + 1, bestNode);
					}
				}
			}
			return bestNode;
		}

		/// <summary>
		/// Returns the highest possible rule number which can match the given list of input tokens, assuming that the list of input tokens is incomplete.
		/// </summary>
		/// <param name="currentNode">The current node being checked</param>
		/// <param name="tokens">The tokens to match</param>
		/// <param name="tokenIdx">Current token index</param>
		/// <param name="currentBestRuleNumber">The highest rule number seen so far. Used to optimize tree traversals.</param>
		/// <returns>New highest rule number</returns>
		static int HighestPossibleIncludeMatch(FileFilterNode currentNode, string[] tokens, int tokenIdx, int currentBestRuleNumber)
		{
			// If we've matched all the input tokens, check if this rule is better than any other we've seen
			if (tokenIdx == tokens.Length)
			{
				return Math.Max(currentBestRuleNumber, currentNode.MaxIncludeRuleNumber);
			}

			// Test all the branches for one that matches
			int bestRuleNumber = currentBestRuleNumber;
			if (currentNode.MaxIncludeRuleNumber > bestRuleNumber)
			{
				foreach (FileFilterNode branch in currentNode.Branches)
				{
					if (branch.Pattern == "...")
					{
						if (branch.MaxIncludeRuleNumber > bestRuleNumber)
						{
							bestRuleNumber = branch.MaxIncludeRuleNumber;
						}
					}
					else
					{
						if (branch.IsMatch(tokens[tokenIdx]))
						{
							bestRuleNumber = HighestPossibleIncludeMatch(branch, tokens, tokenIdx + 1, bestRuleNumber);
						}
					}
				}
			}
			return bestRuleNumber;
		}
	}

	/// <summary>
	/// Type converter for FileFilter objects. Allows parsing filters from command line arguments.
	/// </summary>
	class FileFilterConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
			=> sourceType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return FileFilter.Parse(str);
			}
			return base.ConvertFrom(context, culture, value);
		}
	}
}
