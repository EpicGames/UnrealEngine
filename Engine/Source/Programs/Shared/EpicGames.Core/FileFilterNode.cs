// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Node within a filter tree. Each node matches a single path fragment - a folder or file name, with an include or exclude rule consisting of a sequence of nodes. 
	/// </summary>
	class FileFilterNode
	{
		/// <summary>
		/// Node which this is parented to. Null for the root node.
		/// </summary>
		public readonly FileFilterNode? Parent;

		/// <summary>
		/// Pattern to match for this node. May contain * or ? wildcards.
		/// </summary>
		public readonly string Pattern;

		/// <summary>
		/// Highest include rule number matched by this node or any child nodes.
		/// </summary>
		public int MaxIncludeRuleNumber { get; set; } = -1;

		/// <summary>
		/// Highest exclude rule number matched by this node or any child nodes.
		/// </summary>
		public int MaxExcludeRuleNumber { get; set; } = -1;

		/// <summary>
		/// Child branches of this node, distinct by the the pattern for each.
		/// </summary>
		public List<FileFilterNode> Branches { get; set; } = [];

		/// <summary>
		/// If a path matches the sequence of nodes ending in this node, the number of the rule which added it. -1 if this was not the last node in a rule.
		/// </summary>
		public int RuleNumber { get; set; }

		/// <summary>
		/// Whether a rule terminating in this node should include files or exclude files.
		/// </summary>
		public FileFilterType Type { get; set; }

		/// <summary>
		/// Default constructor.
		/// </summary>
		public FileFilterNode(FileFilterNode? inParent, string inPattern)
		{
			Parent = inParent;
			Pattern = inPattern;
			RuleNumber = -1;
			Type = FileFilterType.Exclude;
		}

		/// <summary>
		/// Determine if the given token matches the pattern in this node
		/// </summary>
		public bool IsMatch(string token)
		{
			if (Pattern.EndsWith('.'))
			{
				return !token.Contains('.', StringComparison.Ordinal) && IsMatch(token, 0, Pattern.Substring(0, Pattern.Length - 1), 0);
			}
			else
			{
				return IsMatch(token, 0, Pattern, 0);
			}
		}

		/// <summary>
		/// Determine if a given token matches a pattern, starting from the given positions within each string.
		/// </summary>
		static bool IsMatch(string token, int tokenIdx, string pattern, int patternIdx)
		{
			for (; ; )
			{
				if (patternIdx == pattern.Length)
				{
					return (tokenIdx == token.Length);
				}
				else if (pattern[patternIdx] == '*')
				{
					for (int nextTokenIdx = token.Length; nextTokenIdx >= tokenIdx; nextTokenIdx--)
					{
						if (IsMatch(token, nextTokenIdx, pattern, patternIdx + 1))
						{
							return true;
						}
					}
					return false;
				}
				else if (tokenIdx < token.Length && (Char.ToLower(token[tokenIdx]) == Char.ToLower(pattern[patternIdx]) || pattern[patternIdx] == '?'))
				{
					tokenIdx++;
					patternIdx++;
				}
				else
				{
					return false;
				}
			}
		}

		/// <summary>
		/// Debugger visualization
		/// </summary>
		/// <returns>Path to this node</returns>
		public override string ToString()
		{
			return (Parent == null) ? "/" : Parent.ToString().TrimEnd('/') + "/" + Pattern;
		}
	}
}
