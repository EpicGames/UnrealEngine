// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.RegularExpressions;

#pragma warning disable CA1045 // Do not pass types by reference

namespace EpicGames.Core
{
	/// <summary>
	/// Invalid file pattern exception
	/// </summary>
	public class FilePatternException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public FilePatternException(string message)
			: base(message)
		{
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return Message;
		}
	}

	/// <summary>
	/// Exception thrown to indicate that a source file is not under the given base directory
	/// </summary>
	public class FilePatternSourceFileNotUnderBaseDirException : FilePatternException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public FilePatternSourceFileNotUnderBaseDirException(string message)
			: base(message)
		{
		}
	}

	/// <summary>
	/// Exception thrown to indicate that a source file is missing
	/// </summary>
	public class FilePatternSourceFileMissingException : FilePatternException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public FilePatternSourceFileMissingException(string message)
			: base(message)
		{
		}
	}

	/// <summary>
	/// Encapsulates a pattern containing the '?', '*', and '...' wildcards.
	/// </summary>
	public class FilePattern
	{
		/// <summary>
		/// Base directory for all matched files
		/// </summary>
		public DirectoryReference BaseDirectory { get; }

		/// <summary>
		/// List of tokens in the pattern. Every second token is a wildcard, other tokens are string fragments. Always has an odd number of elements. Path separators are normalized to the host platform format.
		/// </summary>
		public IReadOnlyList<string> Tokens { get; }

		/// <summary>
		/// Constructs a file pattern which matches a single file
		/// </summary>
		/// <param name="file">Location of the file</param>
		public FilePattern(FileReference file)
		{
			BaseDirectory = file.Directory;
			Tokens = [file.GetFileName()];
		}

		/// <summary>
		/// Constructs a file pattern from the given string, resolving relative paths to the given directory. 
		/// </summary>
		/// <param name="rootDirectory">If a relative path is specified by the pattern, the root directory used to turn it into an absolute path</param>
		/// <param name="pattern">The pattern to match. If the pattern ends with a directory separator, an implicit '...' is appended.</param>
		public FilePattern(DirectoryReference rootDirectory, string pattern)
		{
			// Normalize the path separators
			StringBuilder text = new(pattern);
			if (Path.DirectorySeparatorChar != '\\')
			{
				text.Replace('\\', Path.DirectorySeparatorChar);
			}
			if (Path.DirectorySeparatorChar != '/')
			{
				text.Replace('/', Path.DirectorySeparatorChar);
			}

			// Find the base directory, stopping when we hit a wildcard. The source directory must end with a path specification.
			int baseDirectoryLen = 0;
			for (int idx = 0; idx < text.Length; idx++)
			{
				if (text[idx] == Path.DirectorySeparatorChar)
				{
					baseDirectoryLen = idx + 1;
				}
				else if (text[idx] == '?' || text[idx] == '*' || (idx + 2 < text.Length && text[idx] == '.' && text[idx + 1] == '.' && text[idx + 2] == '.'))
				{
					break;
				}
			}

			// Extract the base directory
			BaseDirectory = DirectoryReference.Combine(rootDirectory, text.ToString(0, baseDirectoryLen));

			// Convert any directory wildcards ("...") into complete directory wildcards ("\\...\\"). We internally treat use "...\\" as the wildcard 
			// token so we can correctly match zero directories. Patterns such as "foo...bar" should require at least one directory separator, so 
			// should be converted to "foo*\\...\\*bar".
			for (int idx = baseDirectoryLen; idx < text.Length; idx++)
			{
				if (text[idx] == '.' && text[idx + 1] == '.' && text[idx + 2] == '.')
				{
					// Insert a directory separator before
					if (idx > baseDirectoryLen && text[idx - 1] != Path.DirectorySeparatorChar)
					{
						text.Insert(idx++, '*');
						text.Insert(idx++, Path.DirectorySeparatorChar);
					}

					// Skip past the ellipsis
					idx += 3;

					// Insert a directory separator after
					if (idx == text.Length || text[idx] != Path.DirectorySeparatorChar)
					{
						text.Insert(idx++, Path.DirectorySeparatorChar);
						text.Insert(idx++, '*');
					}
				}
			}

			// Parse the tokens
			List<string> tokens = [];
			int lastIdx = baseDirectoryLen;
			for (int idx = baseDirectoryLen; idx < text.Length; idx++)
			{
				if (text[idx] == '?' || text[idx] == '*')
				{
					tokens.Add(text.ToString(lastIdx, idx - lastIdx));
					tokens.Add(text.ToString(idx, 1));
					lastIdx = idx + 1;
				}
				else if (idx - 3 >= baseDirectoryLen && text[idx] == Path.DirectorySeparatorChar && text[idx - 1] == '.' && text[idx - 2] == '.' && text[idx - 3] == '.')
				{
					tokens.Add(text.ToString(lastIdx, idx - 3 - lastIdx));
					tokens.Add(text.ToString(idx - 3, 4));
					lastIdx = idx + 1;
				}
			}
			tokens.Add(text.ToString(lastIdx, text.Length - lastIdx));
			Tokens = tokens;
		}

		/// <summary>
		/// A pattern without wildcards may match either a single file or directory based on context. This pattern resolves to the later as necessary, producing a new pattern.
		/// </summary>
		/// <returns>Pattern which matches a directory</returns>
		public FilePattern AsDirectoryPattern()
		{
			if (ContainsWildcards())
			{
				return this;
			}
			else
			{
				StringBuilder pattern = new();
				foreach (string token in Tokens)
				{
					pattern.Append(token);
				}
				if (pattern.Length > 0)
				{
					pattern.Append(Path.DirectorySeparatorChar);
				}
				pattern.Append("...");

				return new FilePattern(BaseDirectory, pattern.ToString());
			}
		}

		/// <summary>
		/// For a pattern that does not contain wildcards, returns the single file location
		/// </summary>
		/// <returns>Location of the referenced file</returns>
		public FileReference GetSingleFile()
		{
			if (Tokens.Count == 1)
			{
				return FileReference.Combine(BaseDirectory, Tokens[0]);
			}
			else
			{
				throw new InvalidOperationException("File pattern does not reference a single file");
			}
		}

		/// <summary>
		/// Checks whether this pattern is explicitly a directory, ie. is terminated with a directory separator
		/// </summary>
		/// <returns>True if the pattern is a directory</returns>
		public bool EndsWithDirectorySeparator()
		{
			string lastToken = Tokens[^1];
			return lastToken.Length > 0 && lastToken[^1] == Path.DirectorySeparatorChar;
		}

		/// <summary>
		/// Determines whether the pattern contains wildcards
		/// </summary>
		/// <returns>True if the pattern contains wildcards, false otherwise.</returns>
		public bool ContainsWildcards()
		{
			return Tokens.Count > 1;
		}

		/// <summary>
		/// Tests whether a pattern is compatible with another pattern (that is, that the number and type of wildcards match)
		/// </summary>
		/// <param name="other">Pattern to compare against</param>
		/// <returns>Whether the patterns are compatible.</returns>
		public bool IsCompatibleWith(FilePattern other)
		{
			// Check there are the same number of tokens in each pattern
			if (Tokens.Count != other.Tokens.Count)
			{
				return false;
			}

			// Check all the wildcard tokens match
			for (int idx = 1; idx < Tokens.Count; idx += 2)
			{
				if (Tokens[idx] != other.Tokens[idx])
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Converts this pattern to a C# regex format string, which matches paths relative to the base directory formatted with native directory separators
		/// </summary>
		/// <returns>The regex pattern</returns>
		public string GetRegexPattern()
		{
			StringBuilder pattern = new("^");
			pattern.Append(Regex.Escape(Tokens[0]));
			for (int idx = 1; idx < Tokens.Count; idx += 2)
			{
				// Append the wildcard expression
				if (Tokens[idx] == "?")
				{
					pattern.Append("([^\\/])");
				}
				else if (Tokens[idx] == "*")
				{
					pattern.Append("([^\\/]*)");
				}
				else
				{
					pattern.AppendFormat("((?:.+{0})?)", Regex.Escape(Path.DirectorySeparatorChar.ToString()));
				}

				// Append the next sequence of characters to match
				pattern.Append(Regex.Escape(Tokens[idx + 1]));
			}
			pattern.Append('$');
			return pattern.ToString();
		}

		/// <summary>
		/// Creates a regex replacement pattern
		/// </summary>
		/// <returns>String representing the regex replacement pattern</returns>
		public string GetRegexReplacementPattern()
		{
			StringBuilder pattern = new();
			for (int idx = 0; ; idx += 2)
			{
				// Append the escaped replacement character
				pattern.Append(Tokens[idx].Replace("$", "$$", StringComparison.Ordinal));

				// Check if we've reached the end of the string
				if (idx == Tokens.Count - 1)
				{
					break;
				}

				// Insert the capture
				pattern.AppendFormat("${0}", (idx / 2) + 1);
			}
			return pattern.ToString();
		}

		/// <summary>
		/// Creates a file mapping between a set of source patterns and a target pattern. All patterns should have a matching order and number of wildcards.
		/// </summary>
		/// <param name="files">Files to use for the mapping</param>
		/// <param name="sourcePattern">List of source patterns</param>
		/// <param name="targetPattern">Matching output pattern</param>
		public static Dictionary<FileReference, FileReference> CreateMapping(HashSet<FileReference>? files, ref FilePattern sourcePattern, ref FilePattern targetPattern)
		{
			// If the source pattern ends in a directory separator, or a set of input files are specified and it doesn't contain wildcards, treat it as a full directory match
			if (sourcePattern.EndsWithDirectorySeparator())
			{
				sourcePattern = new FilePattern(sourcePattern.BaseDirectory, String.Join("", sourcePattern.Tokens) + "...");
			}
			else if (files != null)
			{
				sourcePattern = sourcePattern.AsDirectoryPattern();
			}

			// If we have multiple potential source files, but no wildcards in the output pattern, assume it's a directory and append the pattern from the source.
			if (sourcePattern.ContainsWildcards() && !targetPattern.ContainsWildcards())
			{
				StringBuilder newPattern = new();
				foreach (string token in targetPattern.Tokens)
				{
					newPattern.Append(token);
				}
				if (newPattern.Length > 0 && newPattern[^1] != Path.DirectorySeparatorChar)
				{
					newPattern.Append(Path.DirectorySeparatorChar);
				}
				foreach (string token in sourcePattern.Tokens)
				{
					newPattern.Append(token);
				}
				targetPattern = new FilePattern(targetPattern.BaseDirectory, newPattern.ToString());
			}

			// If the target pattern ends with a directory separator, treat it as a full directory match if it has wildcards, or a copy of the source pattern if not
			if (targetPattern.EndsWithDirectorySeparator())
			{
				targetPattern = new FilePattern(targetPattern.BaseDirectory, String.Join("", targetPattern.Tokens) + "...");
			}

			// Handle the case where source and target pattern are both individual files
			Dictionary<FileReference, FileReference> targetFileToSourceFile = [];
			if (sourcePattern.ContainsWildcards() || targetPattern.ContainsWildcards())
			{
				// Check the two patterns are compatible
				if (!sourcePattern.IsCompatibleWith(targetPattern))
				{
					throw new FilePatternException($"File patterns '{sourcePattern}' and '{targetPattern}' do not have matching wildcards");
				}

				// Create a filter to match the source files
				FileFilter filter = new(FileFilterType.Exclude);
				filter.Include(String.Join("", sourcePattern.Tokens));

				// Apply it to the source directory
				List<FileReference> sourceFiles;
				if (files == null)
				{
					sourceFiles = filter.ApplyToDirectory(sourcePattern.BaseDirectory, true);
				}
				else
				{
					sourceFiles = CheckInputFiles(files, sourcePattern.BaseDirectory);
				}

				// Map them onto output files
				FileReference[] targetFiles = new FileReference[sourceFiles.Count];

				// Get the source and target regexes
				string sourceRegex = sourcePattern.GetRegexPattern();
				string targetRegex = targetPattern.GetRegexReplacementPattern();
				for (int idx = 0; idx < sourceFiles.Count; idx++)
				{
					string sourceRelativePath = sourceFiles[idx].MakeRelativeTo(sourcePattern.BaseDirectory);
					string targetRelativePath = Regex.Replace(sourceRelativePath, sourceRegex, targetRegex);
					targetFiles[idx] = FileReference.Combine(targetPattern.BaseDirectory, targetRelativePath);
				}

				// Add them to the output map
				for (int idx = 0; idx < targetFiles.Length; idx++)
				{
					FileReference? existingSourceFile;
					if (targetFileToSourceFile.TryGetValue(targetFiles[idx], out existingSourceFile) && existingSourceFile != sourceFiles[idx])
					{
						throw new FilePatternException($"Output file '{targetFiles[idx]}' is mapped from '{existingSourceFile}' and '{sourceFiles[idx]}'");
					}
					targetFileToSourceFile[targetFiles[idx]] = sourceFiles[idx];
				}
			}
			else
			{
				// Just copy a single file
				FileReference sourceFile = sourcePattern.GetSingleFile();
				if (FileReference.Exists(sourceFile))
				{
					FileReference targetFile = targetPattern.GetSingleFile();
					targetFileToSourceFile[targetFile] = sourceFile;
				}
				else
				{
					throw new FilePatternSourceFileMissingException($"Source file '{sourceFile}' does not exist");
				}
			}

			// Check that no source file is also destination file
			foreach (FileReference sourceFile in targetFileToSourceFile.Values)
			{
				if (targetFileToSourceFile.ContainsKey(sourceFile))
				{
					throw new FilePatternException($"'{sourceFile}' is listed as a source and target file");
				}
			}

			// Return the map
			return targetFileToSourceFile;
		}

		/// <summary>
		/// Checks that the given input files all exist and are under the given base directory
		/// </summary>
		/// <param name="inputFiles">Input files to check</param>
		/// <param name="baseDirectory">Base directory for files</param>
		/// <returns>List of valid files</returns>
		public static List<FileReference> CheckInputFiles(IEnumerable<FileReference> inputFiles, DirectoryReference baseDirectory)
		{
			List<FileReference> files = [];
			foreach (FileReference inputFile in inputFiles)
			{
				if (!inputFile.IsUnderDirectory(baseDirectory))
				{
					throw new FilePatternSourceFileNotUnderBaseDirException($"Source file '{inputFile}' is not under '{baseDirectory}'");
				}
				else if (!FileReference.Exists(inputFile))
				{
					throw new FilePatternSourceFileMissingException($"Source file '{inputFile}' does not exist");
				}
				else
				{
					files.Add(inputFile);
				}
			}
			return files;
		}

		/// <summary>
		/// Formats the pattern as a string
		/// </summary>
		/// <returns>The original representation of this pattern</returns>
		public override string ToString()
		{
			return BaseDirectory.ToString() + Path.DirectorySeparatorChar + String.Join("", Tokens);
		}
	}
}
