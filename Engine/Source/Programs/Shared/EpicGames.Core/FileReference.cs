// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Representation of an absolute file path. Allows fast hashing and comparisons.
	/// </summary>
	[Serializable]
	[TypeConverter(typeof(FileReferenceTypeConverter))]
	[JsonConverter(typeof(FileReferenceJsonConverter))]
	public class FileReference : FileSystemReference, IEquatable<FileReference>, IComparable<FileReference>
	{
		/// <summary>
		/// Dummy enum to allow invoking the constructor which takes a sanitized full path
		/// </summary>
		public enum Sanitize
		{
			/// <summary>
			/// Dummy value
			/// </summary>
			None
		}

		/// <summary>
		/// Default constructor.
		/// </summary>
		/// <param name="inPath">Path to this file</param>
		public FileReference(string inPath)
			: base(Path.GetFullPath(inPath))
		{
			if (FullName[^1] == '\\' || FullName[^1] == '/')
			{
				throw new ArgumentException("File names may not be terminated by a path separator character");
			}
		}

		/// <summary>
		/// Construct a FileReference from a FileInfo object.
		/// </summary>
		/// <param name="inInfo">Path to this file</param>
		public FileReference(FileInfo inInfo)
			: base(inInfo.FullName)
		{
		}

		/// <summary>
		/// Default constructor.
		/// </summary>
		/// <param name="fullName">The full sanitized path</param>
		/// <param name="sanitize">Sanitize argument. Ignored.</param>
		public FileReference(string fullName, Sanitize sanitize)
			: base(fullName)
		{
			_ = sanitize;
		}

		/// <summary>
		/// Create a FileReference from a string. If the string is null or whitespace, returns a null FileReference.
		/// </summary>
		/// <param name="fileName">FileName for the string</param>
		/// <returns>Returns a FileReference representing the given string, or null.</returns>
		public static FileReference? FromString(string? fileName) => String.IsNullOrWhiteSpace(fileName) ? null : new(fileName);

		/// <summary>
		/// Gets the file name without path information
		/// </summary>
		/// <returns>A string containing the file name</returns>
		public string GetFileName() => Path.GetFileName(FullName);

		/// <summary>
		/// Gets the file name without path information or an extension
		/// </summary>
		/// <returns>A string containing the file name without an extension</returns>
		public string GetFileNameWithoutExtension() => Path.GetFileNameWithoutExtension(FullName);

		/// <summary>
		/// Gets the file name without path or any extensions
		/// </summary>
		/// <returns>A string containing the file name without an extension</returns>
		public string GetFileNameWithoutAnyExtensions()
		{
			int startIdx = FullName.LastIndexOf(Path.DirectorySeparatorChar) + 1;

			int endIdx = FullName.IndexOf('.', startIdx);
			if (endIdx < startIdx)
			{
				return FullName.Substring(startIdx);
			}
			else
			{
				return FullName.Substring(startIdx, endIdx - startIdx);
			}
		}

		/// <summary>
		/// Gets the extension for this filename
		/// </summary>
		/// <returns>A string containing the extension of this filename</returns>
		public string GetExtension() => Path.GetExtension(FullName);

		/// <summary>
		/// Change the file's extension to something else
		/// </summary>
		/// <param name="extension">The new extension</param>
		/// <returns>A FileReference with the same path and name, but with the new extension</returns>
		public FileReference ChangeExtension(string? extension) => new(Path.ChangeExtension(FullName, extension), Sanitize.None);

		/// <summary>
		/// Gets the directory containing this file
		/// </summary>
		/// <returns>A new directory object representing the directory containing this object</returns>
		public DirectoryReference Directory
		{
			get
			{
				int parentLength = FullName.LastIndexOf(Path.DirectorySeparatorChar);

				if (parentLength == 2 && FullName[1] == ':')
				{
					// windows root detected (C:)
					parentLength++;
				}

				if (parentLength == 0 && FullName[0] == Path.DirectorySeparatorChar)
				{
					// nix style root (/) detected
					parentLength = 1;
				}

				return new(FullName.Substring(0, parentLength), DirectoryReference.Sanitize.None);
			}
		}

		/// <summary>
		/// Combine several fragments with a base directory, to form a new filename
		/// </summary>
		/// <param name="baseDirectory">The base directory</param>
		/// <param name="fragments">Fragments to combine with the base directory</param>
		/// <returns>The new file name</returns>
		public static FileReference Combine(DirectoryReference baseDirectory, params string[] fragments) => new(CombineStrings(baseDirectory, fragments), Sanitize.None);

		/// <summary>
		/// Append a string to the end of a filename
		/// </summary>
		/// <param name="a">The base file reference</param>
		/// <param name="b">Suffix to be appended</param>
		/// <returns>The new file reference</returns>
		public static FileReference operator +(FileReference a, string b) => new(a.FullName + b, Sanitize.None);

		/// <summary>
		/// Compares two filesystem object names for equality. Uses the canonical name representation, not the display name representation.
		/// </summary>
		/// <param name="a">First object to compare.</param>
		/// <param name="b">Second object to compare.</param>
		/// <returns>True if the names represent the same object, false otherwise</returns>
		public static bool operator ==(FileReference? a, FileReference? b)
		{
			if (a is null)
			{
				return b is null;
			}
			else if (b is null)
			{
				return false;
			}
			else
			{
				return a.FullName.Equals(b.FullName, Comparison);
			}
		}

		/// <summary>
		/// Compares two filesystem object names for inequality. Uses the canonical name representation, not the display name representation.
		/// </summary>
		/// <param name="a">First object to compare.</param>
		/// <param name="b">Second object to compare.</param>
		/// <returns>False if the names represent the same object, true otherwise</returns>
		public static bool operator !=(FileReference? a, FileReference? b) => !(a == b);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is FileReference file && file == this;

		/// <summary>
		/// Compares against another object for equality.
		/// </summary>
		/// <param name="obj">other instance to compare.</param>
		/// <returns>True if the names represent the same object, false otherwise</returns>
		public bool Equals(FileReference? obj) => obj == this;

		/// <inheritdoc/>
		public override int GetHashCode() => base.GetHashCode();

		/// <inheritdoc/>
		public int CompareTo(FileReference? other) => Comparer.Compare(FullName, other?.FullName);

		/// <inheritdoc/>
		public static bool operator <(FileReference left, FileReference right) => left is null ? right is not null : left.CompareTo(right) < 0;

		/// <inheritdoc/>
		public static bool operator <=(FileReference left, FileReference right) => left is null || left.CompareTo(right) <= 0;

		/// <inheritdoc/>
		public static bool operator >(FileReference left, FileReference right) => left is not null && left.CompareTo(right) > 0;

		/// <inheritdoc/>
		public static bool operator >=(FileReference left, FileReference right) => left is null ? right is null : left.CompareTo(right) >= 0;

		/// <summary>
		/// Helper function to create a remote file reference. Unlike normal FileReference objects, these aren't converted to a full path in the local filesystem, but are
		/// left as they are passed in.
		/// </summary>
		/// <param name="absolutePath">The absolute path in the remote file system</param>
		/// <returns>New file reference</returns>
		public static FileReference MakeRemote(string absolutePath) => new(absolutePath, Sanitize.None);

		/// <summary>
		/// Makes a file location writeable; 
		/// </summary>
		/// <param name="location">Location of the file</param>
		public static void MakeWriteable(FileReference location)
		{
			if (Exists(location))
			{
				FileAttributes attributes = GetAttributes(location);
				if ((attributes & FileAttributes.ReadOnly) != 0)
				{
					SetAttributes(location, attributes & ~FileAttributes.ReadOnly);
				}
			}
		}

		/// <summary>
		/// Get the encoding of a file by checking for BOM bytes
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="skipBytes">Number of bytes to skip</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(FileReference location, out int skipBytes)
		{
			try
			{
				Span<byte> span = stackalloc byte[4];
				using (FileStream stream = Open(location, FileMode.Open, FileAccess.Read, FileShare.Read))
				{
					stream.ReadExactly(span);
				}
				return FileUtils.GetEncoding(span, out skipBytes);
			}
			catch (IOException)
			{
			}
			skipBytes = 0;
			return Encoding.Default;
		}

		/// <summary>
		/// Get the encoding of a file by checking for BOM bytes
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(FileReference location) => GetEncoding(location, out int _);

		/// <summary>
		/// Get the encoding of a file by checking for BOM bytes
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The encoding</returns>
		public static async Task<Encoding> GetEncodingAsync(FileReference location, CancellationToken cancellationToken = default)
		{
			try
			{
				byte[] buffer = new byte[4];
				using (FileStream stream = Open(location, FileMode.Open, FileAccess.Read, FileShare.Read))
				{
					await stream.ReadExactlyAsync(buffer, 0, 4, cancellationToken);
				}
				return FileUtils.GetEncoding(buffer);
			}
			catch (IOException)
			{
			}
			return Encoding.Default;
		}

		/// <summary>
		/// Finds the correct case to match the location of this file on disk. Uses the given case for parts of the path that do not exist.
		/// </summary>
		/// <param name="location">The path to find the correct case for</param>
		/// <returns>Location of the file with the correct case</returns>
		public static FileReference FindCorrectCase(FileReference location) => new(FileUtils.FindCorrectCase(location.ToFileInfo()));

		/// <summary>
		/// Constructs a FileInfo object from this reference
		/// </summary>
		/// <returns>New FileInfo object</returns>
		public FileInfo ToFileInfo() => new(FullName);

		#region System.IO.File methods

		/// <summary>
		/// Copies a file from one location to another
		/// </summary>
		/// <param name="sourceLocation">Location of the source file</param>
		/// <param name="targetLocation">Location of the target file</param>
		public static void Copy(FileReference sourceLocation, FileReference targetLocation) => File.Copy(sourceLocation.FullName, targetLocation.FullName);

		/// <summary>
		/// Copies a file from one location to another
		/// </summary>
		/// <param name="sourceLocation">Location of the source file</param>
		/// <param name="targetLocation">Location of the target file</param>
		/// <param name="bOverwrite">Whether to overwrite the file in the target location</param>
		public static void Copy(FileReference sourceLocation, FileReference targetLocation, bool bOverwrite) => File.Copy(sourceLocation.FullName, targetLocation.FullName, bOverwrite);

		/// <summary>
		/// Deletes this file
		/// </summary>
		public static void Delete(FileReference location) => File.Delete(location.FullName);

		/// <summary>
		/// Determines whether the given filename exists
		/// </summary>
		/// <returns>True if it exists, false otherwise</returns>
		public static bool Exists(FileReference location) => File.Exists(location.FullName);

		/// <summary>
		/// Gets the attributes for a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Attributes for the file</returns>
		public static FileAttributes GetAttributes(FileReference location) => File.GetAttributes(location.FullName);

		/// <summary>
		/// Gets the time that the file was last written to
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Last write time, in local time</returns>
		public static DateTime GetLastWriteTime(FileReference location) => File.GetLastWriteTime(location.FullName);

		/// <summary>
		/// Gets the time that the file was last written to
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Last write time, in UTC time</returns>
		public static DateTime GetLastWriteTimeUtc(FileReference location) => File.GetLastWriteTimeUtc(location.FullName);

		/// <summary>
		/// Moves a file from one location to another
		/// </summary>
		/// <param name="sourceLocation">Location of the source file</param>
		/// <param name="targetLocation">Location of the target file</param>
		public static void Move(FileReference sourceLocation, FileReference targetLocation) => File.Move(sourceLocation.FullName, targetLocation.FullName);

		/// <summary>
		/// Moves a file from one location to another
		/// </summary>
		/// <param name="sourceLocation">Location of the source file</param>
		/// <param name="targetLocation">Location of the target file</param>
		/// <param name="overwrite">Whether to overwrite the file in the target location</param>
		public static void Move(FileReference sourceLocation, FileReference targetLocation, bool overwrite) => File.Move(sourceLocation.FullName, targetLocation.FullName, overwrite);

		/// <summary>
		/// Opens a FileStream on the specified path with read/write access
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="mode">Mode to use when opening the file</param>
		/// <returns>New FileStream for the given file</returns>
		public static FileStream Open(FileReference location, FileMode mode) => File.Open(location.FullName, mode);

		/// <summary>
		/// Opens a FileStream on the specified path
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="mode">Mode to use when opening the file</param>
		/// <param name="access">Sharing mode for the new file</param>
		/// <returns>New FileStream for the given file</returns>
		public static FileStream Open(FileReference location, FileMode mode, FileAccess access) => File.Open(location.FullName, mode, access);

		/// <summary>
		/// Opens a FileStream on the specified path
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="mode">Mode to use when opening the file</param>
		/// <param name="access">Access mode for the new file</param>
		/// <param name="share">Sharing mode for the open file</param>
		/// <returns>New FileStream for the given file</returns>
		public static FileStream Open(FileReference location, FileMode mode, FileAccess access, FileShare share) => File.Open(location.FullName, mode, access, share);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Byte array containing the contents of the file</returns>
		public static byte[] ReadAllBytes(FileReference location) => File.ReadAllBytes(location.FullName);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Byte array containing the contents of the file</returns>
		public static Task<byte[]> ReadAllBytesAsync(FileReference location, CancellationToken cancellationToken = default) => File.ReadAllBytesAsync(location.FullName, cancellationToken);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Contents of the file as a single string</returns>
		public static string ReadAllText(FileReference location) => ReadAllText(location, Encoding.Default);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="encoding">Encoding of the file</param>
		/// <returns>Contents of the file as a single string</returns>
		public static string ReadAllText(FileReference location, Encoding encoding)
		{
			// Try to read the whole file into a buffer created by hand. This avoids a LOT of memory allocations which in turn reduces the
			// GC stress on the system. Removing the StreamReader would be nice in the future.
			using FileStream fs = new(location.FullName, FileMode.Open, FileAccess.Read, FileShare.Read, 4 * 1024, FileOptions.SequentialScan);
			int fileLength = (int)Math.Clamp(fs.Length, 0, Int32.MaxValue);
			if (fileLength == 0)
			{
				return String.Empty;
			}

			// stack allocate the buffer for small files 1k or less to prevent gc
			Span<char> buffer = fileLength <= 1024 ? stackalloc char[fileLength] : new char[fileLength];

			using StreamReader sr = new(fs, encoding);
			int charsRead = sr.Read(buffer);

			return sr.EndOfStream
				? buffer[0..charsRead].ToString()
				: String.Concat(buffer[0..charsRead], sr.ReadToEnd());
		}

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Contents of the file as a single string</returns>
		public static Task<string> ReadAllTextAsync(FileReference location, CancellationToken cancellationToken = default) => ReadAllTextAsync(location, Encoding.Default, cancellationToken);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="encoding">Encoding of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Contents of the file as a single string</returns>
		public static Task<string> ReadAllTextAsync(FileReference location, Encoding encoding, CancellationToken cancellationToken = default) => File.ReadAllTextAsync(location.FullName, encoding, cancellationToken);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>String array containing the contents of the file</returns>
		public static string[] ReadAllLines(FileReference location) => ReadAllLines(location, Encoding.Default);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <returns>String array containing the contents of the file</returns>
		public static string[] ReadAllLines(FileReference location, Encoding encoding) => File.ReadAllLines(location.FullName, encoding);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>String array containing the contents of the file</returns>
		public static Task<string[]> ReadAllLinesAsync(FileReference location, CancellationToken cancellationToken = default) => ReadAllLinesAsync(location, Encoding.Default, cancellationToken);

		/// <summary>
		/// Reads the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>String array containing the contents of the file</returns>
		public static Task<string[]> ReadAllLinesAsync(FileReference location, Encoding encoding, CancellationToken cancellationToken = default) => File.ReadAllLinesAsync(location.FullName, encoding, cancellationToken);

		/// <summary>
		/// Sets the attributes for a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="attributes">New attributes for the file</param>
		public static void SetAttributes(FileReference location, FileAttributes attributes) => File.SetAttributes(location.FullName, attributes);

		/// <summary>
		/// Sets the time that the file was last written to
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="lastWriteTime">Last write time, in local time</param>
		public static void SetLastWriteTime(FileReference location, DateTime lastWriteTime) => File.SetLastWriteTime(location.FullName, lastWriteTime);

		/// <summary>
		/// Sets the time that the file was last written to
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="lastWriteTimeUtc">Last write time, in UTC time</param>
		public static void SetLastWriteTimeUtc(FileReference location, DateTime lastWriteTimeUtc) => File.SetLastWriteTimeUtc(location.FullName, lastWriteTimeUtc);

		/// <summary>
		/// Sets the time that the file was last accessed.
		/// </summary>
		/// <param name="location">Location of the file.</param>
		/// <param name="lastWriteTime">Last access time, in local time.</param>
		public static void SetLastAccessTime(FileReference location, DateTime lastWriteTime) => File.SetLastWriteTime(location.FullName, lastWriteTime);

		/// <summary>
		/// Sets the time that the file was last accessed.
		/// </summary>
		/// <param name="location">Location of the file.</param>
		/// <param name="lastWriteTimeUtc">Last access time, in UTC time.</param>
		public static void SetLastAccessTimeUtc(FileReference location, DateTime lastWriteTimeUtc) => File.SetLastWriteTimeUtc(location.FullName, lastWriteTimeUtc);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static void WriteAllBytes(FileReference location, byte[] contents) => File.WriteAllBytes(location.FullName, contents);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllBytesAsync(FileReference location, byte[] contents, CancellationToken cancellationToken = default) => File.WriteAllBytesAsync(location.FullName, contents, cancellationToken);

		/// <summary>
		/// Writes the data to the given file, if it's different from what's there already.
		/// Returns true if contents were written.
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static bool WriteAllBytesIfDifferent(FileReference location, byte[] contents)
		{
			if (Exists(location))
			{
				ReadOnlySpan<byte> currentContents = ReadAllBytes(location);
				if (contents.AsSpan().SequenceEqual(currentContents))
				{
					return false;
				}
			}
			WriteAllBytes(location, contents);
			return true;
		}

		/// <summary>
		/// Writes the data to the given file, if it's different from what's there already.
		/// Returns true if contents were written.
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<bool> WriteAllBytesIfDifferentAsync(FileReference location, byte[] contents, CancellationToken cancellationToken = default)
		{
			if (Exists(location))
			{
				byte[] currentContents = await ReadAllBytesAsync(location, cancellationToken);
				if (contents.AsSpan().SequenceEqual(currentContents))
				{
					return false;
				}
			}
			await WriteAllBytesAsync(location, contents, cancellationToken);
			return true;
		}

		/// <summary>
		/// Writes the string to the given file, if it's different from what's there already.
		/// Returns true if contents were written.
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static bool WriteAllTextIfDifferent(FileReference location, string contents)
		{
			if (Exists(location))
			{
				string currentContents = ReadAllText(location);
				if (String.Equals(contents, currentContents, StringComparison.Ordinal))
				{
					return false;
				}
			}
			WriteAllText(location, contents);
			return true;
		}

		/// <summary>
		/// Writes the string to the given file, if it's different from what's there already.
		/// Returns true if contents were written.
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<bool> WriteAllTextIfDifferentAsync(FileReference location, string contents, CancellationToken cancellationToken = default)
		{
			if (Exists(location))
			{
				string currentContents = await ReadAllTextAsync(location, cancellationToken);
				if (String.Equals(contents, currentContents, StringComparison.Ordinal))
				{
					return false;
				}
			}
			await WriteAllTextAsync(location, contents, cancellationToken);
			return true;
		}

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static void WriteAllLines(FileReference location, IEnumerable<string> contents) => WriteAllLines(location, contents, Encoding.Default);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void WriteAllLines(FileReference location, IEnumerable<string> contents, Encoding encoding) => File.WriteAllLines(location.FullName, contents, encoding);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static void WriteAllLines(FileReference location, string[] contents) => WriteAllLines(location, contents, Encoding.Default);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void WriteAllLines(FileReference location, string[] contents, Encoding encoding) => File.WriteAllLines(location.FullName, contents, encoding);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllLinesAsync(FileReference location, IEnumerable<string> contents, CancellationToken cancellationToken = default) => WriteAllLinesAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllLinesAsync(FileReference location, IEnumerable<string> contents, Encoding encoding, CancellationToken cancellationToken = default) => File.WriteAllLinesAsync(location.FullName, contents, encoding, cancellationToken);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllLinesAsync(FileReference location, string[] contents, CancellationToken cancellationToken = default) => WriteAllLinesAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllLinesAsync(FileReference location, string[] contents, Encoding encoding, CancellationToken cancellationToken = default) => File.WriteAllLinesAsync(location.FullName, contents, encoding, cancellationToken);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		public static void WriteAllText(FileReference location, string contents) => WriteAllText(location, contents, Encoding.Default);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void WriteAllText(FileReference location, string contents, Encoding encoding) => File.WriteAllText(location.FullName, contents, encoding);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllTextAsync(FileReference location, string contents, CancellationToken cancellationToken = default) => WriteAllTextAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Writes the contents of a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents of the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllTextAsync(FileReference location, string contents, Encoding encoding, CancellationToken cancellationToken = default) => File.WriteAllTextAsync(location.FullName, contents, encoding, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		public static void AppendAllLines(FileReference location, IEnumerable<string> contents) => AppendAllLines(location, contents, Encoding.Default);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void AppendAllLines(FileReference location, IEnumerable<string> contents, Encoding encoding) => File.AppendAllLines(location.FullName, contents, encoding);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllLinesAsync(FileReference location, IEnumerable<string> contents, CancellationToken cancellationToken = default) => AppendAllLinesAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllLinesAsync(FileReference location, IEnumerable<string> contents, Encoding encoding, CancellationToken cancellationToken = default) => File.AppendAllLinesAsync(location.FullName, contents, encoding, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		public static void AppendAllLines(FileReference location, string[] contents) => AppendAllLines(location, contents, Encoding.Default);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void AppendAllLines(FileReference location, string[] contents, Encoding encoding) => File.AppendAllLines(location.FullName, contents, encoding);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllLinesAsync(FileReference location, string[] contents, CancellationToken cancellationToken = default) => AppendAllLinesAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllLinesAsync(FileReference location, string[] contents, Encoding encoding, CancellationToken cancellationToken = default) => File.AppendAllLinesAsync(location.FullName, contents, encoding, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		public static void AppendAllText(FileReference location, string contents) => AppendAllText(location, contents, Encoding.Default);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		public static void AppendAllText(FileReference location, string contents, Encoding encoding) => File.AppendAllText(location.FullName, contents, encoding);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllTextAsync(FileReference location, string contents, CancellationToken cancellationToken = default) => AppendAllTextAsync(location, contents, Encoding.Default, cancellationToken);

		/// <summary>
		/// Appends the contents to a file
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <param name="contents">Contents to append to the file</param>
		/// <param name="encoding">The encoding to use when parsing the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task AppendAllTextAsync(FileReference location, string contents, Encoding encoding, CancellationToken cancellationToken = default) => File.AppendAllTextAsync(location.FullName, contents, encoding, cancellationToken);

		#endregion
	}

	/// <summary>
	/// Type converter to/from strings
	/// </summary>
	class FileReferenceTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || base.CanConvertFrom(context, sourceType);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string stringValue)
			{
				return new FileReference(stringValue);
			}
			return base.ConvertFrom(context, culture, value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			return destinationType == typeof(string) || base.CanConvertTo(context, destinationType);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}

	internal class FileReferenceJsonConverter : JsonConverter<FileReference>
	{
		public override FileReference? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? fullName = reader.GetString();
			return fullName is null ? null : new(fullName);
		}

		public override FileReference ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.PropertyName)
			{
				throw new JsonException("FileReference keys must be represented as JSON strings");
			}
			return new(reader.GetString()!);
		}

		public override void Write(Utf8JsonWriter writer, FileReference value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.FullName);
		}

		public override void WriteAsPropertyName(Utf8JsonWriter writer, [DisallowNull] FileReference value, JsonSerializerOptions options)
		{
			writer.WritePropertyName(value.FullName);
		}
	}

	/// <summary>
	/// Extension methods for FileReference functionality
	/// </summary>
	public static class FileReferenceExtensionMethods
	{
		/// <summary>
		/// Manually serialize a file reference to a binary stream.
		/// </summary>
		/// <param name="writer">Binary writer to write to</param>
		/// <param name="file">The file reference to write</param>
		public static void Write(this BinaryWriter writer, FileReference file) => writer.Write((file == null) ? String.Empty : file.FullName);

		/// <summary>
		/// Serializes a file reference, using a lookup table to avoid serializing the same name more than once.
		/// </summary>
		/// <param name="writer">The writer to save this reference to</param>
		/// <param name="file">A file reference to output; may be null</param>
		/// <param name="fileToUniqueId">A lookup table that caches previous files that have been output, and maps them to unique id's.</param>
		public static void Write(this BinaryWriter writer, FileReference file, Dictionary<FileReference, int> fileToUniqueId)
		{
			int uniqueId;
			if (file == null)
			{
				writer.Write(-1);
			}
			else if (fileToUniqueId.TryGetValue(file, out uniqueId))
			{
				writer.Write(uniqueId);
			}
			else
			{
				writer.Write(fileToUniqueId.Count);
				writer.Write(file);
				fileToUniqueId.Add(file, fileToUniqueId.Count);
			}
		}

		/// <summary>
		/// Manually deserialize a file reference from a binary stream.
		/// </summary>
		/// <param name="reader">Binary reader to read from</param>
		/// <returns>New FileReference object</returns>
		public static FileReference ReadFileReference(this BinaryReader reader) => BinaryArchiveReader.NotNull(ReadFileReferenceOrNull(reader));

		/// <summary>
		/// Manually deserialize a file reference from a binary stream.
		/// </summary>
		/// <param name="reader">Binary reader to read from</param>
		/// <returns>New FileReference object</returns>
		public static FileReference? ReadFileReferenceOrNull(this BinaryReader reader)
		{
			string fullName = reader.ReadString();
			return (fullName.Length == 0) ? null : new(fullName, FileReference.Sanitize.None);
		}

		/// <summary>
		/// Deserializes a file reference, using a lookup table to avoid writing the same name more than once.
		/// </summary>
		/// <param name="reader">The source to read from</param>
		/// <param name="uniqueFiles">List of previously read file references. The index into this array is used in place of subsequent occurrences of the file.</param>
		/// <returns>The file reference that was read</returns>
		public static FileReference ReadFileReference(this BinaryReader reader, List<FileReference> uniqueFiles) => BinaryArchiveReader.NotNull(ReadFileReferenceOrNull(reader, uniqueFiles));

		/// <summary>
		/// Deserializes a file reference, using a lookup table to avoid writing the same name more than once.
		/// </summary>
		/// <param name="reader">The source to read from</param>
		/// <param name="uniqueFiles">List of previously read file references. The index into this array is used in place of subsequent occurrences of the file.</param>
		/// <returns>The file reference that was read</returns>
		public static FileReference? ReadFileReferenceOrNull(this BinaryReader reader, List<FileReference> uniqueFiles)
		{
			int uniqueId = reader.ReadInt32();
			if (uniqueId == -1)
			{
				return null;
			}
			else if (uniqueId < uniqueFiles.Count)
			{
				return uniqueFiles[uniqueId];
			}
			else
			{
				FileReference result = reader.ReadFileReference();
				uniqueFiles.Add(result);
				return result;
			}
		}

		/// <summary>
		/// Writes a FileReference to a binary archive
		/// </summary>
		/// <param name="writer">The writer to output data to</param>
		/// <param name="file">The file reference to write</param>
		public static void WriteFileReference(this BinaryArchiveWriter writer, FileReference? file)
		{
			if (file == null)
			{
				writer.WriteString(null);
			}
			else
			{
				writer.WriteString(file.FullName);
			}
		}

		/// <summary>
		/// Reads a FileReference from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New file reference instance</returns>
		public static FileReference ReadFileReference(this BinaryArchiveReader reader) => BinaryArchiveReader.NotNull(ReadFileReferenceOrNull(reader));

		/// <summary>
		/// Reads a FileReference from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New file reference instance</returns>
		public static FileReference? ReadFileReferenceOrNull(this BinaryArchiveReader reader)
		{
			string? fullName = reader.ReadString();
			return fullName == null ? null : new(fullName, FileReference.Sanitize.None);
		}
	}
}
