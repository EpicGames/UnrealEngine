// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace EpicGames.Core
{
	/// <summary>
	/// Describes a tree of files represented by some arbitrary type. Allows manipulating files/directories in a functional manner; 
	/// filtering a view of a certain directory, mapping files from one location to another, etc... before actually realizing those changes on disk.
	/// </summary>
	public abstract class FileSet : IEnumerable<FileReference>
	{
		/// <summary>
		/// An empty fileset
		/// </summary>
		public static FileSet Empty { get; } = new FileSetFromFiles([]);

		/// <summary>
		/// Path of this tree
		/// </summary>
		public string Path { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Relative path within the tree</param>
		protected FileSet(string path)
		{
			Path = path;
		}

		/// <summary>
		/// Enumerate files in the current tree
		/// </summary>
		/// <returns>Sequence consisting of names and file objects</returns>
		public abstract IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles();

		/// <summary>
		/// Enumerate subtrees in the current tree
		/// </summary>
		/// <returns>Sequence consisting of names and subtree objects</returns>
		public abstract IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories();

		/// <summary>
		/// Creates a file tree from a given set of files
		/// </summary>
		/// <param name="directory">Base directory</param>
		/// <param name="file">File to add</param>
		/// <returns>Tree containing the given files</returns>
		public static FileSet FromFile(DirectoryReference directory, string file)
		{
			return FromFiles([(file, FileReference.Combine(directory, file))]);
		}

		/// <summary>
		/// Creates a file tree from a given set of files
		/// </summary>
		/// <param name="files"></param>
		/// <returns>Tree containing the given files</returns>
		public static FileSet FromFiles(IEnumerable<(string, FileReference)> files)
		{
			return new FileSetFromFiles(files);
		}

		/// <summary>
		/// Creates a file tree from a given set of files
		/// </summary>
		/// <param name="directory">Base directory for the file</param>
		/// <param name="file">File to include</param>
		/// <returns>Tree containing the given files</returns>
		public static FileSet FromFile(DirectoryReference directory, FileReference file)
		{
			return FromFiles(directory, [file]);
		}

		/// <summary>
		/// Creates a file tree from a given file
		/// </summary>
		/// <param name="file">File to add</param>
		/// <returns>Tree containing the given files</returns>
		public static FileSet FromFile(FileReference file)
		{
			return FromFiles([(file.GetFileName(), file)]);
		}

		/// <summary>
		/// Creates a file tree from a given set of files
		/// </summary>
		/// <param name="directory">Base directory for the fileset</param>
		/// <param name="files">Files to include</param>
		/// <returns>Tree containing the given files</returns>
		public static FileSet FromFiles(DirectoryReference directory, IEnumerable<FileReference> files)
		{
			return new FileSetFromFiles(files.Select(x => (x.MakeRelativeTo(directory), x)));
		}

		/// <summary>
		/// Creates a file tree from a folder on disk
		/// </summary>
		/// <param name="directory"></param>
		/// <returns></returns>
		public static FileSet FromDirectory(DirectoryReference directory)
		{
			return new FileSetFromDirectory(new DirectoryInfo(directory.FullName));
		}

		/// <summary>
		/// Creates a file tree from a folder on disk
		/// </summary>
		/// <param name="directoryInfo"></param>
		/// <returns></returns>
		public static FileSet FromDirectory(DirectoryInfo directoryInfo)
		{
			return new FileSetFromDirectory(directoryInfo);
		}

		/// <summary>
		/// Create a tree containing files filtered by any of the given wildcards
		/// </summary>
		/// <param name="rules"></param>
		/// <returns></returns>
		public FileSet Filter(string rules)
		{
			return Filter(rules.Split(';'));
		}

		/// <summary>
		/// Create a tree containing files filtered by any of the given wildcards
		/// </summary>
		/// <param name="rules"></param>
		/// <returns></returns>
		public FileSet Filter(params string[] rules)
		{
			return new FileSetFromFilter(this, new FileFilter(rules));
		}

		/// <summary>
		/// Create a tree containing files filtered by any of the given file filter objects
		/// </summary>
		/// <param name="filters"></param>
		/// <returns></returns>
		public FileSet Filter(params FileFilter[] filters)
		{
			return new FileSetFromFilter(this, filters);
		}

		/// <summary>
		/// Create a tree containing the exception of files with another tree
		/// </summary>
		/// <param name="filter">Files to exclude from the filter</param>
		/// <returns></returns>
		public FileSet Except(string filter)
		{
			return Except(filter.Split(';'));
		}

		/// <summary>
		/// Create a tree containing the exception of files with another tree
		/// </summary>
		/// <param name="rules">Files to exclude from the filter</param>
		/// <returns></returns>
		public FileSet Except(params string[] rules)
		{
			return new FileSetFromFilter(this, new FileFilter(rules.Select(x => $"-{x}"), FileFilterType.Include));
		}

		/// <summary>
		/// Create a tree containing the union of files with another tree
		/// </summary>
		/// <param name="lhs"></param>
		/// <param name="rhs"></param>
		/// <returns></returns>
		public static FileSet Union(FileSet lhs, FileSet rhs)
		{
			return new FileSetFromUnion(lhs, rhs);
		}

		/// <summary>
		/// Create a tree containing the exception of files with another tree
		/// </summary>
		/// <param name="lhs"></param>
		/// <param name="rhs"></param>
		/// <returns></returns>
		public static FileSet Except(FileSet lhs, FileSet rhs)
		{
			return new FileSetFromExcept(lhs, rhs);
		}

		/// <inheritdoc cref="Union(FileSet, FileSet)"/>
		public static FileSet operator +(FileSet lhs, FileSet rhs)
		{
			return Union(lhs, rhs);
		}

		/// <inheritdoc cref="Except(FileSet, FileSet)"/>
		public static FileSet operator -(FileSet lhs, FileSet rhs)
		{
			return Except(lhs, rhs);
		}

		/// <summary>
		/// Flatten to a map of files in a target directory
		/// </summary>
		/// <returns></returns>
		public Dictionary<string, FileReference> Flatten()
		{
			Dictionary<string, FileReference> pathToSourceFile = new(StringComparer.OrdinalIgnoreCase);
			FlattenInternal(String.Empty, pathToSourceFile);
			return pathToSourceFile;
		}

		private void FlattenInternal(string pathPrefix, Dictionary<string, FileReference> pathToSourceFile)
		{
			foreach ((string path, FileReference file) in EnumerateFiles())
			{
				pathToSourceFile[pathPrefix + path] = file;
			}
			foreach ((string path, FileSet fileSet) in EnumerateDirectories())
			{
				fileSet.FlattenInternal(pathPrefix + path + "/", pathToSourceFile);
			}
		}

		/// <summary>
		/// Flatten to a map of files in a target directory
		/// </summary>
		/// <returns></returns>
		public Dictionary<FileReference, FileReference> Flatten(DirectoryReference outputDir)
		{
			Dictionary<FileReference, FileReference> targetToSourceFile = [];
			foreach ((string path, FileReference sourceFile) in Flatten())
			{
				FileReference targetFile = FileReference.Combine(outputDir, path);
				targetToSourceFile[targetFile] = sourceFile;
			}
			return targetToSourceFile;
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <inheritdoc/>
		public IEnumerator<FileReference> GetEnumerator() => Flatten().Values.GetEnumerator();
	}

	/// <summary>
	/// File tree from a known set of files
	/// </summary>
	class FileSetFromFiles : FileSet
	{
		readonly Dictionary<string, FileReference> _files = [];
		readonly Dictionary<string, FileSetFromFiles> _subTrees = [];
		internal static readonly char[] Separator = ['/', '\\'];

		/// <summary>
		/// Private constructor
		/// </summary>
		/// <param name="path"></param>
		private FileSetFromFiles(string path)
			: base(path)
		{
		}

		/// <summary>
		/// Creates a tree from a given set of files
		/// </summary>
		/// <param name="inputFiles"></param>
		public FileSetFromFiles(IEnumerable<(string, FileReference)> inputFiles)
			: this(String.Empty)
		{
			foreach ((string path, FileReference file) in inputFiles)
			{
				string[] fragments = path.Split(Separator, StringSplitOptions.RemoveEmptyEntries);

				FileSetFromFiles current = this;
				for (int idx = 0; idx < fragments.Length - 1; idx++)
				{
					FileSetFromFiles? next;
					if (!current._subTrees.TryGetValue(fragments[idx], out next))
					{
						next = new FileSetFromFiles(current.Path + fragments[idx] + "/");
						current._subTrees.Add(fragments[idx], next);
					}
					current = next;
				}

				current._files.Add(fragments[^1], file);
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles() => _files;

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories() => _subTrees.Select(x => new KeyValuePair<string, FileSet>(x.Key, x.Value));
	}

	/// <summary>
	/// File tree enumerated from the contents of an existing directory
	/// </summary>
	sealed class FileSetFromDirectory : FileSet
	{
		readonly DirectoryInfo _directoryInfo;

		/// <summary>
		/// Constructor
		/// </summary>
		public FileSetFromDirectory(DirectoryInfo directoryInfo)
			: this(directoryInfo, "/")
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FileSetFromDirectory(DirectoryInfo directoryInfo, string path)
			: base(path)
		{
			_directoryInfo = directoryInfo;
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles() => _directoryInfo.EnumerateFiles().Select(x => new KeyValuePair<string, FileReference>(x.Name, new FileReference(x)));

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories() => _directoryInfo.EnumerateDirectories().Select(x => KeyValuePair.Create<string, FileSet>(x.Name, new FileSetFromDirectory(x, $"{Path}{x.Name}/")));
	}

	/// <summary>
	/// File tree enumerated from the combination of two separate trees
	/// </summary>
	class FileSetFromUnion : FileSet
	{
		readonly FileSet _lhs;
		readonly FileSet _rhs;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="lhs">First file tree for the union</param>
		/// <param name="rhs">Other file tree for the union</param>
		public FileSetFromUnion(FileSet lhs, FileSet rhs)
			: base(lhs.Path)
		{
			_lhs = lhs;
			_rhs = rhs;
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles()
		{
			Dictionary<string, FileReference> files = new(_lhs.EnumerateFiles(), StringComparer.OrdinalIgnoreCase);
			foreach ((string name, FileReference file) in _rhs.EnumerateFiles())
			{
				FileReference? existingFile;
				if (!files.TryGetValue(name, out existingFile))
				{
					files.Add(name, file);
				}
				else if (existingFile == null || !existingFile.Equals(file))
				{
					throw new InvalidOperationException($"Conflict for contents of {Path}{name} - could be {existingFile} or {file}");
				}
			}
			return files;
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories()
		{
			Dictionary<string, FileSet> nameToSubTree = new(_lhs.EnumerateDirectories(), StringComparer.OrdinalIgnoreCase);
			foreach ((string name, FileSet subTree) in _rhs.EnumerateDirectories())
			{
				FileSet? existingSubTree;
				if (nameToSubTree.TryGetValue(name, out existingSubTree))
				{
					nameToSubTree[name] = new FileSetFromUnion(existingSubTree, subTree);
				}
				else
				{
					nameToSubTree[name] = subTree;
				}
			}
			return nameToSubTree;
		}
	}

	/// <summary>
	/// File tree enumerated from the combination of two separate trees
	/// </summary>
	class FileSetFromExcept : FileSet
	{
		readonly FileSet _lhs;
		readonly FileSet _rhs;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="lhs">First file tree for the union</param>
		/// <param name="rhs">Other file tree for the union</param>
		public FileSetFromExcept(FileSet lhs, FileSet rhs)
			: base(lhs.Path)
		{
			_lhs = lhs;
			_rhs = rhs;
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles()
		{
			HashSet<string> rhsFiles = new(_rhs.EnumerateFiles().Select(x => x.Key), StringComparer.OrdinalIgnoreCase);
			return _lhs.EnumerateFiles().Where(x => !rhsFiles.Contains(x.Key));
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories()
		{
			Dictionary<string, FileSet> rhsDirs = new(_rhs.EnumerateDirectories(), StringComparer.OrdinalIgnoreCase);
			foreach ((string name, FileSet lhsSet) in _lhs.EnumerateDirectories())
			{
				FileSet? rhsSet;
				if (rhsDirs.TryGetValue(name, out rhsSet))
				{
					yield return KeyValuePair.Create<string, FileSet>(name, new FileSetFromExcept(lhsSet, rhsSet));
				}
				else
				{
					yield return KeyValuePair.Create(name, lhsSet);
				}
			}
		}
	}

	/// <summary>
	/// File tree which includes only those files which match any given filter
	/// </summary>
	class FileSetFromFilter : FileSet
	{
		readonly FileSet _inner;
		readonly FileFilter[] _filters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The tree to filter</param>
		/// <param name="filters"></param>
		public FileSetFromFilter(FileSet inner, params FileFilter[] filters)
			: base(inner.Path)
		{
			_inner = inner;
			_filters = filters;
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileReference>> EnumerateFiles()
		{
			foreach (KeyValuePair<string, FileReference> item in _inner.EnumerateFiles())
			{
				string filterName = _inner.Path + item.Key;
				if (_filters.Any(x => x.Matches(filterName)))
				{
					yield return item;
				}
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<KeyValuePair<string, FileSet>> EnumerateDirectories()
		{
			foreach (KeyValuePair<string, FileSet> item in _inner.EnumerateDirectories())
			{
				string filterName = _inner.Path + item.Key;

				FileFilter[] possibleFilters = [.. _filters.Where(x => x.PossiblyMatches(filterName))];
				if (possibleFilters.Length > 0)
				{
					FileSetFromFilter subTreeFilter = new(item.Value, possibleFilters);
					yield return new KeyValuePair<string, FileSet>(item.Key, subTreeFilter);
				}
			}
		}
	}
}
