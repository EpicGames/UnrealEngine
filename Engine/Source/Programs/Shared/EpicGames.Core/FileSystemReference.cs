// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;

namespace EpicGames.Core
{
	/// <summary>
	/// Base class for file system objects (files or directories).
	/// </summary>
	[Serializable]
	public abstract class FileSystemReference
	{
		/// <summary>
		/// The path to this object. Stored as an absolute path, with O/S preferred separator characters, and no trailing slash for directories.
		/// </summary>
		public string FullName { get; }

		/// <summary>
		/// The comparer to use for file system references
		/// </summary>
		public static StringComparer Comparer { get; } = StringComparer.OrdinalIgnoreCase;

		/// <summary>
		/// The comparison to use for file system references
		/// </summary>
		public static StringComparison Comparison { get; } = StringComparison.OrdinalIgnoreCase;

		/// <summary>
		/// Direct constructor for a path
		/// </summary>
		protected FileSystemReference(string fullName)
		{
			FullName = fullName;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			int h = _hashCode;
			if (h != 0)
			{
				return h;
			}

			h = Comparer.GetHashCode(FullName);

			if (h != 0)
			{
				Interlocked.CompareExchange(ref _hashCode, h, 0);
				return _hashCode; // return whatever won the race
			}

			return 0;
		}

		/// <summary>
		/// Lazy initialized hash code
		/// </summary>
		/// <remarks>
		/// This must not be (de)serialized, as newly constructed instances may produce a different hash code to
		/// deserialized results from previous program runs that are otherwise equal.
		/// </remarks>
		/// <see href="https://learn.microsoft.com/en-us/dotnet/api/system.string.gethashcode"/>
		[NonSerialized]
		private int _hashCode;

		/// <summary>
		/// Create a full path by concatenating multiple strings
		/// </summary>
		[MethodImpl(MethodImplOptions.AggressiveOptimization)]
		protected static string CombineStrings(DirectoryReference baseDirectory, params string[] fragments)
		{
			char sep = Path.DirectorySeparatorChar;
			char alt = Path.AltDirectorySeparatorChar;

			Span<char> initial = stackalloc char[512];
			using StackStringBuilder sb = new(initial);

			foreach (string? fragment in fragments)
			{
				// Check if this fragment is an absolute path
				if ((fragment.Length >= 2 && fragment[1] == ':') || (fragment.Length >= 1 && (fragment[0] == '\\' || fragment[0] == '/')))
				{
					// It is. Reset the new name to the full version of this path.
					sb.Resize(0);
					ReadOnlySpan<char> abs = Path.GetFullPath(fragment).AsSpan();
					if (abs.Length == 3 && abs[2] == ':')
					{
						sb.Append(abs.Slice(0, 2));
					}
					else
					{
						sb.Append(abs);
					}
					continue;
				}
				else if (sb.Length == 0)
				{
					ReadOnlySpan<char> baseSpan = baseDirectory.FullName.AsSpan();

					// If only drive, remove separator (otherwise there should never be a separator)
					if (baseSpan.Length == 3 && baseSpan[2] == ':')
					{
						sb.Append(baseSpan.Slice(0, 2));
					}
					else
					{
						sb.Append(baseSpan);
					}
				}

				// Relative fragment: allow separators (including repeated); skip empty segments
				ReadOnlySpan<char> span = fragment.AsSpan();
				int idx = 0;

				while (idx <= span.Length)
				{
					int rel = (idx < span.Length) ? span.Slice(idx).IndexOfAny(sep, alt) : -1;
					int segLen = (rel < 0) ? (span.Length - idx) : rel;

					if (segLen == 0)
					{
						// Empty segment (e.g., leading/trailing or repeated separators) — skip
					}
					else if (segLen == 1 && span[idx] == '.')
					{
						// Skip "."
					}
					else if (segLen == 2 && span[idx] == '.' && span[idx + 1] == '.')
					{
						// Pop last segment; if none, leave as-is
						for (int i = sb.Length - 1; i >= 0; i--)
						{
							if (sb[i] == sep)
							{
								sb.Resize(i);
								break;
							}
						}
					}
					else
					{
						// Append separator only if there's already content and the last char isn't a separator
						if (sb.Length > 0)
						{
							if (sb[sb.Length - 1] != sep)
							{
								sb.Append(sep);
							}
						}

						// Append segment (no separators inside by construction)
						sb.Append(span.Slice(idx, segLen));
					}

					if (rel < 0)
					{
						break;
					}

					idx += rel + 1; // move past the separator
				}
			}
			return sb.ToString();
		}

		/// <summary>
		/// Checks whether this name has the given extension.
		/// </summary>
		/// <param name="extension">The extension to check</param>
		/// <returns>True if this name has the given extension, false otherwise</returns>
		public bool HasExtension(string extension) => extension.Length > 0 && extension[0] != '.'
				? FullName.Length >= extension.Length + 1 && FullName[FullName.Length - extension.Length - 1] == '.' && FullName.EndsWith(extension, Comparison)
				: FullName.EndsWith(extension, Comparison);

		/// <summary>
		/// Determines if the given object is at or under the given directory
		/// </summary>
		/// <param name="other">Directory to check against</param>
		/// <returns>True if this path is under the given directory</returns>
		public bool IsUnderDirectory(DirectoryReference other) => FullName.StartsWith(other.FullName, Comparison) && (FullName.Length == other.FullName.Length || FullName[other.FullName.Length] == Path.DirectorySeparatorChar || other.IsRootDirectory());

		/// <summary>
		/// Checks to see if this exists as either a file or directory
		/// This is helpful for Mac, because a binary may be a .app which is a directory
		/// </summary>
		/// <param name="location">FileSystem object to check</param>
		/// <returns>True if a file or a directory exists</returns>
		public static bool Exists(FileSystemReference location) => File.Exists(location.FullName) || Directory.Exists(location.FullName);

		/// <summary>
		/// Searches the path fragments for the given name. Only complete fragments are considered a match.
		/// </summary>
		/// <param name="name">Name to check for</param>
		/// <param name="offset">Offset within the string to start the search</param>
		/// <returns>True if the given name is found within the path</returns>
		public bool ContainsName(string name, int offset) => ContainsName(name, offset, FullName.Length - offset);

		/// <summary>
		/// Searches the path fragments for the given name. Only complete fragments are considered a match.
		/// </summary>
		/// <param name="name">Name to check for</param>
		/// <param name="offset">Offset within the string to start the search</param>
		/// <param name="length">Length of the substring to search</param>
		/// <returns>True if the given name is found within the path</returns>
		public bool ContainsName(string name, int offset, int length)
		{
			// Check the substring to search is at least long enough to contain a match
			if (length < name.Length)
			{
				return false;
			}

			// Adjust the offset to ignore drive letters on Windows. This will either set the start to `:` on Windows, or past the initial `/` on Unix.
			// This is fine in both cases, as `:` is a forbidden filename character on Windows, and `/` is a forbidden filename character on Windows and Unix.
			// Also works for UNC paths by skipping the first `\`, which is a forbidden filename character on Windows.
			// As such, moving the offset to a minimum of 1 cannot possibly affect the results of a valid query.
			// This prevents a search for "D" from finding the drive letter and the below access of `FullName[matchIdx - 1]` being out of bounds.
			int originalOffset = offset;
			offset = Math.Max(offset, 1);
			length -= offset - originalOffset;

			// Find each occurrence of the name within the remaining string, then test whether it's surrounded by directory separators
			int matchIdx = offset;
			for (; ; )
			{
				// Find the next occurrence
				matchIdx = FullName.IndexOf(name, matchIdx, offset + length - matchIdx, Comparison);
				if (matchIdx == -1)
				{
					return false;
				}

				// Check if the substring is a directory
				int matchEndIdx = matchIdx + name.Length;
				if (FullName[matchIdx - 1] == Path.DirectorySeparatorChar && (matchEndIdx == FullName.Length || FullName[matchEndIdx] == Path.DirectorySeparatorChar))
				{
					return true;
				}

				// Move past the string that didn't match
				matchIdx += name.Length;
			}
		}

		/// <summary>
		/// Determines if the given object is under the given directory, within a subfolder of the given name. Useful for masking out directories by name.
		/// </summary>
		/// <param name="name">Name of a subfolder to also check for</param>
		/// <param name="baseDir">Base directory to check against</param>
		/// <returns>True if the path is under the given directory</returns>
		public bool ContainsName(string name, DirectoryReference baseDir) => IsUnderDirectory(baseDir) && ContainsName(name, baseDir.FullName.Length);

		/// <summary>
		/// Determines if the given object is under the given directory, within a subfolder of the given name. Useful for masking out directories by name.
		/// </summary>
		/// <param name="names">Names of subfolders to also check for</param>
		/// <param name="baseDir">Base directory to check against</param>
		/// <returns>True if the path is under the given directory</returns>
		public bool ContainsAnyNames(IEnumerable<string> names, DirectoryReference baseDir) => IsUnderDirectory(baseDir) && names.Any(x => ContainsName(x, baseDir.FullName.Length));

		/// <summary>
		/// Creates a relative path from the given base directory
		/// </summary>
		/// <param name="directory">The directory to create a relative path from</param>
		/// <returns>A relative path from the given directory</returns>
		[MethodImpl(MethodImplOptions.AggressiveOptimization)]
		public string MakeRelativeTo(DirectoryReference directory)
		{
			if (String.Equals(FullName, directory.FullName, Comparison))
			{
				return ".";
			}
			Span<char> initial = stackalloc char[512];
			StackStringBuilder sb = new(initial);
			MakeRelativeTo(ref sb, directory);
			string result = sb.ToString();
			sb.Dispose();
			return result;
		}

		/// <summary>
		/// Creates a relative path from the given base directory
		/// </summary>
		/// <param name="outBuilder">The string builder where the result is written to</param>
		/// <param name="directory">The directory to create a relative path from</param>
#pragma warning disable CA1045
		[MethodImpl(MethodImplOptions.AggressiveOptimization)]
		public void MakeRelativeTo(ref StackStringBuilder outBuilder, DirectoryReference directory)
		{
			string targetDir = FullName;
			int targetLength = targetDir.Length;
			string baseDir = directory.FullName;
			int baseLength = baseDir.Length;
			char sep = Path.DirectorySeparatorChar;

			// 1) find common directory boundary by segments
			int ti = 0;
			int bi = 0;
			int common = -1;

			while (true)
			{
				// scan to next separator (or end) in target and base
				int tn = ti;
				while (tn < targetLength && targetDir[tn] != sep)
				{
					tn++;
				}

				int bn = bi;
				while (bn < baseLength && baseDir[bn] != sep)
				{
					bn++;
				}

				int tlen = tn - ti;
				int blen = bn - bi;

				if (tlen != blen || String.Compare(targetDir, ti, baseDir, bi, tlen, Comparison) != 0)
				{
					break; // diverged within this segment
				}

				common = tn; // after matched segment

				if (tn == targetLength || bn == baseLength)
				{
					break;
				}

				ti = tn + 1;
				bi = bn + 1;
			}

			if (common == -1)
			{
				outBuilder.Append(targetDir); // no relative path possible per your original behavior
				return;
			}

			// 2) count how many ".." from base after 'common'
			int ups = 0;
			int scan = (common < baseLength && baseDir[common] == sep) ? common + 1 : common;
			while (scan < baseLength)
			{
				ups++;
				int next = baseDir.IndexOf(sep, scan);
				if (next < 0)
				{
					break;
				}
				scan = next + 1;
			}

			// 3) tail from target after 'common'
			int tailStart = (common < targetLength && targetDir[common] == sep) ? common + 1 : common;
			int tailLen = targetLength - tailStart;

			bool needSep = false;
			for (int i = 0; i < ups; i++)
			{
				if (needSep)
				{
					outBuilder.Append(sep);
				}
				outBuilder.Append("..");
				needSep = true;
			}

			if (tailLen > 0)
			{
				if (needSep)
				{
					outBuilder.Append(sep);
				}
				outBuilder.Append(targetDir.AsSpan(tailStart, tailLen));
			}
			else if (targetLength == baseLength)
			{
				outBuilder.Append('.');
			}
		}
#pragma warning restore CA1045

		/// <summary>
		/// Normalize the path to using forward slashes
		/// </summary>
		/// <returns></returns>
		public string ToNormalizedPath() => FullName.Replace("\\", "/", StringComparison.Ordinal);

		/// <summary>
		/// Returns a string representation of this filesystem object
		/// </summary>
		/// <returns>Full path to the object</returns>
		public override string ToString() => FullName;
	}
}
