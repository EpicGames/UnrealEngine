// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Methods for implementing transactions that replace a files contents, while supporting a fallback path if the operation is interrupted.
	/// </summary>
	public static class FileTransaction
	{
		/// <summary>
		/// Opens a file for reading, recovering from a partial transaction if necessary.
		/// </summary>
		/// <param name="file">File to read from </param>
		/// <returns>Stream to the file, or null if it does not exist</returns>
		public static FileStream? OpenRead(FileReference file)
		{
			if (!FileReference.Exists(file))
			{
				FileReference incomingFile = GetIncomingFile(file);
				if (!FileReference.Exists(incomingFile))
				{
					return null;
				}

				try
				{
					FileReference.Move(incomingFile, file, false);
				}
				catch (IOException) when (FileReference.Exists(file))
				{
				}
				catch
				{
					return null;
				}
			}

			try
			{
				return FileReference.Open(file, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete);
			}
			catch (FileNotFoundException)
			{
				return null;
			}
		}

		/// <summary>
		/// Reads all data from a file into a byte array
		/// </summary>
		/// <param name="file">File to read from</param>
		/// <returns>The data that was read</returns>
		public static byte[]? ReadAllBytes(FileReference file)
		{
			using (Stream? stream = OpenRead(file))
			{
				if (stream != null)
				{
					byte[] data = new byte[stream.Length];
					stream.ReadFixedLengthBytes(data);
					return data;
				}
			}
			return null;
		}

		/// <summary>
		/// Reads all data from a file into a byte array
		/// </summary>
		/// <param name="file">File to read from</param>
		/// <returns>The data that was read</returns>
		public static async Task<byte[]?> ReadAllBytesAsync(FileReference file)
		{
			using (Stream? stream = OpenRead(file))
			{
				if (stream != null)
				{
					byte[] data = new byte[stream.Length];
					await stream.ReadExactlyAsync(data);
					return data;
				}
			}
			return null;
		}

		/// <summary>
		/// Opens a file for writing. Call <see cref="FileTransactionStream.CompleteTransaction"/> on the returned stream to flush its state.
		/// </summary>
		/// <param name="file">File to write to</param>
		/// <returns>Stream to the file, or null if it does not exist</returns>
		public static FileTransactionStream OpenWrite(FileReference file)
		{
			FileReference incomingFile = GetIncomingFile(file);
			Stream stream = FileReference.Open(incomingFile, FileMode.Create, FileAccess.ReadWrite, FileShare.Read);
			return new FileTransactionStream(stream, incomingFile, file);
		}

		/// <summary>
		/// Writes data from a byte array into a file, as a transaction
		/// </summary>
		/// <param name="file">File to write to</param>
		/// <param name="data">Data to be written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task WriteAllBytesAsync(FileReference file, ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default) => WriteAllBytesAsync(file, new ReadOnlySequence<byte>(data), cancellationToken);

		/// <summary>
		/// Writes data from a byte sequence into a file, as a transaction
		/// </summary>
		/// <param name="file">File to write to</param>
		/// <param name="data">Data to be written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task WriteAllBytesAsync(FileReference file, ReadOnlySequence<byte> data, CancellationToken cancellationToken = default)
		{
			using (FileTransactionStream stream = OpenWrite(file))
			{
				foreach (ReadOnlyMemory<byte> memory in data)
				{
					await stream.WriteAsync(memory, cancellationToken);
				}
				stream.CompleteTransaction();
			}
		}

		/// <summary>
		/// Gets the name of the temp file to use for writing
		/// </summary>
		/// <param name="file">File </param>
		/// <returns></returns>
		static FileReference GetIncomingFile(FileReference file)
		{
			return FileReference.Combine(file.Directory, file.GetFileName() + ".incoming");
		}
	}

	/// <summary>
	/// Stream used to write to a new copy of a file in an atomic transaction
	/// </summary>
	public class FileTransactionStream : Stream
	{
		readonly Stream _inner;
		readonly FileReference _file;
		FileReference? _finalFile;

		internal FileTransactionStream(Stream inner, FileReference outputFile, FileReference finalFile)
		{
			_inner = inner;
			_file = outputFile;
			_finalFile = finalFile;
		}

		/// <summary>
		/// Marks the transaction as complete, and moves the file to its final location
		/// </summary>
		public void CompleteTransaction()
		{
			if (_finalFile == null)
			{
				throw new InvalidOperationException("Stream cannot be written to");
			}

			FileReference finalFile = _finalFile;
			Close();

			FileReference.Delete(finalFile);
			FileReference.Move(_file, finalFile);

			_finalFile = null;
		}

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				_inner.Dispose();
			}
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);
			await base.DisposeAsync();
			await _inner.DisposeAsync();
		}

		/// <inheritdoc/>
		public override bool CanRead => _inner.CanRead;

		/// <inheritdoc/>
		public override bool CanSeek => _inner.CanSeek;

		/// <inheritdoc/>
		public override bool CanTimeout => _inner.CanTimeout;

		/// <inheritdoc/>
		public override bool CanWrite => _inner.CanWrite;

		/// <inheritdoc/>
		public override long Length => _inner.Length;

		/// <inheritdoc/>
		public override long Position { get => _inner.Position; set => _inner.Position = value; }

		/// <inheritdoc/>
		public override int ReadTimeout { get => _inner.ReadTimeout; set => _inner.ReadTimeout = value; }

		/// <inheritdoc/>
		public override int WriteTimeout { get => _inner.WriteTimeout; set => _inner.WriteTimeout = value; }

		/// <inheritdoc/>
		public override void Close()
		{
			_inner.Close();
			_finalFile = null;
		}

		/// <inheritdoc/>
		public override void CopyTo(Stream destination, int bufferSize) => _inner.CopyTo(destination, bufferSize);

		/// <inheritdoc/>
		public override Task CopyToAsync(Stream destination, int bufferSize, CancellationToken cancellationToken) => _inner.CopyToAsync(destination, bufferSize, cancellationToken);

		/// <inheritdoc/>
		public override void Flush() => _inner.Flush();

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => _inner.Read(buffer, offset, count);

		/// <inheritdoc/>
		public override Task<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _inner.ReadAsync(buffer, offset, count, cancellationToken);

		/// <inheritdoc/>
		public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => _inner.ReadAsync(buffer, cancellationToken);

		/// <inheritdoc/>
		public override int ReadByte() => _inner.ReadByte();

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => _inner.Seek(offset, origin);

		/// <inheritdoc/>
		public override void SetLength(long value) => _inner.SetLength(value);

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => _inner.Write(buffer, offset, count);

		/// <inheritdoc/>
		public override void Write(ReadOnlySpan<byte> buffer) => _inner.Write(buffer);

		/// <inheritdoc/>
		public override Task WriteAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _inner.WriteAsync(buffer, offset, count, cancellationToken);

		/// <inheritdoc/>
		public override ValueTask WriteAsync(ReadOnlyMemory<byte> buffer, CancellationToken cancellationToken = default) => _inner.WriteAsync(buffer, cancellationToken);

		/// <inheritdoc/>
		public override void WriteByte(byte value) => _inner.WriteByte(value);
	}
}
