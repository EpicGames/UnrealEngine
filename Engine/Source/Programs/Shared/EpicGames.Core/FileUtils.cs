// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Runtime.InteropServices;
using System.Runtime.InteropServices.ComTypes;
using System.Runtime.Versioning;
using System.Text;
using Microsoft.Win32.SafeHandles;

#pragma warning disable CA1707 // Identifiers should not contain underscores
#pragma warning disable CA1806 // Do not ignore method results

namespace EpicGames.Core
{
	/// <summary>
	/// Exception used to represent caught file/directory exceptions.
	/// </summary>
	/// <param name="inner">Inner exception</param>
	/// <param name="message">Message to display</param>
	public class WrappedFileOrDirectoryException(Exception inner, string message) : Exception(message, inner)
	{
		/// <inheritdoc/>
		public override string ToString() => Message;
	}

	/// <summary>
	/// Information about a locked file
	/// </summary>
	/// <param name="processId">Process id</param>
	/// <param name="fileName">Path to the process holding the lock</param>
	/// <param name="appName">Name of the application</param>
	/// <param name="startTime">Time at which the process started</param>
	/// <param name="commandLine">Command-line arguments (may be null if unavailable)</param>
	/// <param name="parentProcessId">Parent process id (0 if unavailable)</param>
	/// <param name="parentFileName">Path to the parent process (may be null)</param>
	/// <param name="parentCommandLine">Parent process command-line (may be null)</param>
	public class FileLockInfoWin32(
		int processId,
		string? fileName,
		string appName,
		DateTime startTime,
		string? commandLine,
		int parentProcessId,
		string? parentFileName,
		string? parentCommandLine)
	{
		/// <summary>
		/// Process id
		/// </summary>
		public int ProcessId => processId;

		/// <summary>
		/// Path to the process holding the lock
		/// </summary>
		public string? FileName => fileName;

		/// <summary>
		/// Name of the application
		/// </summary>
		public string AppName => appName;

		/// <summary>
		/// Time at which the process started
		/// </summary>
		public DateTime StartTime => startTime;

		/// <summary>
		/// Command-line arguments used to start the process (may be null if unavailable)
		/// </summary>
		public string? CommandLine => commandLine;

		/// <summary>
		/// Parent process id (0 if unavailable or parent has exited)
		/// </summary>
		public int ParentProcessId => parentProcessId;

		/// <summary>
		/// Path to the parent process (may be null if parent has exited)
		/// </summary>
		public string? ParentFileName => parentFileName;

		/// <summary>
		/// Parent process command-line (may be null if parent has exited)
		/// </summary>
		public string? ParentCommandLine => parentCommandLine;

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder sb = new ();

			// Main process info
			sb.Append($"{ProcessId}: {FileName ?? AppName} (started {StartTime})");

			// Command line on next line, indented
			if (!String.IsNullOrEmpty(CommandLine))
			{
				string cmdLine = CommandLine.Length > 200 ? CommandLine[..197] + "..." : CommandLine;
				sb.Append($"\n    Command line: {cmdLine}");
			}

			// Parent process info on next line, indented
			if (ParentProcessId > 0)
			{
				string parentName = ParentFileName != null ? Path.GetFileName(ParentFileName) : "(exited)";
				sb.Append($"\n    Parent: {ParentProcessId} {parentName}");

				if (!String.IsNullOrEmpty(ParentCommandLine))
				{
					string parentCmd = ParentCommandLine.Length > 100
						? ParentCommandLine[..97] + "..."
						: ParentCommandLine;
					sb.Append($" [{parentCmd}]");
				}
			}

			return sb.ToString();
		}
	}

	/// <summary>
	/// Utility functions for manipulating files. Where these methods have similar functionality to those in the NET Framework, they generally go the extra mile to produce concise, specific error messages where possible.
	/// </summary>
	public static class FileUtils
	{
		/// <summary>
		/// Comparer that should be used for native path comparisons
		/// </summary>
		public static IEqualityComparer<string> PlatformPathComparer { get; } = OperatingSystem.IsLinux() ? StringComparer.Ordinal : StringComparer.OrdinalIgnoreCase;

		/// <summary>
		/// Utf8 string comparer that should be used for native path comparisons
		/// </summary>
		public static IEqualityComparer<Utf8String> PlatformPathComparerUtf8 { get; } = OperatingSystem.IsLinux() ? Utf8StringComparer.Ordinal : Utf8StringComparer.OrdinalIgnoreCase;

		/// <summary>
		/// Read all text for a file
		/// </summary>
		/// <param name="file"></param>
		/// <returns></returns>
		public static string ReadAllText(FileReference file)
		{
			try
			{
				return FileReference.ReadAllText(file);
			}
			catch (DirectoryNotFoundException ex)
			{
				throw new WrappedFileOrDirectoryException(ex, $"Unable to read file '{file}'. The directory does not exist.");
			}
			catch (FileNotFoundException ex)
			{
				throw new WrappedFileOrDirectoryException(ex, $"Unable to read file '{file}'. The file does not exist.");
			}
			catch (Exception ex)
			{
				throw new WrappedFileOrDirectoryException(ex, $"Unable to read file '{file}'");
			}
		}

		/// <summary>
		/// Finds the on-disk case of a a file
		/// </summary>
		/// <param name="info">FileInfo instance describing the file</param>
		/// <returns>New FileInfo instance that represents the file with the correct case</returns>
		public static FileInfo FindCorrectCase(FileInfo info)
		{
			DirectoryInfo parentInfo = DirectoryUtils.FindCorrectCase(info.Directory!);
			if (info.Exists)
			{
				foreach (FileInfo childInfo in parentInfo.EnumerateFiles())
				{
					if (String.Equals(childInfo.Name, info.Name, FileSystemReference.Comparison))
					{
						return childInfo;
					}
				}
			}
			return new FileInfo(Path.Combine(parentInfo.FullName, info.Name));
		}

		/// <summary>
		/// Creates a directory tree, with all intermediate branches
		/// </summary>
		/// <param name="directory">The directory to create</param>
		public static void CreateDirectoryTree(DirectoryReference directory)
		{
			if (!DirectoryReference.Exists(directory))
			{
				DirectoryReference? parentDirectory = directory.ParentDirectory;
				if (parentDirectory != null)
				{
					CreateDirectoryTree(parentDirectory);
				}
				DirectoryReference.CreateDirectory(directory);
			}
		}

		/// <summary>
		/// Gets information about the processes that have a file locked
		/// </summary>
		/// <param name="fileName">Filename to check</param>
		/// <returns>String containing locking information</returns>
		public static string? GetFileLockInfo(string fileName)
		{
			if (OperatingSystem.IsWindows())
			{
				List<FileLockInfoWin32>? lockInfoList;
				try
				{
					lockInfoList = GetFileLockInfoWin32(fileName);
				}
				catch (Exception ex)
				{
					return $"Failed to query file lock info: {ex.Message}";
				}

				if (lockInfoList != null && lockInfoList.Count > 0)
				{
					StringBuilder message = new("Processes with open handles to file:");
					foreach (FileLockInfoWin32 lockInfo in lockInfoList)
					{
						message.Append($"\n  {lockInfo}");
					}
					return message.ToString();
				}
			}
			return null;
		}

		/// <summary>
		/// Deletes a file, whether it's read-only or not
		/// </summary>
		/// <param name="fileName">Name of the file to delete</param>
		public static void ForceDeleteFile(string fileName)
		{
			if (OperatingSystem.IsWindows())
			{
				ForceDeleteFileWin32(fileName);
			}
			else
			{
				ForceDeleteFile(new FileInfo(fileName));
			}
		}

		/// <summary>
		/// Deletes a file, whether it's read-only or not
		/// </summary>
		/// <param name="file">The file to delete</param>
		public static void ForceDeleteFile(FileInfo file)
		{
			if (OperatingSystem.IsWindows())
			{
				ForceDeleteFileWin32(file.FullName);
				file.Refresh();
			}
			else
			{
				try
				{
					if (file.Exists)
					{
						file.Attributes = FileAttributes.Normal;
						file.Delete();
					}
				}
				catch (Exception ex)
				{
					throw new WrappedFileOrDirectoryException(ex, String.Format("Unable to delete '{0}': {1}", file.FullName, ex.Message.TrimEnd()));
				}
			}
		}

		/// <summary>
		/// Deletes a file, whether it's read-only or not
		/// </summary>
		/// <param name="location">The file to delete</param>
		public static void ForceDeleteFile(FileReference location)
		{
			if (OperatingSystem.IsWindows())
			{
				ForceDeleteFileWin32(location.FullName);
			}
			else
			{
				ForceDeleteFile(new FileInfo(location.FullName));
			}
		}

		/// <summary>
		/// Deletes a directory and all its contents. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="directoryName">Directory to delete</param>
		public static void ForceDeleteDirectory(string directoryName)
		{
			if (OperatingSystem.IsWindows())
			{
				ForceDeleteLongDirectoryWin32("\\\\?\\" + directoryName);
			}
			else
			{
				ForceDeleteDirectory(new DirectoryInfo(directoryName));
			}
		}

		/// <summary>
		/// Deletes a directory and all its contents. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="directory">Directory to delete</param>
		public static void ForceDeleteDirectory(DirectoryInfo directory)
		{
			if (directory.Exists)
			{
				if (OperatingSystem.IsWindows())
				{
					ForceDeleteLongDirectoryWin32("\\\\?\\" + directory.FullName);
				}
				else
				{
					ForceDeleteDirectoryContents(directory);
					ForceDeleteDirectoryInternal(directory);
				}
			}
		}

		/// <summary>
		/// Deletes a directory and all its contents. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="location">Directory to delete</param>
		public static void ForceDeleteDirectory(DirectoryReference location)
		{
			ForceDeleteDirectory(location.FullName);
		}

		/// <summary>
		/// Helper method to delete a directory and throw a WrappedFileOrDirectoryException on failure.
		/// </summary>
		/// <param name="directory">The directory to delete</param>
		static void ForceDeleteDirectoryInternal(DirectoryInfo directory)
		{
			try
			{
				directory.Delete(true);
			}
			catch (DirectoryNotFoundException)
			{
				// Race condition with something else deleting the same directory.
			}
			catch (Exception ex)
			{
				throw new WrappedFileOrDirectoryException(ex, String.Format("Unable to delete '{0}': {1}", directory.FullName, ex.Message.TrimEnd()));
			}
		}

		/// <summary>
		/// Deletes the contents of a directory, without deleting the directory itself. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="directory">Directory to delete</param>
		public static void ForceDeleteDirectoryContents(string directory)
		{
			ForceDeleteDirectoryContents(new DirectoryInfo(directory));
		}

		/// <summary>
		/// Deletes the contents of a directory, without deleting the directory itself. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="directory">Directory to delete</param>
		public static void ForceDeleteDirectoryContents(DirectoryInfo directory)
		{
			if (OperatingSystem.IsWindows())
			{
				ForceDeleteLongDirectoryContentsWin32(directory.FullName);
			}
			else
			{
				if (directory.Exists)
				{
					try
					{
						foreach (FileInfo file in directory.EnumerateFiles())
						{
							ForceDeleteFile(file);
						}
						foreach (DirectoryInfo subDirectory in directory.EnumerateDirectories())
						{
							if (subDirectory.Attributes.HasFlag(FileAttributes.ReparsePoint))
							{
								ForceDeleteDirectoryInternal(subDirectory);
							}
							else
							{
								ForceDeleteDirectory(subDirectory);
							}
						}
					}
					catch (DirectoryNotFoundException)
					{
						// Race condition with something else deleting the same directory.
					}
				}
			}
		}

		/// <summary>
		/// Deletes the contents of a directory, without deleting the directory itself. Attempts to handle directories with long filenames (> 260 chars) on Windows.
		/// </summary>
		/// <param name="directory">Directory to delete</param>
		public static void ForceDeleteDirectoryContents(DirectoryReference directory)
		{
			ForceDeleteDirectoryContents(new DirectoryInfo(directory.FullName));
		}

		/// <summary>
		/// Moves a file from one location to another. Creates the destination directory, and removes read-only files in the target location if necessary.
		/// </summary>
		/// <param name="sourceFileName">Path to the source file</param>
		/// <param name="targetFileName">Path to the target file</param>
		public static void ForceMoveFile(string sourceFileName, string targetFileName)
		{
			ForceMoveFile(new FileReference(sourceFileName), new FileReference(targetFileName));
		}

		/// <summary>
		/// Moves a file from one location to another. Creates the destination directory, and removes read-only files in the target location if necessary.
		/// </summary>
		/// <param name="sourceLocation">Path to the source file</param>
		/// <param name="targetLocation">Path to the target file</param>
		public static void ForceMoveFile(FileReference sourceLocation, FileReference targetLocation)
		{
			// Try to move the file into place
			try
			{
				FileReference.Move(sourceLocation, targetLocation);
				return;
			}
			catch (Exception ex)
			{
				// Try to create the target directory
				try
				{
					if (!DirectoryReference.Exists(targetLocation.Directory))
					{
						CreateDirectoryTree(targetLocation.Directory);
					}
				}
				catch
				{
				}

				// Try to delete an existing file at the target location
				try
				{
					if (FileReference.Exists(targetLocation))
					{
						FileReference.SetAttributes(targetLocation, FileAttributes.Normal);
						FileReference.Delete(targetLocation);
						FileReference.Move(sourceLocation, targetLocation);
						return;
					}
				}
				catch (Exception deleteEx)
				{
					throw new WrappedFileOrDirectoryException(new AggregateException(ex, deleteEx), $"Unable to move {sourceLocation} to {targetLocation} (also tried delete/move): {ex.Message}");
				}

				// Throw the original exception
				throw new WrappedFileOrDirectoryException(ex, $"Unable to move {sourceLocation} to {targetLocation}: {ex.Message}");
			}
		}

		/// <summary>
		/// Get the encoding of a span and number of bytes to skip at the start of the buffer
		/// </summary>
		/// <param name="bytes">Bytes to scan for a BOM</param>
		/// <param name="skipBytes">Number of bytes to skip</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(ReadOnlySpan<byte> bytes, out int skipBytes)
		{
			// https://simple.wikipedia.org/wiki/Byte_order_mark
			switch (bytes.Length)
			{
				// UTF-32, little-endian
				case >= 4 when bytes[0] == 0xff && bytes[1] == 0xfe && bytes[2] == 0x00 && bytes[3] == 0x00:
					skipBytes = 4;
					return Encoding.UTF32;
				// UTF-32, big-endian
				case >= 4 when bytes[0] == 0x00 && bytes[1] == 0x00 && bytes[2] == 0xfe && bytes[3] == 0xff:
					skipBytes = 4;
					// Encoding.BigEndianUTF32 is private, use GetEncoding() to return the static Encoding to prevent additional allocation
					return Encoding.GetEncoding(12001);
				// UTF-8 with BOM
				case >= 3 when bytes[0] == 0xef && bytes[1] == 0xbb && bytes[2] == 0xbf:
					skipBytes = 3;
					return Encoding.UTF8;
				// UTF-16, little-endian
				case >= 2 when bytes[0] == 0xff && bytes[1] == 0xfe:
					skipBytes = 2;
					return Encoding.Unicode;
				// UTF-16, big-endian
				case >= 2 when bytes[0] == 0xfe && bytes[1] == 0xff:
					skipBytes = 2;
					return Encoding.BigEndianUnicode;
				// Default to UTF-8
				default:
					skipBytes = 0;
					return Encoding.UTF8;
			}
		}

		/// <summary>
		/// Get the encoding of a span
		/// </summary>
		/// <param name="bytes">Bytes to scan for a BOM</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(ReadOnlySpan<byte> bytes) => GetEncoding(bytes, out int _);

		/// <summary>
		/// Get the encoding of a file
		/// </summary>
		/// <param name="filePath">File path to scan for a BOM</param>
		/// <param name="skipBytes">Number of bytes to skip</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(string filePath, out int skipBytes)
		{
			try
			{
				byte[] bytes = new byte[4];
				using (FileStream stream = File.Open(filePath, FileMode.Open, FileAccess.Read, FileShare.Read))
				{
					stream.ReadExactly(bytes, 0, 4);
				}
				return GetEncoding(bytes, out skipBytes);
			}
			catch (IOException)
			{
			}
			skipBytes = 0;
			return Encoding.UTF8;
		}

		/// <summary>
		/// Get the encoding of a span
		/// </summary>
		/// <param name="filePath">File path to scan for a BOM</param>
		/// <returns>The encoding</returns>
		public static Encoding GetEncoding(string filePath) => GetEncoding(filePath, out int _);

		/// <summary>
		/// Gets the file mode on Mac
		/// </summary>
		/// <param name="fileName"></param>
		/// <returns></returns>
		public static int GetFileMode_Mac(string fileName)
		{
			stat64_t stat = new();
			int result = stat64(fileName, stat);
			return (result >= 0) ? stat.st_mode : -1;
		}

		/// <summary>
		/// Sets the file mode on Mac
		/// </summary>
		/// <param name="fileName"></param>
		/// <param name="mode"></param>
		public static void SetFileMode_Mac(string fileName, ushort mode)
		{
			chmod(fileName, mode);
		}

		/// <summary>
		/// Gets the file mode on Linux
		/// </summary>
		/// <param name="fileName"></param>
		/// <returns></returns>
		public static int GetFileMode_Linux(string fileName)
		{
			stat64_linux_t stat = new();
			int result = stat64_linux(1, fileName, stat);
			return (result >= 0) ? (int)stat.st_mode : -1;
		}

		/// <summary>
		/// Sets the file mode on Linux
		/// </summary>
		/// <param name="fileName"></param>
		/// <param name="mode"></param>
		public static void SetFileMode_Linux(string fileName, ushort mode)
		{
			chmod_linux(fileName, mode);
		}

		#region Win32 Native File Methods
#pragma warning disable IDE1006 // Naming Styles

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		struct WIN32_FIND_DATA
		{
			public uint dwFileAttributes;
			public System.Runtime.InteropServices.ComTypes.FILETIME ftCreationTime;
			public System.Runtime.InteropServices.ComTypes.FILETIME ftLastAccessTime;
			public System.Runtime.InteropServices.ComTypes.FILETIME ftLastWriteTime;
			public uint nFileSizeHigh;
			public uint nFileSizeLow;
			public uint dwReserved0;
			public uint dwReserved1;
			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = 260)]
			public string cFileName;
			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = 14)]
			public string cAlternateFileName;
		}

		const uint FILE_ATTRIBUTE_READONLY = 0x01;
		const uint FILE_ATTRIBUTE_DIRECTORY = 0x10;
		const uint FILE_ATTRIBUTE_NORMAL = 0x80;

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		static extern IntPtr FindFirstFileW(string FileName, ref WIN32_FIND_DATA FindData);

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool FindNextFileW(IntPtr FindHandle, ref WIN32_FIND_DATA FindData);

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool FindClose(IntPtr findHandle);

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool DeleteFileW(string lpFileName);

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool RemoveDirectory(string lpPathName);

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool SetFileAttributesW(string lpFileName, uint dwFileAttributes);

		static readonly IntPtr INVALID_HANDLE_VALUE = new(-1);

		const int ERROR_FILE_NOT_FOUND = 2;
		const int ERROR_PATH_NOT_FOUND = 3;
		const int ERROR_ACCESS_DENIED = 5;
		const int ERROR_SHARING_VIOLATION = 32;
		const int ERROR_DIR_NOT_EMPTY = 145;
#pragma warning restore IDE1006 // Naming Styles

		[SupportedOSPlatform("windows")]
		private static void ForceDeleteLongDirectoryContentsWin32(string dirName)
		{
			WIN32_FIND_DATA findData = new();

			const string RawPathPrefix = "\\\\?\\";
			if (!dirName.StartsWith(RawPathPrefix, StringComparison.Ordinal))
			{
				dirName = RawPathPrefix + dirName;
			}
			if (!dirName.EndsWith('\\'))
			{
				dirName += "\\";
			}

			IntPtr hFind = FindFirstFileW(dirName + "*", ref findData);
			if (hFind != INVALID_HANDLE_VALUE)
			{
				try
				{
					for (; ; )
					{
						string fullName = dirName + findData.cFileName;
						if ((findData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY) != 0)
						{
							if (findData.cFileName != "." && findData.cFileName != "..")
							{
								if ((findData.dwFileAttributes & FILE_ATTRIBUTE_READONLY) != 0)
								{
									SetFileAttributesW(fullName, (uint)(findData.dwFileAttributes & ~FILE_ATTRIBUTE_READONLY));
								}
								ForceDeleteLongDirectoryWin32(fullName);
							}
						}
						else
						{
							if ((findData.dwFileAttributes & FILE_ATTRIBUTE_READONLY) != 0)
							{
								SetFileAttributesW(fullName, FILE_ATTRIBUTE_NORMAL);
							}
							ForceDeleteFileWin32(fullName);
						}

						if (!FindNextFileW(hFind, ref findData))
						{
							break;
						}
					}
				}
				finally
				{
					FindClose(hFind);
				}
			}
		}

		[SupportedOSPlatform("windows")]
		private static void ForceDeleteLongDirectoryWin32(string dirName)
		{
			ForceDeleteLongDirectoryContentsWin32(dirName);

			// Clear read-only on the directory itself before attempting removal
			int attrs = GetFileAttributesW(dirName);
			if (attrs != -1 && (attrs & FILE_ATTRIBUTE_READONLY) != 0)
			{
				SetFileAttributesW(dirName, (uint)(attrs & ~FILE_ATTRIBUTE_READONLY));
			}

			if (!RemoveDirectory(dirName))
			{
				int errorCode = Marshal.GetLastWin32Error();
				if (errorCode != ERROR_FILE_NOT_FOUND && errorCode != ERROR_PATH_NOT_FOUND)
				{
					Win32Exception ex = new(errorCode);
					StringBuilder message = new($"Unable to delete {dirName} - {ex.Message}");

					if (errorCode is ERROR_ACCESS_DENIED or ERROR_SHARING_VIOLATION or ERROR_DIR_NOT_EMPTY)
					{
						string? lockInfo = GetDirectoryLockInfo(dirName);
						if (lockInfo != null)
						{
							message.Append('\n');
							message.Append(lockInfo);
						}
					}

					throw new WrappedFileOrDirectoryException(ex, message.ToString());
				}
			}
		}

		/// <summary>
		/// Enumerates remaining files in a directory after RemoveDirectory fails and gathers lock info.
		/// Returns null if no useful diagnostics could be gathered.
		/// </summary>
		[SupportedOSPlatform("windows")]
		private static string? GetDirectoryLockInfo(string dirName)
		{
			const string RawPathPrefix = "\\\\?\\";
			const int MaxFiles = 10;
			const int MaxDirs = 5;

			try
			{
				string searchPath = dirName;
				if (!searchPath.StartsWith(RawPathPrefix, StringComparison.Ordinal))
				{
					searchPath = RawPathPrefix + searchPath;
				}
				if (!searchPath.EndsWith('\\'))
				{
					searchPath += "\\";
				}

				WIN32_FIND_DATA findData = new();
				IntPtr hFind = FindFirstFileW(searchPath + "*", ref findData);
				if (hFind == INVALID_HANDLE_VALUE)
				{
					return null;
				}

				List<string> remainingFiles = [];
				List<string> remainingDirs = [];
				try
				{
					for (; ; )
					{
						if (findData.cFileName != "." && findData.cFileName != "..")
						{
							string fullName = searchPath + findData.cFileName;
							if ((findData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY) != 0)
							{
								if (remainingDirs.Count < MaxDirs)
								{
									remainingDirs.Add(fullName);
								}
							}
							else
							{
								if (remainingFiles.Count < MaxFiles)
								{
									remainingFiles.Add(fullName);
								}
							}
						}

						if (!FindNextFileW(hFind, ref findData))
						{
							break;
						}
					}
				}
				finally
				{
					FindClose(hFind);
				}

				if (remainingFiles.Count == 0 && remainingDirs.Count == 0)
				{
					return "Directory is empty but could not be removed; a process may be holding a handle to the directory itself.";
				}

				StringBuilder message = new();
				if (remainingDirs.Count > 0)
				{
					message.Append($"Directory still contains {remainingDirs.Count} subdirector{(remainingDirs.Count == 1 ? "y" : "ies")}:");
					foreach (string dir in remainingDirs)
					{
						message.Append($"\n  {dir}");
					}
				}

				foreach (string file in remainingFiles)
				{
					// Strip \\?\ prefix for Restart Manager compatibility
					string fileForLockInfo = file;
					if (fileForLockInfo.StartsWith(RawPathPrefix, StringComparison.Ordinal))
					{
						fileForLockInfo = fileForLockInfo.Substring(RawPathPrefix.Length);
					}

					string? lockInfo = GetFileLockInfo(fileForLockInfo);
					if (lockInfo != null)
					{
						if (message.Length > 0)
						{
							message.Append('\n');
						}
						message.Append($"File: {file}\n{lockInfo}");
					}
				}

				return message.Length > 0 ? message.ToString() : null;
			}
			catch (Exception ex)
			{
				return $"Failed to gather lock diagnostics: {ex.Message}";
			}
		}

		[DllImport("kernel32.dll", CharSet = CharSet.Unicode, SetLastError = true)]
		internal static extern int GetFileAttributesW(string lpFileName);

		/// <summary>
		/// Force-delete a file (Windows only)
		/// </summary>
		/// <param name="fileName"></param>
		[SupportedOSPlatform("windows")]
		public static void ForceDeleteFileWin32(string fileName)
		{
			if (TryDeleteFileWin32(fileName, out int errorCode))
			{
				return;
			}

			// If the file is read-only, clear the flag and retry
			int attributes = GetFileAttributesW(fileName);
			if (attributes != -1 && (attributes & (int)FileAttributes.ReadOnly) != 0)
			{
				SetFileAttributesW(fileName, FILE_ATTRIBUTE_NORMAL);
				if (TryDeleteFileWin32(fileName, out errorCode))
				{
					return;
				}
			}

			// Build error message with lock info
			Win32Exception ex = new(errorCode);
			StringBuilder message = new($"Unable to delete {fileName} (error {errorCode}) - {ex.Message}");

			if (errorCode is ERROR_ACCESS_DENIED or ERROR_SHARING_VIOLATION)
			{
				string? lockInfo = GetFileLockInfo(fileName);
				if (lockInfo != null)
				{
					message.Append('\n');
					message.Append(lockInfo);
				}
			}

			throw new WrappedFileOrDirectoryException(ex, message.ToString());
		}

		/// <summary>
		/// Attempt to delete a file, returning true on success or if the file does not exist.
		/// </summary>
		[SupportedOSPlatform("windows")]
		private static bool TryDeleteFileWin32(string fileName, out int errorCode)
		{
			if (DeleteFileW(fileName))
			{
				errorCode = 0;
				return true;
			}

			errorCode = Marshal.GetLastWin32Error();
			return errorCode is ERROR_FILE_NOT_FOUND or ERROR_PATH_NOT_FOUND;
		}

		#endregion

		#region Win32 Restart Manager API
#pragma warning disable IDE1006
		[StructLayout(LayoutKind.Sequential)]
		struct RM_UNIQUE_PROCESS
		{
			public int dwProcessId;
			public FILETIME ProcessStartTime;
		}

		static readonly int RM_SESSION_KEY_LEN = Marshal.SizeOf<Guid>();
		static readonly int CCH_RM_SESSION_KEY = RM_SESSION_KEY_LEN * 2;
		const int CCH_RM_MAX_APP_NAME = 255;
		const int CCH_RM_MAX_SVC_NAME = 63;

		enum RM_APP_TYPE
		{
			RmUnknownApp = 0,
			RmMainWindow = 1,
			RmOtherWindow = 2,
			RmService = 3,
			RmExplorer = 4,
			RmConsole = 5,
			RmCritical = 1000
		}

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		struct RM_PROCESS_INFO
		{
			public RM_UNIQUE_PROCESS Process;

			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = CCH_RM_MAX_APP_NAME + 1)]
			public string strAppName;

			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = CCH_RM_MAX_SVC_NAME + 1)]
			public string strServiceShortName;

			public RM_APP_TYPE ApplicationType;
			public uint AppStatus;
			public uint TSSessionId;

			[MarshalAs(UnmanagedType.Bool)]
			public bool bRestartable;
		}

		[DllImport("rstrtmgr.dll", CharSet = CharSet.Unicode)]
		static extern int RmRegisterResources(uint pSessionHandle, uint nFiles, string[] rgsFilenames, uint nApplications, [In] RM_UNIQUE_PROCESS[]? rgApplications, uint nServices, string[]? rgsServiceNames);

		[DllImport("rstrtmgr.dll", CharSet = CharSet.Auto)]
#pragma warning disable CA1838 // Avoid 'StringBuilder' parameters for P/Invokes
		static extern int RmStartSession(out uint pSessionHandle, int dwSessionFlags, StringBuilder strSessionKey);
#pragma warning restore CA1838 // Avoid 'StringBuilder' parameters for P/Invokes

		[DllImport("rstrtmgr.dll")]
		static extern int RmEndSession(uint pSessionHandle);

		[DllImport("rstrtmgr.dll")]
		static extern int RmGetList(uint dwSessionHandle,
									out uint pnProcInfoNeeded,
									ref uint pnProcInfo,
									[In, Out] RM_PROCESS_INFO[] rgAffectedApps,
									ref uint lpdwRebootReasons);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern SafeProcessHandle OpenProcess(
			 int processAccess,
			 bool bInheritHandle,
			 int processId
		);

		[DllImport("kernel32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool GetProcessTimes(SafeProcessHandle hProcess, out FILETIME
		   lpCreationTime, out FILETIME lpExitTime, out FILETIME lpKernelTime,
		   out FILETIME lpUserTime);
#pragma warning restore IDE1006

		/// <summary>
		/// Gets a list of processes that have a handle to the given file open
		/// </summary>
		/// <param name="fileName">File to check</param>
		/// <returns>List of processes with a lock open</returns>
		[SupportedOSPlatform("windows")]
		public static List<FileLockInfoWin32> GetFileLockInfoWin32(string fileName)
		{
			uint sessionHandle = 0;
			try
			{
				StringBuilder sessionKey = new(CCH_RM_SESSION_KEY + 1);

				int result = RmStartSession(out sessionHandle, 0, sessionKey);
				if (result != 0)
				{
					throw new Win32Exception(result, "Unable to open restart manager session");
				}

				result = RmRegisterResources(sessionHandle, 1, [fileName], 0, null, 0, null);
				if (result != 0)
				{
					throw new Win32Exception(result, "Unable to register resource with restart manager");
				}

				uint nProcInfoNeeded = 0;
				uint nProcInfo = 10;
				uint reason = 0;
				RM_PROCESS_INFO[] processInfoArray = new RM_PROCESS_INFO[nProcInfo];
				result = RmGetList(sessionHandle, out nProcInfoNeeded, ref nProcInfo, processInfoArray, ref reason);
				
				// ERROR_MORE_DATA thrown when more processes have an open file handle
				if (result == 234)
				{
					nProcInfo = Math.Min(nProcInfoNeeded, 256); // Cap to 256 processes just in case
					processInfoArray = new RM_PROCESS_INFO[nProcInfo];
					result = RmGetList(sessionHandle, out nProcInfoNeeded, ref nProcInfo, processInfoArray, ref reason);
				}

				if (result != 0)
				{
					throw new Win32Exception(result, $"(error {result}) - Unable to query processes with file handle open");
				}

				List<FileLockInfoWin32> fileLocks = [];
				for (int idx = 0; idx < nProcInfo; idx++)
				{
					RM_PROCESS_INFO processInfo = processInfoArray[idx];
					long startTimeTicks = FileTimeToTicks(processInfo.Process.ProcessStartTime);

					string? imageName = null;
					string? commandLine = null;
					int parentPid = 0;
					string? parentFileName = null;
					string? parentCommandLine = null;

					using (SafeProcessHandle hProcess = OpenProcess(ProcessUtils.PROCESS_QUERY_LIMITED_INFORMATION | ProcessUtils.PROCESS_VM_READ, false, processInfo.Process.dwProcessId))
					{
						if (hProcess is { IsInvalid: false })
						{
							FILETIME createTime, exitTime, kernelTime, userTime;
							if (GetProcessTimes(hProcess, out createTime, out exitTime, out kernelTime, out userTime) && FileTimeToTicks(createTime) == startTimeTicks)
							{
								imageName = ProcessUtils.GetProcessImageName(hProcess);
								commandLine = ProcessUtils.GetProcessCommandLine(hProcess);
								parentPid = ProcessUtils.GetParentProcessId(hProcess);

								// Get parent process info if parent is still running
								if (parentPid > 0)
								{
									parentFileName = ProcessUtils.GetProcessImageName(parentPid);
									parentCommandLine = ProcessUtils.GetProcessCommandLine(parentPid);
								}
							}
						}
					}

					fileLocks.Add(new FileLockInfoWin32(
						processInfo.Process.dwProcessId,
						imageName,
						processInfo.strAppName,
						DateTime.FromFileTime(startTimeTicks),
						commandLine,
						parentPid,
						parentFileName,
						parentCommandLine));
				}
				return fileLocks;
			}
			finally
			{
				if (sessionHandle != 0)
				{
					RmEndSession(sessionHandle);
				}
			}
		}

		private static long FileTimeToTicks(FILETIME fileTime)
		{
			return (long)(uint)fileTime.dwLowDateTime | ((long)(uint)fileTime.dwHighDateTime << 32);
		}

		#endregion

		#region Mac Native File Methods
#pragma warning disable CS0649
#pragma warning disable IDE1006
		struct timespec_t
		{
			public ulong tv_sec;
			public ulong tv_nsec;
		}

		[StructLayout(LayoutKind.Sequential)]
		class stat64_t
		{
			public uint st_dev;
			public ushort st_mode;
			public ushort st_nlink;
			public ulong st_ino;
			public uint st_uid;
			public uint st_gid;
			public uint st_rdev;
			public timespec_t st_atimespec;
			public timespec_t st_mtimespec;
			public timespec_t st_ctimespec;
			public timespec_t st_birthtimespec;
			public ulong st_size;
			public ulong st_blocks;
			public uint st_blksize;
			public uint st_flags;
			public uint st_gen;
			public uint st_lspare;
			public ulong st_qspare1;
			public ulong st_qspare2;
		}

		[DllImport("libSystem.dylib")]
		static extern int stat64(string pathname, stat64_t stat);

		[DllImport("libSystem.dylib")]
		static extern int chmod(string path, ushort mode);

#pragma warning restore IDE1006
#pragma warning restore CS0649
		#endregion

		#region Linux Native File Methods
#pragma warning disable CS0649
#pragma warning disable IDE1006

		[StructLayout(LayoutKind.Sequential)]
		class stat64_linux_t
		{
			public ulong st_dev;
			public ulong st_ino;
			public ulong st_nlink;
			public uint st_mode;
			public uint st_uid;
			public uint st_gid;
			public int pad0;
			public ulong st_rdev;
			public long st_size;
			public long st_blksize;
			public long st_blocks;
			public timespec_t st_atime;
			public timespec_t st_mtime;
			public timespec_t st_ctime;
			public long glibc_reserved0;
			public long glibc_reserved1;
			public long glibc_reserved2;
		};

		/* stat tends to get compiled to another symbol and libc doesnt directly have that entry point */
		[DllImport("libc", EntryPoint = "__xstat64")]
		static extern int stat64_linux(int ver, string pathname, stat64_linux_t stat);

		[DllImport("libc", EntryPoint = "chmod")]
		static extern int chmod_linux(string path, ushort mode);

#pragma warning restore IDE1006
#pragma warning restore CS0649
		#endregion
	}
}
