// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for serializing GUIDs in UE format. UE stores GUIDs as 4 x 32-bit LE values, whereas NET stores GUIDs as 1 x 32-bit LE, 2 x 16-bit LE, 8 x 8-bit values.
	/// </summary>
	public static class GuidUtils
	{
		// String order:              0   1   2   3   4   5   6   7   8   9   a   b   c   d   e   f
		//
		// NET serialiazation order:  3   2   1   0 | 5   4 | 7   6 | 8   9   a   b   c   d   e   f
		// UE serialziation order:    3   2   1   0 | 7   6   5   4 | b   a   9   8 | f   e   d   c

		/// <summary>
		/// Reads a GUID serialized in UE order
		/// </summary>
		public static Guid ReadGuidUnrealOrder(ReadOnlySpan<byte> span)
		{
			Span<byte> buffer =
			[
				span[0x0],
				span[0x1],
				span[0x2],
				span[0x3],
				span[0x6],
				span[0x7],
				span[0x4],
				span[0x5],
				span[0xb],
				span[0xa],
				span[0x9],
				span[0x8],
				span[0xf],
				span[0xe],
				span[0xd],
				span[0xc],
			];
			return new Guid(buffer);
		}

		/// <summary>
		/// Reads a GUID serialized in UE order
		/// </summary>
		public static void WriteGuidUnrealOrder(Span<byte> span, Guid guid)
		{
			Span<byte> buffer = stackalloc byte[16];
			if (!guid.TryWriteBytes(buffer))
			{
				throw new InvalidOperationException("Unable to write guid to buffer");
			}

			span[0x0] = buffer[0x0];
			span[0x1] = buffer[0x1];
			span[0x2] = buffer[0x2];
			span[0x3] = buffer[0x3];

			span[0x4] = buffer[0x6];
			span[0x5] = buffer[0x7];
			span[0x6] = buffer[0x4];
			span[0x7] = buffer[0x5];

			span[0x8] = buffer[0xb];
			span[0x9] = buffer[0xa];
			span[0xa] = buffer[0x9];
			span[0xb] = buffer[0x8];

			span[0xc] = buffer[0xf];
			span[0xd] = buffer[0xe];
			span[0xe] = buffer[0xd];
			span[0xf] = buffer[0xc];
		}
	}
}
