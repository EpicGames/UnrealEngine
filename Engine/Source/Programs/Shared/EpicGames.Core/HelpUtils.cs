// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Linq;
using System.Text;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility functions for showing help for objects
	/// </summary>
	public static class HelpUtils
	{
		/// <summary>
		/// Gets the width of the window for formatting purposes
		/// </summary>
		public static int WindowWidth => ConsoleUtils.WindowWidth;

		/// <summary>
		/// Prints help for the given object type
		/// </summary>
		/// <param name="title"></param>
		/// <param name="type">Type to print help for</param>
		public static void PrintHelp(string title, Type type)
		{
			PrintHelp(title, GetDescription(type), CommandLineArguments.GetParameters(type));
		}

		/// <summary>
		/// Prints help for a command
		/// </summary>
		/// <param name="title">Title for the help text</param>
		/// <param name="description">Description for the command</param>
		/// <param name="parameters">List of parameters</param>
		public static void PrintHelp(string? title, string? description, List<KeyValuePair<string, string>> parameters)
		{
			bool bFirstLine = true;
			if (!String.IsNullOrEmpty(title))
			{
				PrintParagraph(title);
				bFirstLine = false;
			}

			if (!String.IsNullOrEmpty(description))
			{
				if (!bFirstLine)
				{
					Console.WriteLine("");
				}
				PrintParagraph(description);
				bFirstLine = false;
			}

			if (parameters.Count > 0)
			{
				if (!bFirstLine)
				{
					Console.WriteLine("");
				}

				Console.WriteLine("Parameters:");
				PrintTable(parameters, 4, 24);
			}
		}

		/// <summary>
		/// Gets the description from a type
		/// </summary>
		/// <param name="type">The type to get a description for</param>
		/// <returns>The description text</returns>
		public static string GetDescription(Type type)
		{
			StringBuilder descriptionText = new();
			foreach (DescriptionAttribute attribute in type.GetCustomAttributes(typeof(DescriptionAttribute), false).OfType<DescriptionAttribute>())
			{
				if (descriptionText.Length > 0)
				{
					descriptionText.AppendLine();
				}
				descriptionText.AppendLine(attribute.Description);
			}
			return descriptionText.ToString();
		}

		/// <summary>
		/// Prints a paragraph of text using word wrapping
		/// </summary>
		/// <param name="text">Text to print</param>
		public static void PrintParagraph(string text)
		{
			PrintParagraph(text, WindowWidth - 1);
		}

		/// <summary>
		/// Prints a paragraph of text using word wrapping
		/// </summary>
		/// <param name="text">Text to print</param>
		/// <param name="maxWidth">Maximum width for each line</param>
		public static void PrintParagraph(string text, int maxWidth)
		{
			IEnumerable<string> lines = StringUtils.WordWrap(text, maxWidth);
			foreach (string line in lines)
			{
				Console.WriteLine(line);
			}
		}

		/// <summary>
		/// Prints an argument list to the console
		/// </summary>
		/// <param name="items">List of parameters arranged as "-ParamName Param Description"</param>
		/// <param name="indent">Indent from the left hand side</param>
		/// <param name="minFirstColumnWidth">The minimum padding from the start of the param name to the start of the description (resizes with larger param names)</param>
		/// <returns></returns>
		public static void PrintTable(List<KeyValuePair<string, string>> items, int indent, int minFirstColumnWidth)
		{
			List<string> lines = [];
			FormatTable(items, indent, minFirstColumnWidth, WindowWidth - 1, lines);

			foreach (string line in lines)
			{
				Console.WriteLine(line);
			}
		}

		/// <summary>
		/// Prints a table of items to a logging device
		/// </summary>
		/// <param name="items"></param>
		/// <param name="indent"></param>
		/// <param name="minFirstColumnWidth"></param>
		/// <param name="maxWidth"></param>
		/// <param name="logger"></param>
		public static void PrintTable(List<KeyValuePair<string, string>> items, int indent, int minFirstColumnWidth, int maxWidth, ILogger logger)
		{
			List<string> lines = [];
			FormatTable(items, indent, minFirstColumnWidth, maxWidth, lines);

			foreach (string line in lines)
			{
				logger.LogInformation("{Line}", line);
			}
		}

		/// <summary>
		/// Formats the given parameters as so:
		///     -Param1     Param1 Description
		///
		///     -Param2      Param2 Description, this description is
		///                  longer and splits onto a separate line. 
		///
		///     -Param3      Param3 Description continues as before. 
		/// </summary>
		/// <param name="items">List of parameters arranged as "-ParamName Param Description"</param>
		/// <param name="indent">Indent from the left hand side</param>
		/// <param name="minFirstColumnWidth">The minimum padding from the start of the param name to the start of the description (resizes with larger param names)</param>
		/// <param name="maxWidth"></param>
		/// <param name="lines"></param>
		/// <returns>Sequence of formatted lines in the table</returns>
		public static void FormatTable(IReadOnlyList<KeyValuePair<string, string>> items, int indent, int minFirstColumnWidth, int maxWidth, List<string> lines)
		{
			if (items.Count > 0)
			{
				// string used to intent the param
				string indentString = new(' ', indent);

				// default the padding value
				int rightPadding = Math.Max(minFirstColumnWidth, items.Max(x => x.Key.Length + 2));

				// Build the formatted params
				foreach (KeyValuePair<string, string> item in items)
				{
					// build the param first, including intend and padding on the rights size
					string paramString = indentString + item.Key.PadRight(rightPadding);

					// Build the description line by line, adding the same amount of intending each time. 
					IEnumerable<string> descriptionLines = StringUtils.WordWrap(item.Value, maxWidth - paramString.Length);

					foreach (string descriptionLine in descriptionLines)
					{
						// Formatting as following:
						// <Indent>-param<Right Padding>Description<New line>
						lines.Add(paramString + descriptionLine);

						// we replace the param string on subsequent lines with white space of the same length
						paramString = String.Empty.PadRight(indentString.Length + rightPadding);
					}
				}
			}
		}
	}
}
