// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Specialized;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Sockets;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using System.Web;

#pragma warning disable CA2234 // Pass system uri objects instead of strings
#pragma warning disable CA1054 // URI-like parameters should not be strings

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for consuming REST APIs via JSON objects
	/// </summary>
	public static class HttpClientExtensions
	{
		/// <summary>
		/// Gets a resource from an HTTP endpoint and parses it as a JSON object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <param name="client">The http client instance</param>
		/// <param name="url">The url to retrieve</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>New instance of the object</returns>
		public static async Task<TResponse> GetAsync<TResponse>(this HttpClient client, string url, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await client.GetAsync(url, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <inheritdoc cref="GetAsync{TResponse}(HttpClient, String, CancellationToken)"/>
		public static async Task<TResponse> GetAsync<TResponse>(this HttpClient client, Uri url, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await client.GetAsync(url, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <summary>
		/// Posts an object to an HTTP endpoint as a JSON object
		/// </summary>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="client">The http client instance</param>
		/// <param name="url">The url to post to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>Response message</returns>
		public static async Task<HttpResponseMessage> PostAsync<TRequest>(this HttpClient client, string url, TRequest request, CancellationToken cancellationToken)
		{
			using HttpContent content = ToJsonContent(request);
			return await client.PostAsync(url, content, cancellationToken);
		}

		/// <inheritdoc cref="PostAsync{TRequest}(HttpClient, String, TRequest, CancellationToken)"/>
		public static async Task<HttpResponseMessage> PostAsync<TRequest>(this HttpClient client, Uri url, TRequest request, CancellationToken cancellationToken)
		{
			using HttpContent content = ToJsonContent(request);
			return await client.PostAsync(url, content, cancellationToken);
		}

		/// <summary>
		/// Posts an object to an HTTP endpoint as a JSON object, and parses the response object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="client">The http client instance</param>
		/// <param name="url">The url to post to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>The response parsed into the requested type</returns>
		public static async Task<TResponse> PostAsync<TResponse, TRequest>(this HttpClient client, string url, TRequest request, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await PostAsync(client, url, request, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <inheritdoc cref="PostAsync{TRequest}(HttpClient, String, TRequest, CancellationToken)"/>
		public static async Task<TResponse> PostAsync<TResponse, TRequest>(this HttpClient client, Uri url, TRequest request, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await PostAsync(client, url, request, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <summary>
		/// Puts an object to an HTTP endpoint as a JSON object
		/// </summary>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="client">The http client instance</param>
		/// <param name="url">The url to post to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>Response message</returns>
		public static async Task<HttpResponseMessage> PutAsync<TRequest>(this HttpClient client, string url, TRequest request, CancellationToken cancellationToken)
		{
			using HttpContent content = ToJsonContent(request);
			return await client.PutAsync(url, content, cancellationToken);
		}

		/// <inheritdoc cref="PutAsync{TRequest}(HttpClient, String, TRequest, CancellationToken)"/>
		public static async Task<HttpResponseMessage> PutAsync<TRequest>(this HttpClient client, Uri url, TRequest request, CancellationToken cancellationToken)
		{
			using HttpContent content = ToJsonContent(request);
			return await client.PutAsync(url, content, cancellationToken);
		}

		/// <summary>
		/// Puts an object to an HTTP endpoint as a JSON object, and parses the response object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="client">The http client instance</param>
		/// <param name="url">The url to post to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>The response parsed into the requested type</returns>
		public static async Task<TResponse> PutAsync<TResponse, TRequest>(this HttpClient client, string url, TRequest request, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await PutAsync(client, url, request, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <inheritdoc cref="PutAsync{TRequest}(HttpClient, String, TRequest, CancellationToken)"/>
		public static async Task<TResponse> PutAsync<TResponse, TRequest>(this HttpClient client, Uri url, TRequest request, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await PutAsync(client, url, request, cancellationToken))
			{
				response.EnsureSuccessStatusCode();
				return await ParseJsonContentAsync<TResponse>(response, cancellationToken);
			}
		}

		/// <summary>
		/// Converts an object to a JSON http content object
		/// </summary>
		/// <typeparam name="T">Type of the object to parse</typeparam>
		/// <param name="obj">The object instance</param>
		/// <returns>Http content object</returns>
		private static StringContent ToJsonContent<T>(T obj)
		{
			return new StringContent(JsonSerializer.Serialize<T>(obj), Encoding.UTF8, "application/json");
		}

		/// <summary>
		/// Parses a HTTP response as a JSON object
		/// </summary>
		/// <typeparam name="T">Type of the object to parse</typeparam>
		/// <param name="message">The message received</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Parsed object instance</returns>
		private static async Task<T> ParseJsonContentAsync<T>(HttpResponseMessage message, CancellationToken cancellationToken)
		{
			byte[] bytes = await message.Content.ReadAsByteArrayAsync(cancellationToken);
			return JsonSerializer.Deserialize<T>(bytes, new JsonSerializerOptions { PropertyNameCaseInsensitive = true })!;
		}
	}

	/// <summary>
	/// Extension methods for HttpRequestMessage
	/// </summary>
	public static class HttpRequestMessageExtensions
	{
		private static readonly HashSet<string> s_sensitiveKeys = new(StringComparer.OrdinalIgnoreCase)
		{
			"x-amz-security-token",
			"Signature",
			"AWSAccessKeyId"
		};

		/// <summary>
		/// Returns a new URI with sensitive query parameters redacted from the request URI.
		/// </summary>
		public static Uri? RedactedRequestUri(this HttpRequestMessage request)
		{
			Uri? uri = request?.RequestUri;
			if (uri?.Query is not { Length: > 0 })
			{
				return uri;
			}

			NameValueCollection queryParams = HttpUtility.ParseQueryString(uri.Query);
			foreach (string key in queryParams.AllKeys.OfType<string>())
			{
				string value = s_sensitiveKeys.Contains(key) ? "redacted" : queryParams[key] ?? String.Empty;
				queryParams[key] = value;
			}

			return new UriBuilder(uri) { Query = String.Join("&", queryParams) }.Uri;
		}
	}

	/// <summary>
	/// Class that can/should be used to Create http message handlers. Under the hood, same handler will be used to reduce redundant dns lookup calls
	/// </summary>
	public static class HttpMessageHandlers
	{
		/// <summary>
		/// Set to false to disable ipv6 dns resolving
		/// </summary>
		public static bool UseIpv6DnsResolving { get; set; } = true;

		/// <summary>
		/// Creates default http message handler resolving both ipv4/6 dns.
		/// </summary>
		public static HttpMessageHandler CreateDefaultHandler()
		{
			if (UseIpv6DnsResolving)
			{
				return new NonOwningHandler(s_defaultHandler.Value);
			}
			else
			{
				return new NonOwningHandler(s_ipv4OnlyHandler.Value);
			}
		}

		/// <summary>
		/// Creates a http message handler that only resolves ipv4 dns and skips ipv6
		/// This can be used to prevent long stalls on networks where ipv6 is not supported
		/// </summary>
		public static HttpMessageHandler CreateIpv4OnlyHandler()
		{
			return new NonOwningHandler(s_ipv4OnlyHandler.Value);
		}

		private static SocketsHttpHandler InternalCreateDefaultHandler()
		{
			return new SocketsHttpHandler
			{
				ConnectTimeout = TimeSpan.FromSeconds(25),
				PooledConnectionLifetime = TimeSpan.FromMinutes(15),
				PooledConnectionIdleTimeout = TimeSpan.FromMinutes(5),
			};
		}

		private static SocketsHttpHandler InternalCreateIpv4OnlyHandler()
		{
			SocketsHttpHandler socketHandler = InternalCreateDefaultHandler();
			socketHandler.ConnectCallback = async (ctx, ct) =>
			{
				string host = ctx.DnsEndPoint.Host;
				int port = ctx.DnsEndPoint.Port;

				// v4-only resolution (NET 8 overload)
				IPAddress[] addrs = await Dns.GetHostAddressesAsync(host, AddressFamily.InterNetwork, ct);
				if (addrs.Length == 0)
				{
					throw new SocketException((int)SocketError.HostNotFound);
				}

				Exception? last = null;
				foreach (IPAddress ip in addrs)
				{
					Socket sock = new(AddressFamily.InterNetwork, SocketType.Stream, ProtocolType.Tcp) { NoDelay = true };
					using (ct.Register(() =>
					{
						try
						{
							sock.Dispose();
						}
						catch
						{
						}
					}))
					{
					}

					try
					{
						await sock.ConnectAsync(new IPEndPoint(ip, port), ct);
						return new NetworkStream(sock, ownsSocket: true);
					}
					catch (Exception ex)
					{
						last = ex;
					}
				}
				throw last ?? new SocketException((int)SocketError.HostUnreachable);
			};
			return socketHandler;
		}

		private sealed class NonOwningHandler : DelegatingHandler
		{
			public NonOwningHandler(HttpMessageHandler inner) => InnerHandler = inner;

#pragma warning disable CA2215 // Intentionally skip calling base.Dispose to prevent cascading dispose
			protected override void Dispose(bool disposing) { }
#pragma warning restore CA2215 // Dispose objects before losing scope
		}

		private static readonly Lazy<SocketsHttpHandler> s_defaultHandler = new(InternalCreateDefaultHandler);
		private static readonly Lazy<SocketsHttpHandler> s_ipv4OnlyHandler = new(InternalCreateIpv4OnlyHandler);
	}
}
