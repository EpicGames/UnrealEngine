// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.IO.MemoryMappedFiles;
using System.Runtime.InteropServices;
using System.Threading;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for an explicit memory allocator. Meant for use with large memory blocks, where objects would typically be allocated on the LOH and waiting for GC can cause
	/// excessive memory usage.
	/// </summary>
	public interface IMemoryAllocator<T>
	{
		/// <summary>
		/// Allocate a block of memory of the given minimum size
		/// </summary>
		/// <param name="minSize">Minimum size for the allocated data</param>
		/// <param name="tag">Tag for tracking this allocation</param>
		/// <returns>Reference counted block of memory</returns>
		IMemoryOwner<T> Alloc(int minSize, object? tag);
	}

	/// <summary>
	/// Implementation of <see cref="IMemoryAllocator{Byte}"/> which creates regular arrays on the managed heap. Note that these
	/// allocations are subject to GC, and will not be freed immediately.
	/// </summary>
	public class ManagedHeapAllocator : IMemoryAllocator<byte>
	{
		class MemoryOwner : IMemoryOwner<byte>
		{
			readonly ManagedHeapAllocator _outer;

			public Memory<byte> Memory { get; private set; }

			public MemoryOwner(ManagedHeapAllocator outer, int size)
			{
				_outer = outer;
				Interlocked.Add(ref _outer._allocatedSize, size);
				Memory = new byte[size];
			}

			public void Dispose()
			{
#if DEBUG
				Memory.Span.Fill(0xfe);
#endif

				Interlocked.Add(ref _outer._allocatedSize, -Memory.Length);
				Memory = Memory<byte>.Empty;
			}
		}

		long _allocatedSize;

		/// <summary>
		/// Default shared instance
		/// </summary>
		public static ManagedHeapAllocator Shared { get; } = new ManagedHeapAllocator();

		/// <summary>
		/// Currently allocated size using this heap
		/// </summary>
		public long AllocatedSize => _allocatedSize;

		/// <inheritdoc/>
		public IMemoryOwner<byte> Alloc(int minSize, object? tag) => new MemoryOwner(this, minSize);
	}

	/// <summary>
	/// Implementation of <see cref="IMemoryAllocator{Byte}"/> which rents blocks from a memory pool.
	/// </summary>
	public class PoolAllocator : IMemoryAllocator<byte>
	{
		readonly MemoryPool<byte> _pool;

		/// <summary>
		/// Shared allocator instance
		/// </summary>
		public static PoolAllocator Shared { get; } = new PoolAllocator(MemoryPool<byte>.Shared);

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolAllocator(MemoryPool<byte> pool) => _pool = pool;

		/// <inheritdoc/>
		public IMemoryOwner<byte> Alloc(int minSize, object? tag) => _pool.Rent(minSize);
	}

	/// <summary>
	/// Implementation of <see cref="IMemoryAllocator{Byte}"/> which returns blocks from the global heap
	/// </summary>
	public class GlobalHeapAllocator : IMemoryAllocator<byte>
	{
		unsafe class Allocation : MemoryManager<byte>
		{
			readonly GlobalHeapAllocator _outer;
			IntPtr _handle;
			int _length;

			public Allocation(GlobalHeapAllocator outer, int size)
			{
				_outer = outer;
				_handle = Marshal.AllocHGlobal(size);
				_length = size;

				Interlocked.Add(ref _outer._allocatedSize, _length);
			}

			byte* GetPointer() => (byte*)_handle.ToPointer();

			/// <inheritdoc/>
			public override Span<byte> GetSpan() => new(GetPointer(), _length);

			/// <inheritdoc/>
			public override MemoryHandle Pin(int elementIndex) => new(GetPointer() + elementIndex);

			/// <inheritdoc/>
			public override void Unpin() { }

			/// <inheritdoc/>
			protected override void Dispose(bool disposing)
			{
				if (_handle != IntPtr.Zero)
				{
					Marshal.FreeHGlobal(_handle);
					_handle = IntPtr.Zero;

					Interlocked.Add(ref _outer._allocatedSize, -_length);
				}

				_length = -1;
			}
		}

		long _allocatedSize;

		/// <summary>
		/// Shared heap allocator
		/// </summary>
		public static GlobalHeapAllocator Shared { get; } = new GlobalHeapAllocator();

		/// <summary>
		/// Currently allocated size using this heap
		/// </summary>
		public long AllocatedSize => _allocatedSize;

		/// <inheritdoc/>
		public IMemoryOwner<byte> Alloc(int minSize, object? tag) => new Allocation(this, minSize);
	}

	/// <summary>
	/// Implementation of <see cref="IMemoryAllocator{Byte}"/> which returns memory backed by memory mapped files
	/// </summary>
	public class VirtualMemoryAllocator : IMemoryAllocator<byte>
	{
		unsafe class Allocation : MemoryManager<byte>
		{
			MemoryMappedFile? _file;
			MemoryMappedViewAccessor? _viewAccessor;
			byte* _pointer;
			readonly int _length;

			public Allocation(int length)
			{
				_file = MemoryMappedFile.CreateNew(null, length, MemoryMappedFileAccess.ReadWrite);
				_viewAccessor = _file.CreateViewAccessor();
				_viewAccessor.SafeMemoryMappedViewHandle.AcquirePointer(ref _pointer);
				_length = length;
			}

			/// <inheritdoc/>
			public override Span<byte> GetSpan() => new(_pointer, _length);

			/// <inheritdoc/>
			public override MemoryHandle Pin(int elementIndex) => new(_pointer + elementIndex);

			/// <inheritdoc/>
			public override void Unpin() { }

			/// <inheritdoc/>
			protected override void Dispose(bool disposing)
			{
				if (disposing)
				{
					if (_pointer != null)
					{
						_viewAccessor!.SafeMemoryMappedViewHandle.ReleasePointer();
						_pointer = null;
					}

					_viewAccessor?.Dispose();
					_viewAccessor = null;
					_file?.Dispose();
					_file = null;
				}
			}
		}

		/// <inheritdoc/>
		public IMemoryOwner<byte> Alloc(int minSize, object? tag) => new Allocation(minSize);
	}

	/// <summary>
	/// Implementation of <see cref="IMemoryAllocator{Byte}"/> which collects stats on the allocations performed.
	/// </summary>
	public class TrackingMemoryAllocator : IMemoryAllocator<byte>
	{
		// Tracks an owned blocks of memory against the cache budged.
		sealed class MemoryAllocation : IMemoryOwner<byte>
		{
			readonly TrackingMemoryAllocator _allocator;
			readonly LinkedListNode<MemoryAllocation> _node;
			readonly object? _tag;
			IMemoryOwner<byte> _allocation;

			public Memory<byte> Memory => _allocation.Memory;

			public MemoryAllocation(TrackingMemoryAllocator allocator, IMemoryOwner<byte> allocation, object? tag)
			{
				_allocator = allocator;
				_allocation = allocation;
				_tag = tag;

				lock (allocator._lockObject)
				{
					_allocator._allocatedSize += allocation.Memory.Length;
					_node = allocator._allocations.AddLast(this);
				}
			}

			/// <inheritdoc/>
			public void Dispose()
			{
				if (_allocation != null)
				{
					lock (_allocator._lockObject)
					{
						_allocator._allocatedSize -= _allocation.Memory.Length;
						_allocator._allocations.Remove(_node);
					}

					_allocation.Dispose();
					_allocation = null!;
				}
			}

			/// <inheritdoc/>
			public override string ToString()
				=> $"{_tag ?? "???"} ({_allocation?.Memory.Length ?? 0} bytes)";
		}

		readonly Lock _lockObject = new();
		readonly IMemoryAllocator<byte> _inner;
		long _allocatedSize;
		readonly LinkedList<MemoryAllocation> _allocations = new();

		/// <summary>
		/// Total size of allocated memory
		/// </summary>
		public long AllocatedSize => _allocatedSize;

		/// <summary>
		/// Constructor
		/// </summary>
		public TrackingMemoryAllocator(IMemoryAllocator<byte> inner)
			=> _inner = inner;

		/// <inheritdoc/>
		public IMemoryOwner<byte> Alloc(int minSize, object? tag)
			=> new MemoryAllocation(this, _inner.Alloc(minSize, tag), tag);
	}
}
