// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Core;

/// <summary>
/// See <see cref="CsProjBuildHook"/>.
/// </summary>
public interface IMsBuildRegistration
{

	/// <summary>
	/// The directory of the dotnet install shipped with Unreal Engine.
	/// </summary>
	DirectoryReference DotnetDirectory { get; }

	/// <summary>
	/// The dotnet executable that comprises part of the dotnet install shipped with Unreal Engine.
	/// </summary>
	FileReference DotnetPath { get; }
}
