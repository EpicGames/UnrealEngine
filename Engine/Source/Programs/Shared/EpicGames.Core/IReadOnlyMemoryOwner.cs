// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Handle to a shared block of memory with a concrete lifetime
	/// </summary>
	public interface IReadOnlyMemoryOwner<T> : IDisposable
	{
		/// <summary>
		/// The underlying memory object
		/// </summary>
		public ReadOnlyMemory<T> Memory { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IReadOnlyMemoryOwner{T}"/>
	/// </summary>
	public static class ReadOnlyMemoryOwner
	{
		class DefaultReadOnlyMemoryOwner<T> : IReadOnlyMemoryOwner<T>
		{
			ReadOnlyMemory<T>? _memory;
			IDisposable? _owner;

			public ReadOnlyMemory<T> Memory => _memory ?? throw new ObjectDisposedException(typeof(DefaultReadOnlyMemoryOwner<T>).FullName);

			public DefaultReadOnlyMemoryOwner(ReadOnlyMemory<T> memory, IDisposable? owner)
			{
				_memory = memory;
				_owner = owner;
			}

			public void Dispose()
			{
				_memory = null;
				_owner?.Dispose();
				_owner = null;
			}
		}

		/// <summary>
		/// Wrap an array as a <see cref="IReadOnlyMemoryOwner{T}"/>
		/// </summary>
		public static IReadOnlyMemoryOwner<T> Create<T>(T[] memory, IDisposable? owner = null)
		{
			return new DefaultReadOnlyMemoryOwner<T>(memory, owner);
		}

		/// <summary>
		/// Wrap a <see cref="ReadOnlyMemory{T}"/> as a <see cref="IReadOnlyMemoryOwner{T}"/>
		/// </summary>
		public static IReadOnlyMemoryOwner<T> Create<T>(ReadOnlyMemory<T> memory, IDisposable? owner = null)
		{
			return new DefaultReadOnlyMemoryOwner<T>(memory, owner);
		}

		// Wraps a ReadOnlyMemoryOwner in a stream
		class ReadOnlyMemoryOwnerStream : ReadOnlyMemoryStream
		{
			readonly IReadOnlyMemoryOwner<byte> _owner;

			public ReadOnlyMemoryOwnerStream(IReadOnlyMemoryOwner<byte> owner)
				: base(owner.Memory) => _owner = owner;

			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				if (disposing)
				{
					_owner.Dispose();
				}
			}
		}

		/// <summary>
		/// Create a stream to wrap a storage object. The object will be disposed when the stream is closed.
		/// </summary>
		/// <param name="owner">Storage object to wrap</param>
		/// <returns>Stream for the given storage object</returns>
		public static Stream AsStream(this IReadOnlyMemoryOwner<byte> owner) => new ReadOnlyMemoryOwnerStream(owner);

		// Creates a slice of a storage object
		class ReadOnlyMemoryOwnerSlice<T> : IReadOnlyMemoryOwner<T>
		{
			readonly IReadOnlyMemoryOwner<T> _inner;

			public ReadOnlyMemory<T> Memory { get; }

			public ReadOnlyMemoryOwnerSlice(IReadOnlyMemoryOwner<T> inner, ReadOnlyMemory<T> memory)
			{
				_inner = inner;
				Memory = memory;
			}

			public void Dispose() => _inner.Dispose();
		}

		/// <summary>
		/// Creates a slice of an existing memory object.
		/// </summary>
		/// <param name="owner">Storage object to wrap</param>
		/// <param name="offset">Offset of the slice</param>
		/// <param name="length">Length to take for the slice</param>
		/// <returns>Stream for the given storage object</returns>
		public static IReadOnlyMemoryOwner<T> Slice<T>(this IReadOnlyMemoryOwner<T> owner, int offset, int? length)
		{
			int maxLength = owner.Memory.Length - offset;
			int actualLength = length.HasValue ? Math.Min(length.Value, maxLength) : maxLength;
			if (offset == 0 && actualLength == owner.Memory.Length)
			{
				return owner;
			}
			else
			{
				return new ReadOnlyMemoryOwnerSlice<T>(owner, owner.Memory.Slice(offset, actualLength));
			}
		}
	}
}
