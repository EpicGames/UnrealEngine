// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json;

namespace EpicGames.Core
{
	/// <summary>
	/// Extensions for parsing values out of generic dictionary objects
	/// </summary>
	public static class JsonElementExtensions
	{
		/// <summary>
		/// Checks if the element has a property with the given value
		/// </summary>
		/// <param name="element">Element to search</param>
		/// <param name="name">Name of the property</param>
		/// <param name="value">Expected value of the property</param>
		/// <returns>True if the property exists and matches</returns>
		public static bool HasStringProperty(this JsonElement element, ReadOnlySpan<byte> name, string value)
		{
			JsonElement property;
			return element.ValueKind == JsonValueKind.Object && element.TryGetProperty(name, out property) && property.ValueEquals(value);
		}

		/// <summary>
		/// Checks if the element has a property with the given value
		/// </summary>
		/// <param name="element">Element to search</param>
		/// <param name="name">Name of the property</param>
		/// <param name="value">Expected value of the property</param>
		/// <returns>True if the property exists and matches</returns>
		public static bool HasStringProperty(this JsonElement element, string name, string value)
		{
			JsonElement property;
			return element.ValueKind == JsonValueKind.Object && element.TryGetProperty(name, out property) && property.ValueEquals(value);
		}

		/// <summary>
		/// Gets a property value of a certain type
		/// </summary>
		/// <param name="element">The element to get a property from</param>
		/// <param name="name">Name of the element</param>
		/// <param name="valueKind">The required type of property</param>
		/// <param name="value">Value of the property</param>
		/// <returns>True if the property exists and was a string</returns>
		public static bool TryGetProperty(this JsonElement element, ReadOnlySpan<byte> name, JsonValueKind valueKind, [NotNullWhen(true)] out JsonElement value)
		{
			JsonElement property;
			if (element.TryGetProperty(name, out property) && property.ValueKind == valueKind)
			{
				value = property;
				return true;
			}
			else
			{
				value = new JsonElement();
				return false;
			}
		}

		/// <summary>
		/// Gets a property value of a certain type
		/// </summary>
		/// <param name="element">The element to get a property from</param>
		/// <param name="name">Name of the element</param>
		/// <param name="valueKind">The required type of property</param>
		/// <param name="value">Value of the property</param>
		/// <returns>True if the property exists and was a string</returns>
		public static bool TryGetProperty(this JsonElement element, string name, JsonValueKind valueKind, [NotNullWhen(true)] out JsonElement value)
		{
			JsonElement property;
			if (element.TryGetProperty(name, out property) && property.ValueKind == valueKind)
			{
				value = property;
				return true;
			}
			else
			{
				value = new JsonElement();
				return false;
			}
		}

		/// <summary>
		/// Gets a string property value
		/// </summary>
		/// <param name="element">The element to get a property from</param>
		/// <param name="name">Name of the element</param>
		/// <param name="value">Value of the property</param>
		/// <returns>True if the property exists and was a string</returns>
		public static bool TryGetStringProperty(this JsonElement element, ReadOnlySpan<byte> name, [NotNullWhen(true)] out string? value)
		{
			JsonElement property;
			if (element.TryGetProperty(name, out property) && property.ValueKind == JsonValueKind.String)
			{
				value = property.GetString()!;
				return true;
			}
			else
			{
				value = null;
				return false;
			}
		}

		/// <summary>
		/// Gets a string property value
		/// </summary>
		/// <param name="element">The element to get a property from</param>
		/// <param name="name">Name of the element</param>
		/// <param name="value">Value of the property</param>
		/// <returns>True if the property exists and was a string</returns>
		public static bool TryGetStringProperty(this JsonElement element, string name, [NotNullWhen(true)] out string? value)
		{
			return TryGetStringProperty(element, Encoding.UTF8.GetBytes(name).AsSpan(), out value);
		}

		/// <summary>
		/// Gets a property value from a document or subdocument, indicated with dotted notation
		/// </summary>
		/// <param name="element">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outElement">Receives the nexted element</param>
		/// <returns>True if the property exists and was of the correct type</returns>
		public static bool TryGetNestedProperty(this JsonElement element, ReadOnlySpan<byte> name, [NotNullWhen(true)] out JsonElement outElement)
		{
			int dotIdx = name.IndexOf((byte)'.');
			if (dotIdx == -1)
			{
				return element.TryGetProperty(name, out outElement);
			}

			JsonElement docValue;
			if (element.TryGetProperty(name.Slice(0, dotIdx), out docValue) && docValue.ValueKind == JsonValueKind.Object)
			{
				return TryGetNestedProperty(docValue, name.Slice(dotIdx + 1), out outElement);
			}

			outElement = new JsonElement();
			return false;
		}

		/// <summary>
		/// Gets a property value from a document or subdocument, indicated with dotted notation
		/// </summary>
		/// <param name="element">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outElement">Receives the nexted element</param>
		/// <returns>True if the property exists and was of the correct type</returns>
		public static bool TryGetNestedProperty(this JsonElement element, string name, [NotNullWhen(true)] out JsonElement outElement)
		{
			return TryGetNestedProperty(element, Encoding.UTF8.GetBytes(name).AsSpan(), out outElement);
		}

		/// <summary>
		/// Gets an int32 value from the document
		/// </summary>
		/// <param name="element">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property was retrieved</returns>
		public static bool TryGetNestedProperty(this JsonElement element, string name, out int outValue)
		{
			JsonElement value;
			if (element.TryGetNestedProperty(name, out value) && value.ValueKind == JsonValueKind.Number)
			{
				outValue = value.GetInt32();
				return true;
			}
			else
			{
				outValue = 0;
				return false;
			}
		}

		/// <summary>
		/// Gets a string value from the document
		/// </summary>
		/// <param name="element">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property was retrieved</returns>
		public static bool TryGetNestedProperty(this JsonElement element, string name, [NotNullWhen(true)] out string? outValue)
		{
			JsonElement value;
			if (element.TryGetNestedProperty(name, out value) && value.ValueKind == JsonValueKind.String)
			{
				outValue = value.GetString();
				return outValue != null;
			}
			else
			{
				outValue = null;
				return false;
			}
		}
	}
}
