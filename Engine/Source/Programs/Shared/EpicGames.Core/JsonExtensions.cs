// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Diagnostics;
using System.Text.Json;

#pragma warning disable CA1045 // Do not pass types by reference

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for utf8 json reader/writer
	/// </summary>
	public static class JsonExtensions
	{
		/// <summary>
		/// Tries to read a property name then move to the value token
		/// </summary>
		/// <param name="reader">Token reader</param>
		/// <param name="propertyName">Receives the property name on success</param>
		/// <returns>True if the read succeeded</returns>
		public static bool TryReadNextPropertyName(scoped ref Utf8JsonReader reader, out ReadOnlySpan<byte> propertyName)
		{
			if (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
			{
				propertyName = reader.GetUtf8String();
				return reader.Read();
			}
			else
			{
				propertyName = [];
				return false;
			}
		}

		/// <summary>
		/// Reads a Utf8 string from the current json token
		/// </summary>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static ReadOnlySpan<byte> GetUtf8String(this Utf8JsonReader reader)
		{
			ReadOnlySpan<byte> span;

			if (reader.HasValueSequence)
			{
				ReadOnlySequence<byte> sequence = reader.ValueSequence;
				byte[] buffer = new byte[checked((int)sequence.Length)];
				sequence.CopyTo(buffer);
				span = buffer;
			}
			else
			{
				span = reader.ValueSpan;
			}

			// Scan for the first backslash
			for (int firstIdx = 0; firstIdx < span.Length; firstIdx++)
			{
				if (span[firstIdx] == '\\')
				{
					// Compute the length of the unescaped string
					int length = span.Length;
					for (int idx = firstIdx; idx < span.Length; idx++)
					{
						if (span[idx] == '\\')
						{
							switch (span[idx + 1])
							{
								case (byte)'"':
								case (byte)'\\':
								case (byte)'/':
								case (byte)'b':
								case (byte)'f':
								case (byte)'n':
								case (byte)'r':
								case (byte)'t':
									idx += 2 - 1;
									length--;
									break;
								case (byte)'u':
									int c = (StringUtils.GetHexDigit(span[idx + 2]) << 12) + (StringUtils.GetHexDigit(span[idx + 3]) << 8) + (StringUtils.GetHexDigit(span[idx + 4]) << 4) + StringUtils.GetHexDigit(span[idx + 5]);
									idx += 6 - 1;
									length += GetUtf8ByteCount(c) - 6;
									break;
							}
						}
					}

					// Write the new string
					byte[] result = new byte[length];
					span.Slice(0, firstIdx).CopyTo(result);

					// Trim the output spans
					ReadOnlySpan<byte> input = span.Slice(firstIdx);
					Span<byte> output = result.AsSpan(firstIdx);

					// Compute the length of the unescaped string
					for (; ; )
					{
						switch (input[1])
						{
							case (byte)'"':
								input = input.Slice(2);
								output[0] = (byte)'\"';
								output = output.Slice(1);
								break;
							case (byte)'\\':
								input = input.Slice(2);
								output[0] = (byte)'\\';
								output = output.Slice(1);
								break;
							case (byte)'/':
								input = input.Slice(2);
								output[0] = (byte)'/';
								output = output.Slice(1);
								break;
							case (byte)'b':
								input = input.Slice(2);
								output[0] = (byte)'\b';
								output = output.Slice(1);
								break;
							case (byte)'f':
								input = input.Slice(2);
								output[0] = (byte)'\f';
								output = output.Slice(1);
								break;
							case (byte)'n':
								input = input.Slice(2);
								output[0] = (byte)'\n';
								output = output.Slice(1);
								break;
							case (byte)'r':
								input = input.Slice(2);
								output[0] = (byte)'\r';
								output = output.Slice(1);
								break;
							case (byte)'t':
								input = input.Slice(2);
								output[0] = (byte)'\t';
								output = output.Slice(1);
								break;
							case (byte)'u':
								int c = (StringUtils.GetHexDigit(input[2]) << 12) + (StringUtils.GetHexDigit(input[3]) << 8) + (StringUtils.GetHexDigit(input[4]) << 4) + StringUtils.GetHexDigit(input[5]);
								input = input.Slice(6);
								int len = GetUtf8Bytes(c, output);
								output = output.Slice(len);
								break;
						}

						int escapeIdx = input.IndexOf((byte)'\\');
						if (escapeIdx == -1)
						{
							Debug.Assert(input.Length == output.Length);
							input.CopyTo(output);
							return result;
						}

						input.Slice(0, escapeIdx).CopyTo(output);
						input = input.Slice(escapeIdx);
						output = output.Slice(escapeIdx);
					}
				}
			}

			return span;
		}

		static int GetUtf8Bytes(int c, Span<byte> output)
		{
			if (c <= 0x7f)
			{
				output[0] = (byte)c;
				return 1;
			}
			else if (c <= 0x7ff)
			{
				output[0] = (byte)(0b11000000 | (c >> 6));
				output[1] = (byte)(0b10000000 | (c & 0x1f));
				return 2;
			}
			else if (c <= 0xffff)
			{
				output[0] = (byte)(0b11100000 | (c >> 12));
				output[1] = (byte)(0b10000000 | ((c >> 6) & 0x1f));
				output[2] = (byte)(0b10000000 | (c & 0x1f));
				return 3;
			}
			else
			{
				output[0] = (byte)(0b11110000 | (c >> 18));
				output[1] = (byte)(0b10000000 | ((c >> 12) & 0x1f));
				output[2] = (byte)(0b10000000 | ((c >> 6) & 0x1f));
				output[3] = (byte)(0b10000000 | (c & 0x1f));
				return 4;
			}
		}

		static int GetUtf8ByteCount(int c)
		{
			if (c <= 0x7f)
			{
				return 1;
			}
			else if (c <= 0x7ff)
			{
				return 2;
			}
			else if (c <= 0xffff)
			{
				return 3;
			}
			else
			{
				return 4;
			}
		}
	}
}
