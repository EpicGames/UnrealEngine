// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Sets a name used to discriminate between classes derived from a base class
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class JsonDiscriminatorAttribute : Attribute
	{
		/// <summary>
		/// Name to use to discriminate between different classes
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name to use to discriminate between different classes</param>
		public JsonDiscriminatorAttribute(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Marks a class as the base class of a hierarchy, allowing any known subclasses to be serialized for it.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class JsonKnownTypesAttribute : Attribute
	{
		/// <summary>
		/// Array of derived classes
		/// </summary>
		public Type[] Types { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="types">Array of derived classes</param>
		public JsonKnownTypesAttribute(params Type[] types)
		{
			Types = types;
		}
	}

	/// <summary>
	/// Serializer for polymorphic objects
	/// </summary>
	public class JsonKnownTypesConverter<T> : JsonConverter<T>
	{
		/// <summary>
		/// Name of the 'Type' field used to store discriminators in the serialized objects
		/// </summary>
		const string TypePropertyName = "Type";

		/// <summary>
		/// Mapping from discriminator to class type
		/// </summary>
		readonly Dictionary<string, Type> _discriminatorToType;

		/// <summary>
		/// Mapping from class type to discriminator
		/// </summary>
		readonly Dictionary<Type, JsonEncodedText> _typeToDiscriminator;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="knownTypes">Set of derived class types</param>
		public JsonKnownTypesConverter(IEnumerable<Type> knownTypes)
		{
			_discriminatorToType = new Dictionary<string, Type>(StringComparer.OrdinalIgnoreCase);
			foreach (Type knownType in knownTypes)
			{
				if (!typeof(T).IsAssignableFrom(knownType))
				{
					throw new InvalidOperationException($"{knownType} is not derived from {typeof(T)}");
				}
				foreach (JsonDiscriminatorAttribute discriminatorAttribute in knownType.GetCustomAttributes(typeof(JsonDiscriminatorAttribute), true).OfType<JsonDiscriminatorAttribute>())
				{
					_discriminatorToType.Add(discriminatorAttribute.Name, knownType);
				}
			}

			_typeToDiscriminator = _discriminatorToType.ToDictionary(x => x.Value, x => JsonEncodedText.Encode(x.Key));
		}

		/// <summary>
		/// Determines whether the given type can be converted
		/// </summary>
		/// <param name="typeToConvert">Type to convert</param>
		/// <returns>True if the type can be converted</returns>
		public override bool CanConvert(Type typeToConvert)
		{
			return typeToConvert == typeof(T) || _typeToDiscriminator.ContainsKey(typeToConvert);
		}

		/// <summary>
		/// Reads an object from the given input stream
		/// </summary>
		/// <param name="reader">UTF8 reader</param>
		/// <param name="typeToConvert">Type to be read from the stream</param>
		/// <param name="options">Options for serialization</param>
		/// <returns>Instance of the serialized object</returns>
		public override T Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			using (JsonDocument document = JsonDocument.ParseValue(ref reader))
			{
				JsonElement element;
				if (!TryGetPropertyRespectingCase(document.RootElement, TypePropertyName, options, out element))
				{
					throw new JsonException($"Missing '{TypePropertyName}' field for serializing {typeToConvert.Name}");
				}

				Type? targetType;
				if (!_discriminatorToType.TryGetValue(element.GetString()!, out targetType))
				{
					throw new JsonException($"Invalid '{TypePropertyName}' field ('{element.GetString()}') for serializing {typeToConvert.Name}");
				}

				string text = document.RootElement.GetRawText();
				T result = (T)JsonSerializer.Deserialize(text, targetType, options)!;
				return result;
			}
		}

		/// <summary>
		/// Finds a property within an object, ignoring case
		/// </summary>
		/// <param name="element">The object to search</param>
		/// <param name="propertyName">Name of the property to search</param>
		/// <param name="options">Options for serialization. The <see cref="JsonSerializerOptions.PropertyNameCaseInsensitive"/> property defines whether a case insensitive scan will be performed.</param>
		/// <param name="outElement">The property value, if successful</param>
		/// <returns>True if the property was found, false otherwise</returns>
		static bool TryGetPropertyRespectingCase(JsonElement element, string propertyName, JsonSerializerOptions options, out JsonElement outElement)
		{
			JsonElement newElement;
			if (element.TryGetProperty(propertyName, out newElement))
			{
				outElement = newElement;
				return true;
			}

			if (options.PropertyNameCaseInsensitive)
			{
				foreach (JsonProperty property in element.EnumerateObject())
				{
					if (String.Equals(property.Name, propertyName, StringComparison.OrdinalIgnoreCase))
					{
						outElement = property.Value;
						return true;
					}
				}
			}

			outElement = new JsonElement();
			return false;
		}

		/// <summary>
		/// Writes an object to the given output stream
		/// </summary>
		/// <param name="writer">UTF8 writer</param>
		/// <param name="value">Object to be serialized to the stream</param>
		/// <param name="options">Options for serialization</param>
		public override void Write(Utf8JsonWriter writer, T value, JsonSerializerOptions options)
		{
			if (value == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				Type valueType = value.GetType();

				writer.WriteStartObject();
				writer.WriteString(options.PropertyNamingPolicy?.ConvertName(TypePropertyName) ?? TypePropertyName, _typeToDiscriminator[valueType]);

				byte[] bytes = JsonSerializer.SerializeToUtf8Bytes(value, valueType, options);
				using (JsonDocument document = JsonDocument.Parse(bytes))
				{
					foreach (JsonProperty property in document.RootElement.EnumerateObject())
					{
						property.WriteTo(writer);
					}
				}

				writer.WriteEndObject();
			}
		}
	}

	/// <summary>
	/// Factory for JsonKnownTypes converters
	/// </summary>
	public class JsonKnownTypesConverterFactory : JsonConverterFactory
	{
		/// <summary>
		/// Determines whether it's possible to create a converter for the given type
		/// </summary>
		/// <param name="typeToConvert">The type being converted</param>
		/// <returns>True if the type can be converted</returns>
		public override bool CanConvert(Type typeToConvert)
		{
			object[] attributes = typeToConvert.GetCustomAttributes(typeof(JsonKnownTypesAttribute), false);
			return attributes.Length > 0;
		}

		/// <summary>
		/// Creates a converter for the given type
		/// </summary>
		/// <param name="typeToConvert">The type being converted</param>
		/// <param name="options">Options for serialization</param>
		/// <returns>New converter for the requested type</returns>
		public override JsonConverter CreateConverter(Type typeToConvert, JsonSerializerOptions options)
		{
			object[] attributes = typeToConvert.GetCustomAttributes(typeof(JsonKnownTypesAttribute), false);
			Type genericType = typeof(JsonKnownTypesConverter<>).MakeGenericType(typeToConvert);
			return (JsonConverter)Activator.CreateInstance(genericType, attributes.SelectMany(x => ((JsonKnownTypesAttribute)x).Types))!;
		}
	}
}
