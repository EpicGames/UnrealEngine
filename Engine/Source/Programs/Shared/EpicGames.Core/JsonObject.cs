// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text.Json;
using System.Collections.Specialized;
using System.Collections;
using System.IO;
using System.Text;
using System.Text.Encodings.Web;
using System.Buffers;

namespace EpicGames.Core
{
	using SystemJsonObject = System.Text.Json.Nodes.JsonObject;
	using SystemJsonNode = System.Text.Json.Nodes.JsonNode;
	using SystemJsonValue = System.Text.Json.Nodes.JsonValue;
	using SystemJsonArray = System.Text.Json.Nodes.JsonArray;

	/// <summary>
	/// Extension methods for OrderedDictionary to have functionality more similar to Dictionary.
	/// </summary>
	public static class OrderedDictionaryExtensions
	{
		/// <summary>
		/// Tries to get a value associated with a key in the OrderedDictionary. 
		/// </summary>
		/// <param name="dictionary"></param>
		/// <param name="key"> The key to find a value for.</param>
		/// <param name="value"> The value associated with the key. Is null if no key is found.</param>
		/// <returns> Returns true if the associated value with the key exists and is found. Else returns false.</returns>
		public static bool TryGetValue(this OrderedDictionary dictionary, object key, [NotNullWhen(true)] out object? value)
		{
			if (dictionary.Contains(key))
			{
				value = dictionary[key]!;
				return true;
			}
			value = null;
			return false;
		}

		private static object? DeepCopyHelper(object? original)
		{
			if (original is null)
			{
				return null;
			}

			if (original is OrderedDictionary dictionary)
			{
				OrderedDictionary copy = new(StringComparer.InvariantCultureIgnoreCase);
				foreach (DictionaryEntry entry in dictionary)
				{
					copy.Add(entry.Key, DeepCopyHelper(entry.Value));
				}
				return copy;
			}

			if (original is Array array)
			{
				Array copy = Array.CreateInstance(array.GetType().GetElementType()!, array.Length);
				for (int index = 0; index < array.Length; ++index)
				{
					copy.SetValue(DeepCopyHelper(array.GetValue(index)), index);
				}
				return copy;
			}
			// It's a value type or a string, it's safe to just copy it 
			return original;
		}
		/// <summary>
		/// Creates a deep copy of all the elements in an OrderedDictionary.
		/// </summary>
		/// <param name="dictionary"></param>
		/// <returns> An ordered dictionary with a deep copy of elements from dictionary.</returns>
		public static OrderedDictionary CreateDeepCopy(this OrderedDictionary dictionary)
		{
			OrderedDictionary copy = new(StringComparer.InvariantCultureIgnoreCase);
			foreach (DictionaryEntry entry in dictionary)
			{
				copy.Add(entry.Key, DeepCopyHelper(entry.Value));
			}
			return copy;
		}
	}

	/// <summary>
	/// Stores a JSON object in memory
	/// </summary>
	public class JsonObject
	{
		readonly OrderedDictionary _rawOrderedObject;

		/// <summary>
		/// Default constructor. Use this to create new JsonObject elements and use the AddOrSetFieldValue API to add values to this JsonObject.
		/// </summary>
		public JsonObject()
		{
			_rawOrderedObject = new OrderedDictionary(StringComparer.InvariantCultureIgnoreCase);
		}

		/// <summary>
		/// Construct a JSON object from the OrderedDictionary obtained from reading a file on disk or parsing valid json text.
		/// </summary>
		/// <param name="inRawObject">Raw object parsed from disk</param>
		/// <param name="createCopy">raw object should be copied or not</param>
		private JsonObject(OrderedDictionary inRawObject, bool createCopy)
		{
			if (createCopy)
			{
				_rawOrderedObject = inRawObject.CreateDeepCopy();
			}
			else
			{
				_rawOrderedObject = inRawObject;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="element"></param>
		public JsonObject(JsonElement element)
		{
			_rawOrderedObject = new OrderedDictionary(StringComparer.InvariantCultureIgnoreCase);
			foreach (JsonProperty property in element.EnumerateObject())
			{
				_rawOrderedObject[property.Name] = ParseElement(property.Value);
			}
		}

		/// <summary>
		/// Override of the Equals method to check for equality between 2 JsonObject instances.
		/// </summary>
		/// <param name="obj"></param>
		/// <returns> Returns true if this object and obj are equal. Else returns false.</returns>
		public override bool Equals(object? obj)
		{
			if (obj is null || GetType() != obj.GetType())
			{
				return false;
			}
			JsonObject jsonObject = (JsonObject)obj;
			if (_rawOrderedObject.Count != jsonObject._rawOrderedObject.Count)
			{
				return false;
			}

			foreach (DictionaryEntry entry in _rawOrderedObject)
			{
				if (!jsonObject._rawOrderedObject.Contains(entry.Key))
				{
					return false;
				}
				if (!entry.Value!.Equals(jsonObject._rawOrderedObject[entry.Key!]))
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Returns the calculated hash code for a JsonObject instance.
		/// </summary>
		/// <returns> Returns the hash code for a JsonObject instance. </returns>
		public override int GetHashCode()
		{
			return HashCode.Combine(_rawOrderedObject);
		}

		/// <summary>
		/// Parse an individual element
		/// </summary>
		/// <param name="element"></param>
		/// <returns></returns>
		public static object? ParseElement(JsonElement element)
		{
			switch (element.ValueKind)
			{
				case JsonValueKind.Array:
					{
						int len = element.GetArrayLength();
						object?[] elementArray = new object[len];
						int index = 0;
						foreach (JsonElement x in element.EnumerateArray())
						{
							elementArray[index++] = ParseElement(x);
						}
						return elementArray;
					}
				case JsonValueKind.Number:
					return element.GetDouble();
				case JsonValueKind.Object:
					OrderedDictionary dictionary = new(StringComparer.InvariantCultureIgnoreCase);
					foreach (JsonProperty property in element.EnumerateObject())
					{
						dictionary.Add(property.Name, ParseElement(property.Value));
					}
					return dictionary;
				case JsonValueKind.String:
					return element.GetString();
				case JsonValueKind.False:
					return false;
				case JsonValueKind.True:
					return true;
				case JsonValueKind.Null:
					return null;
				default:
					throw new NotImplementedException();
			}
		}

		private static SystemJsonNode ToJsonNode(object? obj)
		{
			// All values in the JsonObject are either parsed from a string, read from a file or set/added with the API
			// All values at this point must be supported and the correct types 
			switch (obj)
			{
				case OrderedDictionary objDictionary:
					SystemJsonObject dictionaryJson = [];
					foreach (object? key in objDictionary.Keys)
					{
						dictionaryJson.Add((string)key, ToJsonNode(objDictionary[key]));
					}
					return dictionaryJson;
				case Array objArray:
					SystemJsonArray tempJsonArray = [];
					foreach (object? element in objArray)
					{
						tempJsonArray.Add(ToJsonNode(element));
					}
					return tempJsonArray;
				default:
					// We support null as per the json spec 
					return SystemJsonValue.Create(obj)!;
			}
		}

		/// <summary>
		/// Converts a JsonObject instance to a System.Text.Json.Nodes.JsonObject instance. Users can then use .NET facilities like Utf8JsonWriter to interact with this JsonObject.
		/// </summary>
		/// <returns> Returns the System.Text.Json.Nodes.JsonObject representation of this object.</returns>
		public SystemJsonObject ToSystemJsonObject()
		{
			SystemJsonObject returnObject = [];
			foreach (DictionaryEntry entry in _rawOrderedObject)
			{
				returnObject.Add((string)entry.Key, ToJsonNode(entry.Value));
			}
			return returnObject;
		}

		static JsonDocument ParseUtf8File(string path)
		{
			using FileStream fs = new(path, FileMode.Open, FileAccess.Read, FileShare.Read, bufferSize: 1, FileOptions.SequentialScan);

			int len = checked((int)fs.Length);
			byte[] buf = len <= 128 * 1024 ? new byte[len] : ArrayPool<byte>.Shared.Rent(len);
			try
			{
				fs.ReadExactly(buf.AsSpan(0, len));
				int off = (len >= 3 && buf[0] == 0xEF && buf[1] == 0xBB && buf[2] == 0xBF) ? 3 : 0; // Skip UTF-8 BOM if present

				try
				{
					return JsonDocument.Parse(new ReadOnlyMemory<byte>(buf, off, len - off), new() { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip });
				}
				catch (JsonException)
				{
					throw;
				}
				catch (Exception ex)
				{
					throw new JsonException($"Unable to parse {path}: {ex.Message}", path, null, null, ex);
				}
			}
			finally
			{
				if (buf.Length > 128 * 1024)
				{
					ArrayPool<byte>.Shared.Return(buf);
				}
			}
		}
		/// <summary>
		/// Read a JSON file from disk and construct a JsonObject from it
		/// </summary>
		/// <param name="file">File to read from</param>
		/// <returns>New JsonObject instance</returns>
		public static JsonObject Read(FileReference file)
		{
			JsonDocument doc = ParseUtf8File(file.FullName);
			try
			{
				return new JsonObject(doc.RootElement);
			}
			catch (Exception ex)
			{
				throw new JsonException($"Unable to parse {file}: {ex.Message}", file.FullName, null, null, ex);
			}
		}

		/// <summary>
		/// Tries to read a JSON file from disk
		/// </summary>
		/// <param name="fileName">File to read from</param>
		/// <param name="result">On success, receives the parsed object</param>
		/// <returns>True if the file was read, false otherwise</returns>
		public static bool TryRead(FileReference fileName, [NotNullWhen(true)] out JsonObject? result)
		{
			result = null;

			if (!FileReference.Exists(fileName))
			{
				return false;
			}

			try
			{
				JsonDocument doc = ParseUtf8File(fileName.FullName);
				result = new JsonObject(doc.RootElement);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		/// <summary>
		/// Parse a JsonObject from the given raw text string
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <returns>New JsonObject instance</returns>
		public static JsonObject Parse(string text)
		{
			try
			{
				JsonDocument document = JsonDocument.Parse(text, new JsonDocumentOptions { AllowTrailingCommas = true });
				return new JsonObject(document.RootElement);
			}
			catch (JsonException)
			{
				throw;
			}
			catch (Exception ex)
			{
				throw new JsonException($"Failed to parse json text '{text}'. {ex.Message}", ex);
			}
		}

		/// <summary>
		/// Try to parse a JsonObject from the given raw text string
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <param name="result">On success, receives the new JsonObject</param>
		/// <returns>True if the object was parsed</returns>
		public static bool TryParse(string text, [NotNullWhen(true)] out JsonObject? result)
		{
			try
			{
				result = Parse(text);
				return true;
			}
			catch (Exception)
			{
				result = null;
				return false;
			}
		}

		/// <summary>
		/// List of key names in this object
		/// </summary>
		public IEnumerable<string> KeyNames => _rawOrderedObject.Keys.Cast<string>();

		/// <summary>
		/// Gets a string field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public string GetStringField(string fieldName)
		{
			string? stringValue;
			if (!TryGetStringField(fieldName, out stringValue))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return stringValue;
		}

		/// <summary>
		/// Tries to read a string field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetStringField(string fieldName, [NotNullWhen(true)] out string? result)
		{
			object? rawValue;
			if (_rawOrderedObject.TryGetValue(fieldName, out rawValue) && (rawValue is string strValue))
			{
				result = strValue;
				return true;
			}
			else
			{
				result = null;
				return false;
			}
		}

		/// <summary>
		/// Gets a string array field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public string[] GetStringArrayField(string fieldName)
		{
			string[]? stringValues;
			if (!TryGetStringArrayField(fieldName, out stringValues))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return stringValues;
		}

		/// <summary>
		/// Tries to read a string array field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetStringArrayField(string fieldName, [NotNullWhen(true)] out string[]? result)
		{
			object? rawValue;
			if (_rawOrderedObject.TryGetValue(fieldName, out rawValue) && (rawValue is IEnumerable<object> enumValue) && enumValue.All(x => x is string))
			{
				result = [.. enumValue.Select(x => (string)x)];
				return true;
			}
			else
			{
				result = null;
				return false;
			}
		}

		/// <summary>
		/// Gets a boolean field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public bool GetBoolField(string fieldName)
		{
			bool boolValue;
			if (!TryGetBoolField(fieldName, out boolValue))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return boolValue;
		}

		/// <summary>
		/// Tries to read a bool field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetBoolField(string fieldName, out bool result)
		{
			object? rawValue;
			if (_rawOrderedObject.TryGetValue(fieldName, out rawValue) && (rawValue is bool boolValue))
			{
				result = boolValue;
				return true;
			}
			else
			{
				result = false;
				return false;
			}
		}

		/// <summary>
		/// Gets an integer field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public int GetIntegerField(string fieldName)
		{
			int integerValue;
			if (!TryGetIntegerField(fieldName, out integerValue))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return integerValue;
		}

		/// <summary>
		/// Tries to read an integer field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetIntegerField(string fieldName, out int result)
		{
			object? rawValue;
			if (!_rawOrderedObject.TryGetValue(fieldName, out rawValue) || !Int32.TryParse(rawValue?.ToString(), out result))
			{
				result = 0;
				return false;
			}
			return true;
		}

		/// <summary>
		/// Tries to read an unsigned integer field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetUnsignedIntegerField(string fieldName, out uint result)
		{
			object? rawValue;
			if (!_rawOrderedObject.TryGetValue(fieldName, out rawValue) || !UInt32.TryParse(rawValue?.ToString(), out result))
			{
				result = 0;
				return false;
			}
			return true;
		}

		/// <summary>
		/// Gets a double field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public double GetDoubleField(string fieldName)
		{
			double doubleValue;
			if (!TryGetDoubleField(fieldName, out doubleValue))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return doubleValue;
		}

		/// <summary>
		/// Tries to read a double field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetDoubleField(string fieldName, out double result)
		{
			object? rawValue;
			if (!_rawOrderedObject.TryGetValue(fieldName, out rawValue) || !Double.TryParse(rawValue?.ToString(), out result))
			{
				result = 0.0;
				return false;
			}
			return true;
		}

		/// <summary>
		/// Gets an enum field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public T GetEnumField<T>(string fieldName) where T : struct
		{
			T enumValue;
			if (!TryGetEnumField(fieldName, out enumValue))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return enumValue;
		}

		/// <summary>
		/// Tries to read an enum field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetEnumField<T>(string fieldName, out T result) where T : struct
		{
			string? stringValue;
			if (!TryGetStringField(fieldName, out stringValue) || !Enum.TryParse<T>(stringValue, true, out result))
			{
				result = default;
				return false;
			}
			return true;
		}

		/// <summary>
		/// Tries to read an enum array field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <param name="errors">On failure, details the error(s) that occurred</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetEnumArrayField<T>(string fieldName, [NotNullWhen(true)] out T[]? result, [NotNullWhen(false)] out string? errors) where T : struct
		{
			if (!ContainsField(fieldName))
			{
				result = null;
				errors = $"Field '{fieldName}' not found";
				return false;
			}

			string[]? stringValues;
			if (!TryGetStringArrayField(fieldName, out stringValues))
			{
				result = null;
				errors = $"Field '{fieldName}' has an incorrect type - it must be an array of strings";
				return false;
			}

			T[] enumValues = new T[stringValues.Length];
			bool success = true;
			StringBuilder? errorBuilder = null;
			for (int idx = 0; idx < stringValues.Length; idx++)
			{
				if (!Enum.TryParse(stringValues[idx], true, out enumValues[idx]))
				{
					success = false;
					errorBuilder ??= new StringBuilder();
					errorBuilder.AppendLine($"Field '{fieldName}' entry at index {idx} '{stringValues[idx]}' is not a valid {typeof(T).Name}");
				}
			}

			if (!success)
			{
				result = null;
#pragma warning disable CS8602 // If !success then errorBuilder != null
				errors = errorBuilder.ToString();
#pragma warning restore CS8602
				return false;
			}

			result = enumValues;
			errors = null;
			return true;
		}

		/// <summary>
		/// Tries to read an enum array field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetEnumArrayField<T>(string fieldName, [NotNullWhen(true)] out T[]? result) where T : struct
			=> TryGetEnumArrayField(fieldName, out result, out _);

		/// <summary>
		/// Gets an object field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public JsonObject GetObjectField(string fieldName)
		{
			JsonObject? result;
			if (!TryGetObjectField(fieldName, out result))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return result;
		}

		/// <summary>
		/// Tries to read an object field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetObjectField(string fieldName, [NotNullWhen(true)] out JsonObject? result)
		{
			object? rawValue;
			if (_rawOrderedObject.TryGetValue(fieldName, out rawValue) && (rawValue is OrderedDictionary orderedDictValue))
			{
				result = new JsonObject(orderedDictValue, false);
				return true;
			}
			else
			{
				result = null;
				return false;
			}
		}

		/// <summary>
		/// Gets an object array field by the given name from the object, throwing an exception if it is not there or cannot be parsed.
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <returns>The field value</returns>
		public JsonObject[] GetObjectArrayField(string fieldName)
		{
			JsonObject[]? result;
			if (!TryGetObjectArrayField(fieldName, out result))
			{
				throw new JsonException($"Missing or invalid '{fieldName}' field");
			}
			return result;
		}

		/// <summary>
		/// Tries to read an object array field by the given name from the object
		/// </summary>
		/// <param name="fieldName">Name of the field to get</param>
		/// <param name="result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public bool TryGetObjectArrayField(string fieldName, [NotNullWhen(true)] out JsonObject[]? result)
		{
			object? rawValue;
			if (_rawOrderedObject.TryGetValue(fieldName, out rawValue) && (rawValue is IEnumerable<object> enumValue) && enumValue.All(x => x is OrderedDictionary))
			{
				result = [.. enumValue.Select(x => new JsonObject((OrderedDictionary)x, false))];
				return true;
			}
			else
			{
				result = null;
				return false;
			}
		}

		/// <summary>
		/// Checks if the provided field exists in this Json object.
		/// </summary>
		/// <param name="fieldName">Name of the field to check if it is contained. </param>
		/// <returns>True if the field exists in the JsonObject, false otherwise</returns>
		public bool ContainsField(string fieldName)
		{
			return _rawOrderedObject.Contains(fieldName);
		}

		/// <summary>
		/// Removes a field from the given json object.
		/// </summary>
		/// <param name="fieldName">Name of the field to remove </param>
		public void RemoveField(string fieldName)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			if (ContainsField(fieldName))
			{
				_rawOrderedObject.Remove(fieldName);
			}
		}

		/// <summary>
		/// Sets the integer value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The integer value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, int value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			_rawOrderedObject[fieldName] = Convert.ToDouble(value);
		}

		/// <summary>
		/// Sets the unsigned integer value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The unsigned integer value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, uint value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			_rawOrderedObject[fieldName] = Convert.ToDouble(value);
		}

		/// <summary>
		/// Sets the double value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The double value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, double value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			_rawOrderedObject[fieldName] = value;
		}

		/// <summary>
		/// Sets the bool value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The bool value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, bool value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			_rawOrderedObject[fieldName] = value;
		}

		/// <summary>
		/// Sets the string value of a field if it exists. Otherwise, adds the field and value.
		/// Note: If a null string is passed in as a value, it will be replaced with an empty string "" instead to ensure the field can still be added or set.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The string value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, string? value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			// We set null strings to empty strings to follow the behavior of EpicGames.Core.JsonWriter
			value ??= "";
			_rawOrderedObject[fieldName] = value;
		}

		/// <summary>
		/// Sets the enum value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The enum value to add or set. </param>
		public void AddOrSetFieldValue<T>(string fieldName, T value) where T : Enum
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}

			_rawOrderedObject[fieldName] = value.ToString();
		}

		/// <summary>
		/// Sets the JsonObject value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The JsonObject value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, JsonObject? value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}

			_rawOrderedObject[fieldName] = value?._rawOrderedObject.CreateDeepCopy();
		}

		/// <summary>
		/// Sets the string array value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The string array value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, string?[]? value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			if (value is null)
			{
				_rawOrderedObject[fieldName] = value;
				return;
			}
			// Contents of the strings array should never be null and should be "" instead to match behavior EpicGames.Core.JsonWriter
			string[] stringArray = [.. value.Select(x => x ?? "")];
			_rawOrderedObject[fieldName] = stringArray;
		}

		/// <summary>
		/// Sets the JsonObject array value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The JsonObject array value to add or set. </param>
		public void AddOrSetFieldValue(string fieldName, JsonObject[]? value)
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			if (value is null)
			{
				_rawOrderedObject[fieldName] = null;
				return;
			}
			List<OrderedDictionary?> objList = [];
			foreach (JsonObject? obj in value)
			{
				objList.Add(obj?._rawOrderedObject.CreateDeepCopy());
			}
			_rawOrderedObject[fieldName] = objList.ToArray();
		}

		/// <summary>
		/// Sets the enum array value of a field if it exists. Otherwise, adds the field and value.
		/// </summary>
		/// <param name="fieldName"> The field name for the value to add or update.</param>
		/// <param name="value"> The enum array value to add or set. </param>
		public void AddOrSetFieldValue<T>(string fieldName, T[]? value) where T : Enum
		{
			if (String.IsNullOrEmpty(fieldName))
			{
				return;
			}
			if (value is null)
			{
				_rawOrderedObject[fieldName] = null;
				return;
			}

			string[] stringArray = [.. value.Select(x => x.ToString()!)];
			_rawOrderedObject[fieldName] = stringArray;
		}

		/// <summary>
		/// Converts this Json Object to a string representation. This follows formatting of UE .uplugin files with 4 spaces for tabs and indentation enabled.
		/// IMPORTANT: If this JsonObject contains HTML, the returned string should NOT be used directly for HTML or a script. Read the note below.
		/// </summary>
		/// <returns>The formatted, prettified string representation of this Json Object.</returns>
		public string ToJsonString()
		{
			SystemJsonObject jsonObjectToWrite = ToSystemJsonObject();
			JsonWriterOptions options = new();
			options.Indented = true;
			// IMPORTANT: Utf8JsonWriter blocks certain characters like +, &, <, >,` from being escaped in a global block list
			// Best way around is to use the relaxed JavaScriptEncoder.UnsafeRelaxedJsonEscaping.
			// However this can have security implications if this string is ever written to an HTML page or script
			// https://learn.microsoft.com/en-us/dotnet/standard/serialization/system-text-json/character-encoding
			options.Encoder = JavaScriptEncoder.UnsafeRelaxedJsonEscaping;
			using MemoryStream jsonMemoryStream = new();
			using (Utf8JsonWriter writer = new(jsonMemoryStream, options))
			{
				jsonObjectToWrite.WriteTo(writer);
				writer.Flush();
			}
			// The Utf8JsonWriter doesn't format json the same as we want in UE, we massage the string to meet our standards 
			string jsonString = Encoding.UTF8.GetString(jsonMemoryStream.ToArray());
			string[] lines = jsonString.Split([Environment.NewLine], StringSplitOptions.None);
			StringBuilder jsonStringBuilder = new();
			foreach (string line in lines)
			{
				// Utf8JsonWriter uses 2 spaces for indents, we replace them with tabs here 
				int numLeadingSpaces = line.TakeWhile(x => x == ' ').Count();
				int numLeadingTabs = numLeadingSpaces / 2;
				jsonStringBuilder.Append('\t', numLeadingTabs);
				jsonStringBuilder.AppendLine(line.Substring(numLeadingSpaces));
			}

			return jsonStringBuilder.ToString();
		}
	}
}
