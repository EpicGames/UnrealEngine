// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface used to write JSON schema types. This is abstracted to allow multiple passes over the document structure, in order to optimize multiple references to the same type definition.
	/// </summary>
	public interface IJsonSchemaWriter
	{
		/// <inheritdoc cref="Utf8JsonWriter.WriteStartObject()"/>
		void WriteStartObject();

		/// <inheritdoc cref="Utf8JsonWriter.WriteStartObject(String)"/>
		void WriteStartObject(string name);

		/// <inheritdoc cref="Utf8JsonWriter.WriteEndObject()"/>
		void WriteEndObject();

		/// <inheritdoc cref="Utf8JsonWriter.WriteStartArray()"/>
		void WriteStartArray();

		/// <inheritdoc cref="Utf8JsonWriter.WriteStartArray(String)"/>
		void WriteStartArray(string name);

		/// <inheritdoc cref="Utf8JsonWriter.WriteEndObject()"/>
		void WriteEndArray();

		/// <inheritdoc cref="Utf8JsonWriter.WriteBoolean(String, Boolean)"/>
		void WriteBoolean(string name, bool value);

		/// <inheritdoc cref="Utf8JsonWriter.WriteString(String, String)"/>
		void WriteString(string key, string value);

		/// <inheritdoc cref="Utf8JsonWriter.WriteStringValue(String)"/>
		void WriteStringValue(string name);

		/// <summary>
		/// Serialize a type to JSON
		/// </summary>
		/// <param name="type"></param>
		void WriteType(JsonSchemaType type);
	}

	/// <summary>
	/// Implementation of a JSON schema. Implements draft 04 (latest supported by Visual Studio 2019).
	/// </summary>
	public class JsonSchema
	{
		class JsonTypeRefCollector : IJsonSchemaWriter
		{
			/// <summary>
			/// Reference counts for each type (max of 2)
			/// </summary>
			public Dictionary<JsonSchemaType, int> TypeRefCount { get; } = [];

			/// <inheritdoc/>
			public void WriteBoolean(string name, bool value) { }

			/// <inheritdoc/>
			public void WriteString(string key, string value) { }

			/// <inheritdoc/>
			public void WriteStringValue(string name) { }

			/// <inheritdoc/>
			public void WriteStartObject() { }

			/// <inheritdoc/>
			public void WriteStartObject(string name) { }

			/// <inheritdoc/>
			public void WriteEndObject() { }

			/// <inheritdoc/>
			public void WriteStartArray() { }

			/// <inheritdoc/>
			public void WriteStartArray(string name) { }

			/// <inheritdoc/>
			public void WriteEndArray() { }

			/// <inheritdoc/>
			public void WriteType(JsonSchemaType type)
			{
				if (type is not JsonSchemaPrimitiveType)
				{
					TypeRefCount.TryGetValue(type, out int refCount);
					if (refCount < 2)
					{
						TypeRefCount[type] = ++refCount;
					}
					if (refCount < 2)
					{
						type.Write(this);
					}
				}
			}
		}

		/// <summary>
		/// Implementation of <see cref="IJsonSchemaWriter"/>
		/// </summary>
		class JsonSchemaWriter : IJsonSchemaWriter
		{
			/// <summary>
			/// Raw Json output
			/// </summary>
			readonly Utf8JsonWriter _jsonWriter;

			/// <summary>
			/// Mapping of type to definition name
			/// </summary>
			readonly Dictionary<JsonSchemaType, string> _typeToDefinition;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="writer"></param>
			/// <param name="typeToDefinition"></param>
			public JsonSchemaWriter(Utf8JsonWriter writer, Dictionary<JsonSchemaType, string> typeToDefinition)
			{
				_jsonWriter = writer;
				_typeToDefinition = typeToDefinition;
			}

			/// <inheritdoc/>
			public void WriteBoolean(string name, bool value) => _jsonWriter.WriteBoolean(name, value);

			/// <inheritdoc/>
			public void WriteString(string key, string value) => _jsonWriter.WriteString(key, value);

			/// <inheritdoc/>
			public void WriteStringValue(string name) => _jsonWriter.WriteStringValue(name);

			/// <inheritdoc/>
			public void WriteStartObject() => _jsonWriter.WriteStartObject();

			/// <inheritdoc/>
			public void WriteStartObject(string name) => _jsonWriter.WriteStartObject(name);

			/// <inheritdoc/>
			public void WriteEndObject() => _jsonWriter.WriteEndObject();

			/// <inheritdoc/>
			public void WriteStartArray() => _jsonWriter.WriteStartArray();

			/// <inheritdoc/>
			public void WriteStartArray(string name) => _jsonWriter.WriteStartArray(name);

			/// <inheritdoc/>
			public void WriteEndArray() => _jsonWriter.WriteEndArray();

			/// <summary>
			/// Writes a type, either inline or as a reference to a definition elsewhere
			/// </summary>
			/// <param name="type"></param>
			public void WriteType(JsonSchemaType type)
			{
				if (_typeToDefinition.TryGetValue(type, out string? definition))
				{
					_jsonWriter.WriteString("$ref", $"#/definitions/{definition}");
				}
				else
				{
					type.Write(this);
				}
			}
		}

		/// <summary>
		/// Identifier for the schema
		/// </summary>
		public string? Id { get; set; }

		/// <summary>
		/// The root schema type
		/// </summary>
		public JsonSchemaType RootType { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Id for the schema</param>
		/// <param name="rootType"></param>
		public JsonSchema(string? id, JsonSchemaType rootType)
		{
			Id = id;
			RootType = rootType;
		}

		/// <summary>
		/// Write this schema to a byte array
		/// </summary>
		/// <param name="writer"></param>
		public void Write(Utf8JsonWriter writer)
		{
			// Determine reference counts for each type. Any type referenced at least twice will be split off into a separate definition.
			JsonTypeRefCollector refCollector = new();
			refCollector.WriteType(RootType);

			// Assign names to each type definition
			HashSet<string> definitionNames = [];
			Dictionary<JsonSchemaType, string> typeToDefinition = [];
			foreach ((JsonSchemaType type, int refCount) in refCollector.TypeRefCount)
			{
				if (refCount > 1)
				{
					string baseName = type.Name ?? "unnamed";

					string name = baseName;
					for (int idx = 1; !definitionNames.Add(name); idx++)
					{
						name = $"{baseName}{idx}";
					}

					typeToDefinition[type] = name;
				}
			}

			// Write the schema
			writer.WriteStartObject();
			writer.WriteString("$schema", "http://json-schema.org/draft-04/schema#");
			if (Id != null)
			{
				writer.WriteString("$id", Id);
			}

			JsonSchemaWriter schemaWriter = new(writer, typeToDefinition);
			RootType.Write(schemaWriter);

			if (typeToDefinition.Count > 0)
			{
				writer.WriteStartObject("definitions");
				foreach ((JsonSchemaType type, string refName) in typeToDefinition)
				{
					writer.WriteStartObject(refName);
					type.Write(schemaWriter);
					writer.WriteEndObject();
				}
				writer.WriteEndObject();
			}

			writer.WriteEndObject();
		}

		/// <summary>
		/// Write this schema to a stream
		/// </summary>
		/// <param name="stream">The output stream</param>
		public void Write(Stream stream)
		{
			using (Utf8JsonWriter writer = new(stream, new JsonWriterOptions { Indented = true }))
			{
				Write(writer);
			}
		}

		/// <summary>
		/// Writes this schema to a file
		/// </summary>
		/// <param name="file">The output file</param>
		public void Write(FileReference file)
		{
			using (FileStream stream = FileReference.Open(file, FileMode.Create))
			{
				Write(stream);
			}
		}

		/// <summary>
		/// Constructs a Json schema from a type
		/// </summary>
		/// <param name="type">The type to construct from</param>
		/// <param name="xmlDocReader">Reads documentation for requested types</param>
		/// <returns>New schema object</returns>
		public static JsonSchema FromType(Type type, XmlDocReader? xmlDocReader)
		{
			JsonSchemaFactory factory = new(xmlDocReader);
			return factory.CreateSchema(type);
		}

		/// <summary>
		/// Create a Json schema (or retrieve a cached schema)
		/// </summary>
		public static JsonSchema CreateSchema(Type type, XmlDocReader? xmlDocReader)
		{
			JsonSchemaFactory factory = new(xmlDocReader);
			return factory.CreateSchema(type);
		}
	}
}
