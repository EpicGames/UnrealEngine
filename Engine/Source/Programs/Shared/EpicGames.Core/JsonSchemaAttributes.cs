// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Attribute setting the identifier for a schema
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class JsonSchemaAttribute : Attribute
	{
		/// <summary>
		/// The schema identifier
		/// </summary>
		public string Id { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id"></param>
		public JsonSchemaAttribute(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Attribute setting catalog entries for a schema
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class JsonSchemaCatalogAttribute : Attribute
	{
		/// <summary>
		/// The schema name
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Description of the schema
		/// </summary>
		public string Description { get; }

		/// <summary>
		/// File patterns to match
		/// </summary>
		public string[]? FileMatch { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		/// <param name="description"></param>
		/// <param name="fileMatch">File patterns to match</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public JsonSchemaCatalogAttribute(string name, string description, string? fileMatch)
#pragma warning restore CA1019 // Define accessors for attribute arguments
			: this(name, description, (fileMatch == null) ? (string[]?)null : [fileMatch])
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		/// <param name="description"></param>
		/// <param name="fileMatch">File patterns to match</param>
		public JsonSchemaCatalogAttribute(string name, string description, string[]? fileMatch)
		{
			Name = name;
			Description = description;
			FileMatch = fileMatch;
		}
	}

	/// <summary>
	/// Attribute setting properties for a type to be serialized as a string
	/// </summary>
	public abstract class JsonSchemaTypeAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute allowing any possible value in a json schema
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct | AttributeTargets.Property)]
	public sealed class JsonSchemaAnyAttribute : JsonSchemaTypeAttribute
	{
	}

	/// <summary>
	/// Attribute setting properties for a type to be serialized as a string
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct | AttributeTargets.Property)]
	public sealed class JsonSchemaStringAttribute : JsonSchemaTypeAttribute
	{
		/// <summary>
		/// Format of the string
		/// </summary>
		public JsonSchemaStringFormat? Format { get; set; }
	}
}
