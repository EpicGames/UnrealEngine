// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Factory for creating Json schema elements from native properties and types
	/// </summary>
	public class JsonSchemaFactory
	{
		/// <summary>
		/// Reader for XML documentation
		/// </summary>
		public XmlDocReader? XmlDocReader { get; }

		/// <summary>
		/// Cache of known types
		/// </summary>
		public Dictionary<Type, JsonSchemaType> TypeCache { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonSchemaFactory(XmlDocReader? xmlDocReader)
		{
			XmlDocReader = xmlDocReader;
		}

		/// <summary>
		/// Creates a schema object for the given type
		/// </summary>
		public JsonSchema CreateSchema(Type type)
		{
			JsonSchemaAttribute? schemaAttribute = type.GetCustomAttribute<JsonSchemaAttribute>();
			return new JsonSchema(schemaAttribute?.Id, CreateSchemaType(type));
		}

		/// <summary>
		/// Constructs a schema type for the given property
		/// </summary>
		public virtual JsonSchemaType CreateSchemaType(PropertyInfo propertyInfo)
		{
			if (propertyInfo.GetCustomAttribute<JsonSchemaAnyAttribute>() != null)
			{
				return new JsonSchemaAny();
			}
			if (propertyInfo.GetCustomAttribute<JsonSchemaStringAttribute>() != null)
			{
				return new JsonSchemaString();
			}

			return CreateSchemaType(propertyInfo.PropertyType);
		}

		/// <summary>
		/// Constructs a schema type from the given type object
		/// </summary>
		public virtual JsonSchemaType CreateSchemaType(Type type)
		{
			switch (Type.GetTypeCode(type))
			{
				case TypeCode.Boolean:
					return new JsonSchemaBoolean();
				case TypeCode.Byte:
				case TypeCode.SByte:
				case TypeCode.Int16:
				case TypeCode.UInt16:
				case TypeCode.Int32:
				case TypeCode.UInt32:
				case TypeCode.Int64:
				case TypeCode.UInt64:
					if (type.IsEnum)
					{
						return CreateEnumSchemaType(type);
					}
					else
					{
						return new JsonSchemaInteger();
					}
				case TypeCode.Single:
				case TypeCode.Double:
					return new JsonSchemaNumber();
				case TypeCode.String:
					return new JsonSchemaString();
			}

			JsonSchemaTypeAttribute? attribute = type.GetCustomAttribute<JsonSchemaTypeAttribute>();
			switch (attribute)
			{
				case JsonSchemaAnyAttribute _:
					return new JsonSchemaAny();
				case JsonSchemaStringAttribute str:
					return new JsonSchemaString(str.Format);
			}

			if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				return CreateSchemaType(type.GetGenericArguments()[0]);
			}
			if (type == typeof(DateTime) || type == typeof(DateTimeOffset))
			{
				return new JsonSchemaString(JsonSchemaStringFormat.DateTime);
			}
			if (type == typeof(TimeSpan))
			{
				return new JsonSchemaString();
			}
			if (type == typeof(Uri))
			{
				return new JsonSchemaString(JsonSchemaStringFormat.Uri);
			}
			if (type == typeof(System.Text.Json.Nodes.JsonObject))
			{
				return new JsonSchemaObject();
			}

			if (type.IsClass)
			{
				if (TypeCache.TryGetValue(type, out JsonSchemaType? schemaType))
				{
					return schemaType;
				}

				Type[] interfaceTypes = type.GetInterfaces();
				foreach (Type interfaceType in interfaceTypes)
				{
					if (interfaceType.IsGenericType)
					{
						Type[] arguments = interfaceType.GetGenericArguments();
						if (interfaceType.GetGenericTypeDefinition() == typeof(IList<>))
						{
							return new JsonSchemaArray(CreateSchemaType(arguments[0]));
						}
						if (interfaceType.GetGenericTypeDefinition() == typeof(IDictionary<,>))
						{
							JsonSchemaObject obj = new();
							obj.AdditionalProperties = CreateSchemaType(arguments[1]);
							return obj;
						}
					}
				}

				JsonKnownTypesAttribute? knownTypes = type.GetCustomAttribute<JsonKnownTypesAttribute>(false);
				if (knownTypes != null)
				{
					JsonSchemaOneOf obj = new();
					TypeCache[type] = obj;
					SetOneOfProperties(obj, type, knownTypes.Types);
					return obj;
				}
				else
				{
					JsonSchemaObject obj = new();
					TypeCache[type] = obj;
					SetObjectProperties(obj, type);
					return obj;
				}
			}

			throw new Exception($"Unknown type for schema generation: {type}");
		}

		/// <summary>
		/// Create an enum schema type
		/// </summary>
		protected JsonSchemaEnum CreateEnumSchemaType(Type type)
		{
			string[] names = Enum.GetNames(type);
			string[] descriptions = new string[names.Length];

			for (int idx = 0; idx < names.Length; idx++)
			{
				descriptions[idx] = XmlDocReader?.GetDescription(type, names[idx]) ?? String.Empty;
			}

			string? enumDescription = XmlDocReader?.GetDescription(type);
			return new JsonSchemaEnum(names, descriptions) { Name = type.Name, Description = enumDescription };
		}

		/// <summary>
		/// Set a one-of schema element to the known types
		/// </summary>
		protected void SetOneOfProperties(JsonSchemaOneOf obj, Type type, Type[] knownTypes)
		{
			obj.Name = type.Name;

			foreach (Type knownType in knownTypes)
			{
				JsonDiscriminatorAttribute? attribute = knownType.GetCustomAttribute<JsonDiscriminatorAttribute>();
				if (attribute != null)
				{
					JsonSchemaObject knownObject = new();
					knownObject.Properties.Add(new JsonSchemaProperty("type", "Type discriminator", new JsonSchemaEnum([attribute.Name], ["Identifier for the derived type"])));
					SetObjectProperties(knownObject, knownType);
					obj.Types.Add(knownObject);
				}
			}
		}

		/// <summary>
		/// Fill out the properties for a schema object
		/// </summary>
		protected void SetObjectProperties(JsonSchemaObject obj, Type type)
		{
			obj.Name = type.Name;
			obj.Description = XmlDocReader?.GetDescription(type);

			PropertyInfo[] properties = type.GetProperties(BindingFlags.Instance | BindingFlags.Public);
			foreach (PropertyInfo property in properties)
			{
				if (property.GetCustomAttribute<JsonIgnoreAttribute>() == null)
				{
					string? description = XmlDocReader?.GetDescription(property);
					JsonSchemaType propertyType = CreateSchemaType(property);
					obj.Properties.Add(new JsonSchemaProperty(property.Name, description, propertyType));
				}
			}
		}
	}
}
