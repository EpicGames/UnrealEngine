// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Base class for Json schema types
	/// </summary>
	public abstract class JsonSchemaType
	{
		/// <summary>
		/// Name for this type, if stored in a standalone definition
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Description of this type
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Write this type to the given archive
		/// </summary>
		/// <param name="writer"></param>
		public abstract void Write(IJsonSchemaWriter writer);
	}

	/// <summary>
	/// Base class for types that cannot contain references to other types
	/// </summary>
	public abstract class JsonSchemaPrimitiveType : JsonSchemaType
	{
	}

	/// <summary>
	/// Represents any type in a Json schema
	/// </summary>
	public class JsonSchemaAny : JsonSchemaPrimitiveType
	{
		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
		}
	}

	/// <summary>
	/// Represents a boolean in a Json schema
	/// </summary>
	public class JsonSchemaBoolean : JsonSchemaPrimitiveType
	{
		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "boolean");
		}
	}

	/// <summary>
	/// Represents an integer in a Json schema
	/// </summary>
	public class JsonSchemaInteger : JsonSchemaPrimitiveType
	{
		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "integer");
		}
	}

	/// <summary>
	/// Represents a number of any type in a Json schema
	/// </summary>
	public class JsonSchemaNumber : JsonSchemaPrimitiveType
	{
		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "integer");
		}
	}

	/// <summary>
	/// Format for a string element schema
	/// </summary>
	public enum JsonSchemaStringFormat
	{
		/// <summary>
		/// A date/time value
		/// </summary>
		DateTime,

		/// <summary>
		/// Internet email address, see RFC 5322, section 3.4.1.
		/// </summary>
		Email,

		/// <summary>
		/// Internet host name, see RFC 1034, section 3.1.
		/// </summary>
		HostName,

		/// <summary>
		/// IPv4 address, according to dotted-quad ABNF syntax as defined in RFC 2673, section 3.2.
		/// </summary>
		Ipv4,

		/// <summary>
		/// IPv6 address, as defined in RFC 2373, section 2.2.
		/// </summary>
		Ipv6,

		/// <summary>
		/// A universal resource identifier (URI), according to RFC3986.
		/// </summary>
		Uri
	}

	/// <summary>
	/// Represents a string in a Json schema
	/// </summary>
	public class JsonSchemaString : JsonSchemaPrimitiveType
	{
		/// <summary>
		/// Optional string describing the formatting of this type
		/// </summary>
		public JsonSchemaStringFormat? Format { get; set; }

		/// <summary>
		/// Pattern for matched strings
		/// </summary>
		public string? Pattern { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format"></param>
		/// <param name="pattern"></param>
		public JsonSchemaString(JsonSchemaStringFormat? format = null, string? pattern = null)
		{
			Format = format;
			Pattern = pattern;
		}

		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "string");
			if (Format != null)
			{
#pragma warning disable CA1308 // Normalize strings to uppercase
				writer.WriteString("format", Format.Value.ToString().ToLowerInvariant());
#pragma warning restore CA1308 // Normalize strings to uppercase
			}
			if (Pattern != null)
			{
				writer.WriteString("pattern", Pattern);
			}
		}
	}

	/// <summary>
	/// Represents an enum in a Json schema
	/// </summary>
	public class JsonSchemaEnum : JsonSchemaType
	{
		/// <summary>
		/// Values for the enum
		/// </summary>
		public List<string> Values { get; } = [];

		/// <summary>
		/// Descriptions for each enum value
		/// </summary>
		public List<string> Descriptions { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonSchemaEnum(IEnumerable<string> values, IEnumerable<string> descriptions)
		{
			Values.AddRange(values);
			Descriptions.AddRange(descriptions);
		}

		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteStartArray("enum");
			foreach (string value in Values)
			{
				writer.WriteStringValue(value);
			}
			writer.WriteEndArray();
		}
	}

	/// <summary>
	/// Represents an array in a Json schema
	/// </summary>
	public class JsonSchemaArray : JsonSchemaType
	{
		/// <summary>
		/// The type of each element
		/// </summary>
		public JsonSchemaType ItemType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonSchemaArray(JsonSchemaType itemType)
		{
			ItemType = itemType;
		}

		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "array");
			writer.WriteStartObject("items");
			writer.WriteType(ItemType);
			writer.WriteEndObject();
		}
	}

	/// <summary>
	/// Property within a <see cref="JsonSchemaObject"/>
	/// </summary>
	public class JsonSchemaProperty
	{
		/// <summary>
		/// Name of this property
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// The camelcase name for this property
		/// </summary>
		public string CamelCaseName => Char.ToLowerInvariant(Name[0]) + Name.Substring(1);

		/// <summary>
		/// Description of the property
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// The property type
		/// </summary>
		public JsonSchemaType Type { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonSchemaProperty(string name, string? description, JsonSchemaType type)
		{
			Name = name;
			Description = description;
			Type = type;
		}

		/// <summary>
		/// Writes a property to a schema
		/// </summary>
		/// <param name="writer"></param>
		public void Write(IJsonSchemaWriter writer)
		{
			writer.WriteStartObject(CamelCaseName);
			writer.WriteType(Type);
			if (Description != null)
			{
				writer.WriteString("description", Description);
			}
			writer.WriteEndObject();
		}
	}

	/// <summary>
	/// Represents an object in a Json schema
	/// </summary>
	public class JsonSchemaObject : JsonSchemaType
	{
		/// <summary>
		/// Properties to allow in this object
		/// </summary>
		public List<JsonSchemaProperty> Properties { get; } = [];

		/// <summary>
		/// Type for additional properties
		/// </summary>
		public JsonSchemaType? AdditionalProperties { get; set; }

		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteString("type", "object");
			if (Properties.Count > 0)
			{
				writer.WriteStartObject("properties");
				foreach (JsonSchemaProperty property in Properties)
				{
					property.Write(writer);
				}
				writer.WriteEndObject();
			}

			if (AdditionalProperties == null)
			{
				writer.WriteBoolean("additionalProperties", false);
			}
			else
			{
				writer.WriteStartObject("additionalProperties");
				AdditionalProperties.Write(writer);
				writer.WriteEndObject();
			}
		}
	}

	/// <summary>
	/// Represents one of a set of object types in a Json schema
	/// </summary>
	public class JsonSchemaOneOf : JsonSchemaType
	{
		/// <summary>
		/// List of valid types
		/// </summary>
		public List<JsonSchemaType> Types { get; } = [];

		/// <inheritdoc/>
		public override void Write(IJsonSchemaWriter writer)
		{
			writer.WriteStartArray("oneOf");
			foreach (JsonSchemaType type in Types)
			{
				writer.WriteStartObject();
				type.Write(writer);
				writer.WriteEndObject();
			}
			writer.WriteEndArray();
		}
	}
}
