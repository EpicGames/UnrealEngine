// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading;
using OpenTracing;
using OpenTracing.Mock;
using OpenTracing.Propagation;
using OpenTracing.Tag;
using OpenTracing.Util;

#pragma warning disable CS1591

namespace EpicGames.Core
{
	public class JsonTracerSpanContext : ISpanContext
	{
		public IEnumerable<KeyValuePair<string, string>> GetBaggageItems()
		{
			throw new NotImplementedException();
		}

		public string TraceId { get; }
		public string SpanId { get; }

		public JsonTracerSpanContext(string traceId, string spanId)
		{
			TraceId = traceId;
			SpanId = spanId;
		}
	}

	public class JsonTracerSpan : ISpan
	{
		/// <summary>
		/// Used to monotonically update ids
		/// </summary>
		private static long s_nextIdCounter = 0;

		/// <summary>
		/// Unique GUID for this process so IDs are always unique
		/// </summary>
		private static readonly string s_processGuidStr = Guid.NewGuid().ToString();

		/// <summary>
		/// A simple-as-possible (consecutive for repeatability) id generator.
		/// </summary>
		private static string NextId()
		{
			return s_processGuidStr + "." + Interlocked.Increment(ref s_nextIdCounter).ToString(CultureInfo.InvariantCulture);
		}

		// C# doesn't have "return type covariance" so we use the trick with the explicit interface implementation
		// and this separate property.
		public JsonTracerSpanContext Context => _jsonTracerContext;

		ISpanContext ISpan.Context => Context;

		public string OperationName { get; private set; }

		private readonly JsonTracer _tracer;
		private readonly JsonTracerSpanContext _jsonTracerContext;
		private DateTimeOffset _finishTimestamp;
		private bool _finished;
		private readonly Dictionary<string, object> _tags;
		private readonly List<JsonTracerSpan.Reference> _references;

		public DateTimeOffset StartTimestamp { get; }
		public Dictionary<string, object> Tags => new(_tags);

		/// <summary>
		/// The finish time of the span; only valid after a call to <see cref="Finish()"/>.
		/// </summary>
		public DateTimeOffset FinishTimestamp
		{
			get
			{
				if (_finishTimestamp == DateTimeOffset.MinValue)
				{
					throw new InvalidOperationException("Must call Finish() before FinishTimestamp");
				}

				return _finishTimestamp;
			}
		}

		/// <summary>
		/// The spanId of the span's first <see cref="OpenTracing.References.ChildOf"/> reference, or the first reference of any type,
		/// or null if no reference exists.
		/// </summary>
		/// <seealso cref="MockSpanContext.SpanId"/>
		/// <seealso cref="MockSpan.References"/>
		public string? ParentId { get; }

		private readonly Lock _lock = new();

		public JsonTracerSpan(JsonTracer jsonTracer, string operationName, DateTimeOffset startTimestamp, Dictionary<string, object>? initialTags, List<Reference>? references)
		{
			_tracer = jsonTracer;
			OperationName = operationName;
			StartTimestamp = startTimestamp;

			_tags = initialTags == null
				? []
				: new Dictionary<string, object>(initialTags);

			_references = references == null
				? []
				: [.. references];

			JsonTracerSpanContext? parentContext = FindPreferredParentRef(_references);
			if (parentContext == null)
			{
				// we are a root span
				_jsonTracerContext = new JsonTracerSpanContext(NextId(), NextId());
				ParentId = null;
			}
			else
			{
				// we are a child span
				_jsonTracerContext = new JsonTracerSpanContext(parentContext.TraceId, NextId());
				ParentId = parentContext.SpanId;
			}
		}

		public ISpan SetTag(string key, string value) => SetObjectTag(key, value);
		public ISpan SetTag(string key, bool value) => SetObjectTag(key, value);
		public ISpan SetTag(string key, int value) => SetObjectTag(key, value);
		public ISpan SetTag(string key, double value) => SetObjectTag(key, value);

		public ISpan SetTag(BooleanTag tag, bool value)
		{
			SetObjectTag(tag.Key, value);
			return this;
		}

		public ISpan SetTag(IntOrStringTag tag, string value)
		{
			SetObjectTag(tag.Key, value);
			return this;
		}

		public ISpan SetTag(IntTag tag, int value)
		{
			SetObjectTag(tag.Key, value);
			return this;
		}

		public ISpan SetTag(StringTag tag, string value)
		{
			SetObjectTag(tag.Key, value);
			return this;
		}

		private JsonTracerSpan SetObjectTag(string key, object value)
		{
			lock (_lock)
			{
				CheckForFinished("Setting tag [{0}:{1}] on already finished span", key, value);
				_tags[key] = value;
				return this;
			}
		}

		public ISpan Log(IEnumerable<KeyValuePair<string, object>> fields) { throw new NotSupportedException("Log() calls are not supported in JsonTracerSpans"); }
		public ISpan Log(DateTimeOffset timestamp, IEnumerable<KeyValuePair<string, object>> fields) { throw new NotSupportedException("Log() calls are not supported in JsonTracerSpans"); }
		public ISpan Log(string message) { throw new NotSupportedException("Log() calls are not supported in JsonTracerSpans"); }
		public ISpan Log(DateTimeOffset timestamp, string message) { throw new NotSupportedException("Log() calls are not supported in JsonTracerSpans"); }

		public ISpan SetBaggageItem(string key, string value) { throw new NotImplementedException(); }
		public string GetBaggageItem(string key) { throw new NotImplementedException(); }

		public ISpan SetOperationName(string operationName)
		{
			CheckForFinished("Setting operationName [{0}] on already finished span", operationName);
			OperationName = operationName;
			return this;
		}

		public void Finish()
		{
			Finish(DateTimeOffset.UtcNow);
		}

		public void Finish(DateTimeOffset finishTimestamp)
		{
			lock (_lock)
			{
				CheckForFinished("Tried to finish already finished span");
				_finishTimestamp = finishTimestamp;
				_tracer.AppendFinishedSpan(this);
				_finished = true;
			}
		}

		private static JsonTracerSpanContext? FindPreferredParentRef(IList<Reference> references)
		{
			if (!references.Any())
			{
				return null;
			}

			// return the context of the parent, if applicable
			foreach (Reference reference in references)
			{
				if (References.ChildOf.Equals(reference.ReferenceType, StringComparison.Ordinal))
				{
					return reference.Context;
				}
			}

			// otherwise, return the context of the first reference
			return references.First().Context;
		}

		private void CheckForFinished(string format, params object[] args)
		{
			if (_finished)
			{
				throw new InvalidOperationException(String.Format(format, args));
			}
		}

#pragma warning disable CA1034 // Nested types should not be visible
		public sealed class Reference : IEquatable<Reference>
#pragma warning restore CA1034 // Nested types should not be visible
		{
			public JsonTracerSpanContext Context { get; }

			/// <summary>
			/// See <see cref="OpenTracing.References"/>.
			/// </summary>
			public string ReferenceType { get; }

			public Reference(JsonTracerSpanContext context, string referenceType)
			{
				Context = context ?? throw new ArgumentNullException(nameof(context));
				ReferenceType = referenceType ?? throw new ArgumentNullException(nameof(referenceType));
			}

			public override bool Equals(object? obj)
			{
				return Equals(obj as Reference);
			}

			public bool Equals(Reference? other)
			{
				return other != null &&
					   EqualityComparer<JsonTracerSpanContext>.Default.Equals(Context, other.Context) &&
					   ReferenceType == other.ReferenceType;
			}

#pragma warning disable IDE0070 // Use 'System.HashCode'
			public override int GetHashCode()
			{
				int hashCode = 2083322454;
				hashCode = hashCode * -1521134295 + EqualityComparer<JsonTracerSpanContext>.Default.GetHashCode(Context);
				hashCode = hashCode * -1521134295 + StringComparer.Ordinal.GetHashCode(ReferenceType);
				return hashCode;
			}
#pragma warning restore IDE0070 // Use 'System.HashCode'
		}
	}

	public class JsonTracerSpanBuilder : ISpanBuilder
	{
		private readonly JsonTracer _tracer;
		private readonly string _operationName;
		private DateTimeOffset _startTimestamp = DateTimeOffset.MinValue;
		private readonly List<JsonTracerSpan.Reference> _references = [];
		private readonly Dictionary<string, object> _tags = [];
		private bool _ignoreActiveSpan;

		public JsonTracerSpanBuilder(JsonTracer tracer, string operationName)
		{
			_tracer = tracer;
			_operationName = operationName;
		}

		public ISpanBuilder AsChildOf(ISpanContext? parent)
		{
			if (parent == null)
			{
				return this;
			}

			return AddReference(References.ChildOf, parent);
		}

		public ISpanBuilder AsChildOf(ISpan? parent)
		{
			if (parent == null)
			{
				return this;
			}

			return AddReference(References.ChildOf, parent.Context);
		}

		public ISpanBuilder AddReference(string referenceType, ISpanContext? referencedContext)
		{
			if (referencedContext != null)
			{
				_references.Add(new JsonTracerSpan.Reference((JsonTracerSpanContext)referencedContext, referenceType));
			}

			return this;
		}

		public ISpanBuilder IgnoreActiveSpan()
		{
			_ignoreActiveSpan = true;
			return this;
		}

		public ISpanBuilder WithTag(string key, string value)
		{
			_tags[key] = value;
			return this;
		}

		public ISpanBuilder WithTag(string key, bool value)
		{
			_tags[key] = value;
			return this;
		}

		public ISpanBuilder WithTag(string key, int value)
		{
			_tags[key] = value;
			return this;
		}

		public ISpanBuilder WithTag(string key, double value)
		{
			_tags[key] = value;
			return this;
		}

		public ISpanBuilder WithTag(BooleanTag tag, bool value)
		{
			_tags[tag.Key] = value;
			return this;
		}

		public ISpanBuilder WithTag(IntOrStringTag tag, string value)
		{
			_tags[tag.Key] = value;
			return this;
		}

		public ISpanBuilder WithTag(IntTag tag, int value)
		{
			_tags[tag.Key] = value;
			return this;
		}

		public ISpanBuilder WithTag(StringTag tag, string value)
		{
			_tags[tag.Key] = value;
			return this;
		}

		public ISpanBuilder WithStartTimestamp(DateTimeOffset timestamp)
		{
			_startTimestamp = timestamp;
			return this;
		}

		public IScope StartActive()
		{
			return StartActive(true);
		}

		public IScope StartActive(bool finishSpanOnDispose)
		{
			ISpan span = Start();
			return _tracer.ScopeManager.Activate(span, finishSpanOnDispose);
		}

		public ISpan Start()
		{
			if (_startTimestamp == DateTimeOffset.MinValue) // value was not set by builder
			{
				_startTimestamp = DateTimeOffset.UtcNow;
			}

			ISpanContext? activeSpanContext = _tracer.ActiveSpan?.Context;
			if (_references.Count == 0 && !_ignoreActiveSpan && activeSpanContext != null)
			{
				_references.Add(new JsonTracerSpan.Reference((JsonTracerSpanContext)activeSpanContext, References.ChildOf));
			}

			return new JsonTracerSpan(_tracer, _operationName, _startTimestamp, _tags, _references);
		}
	}

	public class JsonTracer : ITracer
	{
		public IScopeManager ScopeManager { get; }
		public ISpan? ActiveSpan => ScopeManager.Active?.Span;

		private readonly Lock _lock = new();
		private readonly List<JsonTracerSpan> _finishedSpans = [];
		private readonly DirectoryReference? _telemetryDir;

		public JsonTracer(DirectoryReference? telemetryDir = null)
		{
			ScopeManager = new AsyncLocalScopeManager();
			_telemetryDir = telemetryDir;
		}

		public static JsonTracer? TryRegisterAsGlobalTracer()
		{
			string? telemetryDir = Environment.GetEnvironmentVariable("UE_TELEMETRY_DIR");
			if (telemetryDir != null)
			{
				JsonTracer tracer = new(new DirectoryReference(telemetryDir));
				return GlobalTracer.RegisterIfAbsent(tracer) ? tracer : null;
			}

			return null;
		}

		public ISpanBuilder BuildSpan(string operationName)
		{
			return new JsonTracerSpanBuilder(this, operationName);
		}

		public void Inject<TCarrier>(ISpanContext spanContext, IFormat<TCarrier> format, TCarrier carrier)
		{
			throw new NotSupportedException(String.Format("Tracer.Inject is not implemented for {0} by JsonTracer", format));
		}

		public ISpanContext Extract<TCarrier>(IFormat<TCarrier> format, TCarrier carrier)
		{
			throw new NotSupportedException(String.Format("Tracer.Extract is not implemented for {0} by JsonTracer", format));
		}

		public void Flush()
		{
			if (_telemetryDir != null)
			{
				string telemetryScopeId = Environment.GetEnvironmentVariable("UE_TELEMETRY_SCOPE_ID") ?? "noscope";

				FileReference file;
				using (System.Diagnostics.Process process = System.Diagnostics.Process.GetCurrentProcess())
				{
					DirectoryReference.CreateDirectory(_telemetryDir);

					string fileName = String.Format("{0}.{1}.{2}.opentracing.json", Path.GetFileName(Assembly.GetEntryAssembly()!.Location), telemetryScopeId, process.Id);
					file = FileReference.Combine(_telemetryDir, fileName);
				}

				using (JsonWriter writer = new(file))
				{
					GetFinishedSpansAsJson(writer);
				}
			}
		}

		public List<JsonTracerSpan> GetFinishedSpans()
		{
			lock (_lock)
			{
				return [.. _finishedSpans];
			}
		}

		public void GetFinishedSpansAsJson(JsonWriter writer)
		{
			writer.WriteObjectStart();
			writer.WriteArrayStart("Spans");
			foreach (JsonTracerSpan span in GetFinishedSpans())
			{
				writer.WriteObjectStart();
				writer.WriteValue("Name", span.OperationName);
				Dictionary<string, object> tags = span.Tags;
				if (tags.TryGetValue("Resource", out object? resource) && resource is string resourceString)
				{
					writer.WriteValue("Resource", resourceString);
				}
				if (tags.TryGetValue("Service", out object? service) && service is string serviceString)
				{
					writer.WriteValue("Service", serviceString);
				}
				writer.WriteValue("StartTime", span.StartTimestamp.ToString("o", CultureInfo.InvariantCulture));
				writer.WriteValue("FinishTime", span.FinishTimestamp.ToString("o", CultureInfo.InvariantCulture));
				writer.WriteValue("SpanId", span.Context.SpanId);
				if (span.ParentId != null)
				{
					writer.WriteValue("ParentId", span.ParentId);
				}
				writer.WriteObjectStart("Metadata");
				// TODO: Write tags as metadata?
				writer.WriteObjectEnd();
				writer.WriteObjectEnd();
			}
			writer.WriteArrayEnd();
			writer.WriteObjectEnd();
		}

		internal void AppendFinishedSpan(JsonTracerSpan jsonTracerSpan)
		{
			lock (_lock)
			{
				_finishedSpans.Add(jsonTracerSpan);
			}
		}
	}
}
