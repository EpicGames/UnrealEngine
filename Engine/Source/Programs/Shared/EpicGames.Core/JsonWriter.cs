// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Specifies how to format JSON output
	/// </summary>
	public enum JsonWriterStyle
	{
		/// <summary>
		/// Omit spaces between elements
		/// </summary>
		Compact,

		/// <summary>
		/// Put each value on a newline, and indent output
		/// </summary>
		Readable
	}

	/// <summary>
	/// Writer for JSON data, which indents the output text appropriately, and adds commas and newlines between fields
	/// </summary>
	public sealed class JsonWriter : IDisposable
	{
		TextWriter _writer;
		readonly bool _leaveOpen;
		readonly JsonWriterStyle _style;
		bool _bRequiresComma;
		string _indent;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileName">File to write to</param>
		/// <param name="style">Should use packed JSON or not</param>
		public JsonWriter(string fileName, JsonWriterStyle style = JsonWriterStyle.Readable)
			: this(new StreamWriter(new FileStream(fileName, FileMode.Create, FileAccess.Write, FileShare.Read)))
		{
			_style = style;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileName">File to write to</param>
		/// <param name="style">Should use packed JSON or not</param>
		public JsonWriter(FileReference fileName, JsonWriterStyle style = JsonWriterStyle.Readable)
			: this(new StreamWriter(new FileStream(fileName.FullName, FileMode.Create, FileAccess.Write, FileShare.Read)))
		{
			_style = style;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="writer">The text writer to output to</param>
		/// <param name="leaveOpen">Whether to leave the writer open when the object is disposed</param>
		/// <param name="style">The output style</param>
		public JsonWriter(TextWriter writer, bool leaveOpen = false, JsonWriterStyle style = JsonWriterStyle.Readable)
		{
			_writer = writer;
			_leaveOpen = leaveOpen;
			_style = style;
			_indent = "";
		}

		/// <summary>
		/// Dispose of any managed resources
		/// </summary>
		public void Dispose()
		{
			if (!_leaveOpen && _writer != null)
			{
				_writer.Dispose();
				_writer = null!;
			}
		}

		private void IncreaseIndent()
		{
			if (_style == JsonWriterStyle.Readable)
			{
				_indent += "\t";
			}
		}

		private void DecreaseIndent()
		{
			if (_style == JsonWriterStyle.Readable)
			{
				_indent = _indent.Substring(0, _indent.Length - 1);
			}
		}

		/// <summary>
		/// Write the opening brace for an object
		/// </summary>
		public void WriteObjectStart()
		{
			WriteCommaNewline();

			_writer.Write(_indent);
			_writer.Write("{");

			IncreaseIndent();
			_bRequiresComma = false;
		}

		/// <summary>
		/// Write the name and opening brace for an object
		/// </summary>
		/// <param name="objectName">Name of the field</param>
		public void WriteObjectStart(string objectName)
		{
			WriteCommaNewline();

			WriteName(objectName);

			_bRequiresComma = false;

			WriteObjectStart();
		}

		/// <summary>
		/// Write the closing brace for an object
		/// </summary>
		public void WriteObjectEnd()
		{
			DecreaseIndent();

			WriteLine();
			_writer.Write(_indent);
			_writer.Write("}");

			_bRequiresComma = true;
		}

		/// <summary>
		/// Write the opening bracket for an unnamed array
		/// </summary>
		public void WriteArrayStart()
		{
			WriteCommaNewline();

			_writer.Write("{0}[", _indent);

			IncreaseIndent();
			_bRequiresComma = false;
		}

		/// <summary>
		/// Write the name and opening bracket for an array
		/// </summary>
		/// <param name="arrayName">Name of the field</param>
		public void WriteArrayStart(string arrayName)
		{
			WriteCommaNewline();

			WriteName(arrayName);
			_writer.Write('[');

			IncreaseIndent();
			_bRequiresComma = false;
		}

		/// <summary>
		/// Write the closing bracket for an array
		/// </summary>
		public void WriteArrayEnd()
		{
			DecreaseIndent();

			WriteLine();
			_writer.Write("{0}]", _indent);

			_bRequiresComma = true;
		}

		private void WriteLine()
		{
			if (_style == JsonWriterStyle.Readable)
			{
				_writer.WriteLine();
			}
		}

		private void WriteLine(string line)
		{
			if (_style == JsonWriterStyle.Readable)
			{
				_writer.WriteLine(line);
			}
			else
			{
				_writer.Write(line);
			}
		}

		/// <summary>
		/// Write an array of strings
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="values">Values for the field</param>
		public void WriteStringArrayField(string name, IEnumerable<string> values)
		{
			WriteArrayStart(name);
			foreach (string value in values)
			{
				WriteValue(value);
			}
			WriteArrayEnd();
		}

		/// <summary>
		/// Write an array of enum values
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="values">Values for the field</param>
		public void WriteEnumArrayField<T>(string name, IEnumerable<T> values) where T : struct
		{
			WriteStringArrayField(name, values.Select(x => x.ToString()!));
		}

		/// <summary>
		/// Write a value with no field name, for the contents of an array
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteValue(int value)
		{
			WriteCommaNewline();

			_writer.Write(_indent);
			_writer.Write(value);

			_bRequiresComma = true;
		}

		/// <summary>
		/// Write a value with no field name, for the contents of an array
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteValue(string value)
		{
			WriteCommaNewline();

			_writer.Write(_indent);
			WriteEscapedString(value);

			_bRequiresComma = true;
		}

		/// <summary>
		/// Write a field name and string value
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteValue(string name, string? value)
		{
			WriteCommaNewline();

			WriteName(name);
			WriteEscapedString(value);

			_bRequiresComma = true;
		}

		/// <summary>
		/// Write a field name and integer value
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteValue(string name, int value)
		{
			WriteValueInternal(name, value.ToString());
		}

		/// <summary>
		/// Write a field name and unsigned integer value
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteValue(string name, uint value)
		{
			WriteValueInternal(name, value.ToString());
		}

		/// <summary>
		/// Write a field name and double value
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteValue(string name, double value)
		{
			WriteValueInternal(name, value.ToString());
		}

		/// <summary>
		/// Write a field name and bool value
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteValue(string name, bool value)
		{
			WriteValueInternal(name, value ? "true" : "false");
		}

		/// <summary>
		/// Write a field name and enum value
		/// </summary>
		/// <typeparam name="T">The enum type</typeparam>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value for the field</param>
		public void WriteEnumValue<T>(string name, T value) where T : struct
		{
			WriteValue(name, value.ToString()!);
		}

		void WriteCommaNewline()
		{
			if (_bRequiresComma)
			{
				WriteLine(",");
			}
			else if (_indent.Length > 0)
			{
				WriteLine();
			}
		}

		void WriteName(string name)
		{
			string space = (_style == JsonWriterStyle.Readable) ? " " : "";
			_writer.Write(_indent);
			WriteEscapedString(name);
			_writer.Write(":{0}", space);
		}

		void WriteValueInternal(string name, string value)
		{
			WriteCommaNewline();

			WriteName(name);
			_writer.Write(value);

			_bRequiresComma = true;
		}

		void WriteEscapedString(string? value)
		{
			// Escape any characters which may not appear in a JSON string (see http://www.json.org).
			_writer.Write("\"");
			if (value != null)
			{
				_writer.Write(EscapeString(value));
			}
			_writer.Write("\"");
		}

		/// <summary>
		/// Escapes a string for serializing to JSON
		/// </summary>
		/// <param name="value">The string to escape</param>
		/// <returns>The escaped string</returns>
		public static string EscapeString(string value)
		{

			// Prescan the string looking for things to escape.  If not found, we don't need to
			// create the string builder
			int idx = 0;
			for (; idx < value.Length; idx++)
			{
				char c = value[idx];
				if (c == '\"' || c == '\\' || Char.IsControl(c))
				{
					break;
				}
			}
			if (idx == value.Length)
			{
				return value;
			}

			// Otherwise, create the string builder, append the known portion that doesn't have an escape
			// and continue processing the string starting at the first character needing to be escaped.
			StringBuilder result = new();
			result.Append(value.AsSpan(0, idx));
			for (; idx < value.Length; idx++)
			{
				switch (value[idx])
				{
					case '\"':
						result.Append("\\\"");
						break;
					case '\\':
						result.Append("\\\\");
						break;
					case '\b':
						result.Append("\\b");
						break;
					case '\f':
						result.Append("\\f");
						break;
					case '\n':
						result.Append("\\n");
						break;
					case '\r':
						result.Append("\\r");
						break;
					case '\t':
						result.Append("\\t");
						break;
					default:
						if (Char.IsControl(value[idx]))
						{
							result.AppendFormat("\\u{0:X4}", (int)value[idx]);
						}
						else
						{
							result.Append(value[idx]);
						}
						break;
				}
			}
			return result.ToString();
		}
	}
}
