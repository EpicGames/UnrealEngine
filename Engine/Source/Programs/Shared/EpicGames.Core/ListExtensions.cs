// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for lists
	/// </summary>s
	public static class ListExtensions
	{
		/// <summary>
		/// Sorts a list by a particular field
		/// </summary>
		/// <typeparam name="TElement">List element</typeparam>
		/// <typeparam name="TField">Field type to sort by</typeparam>
		/// <param name="list">List to sort</param>
		/// <param name="selector">Selects a field from the element</param>
		public static void SortBy<TElement, TField>(this List<TElement> list, Func<TElement, TField> selector)
		{
			IComparer<TField> defaultComparer = Comparer<TField>.Default;
			SortBy(list, selector, defaultComparer);
		}

		/// <summary>
		/// Sorts a list by a particular field
		/// </summary>
		/// <typeparam name="TElement">List element</typeparam>
		/// <typeparam name="TField">Field type to sort by</typeparam>
		/// <param name="list">List to sort</param>
		/// <param name="selector">Selects a field from the element</param>
		/// <param name="comparer">Comparer for fields</param>
		public static void SortBy<TElement, TField>(this List<TElement> list, Func<TElement, TField> selector, IComparer<TField> comparer)
		{
			list.Sort((x, y) => comparer.Compare(selector(x), selector(y)));
		}

		/// <summary>
		/// Add all arguments to the list
		/// </summary>
		/// <typeparam name="TElement">List element</typeparam>
		/// <param name="list">List to add to</param>
		/// <param name="arguments">Arguments to add to list</param>
		public static void AddAll<TElement>(this List<TElement> list, params TElement[] arguments)
		{
			list.AddRange(arguments.AsEnumerable());
		}

		/// <summary>
		/// Add to a list of definitions with the form of "Name=Value".
		/// </summary>
		/// <param name="list">List to add to</param>
		/// <param name="definitionName">Definition name</param>
		/// <param name="value">Value to assign to Definition</param>
		public static void AddDefinition(this List<string> list, string definitionName, string value)
		{
			list.Add($"{definitionName}={value}");
		}

		/// <summary>
		/// Add to a list of definitions with the form of "Name=Value" where Value is 1 or 0 based on bValue.
		/// </summary>
		/// <param name="list">List to add to</param>
		/// <param name="definitionName">Definition name</param>
		/// <param name="bValue">Value to assign to Definition</param>
		public static void AddDefinition(this List<string> list, string definitionName, bool bValue)
		{
			list.Add($"{definitionName}={(bValue ? "1" : "0")}");
		}

		/// <summary>
		/// Add to a list of definitions with the form of "Name=Value".
		/// </summary>
		/// <param name="list">List to add to</param>
		/// <param name="definitionName">Definition name</param>
		/// <param name="value">Value to assign to Definition</param>
		public static void AddDefinition(this List<string> list, string definitionName, int value)
		{
			list.Add($"{definitionName}={value}");
		}

		/// <summary>
		/// Convert all elements of a list to a base type
		/// </summary>
		/// <typeparam name="TInput">Input element type</typeparam>
		/// <typeparam name="TOutput">Output element type</typeparam>
		/// <param name="input">List to convert</param>
		/// <returns>Converted list</returns>
		public static List<TOutput> ConvertAll<TInput, TOutput>(this List<TInput> input) where TInput : TOutput
		{
			List<TOutput> output = [.. input];
			return output;
		}

		/// <summary>
		/// Convert all elements of a list to a base type
		/// </summary>
		/// <typeparam name="TInput">Input element type</typeparam>
		/// <typeparam name="TOutput">Output element type</typeparam>
		/// <param name="input">List to convert</param>
		/// <returns>Converted list</returns>
		public static async Task<List<TOutput>> ConvertAllAsync<TInput, TOutput>(this Task<List<TInput>> input) where TInput : TOutput
		{
			return ConvertAll<TInput, TOutput>(await input);
		}

		/// <summary>
		/// Convert all elements of a list to a base type
		/// </summary>
		/// <typeparam name="TInput">Input element type</typeparam>
		/// <typeparam name="TOutput">Output element type</typeparam>
		/// <param name="input">List to convert</param>
		/// <param name="converter">Converter for elements</param>
		/// <returns>Converted list</returns>
		public static async Task<List<TOutput>> ConvertAllAsync<TInput, TOutput>(this Task<List<TInput>> input, Converter<TInput, TOutput> converter)
		{
			return (await input).ConvertAll(converter);
		}
	}
}
