// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;

#pragma warning disable CA1710 // Identifiers should have correct suffix

namespace EpicGames.Core
{
	/// <summary>
	/// Static methods for <see cref="ListSegment{T}"/>
	/// </summary>
	public static class ListSegment
	{
		/// <summary>
		/// An empty list segment of the given element type
		/// </summary>
		public static ListSegment<T> Empty<T>() => new(Array.Empty<T>(), 0, 0);
	}

	/// <summary>
	/// Delimits a section of a list
	/// </summary>
	/// <typeparam name="T">The list element type</typeparam>
	public readonly struct ListSegment<T> : IEnumerable<T>, IEnumerable, IReadOnlyCollection<T>, IReadOnlyList<T>
	{
		/// <summary>
		/// Gets the original array containing the range of elements that the list segment
		/// delimits.
		/// </summary>
		public IList<T> List { get; }

		/// <summary>
		/// Gets the position of the first element in the range delimited by the list segment,
		/// relative to the start of the original list.
		/// </summary>
		public int Offset { get; }

		/// <summary>
		/// Gets the number of elements in the range delimited by the array segment.
		/// </summary>
		public int Count { get; }

		/// <inheritdoc/>
		public bool IsReadOnly => List.IsReadOnly;

		/// <summary>
		/// Initializes a new instance of the System.ArraySegment`1 structure that delimits
		/// all the elements in the specified array.
		/// </summary>
		public ListSegment(IList<T> list)
		{
			List = list;
			Offset = 0;
			Count = list.Count;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="list"></param>
		/// <param name="offset"></param>
		/// <param name="length"></param>
		public ListSegment(IList<T> list, int offset, int length)
		{
			if (offset < 0 || offset > list.Count)
			{
				throw new ArgumentOutOfRangeException(nameof(offset));
			}
			if (length < 0 || offset + length > list.Count)
			{
				throw new ArgumentOutOfRangeException(nameof(length));
			}

			List = list;
			Offset = offset;
			Count = length;
		}

		/// <summary>
		/// Create a new slice of this list segment
		/// </summary>
		/// <param name="offset">Offset for the start of the segment</param>
		public ListSegment<T> Slice(int offset)
		{
			if (offset < 0)
			{
				throw new ArgumentException("Offset may not be negative", nameof(offset));
			}
			if (offset > Count)
			{
				throw new ArgumentException("Offset may not exceed the length of the existing segment", nameof(offset));
			}
			return new ListSegment<T>(List, Offset + offset, Count - offset);
		}

		/// <summary>
		/// Create a new slice of this list segment
		/// </summary>
		/// <param name="offset">Offset for the start of the segment</param>
		/// <param name="count">Number of elements to include</param>
		public ListSegment<T> Slice(int offset, int count)
		{
			if (offset < 0)
			{
				throw new ArgumentException("Offset may not be negative", nameof(offset));
			}
			if (offset > Count)
			{
				throw new ArgumentException("Offset may not exceed the length of the existing segment", nameof(offset));
			}
			if (count < 0)
			{
				throw new ArgumentException("Count may not be negative", nameof(count));
			}
			if (offset + count > Count)
			{
				throw new ArgumentException("Offset plus count may not exceed the length of the segment", nameof(count));
			}
			return new ListSegment<T>(List, Offset + offset, count);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="index"></param>
		/// <returns></returns>
		/// <exception cref="ArgumentOutOfRangeException">Index is not a valid index in the ListSegment</exception>
		public T this[int index]
		{
			get
			{
				if (index < 0 || index >= Count)
				{
					throw new ArgumentOutOfRangeException(nameof(index));
				}
				return List[index + Offset];
			}
			set
			{
				if (index < 0 || index >= Count)
				{
					throw new ArgumentOutOfRangeException(nameof(index));
				}
				List[index + Offset] = value;
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="list"></param>
		public static implicit operator ListSegment<T>(List<T> list)
		{
			return new ListSegment<T>(list);
		}

		/// <summary>
		/// Enumerator implementation
		/// </summary>
		struct Enumerator : IEnumerator<T>, IEnumerator
		{
			readonly IList<T> _list;
			int _index;
			readonly int _maxIndex;

			/// <inheritdoc/>
			public readonly T Current
			{
				get
				{
					if (_index >= _maxIndex)
					{
						throw new IndexOutOfRangeException();
					}
					return _list[_index];
				}
			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="segment"></param>
			public Enumerator(ListSegment<T> segment)
			{
				_list = segment.List;
				_index = segment.Offset - 1;
				_maxIndex = segment.Offset + segment.Count;
			}

			/// <inheritdoc/>
			readonly void IDisposable.Dispose()
			{
			}

			/// <inheritdoc/>
			public bool MoveNext()
			{
				return ++_index < _maxIndex;
			}

			/// <inheritdoc/>
			readonly object? IEnumerator.Current => Current;

			/// <inheritdoc/>
			void IEnumerator.Reset() => throw new InvalidOperationException();
		}

		/// <inheritdoc/>
		public IEnumerator<T> GetEnumerator() => new Enumerator(this);

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}

	/// <summary>
	/// Extension method to slice a list
	/// </summary>
	public static class ListSegmentExtensions
	{
		/// <summary>
		/// Create a fixed size list segment
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> AsSegment<T>(this IList<T> list)
		{
			return new ListSegment<T>(list);
		}

		/// <summary>
		/// Create a fixed size list segment
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <param name="offset">Starting position within the list</param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> AsSegment<T>(this IList<T> list, int offset)
		{
			return new ListSegment<T>(list, offset, list.Count - offset);
		}

		/// <summary>
		/// Create a fixed size list segment
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <param name="offset">Starting position within the list</param>
		/// <param name="length">Length of the list</param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> AsSegment<T>(this IList<T> list, int offset, int length)
		{
			return new ListSegment<T>(list, offset, length);
		}

		/// <summary>
		/// Create a fixed size list segment
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <param name="range">Range to create the segment from</param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> AsSegment<T>(this IList<T> list, Range range)
		{
			(int offset, int length) = range.GetOffsetAndLength(list.Count);
			return new ListSegment<T>(list, offset, length);
		}

		/// <summary>
		/// Take a slice from a list
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <param name="offset">Starting position within the list</param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> Slice<T>(this List<T> list, int offset)
		{
			return new ListSegment<T>(list, offset, list.Count - offset);
		}

		/// <summary>
		/// Take a slice from a list
		/// </summary>
		/// <typeparam name="T">List element type</typeparam>
		/// <param name="list">Base list to turn into a segment</param>
		/// <param name="offset">Starting position within the list</param>
		/// <param name="length"></param>
		/// <returns>New segment for the list</returns>
		public static ListSegment<T> Slice<T>(this List<T> list, int offset, int length)
		{
			return new ListSegment<T>(list, offset, length);
		}
	}
}
