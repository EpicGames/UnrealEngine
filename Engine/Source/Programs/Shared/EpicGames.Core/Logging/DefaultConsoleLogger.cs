// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Simple implementation of <see cref="ILogger"/> which formats output to the console similar to Serilog's 'Code' style.
	/// </summary>
	public class DefaultConsoleLogger : ILogger
	{
		sealed class Disposable : IDisposable
		{
			public void Dispose() { }
		}

		const string AnsiReset = "\x1b[0m";

		const string AnsiTime = "\x1b[38;5;0246m";

		const string AnsiTrace = "\x1b[37m";
		const string AnsiDebug = "\x1b[37m";
		const string AnsiInformation = "\x1b[37;1m";
		const string AnsiWarning = "\x1b[38;5;0229m";
		const string AnsiError = "\x1b[38;5;0197m\x1b[48;5;0238m";

		const string AnsiNull = "\x1b[38;5;0038m";
		const string AnsiBool = "\x1b[38;5;0038m";
		const string AnsiNumber = "\x1b[38;5;151m";
		const string AnsiString = "\x1b[38;5;0216m";

		readonly LogLevel _minLevel;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="minLevel">Minimum level for messages to be output</param>
		public DefaultConsoleLogger(LogLevel minLevel = LogLevel.Debug)
		{
			_minLevel = minLevel;
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => logLevel >= _minLevel;

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => new Disposable();

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			using StringWriter writer = new();
			Write<TState>(logLevel, state, exception, formatter, writer);
			Console.WriteLine(writer.ToString());
		}

		static void Write<TState>(LogLevel logLevel, TState state, Exception? exception, Func<TState, Exception?, string> formatter, TextWriter textWriter)
		{
			DateTime time = DateTime.Now;

			textWriter.Write(AnsiTime);
			textWriter.Write($"[{time:HH:mm:ss} ");
			switch (logLevel)
			{
				case LogLevel.Trace:
					textWriter.Write(AnsiTrace);
					textWriter.Write("trc");
					break;
				case LogLevel.Debug:
					textWriter.Write(AnsiDebug);
					textWriter.Write("dbg");
					break;
				case LogLevel.Information:
					textWriter.Write(AnsiInformation);
					textWriter.Write("inf" + AnsiTime);
					break;
				case LogLevel.Warning:
					textWriter.Write(AnsiWarning);
					textWriter.Write("wrn" + AnsiTime);
					break;
				case LogLevel.Error:
					textWriter.Write(AnsiError);
					textWriter.Write("err");
					break;
				case LogLevel.Critical:
					textWriter.Write(AnsiError);
					textWriter.Write("ctl");
					break;
			}
			textWriter.Write(AnsiReset);
			textWriter.Write(AnsiTime);
			textWriter.Write("] ");
			textWriter.Write(AnsiReset);

			const string OriginalFormatKeyName = "{OriginalFormat}";
			if (state is IEnumerable<KeyValuePair<string, object>> enumerable)
			{
				string? originalFormat = enumerable.FirstOrDefault(x => x.Key.Equals(OriginalFormatKeyName, StringComparison.OrdinalIgnoreCase)).Value as string;
				if (originalFormat != null)
				{
					ReadOnlySpan<char> fmt = originalFormat.AsSpan();
					while (fmt.Length > 0)
					{
						fmt = WriteToken(fmt, enumerable, textWriter);
					}
					return;
				}
			}

			textWriter.Write(formatter(state, exception));
		}

		static ReadOnlySpan<char> WriteToken(ReadOnlySpan<char> fmt, IEnumerable<KeyValuePair<string, object>> properties, TextWriter writer)
		{
			int idx = fmt.IndexOf('{');
			if (idx == -1 || idx + 1 == fmt.Length)
			{
				writer.Write(fmt);
				return [];
			}
			if (fmt[idx + 1] == '{')
			{
				writer.Write(fmt.Slice(0, idx + 1));
				return fmt.Slice(idx + 2);
			}

			writer.Write(fmt.Slice(0, idx));
			fmt = fmt.Slice(idx);

			int endIdx = fmt.IndexOf('}');
			if (endIdx == -1)
			{
				writer.Write(fmt);
				return [];
			}

			ReadOnlySpan<char> name = fmt.Slice(1, endIdx - 1);
			ReadOnlySpan<char> spec = [];

			int specIdx = name.IndexOfAny(':', ',');
			if (specIdx != -1)
			{
				spec = name.Slice(specIdx);
				name = name.Slice(0, specIdx);
			}

			object? obj = GetPropertyValue(properties, name);
			string? color = obj switch
			{
				null => AnsiNull,

				bool _ => AnsiBool,

				byte _ => AnsiNumber,
				sbyte _ => AnsiNumber,
				short _ => AnsiNumber,
				ushort _ => AnsiNumber,
				int _ => AnsiNumber,
				ulong _ => AnsiNumber,
				float _ => AnsiNumber,
				double _ => AnsiNumber,

				string _ => AnsiString,

				_ => null
			};

			if (color != null)
			{
				writer.Write(color);
			}

			if (spec.Length > 0)
			{
				writer.Write($"{{0{spec}}}", obj);
			}
			else
			{
				writer.Write(obj);
			}

			if (color != null)
			{
				writer.Write(AnsiReset);
			}

			return fmt.Slice(endIdx + 1);
		}

		static object? GetPropertyValue(IEnumerable<KeyValuePair<string, object>> enumerable, ReadOnlySpan<char> name)
		{
			foreach (KeyValuePair<string, object> item in enumerable)
			{
				if (name.Equals(item.Key, StringComparison.Ordinal))
				{
					return item.Value;
				}
			}
			return null;
		}
	}
}
