// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Logger which forwards output to several other loggers
	/// </summary>
	public class ForwardingLogger : ILogger
	{
		/// <summary>
		/// List of IDisposable instances
		/// </summary>
		class DisposableList : List<IDisposable>, IDisposable
		{
			/// <inheritdoc/>
			public void Dispose()
			{
				foreach (IDisposable child in this)
				{
					child.Dispose();
				}
			}
		}

		/// <summary>
		/// The loggers that are being written to
		/// </summary>
		public IReadOnlyList<ILogger> Loggers
		{
			get;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="loggers">Array of loggers to forward to</param>
		public ForwardingLogger(params ILogger[] loggers)
		{
			Loggers = loggers;
		}

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
		{
			DisposableList scope = new();
			foreach (ILogger logger in Loggers)
			{
				IDisposable? next = logger.BeginScope(state);
				if (next != null)
				{
					scope.Add(next);
				}
			}
			return scope;
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel)
		{
			foreach (ILogger logger in Loggers)
			{
				if (logger.IsEnabled(logLevel))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			foreach (ILogger logger in Loggers)
			{
				logger.Log(logLevel, eventId, state, exception, formatter);
			}
		}
	}
}
