// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text.RegularExpressions;

namespace EpicGames.Core
{
	/// <summary>
	/// Allows querying the input text from the current cursor position
	/// </summary>
	public interface ILogCursor
	{
		/// <summary>
		/// Text for the current line
		/// </summary>
		string? CurrentLine
		{
			get;
		}

		/// <summary>
		/// The current line number
		/// </summary>
		int CurrentLineNumber
		{
			get;
		}

		/// <summary>
		/// Returns the line count
		/// </summary>
		int Length
		{
			get;
		}

		/// <summary>
		/// Index to find the string at the given offset
		/// </summary>
		/// <param name="offset"></param>
		/// <returns></returns>
		string? this[int offset]
		{
			get;
		}
	}

	/// <summary>
	/// Extension methods for log cursors
	/// </summary>
	public static partial class LogCursorExtensions
	{
		/// <summary>
		/// Implementation of ILogCursor which positions the cursor at a fixed offset from the inner cursor
		/// </summary>
		class RebasedLogCursor : ILogCursor
		{
			readonly ILogCursor _inner;
			readonly int _baseLineNumber;

			public RebasedLogCursor(ILogCursor inner, int baseLineNumber)
			{
				_inner = inner;
				_baseLineNumber = baseLineNumber;
			}

			public string? this[int index] => _inner[(_baseLineNumber + index) - _inner.CurrentLineNumber];
			public string? CurrentLine => _inner[_baseLineNumber - _inner.CurrentLineNumber];
			public int CurrentLineNumber => _baseLineNumber;
			public int Length => _inner.Length - (_baseLineNumber - _inner.CurrentLineNumber);
		}

		/// <summary>
		/// Creates a new log cursor based at an offset from the current line
		/// </summary>
		/// <param name="cursor">The current log cursor instance</param>
		/// <param name="offset">Line number offset from the current</param>
		/// <returns>New log cursor instance</returns>
		public static ILogCursor Rebase(this ILogCursor cursor, int offset)
		{
			return new RebasedLogCursor(cursor, cursor.CurrentLineNumber + offset);
		}

		/// <summary>
		/// Attempts to get a line at the given offset
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">Offset of the line to retrieve</param>
		/// <param name="nextLine">On success, receives the matched line</param>
		/// <returns>True if the line was retrieved</returns>
		public static bool TryGetLine(this ILogCursor cursor, int offset, [NotNullWhen(true)] out string? nextLine)
		{
			nextLine = cursor[offset];
			return nextLine != null;
		}

		/// <summary>
		/// Tests whether the current line contains the given text
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="text">The text to look for</param>
		/// <returns>True if the line contains the given string</returns>
		public static bool Contains(this ILogCursor cursor, string text) => Contains(cursor, 0, text);

		/// <summary>
		/// Tests whether the current line contains the given text
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">Offset of the line to match</param>
		/// <param name="text">The text to look for</param>
		/// <returns>True if the line contains the given string</returns>
		public static bool Contains(this ILogCursor cursor, int offset, string text)
		{
			string? line;
			return cursor.TryGetLine(offset, out line) && line.Contains(text, StringComparison.Ordinal);
		}

		/// <summary>
		/// Tests whether the current line starts with the given text
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="text">The text to look for</param>
		/// <returns>True if the line starts with the given string</returns>
		public static bool StartsWith(this ILogCursor cursor, string text) => StartsWith(cursor, 0, text);

		/// <summary>
		/// Tests whether the current line starts with the given text
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">Offset of the line to match</param>
		/// <param name="text">The text to look for</param>
		/// <returns>True if the line starts with the given string</returns>
		public static bool StartsWith(this ILogCursor cursor, int offset, string text)
		{
			string? line;
			return cursor.TryGetLine(offset, out line) && line.StartsWith(text, StringComparison.Ordinal);
		}

		/// <summary>
		/// Determines if the current line matches the given regex
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="pattern">The regex pattern to match</param>
		/// <returns>True if the current line matches the given patter</returns>
		public static bool IsMatch(this ILogCursor cursor, Regex pattern)
		{
			return IsMatch(cursor, 0, pattern);
		}

		/// <summary>
		/// Determines if the line at the given offset matches the given regex
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">Offset of the line to match</param>
		/// <param name="pattern">The regex pattern to match</param>
		/// <returns>True if the requested line matches the given patter</returns>
		public static bool IsMatch(this ILogCursor cursor, int offset, Regex pattern)
		{
			string? line;
			return cursor.TryGetLine(offset, out line) && pattern.IsMatch(line!);
		}

		/// <summary>
		/// Determines if the current line matches the given regex
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="pattern">The regex pattern to match</param>
		/// <param name="outMatch">On success, receives the match result</param>
		/// <returns>True if the current line matches the given patter</returns>
		public static bool TryMatch(this ILogCursor cursor, Regex pattern, [NotNullWhen(true)] out Match? outMatch)
		{
			return TryMatch(cursor, 0, pattern, out outMatch);
		}

		/// <summary>
		/// Determines if the line at the given offset matches the given regex
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">The line offset to check</param>
		/// <param name="pattern">The regex pattern to match</param>
		/// <param name="outMatch">On success, receives the match result</param>
		/// <returns>True if the current line matches the given patter</returns>
		public static bool TryMatch(this ILogCursor cursor, int offset, Regex pattern, [NotNullWhen(true)] out Match? outMatch)
		{
			string? line;
			if (!cursor.TryGetLine(offset, out line))
			{
				outMatch = null;
				return false;
			}

			Match match = pattern.Match(line);
			if (!match.Success)
			{
				outMatch = null;
				return false;
			}

			outMatch = match;
			return true;
		}

		/// <summary>
		/// Matches lines forward from the given offset while the given pattern matches
		/// </summary>
		/// <param name="cursor">The log cursor instance</param>
		/// <param name="offset">Initial offset</param>
		/// <param name="pattern">Pattern to match</param>
		/// <returns>Offset of the last line that still matches the pattern (inclusive)</returns>
		public static int MatchForwards(this ILogCursor cursor, int offset, Regex pattern)
		{
			while (IsMatch(cursor, offset + 1, pattern))
			{
				offset++;
			}
			return offset;
		}

		/// <summary>
		/// Matches lines forwards from the given offset until the given pattern matches
		/// </summary>
		/// <param name="cursor">The log cursor</param>
		/// <param name="offset">Initial offset</param>
		/// <param name="pattern">Pattern to match</param>
		/// <returns>Offset of the line that matches the pattern (inclusive), or EOF is encountered</returns>
		public static int MatchForwardsUntil(this ILogCursor cursor, int offset, Regex pattern)
		{
			string? nextLine;
			for (int nextOffset = offset + 1; cursor.TryGetLine(nextOffset, out nextLine); nextOffset++)
			{
				if (pattern.IsMatch(nextLine))
				{
					return nextOffset;
				}
			}
			return offset;
		}

		/// <summary>
		/// Matches lines forwards from the given offset until the given pattern matches
		/// </summary>
		/// <param name="cursor">The log cursor</param>
		/// <param name="offset">Initial offset</param>
		/// <param name="pattern">Pattern to match</param>
		/// <param name="maxLines">Max number of lines to look forward</param>
		/// <returns>Offset of the line that matches the pattern (inclusive), or EOF is encountered</returns>
		public static int MatchForwardsLimited(this ILogCursor cursor, int offset, Regex pattern, int maxLines)
		{
			string? nextLine;
			int numLines = 0;
			int nextOffset = offset + 1;
			cursor.TryGetLine(nextOffset, out nextLine);

			while (numLines < maxLines && nextLine != null)
			{

				if (pattern.IsMatch(nextLine))
				{
					return nextOffset;
				}
				cursor.TryGetLine(nextOffset, out nextLine);
				nextOffset++;
				numLines++;
			}
			return offset;
		}

		/// <summary>
		/// Tests if a line consists only of whitespace
		/// </summary>
		/// <param name="cursor">The log cursor</param>
		/// <param name="offset">Offset of the line to check</param>
		/// <returns></returns>
		public static bool IsBlank(this ILogCursor cursor, int offset)
		{
			if (!cursor.TryGetLine(offset, out string? line))
			{
				return false;
			}

			for (int idx = 0; idx < line.Length; idx++)
			{
				if (!Char.IsWhiteSpace(line[idx]))
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Check if a line at a given offset is left-aligned with at least this indent of another line
		/// </summary>
		/// <param name="cursor">The log cursor</param>
		/// <param name="offset">Offset of the line to check</param>
		/// <param name="firstLine">The first line to compare with</param>
		/// <returns>True if the line at the given offset is aligned at least as much as the current lien</returns>
		public static bool IsAligned(this ILogCursor cursor, int offset, string? firstLine)
		{
			if (firstLine == null || !cursor.TryGetLine(offset, out string? line))
			{
				return false;
			}

			for (int idx = 0; idx < line.Length; idx++)
			{
				if (idx == firstLine.Length || !Char.IsWhiteSpace(firstLine[idx]))
				{
					return true;
				}
				if (line[idx] != firstLine[idx])
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Check if a line at a given offset is left-aligned with at least this indent of another line
		/// </summary>
		/// <param name="cursor">The log cursor</param>
		/// <param name="offset">Offset of the line to check</param>
		/// <param name="firstLine">The first line to compare with</param>
		/// <returns>True if the line at the given offset is aligned at least as much as the current lien</returns>
		public static bool IsHanging(this ILogCursor cursor, int offset, string? firstLine)
		{
			if (firstLine == null || !cursor.TryGetLine(offset, out string? line))
			{
				return false;
			}

			for (int idx = 0; idx < line.Length; idx++)
			{
				if (idx == firstLine.Length || !Char.IsWhiteSpace(firstLine[idx]))
				{
					return Char.IsWhiteSpace(line[idx]);
				}
				if (line[idx] != firstLine[idx])
				{
					return false;
				}
			}

			return true;
		}
	}
}
