// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Class to apply a log indent for the lifetime of an object 
	/// </summary>
	public interface ILoggerIndent
	{
		/// <summary>
		/// The indent to apply
		/// </summary>
		string Indent { get; }
	}

	/// <summary>
	/// Wrapper class for ILogger classes which supports LoggerStatusScope
	/// </summary>
	public class DefaultLoggerIndentHandler : ILogger
	{
		/// <summary>
		/// Scoped indent message
		/// </summary>
		class Scope : IDisposable
		{
			/// <summary>
			/// Owning object
			/// </summary>
			readonly DefaultLoggerIndentHandler _owner;

			/// <summary>
			/// The indent scope object
			/// </summary>
			public ILoggerIndent Indent { get; }

			/// <summary>
			/// Constructor
			/// </summary>
			public Scope(DefaultLoggerIndentHandler owner, ILoggerIndent indent)
			{
				_owner = owner;
				Indent = indent;

				lock (owner._scopes)
				{
					owner._scopes.Add(this);
				}
			}

			/// <summary>
			/// Remove this indent from the list
			/// </summary>
			public void Dispose()
			{
				lock (_owner._scopes)
				{
					_owner._scopes.Remove(this);
				}
			}
		}

		/// <summary>
		/// Struct to wrap a formatted set of log values with applied indent
		/// </summary>
		/// <typeparam name="TState">Arbitrary type parameter</typeparam>
		readonly struct FormattedLogValues<TState> : IEnumerable<KeyValuePair<string, object>>
		{
			/// <summary>
			/// The indent to apply
			/// </summary>
			readonly string _indent;

			/// <summary>
			/// The inner state
			/// </summary>
			readonly TState _state;

			/// <summary>
			/// Formatter for the inner state
			/// </summary>
			readonly Func<TState, Exception?, string> _formatter;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="indent">The indent to apply</param>
			/// <param name="state">The inner state</param>
			/// <param name="formatter">Formatter for the inner state</param>
			public FormattedLogValues(string indent, TState state, Func<TState, Exception?, string> formatter)
			{
				_indent = indent;
				_state = state;
				_formatter = formatter;
			}

			/// <inheritdoc/>
			public IEnumerator<KeyValuePair<string, object>> GetEnumerator()
			{
				IEnumerable<KeyValuePair<string, object>>? innerEnumerable = _state as IEnumerable<KeyValuePair<string, object>>;
				if (innerEnumerable != null)
				{
					foreach (KeyValuePair<string, object> pair in innerEnumerable)
					{
						if (pair.Key.Equals("{OriginalFormat}", StringComparison.Ordinal))
						{
							yield return new KeyValuePair<string, object>(pair.Key, _indent + pair.Value.ToString());
						}
						else
						{
							yield return pair;
						}
					}
				}
			}

			/// <inheritdoc/>
			IEnumerator IEnumerable.GetEnumerator()
			{
				throw new NotImplementedException();
			}

			/// <summary>
			/// Formats an instance of this object
			/// </summary>
			/// <param name="values">The object instance</param>
			/// <param name="exception">The exception to format</param>
			/// <returns>The formatted string</returns>
			public static string Format(FormattedLogValues<TState> values, Exception? exception)
			{
				return values._indent + values._formatter(values._state, exception);
			}
		}

		/// <summary>
		/// The internal logger
		/// </summary>
		readonly ILogger _inner;

		/// <summary>
		/// Current list of indents
		/// </summary>
		readonly List<Scope> _scopes = [];

		/// <summary>
		/// The inner logger
		/// </summary>
		public ILogger Inner => _inner;

		/// <summary>
		/// The current indent text
		/// </summary>
		public string Indent
		{
			get;
			private set;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The logger to wrap</param>
		public DefaultLoggerIndentHandler(ILogger inner)
		{
			_inner = inner;
			Indent = "";
		}

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
		{
			ILoggerIndent? indent = state as ILoggerIndent;
			if (indent != null)
			{
				return new Scope(this, indent);
			}

			return _inner.BeginScope(state);
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel)
		{
			return _inner.IsEnabled(logLevel);
		}

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			if (_scopes.Count > 0)
			{
				string indent = String.Join("", _scopes.Select(x => x.Indent.Indent));
				_inner.Log(logLevel, eventId, new FormattedLogValues<TState>(indent, state, formatter), exception, FormattedLogValues<TState>.Format);
				return;
			}

			_inner.Log(logLevel, eventId, state, exception, formatter);
		}
	}

	/// <summary>
	/// Extension methods for creating an indent
	/// </summary>
	public static class LoggerIndentExtensions
	{
		/// <summary>
		/// Class to apply a log indent for the lifetime of an object 
		/// </summary>
		class LoggerIndent : ILoggerIndent
		{
			/// <summary>
			/// The previous indent
			/// </summary>
			public string Indent { get; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="indent">Indent to append to the existing indent</param>
			public LoggerIndent(string indent)
			{
				Indent = indent;
			}
		}

		/// <summary>
		/// Create an indent
		/// </summary>
		/// <param name="logger">Logger interface</param>
		/// <param name="indent">The indent to apply</param>
		/// <returns>Disposable object</returns>
		public static IDisposable? BeginIndentScope(this ILogger logger, string indent)
		{
			return logger.BeginScope(new LoggerIndent(indent));
		}
	}
}
