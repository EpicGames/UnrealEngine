// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Manages a status message for a long running operation, which can be updated with progress. Typically transient on consoles, and written to the same line.
	/// </summary>
	public interface ILoggerProgress : IDisposable
	{
		/// <summary>
		/// Prefix message for the status
		/// </summary>
		string Message
		{
			get;
		}

		/// <summary>
		/// The current 
		/// </summary>
		string Progress
		{
			get; set;
		}
	}

	/// <summary>
	/// Extension methods for status objects
	/// </summary>
	public static class LoggerProgressExtensions
	{
		/// <summary>
		/// Concrete implementation of <see cref="ILoggerProgress"/>
		/// </summary>
		class LoggerProgress : ILoggerProgress
		{
			/// <summary>
			/// The logger to output to
			/// </summary>
			private readonly ILogger _logger;

			/// <summary>
			/// Prefix message for the status
			/// </summary>
			public string Message
			{
				get;
			}

			/// <summary>
			/// The current 
			/// </summary>
			public string Progress
			{
				get => _progressInternal;
				set
				{
					_progressInternal = value;

					if (_timer.Elapsed > TimeSpan.FromSeconds(3.0))
					{
						_lastOutput = String.Empty;
						Flush();
						_timer.Restart();
					}
				}
			}

			/// <summary>
			/// The last string that was output
			/// </summary>
			string _lastOutput = String.Empty;

			/// <summary>
			/// Backing storage for the Progress string
			/// </summary>
			string _progressInternal = String.Empty;

			/// <summary>
			/// Timer since the last update
			/// </summary>
			readonly Stopwatch _timer = Stopwatch.StartNew();

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="logger">The logger to write to</param>
			/// <param name="message">The base message to display</param>
			public LoggerProgress(ILogger logger, string message)
			{
				_logger = logger;
				Message = message;
				_progressInternal = String.Empty;

				logger.LogInformation("{Progress}", message);
			}

			/// <summary>
			/// Dispose of this object
			/// </summary>
			public void Dispose()
			{
				Flush();
			}

			/// <summary>
			/// Flushes the current output to the log
			/// </summary>
			void Flush()
			{
				string output = Message;
				if (!String.IsNullOrEmpty(Progress))
				{
					output += $" {Progress}";
				}
				if (!String.Equals(output, _lastOutput, StringComparison.Ordinal))
				{
					_logger.LogInformation("{Progress}", output);
					_lastOutput = output;
				}
			}
		}

		/// <summary>
		/// Begins a new progress scope
		/// </summary>
		/// <param name="logger">The logger being written to</param>
		/// <param name="message">The message prefix</param>
		/// <returns>Scope object, which should be disposed when finished</returns>
		public static ILoggerProgress BeginProgressScope(this ILogger logger, string message)
		{
			return new LoggerProgress(logger, message);
		}
	}
}
