// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for more convenient attaching of properties to log messages
	/// </summary>
	public static class LoggerPropertyExtensions
	{
		/// <summary>
		/// Builder struct for attaching properties to log messages
		/// Wraps and disposes log scope.
		/// </summary>
		public readonly struct LogScopedPropertyBuilder
		{
			private readonly Dictionary<string, object> _properties;
			private readonly ILogger _logger;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="logger">Logger to wrap</param>
			public LogScopedPropertyBuilder(ILogger logger)
			{
				_properties = [];
				_logger = logger;
			}

			/// <summary>
			/// Set a key/value property for current log scope
			/// </summary>
			/// <param name="key">Arbitrary key</param>
			/// <param name="value">Arbitrary value</param>
			/// <returns></returns>
			public LogScopedPropertyBuilder WithProperty(string key, object value)
			{
				_properties[key] = value;
				return this;
			}

			/// <summary>
			/// Starts new scope on underlying logger, passing previously set properties
			/// </summary>
			/// <returns>Original IDisposable from ILogger.BeginScope()</returns>
			public IDisposable? BeginScope()
			{
				return _logger.BeginScope(_properties);
			}
		}

		/// <summary>
		/// Creates a new log scope and attaches a key/value property
		/// Additional properties can be added with <see cref="LogScopedPropertyBuilder.WithProperty" />.
		/// </summary>
		/// <param name="logger">Logger to use</param>
		/// <param name="key">Arbitrary key</param>
		/// <param name="value">Arbitrary value</param>
		/// <returns>A disposable log property builder that wraps the original ILogger scope</returns>
		/// <example>
		/// Simple example of attaching two properties
		/// <code>
		/// using (_logger.WithProperty("stream", "ue5").WithProperty("os", "windows").BuildScope())
		/// {
		///     _logger.LogInformation("A log message with properties attached");
		/// }
		/// </code>
		/// </example>
		public static LogScopedPropertyBuilder WithProperty(this ILogger logger, string key, object value)
		{
			LogScopedPropertyBuilder builder = new(logger);
			builder.WithProperty(key, value);
			return builder;
		}
	}
}
