// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Text;
using System.Text.Json;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Defines a preformatted Json log event, which can pass through raw Json data directly or format it as a regular string
	/// </summary>
	public readonly struct JsonLogEvent : IEnumerable<KeyValuePair<string, object?>>
	{
		/// <summary>
		/// The log level
		/// </summary>
		public LogLevel Level { get; }

		/// <summary>
		/// The event id, if set
		/// </summary>
		public EventId EventId { get; }

		/// <summary>
		/// Index of this line
		/// </summary>
		public int LineIndex { get; }

		/// <summary>
		/// Number of lines in a multi-line message
		/// </summary>
		public int LineCount { get; }

		/// <summary>
		/// The utf-8 encoded JSON event
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonLogEvent(LogEvent logEvent)
			: this(logEvent.Level, logEvent.Id, logEvent.LineIndex, logEvent.LineCount, logEvent.ToJsonBytes())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonLogEvent(LogLevel level, EventId eventId, int lineIndex, int lineCount, ReadOnlyMemory<byte> data)
		{
			Level = level;
			EventId = eventId;
			LineIndex = lineIndex;
			LineCount = lineCount;
			Data = data;
		}

		/// <summary>
		/// Creates a json log event from the given logger parameters
		/// </summary>
		/// <inheritdoc cref="ILogger.Log{TState}(LogLevel, EventId, TState, Exception, Func{TState, Exception, String})"/>
		public static JsonLogEvent FromLoggerState<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			if (state is JsonLogEvent jsonLogEvent)
			{
				return jsonLogEvent;
			}

			LogEvent? logEvent = state as LogEvent ?? LogEvent.FromState(logLevel, eventId, state, exception, formatter);
			return new JsonLogEvent(logLevel, eventId, 0, 1, logEvent.ToJsonBytes());
		}

		/// <summary>
		/// Parse an event from the given data
		/// </summary>
		/// <param name="data"></param>
		/// <returns></returns>
		public static JsonLogEvent Parse(ReadOnlyMemory<byte> data)
		{
			JsonLogEvent logEvent;
			if (!TryParseInternal(data, out logEvent))
			{
				throw new InvalidOperationException("Cannot parse string");
			}
			return logEvent;
		}

		/// <summary>
		/// Tries to parse a Json log event from the given string
		/// </summary>
		/// <param name="data"></param>
		/// <param name="logEvent"></param>
		/// <returns></returns>
		public static bool TryParse(ReadOnlyMemory<byte> data, out JsonLogEvent logEvent)
		{
			try
			{
				return TryParseInternal(data, out logEvent);
			}
			catch
			{
				logEvent = default;
				return false;
			}
		}

		/// <summary>
		/// Tries to parse a Json log event from the given string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="logEvent"></param>
		/// <returns></returns>
		public static bool TryParse(string text, out JsonLogEvent logEvent)
		{
			byte[] data = Encoding.UTF8.GetBytes(text);
			return TryParse(data, out logEvent);
		}

		static bool TryParseInternal(ReadOnlyMemory<byte> data, out JsonLogEvent logEvent)
		{
			LogLevel level = LogLevel.None;
			int eventId = 0;
			int lineIndex = 0;
			int lineCount = 1;

			Utf8JsonReader reader = new(data.Span);
			if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					ReadOnlySpan<byte> propertyName = reader.ValueSpan;

					if (!reader.Read())
					{
						break;
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Level) && reader.TokenType == JsonTokenType.String)
					{
						level = ParseLevel(reader.ValueSpan);
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Id) && reader.TokenType == JsonTokenType.Number)
					{
						eventId = reader.GetInt32();
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Line) && reader.TokenType == JsonTokenType.Number)
					{
						reader.TryGetInt32(out lineIndex);
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.LineCount) && reader.TokenType == JsonTokenType.Number)
					{
						reader.TryGetInt32(out lineCount);
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Properties) && reader.TokenType == JsonTokenType.StartObject)
					{
						while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
						{
							ReadOnlySpan<byte> innerPropertyName = reader.ValueSpan;

							if (!reader.Read())
							{
								break;
							}

							// In the case of UE_LOGFMT the event id can be nested within the properties instead
							if (innerPropertyName.SequenceEqual(LogEventPropertyName.Id) && reader.TokenType == JsonTokenType.Number)
							{
								eventId = reader.GetInt32();
							}
							reader.Skip();
						}
					}
					reader.Skip();
				}
			}

			if (reader.TokenType == JsonTokenType.EndObject && level != LogLevel.None && reader.BytesConsumed == data.Length)
			{
				logEvent = new JsonLogEvent(level, new EventId(eventId), lineIndex, lineCount, data.ToArray());
				return true;
			}
			else
			{
				logEvent = default;
				return false;
			}
		}

		static readonly sbyte[] s_firstCharToLogLevel;
		static readonly byte[][] s_logLevelNames;

#pragma warning disable CA2207 // Initialize value type static fields inline
		static JsonLogEvent()
#pragma warning restore CA2207 // Initialize value type static fields inline
		{
			const int LogLevelCount = (int)LogLevel.None;

			s_firstCharToLogLevel = new sbyte[256];
			Array.Fill(s_firstCharToLogLevel, (sbyte)-1);

			s_logLevelNames = new byte[LogLevelCount][];
			for (int idx = 0; idx < (int)LogLevel.None; idx++)
			{
				byte[] name = Encoding.UTF8.GetBytes(Enum.GetName(typeof(LogLevel), (LogLevel)idx)!);
				s_logLevelNames[idx] = name;
				s_firstCharToLogLevel[name[0]] = (sbyte)idx;
			}
		}

		static LogLevel ParseLevel(ReadOnlySpan<byte> level)
		{
			int result = s_firstCharToLogLevel[level[0]];
			if (!level.SequenceEqual(s_logLevelNames[result]))
			{
				throw new InvalidOperationException();
			}
			return (LogLevel)result;
		}

		static readonly Utf8String s_newlineEscaped = new("\\n");

		/// <summary>
		/// Gets the rendered message from the event data
		/// </summary>
		public Utf8String GetRenderedMessage()
		{
			Utf8JsonReader reader = new(Data.Span);
			if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					ReadOnlySpan<byte> propertyName = reader.ValueSpan;
					if (!reader.Read())
					{
						break;
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Message) && reader.TokenType == JsonTokenType.String)
					{
						return new Utf8String(reader.GetUtf8String().ToArray());
					}
				}
			}
			return Utf8String.Empty;
		}

		/// <summary>
		/// Gets the event data rendered as a legacy unreal log line of the format:
		/// [timestamp][frame number]LogChannel: LogVerbosity: Message
		/// </summary>
		public string GetLegacyLogLine()
		{
			Utf8JsonReader reader = new(Data.Span);
			if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
			{
				Utf8String? message = null;
				ulong frame = 0;
				DateTime? time = null;
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					ReadOnlySpan<byte> propertyName = reader.ValueSpan;
					if (!reader.Read())
					{
						break;
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Time) && reader.TokenType == JsonTokenType.String)
					{
						time = reader.GetDateTime();
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Frame) && reader.TokenType == JsonTokenType.Number)
					{
						frame = reader.GetUInt64();
					}
					else if (propertyName.SequenceEqual(LogEventPropertyName.Message) && reader.TokenType == JsonTokenType.String)
					{
						message = new Utf8String(reader.GetUtf8String().ToArray());
					}
				}

				if (message is not null)
				{
					if (time is not null)
					{
						return $"[{time:yyyy.MM.dd-HH.mm.ss:fff}][{frame % 1000,3}]{message}";
					}
					else
					{
						return $"{message}";
					}
				}
			}
			return String.Empty;
		}

		/// <summary>
		/// Count the number of lines in the message field of a log event
		/// </summary>
		/// <returns>Number of lines in the message</returns>
		public int GetMessageLineCount()
		{
			if (Data.Span.IndexOf(s_newlineEscaped.Span) != -1)
			{
				Utf8JsonReader reader = new(Data.Span);
				if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
				{
					while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
					{
						ReadOnlySpan<byte> propertyName = reader.ValueSpan;
						if (!reader.Read())
						{
							break;
						}
						else if (propertyName.SequenceEqual(LogEventPropertyName.Message) && reader.TokenType == JsonTokenType.String)
						{
							return CountLines(reader.ValueSpan);
						}
					}
				}
			}
			return 1;
		}

		/// <summary>
		/// Counts the number of newlines in an escaped JSON string
		/// </summary>
		/// <param name="str">The escaped string</param>
		/// <returns></returns>
		static int CountLines(ReadOnlySpan<byte> str)
		{
			int lines = 1;
			for (int idx = 0; idx < str.Length - 1; idx++)
			{
				if (str[idx] == '\\')
				{
					if (str[idx + 1] == 'n')
					{
						lines++;
					}
					idx++;
				}
			}
			return lines;
		}

		/// <summary>
		/// Formats an event as a string
		/// </summary>
		/// <param name="state"></param>
		/// <param name="ex"></param>
		/// <returns></returns>
		public static string Format(JsonLogEvent state, Exception? ex)
		{
			_ = ex;
			return LogEvent.Read(state.Data.Span).ToString();
		}

		/// <summary>
		/// Find all properties of the given type in a particular log line
		/// </summary>
		/// <param name="type">Type of property to return</param>
		/// <returns></returns>
		public IEnumerable<JsonProperty> FindPropertiesOfType(Utf8String type)
		{
			JsonDocument document = JsonDocument.Parse(Data);
			return FindPropertiesOfType(document.RootElement, type);
		}

		/// <summary>
		/// Find all properties of the given type in a particular log line
		/// </summary>
		/// <param name="line">Line data</param>
		/// <param name="type">Type of property to return</param>
		/// <returns></returns>
		public static IEnumerable<JsonProperty> FindPropertiesOfType(JsonElement line, Utf8String type)
		{
			JsonElement properties;
			if (line.TryGetProperty("properties", out properties) && properties.ValueKind == JsonValueKind.Object)
			{
				foreach (JsonProperty property in properties.EnumerateObject())
				{
					if (property.Value.ValueKind == JsonValueKind.Object)
					{
						foreach (JsonProperty subProperty in property.Value.EnumerateObject())
						{
							if (subProperty.NameEquals(LogEventPropertyName.Type.Span))
							{
								if (subProperty.Value.ValueKind == JsonValueKind.String && subProperty.Value.ValueEquals(type.Span))
								{
									yield return property;
								}
								else
								{
									break;
								}
							}
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Encoding.UTF8.GetString(Data.ToArray());

		/// <inheritdoc/>
		public IEnumerator<KeyValuePair<string, object?>> GetEnumerator() => LogEvent.Read(Data.Span).GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}

	/// <summary>
	/// Extension methods for <see cref="JsonLogEvent"/>
	/// </summary>
	public static class JsonLogEventExtensions
	{
		/// <summary>
		/// Logs a <see cref="JsonLogEvent"/> to the given logger
		/// </summary>
		/// <param name="logger">Logger to write to</param>
		/// <param name="jsonLogEvent">Json log event to write</param>
		public static void LogJsonLogEvent(this ILogger logger, JsonLogEvent jsonLogEvent)
		{
			logger.Log(jsonLogEvent.Level, jsonLogEvent.EventId, jsonLogEvent, null, JsonLogEvent.Format);
		}
	}
}
