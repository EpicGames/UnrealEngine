// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Well known log events
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores", Justification = "Intentional")]
	public static class KnownLogEvents
	{
		/// <summary>
		/// Unset
		/// </summary>
		public static EventId None { get; } = new EventId(0);

		/// <summary>
		/// Generic error
		/// </summary>
		public static EventId Generic { get; } = new EventId(1);

		/// <summary>
		/// Generic exception
		/// </summary>
		public static EventId Exception { get; } = new EventId(2);

		/// <summary>
		/// Exit code error
		/// </summary>
		public static EventId ExitCode { get; } = new EventId(3);

		/// <summary>
		/// Compiler error
		/// </summary>
		public static EventId Compiler { get; } = new EventId(100);

		/// <summary>
		/// Compiler summary line (eg. 'XYZ failed to compile')
		/// </summary>
		public static EventId Compiler_Summary { get; } = new EventId(101);

		/// <summary>
		/// TargetRules or ModuleRules error
		/// </summary>
		public static EventId RulesAssembly { get; } = new EventId(102);

		/// <summary>
		/// Validation messages
		/// </summary>
		public static EventId Validation { get; } = new EventId(103);

		/// <summary>
		/// Circular dependency messages
		/// </summary>
		public static EventId CircularDependency { get; } = new EventId(104);

		/// <summary>
		/// UHT compiler message
		/// </summary>
		public static EventId UHT { get; } = new EventId(110);

		/// <summary>
		/// UBA message
		/// </summary>
		public static EventId UBA { get; } = new EventId(111);

		/// <summary>
		/// Linker error
		/// </summary>
		public static EventId Linker { get; } = new EventId(200);

		/// <summary>
		/// Linker: Undefined symbol
		/// </summary>
		public static EventId Linker_UndefinedSymbol { get; } = new EventId(201);

		/// <summary>
		/// Linker: Multiply defined symbol
		/// </summary>
		public static EventId Linker_DuplicateSymbol { get; } = new EventId(202);

		/// <summary>
		/// Engine error
		/// </summary>
		public static EventId Engine { get; } = new EventId(300);

		/// <summary>
		/// Engine log channel
		/// </summary>
		public static EventId Engine_LogChannel { get; } = new EventId(301);

		/// <summary>
		/// Engine: Crash dump
		/// </summary>
		public static EventId Engine_Crash { get; } = new EventId(302);

		/// <summary>
		/// Engine: UE_ASSET_LOG output
		/// </summary>
		public static EventId Engine_AssetLog { get; } = new EventId(303);

		/// <summary>
		/// Engine: Localization commandlet output
		/// </summary>
		public static EventId Engine_Localization { get; } = new EventId(304);

		/// <summary>
		/// Engine: appErrorf called
		/// </summary>
		public static EventId Engine_AppError { get; } = new EventId(305);

		/// <summary>
		/// Engine: Assertion failed
		/// </summary>
		public static EventId Engine_AssertionFailed { get; } = new EventId(306);

		/// <summary>
		/// Engine: Shader compiler output
		/// </summary>
		public static EventId Engine_ShaderCompiler { get; } = new EventId(310);

		/// <summary>
		/// UAT error
		/// </summary>
		public static EventId AutomationTool { get; } = new EventId(400);

		/// <summary>
		/// UAT: Crash dump
		/// </summary>
		public static EventId AutomationTool_Crash { get; } = new EventId(401);

		/// <summary>
		/// UAT: Exit code indicating a crash
		/// </summary>
		public static EventId AutomationTool_CrashExitCode { get; } = new EventId(402);

		/// <summary>
		/// UAT: Source file with line number
		/// </summary>
		public static EventId AutomationTool_SourceFileLine { get; } = new EventId(403);

		/// <summary>
		/// UAT: Missing copyright notice
		/// </summary>
		public static EventId AutomationTool_MissingCopyright { get; } = new EventId(404);

		/// <summary>
		/// UAT: Mismatched Perforce case
		/// </summary>
		public static EventId AutomationTool_PerforceCase { get; } = new EventId(405);

		/// <summary>
		/// UAT: Unacceptable words
		/// </summary>
		public static EventId AutomationTool_UnacceptableWords { get; } = new EventId(406);

		/// <summary>
		/// UAT: BuildGraph script parsing error
		/// </summary>
		public static EventId AutomationTool_BuildGraphScript { get; } = new EventId(407);

		/// <summary>
		/// UAT: CodeSign message
		/// </summary>
		public static EventId AutomationTool_CodeSign { get; } = new EventId(408);

		/// <summary>
		/// MSBuild: Generic error
		/// </summary>
		public static EventId MSBuild { get; } = new EventId(500);

		/// <summary>
		/// Microsoft: Generic MSTest error
		/// </summary>
		public static EventId MSTest { get; } = new EventId(540);

		/// <summary>
		/// Microsoft: Generic Visual Studio error
		/// </summary>
		public static EventId Microsoft { get; } = new EventId(550);

		/// <summary>
		/// Error message from Gauntlet
		/// </summary>
		public static EventId Gauntlet { get; } = new EventId(600);

		/// <summary>
		/// Error message from Gauntlet test events
		/// </summary>
		public static EventId Gauntlet_TestEvent { get; } = new EventId(601);

		/// <summary>
		/// Error message from Gauntlet device events
		/// </summary>
		public static EventId Gauntlet_DeviceEvent { get; } = new EventId(602);

		/// <summary>
		/// Error message from Gauntlet Unreal Engine test events
		/// </summary>
		public static EventId Gauntlet_UnrealEngineTestEvent { get; } = new EventId(603);

		/// <summary>
		/// Error message from Gauntlet build drop events
		/// </summary>
		public static EventId Gauntlet_BuildDropEvent { get; } = new EventId(604);

		/// <summary>
		/// Fatal Error message from Gauntlet events
		/// </summary>
		public static EventId Gauntlet_FatalEvent { get; } = new EventId(605);

		/// <summary>
		/// A systemic event, relating to the health of the farm
		/// </summary>
		public static EventId Systemic { get; } = new EventId(700);

		/// <summary>
		/// A systemic event relating to Perforce
		/// </summary>
		public static EventId Systemic_Perforce { get; } = new EventId(701);

		/// <summary>
		/// A systemic event from XGE
		/// </summary>
		public static EventId Systemic_Xge { get; } = new EventId(710);

		/// <summary>
		/// Builds will run in standalone mode
		/// </summary>
		public static EventId Systemic_Xge_Standalone { get; } = new EventId(711);

		/// <summary>
		/// BuildService.exe is not running
		/// </summary>
		public static EventId Systemic_Xge_ServiceNotRunning { get; } = new EventId(712);

		/// <summary>
		/// General build failed error
		/// </summary>
		public static EventId Systemic_Xge_BuildFailed { get; } = new EventId(713);

		/// <summary>
		/// Cache size reached
		/// </summary>
		public static EventId Systemic_Xge_CacheLimit { get; } = new EventId(714);

		/// <summary>
		/// Current logging level may impact performance
		/// </summary>
		public static EventId Systemic_Xge_DetailedLogging { get; } = new EventId(715);

		/// <summary>
		/// Metadata about an XGE task (local or remote execution, start/end time etc)
		/// </summary>
		public static EventId Systemic_Xge_TaskMetadata { get; } = new EventId(716);

		/// <summary>
		/// DDC is slow
		/// </summary>
		public static EventId Systemic_SlowDDC { get; } = new EventId(720);

		/// <summary>
		/// Internal Horde error
		/// </summary>
		public static EventId Systemic_Horde { get; } = new EventId(730);

		/// <summary>
		/// Artifact upload failed
		/// </summary>
		public static EventId Systemic_Horde_ArtifactUpload { get; } = new EventId(731);

		/// <summary>
		/// Internal Horde error
		/// </summary>
		public static EventId Systemic_Horde_Compute { get; } = new EventId(732);

		/// <summary>
		/// HTTP error
		/// </summary>
		public static EventId Systemic_Horde_Http { get; } = new EventId(733);

		/// <summary>
		/// Harmless pdbutil error
		/// </summary>
		public static EventId Systemic_PdbUtil { get; } = new EventId(740);

		/// <summary>
		/// A systemic event from MSBuild
		/// </summary>
		public static EventId Systemic_MSBuild { get; } = new EventId(750);

		/// <summary>
		/// A systemic event from XCode
		/// </summary>
		public static EventId Systemic_XCode { get; } = new EventId(755);

		/// <summary>
		/// Robomerge gate is locked
		/// </summary>
		public static EventId Systemic_RoboMergeGateLocked { get; } = new EventId(760);

		/// <summary>
		/// XGEControlWorker is missing
		/// </summary>
		public static EventId Systemic_MissingXgeControlWorker { get; } = new EventId(761);

		/// <summary>
		/// Log parser is taking a significant amount of time; may be bottleneck.
		/// </summary>
		public static EventId Systemic_LogParserBottleneck { get; } = new EventId(762);

		/// <summary>
		/// Exception parsing a log event.
		/// </summary>
		public static EventId Systemic_LogEventMatcher { get; } = new EventId(763);

		/// <summary>
		/// Host is down during I/O operation, usually meaning a share isn't accessible.
		/// </summary>
		public static EventId Systemic_HostDownIOException { get; } = new EventId(764);

		/// <summary>
		/// Microsoft SignTool cannot reach specified timestamp server 
		/// 
		/// Signing performs a looping retry using different servers and throws when all retries attempts have been exhausted.
		/// </summary>
		public static EventId Systemic_SignToolTimeStampServer { get; } = new EventId(765);

		/// <summary>
		/// Microsoft SignTool generic error
		/// 
		/// Usually preceded by another type of error.
		/// </summary>
		public static EventId Systemic_SignTool { get; } = new EventId(766);

		/// <summary>
		/// Missing file list error
		/// 
		/// Usually due to a volume that is not mounted or some of other IO error
		/// </summary>
		public static EventId Systemic_MissingFileList { get; } = new EventId(767);

		/// <summary>
		/// Out of disk space
		/// </summary>
		public static EventId Systemic_OutOfDiskSpace { get; } = new EventId(768);

		/// <summary>
		/// Error moving files to cache
		/// </summary>
		public static EventId Systemic_ManagedWorkspace { get; } = new EventId(769);

		/// <summary>
		/// Generic timeout message emmitted by UBA
		/// </summary>
		public static EventId Systemic_UbaCacheResponseTimeout { get; } = new EventId(770);

		/// <summary>
		/// Docker messages that are unactionable and can be ignored
		/// </summary>
		public static EventId Systemic_Docker { get; } = new EventId(771);

		/// <summary>
		/// Maximum code for systemic events. Add new events in the 700-799 range.
		/// </summary>
		public static EventId Systemic_Max { get; } = new EventId(799);

		/// <summary>
		/// Generic Sanitizer Error
		/// </summary>
		public static EventId Sanitizer { get; } = new EventId(800);

		/// <summary>
		/// Thread Sanitizer Error
		/// </summary>
		public static EventId Sanitizer_Thread { get; } = new EventId(801);

		/// <summary>
		/// Address Sanitizer Error
		/// </summary>
		public static EventId Sanitizer_Address { get; } = new EventId(802);

		/// <summary>
		/// RaceDetector Sanitizer Error
		/// </summary>
		public static EventId Sanitizer_RaceDetector { get; } = new EventId(803);

		/// <summary>
		/// World Leak Error
		/// </summary>
		public static EventId WorldLeak { get; } = new EventId(900);

		/// <summary>
		/// Horde error codes
		/// </summary>
		public static EventId Horde { get; } = new EventId(1000);

		/// <summary>
		/// Invalid preflight change
		/// </summary>
		public static EventId Horde_InvalidPreflight { get; } = new EventId(1001);

		/// <summary>
		/// Information about blobs being read
		/// </summary>
		public static EventId Horde_BlobRead { get; } = new EventId(1002);

		/// <summary>
		/// Event used to supporess the logging from the Horde build health service, regardless of log level
		/// </summary>
		public static EventId Horde_BuildHealth_Ignore { get; } = new EventId(1003);

		/// <summary>
		/// Maximum code for Horde events. Add new events in the 1000-1099 range.
		/// </summary>
		public static EventId Horde_Max { get; } = new EventId(1099);

		/// <summary>
		/// An event while running zen cli
		/// </summary>
		public static EventId ZenCli { get; } = new EventId(1100);

		/// <summary>
		/// Strongly typed zen cli event
		/// </summary>
		public static EventId ZenCli_Event { get; } = new EventId(1101);

		/// <summary>
		/// Maximum code for zencli events. Add new events in the 1100-1200 range.
		/// </summary>
		public static EventId ZenCli_Max { get; } = new EventId(1199);
	}
}
