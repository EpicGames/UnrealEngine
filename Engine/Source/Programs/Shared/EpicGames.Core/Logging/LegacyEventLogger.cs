// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Channels;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper around a custom logger interface which flushes the event parser when switching between legacy
	/// and native structured logging. Also support threading the log messages to minimize performance overhead
	/// </summary>
	sealed class LegacyEventLogger : ILogger, IDisposable
	{
		private volatile ILogger _inner;
		private readonly LogEventParser _parser;
		private ThreadingState? _state;

		public LogEventParser Parser => _parser;

		public LegacyEventLogger(ILogger inner)
		{
			_inner = inner;
			_parser = new LogEventParser(inner);
		}

		public void Dispose()
		{
			// Drain and stop the worker if enabled, then dispose parser.
			DisableThreaded();
			_parser.Dispose();
		}

		/// <summary>
		/// Set inner logger to transfer log calls to
		/// </summary>
		/// <param name="inner"></param>
		public void SetInnerLogger(ILogger inner)
		{
			lock (_parser)
			{
				_parser.Flush();
				_inner = inner;
				_parser.Logger = inner;
			}
		}

		/// <summary>
		/// Enable background logging (idempotent).
		/// </summary>
		public void EnableThreaded()
		{
			ThreadingState created = new();
			Interlocked.CompareExchange(ref _state, created, null);
		}

		/// <summary>
		/// Disable background logging and drain queued items (idempotent).
		/// </summary>
		public void DisableThreaded()
		{
			ThreadingState? state = Interlocked.Exchange(ref _state, null);
			state?.Stop(); // completes channel, drains, joins worker thread
		}

		// ---- ILogger ----
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _inner.BeginScope(state);

		public bool IsEnabled(LogLevel logLevel) => _inner.IsEnabled(logLevel);

		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			void DoLog()
			{
				ILogger logger = _inner;
				lock (_parser)
				{
					_parser.Flush();
				}
				logger.Log(logLevel, eventId, state, exception, formatter);
			}

			ThreadingState? s = Volatile.Read(ref _state);
			if (s == null)
			{
				DoLog();
				return;
			}

			if (logLevel < LogLevel.Error)
			{
				// Non-blocking enqueue; unbounded channel TryWrite is always true.
				s.Writer.TryWrite(DoLog);
				return;
			}

			// Error+ logs: preserve synchronous semantics.
			using ManualResetEventSlim done = new(false, spinCount: 1);
			s.Writer.TryWrite(() =>
			{
				try
				{
					DoLog();
				}
				finally
				{
					done.Set();
				}
			});
			done.Wait();
		}

		private sealed class ThreadingState
		{
			public readonly ChannelWriter<Action> Writer;
			private readonly ChannelReader<Action> _reader;
			private readonly Thread _thread;

			public ThreadingState()
			{
				// Single reader = true => fewer fences and better throughput.
				Channel<Action> ch = Channel.CreateUnbounded<Action>(new UnboundedChannelOptions
				{
					SingleReader = true,
					SingleWriter = false,
					AllowSynchronousContinuations = false
				});
				Writer = ch.Writer;
				_reader = ch.Reader;

				_thread = new Thread(static o =>
				{
					ThreadingState self = (ThreadingState)o!;
					ChannelReader<Action> reader = self._reader;

					while (reader.WaitToReadAsync().AsTask().GetAwaiter().GetResult())
					{
						while (reader.TryRead(out Action? work))
						{
							try
							{
								work();
							}
							catch
							{
								// Swallow exceptions to keep the logging loop healthy.
								// Consider publishing to a fallback sink if needed.
							}
						}
					}
				})
				{
					IsBackground = true,
					Name = "LegacyEventLogger",
				};
				_thread.Start(this);
			}

			public void Stop()
			{
				// Complete the writer so the reader drains and then exits cleanly.
				Writer.TryComplete();
				_thread.Join();
			}
		}
	}
}