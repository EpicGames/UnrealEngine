// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using JetBrains.Annotations;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Log Event Type
	/// </summary>
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum LogEventType
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// The log event is a fatal error
		/// </summary>
		Fatal = LogLevel.Critical,

		/// <summary>
		/// The log event is an error
		/// </summary>
		Error = LogLevel.Error,

		/// <summary>
		/// The log event is a warning
		/// </summary>
		Warning = LogLevel.Warning,

		/// <summary>
		/// Output the log event to the console
		/// </summary>
		Console = LogLevel.Information,

		/// <summary>
		/// Output the event to the on-disk log
		/// </summary>
		Log = LogLevel.Debug,

		/// <summary>
		/// The log event should only be displayed if verbose logging is enabled
		/// </summary>
		Verbose = LogLevel.Trace,

		/// <summary>
		/// The log event should only be displayed if very verbose logging is enabled
		/// </summary>
#pragma warning disable CA1069 // Enums values should not be duplicated
		VeryVerbose = LogLevel.Trace
#pragma warning restore CA1069 // Enums values should not be duplicated
	}

	/// <summary>
	/// Options for formatting messages
	/// </summary>
	[Flags]
	public enum LogFormatOptions
	{
		/// <summary>
		/// Format normally
		/// </summary>
		None = 0,

		/// <summary>
		/// Never write a severity prefix. Useful for pre-formatted messages that need to be in a particular format for, eg. the Visual Studio output window
		/// </summary>
		NoSeverityPrefix = 1,

		/// <summary>
		/// Do not output text to the console
		/// </summary>
		NoConsoleOutput = 2,
	}

	/// <summary>
	/// UAT/UBT Custom log system.
	/// 
	/// This lets you use any TraceListeners you want, but you should only call the static 
	/// methods below, not call Trace.XXX directly, as the static methods
	/// This allows the system to enforce the formatting and filtering conventions we desire.
	///
	/// For posterity, we cannot use the Trace or TraceSource class directly because of our special log requirements:
	///   1. We possibly capture the method name of the logging event. This cannot be done as a macro, so must be done at the top level so we know how many layers of the stack to peel off to get the real function.
	///   2. We have a verbose filter we would like to apply to all logs without having to have each listener filter individually, which would require our string formatting code to run every time.
	///   3. We possibly want to ensure severity prefixes are logged, but Trace.WriteXXX does not allow any severity info to be passed down.
	/// </summary>
	public static class Log
	{
		/// <summary>
		/// Singleton instance of the default output logger
		/// </summary>
		private static readonly DefaultLogger s_defaultLogger = new();

		/// <summary>
		/// Logger instance which parses events and forwards them to the main logger.
		/// </summary>
		private static readonly LegacyEventLogger s_legacyLogger = new(s_defaultLogger);

		/// <summary>
		/// Accessor for the global event parser from legacy events
		/// </summary>
		public static LogEventParser EventParser => s_legacyLogger.Parser;

		/// <summary>
		/// Logger instance
		/// </summary>
		public static ILogger Logger => s_legacyLogger;

		/// <summary>
		/// When true, verbose logging is enabled.
		/// </summary>
		public static LogEventType OutputLevel
		{
			get => (LogEventType)s_defaultLogger.OutputLevel;
			set => s_defaultLogger.OutputLevel = (LogLevel)value;
		}

		/// <summary>
		/// Whether to include timestamps on each line of log output
		/// </summary>
		public static bool IncludeTimestamps
		{
			get => s_defaultLogger.IncludeTimestamps;
			set => s_defaultLogger.IncludeTimestamps = value;
		}

		/// <summary>
		/// When true, warnings and errors will have a WARNING: or ERROR: prefix, respectively.
		/// </summary>
		public static bool IncludeSeverityPrefix { get; set; } = true;

		/// <summary>
		/// When true, warnings and errors will have a prefix suitable for display by MSBuild (avoiding error messages showing as (EXEC : Error : ")
		/// </summary>
		public static bool IncludeProgramNameWithSeverityPrefix { get; set; }

		/// <summary>
		/// When true, will detect warnings and errors and set the console output color to yellow and red.
		/// </summary>
		public static bool ColorConsoleOutput
		{
			get => s_defaultLogger.ColorConsoleOutput;
			set => s_defaultLogger.ColorConsoleOutput = value;
		}

		/// <summary>
		/// When true, a timestamp will be written to the log file when the first listener is added
		/// </summary>
		public static bool IncludeStartingTimestamp
		{
			get => s_defaultLogger.IncludeStartingTimestamp;
			set => s_defaultLogger.IncludeStartingTimestamp = value;
		}

		/// <summary>
		/// When true, create a backup of any text log file that would be overwritten by a new log
		/// Log.txt will be backed up with its UTC creation time in the name e.g.
		/// Log-backup-2021.10.29-19.53.17.txt
		/// </summary>
		public static bool BackupLogFiles { get; set; } = true;

		/// <summary>
		/// The number of backups to be preserved - when there are more than this, the oldest backups will be deleted.
		/// Backups will not be deleted if BackupLogFiles is false.
		/// </summary>
		public static int LogFileBackupCount { get; set; } = 10;

		/// <summary>
		/// Path to the log file being written to. May be null.
		/// </summary>
		public static FileReference? OutputFile => s_defaultLogger?.OutputFile;

		/// <summary>
		/// A collection of strings that have been already written once
		/// </summary>
		private static readonly ConcurrentDictionary<string, bool> s_writeOnceSet = new();

		/// <summary>
		/// Overrides the logger used for formatting output, after event parsing
		/// </summary>
		/// <param name="logger"></param>
		public static void SetInnerLogger(ILogger logger)
		{
			s_legacyLogger.SetInnerLogger(logger);
		}

		/// <summary>
		/// Flush the current log output
		/// </summary>
		/// <returns></returns>
		public static async Task FlushAsync()
		{
			await s_defaultLogger.FlushAsync();
		}

		/// <summary>
		/// Enable threaded logging. This can be used for performance reasons to prevent stalls when lots of logging is made from multiple threads
		/// </summary>
		public static void EnableThreaded()
		{
			s_legacyLogger.EnableThreaded();
		}

		/// <summary>
		/// Disable threaded logging. will flush queued log entries and get back to synchronous logging
		/// </summary>
		public static void DisableThreaded()
		{
			s_legacyLogger.DisableThreaded();
		}

		/// <summary>
		/// Backup an existing log file if it already exists at the output path
		/// </summary>
		/// <param name="outputFile">The file to back up</param>
		public static void BackupLogFile(FileReference outputFile)
		{
			if (!BackupLogFiles || !FileReference.Exists(outputFile))
			{
				return;
			}

			// before creating a new backup, cap the number of existing files
			string filenameWithoutExtension = outputFile.GetFileNameWithoutExtension();
			string extension = outputFile.GetExtension();

			Regex backupForm =
				new(filenameWithoutExtension + @"-backup-\d\d\d\d\.\d\d\.\d\d-\d\d\.\d\d\.\d\d" + extension);

			foreach (FileReference oldBackup in DirectoryReference
				.EnumerateFiles(outputFile.Directory)
				// find files that match the way that we name backup files
				.Where(x => backupForm.IsMatch(x.GetFileName()))
				// sort them from newest to oldest
				.OrderByDescending(x => x.GetFileName())
				// skip the newest ones that are to be kept; -1 because we're about to create another backup.
				.Skip(LogFileBackupCount - 1))
			{
				Logger.LogDebug("Deleting old log file: {File}", oldBackup);
				FileReference.Delete(oldBackup);
			}

			// Ensure that the backup gets a unique name, in the extremely unlikely case that UBT was run twice during
			// the same second.
			DateTime fileTime = File.GetCreationTimeUtc(outputFile.FullName);

			FileReference backupFile;
			for (; ; )
			{
				string timestamp = $"{fileTime:yyyy.MM.dd-HH.mm.ss}";
				backupFile = FileReference.Combine(outputFile.Directory,
					$"{filenameWithoutExtension}-backup-{timestamp}{extension}");
				if (!FileReference.Exists(backupFile))
				{
					break;
				}

				fileTime = fileTime.AddSeconds(1);
			}

			FileReference.Move(outputFile, backupFile);
		}

		/// <summary>
		/// Adds a trace listener that writes to a log file.
		/// If a StartupTraceListener was in use, this function will copy its captured data to the log file(s)
		/// and remove the startup listener from the list of registered listeners.
		/// </summary>
		/// <param name="name">Identifier for the writer</param>
		/// <param name="outputFile">The file to write to</param>
		public static void AddFileWriter(string name, FileReference outputFile)
		{
			s_defaultLogger.AddFileWriter(name, outputFile);
		}

		/// <summary>
		/// Adds a trace listener that writes to a log file.
		/// If a StartupTraceListener was in use, this function will copy its captured data to the log file(s)
		/// and remove the startup listener from the list of registered listeners.
		/// </summary>
		/// <param name="name">Identifier for the writer</param>
		/// <param name="outputFile">The file to write to</param>
		public static void AddFileWriterWithoutBackup(string name, FileReference outputFile)
		{
			s_defaultLogger.AddFileWriterWithoutBackup(name, outputFile);
		}

		/// <summary>
		/// Adds a <see cref="StartupTraceListener"/> to the collection in a safe manner.
		/// </summary>
		public static void AddStartupTraceListener()
		{
			s_defaultLogger.AddStartupTraceListener();
		}

		/// <summary>
		/// Removes a <see cref="StartupTraceListener"/> from the collection in a safe manner.
		/// </summary>
		public static void RemoveStartupTraceListener()
		{
			s_defaultLogger.RemoveStartupTraceListener();
		}

		/// <summary>
		/// Determines if a TextWriterTraceListener has been added to the list of trace listeners
		/// </summary>
		/// <returns>True if a TextWriterTraceListener has been added</returns>
		public static bool HasFileWriter()
		{
			return s_defaultLogger.HasFileWriter();
		}

		/// <summary>
		/// Converts a LogEventType into a log prefix. Only used when bLogSeverity is true.
		/// </summary>
		/// <param name="severity"></param>
		/// <returns></returns>
		private static string GetSeverityPrefix(LogEventType severity)
		{
			switch (severity)
			{
				case LogEventType.Fatal:
					return "FATAL ERROR: ";
				case LogEventType.Error:
					return "ERROR: ";
				case LogEventType.Warning:
					return "WARNING: ";
				case LogEventType.Console:
					return "";
				case LogEventType.Verbose:
					return "VERBOSE: ";
				default:
					return "";
			}
		}

		/// <summary>
		/// Writes a formatted message to the console. All other functions should boil down to calling this method.
		/// </summary>
		/// <param name="bWriteOnce">If true, this message will be written only once</param>
		/// <param name="verbosity">Message verbosity level. We only meaningfully use values up to Verbose</param>
		/// <param name="formatOptions">Options for formatting messages</param>
		/// <param name="format">Message format string.</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		private static void WriteLinePrivate(bool bWriteOnce, LogEventType verbosity, LogFormatOptions formatOptions, string format, params object?[] args)
		{
			if (Logger.IsEnabled((LogLevel)verbosity))
			{
				StringBuilder message = new();

				// Get the severity prefix for this message
				if (IncludeSeverityPrefix && ((formatOptions & LogFormatOptions.NoSeverityPrefix) == 0))
				{
					message.Append(GetSeverityPrefix(verbosity));
					if (message.Length > 0 && IncludeProgramNameWithSeverityPrefix)
					{
						// Include the executable name when running inside MSBuild. If unspecified, MSBuild re-formats them with an "EXEC :" prefix.
						message.Insert(0, $"{Path.GetFileNameWithoutExtension(Assembly.GetEntryAssembly()!.Location)}: ");
					}
				}

				// Append the formatted string
				int indentLen = message.Length;
				if (args.Length == 0)
				{
					message.Append(format);
				}
				else
				{
					message.AppendFormat(format, args);
				}

				// Replace any Windows \r\n sequences with \n
				message.Replace("\r\n", "\n");

				// Remove any trailing whitespace
				int trimLen = message.Length;
				while (trimLen > 0 && " \t\r\n".Contains(message[trimLen - 1], StringComparison.Ordinal))
				{
					trimLen--;
				}
				message.Remove(trimLen, message.Length - trimLen);

				// Update the indent length to include any whitespace at the start of the message
				while (indentLen < message.Length && message[indentLen] == ' ')
				{
					indentLen++;
				}

				// If there are multiple lines, insert a prefix at the start of each one
				for (int idx = 0; idx < message.Length; idx++)
				{
					if (message[idx] == '\n')
					{
						message.Insert(idx + 1, " ", indentLen);
						idx += indentLen;
					}
				}

				// if we want this message only written one time, check if it was already written out
				if (bWriteOnce && !s_writeOnceSet.TryAdd(message.ToString(), true))
				{
					return;
				}

				// Forward it on to the internal logger
				if (verbosity < LogEventType.Console)
				{
					Logger.Log((LogLevel)verbosity, "{Message}", message.ToString());
				}
				else
				{
					lock (EventParser)
					{
						int baseIdx = 0;
						for (int idx = 0; idx < message.Length; idx++)
						{
							if (message[idx] == '\n')
							{
								EventParser.WriteLine(message.ToString(baseIdx, idx - baseIdx));
								baseIdx = idx + 1;
							}
						}
						EventParser.WriteLine(message.ToString(baseIdx, message.Length - baseIdx));
					}
				}
			}
		}

		/// <summary>
		/// Similar to Trace.WriteLineIf
		/// </summary>
		/// <param name="condition"></param>
		/// <param name="verbosity"></param>
		/// <param name="format"></param>
		/// <param name="args"></param>
		[StringFormatMethod("Format")]
		public static void WriteLineIf(bool condition, LogEventType verbosity, string format, params object?[] args)
		{
			if (condition)
			{
				WriteLinePrivate(false, verbosity, LogFormatOptions.None, format, args);
			}
		}

		/// <summary>
		/// Similar to Trace.WriteLine
		/// </summary>
		/// <param name="verbosity"></param>
		/// <param name="format"></param>
		/// <param name="args"></param>
		[StringFormatMethod("Format")]
		public static void WriteLine(LogEventType verbosity, string format, params object?[] args)
		{
			WriteLinePrivate(false, verbosity, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Similar to Trace.WriteLine
		/// </summary>
		/// <param name="verbosity"></param>
		/// <param name="formatOptions"></param>
		/// <param name="format"></param>
		/// <param name="args"></param>
		[StringFormatMethod("Format")]
		public static void WriteLine(LogEventType verbosity, LogFormatOptions formatOptions, string format, params object?[] args)
		{
			WriteLinePrivate(false, verbosity, formatOptions, format, args);
		}

		/// <summary>
		/// Formats an exception for display in the log. The exception message is shown as an error, and the stack trace is included in the log.
		/// </summary>
		/// <param name="ex">The exception to display</param>
		/// <param name="logFileName">The log filename to display, if any</param>
		public static void WriteException(Exception ex, FileReference? logFileName)
		{
			string logSuffix = (logFileName == null) ? "" : String.Format("\n(see {0} for full exception trace)", logFileName);
			Logger.LogDebug("==============================================================================");
			Logger.LogError(ex, "{Message}{Suffix}", ExceptionUtils.FormatException(ex), logSuffix);
			Logger.LogDebug("");
			Logger.LogDebug("{Details}", ExceptionUtils.FormatExceptionDetails(ex));
			Logger.LogDebug("==============================================================================");
		}

		/// <summary>
		/// Writes an error message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogError with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceError(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Error, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes an error message to the console, in a format suitable for Visual Studio to parse.
		/// </summary>
		/// <param name="file">The file containing the error</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceErrorTask(FileReference file, string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Error, LogFormatOptions.NoSeverityPrefix, "{0}: error: {1}", file, String.Format(format, args));
		}

		/// <summary>
		/// Writes an error message to the console, in a format suitable for Visual Studio to parse.
		/// </summary>
		/// <param name="file">The file containing the error</param>
		/// <param name="line">Line number of the error</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceErrorTask(FileReference file, int line, string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Error, LogFormatOptions.NoSeverityPrefix, "{0}({1}): error: {2}", file, line, String.Format(format, args));
		}

		/// <summary>
		/// Writes a verbose message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceVerbose(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Verbose, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogInformation with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceInformation(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Console, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a warning message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogWarning with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceWarning(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Warning, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a warning message to the console, in a format suitable for Visual Studio to parse.
		/// </summary>
		/// <param name="file">The file containing the warning</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceWarningTask(FileReference file, string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Warning, LogFormatOptions.NoSeverityPrefix, "{0}: warning: {1}", file, String.Format(format, args));
		}

		/// <summary>
		/// Writes a warning message to the console, in a format suitable for Visual Studio to parse.
		/// </summary>
		/// <param name="file">The file containing the warning</param>
		/// <param name="line">Line number of the warning</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceWarningTask(FileReference file, int line, string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Warning, LogFormatOptions.NoSeverityPrefix, "{0}({1}): warning: {2}", file, line, String.Format(format, args));
		}

		/// <summary>
		/// Writes a message to the console.
		/// </summary>
		/// <param name="file">The file containing the message</param>
		/// <param name="line">Line number of the message</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		public static void TraceConsoleTask(FileReference file, int line, string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Console, LogFormatOptions.NoSeverityPrefix, "{0}({1}): {2}", file, line, String.Format(format, args));
		}

		/// <summary>
		/// Writes a very verbose message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogTrace with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceVeryVerbose(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.VeryVerbose, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a message to the log only.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void TraceLog(string format, params object?[] args)
		{
			WriteLinePrivate(false, LogEventType.Log, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Similar to Trace.WriteLine
		/// </summary>
		/// <param name="verbosity"></param>
		/// <param name="format"></param>
		/// <param name="args"></param>
		[StringFormatMethod("Format")]
		public static void WriteLineOnce(LogEventType verbosity, string format, params object?[] args)
		{
			WriteLinePrivate(true, verbosity, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Similar to Trace.WriteLine
		/// </summary>
		/// <param name="verbosity"></param>
		/// <param name="options"></param>
		/// <param name="format"></param>
		/// <param name="args"></param>
		[StringFormatMethod("Format")]
		public static void WriteLineOnce(LogEventType verbosity, LogFormatOptions options, string format, params object?[] args)
		{
			WriteLinePrivate(true, verbosity, options, format, args);
		}

		/// <summary>
		/// Writes an error message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceErrorOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Error, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a verbose message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		public static void TraceVerboseOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Verbose, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceInformationOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Console, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a warning message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceWarningOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Warning, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a warning message to the console.
		/// </summary>
		/// <param name="file">The file containing the error</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceWarningOnce(FileReference file, string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Warning, LogFormatOptions.NoSeverityPrefix, "{0}: warning: {1}", file, String.Format(format, args));
		}

		/// <summary>
		/// Writes a warning message to the console.
		/// </summary>
		/// <param name="file">The file containing the error</param>
		/// <param name="line">Line number of the error</param>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[StringFormatMethod("Format")]
		public static void TraceWarningOnce(FileReference file, int line, string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Warning, LogFormatOptions.NoSeverityPrefix, "{0}({1}): warning: {2}", file, line, String.Format(format, args));
		}

		/// <summary>
		/// Writes a very verbose message to the console.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		public static void TraceVeryVerboseOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.VeryVerbose, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Writes a message to the log only.
		/// </summary>
		/// <param name="format">Message format string</param>
		/// <param name="args">Optional arguments</param>
		[Conditional("TRACE")]
		[StringFormatMethod("Format")]
		public static void TraceLogOnce(string format, params object?[] args)
		{
			WriteLinePrivate(true, LogEventType.Log, LogFormatOptions.None, format, args);
		}

		/// <summary>
		/// Enter a scope with the given status message. The message will be written to the console without a newline, allowing it to be updated through subsequent calls to UpdateStatus().
		/// The message will be written to the log immediately. If another line is written while in a status scope, the initial status message is flushed to the console first.
		/// </summary>
		/// <param name="message">The status message</param>
		[Conditional("TRACE")]
		public static void PushStatus(string message)
		{
			s_defaultLogger.PushStatus(message);
		}

		/// <summary>
		/// Updates the current status message. This will overwrite the previous status line.
		/// </summary>
		/// <param name="message">The status message</param>
		[Conditional("TRACE")]
		public static void UpdateStatus(string message)
		{
			s_defaultLogger.UpdateStatus(message);
		}

		/// <summary>
		/// Updates the Pops the top status message from the stack. The mess
		/// </summary>
		[Conditional("TRACE")]
		public static void PopStatus()
		{
			s_defaultLogger.PopStatus();
		}
	}

	/// <summary>
	/// Base implementation for loggers which handles creation of LogEvents from log commands
	/// </summary>
	public sealed class LoggerScopeCollection
	{
		class Scope : IDisposable
		{
			readonly LoggerScopeCollection _outer;
			readonly IEnumerable<KeyValuePair<string, object?>>? _properties;

			public IEnumerable<KeyValuePair<string, object?>>? Properties => _properties;

			public Scope(LoggerScopeCollection outer, object? value)
			{
				_outer = outer;
				_properties = value switch
				{
					IEnumerable<KeyValuePair<string, object?>> props => props,
					_ => null
				};

				_outer._scopes.Add(this);
			}

			public void Dispose()
				=> _outer._scopes.Remove(this);
		}

		readonly List<Scope> _scopes = [];

		/// <summary>
		/// Tests whether the collection is empty
		/// </summary>
		public bool IsEmpty()
			=> _scopes.Count == 0;

		/// <summary>
		/// Creates a scope with the given state
		/// </summary>
		public IDisposable BeginScope<TState>(TState state)
			=> new Scope(this, state);

		/// <summary>
		/// Enumerates all current properties
		/// </summary>
		public IEnumerable<KeyValuePair<string, object?>> GetProperties()
		{
			foreach (Scope scope in _scopes)
			{
				if (scope.Properties != null)
				{
					foreach (KeyValuePair<string, object?> property in scope.Properties)
					{
						yield return property;
					}
				}
			}
		}
	}

	/// <summary>
	/// Logger which captures the output for rendering later
	/// </summary>
	public class CaptureLogger : ILogger
	{
		readonly LoggerScopeCollection _scopeCollection = new();

		/// <summary>
		/// List of captured events
		/// </summary>
		public List<LogEvent> Events { get; } = [];

		/// <summary>
		/// Renders the captured events as a single string
		/// </summary>
		/// <returns>Rendered log text</returns>
		public string Render() => Render("\n");

		/// <summary>
		/// Renders the captured events as a single string
		/// </summary>
		/// <returns>Rendered log text</returns>
		public string Render(string newLine) => String.Join(newLine, RenderLines());

		/// <summary>
		/// Renders all the captured events
		/// </summary>
		/// <returns>List of rendered log lines</returns>
		public List<string> RenderLines() => Events.ConvertAll(x => x.ToString());

		/// <summary>
		/// Renders all the captured events to another logger
		/// </summary>
		/// <param name="logger"></param>
		public void RenderTo(ILogger logger) => Events.ForEach(x => logger.Log(x.Level, x.Id, x, null, (s, e) => s.ToString()));

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _scopeCollection.BeginScope(state);

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => true;

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			LogEvent logEvent = LogEvent.FromState(logLevel, eventId, state, exception, formatter);
			logEvent.AddProperties(_scopeCollection.GetProperties());
			Events.Add(logEvent);
		}
	}

	/// <summary>
	/// Default log output device
	/// </summary>
	class DefaultLogger : ILogger, IDisposable
	{
		/// <summary>
		/// Temporary status message displayed on the console.
		/// </summary>
		[DebuggerDisplay("{_headingText}")]
		class StatusMessage
		{
			/// <summary>
			/// The heading for this status message.
			/// </summary>
			public string _headingText;

			/// <summary>
			/// The current status text.
			/// </summary>
			public string _currentText;

			/// <summary>
			/// Whether the heading has been written to the console. Before the first time that lines are output to the log in the midst of a status scope, the heading will be written on a line of its own first.
			/// </summary>
			public bool _hasFlushedHeadingText;

			/// <summary>
			/// Constructor
			/// </summary>
			public StatusMessage(string headingText, string currentText)
			{
				_headingText = headingText;
				_currentText = currentText;
			}
		}

		private readonly LoggerScopeCollection _scopeCollection = new();
		private readonly Lock _syncObject = new();

		/// <summary>
		/// Minimum level for outputting messages
		/// </summary>
		public LogLevel OutputLevel
		{
			get; set;
		}

		/// <summary>
		/// Whether to include timestamps on each line of log output
		/// </summary>
		public bool IncludeTimestamps
		{
			get; set;
		}

		/// <summary>
		/// When true, will detect warnings and errors and set the console output color to yellow and red.
		/// </summary>
		public bool ColorConsoleOutput
		{
			get; set;
		}

		/// <summary>
		/// Whether to write JSON to stdout
		/// </summary>
		public bool WriteJsonToStdOut
		{
			get; set;
		}

		/// <summary>
		/// When true, a timestamp will be written to the log file when the first listener is added
		/// </summary>
		public bool IncludeStartingTimestamp
		{
			get; set;
		}
		private bool _includeStartingTimestampWritten = false;

		/// <summary>
		/// Path to the log file being written to. May be null.
		/// </summary>
		public FileReference? OutputFile
		{
			get; private set;
		}

		/// <summary>
		/// Whether console output is redirected. This prevents writing status updates that rely on moving the cursor.
		/// </summary>
		private static bool AllowStatusUpdates { get; set; } = !Console.IsOutputRedirected;

		/// <summary>
		/// When configured, this tracks time since initialization to prepend a timestamp to each log.
		/// </summary>
		private readonly Stopwatch _timer = Stopwatch.StartNew();

		/// <summary>
		/// Stack of status scope information.
		/// </summary>
		private readonly Stack<StatusMessage> _statusMessageStack = new();

		/// <summary>
		/// The currently visible status text
		/// </summary>
		private string _statusText = "";

		/// <summary>
		/// Parser for transforming legacy log output into structured events
		/// </summary>
		public LogEventParser EventParser { get; }

		/// <summary>
		/// Last time a status message was pushed to the stack
		/// </summary>
		private readonly Stopwatch _statusTimer = new();

		/// <summary>
		/// Background task for writing to files
		/// </summary>
		private Task _writeTask;

		/// <summary>
		/// Channel for new log events
		/// </summary>
		private Channel<JsonLogEvent> _eventChannel = Channel.CreateUnbounded<JsonLogEvent>();

		/// <summary>
		/// Output streams for structured log data
		/// </summary>
		private ImmutableArray<FileStream> _jsonStreams = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultLogger()
		{
			OutputLevel = LogLevel.Debug;
			ColorConsoleOutput = true;
			IncludeStartingTimestamp = true;
			EventParser = new LogEventParser(this);

			bool bJsonStdOut = Environment.CommandLine.Contains("JsonStdOut", StringComparison.Ordinal);
			string? envVar = Environment.GetEnvironmentVariable("UE_LOG_JSON_TO_STDOUT");
			if (bJsonStdOut || envVar != null && Int32.TryParse(envVar, out int value) && value != 0)
			{
				WriteJsonToStdOut = true;
			}

			_writeTask = Task.Run(() => WriteFilesAsync());
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_eventChannel.Writer.TryComplete();
			_writeTask.Wait();
		}

		/// <summary>
		/// Flush the log queue
		/// </summary>
		/// <returns></returns>
		public async Task FlushAsync()
		{
			lock (_syncObject)
			{
				Channel<JsonLogEvent> prevEventChannel = _eventChannel;
				_eventChannel = Channel.CreateUnbounded<JsonLogEvent>();
				prevEventChannel.Writer.TryComplete();
			}

			await _writeTask;
			_writeTask = Task.Run(() => WriteFilesAsync());
		}

		/// <summary>
		/// Background task to write events to sinks
		/// </summary>
		async Task WriteFilesAsync()
		{
			byte[] newline = [(byte)'\n'];
			while (await _eventChannel.Reader.WaitToReadAsync())
			{
				IReadOnlyList<FileStream> streams = _jsonStreams;

				JsonLogEvent logEvent;
				while (_eventChannel.Reader.TryRead(out logEvent))
				{
					foreach (FileStream stream in streams)
					{
						await stream.WriteAsync(logEvent.Data);
						await stream.WriteAsync(newline);
					}
				}

				foreach (FileStream stream in streams)
				{
					await stream.FlushAsync();
				}
			}
		}

		/// <summary>
		/// Adds a trace listener that writes to a log file
		/// </summary>
		/// <param name="name">Listener name</param>
		/// <param name="outputFile">The file to write to</param>
		/// <returns>The created trace listener</returns>
		private TextWriterTraceListener AddTextWriter(string name, FileReference outputFile)
		{
			try
			{
				OutputFile = outputFile;
				DirectoryReference.CreateDirectory(outputFile.Directory);
				TextWriterTraceListener logTraceListener = new(new StreamWriter(outputFile.FullName), name);
				lock (_syncObject)
				{
					Trace.Listeners.Add(logTraceListener);
					WriteInitialTimestamp();
				}
				return logTraceListener;
			}
			catch (Exception ex)
			{
				ex.Data["LogFile"] = outputFile;
				throw;
			}
		}

		/// <summary>
		/// Adds a JSON stream to the collection
		/// </summary>
		/// <param name="outputFile">The file to write to</param>
		private void AddJsonWriter(FileReference outputFile)
		{
			try
			{
				DirectoryReference.CreateDirectory(outputFile.Directory);
				lock (_syncObject)
				{
					FileStream jsonStream = FileReference.Open(outputFile, FileMode.Create, FileAccess.Write, FileShare.Read | FileShare.Delete);
					_jsonStreams = _jsonStreams.Add(jsonStream);
				}
			}
			catch (Exception ex)
			{
				ex.Data["LogFile"] = outputFile;
				throw;
			}
		}

		/// <summary>
		/// Adds a <see cref="StartupTraceListener"/> to the collection in a safe manner.
		/// </summary>
		public void AddStartupTraceListener()
		{
			lock (_syncObject)
			{
				if (!Trace.Listeners.OfType<StartupTraceListener>().Any())
				{
					Trace.Listeners.Add(new StartupTraceListener());
					WriteInitialTimestamp();
				}
			}
		}

		/// <summary>
		/// Write a timestamp to the log, once. To be called when a new listener is added.
		/// </summary>
		private void WriteInitialTimestamp()
		{
			if (IncludeStartingTimestamp && !_includeStartingTimestampWritten)
			{
				DateTime now = DateTime.Now;
				this.LogDebug("{Message}", $"Log started at {now} ({now.ToUniversalTime():yyyy-MM-ddTHH\\:mm\\:ssZ})");
				_includeStartingTimestampWritten = true;
			}
		}

		/// <summary>
		/// Removes a <see cref="StartupTraceListener"/> from the collection in a safe manner.
		/// </summary>
		public StartupTraceListener? RemoveStartupTraceListener()
		{
			lock (_syncObject)
			{
				StartupTraceListener? startupListener = Trace.Listeners.OfType<StartupTraceListener>().SingleOrDefault();
				if (startupListener is not null)
				{
					Trace.Listeners.Remove(startupListener);
				}

				return startupListener;
			}
		}

		public void AddFileWriter(string name, FileReference outputFile)
		{
			this.LogInformation("Log file: {OutputFile}", outputFile);
			Core.Log.BackupLogFile(outputFile);
			Core.Log.BackupLogFile(outputFile.ChangeExtension(".json"));
			AddFileWriterWithoutBackup(name, outputFile);
		}

		/// <summary>
		/// Adds a trace listener that writes to a log file.
		/// If a StartupTraceListener was in use, this function will copy its captured data to the log file(s)
		/// and remove the startup listener from the list of registered listeners.
		/// </summary>
		/// <param name="name">Identifier for the writer</param>
		/// <param name="outputFile">The file to write to</param>
		public void AddFileWriterWithoutBackup(string name, FileReference outputFile)
		{
			TextWriterTraceListener textWriter = AddTextWriter(name, outputFile);
			// Note that we can't correctly copy the startup listener to the JSON writer as the output has already been formatted.
			AddJsonWriter(outputFile.ChangeExtension(".json"));

			lock (_syncObject)
			{
				StartupTraceListener? startupListener = RemoveStartupTraceListener();
				startupListener?.CopyTo(textWriter);
			}
		}

		/// <summary>
		/// Determines if a TextWriterTraceListener has been added to the list of trace listeners
		/// </summary>
		/// <returns>True if a TextWriterTraceListener has been added</returns>
		public bool HasFileWriter()
		{
			lock (_syncObject)
			{
				return Trace.Listeners.OfType<TextWriterTraceListener>().Any();
			}
		}

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
			=> _scopeCollection.BeginScope(state);

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel)
		{
			return logLevel >= OutputLevel;
		}

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			if (!IsEnabled(logLevel))
			{
				return;
			}

			string[] lines = formatter(state, exception).Split('\n');
			lock (_syncObject)
			{
				// Output to all the other trace listeners
				string timePrefix = String.Format("[{0:hh\\:mm\\:ss\\.fff}] ", _timer.Elapsed);
				foreach (TraceListener? listener in Trace.Listeners)
				{
					if (listener != null)
					{
						if (listener is DefaultTraceListener && logLevel < LogLevel.Information)
						{
							continue;
						}

						string timePrefixActual =
							IncludeTimestamps &&
							listener is not DefaultTraceListener // no timestamps when writing to the Visual Studio debug window
							? timePrefix
							: String.Empty;

						foreach (string line in lines)
						{
							string lineWithTime = timePrefixActual + line;
							listener.WriteLine(lineWithTime);
							listener.Flush();
						}
					}
				}

				Activity? activity = Activity.Current;

				JsonLogEvent jsonLogEvent;
				if (activity == null && _scopeCollection.IsEmpty())
				{
					jsonLogEvent = JsonLogEvent.FromLoggerState(logLevel, eventId, state, exception, formatter);
				}
				else
				{
					LogEvent logEvent = LogEvent.FromState(logLevel, eventId, state, exception, formatter);
					logEvent.AddProperties(_scopeCollection.GetProperties());
					if (activity != null)
					{
						logEvent.AddProperty("Activity", activity);
					}
					jsonLogEvent = new JsonLogEvent(logEvent);
				}
				_eventChannel.Writer.TryWrite(jsonLogEvent);

				// Handle the console output separately; we format things differently
				if (logLevel >= LogLevel.Information)
				{
					FlushStatusHeading();

					bool bResetConsoleColor = false;
					if (ColorConsoleOutput)
					{
						if (logLevel == LogLevel.Warning)
						{
							Console.ForegroundColor = ConsoleColor.Yellow;
							bResetConsoleColor = true;
						}
						if (logLevel >= LogLevel.Error)
						{
							Console.ForegroundColor = ConsoleColor.Red;
							bResetConsoleColor = true;
						}
					}
					try
					{
						if (WriteJsonToStdOut)
						{
							Console.WriteLine(Encoding.UTF8.GetString(jsonLogEvent.Data.Span));
						}
						else
						{
							foreach (string line in lines)
							{
								Console.WriteLine(line);
							}
						}
					}
					catch (IOException)
					{
						// Potential file access/sharing issue on std out
					}
					finally
					{
						// make sure we always put the console color back.
						if (bResetConsoleColor)
						{
							Console.ResetColor();
						}
					}

					if (_statusMessageStack.Count > 0 && AllowStatusUpdates)
					{
						SetStatusText(_statusMessageStack.Peek()._currentText);
					}
				}
			}
		}

		/// <summary>
		/// Flushes the current status text before writing out a new log line or status message
		/// </summary>
		void FlushStatusHeading()
		{
			if (_statusMessageStack.Count > 0)
			{
				StatusMessage currentStatus = _statusMessageStack.Peek();
				if (currentStatus._headingText.Length > 0 && !currentStatus._hasFlushedHeadingText && AllowStatusUpdates)
				{
					SetStatusText(currentStatus._headingText);
					Console.WriteLine();
					_statusText = "";
					currentStatus._hasFlushedHeadingText = true;
				}
				else
				{
					SetStatusText("");
				}
			}
		}

		/// <summary>
		/// Enter a scope with the given status message. The message will be written to the console without a newline, allowing it to be updated through subsequent calls to UpdateStatus().
		/// The message will be written to the log immediately. If another line is written while in a status scope, the initial status message is flushed to the console first.
		/// </summary>
		/// <param name="message">The status message</param>
		[Conditional("TRACE")]
		public void PushStatus(string message)
		{
			lock (_syncObject)
			{
				FlushStatusHeading();

				StatusMessage newStatusMessage = new(message, message);
				_statusMessageStack.Push(newStatusMessage);

				_statusTimer.Restart();

				if (message.Length > 0)
				{
					this.LogDebug("{Message}", message);
					SetStatusText(message);
				}
			}
		}

		/// <summary>
		/// Updates the current status message. This will overwrite the previous status line.
		/// </summary>
		/// <param name="message">The status message</param>
		[Conditional("TRACE")]
		public void UpdateStatus(string message)
		{
			lock (_syncObject)
			{
				StatusMessage currentStatusMessage = _statusMessageStack.Peek();
				currentStatusMessage._currentText = message;

				if (AllowStatusUpdates || _statusTimer.Elapsed.TotalSeconds > 10.0)
				{
					SetStatusText(message);
					_statusTimer.Restart();
				}
			}
		}

		/// <summary>
		/// Updates the Pops the top status message from the stack. The mess
		/// </summary>
		[Conditional("TRACE")]
		public void PopStatus()
		{
			lock (_syncObject)
			{
				StatusMessage currentStatusMessage = _statusMessageStack.Peek();
				SetStatusText(currentStatusMessage._currentText);

				if (_statusText.Length > 0)
				{
					Console.WriteLine();
					_statusText = "";
				}

				_statusMessageStack.Pop();
			}
		}

		/// <summary>
		/// Update the status text. For internal use only; does not modify the StatusMessageStack objects.
		/// </summary>
		/// <param name="newStatusText">New status text to display</param>
		private void SetStatusText(string newStatusText)
		{
			if (newStatusText.Length > 0)
			{
				newStatusText = LogIndent.Current + newStatusText;
			}

			if (_statusText != newStatusText)
			{
				int numCommonChars = 0;
				while (numCommonChars < _statusText.Length && numCommonChars < newStatusText.Length && _statusText[numCommonChars] == newStatusText[numCommonChars])
				{
					numCommonChars++;
				}

				if (!AllowStatusUpdates && numCommonChars < _statusText.Length && _statusText.Length > 0)
				{
					// Prevent writing backspace characters if the console doesn't support it
					Console.WriteLine();
					_statusText = "";
					numCommonChars = 0;
				}

				StringBuilder text = new();
				text.Append('\b', _statusText.Length - numCommonChars);
				text.Append(newStatusText, numCommonChars, newStatusText.Length - numCommonChars);
				if (newStatusText.Length < _statusText.Length)
				{
					int numChars = _statusText.Length - newStatusText.Length;
					text.Append(' ', numChars);
					text.Append('\b', numChars);
				}
				Console.Write(text.ToString());

				_statusText = newStatusText;
				_statusTimer.Restart();
			}
		}
	}

	/// <summary>
	/// Provider for default logger instances
	/// </summary>
	public sealed class DefaultLoggerProvider : ILoggerProvider
	{
		/// <inheritdoc/>
		public ILogger CreateLogger(string categoryName)
		{
			return new DefaultLogger();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
		}
	}

	/// <summary>
	/// Extension methods to support the default logger
	/// </summary>
	public static class DefaultLoggerExtensions
	{
		/// <summary>
		/// Adds a regular Epic logger to the builder
		/// </summary>
		/// <param name="builder">Logging builder</param>
		public static void AddEpicDefault(this ILoggingBuilder builder)
		{
			builder.Services.AddSingleton<ILoggerProvider, DefaultLoggerProvider>();
		}
	}
}
