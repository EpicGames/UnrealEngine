// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Implementation of <see cref="ILogCursor"/> used to parse events from a stream of lines
	/// </summary>
	public class LogBuffer : ILogCursor
	{
		int _lineNumber;
		readonly string?[] _history;
		int _historyIdx;
		int _historyCount;
		readonly List<string?> _nextLines;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="historySize">Number of previous lines to keep in the buffer</param>
		public LogBuffer(int historySize)
		{
			_lineNumber = 1;
			_history = new string?[historySize];
			_nextLines = [];
		}

		/// <summary>
		/// Whether the buffer needs more data
		/// </summary>
		public bool NeedMoreData
		{
			get;
			private set;
		}

		/// <inheritdoc/>
		public string? CurrentLine => this[0];

		/// <inheritdoc/>
		public int CurrentLineNumber => _lineNumber;

		/// <summary>
		/// Number of forward lines in the buffer
		/// </summary>
		public int Length => _nextLines.Count;

		/// <summary>
		/// Add a new line to the buffer
		/// </summary>
		/// <param name="line">The new line</param>
		public void AddLine(string? line)
		{
			_nextLines.Add(line);
			NeedMoreData = false;
		}

		/// <summary>
		/// Add a collection of lines to the buffer
		/// </summary>
		/// <param name="lines">The new lines</param>
		public void AddLines(IEnumerable<string?> lines)
		{
			_nextLines.AddRange(lines);
			NeedMoreData = false;
		}

		/// <summary>
		/// Move forward a given number of lines
		/// </summary>
		/// <param name="count">Number of lines to advance by</param>
		public void Advance(int count)
		{
			for (int idx = 0; idx < count; idx++)
			{
				MoveNext();
			}
		}

		/// <summary>
		/// Move to the next line
		/// </summary>
		public void MoveNext()
		{
			if (_nextLines.Count == 0)
			{
				throw new InvalidOperationException("Attempt to move past end of line buffer");
			}

			_historyIdx++;
			if (_historyIdx >= _history.Length)
			{
				_historyIdx = 0;
			}
			if (_historyCount < _history.Length)
			{
				_historyCount++;
			}

			_history[_historyIdx] = _nextLines[0];
			_nextLines.RemoveAt(0);

			_lineNumber++;
		}

		/// <inheritdoc/>
		public string? this[int offset]
		{
			get
			{
				if (offset >= 0)
				{
					// Add new lines to the buffer
					if (offset >= _nextLines.Count)
					{
						NeedMoreData = true;
						return null;
					}
					return _nextLines[offset];
				}
				else if (offset >= -_historyCount)
				{
					// Retrieve a line from the history buffer
					int idx = _historyIdx + 1 + offset;
					if (idx < 0)
					{
						idx += _history.Length;
					}
					return _history[idx];
				}
				else
				{
					// Invalid index
					return null;
				}
			}
		}

		/// <inheritdoc/>
		public override string? ToString()
		{
			return CurrentLine;
		}
	}
}
