// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Static read-only utf8 strings for parsing log events
	/// </summary>
	public static class LogEventPropertyName
	{
#pragma warning disable CS1591 // Missing documentation
		public static readonly Utf8String Time = new("time");
		public static readonly Utf8String Frame = new("frame");
		public static readonly Utf8String Level = new("level");
		public static readonly Utf8String Id = new("id");
		public static readonly Utf8String Line = new("line");
		public static readonly Utf8String LineCount = new("lineCount");
		public static readonly Utf8String Message = new("message");
		public static readonly Utf8String Format = new("format");
		public static readonly Utf8String Properties = new("properties");

		public static readonly Utf8String Type = new("$type");
		public static readonly Utf8String Text = new("$text");

		public static readonly Utf8String File = new("file"); // For source file / asset types
		public static readonly Utf8String Identifier = new("identifier"); // For symbols
		public static readonly Utf8String RelativePath = new("relativePath");
		public static readonly Utf8String DepotPath = new("depotPath");
		public static readonly Utf8String Target = new("target"); // For hyperlinks

		public static readonly Utf8String Exception = new("exception");
		public static readonly Utf8String Trace = new("trace");
		public static readonly Utf8String InnerException = new("innerException");
		public static readonly Utf8String InnerExceptions = new("innerExceptions");
#pragma warning restore CS1591 // Missing documentation
	}

	/// <summary>
	/// Epic representation of a log event. Can be serialized to/from Json for the Horde dashboard, and passed directly through ILogger interfaces.
	/// </summary>
	[JsonConverter(typeof(LogEventConverter))]
	public class LogEvent : IEnumerable<KeyValuePair<string, object?>>
	{
		/// <summary>
		/// Time that the event was emitted
		/// </summary>
		public DateTime Time { get; set; }

		/// <summary>
		/// Frame that the event was emitted, or 0 when not applicable
		/// </summary>
		public ulong Frame { get; set; }

		/// <summary>
		/// The log level
		/// </summary>
		public LogLevel Level { get; set; }

		/// <summary>
		/// Unique id associated with this event. See <see cref="KnownLogEvents"/> for possible values.
		/// </summary>
		public EventId Id { get; set; }

		/// <summary>
		/// Index of the line within a multi-line message
		/// </summary>
		public int LineIndex { get; set; }

		/// <summary>
		/// Number of lines in the message
		/// </summary>
		public int LineCount { get; set; }

		/// <summary>
		/// The formatted message
		/// </summary>
		public string Message { get; set; }

		/// <summary>
		/// Message template string
		/// </summary>
		public string? Format { get; set; }

		/// <summary>
		/// Map of property name to value
		/// </summary>
		public IEnumerable<KeyValuePair<string, object?>>? Properties { get; set; }

		/// <summary>
		/// The exception value
		/// </summary>
		public LogException? Exception { get; }

		class MergedPropertyList : IEnumerable<KeyValuePair<string, object?>>
		{
			readonly HashSet<string> _names = [];
			readonly List<KeyValuePair<string, object?>> _properties = [];

			public void AddRange(IEnumerable<KeyValuePair<string, object?>> properties)
			{
				foreach (KeyValuePair<string, object?> property in properties)
				{
					if (_names.Add(property.Key))
					{
						_properties.Add(property);
					}
				}
			}

			/// <inheritdoc/>
			public IEnumerator<KeyValuePair<string, object?>> GetEnumerator()
				=> _properties.GetEnumerator();

			/// <inheritdoc/>
			IEnumerator IEnumerable.GetEnumerator()
				=> _properties.GetEnumerator();
		}

		static readonly JsonSerializerOptions s_jsonSerializerOptions = new() { PropertyNamingPolicy = JsonNamingPolicy.CamelCase, DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull };

		/// <summary>
		/// Constructor
		/// </summary>
		public LogEvent(DateTime time, LogLevel level, EventId eventId, string message, string? format, IEnumerable<KeyValuePair<string, object?>>? properties, LogException? exception)
			: this(time, level, eventId, 0, 1, message, format, properties, exception)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public LogEvent(DateTime time, LogLevel level, EventId eventId, int lineIndex, int lineCount, string message, string? format, IEnumerable<KeyValuePair<string, object?>>? properties, LogException? exception)
			: this(time, 0, level, eventId, lineIndex, lineCount, message, format, properties, exception)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public LogEvent(DateTime time, ulong frame, LogLevel level, EventId eventId, int lineIndex, int lineCount, string message, string? format, IEnumerable<KeyValuePair<string, object?>>? properties, LogException? exception)
		{
			Time = time;
			Frame = frame;
			Level = level;
			Id = eventId;
			LineIndex = lineIndex;
			LineCount = lineCount;
			Message = message;
			Format = format;
			Properties = properties;
			Exception = exception;
		}

		/// <summary>
		/// Add a new property to this event
		/// </summary>
		/// <param name="name">Name of the property to add</param>
		/// <param name="value">Value for the property</param>
		public void AddProperty(string name, object? value)
			=> AddProperties([KeyValuePair.Create(name, value)]);

		/// <summary>
		/// Add new properties to this event
		/// </summary>
		/// <param name="properties">Properties to add</param>
		public void AddProperties(IEnumerable<KeyValuePair<string, object?>> properties)
		{
			if (Properties == null)
			{
				Properties = properties;
			}
			else
			{
				MergedPropertyList? list = Properties as MergedPropertyList;
				if (list == null)
				{
					list = new MergedPropertyList();
					list.AddRange(Properties);
					Properties = list;
				}
				list.AddRange(properties);
			}
		}

		/// <summary>
		/// Gets an untyped property with the given name
		/// </summary>
		/// <param name="name"></param>
		/// <returns></returns>
		public object GetProperty(string name)
		{
			object? value;
			if (TryGetProperty(name, out value))
			{
				return value;
			}
			throw new KeyNotFoundException($"Property {name} not found");
		}

		/// <summary>
		/// Gets a property with the given name
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="name">Name of the property</param>
		/// <returns></returns>
		public T GetProperty<T>(string name) => (T)GetProperty(name);

		/// <summary>
		/// Finds a property with the given name
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <param name="value">Value for the property, on success</param>
		/// <returns>True if the property was found, false otherwise</returns>
		public bool TryGetProperty(string name, [NotNullWhen(true)] out object? value)
		{
			if (Properties != null)
			{
				foreach (KeyValuePair<string, object?> pair in Properties)
				{
					if (pair.Key.Equals(name, StringComparison.Ordinal) && pair.Value != null)
					{
						value = pair.Value;
						return true;
					}
				}
			}

			value = null;
			return false;
		}

		/// <summary>
		/// Finds a typed property with the given name
		/// </summary>
		/// <typeparam name="T">Type of the property to receive</typeparam>
		/// <param name="name">Name of the property</param>
		/// <param name="value">Value for the property, on success</param>
		/// <returns>True if the property was found, false otherwise</returns>
		public bool TryGetProperty<T>(string name, [NotNullWhen(true)] out T value)
		{
			object? untypedValue;
			if (TryGetProperty(name, out untypedValue) && untypedValue is T typedValue)
			{
				value = typedValue;
				return true;
			}
			else
			{
				value = default!;
				return false;
			}
		}

		/// <summary>
		/// Read a log event from a utf-8 encoded json byte array
		/// </summary>
		/// <param name="data"></param>
		/// <returns></returns>
		public static LogEvent Read(ReadOnlySpan<byte> data)
		{
			Utf8JsonReader reader = new(data);
			reader.Read();
			return Read(ref reader);
		}

		/// <summary>
		/// Read a log event from Json
		/// </summary>
		/// <param name="reader">The Json reader</param>
		/// <returns>New log event</returns>
#pragma warning disable CA1045 // Do not pass types by reference
		public static LogEvent Read(ref Utf8JsonReader reader)
#pragma warning restore CA1045 // Do not pass types by reference
		{
			DateTime time = new(0);
			ulong frame = 0;
			LogLevel level = LogLevel.None;
			EventId eventId = new(0);
			int line = 0;
			int lineCount = 1;
			string message = String.Empty;
			string? format = null;
			Dictionary<string, object?>? properties = null;
			LogException? exception = null;

			ReadOnlySpan<byte> propertyName;
			for (; JsonExtensions.TryReadNextPropertyName(ref reader, out propertyName); reader.Skip())
			{
				if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Time.Span))
				{
					time = reader.GetDateTime();
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Frame.Span))
				{
					frame = reader.GetUInt64();
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Level.Span))
				{
					level = Enum.Parse<LogLevel>(reader.GetString()!);
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Id.Span))
				{
					eventId = reader.GetInt32();
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Line.Span))
				{
					line = reader.GetInt32();
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.LineCount.Span))
				{
					lineCount = reader.GetInt32();
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Message.Span))
				{
					message = reader.GetString()!;
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Format.Span))
				{
					format = reader.GetString()!;
				}
				else if (Utf8StringComparer.OrdinalIgnoreCase.Equals(propertyName, LogEventPropertyName.Properties.Span))
				{
					properties = ReadProperties(ref reader);
				}
			}

			return new LogEvent(time, frame, level, eventId, line, lineCount, message, format, properties, exception);
		}

		static Dictionary<string, object?> ReadProperties(ref Utf8JsonReader reader)
		{
			Dictionary<string, object?> properties = [];

			ReadOnlySpan<byte> propertyName;
			for (; JsonExtensions.TryReadNextPropertyName(ref reader, out propertyName); reader.Skip())
			{
				string name = Encoding.UTF8.GetString(propertyName);

				object? value = ReadPropertyValue(ref reader);
				properties.Add(name, value);
			}

			return properties;
		}

		static object? ReadPropertyValue(ref Utf8JsonReader reader)
		{
			switch (reader.TokenType)
			{
				case JsonTokenType.Null:
					return null;
				case JsonTokenType.True:
					return true;
				case JsonTokenType.False:
					return false;
				case JsonTokenType.StartArray:
					return ReadArrayPropertyValue(ref reader);
				case JsonTokenType.StartObject:
					return ReadObjectPropertyValue(ref reader);
				case JsonTokenType.String:
					return reader.GetString()!;
				case JsonTokenType.Number:
					if (reader.TryGetInt32(out int intValue))
					{
						return intValue;
					}
					else if (reader.TryGetDouble(out double doubleValue))
					{
						return doubleValue;
					}
					else
					{
						return reader.GetString()!;
					}
				default:
					throw new InvalidOperationException("Unhandled property type");
			}
		}

		static List<object?> ReadArrayPropertyValue(ref Utf8JsonReader reader)
		{
			List<object?> result = [];
			while (reader.Read() && reader.TokenType != JsonTokenType.EndArray)
			{
				result.Add(ReadPropertyValue(ref reader));
			}
			return result;
		}

		static object ReadObjectPropertyValue(ref Utf8JsonReader reader)
		{
			// Read all the properties
			Dictionary<Utf8String, object?>? properties = [];
			for (; JsonExtensions.TryReadNextPropertyName(ref reader, out ReadOnlySpan<byte> propertyName); reader.Skip())
			{
				properties.Add(new Utf8String(propertyName.ToArray()), ReadPropertyValue(ref reader));
			}

			// Check if we can convert it to a LogValue
			if (properties.TryGetValue(LogEventPropertyName.Type, out object? type) && type is string typeStr && properties.TryGetValue(LogEventPropertyName.Text, out object? text) && text is string textStr)
			{
				properties.Remove(LogEventPropertyName.Type);
				properties.Remove(LogEventPropertyName.Text);

				if (properties.Count == 0)
				{
					return new LogValue(new Utf8String(typeStr), textStr);
				}
				else
				{
					return new LogValue(new Utf8String(typeStr), textStr, properties);
				}
			}

			return properties;
		}

		/// <summary>
		/// Writes a log event to Json
		/// </summary>
		/// <param name="writer"></param>
		public void Write(Utf8JsonWriter writer)
		{
			writer.WriteStartObject();
			writer.WriteString(LogEventPropertyName.Time.Span, Time.ToString("s", CultureInfo.InvariantCulture));

			if (Frame != 0)
			{
				writer.WriteNumber(LogEventPropertyName.Frame.Span, Frame);
			}

			writer.WriteString(LogEventPropertyName.Level.Span, Level.ToString());
			writer.WriteString(LogEventPropertyName.Message.Span, Message);

			if (Id.Id != 0)
			{
				writer.WriteNumber(LogEventPropertyName.Id.Span, Id.Id);
			}

			if (LineIndex > 0)
			{
				writer.WriteNumber(LogEventPropertyName.Line.Span, LineIndex);
			}

			if (LineCount > 1)
			{
				writer.WriteNumber(LogEventPropertyName.LineCount.Span, LineCount);
			}

			if (Format != null)
			{
				writer.WriteString(LogEventPropertyName.Format.Span, Format);
			}

			if (Properties != null && Properties.Any())
			{
				writer.WriteStartObject(LogEventPropertyName.Properties.Span);
				foreach ((string name, object? value) in Properties!)
				{
					if (!name.Equals(MessageTemplate.FormatPropertyName, StringComparison.Ordinal))
					{
						if (name.StartsWith('@'))
						{
							writer.WritePropertyName(name[1..]);
							JsonSerializer.Serialize(writer, value, value?.GetType() ?? typeof(object), s_jsonSerializerOptions);
						}
						else
						{
							writer.WritePropertyName(name);
							LogValueFormatter.Format(value, writer);
						}
					}
				}
				writer.WriteEndObject();
			}

			if (Exception != null)
			{
				writer.WriteStartObject(LogEventPropertyName.Exception.Span);
				WriteException(ref writer, Exception);
				writer.WriteEndObject();
			}
			writer.WriteEndObject();
		}

		/// <summary>
		/// Writes an exception to a json object
		/// </summary>
		/// <param name="writer">Writer to receive the exception data</param>
		/// <param name="exception">The exception</param>
		static void WriteException(ref Utf8JsonWriter writer, LogException exception)
		{
			writer.WriteString("message", exception.Message);
			writer.WriteString("trace", exception.Trace);

			if (exception.InnerException != null)
			{
				writer.WriteStartObject("innerException");
				WriteException(ref writer, exception.InnerException);
				writer.WriteEndObject();
			}

			if (exception.InnerExceptions != null)
			{
				writer.WriteStartArray("innerExceptions");
				for (int idx = 0; idx < 16 && idx < exception.InnerExceptions.Count; idx++) // Cap number of exceptions returned to avoid huge messages
				{
					LogException innerException = exception.InnerExceptions[idx];
					writer.WriteStartObject();
					WriteException(ref writer, innerException);
					writer.WriteEndObject();
				}
				writer.WriteEndArray();
			}
		}

		/// <summary>
		/// Create a new log event
		/// </summary>
		public static LogEvent Create(LogLevel level, string format, params object[] args)
			=> Create(level, KnownLogEvents.None, null, format, args);

		/// <summary>
		/// Create a new log event
		/// </summary>
		public static LogEvent Create(LogLevel level, EventId eventId, string format, params object[] args)
			=> Create(level, eventId, null, format, args);

		/// <summary>
		/// Create a new log event
		/// </summary>
		public static LogEvent Create(LogLevel level, EventId eventId, Exception? exception, string format, params object[] args)
		{
			Dictionary<string, object?> properties = [];
			MessageTemplate.ParsePropertyValues(format, args, properties);

			string message = MessageTemplate.Render(format, properties!);
			return new LogEvent(DateTime.UtcNow, level, eventId, message, format, properties, LogException.FromException(exception));
		}

		/// <summary>
		/// Creates a log event from an ILogger parameters
		/// </summary>
		public static LogEvent FromState<TState>(LogLevel level, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			_ = formatter;

			if (state is LogEvent logEvent)
			{
				return logEvent;
			}
			if (state is JsonLogEvent jsonLogEvent)
			{
				return Read(jsonLogEvent.Data.Span);
			}

			DateTime time = DateTime.UtcNow;

			// Try to log the event
			IEnumerable<KeyValuePair<string, object?>>? values = state as IEnumerable<KeyValuePair<string, object?>>;
			string? format = values?.FirstOrDefault(x => x.Key.Equals(MessageTemplate.FormatPropertyName, StringComparison.Ordinal)).Value?.ToString();
			string message = (format == null) ? formatter(state, exception) : MessageTemplate.Render(format, values);
			return new LogEvent(time, level, eventId, message, format, values, LogException.FromException(exception));
		}

		/// <summary>
		/// Enumerates all the properties in this object
		/// </summary>
		/// <returns>Property pairs</returns>
		public IEnumerator<KeyValuePair<string, object?>> GetEnumerator()
		{
			if (Format != null)
			{
				yield return new KeyValuePair<string, object?>(MessageTemplate.FormatPropertyName, Format.ToString());
			}

			if (Properties != null)
			{
				foreach ((string name, object? value) in Properties)
				{
					if (!name.Equals(MessageTemplate.FormatPropertyName, StringComparison.OrdinalIgnoreCase))
					{
						yield return new KeyValuePair<string, object?>(name, value?.ToString());
					}
				}
			}
		}

		/// <summary>
		/// Enumerates all the properties in this object
		/// </summary>
		/// <returns>Property pairs</returns>
		System.Collections.IEnumerator System.Collections.IEnumerable.GetEnumerator()
		{
			foreach (KeyValuePair<string, object?> pair in this)
			{
				yield return pair;
			}
		}

		/// <summary>
		/// Serialize a message template to JOSN
		/// </summary>
		public byte[] ToJsonBytes()
		{
			ArrayBufferWriter<byte> buffer = new();
			using (Utf8JsonWriter writer = new(buffer))
			{
				Write(writer);
			}
			return buffer.WrittenSpan.ToArray();
		}

		/// <summary>
		/// Serialize a message template to JOSN
		/// </summary>
		public string ToJson()
		{
			ArrayBufferWriter<byte> buffer = new();
			using (Utf8JsonWriter writer = new(buffer))
			{
				Write(writer);
			}
			return Encoding.UTF8.GetString(buffer.WrittenSpan);
		}

		/// <inheritdoc/>
		public override string ToString() => Message;
	}

	/// <summary>
	/// Information about an exception in a log event
	/// </summary>
	public sealed class LogException
	{
		/// <summary>
		/// Exception message
		/// </summary>
		public string Message { get; set; }

		/// <summary>
		/// Stack trace for the exception
		/// </summary>
		public string Trace { get; set; }

		/// <summary>
		/// Optional inner exception information
		/// </summary>
		public LogException? InnerException { get; set; }

		/// <summary>
		/// Multiple inner exceptions, in the case of an <see cref="AggregateException"/>
		/// </summary>
		public List<LogException> InnerExceptions { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		/// <param name="trace"></param>
		public LogException(string message, string trace)
		{
			Message = message;
			Trace = trace;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="exception"></param>
		[return: NotNullIfNotNull(nameof(exception))]
		public static LogException? FromException(Exception? exception)
		{
			LogException? result = null;
			if (exception != null)
			{
				result = new LogException(exception.Message, exception.StackTrace ?? String.Empty);

				if (exception.InnerException != null)
				{
					result.InnerException = FromException(exception.InnerException);
				}

				AggregateException? aggregateException = exception as AggregateException;
				if (aggregateException != null && aggregateException.InnerExceptions.Count > 0)
				{
					for (int idx = 0; idx < 16 && idx < aggregateException.InnerExceptions.Count; idx++) // Cap number of exceptions returned to avoid huge messages
					{
						LogException innerException = FromException(aggregateException.InnerExceptions[idx]);
						result.InnerExceptions.Add(innerException);
					}
				}
			}
			return result;
		}
	}

	/// <summary>
	/// Interface for a log event sink
	/// </summary>
	public interface ILogEventSink
	{
		/// <summary>
		/// Process log event 
		/// </summary>
		/// <param name="logEvent">Log event</param>
		void ProcessEvent(LogEvent logEvent);
	}

	/// <summary>
	/// Simple filtering log event sink with a callback for convenience
	/// </summary>
	public class FilteringEventSink : ILogEventSink
	{
		/// <summary>
		/// Log events received
		/// </summary>
		public IReadOnlyList<LogEvent> LogEvents => _logEvents;

		private readonly List<LogEvent> _logEvents = [];
		private readonly int[] _includeEventIds;
		private readonly Action<LogEvent>? _eventCallback;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="includeEventIds">Event IDs to include</param>
		/// <param name="eventCallback">Optional callback function for each event received</param>
		public FilteringEventSink(int[] includeEventIds, Action<LogEvent>? eventCallback = null)
		{
			_includeEventIds = includeEventIds;
			_eventCallback = eventCallback;
		}

		/// <inheritdoc/>
		public void ProcessEvent(LogEvent logEvent)
		{
			if (_includeEventIds.Any(x => x == logEvent.Id.Id))
			{
				_logEvents.Add(logEvent);
				_eventCallback?.Invoke(logEvent);
			}
		}
	}

	/// <summary>
	/// Converter for serialization of <see cref="LogEvent"/> instances to Json streams
	/// </summary>
	public class LogEventConverter : JsonConverter<LogEvent>
	{
		/// <inheritdoc/>
		public override LogEvent Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return LogEvent.Read(ref reader);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, LogEvent value, JsonSerializerOptions options)
		{
			value.Write(writer);
		}
	}
}
