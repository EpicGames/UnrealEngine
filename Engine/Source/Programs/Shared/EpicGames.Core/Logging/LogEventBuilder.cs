// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.RegularExpressions;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Stores information about a span within a line
	/// </summary>
	public class LogEventSpan
	{
		/// <summary>
		/// Starting offset within the line
		/// </summary>
		public int Offset
		{
			get;
		}

		/// <summary>
		/// Text for this span
		/// </summary>
		public string Text
		{
			get;
		}

		/// <summary>
		/// Storage for properties
		/// </summary>
		public Dictionary<string, object> Properties
		{
			get;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="offset">Starting offset within the line</param>
		/// <param name="text">The text for this span</param>
		public LogEventSpan(int offset, string text)
		{
			Offset = offset;
			Text = text;
			Properties = new Dictionary<string, object>(StringComparer.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Converts this object to a string. This determines how the span will be rendered by the default console logger, so should return the original text.
		/// </summary>
		/// <returns>Original text for this span</returns>
		public override string ToString()
		{
			return Text;
		}
	}

	/// <summary>
	/// Individual line in the log output
	/// </summary>
	public class LogEventLine
	{
		/// <summary>
		/// The raw text
		/// </summary>
		public string Text
		{
			get;
		}

		/// <summary>
		/// List of spans for markup
		/// </summary>
		public Dictionary<string, LogEventSpan> Spans
		{
			get;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Text for the line</param>
		public LogEventLine(string text)
		{
			Text = text;
			Spans = [];
		}

		/// <summary>
		/// Adds a span containing markup on the source text
		/// </summary>
		/// <param name="offset">Offset within the line</param>
		/// <param name="length">Length of the span</param>
		/// <param name="name">Name to use to identify the item in the format string</param>
		/// <returns>New span for the given range</returns>
		public LogEventSpan AddSpan(int offset, int length, string name)
		{
			LogEventSpan span = new(offset, Text.Substring(offset, length));
			Spans.Add(name, span);
			return span;
		}

		/// <summary>
		/// Adds a span containing markup for a regex match group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="name">Name to use to identify the item in the format string</param>
		public LogEventSpan AddSpan(Group group, string name)
		{
			return AddSpan(group.Index, group.Length, name);
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		public LogEventSpan AddSpan(Group group)
		{
			return AddSpan(group.Index, group.Length, group.Name);
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="name">Name to use to identify the item in the format string</param>
		public LogEventSpan? TryAddSpan(Group group, string name)
		{
			if (group.Success)
			{
				return AddSpan(group, name);
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		public LogEventSpan? TryAddSpan(Group group)
		{
			if (group.Success)
			{
				return AddSpan(group);
			}
			else
			{
				return null;
			}
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return Text;
		}
	}

	/// <summary>
	/// Allows building log events by annotating a window of lines around the current cursor position.
	/// </summary>
	public class LogEventBuilder
	{
		class LogSpan
		{
			public string _name;
			public int _offset;
			public int _length;
			public object? _value;

			public LogSpan(string name, int offset, int length, object? value)
			{
				_name = name;
				_offset = offset;
				_length = length;
				_value = value;
			}
		}

		class LogLine
		{
			public string _message;
			public string? _format;
			public Dictionary<string, object>? _properties;

			public LogLine(string message, string? format, Dictionary<string, object>? properties)
			{
				_message = message;
				_format = format;
				_properties = properties;
			}
		}

		/// <summary>
		/// The current cursor position
		/// </summary>
		public ILogCursor Current { get; private set; }

		/// <summary>
		/// The next cursor position
		/// </summary>
		public ILogCursor Next { get; private set; }

		/// <summary>
		/// Events which have been parsed so far
		/// </summary>
		List<LogLine>? _lines;

		/// <summary>
		/// Spans for the current line
		/// </summary>
		List<LogSpan>? _spans;

		/// <summary>
		/// Additional properties for this line
		/// </summary>
		Dictionary<string, object>? _properties;

		/// <summary>
		/// Starts building a log event at the current cursor position
		/// </summary>
		/// <param name="cursor">The current cursor position</param>
		/// <param name="lineCount">Number of lines to consume</param>
		public LogEventBuilder(ILogCursor cursor, int lineCount = 1)
		{
			Current = cursor;
			Next = cursor.Rebase(1);

			if (lineCount > 1)
			{
				MoveNext(lineCount - 1);
			}
		}

		/// <summary>
		/// Creates a log event from the current line
		/// </summary>
		/// <returns></returns>
		LogLine CreateLine()
		{
			int offset = 0;
			string currentLine = Current.CurrentLine!;

			string? format = null;
			if (_spans != null)
			{
				StringBuilder builder = new();
				foreach (LogSpan span in _spans)
				{
					if (span._offset >= offset)
					{
						builder.Append(currentLine, offset, span._offset - offset);
						builder.Append($"{{{span._name}}}");
						offset = span._offset + span._length;
					}
				}
				builder.Append(currentLine, offset, currentLine.Length - offset);
				format = builder.ToString();
			}

			return new LogLine(currentLine, format, _properties);
		}

		/// <summary>
		/// Adds a span containing markup on the source text
		/// </summary>
		/// <param name="name">Name to use to identify the item in the format string</param>
		/// <param name="offset">Offset within the line</param>
		/// <param name="length">Length of the span</param>
		/// <param name="value">Data of the span</param>
		/// <returns>New span for the given range</returns>
		public void Annotate(string name, int offset, int length, object? value = null)
		{
			LogSpan span = new(name, offset, length, value);

			_properties ??= new Dictionary<string, object>(StringComparer.OrdinalIgnoreCase);
			_properties.Add(name, span);

			_spans ??= [];
			for (int insertIdx = _spans.Count; ; insertIdx--)
			{
				if (insertIdx == 0 || _spans[insertIdx - 1]._offset < offset)
				{
					_spans.Insert(insertIdx, span);
					break;
				}
			}
		}

		/// <summary>
		/// Adds a span containing markup for a regex match group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="name">Name to use to identify the item in the format string</param>
		/// <param name="value">Optional value for the annotation</param>
		public void Annotate(string name, Group group, object? value = null)
		{
			Annotate(name, group.Index, group.Length, value);
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="value">Optional value for the annotation</param>
		public void Annotate(Group group, object? value = null)
		{
			Annotate(group.Name, group.Index, group.Length, value);
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="name">Name to use to identify the item in the format string</param>
		/// <param name="value">Optional value for the annotation</param>
		public bool TryAnnotate(string name, Group group, object? value = null)
		{
			if (group.Success)
			{
				Annotate(name, group, value);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Adds a span naming a regex match group, using the name of the group
		/// </summary>
		/// <param name="group">The match group</param>
		/// <param name="value">Optional value for the annotation</param>
		public bool TryAnnotate(Group group, object? value = null)
		{
			if (group.Success)
			{
				Annotate(group, value);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Adds an additional named property
		/// </summary>
		/// <param name="name">Name of the argument</param>
		/// <param name="value">Value to associate with it</param>
		public void AddProperty(string name, object value)
		{
			_properties ??= new Dictionary<string, object>(StringComparer.OrdinalIgnoreCase);
			_properties.Add(name, value);
		}

		string? GetFirstLine()
		{
			if (_lines == null || _lines.Count == 0)
			{
				return Current.CurrentLine;
			}
			else
			{
				return _lines[0]._message;
			}
		}

		/// <summary>
		/// Check if the next line is aligned or indented from the first line
		/// </summary>
		/// <returns>True if the next line is aligned with this</returns>
		public bool IsNextLineAligned() => Next.IsAligned(0, GetFirstLine());

		/// <summary>
		/// Check if the next line is indented from the first line
		/// </summary>
		/// <returns>True if the next line is aligned with this</returns>
		public bool IsNextLineHanging() => Next.IsHanging(0, GetFirstLine());

		/// <summary>
		/// Complete the current line and move to the next
		/// </summary>
		public void MoveNext()
		{
			_lines ??= [];
			_lines.Add(CreateLine());

			_spans = null;
			_properties = null;

			Current = Next;
			Next = Next.Rebase(1);
		}

		/// <summary>
		/// Advance by the given number of lines
		/// </summary>
		/// <param name="count"></param>
		public void MoveNext(int count)
		{
			for (int idx = 0; idx < count; idx++)
			{
				MoveNext();
			}
		}

		/// <summary>
		/// Returns an array of log events
		/// </summary>
		/// <returns></returns>
		public LogEvent[] ToArray(LogLevel level, EventId eventId)
		{
			DateTime time = DateTime.UtcNow;

			int numLines = _lines?.Count ?? 0;
			int numEvents = numLines;
			if (Current.CurrentLine != null)
			{
				numEvents++;
			}

			LogEvent[] events = new LogEvent[numEvents];
			for (int idx = 0; idx < numLines; idx++)
			{
				events[idx] = CreateEvent(time, level, eventId, idx, numEvents, _lines![idx]);
			}
			if (Current.CurrentLine != null)
			{
				events[numEvents - 1] = CreateEvent(time, level, eventId, numEvents - 1, numEvents, CreateLine());
			}

			return events;
		}

		static LogEvent CreateEvent(DateTime time, LogLevel level, EventId eventId, int lineIndex, int lineCount, LogLine line)
		{
			Dictionary<string, object?>? properties = null;
			if (line._properties != null)
			{
				properties = [];
				foreach ((string name, object value) in line._properties)
				{
					object newValue;
					if (value is LogSpan span)
					{
						string text = line._message.Substring(span._offset, span._length);
						if (span._value == null)
						{
							newValue = text;
						}
						else if (span._value is LogValue newLogValue)
						{
							newLogValue.Text = text;
							newValue = newLogValue;
						}
						else
						{
							newValue = text;
						}
					}
					else
					{
						newValue = value;
					}
					properties[name] = newValue;
				}
			}
			return new LogEvent(time, level, eventId, lineIndex, lineCount, line._message, line._format, properties, null);
		}

		/// <summary>
		/// Creates a match object at the given priority
		/// </summary>
		/// <param name="level"></param>
		/// <param name="eventId">The event id</param>
		/// <param name="priority"></param>
		/// <param name="isComplete"></param>
		/// <returns></returns>
		public LogEventMatch ToMatch(LogEventPriority priority, LogLevel level, EventId eventId, bool isComplete = true)
		{
			return new LogEventMatch(priority, ToArray(level, eventId), isComplete);
		}
		
		/// <summary>
		/// Creates a match object at the given priority with hanging lines
		/// </summary>
		/// <param name="level"></param>
		/// <param name="eventId">The event id</param>
		/// <param name="priority"></param>
		/// <returns></returns>
		public LogEventMatch ToMatchWithHangingLines(LogEventPriority priority, LogLevel level, EventId eventId)
		{
			// If there are more lines that are not hanging, is must be complete
			bool isComplete = Next.Length > 0;
			for (; isComplete && IsNextLineHanging(); isComplete = Next.Length > 0)
			{
				MoveNext();
			}

			return new LogEventMatch(priority, ToArray(level, eventId), isComplete);
		}
	}
}
