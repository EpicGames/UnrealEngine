// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.RegularExpressions;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods to allow adding markup to log event spans
	/// </summary>
	public static class LogEventMarkup
	{
#pragma warning disable CS1591
		public static LogValue Channel => new(LogValueType.Channel, "");
		public static LogValue Severity => new(LogValueType.Severity, "");
		public static LogValue Message => new(LogValueType.Message, "");
		public static LogValue LineNumber => new(LogValueType.LineNumber, "");
		public static LogValue ColumnNumber => new(LogValueType.ColumnNumber, "");
		public static LogValue Symbol => new(LogValueType.Symbol, "");
		public static LogValue ErrorCode => new(LogValueType.ErrorCode, "");
		public static LogValue ToolName => new(LogValueType.ToolName, "");
		public static LogValue ScreenshotTest => new(LogValueType.ScreenshotTest, "");
#pragma warning restore CS1591

		static readonly char[] s_pathChars = ['/', '\\'];

		static string RemoveRelativeDirs(string path)
		{
			int idx0 = path.IndexOfAny(s_pathChars);
			if (idx0 != -1)
			{
				int idx1 = path.IndexOfAny(s_pathChars, idx0 + 1);
				while (idx1 != -1)
				{
					int idx2 = path.IndexOfAny(s_pathChars, idx1 + 1);
					if (idx2 == -1)
					{
						break;
					}

					if (idx2 == idx1 + 3 && path[idx1 + 1] == '.' && path[idx1 + 2] == '.')
					{
						path = path.Remove(idx0, idx2 - idx0);
						idx1 = path.IndexOfAny(s_pathChars, idx0 + 1);
					}
					else
					{
						idx0 = idx1;
						idx1 = idx2;
					}
				}
			}
			return path;
		}

		/// <summary>
		/// Marks a span of text as a source file
		/// </summary>
		public static void AnnotateSourceFile(this LogEventBuilder builder, Group group, string? baseDir)
		{
			Dictionary<Utf8String, object?>? properties = null;
			if (!String.IsNullOrEmpty(baseDir))
			{
				string file = group.Value;
				if (!Path.IsPathRooted(file))
				{
					try
					{
						string combinedPath = RemoveRelativeDirs(Path.Combine(baseDir, file));
						properties = [];
						properties[LogEventPropertyName.File] = combinedPath;
					}
					catch
					{
					}
				}
			}
			builder.Annotate(group, new LogValue(LogValueType.SourceFile, group.Value, properties));
		}

		/// <summary>
		/// Marks a span of text as a source file
		/// </summary>
		public static void AnnotateAsset(this LogEventBuilder builder, Group group)
		{
			builder.Annotate(group, new LogValue(LogValueType.Asset, group.Value));
		}

		/// <summary>
		/// Marks a span of text as a symbol
		/// </summary>
		public static void AnnotateSymbol(this LogEventBuilder builder, Group group)
		{
			string identifier = group.Value;

			// Remove any __declspec qualifiers
			identifier = Regex.Replace(identifier, "(?<![^a-zA-Z_])__declspec\\([^\\)]+\\)", "");

			// Remove any argument lists for functions (anything after the first paren)
			identifier = Regex.Replace(identifier, "\\(.*$", "");

			// Remove any decorators and type information (greedy match up to the last space)
			identifier = Regex.Replace(identifier, "^.* ", "");

			// Add it to the list
			Dictionary<Utf8String, object?> properties = [];
			properties[LogEventPropertyName.Identifier] = identifier;
			builder.Annotate(group, new LogValue(LogValueType.Symbol, "", properties));
		}
	}
}
