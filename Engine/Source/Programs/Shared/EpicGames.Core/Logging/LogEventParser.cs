// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Confidence of a matched log event being the correct derivation
	/// </summary>
	public enum LogEventPriority
	{
		/// <summary>
		/// Unspecified priority
		/// </summary>
		None,

		/// <summary>
		/// Lowest confidence match
		/// </summary>
		Lowest,

		/// <summary>
		/// Low confidence match
		/// </summary>
		Low,

		/// <summary>
		/// Below normal confidence match
		/// </summary>
		BelowNormal,

		/// <summary>
		/// Normal confidence match
		/// </summary>
		Normal,

		/// <summary>
		/// Above normal confidence match
		/// </summary>
		AboveNormal,

		/// <summary>
		/// High confidence match
		/// </summary>
		High,

		/// <summary>
		/// Highest confidence match
		/// </summary>
		Highest,
	}

	/// <summary>
	/// Information about a matched event
	/// </summary>
	public class LogEventMatch
	{
		/// <summary>
		/// Confidence of the match
		/// </summary>
		public LogEventPriority Priority { get; }

		/// <summary>
		/// Matched events
		/// </summary>
		public List<LogEvent> Events { get; }

		/// <summary>
		/// Match is complete
		/// </summary>
		public bool IsComplete { get; }
		
		/// <summary>
		/// Constructor
		/// </summary>
		public LogEventMatch(LogEventPriority priority, LogEvent logEvent, bool isComplete)
		{
			Priority = priority;
			Events = [logEvent];
			IsComplete = isComplete;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public LogEventMatch(LogEventPriority priority, IEnumerable<LogEvent> events, bool isComplete)
		{
			Priority = priority;
			Events = [.. events];
			IsComplete = isComplete;
		}
	}

	/// <summary>
	/// Interface for a class which matches error strings
	/// </summary>
	public interface ILogEventMatcher
	{
		/// <summary>
		/// Attempt to match events from the given input buffer
		/// </summary>
		/// <param name="cursor">The input buffer</param>
		/// <returns>Information about the error that was matched, or null if an error was not matched</returns>
		LogEventMatch? Match(ILogCursor cursor);

		/// <summary>
		/// Get a name for this matcher, used to support dynamic matcher toggling. Names must not contain whitespace characters.
		/// </summary>
		/// <remarks>
		/// This is an interface method with a default interface to allow for edge-cases like multiple matchers of the same type
		/// but differing behaviour, matchers that want to be enabled or disabled together, inheritance etc.
		/// </remarks>
		string GetMatcherName() => GetType().Name;
	}

	/// <summary>
	/// An ignore pattern with metadata about its source file
	/// </summary>
	/// <param name="Pattern">The compiled regex pattern</param>
	/// <param name="SourceFile">Absolute path to the file that defined this pattern</param>
	public readonly record struct IgnorePatternEntry(Regex Pattern, string SourceFile)
	{
		/// <summary>
		/// Implicit conversion to Regex for backward compatibility with code that expects IgnorePatterns to contain Regex objects.
		/// </summary>
		public static implicit operator Regex(IgnorePatternEntry entry) => entry.Pattern;

		/// <summary>
		/// Implicit conversion from Regex for backward compatibility with code that adds Regex objects directly.
		/// </summary>
		public static implicit operator IgnorePatternEntry(Regex pattern) => new(pattern, "unknown");
	}

	/// <summary>
	/// Extension methods for backward compatibility with code that passes Regex collections to IgnorePatternEntry lists.
	/// </summary>
	public static class IgnorePatternEntryExtensions
	{
		/// <summary>
		/// Add a range of Regex objects as ignore pattern entries.
		/// </summary>
		public static void AddRange(this List<IgnorePatternEntry> list, IEnumerable<Regex> patterns)
		{
			foreach (Regex regex in patterns)
			{
				list.Add(new IgnorePatternEntry(regex, "unknown"));
			}
		}
	}

	/// <summary>
	/// Turns raw text output into structured logging events
	/// </summary>
	/// <remarks>
	/// Log events matching the expected format will be processed by the parser, taking the specified action if possible,
	/// and may not be output to users. The format is specified in <see cref="MatcherToggleRegex"/>, in addition to the commands
	/// <c>&lt;-- Suspend Log Parsing --&gt;</c> and <c>&lt;-- Resume Log Parsing --&gt;</c>.
	/// </remarks>
	public partial class LogEventParser : IDisposable
	{
		/// <summary>
		/// Timeout applied to each ignore pattern regex match.
		/// </summary>
		static readonly TimeSpan s_ignorePatternTimeout = TimeSpan.FromMilliseconds(200);

		/// <summary>
		/// Timeout applied to each ignore pattern regex match. Exposed for derived classes.
		/// </summary>
		protected static TimeSpan IgnorePatternTimeout => s_ignorePatternTimeout;

		/// <summary>
		/// A list of enabled event matchers for this parser.
		/// </summary>
		public IEnumerable<ILogEventMatcher> EnabledMatchers => [.. _enabledMatchers];

		/// <summary>
		/// List of patterns to ignore
		/// </summary>
		public List<IgnorePatternEntry> IgnorePatterns { get; } = [];

		/// <summary>
		/// A list of matchers all matchers, including disabled matchers.
		/// </summary>
		private readonly List<ILogEventMatcher> _matchers = [];

		/// <summary>
		/// A list of enabled matchers, to allow quicker iteration when parsing log lines.
		/// It's expected that we'll parse significantly more log lines than we will toggle matcher states.
		/// </summary>
		private readonly List<ILogEventMatcher> _enabledMatchers = [];

		/// <summary>
		/// A mapping of matcher names to how many times they've been enabled / disabled. A negative value indicates a status of disabled.
		/// Items not present are enabled by default.
		/// </summary>
		private readonly Dictionary<string, int> _matcherNameToEnabled = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// The layers of enabling / disabling matchers, so we can undo applications.
		/// </summary>
		private readonly Stack<(string MatcherName, string? GroupName, bool Enabled)> _matcherToggleLayers = [];

		/// <summary>
		/// Buffer of input lines
		/// </summary>
		readonly LogBuffer _buffer;

		/// <summary>
		/// Buffer for holding partial line data
		/// </summary>
		readonly ByteArrayBuilder _partialLine = new();

		/// <summary>
		/// Whether matching is currently enabled
		/// </summary>
		int _matchingEnabled;

		/// <summary>
		/// The inner logger
		/// </summary>
		ILogger _logger;

		/// <summary>
		/// Log events sinks in addition to <see cref="_logger" />
		/// </summary>
		readonly List<ILogEventSink> _logEventSinks = [];

		/// <summary>
		/// Timer for the parser being active
		/// </summary>
		readonly Stopwatch _timer = Stopwatch.StartNew();

		/// <summary>
		/// Amount of time that the log parser has been processing events
		/// </summary>
		readonly Stopwatch _activeTimer = new();

		/// <summary>
		/// Number of lines parsed in the last interval
		/// </summary>
		int _linesParsed = 0;

		/// <summary>
		/// Incomplete matcher being processed
		/// </summary>
		ILogEventMatcher? _incompleteMatcher;
		
		/// <summary>
		/// Incomplete event count being processed
		/// </summary>
		int _incompleteEventCount;
		
		/// <summary>
		/// Public accessor for the logger
		/// </summary>
		public ILogger Logger
		{
			get => _logger;
			set => _logger = value;
		}

		/// <summary>
		/// The pattern matcher toggle commands must conform to.
		/// </summary>
		/// <remarks>
		/// <c>LayerName</c> is an arbitrary string to allow validation that users are popping the layer they expect to be - if not, a warning will be issued.
		/// 
		/// <c>MatcherName</c> is either the literal name of a matcher, or the string <c>*</c> for all matchers.
		/// </remarks>
		[GeneratedRegex("^(?:<|\\\\u003C)-- logeventparser (layer=(?<LayerName>\\S+)? )?((?<Action>suspend|resume) (?<MatcherName>\\S+)|(?<Action>pop)) --(?:>|\\\\u003E)$", RegexOptions.IgnoreCase | RegexOptions.ExplicitCapture)]
		public static partial Regex MatcherToggleRegex();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">The logger to receive parsed output messages</param>
		/// <param name="logEventSinks">Additional sinks to receive log events</param>
		public LogEventParser(ILogger logger, List<ILogEventSink>? logEventSinks = null)
		{
			_logger = logger;
			_buffer = new LogBuffer(50);

			if (logEventSinks != null)
			{
				_logEventSinks.AddRange(logEventSinks);
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Standard Dispose pattern method
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				Flush();
			}
		}

		/// <summary>
		/// Add the selected matchers to the log event parser
		/// </summary>
		/// <param name="matchers">The matchers to add</param>
		public void AddMatchers(IEnumerable<ILogEventMatcher> matchers)
		{
			foreach (ILogEventMatcher matcher in matchers)
			{
				AddMatcher(matcher);
			}
		}

		/// <summary>
		/// Enumerate all the types that implement <see cref="ILogEventMatcher"/> in the given assembly, and create instances of them
		/// </summary>
		/// <param name="assembly">The assembly to enumerate matchers from</param>
		public void AddMatchersFromAssembly(Assembly assembly)
		{
			foreach (Type type in assembly.GetTypes())
			{
				if (type.IsClass && typeof(ILogEventMatcher).IsAssignableFrom(type))
				{
					ILogEventMatcher matcher = (ILogEventMatcher)Activator.CreateInstance(type)!;
					AddMatcher(matcher);
				}
			}
		}

		/// <summary>
		/// Read ignore patterns from the given root directory
		/// </summary>
		/// <param name="rootDir"></param>
		/// <returns></returns>
		[Obsolete("Use ReadIgnorePatternsFromWorkspaceAsync or DiscoverIgnorePatternFiles instead")]
		public async Task ReadIgnorePatternsAsync(DirectoryReference rootDir)
		{
			Stopwatch timer = Stopwatch.StartNew();

			List<DirectoryReference> baseDirs = [rootDir];
			AddRestrictedDirs(baseDirs, "Restricted");
			AddRestrictedDirs(baseDirs, "Platforms");

			List<(FileReference, Task)> tasks = [];
			foreach (DirectoryReference baseDir in baseDirs)
			{
				FileReference ignorePatternFile = FileReference.Combine(baseDir, "Build", "Horde", "IgnorePatterns.txt");
				if (FileReference.Exists(ignorePatternFile))
				{
					_logger.LogDebug("Reading ignore patterns from {File}...", ignorePatternFile);
					tasks.Add((ignorePatternFile, ReadIgnorePatternsAsync(ignorePatternFile)));
				}
			}

			foreach ((FileReference file, Task task) in tasks)
			{
				try
				{
					await task;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception reading patterns from {File}: {Message}", file, ex.Message);
				}
			}

			_logger.LogDebug("Took {TimeMs}ms to read ignore patterns", timer.ElapsedMilliseconds);
		}

		/// <summary>
		/// Read ignore patterns from a single file
		/// </summary>
		/// <param name="ignorePatternFile"></param>
		/// <returns></returns>
		public async Task ReadIgnorePatternsAsync(FileReference ignorePatternFile)
		{
			string[] lines = await FileReference.ReadAllLinesAsync(ignorePatternFile);

			List<IgnorePatternEntry> entries = [];
			for (int idx = 0; idx < lines.Length; idx++)
			{
				string trimLine = lines[idx].Trim();
				if (trimLine.Length > 0 && trimLine[0] != '#')
				{
					try
					{
						Regex regex = new(trimLine, RegexOptions.Compiled, s_ignorePatternTimeout);
						entries.Add(new IgnorePatternEntry(regex, ignorePatternFile.FullName));
					}
					catch (ArgumentException ex)
					{
						_logger.LogWarning("Invalid regex in {File} line {LineNumber}: {Pattern} ({Message})",
							ignorePatternFile, idx + 1, trimLine, ex.Message);
					}
				}
			}

			lock (IgnorePatterns)
			{
				IgnorePatterns.AddRange(entries);
			}
		}

		static void AddRestrictedDirs(List<DirectoryReference> directories, string subFolder)
		{
			int numDirs = directories.Count;
			for (int idx = 0; idx < numDirs; idx++)
			{
				DirectoryReference subDir = DirectoryReference.Combine(directories[idx], subFolder);
				if (DirectoryReference.Exists(subDir))
				{
					directories.AddRange(DirectoryReference.EnumerateDirectories(subDir));
				}
			}
		}

		static IgnorePatternEntry? FindMatchingIgnorePattern(List<IgnorePatternEntry> ignorePatterns, string line, ILogger logger)
		{
			foreach (IgnorePatternEntry entry in ignorePatterns)
			{
				try
				{
					if (entry.Pattern.IsMatch(line))
					{
						return entry;
					}
				}
				catch (RegexMatchTimeoutException)
				{
					logger.LogWarning("Ignore pattern timed out on line (pattern: {Pattern}): {Line}",
						entry.Pattern.ToString(), line.Length > 200 ? line[..200] + "..." : line);
				}
			}
			return null;
		}

		/// <summary>
		/// Writes a line that was suppressed by an ignore pattern, including structured properties
		/// identifying which pattern file and regex caused the suppression.
		/// </summary>
		/// <param name="line">The original log line text</param>
		/// <param name="ignoredBy">The ignore pattern entry that matched</param>
		/// <param name="originalLevel">The log level the line would have been emitted at before suppression</param>
		void WriteIgnoredLine(string line, IgnorePatternEntry ignoredBy, LogLevel originalLevel)
		{
			LogValue ignoredLineValue = new(LogValueType.IgnoredLine, line, new Dictionary<Utf8String, object?>
			{
				[new Utf8String("ignorePatternFile")] = ignoredBy.SourceFile,
				[new Utf8String("ignorePattern")] = ignoredBy.Pattern.ToString(),
				[new Utf8String("originalLevel")] = originalLevel.ToString(),
			});

			KeyValuePair<string, object?>[] state =
			[
				new(MessageTemplate.FormatPropertyName, "{IgnoredLine}"),
				new("IgnoredLine", ignoredLineValue),
			];

			_logger.Log(LogLevel.Information, KnownLogEvents.None, state, null, (s, _) => line);
		}

		/// <summary>
		/// Checks whether a structured JSON log event should be suppressed by an ignore pattern.
		/// Only events at Warning level or above are checked. If a matching pattern is found,
		/// the event is re-logged at Information level (preventing build health issue creation).
		/// </summary>
		/// <returns>True if the event was suppressed, false if it should be logged normally</returns>
		bool TryIgnoreStructuredEvent(JsonLogEvent jsonEvent)
		{
			if (jsonEvent.Level >= LogLevel.Warning && IgnorePatterns.Count > 0)
			{
				string message = jsonEvent.GetRenderedMessage().ToString();
				IgnorePatternEntry? ignoredBy = FindMatchingIgnorePattern(IgnorePatterns, message, _logger);
				if (ignoredBy != null)
				{
					WriteIgnoredLine(message, ignoredBy.Value, jsonEvent.Level);
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Discover all IgnorePatterns.txt files in a workspace by scanning Engine, project,
		/// and Restricted/Platforms subdirectories.
		/// </summary>
		public static List<FileReference> DiscoverIgnorePatternFiles(DirectoryReference workspaceDir, string enginePath = "Engine")
		{
			HashSet<DirectoryReference> scannedDirs = [];
			List<DirectoryReference> baseDirs = [];

			// Engine directory
			DirectoryReference engineDir = DirectoryReference.Combine(workspaceDir, enginePath);
			scannedDirs.Add(engineDir);
			baseDirs.Add(engineDir);

			// All top-level workspace directories
			if (DirectoryReference.Exists(workspaceDir))
			{
				foreach (DirectoryReference subDir in DirectoryReference.EnumerateDirectories(workspaceDir))
				{
					if (scannedDirs.Add(subDir))
					{
						baseDirs.Add(subDir);
					}
				}
			}

			// Restricted/Platforms under each base directory
			int count = baseDirs.Count;
			for (int idx = 0; idx < count; idx++)
			{
				foreach (string subFolder in new[] { "Restricted", "Platforms" })
				{
					DirectoryReference subDir = DirectoryReference.Combine(baseDirs[idx], subFolder);
					if (DirectoryReference.Exists(subDir))
					{
						foreach (DirectoryReference child in DirectoryReference.EnumerateDirectories(subDir))
						{
							if (scannedDirs.Add(child))
							{
								baseDirs.Add(child);
							}
						}
					}
				}
			}

			// Find all existing IgnorePatterns.txt files
			List<FileReference> files = [];
			foreach (DirectoryReference baseDir in baseDirs)
			{
				FileReference file = FileReference.Combine(baseDir, "Build", "Horde", "IgnorePatterns.txt");
				if (FileReference.Exists(file))
				{
					files.Add(file);
				}
			}
			return files;
		}

		/// <summary>
		/// Legacy ignore pattern file discovery - scans only Engine + Engine/Restricted/* + Engine/Platforms/*.
		/// Use <see cref="DiscoverIgnorePatternFiles"/> for expanded workspace-wide scanning.
		/// </summary>
		public static List<FileReference> DiscoverLegacyIgnorePatternFiles(DirectoryReference workspaceDir, string enginePath = "Engine")
		{
			DirectoryReference engineDir = DirectoryReference.Combine(workspaceDir, enginePath);
			List<DirectoryReference> baseDirs = [engineDir];
			AddRestrictedDirs(baseDirs, "Restricted");
			AddRestrictedDirs(baseDirs, "Platforms");

			List<FileReference> files = [];
			foreach (DirectoryReference baseDir in baseDirs)
			{
				FileReference file = FileReference.Combine(baseDir, "Build", "Horde", "IgnorePatterns.txt");
				if (FileReference.Exists(file))
				{
					files.Add(file);
				}
			}
			return files;
		}

		/// <summary>
		/// Read ignore patterns from the workspace, scanning Engine and project directories
		/// </summary>
		public async Task ReadIgnorePatternsFromWorkspaceAsync(DirectoryReference workspaceDir, string enginePath = "Engine")
		{
			Stopwatch timer = Stopwatch.StartNew();

			List<FileReference> files = DiscoverIgnorePatternFiles(workspaceDir, enginePath);

			List<(FileReference, Task)> tasks = [];
			foreach (FileReference file in files)
			{
				_logger.LogDebug("Reading ignore patterns from {File}...", file);
				tasks.Add((file, ReadIgnorePatternsAsync(file)));
			}

			foreach ((FileReference file, Task task) in tasks)
			{
				try
				{
					await task;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception reading patterns from {File}: {Message}", file, ex.Message);
				}
			}

			_logger.LogDebug("Read {PatternCount} ignore patterns from {FileCount} files in {TimeMs}ms",
				IgnorePatterns.Count, tasks.Count, timer.ElapsedMilliseconds);
		}

		/// <summary>
		/// Read ignore patterns from additional file paths
		/// </summary>
		public async Task ReadIgnorePatternsAsync(IEnumerable<FileReference> additionalFiles)
		{
			List<(FileReference, Task)> tasks = [];
			foreach (FileReference file in additionalFiles)
			{
				if (FileReference.Exists(file))
				{
					_logger.LogDebug("Reading ignore patterns from {File}...", file);
					tasks.Add((file, ReadIgnorePatternsAsync(file)));
				}
				else
				{
					_logger.LogWarning("Ignore pattern file not found: {File}", file);
				}
			}

			foreach ((FileReference file, Task task) in tasks)
			{
				try
				{
					await task;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception reading patterns from {File}: {Message}", file, ex.Message);
				}
			}
		}

		/// <summary>
		/// Writes a line to the event filter
		/// </summary>
		/// <param name="line">The line to output</param>
		public void WriteLine(string line)
		{
			bool invalidJson = false;

			if (line.Length > 0 && line[0] == '{')
			{
				int length = line.Length;
				while (length > 0 && Char.IsWhiteSpace(line[length - 1]))
				{
					length--;
				}

				byte[] data = Encoding.UTF8.GetBytes(line, 0, length);
				try
				{
					JsonLogEvent jsonEvent;
					if (JsonLogEvent.TryParse(data, out jsonEvent))
					{
						ProcessData(true);

						if (Regex.IsMatch(line, @"\\u003c-- Suspend Log Parsing --\\u003e", RegexOptions.IgnoreCase))
						{
							_matchingEnabled--;
						}
						else if (Regex.IsMatch(line, @"\\u003c-- Resume Log Parsing --\\u003e", RegexOptions.IgnoreCase))
						{
							_matchingEnabled++;
						}

						if (!TryIgnoreStructuredEvent(jsonEvent))
						{
							_logger.Log(jsonEvent.Level, jsonEvent.EventId, jsonEvent, null, JsonLogEvent.Format);
						}
						return;
					}

					invalidJson = true;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while parsing log event: {Message}", ex.Message);
				}
			}

			_buffer.AddLine(StringUtils.ParseEscapeCodes(line));

			ProcessData(false, disableMatching: invalidJson);
		}

		/// <summary>
		/// Writes data to the log parser
		/// </summary>
		/// <param name="data">Data to write</param>
		public void WriteData(ReadOnlyMemory<byte> data)
		{
			int baseIdx = 0;
			int scanIdx = 0;
			ReadOnlySpan<byte> span = data.Span;

			// Handle a partially existing line
			if (_partialLine.Length > 0)
			{
				for (; scanIdx < span.Length; scanIdx++)
				{
					if (span[scanIdx] == '\n')
					{
						_partialLine.WriteFixedLengthBytes(span.Slice(baseIdx, scanIdx - baseIdx));
						FlushPartialLine();
						baseIdx = ++scanIdx;
						break;
					}
				}
			}

			// Handle any complete lines
			for (; scanIdx < span.Length; scanIdx++)
			{
				if (span[scanIdx] == '\n')
				{
					AddLine(data.Slice(baseIdx, scanIdx - baseIdx));
					baseIdx = scanIdx + 1;
				}
			}

			// Add the rest of the text to the partial line buffer
			_partialLine.WriteFixedLengthBytes(span.Slice(baseIdx));

			// Process the new data
			ProcessData(false);
		}

		/// <summary>
		/// Flushes the current contents of the parser
		/// </summary>
		public void Flush()
		{
			// If there's a partially written line, write that out first
			if (_partialLine.Length > 0)
			{
				FlushPartialLine();
			}

			// Process any remaining data
			ProcessData(true);
		}

		/// <summary>
		/// Add the new matcher, making sure its enabled state is correct.
		/// </summary>
		private void AddMatcher(ILogEventMatcher matcher)
		{
			if (matcher is null)
			{
				return;
			}

			_matchers.Add(matcher);
			// This is a little inefficient, but this function shouldn't be called often.
			RecalculateEnabledMatchers();
		}

		/// <summary>
		/// Set the given matcher to the given enabled state.
		/// </summary>
		private void PushToggle(string matcherName, string layerName, bool enabled)
		{
			if (_matcherToggleLayers.Count > 1024)
			{
				_logger.LogWarning(
					"[LogEventParser] Over 1024 log event parser toggle layers have been pushed, no longer processing any more to avoid unbounded memory usage. Most recent toggle: matcher='{MatcherName}', layer='{LayerName}', enabled={Enabled}",
					matcherName,
					layerName,
					enabled);
				return;
			}

			// Allow debugging if all log matchers get disabled and this isn't popped.
			_logger.LogTrace(
				"[LogEventParser] Most recent toggle: matcher='{MatcherName}', layer='{LayerName}', enabled={Enabled}",
				matcherName,
				layerName,
				enabled);

			_matcherToggleLayers.Push((matcherName, layerName, enabled));
			_matcherNameToEnabled.TryGetValue(matcherName, out int enabledCount);
			enabledCount += enabled ? 1 : -1;
			_matcherNameToEnabled[matcherName] = enabledCount;

			RecalculateEnabledMatchers();
		}

		private void PopToggle(string layerName)
		{
			_logger.LogTrace("[LogEventParser] Popping most recent toggle: layer='{LayerName}'", layerName);

			if (_matcherToggleLayers.Count == 0)
			{
				_logger.LogWarning("[LogEventParser] Popped empty stack, layer='{LayerName}'", layerName);
				return;
			}

			(string matcherName, string? layerNameFromPop, bool enabled) = _matcherToggleLayers.Pop();

			if (layerName.Length != 0 && !String.Equals(layerName, layerNameFromPop, StringComparison.OrdinalIgnoreCase))
			{
				_logger.LogWarning(
					"[LogEventParser] Given layer '{LayerName}' is not the same as last pushed layer '{LayerNameFromPop}', please check your logic.",
					layerName,
					layerNameFromPop);
				// Arguable if we should pop anyway, but I think it's slightly safer to do that.
			}

			// We're popping a layer, so if it was enabled, we're adding a disable and vice versa - hence why the sign is flipped compared to PushToggle.
			_matcherNameToEnabled[matcherName] += enabled ? -1 : 1;

			RecalculateEnabledMatchers();
		}

		/// <summary>
		/// Clear and repopulate the list of enabled matchers.
		/// </summary>
		private void RecalculateEnabledMatchers()
		{
			_enabledMatchers.Clear();

			// To allow disabling all loggers then enabling specific ones, we add the value for each logger to the value for "*".
			_matcherNameToEnabled.TryGetValue("*", out int allCount);

			foreach (ILogEventMatcher matcher in _matchers)
			{
				_matcherNameToEnabled.TryGetValue(matcher.GetMatcherName(), out int enabledCount);

				if (enabledCount + allCount >= 0)
				{
					_enabledMatchers.Add(matcher);
				}
			}
		}

		/// <summary>
		/// Adds a raw utf-8 string to the buffer
		/// </summary>
		/// <param name="data">The string data</param>
		private void AddLine(ReadOnlyMemory<byte> data)
		{
			if (data.Length > 0 && data.Span[data.Length - 1] == '\r')
			{
				data = data.Slice(0, data.Length - 1);
			}
			if (data.Length > 0 && data.Span[0] == '{')
			{
				JsonLogEvent jsonEvent;
				if (JsonLogEvent.TryParse(data, out jsonEvent))
				{
					ProcessData(true);
					if (!TryIgnoreStructuredEvent(jsonEvent))
					{
						_logger.LogJsonLogEvent(jsonEvent);
					}
					return;
				}
			}
			_buffer.AddLine(StringUtils.ParseEscapeCodes(Encoding.UTF8.GetString(data.Span)));
		}

		/// <summary>
		/// Writes the current partial line data, with the given data appended to it, then clear the buffer
		/// </summary>
		private void FlushPartialLine()
		{
			AddLine(_partialLine.ToByteArray());
			_partialLine.Clear();
		}

		/// <summary>
		/// Check for and handle a log event parser command.
		/// </summary>
		/// <param name="line">The line to check for a log event parser command.</param>
		/// <returns>Whether a log event parser command was found and handled.</returns>
		private bool HandleLogEventParserCommand(string line)
		{
			Match match = MatcherToggleRegex().Match(line);
			if (match.Success)
			{
				string action = match.Groups["Action"].Value;
				string matcherName = match.Groups["MatcherName"].Value;
				string layerName = match.Groups["LayerName"].Value;
				switch (action.ToUpperInvariant())
				{
					case "RESUME":
						PushToggle(matcherName, layerName, true);
						break;
					case "SUSPEND":
						PushToggle(matcherName, layerName, false);
						break;
					case "POP":
						PopToggle(layerName);
						break;
				}
				_buffer.MoveNext();
				return true;
			}

			return false;
		}

		/// <summary>
		/// Process any data in the buffer
		/// </summary>
		/// <param name="bFlush">Whether we've reached the end of the stream</param>
		/// <param name="disableMatching">Whether to skip matching</param>
		void ProcessData(bool bFlush, bool disableMatching = false)
		{
			_activeTimer.Start();
			int startLineCount = _buffer.Length;
			if (ProcessIncomplete(bFlush))
			{
				while (_buffer.Length > 0)
				{
					string? line = _buffer[0];
					if (line == null)
					{
						line = "";
					}
					else if (HandleLogEventParserCommand(line))
					{
						continue;
					}
					else if (Regex.IsMatch(line, "<-- Suspend Log Parsing -->", RegexOptions.IgnoreCase))
					{
						_matchingEnabled--;
					}
					else if (Regex.IsMatch(line, "<-- Resume Log Parsing -->", RegexOptions.IgnoreCase))
					{
						_matchingEnabled++;
					}
					else if (_matchingEnabled >= 0 && !disableMatching)
					{
						(ILogEventMatcher? matcher, LogEventMatch? match) = MatchEvent();

						// Bail out if we need more data, or match is incomplete
						if (_buffer.Length < 1024 && !bFlush && (_buffer.NeedMoreData || match is { IsComplete: false }))
						{
							if (!_buffer.NeedMoreData && FindMatchingIgnorePattern(IgnorePatterns, line, _logger) == null)
							{
								WriteEvents(match!.Events);
								Debug.Assert(matcher != null);
								_incompleteMatcher = matcher;
								_incompleteEventCount = match.Events.Count;
							}
							break;
						}

						// Report the error to the listeners
						// If we did match something, check if it's not negated by an ignore pattern. We typically have relatively few errors and many more ignore patterns than matchers, so it's quicker
						// to check them in response to an identified error than to treat them as matchers of their own.
						if (match != null)
						{
							IgnorePatternEntry? ignoredBy = FindMatchingIgnorePattern(IgnorePatterns, line, _logger);
							if (ignoredBy == null)
							{
								WriteEvents(match.Events);
								_buffer.Advance(match.Events.Count);
								continue;
							}

							WriteIgnoredLine(line, ignoredBy.Value, match.Events[0].Level);
							_buffer.MoveNext();
							continue;
						}
					}

					_logger.Log(LogLevel.Information, KnownLogEvents.None, line, null, (state, exception) => state);
					_buffer.MoveNext();
				}
			}
			_linesParsed += startLineCount - _buffer.Length;
			_activeTimer.Stop();

			const double UpdateIntervalSeconds = 30.0;
			double elapsedSeconds = _timer.Elapsed.TotalSeconds;
			if (elapsedSeconds > UpdateIntervalSeconds)
			{
				const double WarnPct = 0.5;
				double activeSeconds = _activeTimer.Elapsed.TotalSeconds;
				double activePct = activeSeconds / elapsedSeconds;

				if (activePct > WarnPct)
				{
					_logger.LogInformation(KnownLogEvents.Systemic_LogParserBottleneck, "EpicGames.Core.LogEventParser is taking a significant amount of CPU time: {Active:n1}s/{Total:n1}s ({Pct:n1}%). Processed {NumLines} lines in last {Interval} seconds ({NumLinesInBuffer} in buffer).", activeSeconds, elapsedSeconds, activePct * 100.0, _linesParsed, UpdateIntervalSeconds, _buffer.Length);
				}

				_activeTimer.Reset();
				_timer.Restart();
				_linesParsed = 0;
			}
		}

		/// <summary>
		/// Process any incomplete match
		/// </summary>
		/// <param name="bFlush">Whether we've reached the end of the stream</param>
		/// <returns>Whether it finished processing the incomplete match</returns>
		bool ProcessIncomplete(bool bFlush)
		{
			if (_incompleteMatcher != null)
			{
				LogEventMatch match = _incompleteMatcher.Match(_buffer)!;
				Debug.Assert(match.Events.Count >= _incompleteEventCount);
				WriteEvents(match.Events.Skip(_incompleteEventCount));

				// Bail out if we need more data
				if (_buffer.Length < 1024 && !bFlush && !match.IsComplete)
				{
					_incompleteEventCount = match.Events.Count;
					return false;
				}

				_incompleteMatcher = null;
				_incompleteEventCount = 0;
				_buffer.Advance(match.Events.Count);
			}

			return true;
		}
		
		/// <summary>
		/// Try to match an event from the current buffer
		/// </summary>
		/// <returns>The matcher and the match</returns>
		private (ILogEventMatcher?, LogEventMatch?) MatchEvent()
		{
			ILogEventMatcher? currentMatcher = null;
			LogEventMatch? currentMatch = null;
			foreach (ILogEventMatcher matcher in _enabledMatchers)
			{
				try
				{
					LogEventMatch? match = matcher.Match(_buffer);
					if (match != null && (currentMatch == null || match.Priority > currentMatch.Priority))
					{
						currentMatcher = matcher;
						currentMatch = match;
					}
				}
				catch (Exception ex)
				{
					_logger.LogWarning(KnownLogEvents.Systemic_LogEventMatcher, ex, "Exception while parsing log events with {Type}. Buffer size {Length}.", matcher.GetType().Name, _buffer.Length);
				}
			}
			return (currentMatcher, currentMatch);
		}

		/// <summary>
		/// Writes an event to the log
		/// </summary>
		/// <param name="logEvents">The event to write</param>
		protected virtual void WriteEvents(IEnumerable<LogEvent> logEvents)
		{
			foreach (LogEvent logEvent in logEvents)
			{
				_logger.Log(logEvent.Level, logEvent.Id, logEvent, null, (state, exception) => state.ToString());
				foreach (ILogEventSink logEventSink in _logEventSinks)
				{
					logEventSink.ProcessEvent(logEvent);
				}
			}
		}
	}
}
