// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;

namespace EpicGames.Core
{
	/// <summary>
	/// Static class for tracking a stack of log indent prefixes
	/// </summary>
	public static class LogIndent
	{
		/// <summary>
		/// Tracks the current state
		/// </summary>
		class State
		{
			public State? PrevState { get; }
			public string Indent { get; }

			public State(State? prevState, string indent)
			{
				PrevState = prevState;
				Indent = indent;
			}
		}

		/// <summary>
		/// The current state value
		/// </summary>
		static readonly AsyncLocal<State?> s_currentState = new();

		/// <summary>
		/// Gets the current indent string
		/// </summary>
		public static string Current => s_currentState.Value?.Indent ?? String.Empty;

		/// <summary>
		/// Push a new indent onto the stack
		/// </summary>
		/// <param name="indent">The indent to add</param>
		public static void Push(string indent)
		{
			State? prevState = s_currentState.Value;
			if (prevState != null)
			{
				indent = prevState.Indent + indent;
			}
			s_currentState.Value = new State(prevState, indent);
		}

		/// <summary>
		/// Pops an indent off the stack
		/// </summary>
		public static void Pop()
		{
			s_currentState.Value = s_currentState.Value!.PrevState;
		}
	}
}
