// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Class to apply a log indent for the lifetime of an object 
	/// </summary>
	public sealed class LogIndentScope : IDisposable
	{
		/// <summary>
		/// Whether the object has been disposed
		/// </summary>
		bool _disposed;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="indent">Indent to append to the existing indent</param>
		public LogIndentScope(string indent)
		{
			LogIndent.Push(indent);
		}

		/// <summary>
		/// Restore the log indent to normal
		/// </summary>
		public void Dispose()
		{
			if (!_disposed)
			{
				LogIndent.Pop();
				_disposed = true;
			}
		}
	}
}
