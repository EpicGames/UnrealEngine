// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Writes a status message to the log, which can be updated with a progress indicator as a slow task is being performed.
	/// </summary>
	public sealed class LogStatusScope : IDisposable
	{
		/// <summary>
		/// The base status message
		/// </summary>
		string _message;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">The status message</param>
		public LogStatusScope(string message)
		{
			_message = message;
			Log.PushStatus(message);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">The format specifier for the message</param>
		/// <param name="args">Arguments for the status message</param>
		public LogStatusScope(string format, params object[] args)
			: this(String.Format(format, args))
		{
		}

		/// <summary>
		/// Updates the base status message passed into the constructor.
		/// </summary>
		/// <param name="message">The status message</param>
		public void SetMessage(string message)
		{
			_message = message;
			Log.UpdateStatus(message);
		}

		/// <summary>
		/// Updates the base status message passed into the constructor.
		/// </summary>
		/// <param name="format">The format specifier for the message</param>
		/// <param name="args">Arguments for the status message</param>
		public void SetMessage(string format, params object[] args)
		{
			SetMessage(String.Format(format, args));
		}

		/// <summary>
		/// Appends a progress string to the status message. Overwrites any previous progress message.
		/// </summary>
		/// <param name="progress">The progress message</param>
		public void SetProgress(string progress)
		{
			StringBuilder fullMessage = new(_message);
			fullMessage.Append(' ');
			fullMessage.Append(progress);
			Log.UpdateStatus(fullMessage.ToString());
		}

		/// <summary>
		/// Appends a progress string to the status message. Overwrites any previous progress message.
		/// </summary>
		/// <param name="format">The format specifier for the message</param>
		/// <param name="args">Arguments for the status message</param>
		public void SetProgress(string format, params object[] args)
		{
			StringBuilder fullMessage = new(_message);
			fullMessage.Append(' ');
			fullMessage.AppendFormat(format, args);
			Log.UpdateStatus(fullMessage.ToString());
		}

		/// <summary>
		/// Pops the status message from the log.
		/// </summary>
		public void Dispose()
		{
			if (_message != null)
			{
				Log.PopStatus();
				_message = null!;
			}
		}
	}
}
