// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Different types of log values (stored in the $type field of log properties)
	/// </summary>
	public static class LogValueType
	{
#pragma warning disable CS1591
		public static readonly Utf8String Asset = new("Asset");
		public static readonly Utf8String SourceFile = new("SourceFile");
		public static readonly Utf8String Object = new("Object"); // Arbitrary structured object
		public static readonly Utf8String Channel = new("Channel");
		public static readonly Utf8String Severity = new("Severity");
		public static readonly Utf8String Message = new("Message");
		public static readonly Utf8String LineNumber = new("Line");
		public static readonly Utf8String ColumnNumber = new("Column");
		public static readonly Utf8String Symbol = new("Symbol");
		public static readonly Utf8String ErrorCode = new("ErrorCode");
		public static readonly Utf8String ToolName = new("ToolName");
		public static readonly Utf8String ScreenshotTest = new("ScreenshotTest");
		public static readonly Utf8String DepotPath = new("DepotPath");
		public static readonly Utf8String Link = new("Link");
		public static readonly Utf8String IgnoredLine = new("IgnoredLine");
#pragma warning restore CS1591
	}

	/// <summary>
	/// Attribute indicating that a type should be tagged in log output
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct)]
	public sealed class LogValueTypeAttribute : Attribute
	{
		/// <summary>
		/// Name to use for the type tag
		/// </summary>
		public string? Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public LogValueTypeAttribute(string? name = null) => Name = name;
	}

	/// <summary>
	/// Information for a structured value for use in log events
	/// </summary>
	public sealed class LogValue
	{
		/// <summary>
		/// Type of the event
		/// </summary>
		public Utf8String Type { get; set; }

		/// <summary>
		/// Rendering of the value
		/// </summary>
		public string Text { get; set; }

		/// <summary>
		/// Properties associated with the value
		/// </summary>
		public Dictionary<Utf8String, object?>? Properties { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">Type of the value</param>
		/// <param name="text">Rendering of the value as text</param>
		/// <param name="properties">Additional properties for this value</param>
		public LogValue(Utf8String type, string text, Dictionary<Utf8String, object?>? properties = null)
		{
			Type = type;
			Text = text;
			Properties = properties;
		}

		/// <summary>
		/// Construct a source file log value with an overridden display text
		/// </summary>
		/// <param name="file">Source file to reference</param>
		/// <param name="text">Display text for the fiel</param>
		public static LogValue SourceFile(FileReference file, string text)
		{
			return new LogValue(LogValueType.SourceFile, text, new Dictionary<Utf8String, object?> { [LogEventPropertyName.File] = file.FullName });
		}

		/// <summary>
		/// Constructs a value which contains a hyperlink to an external page
		/// </summary>
		/// <param name="target">Target URL</param>
		public static LogValue Link(Uri target)
		{
			return Link(target, target.ToString());
		}

		/// <summary>
		/// Constructs a value which contains a hyperlink to an external page
		/// </summary>
		/// <param name="target">Target URL</param>
		/// <param name="text">Text to render for the link</param>
		public static LogValue Link(Uri target, string text)
		{
			return new LogValue(LogValueType.Link, text, new Dictionary<Utf8String, object?> { [LogEventPropertyName.Target] = target.ToString() });
		}

		/// <summary>
		/// Creates a LogValue from an object, overriding the type and display text for it
		/// </summary>
		/// <param name="obj">The object to construct from</param>
		/// <returns></returns>
		public static LogValue FromObject(object obj) => FromObject(LogValueType.Object, obj.ToString() ?? String.Empty, obj);

		/// <summary>
		/// Creates a LogValue from an object, overriding the type and display text for it
		/// </summary>
		/// <param name="type">Type of the object</param>
		/// <param name="text">Rendered representation of the object in the output string</param>
		/// <param name="obj">The object to construct from</param>
		/// <returns></returns>
		public static LogValue FromObject(Utf8String type, string text, object obj)
		{
			Type objType = obj.GetType();

			Dictionary<Utf8String, object?>? properties = null;
			foreach (PropertyInfo propertyInfo in objType.GetProperties())
			{
				if (propertyInfo.GetCustomAttribute<JsonIgnoreAttribute>() == null)
				{
					string name = propertyInfo.GetCustomAttribute<JsonPropertyNameAttribute>()?.Name ?? propertyInfo.Name;
					properties ??= [];
					properties[new Utf8String(name)] = propertyInfo.GetValue(obj)!;
				}
			}

			return new LogValue(type, text, properties);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return Text;
		}
	}
}
