// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Reflection;
using System.Text.Json;

namespace EpicGames.Core
{
	/// <summary>
	/// Formatter for objects of a given type
	/// </summary>
	public interface ILogValueFormatter
	{
		/// <summary>
		/// Format an object to a JSON output stream
		/// </summary>
		/// <param name="value"></param>
		/// <param name="writer"></param>
		void Format(object value, Utf8JsonWriter writer);
	}

	/// <summary>
	/// Specifies a type to use for serializing objects to a log event
	/// </summary>
	[AttributeUsage(AttributeTargets.Struct | AttributeTargets.Class)]
	public sealed class LogValueFormatterAttribute : Attribute
	{
		/// <summary>
		/// Type of the formatter
		/// </summary>
		public Type Type { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type"></param>
		public LogValueFormatterAttribute(Type type)
		{
			Type = type;
		}
	}

	/// <summary>
	/// Utility methods for manipulating formatters
	/// </summary>
	public static class LogValueFormatter
	{
		class BoolFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteBooleanValue((bool)value);
		}

		class Int32Formatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((int)value);
		}

		class UInt32Formatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((uint)value);
		}

		class Int64Formatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((long)value);
		}

		class UInt64Formatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((ulong)value);
		}

		class FloatFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((float)value);
		}

		class DoubleFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteNumberValue((double)value);
		}

		class StringFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer) => writer.WriteStringValue(value.ToString());
		}

		class ActivityFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer)
			{
				writer.WriteStartArray();
				Format((Activity)value, writer);
				writer.WriteEndArray();
			}

			static void Format(Activity activity, Utf8JsonWriter writer)
			{
				if (activity.Parent != null)
				{
					Format(activity.Parent, writer);
				}

				writer.WriteStartObject();
				writer.WriteString("name", activity.OperationName);
				if (activity.Tags.Any())
				{
					writer.WriteStartObject("tags");
					foreach (KeyValuePair<string, string?> pair in activity.Tags)
					{
						writer.WriteString(pair.Key, pair.Value);
					}
					writer.WriteEndObject();
				}
				writer.WriteEndObject();
			}
		}

		class EnumerableFormatter : ILogValueFormatter
		{
			readonly ILogValueFormatter _elementFormatter;

			public EnumerableFormatter(ILogValueFormatter elementFormatter) => _elementFormatter = elementFormatter;

			public void Format(object value, Utf8JsonWriter writer)
			{
				writer.WriteStartArray();
				foreach (object element in (IEnumerable)value)
				{
					_elementFormatter.Format(element, writer);
				}
				writer.WriteEndArray();
			}
		}

		class StructuredLogValueFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer)
			{
				LogValue logValue = (LogValue)value;
				writer.WriteStartObject();
				writer.WriteString(LogEventPropertyName.Type, logValue.Type);
				writer.WriteString(LogEventPropertyName.Text, logValue.Text);
				if (logValue.Properties != null)
				{
					foreach (KeyValuePair<Utf8String, object?> pair in logValue.Properties)
					{
						writer.WritePropertyName(pair.Key);
						WritePropertyValue(pair.Value, writer);
					}
				}
				writer.WriteEndObject();
			}

			static void WritePropertyValue(object? value, Utf8JsonWriter writer)
			{
				if (value == null)
				{
					writer.WriteNullValue();
				}
				else
				{
					Type valueType = value.GetType();
					switch (Type.GetTypeCode(valueType))
					{
						case TypeCode.Boolean:
							writer.WriteBooleanValue((bool)value);
							break;
						case TypeCode.Byte:
							writer.WriteNumberValue((byte)value);
							break;
						case TypeCode.SByte:
							writer.WriteNumberValue((sbyte)value);
							break;
						case TypeCode.UInt16:
							writer.WriteNumberValue((ushort)value);
							break;
						case TypeCode.UInt32:
							writer.WriteNumberValue((uint)value);
							break;
						case TypeCode.UInt64:
							writer.WriteNumberValue((long)value);
							break;
						case TypeCode.Int16:
							writer.WriteNumberValue((short)value);
							break;
						case TypeCode.Int32:
							writer.WriteNumberValue((int)value);
							break;
						case TypeCode.Int64:
							writer.WriteNumberValue((long)value);
							break;
						case TypeCode.Decimal:
							writer.WriteNumberValue((decimal)value);
							break;
						case TypeCode.Double:
							writer.WriteNumberValue((double)value);
							break;
						case TypeCode.Single:
							writer.WriteNumberValue((float)value);
							break;
						case TypeCode.String:
							writer.WriteStringValue((string)value);
							break;
						default:
							JsonSerializer.Serialize(writer, value);
							break;
					}
				}
			}
		}

		class AnnotateTypeFormatter : ILogValueFormatter
		{
			readonly Utf8String _type;

			public AnnotateTypeFormatter(string type) => _type = new Utf8String(type);

			public void Format(object value, Utf8JsonWriter writer)
			{
				writer.WriteStartObject();
				writer.WriteString(LogEventPropertyName.Type, _type);
				writer.WriteString(LogEventPropertyName.Text, value.ToString());
				writer.WriteEndObject();
			}
		}

		class DictionaryFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer)
			{
				Dictionary<string, object> dictionary = (Dictionary<string, object>)value;
				writer.WriteStartObject();
				foreach (KeyValuePair<string, object> kvp in dictionary)
				{
					writer.WritePropertyName(kvp.Key);
					Format(kvp.Value, writer);
				}
				writer.WriteEndObject();
			}
		}

		class Utf8DictionaryFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer)
			{
				Dictionary<Utf8String, object> dictionary = (Dictionary<Utf8String, object>)value;
				writer.WriteStartObject();
				foreach (KeyValuePair<Utf8String, object> kvp in dictionary)
				{
					writer.WritePropertyName(kvp.Key);
					LogValueFormatter.Format(kvp.Value, writer);
				}
				writer.WriteEndObject();
			}
		}

		class FileReferenceFormatter : ILogValueFormatter
		{
			public void Format(object value, Utf8JsonWriter writer)
			{
				FileReference file = (FileReference)value;
				writer.WriteStartObject();
				writer.WriteString(LogEventPropertyName.Type, (file.HasExtension(".uasset") || file.HasExtension(".umap")) ? LogValueType.Asset : LogValueType.SourceFile);
				writer.WriteString(LogEventPropertyName.Text, file.FullName);
				writer.WriteEndObject();
			}
		}

		static readonly StringFormatter s_stringFormatter = new();

		static readonly ConcurrentDictionary<Type, ILogValueFormatter> s_formatters = GetDefaultFormatters();

		static ConcurrentDictionary<Type, ILogValueFormatter> GetDefaultFormatters()
		{
			ConcurrentDictionary<Type, ILogValueFormatter> formatters = new();
			formatters.TryAdd(typeof(bool), new BoolFormatter());
			formatters.TryAdd(typeof(int), new Int32Formatter());
			formatters.TryAdd(typeof(uint), new UInt32Formatter());
			formatters.TryAdd(typeof(long), new Int64Formatter());
			formatters.TryAdd(typeof(ulong), new UInt64Formatter());
			formatters.TryAdd(typeof(float), new FloatFormatter());
			formatters.TryAdd(typeof(double), new DoubleFormatter());
			formatters.TryAdd(typeof(string), s_stringFormatter);
			formatters.TryAdd(typeof(Dictionary<string, object>), new DictionaryFormatter());
			formatters.TryAdd(typeof(Dictionary<Utf8String, object>), new Utf8DictionaryFormatter());
			formatters.TryAdd(typeof(LogValue), new StructuredLogValueFormatter());
			formatters.TryAdd(typeof(FileReference), new FileReferenceFormatter());
			formatters.TryAdd(typeof(Activity), new ActivityFormatter());
			return formatters;
		}

		/// <summary>
		/// Writes a typed 
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="type"></param>
		/// <param name="text"></param>
		public static void WriteTypedValue(Utf8JsonWriter writer, string type, string text)
		{
			writer.WriteStartObject();
			writer.WriteString(LogEventPropertyName.Type, type);
			writer.WriteString(LogEventPropertyName.Text, text);
			writer.WriteEndObject();
		}

		/// <summary>
		/// Registers a formatter for a particular type
		/// </summary>
		/// <param name="type"></param>
		/// <param name="formatter"></param>
		public static void RegisterFormatter(Type type, ILogValueFormatter formatter)
		{
			s_formatters.TryAdd(type, formatter);
		}

		/// <summary>
		/// Registers a formatter for a particular type that adds a $type field to the serialized output
		/// </summary>
		/// <param name="type">Name of the type</param>
		public static void RegisterTypeAnnotation<T>(string type)
		{
			s_formatters.TryAdd(typeof(T), new AnnotateTypeFormatter(type));
		}

		/// <summary>
		/// Gets a formatter for the specified type
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		public static ILogValueFormatter GetFormatter(Type type)
		{
			for (; ; )
			{
				ILogValueFormatter? formatter;
				if (s_formatters.TryGetValue(type, out formatter))
				{
					return formatter;
				}

				formatter = GetFormatterUncached(type);
				if (s_formatters.TryAdd(type, formatter))
				{
					return formatter;
				}
			}
		}

		static ILogValueFormatter GetFormatterUncached(Type type)
		{
			LogValueFormatterAttribute? formatterAttribute = type.GetCustomAttribute<LogValueFormatterAttribute>();
			if (formatterAttribute != null)
			{
				return (ILogValueFormatter)Activator.CreateInstance(formatterAttribute.Type)!;
			}

			LogValueTypeAttribute? typeAttribute = type.GetCustomAttribute<LogValueTypeAttribute>();
			if (typeAttribute != null)
			{
				return new AnnotateTypeFormatter(typeAttribute.Name ?? type.Name);
			}

			if (TryGetEnumerableType(type, out Type? elementType))
			{
				return new EnumerableFormatter(GetFormatter(elementType));
			}

			return s_stringFormatter;
		}

		static bool TryGetEnumerableType(Type type, [NotNullWhen(true)] out Type? elementType)
		{
			foreach (Type interfaceType in type.GetInterfaces())
			{
				if (interfaceType.IsGenericType && interfaceType.GetGenericTypeDefinition() == typeof(IEnumerable<>))
				{
					elementType = interfaceType.GetGenericArguments()[0];
					return true;
				}
			}

			elementType = null;
			return false;
		}

		/// <summary>
		/// Formats a value to a Json log stream
		/// </summary>
		/// <param name="value">Value to write</param>
		/// <param name="writer">Json writer</param>
		public static void Format(object? value, Utf8JsonWriter writer)
		{
			if (value == null)
			{
				writer.WriteNullValue();
				return;
			}

			ILogValueFormatter formatter = GetFormatter(value.GetType());
			formatter.Format(value, writer);
		}
	}
}
