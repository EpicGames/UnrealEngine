// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Text;
using System.Text.Json;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility class for dealing with message templates
	/// </summary>
	public static class MessageTemplate
	{
		/// <summary>
		/// The default property name for the message template format string in an enumerable log state parameter
		/// </summary>
		public const string FormatPropertyName = "{OriginalFormat}";

		/// <summary>
		/// Renders a format string
		/// </summary>
		/// <param name="format">The format string</param>
		/// <param name="properties">Property values to embed</param>
		/// <returns>The rendered string</returns>
		public static string Render(string format, IEnumerable<KeyValuePair<string, object?>>? properties)
		{
			StringBuilder result = new();
			Render(format, properties, result);
			return result.ToString();
		}

		/// <summary>
		/// Renders a format string to the end of a string builder
		/// </summary>
		/// <param name="format">The format string to render</param>
		/// <param name="properties">Sequence of key/value properties</param>
		/// <param name="result">Buffer to append the rendered string to</param>
		public static void Render(string format, IEnumerable<KeyValuePair<string, object?>>? properties, StringBuilder result)
		{
			int nextOffset = 0;

			List<(int, int)>? names = ParsePropertyNames(format);
			if (names != null && properties != null)
			{
				foreach ((int offset, int length) in names)
				{
					ReadOnlySpan<char> argument = format.AsSpan(offset, length);

					string? formatSpecifier = null;

					// Parse the format specifier
					int formatSpecifierIdx = argument.IndexOf(':');
					if (formatSpecifierIdx != -1)
					{
						formatSpecifier = argument.Slice(formatSpecifierIdx + 1).ToString();
						argument = argument.Slice(0, formatSpecifierIdx);
					}

					// Parse the property width
					int alignment = 0;

					int alignmentIdx = argument.IndexOf(',');
					if (alignmentIdx != -1 && Int32.TryParse(argument.Slice(alignmentIdx + 1), NumberStyles.Integer | NumberStyles.AllowLeadingSign, null, out alignment))
					{
						argument = argument.Slice(0, alignmentIdx);
					}

					// Try to get the property value
					object? value;
					if (TryGetPropertyValue(argument, properties, out value))
					{
						// Append the text up to this argument
						int startOffset = offset - 1;
						if (format[startOffset] == '@' || format[startOffset] == '$')
						{
							startOffset--;
						}
						Unescape(format.AsSpan(nextOffset, startOffset - nextOffset), result);

						// Render the property
						string? rendered;
						if (format[offset] == '@')
						{
							rendered = JsonSerializer.Serialize(value, value?.GetType() ?? typeof(object), new JsonSerializerOptions { PropertyNamingPolicy = JsonNamingPolicy.CamelCase });
						}
						else if (formatSpecifier != null && value is IFormattable formattable)
						{
							rendered = formattable.ToString(formatSpecifier, null);
						}
						else
						{
							rendered = value?.ToString() ?? "null";
						}

						// Apply a postive width adjustment (right align)
						int leftPad = Math.Max(alignment - rendered.Length, 0);
						if (leftPad > 0)
						{
							result.Append(' ', leftPad);
						}

						// Append the argument
						result.Append(rendered);

						// Apply a negative width adjustment (left align)
						int rightPad = Math.Max(-alignment - rendered.Length, 0);
						if (rightPad > 0)
						{
							result.Append(' ', rightPad);
						}

						// Start the next plain-text run after the closing brace
						nextOffset = offset + length + 1;
					}
				}
			}

			Unescape(format.AsSpan(nextOffset, format.Length - nextOffset), result);
		}

		/// <summary>
		/// Escapes a string for use in a message template
		/// </summary>
		/// <param name="text">Text to escape</param>
		/// <returns>The escaped string</returns>
		public static string Escape(string text)
		{
			StringBuilder result = new();
			Escape(text, result);
			return result.ToString();
		}

		/// <summary>
		/// Escapes a span of characters and appends the result to a string
		/// </summary>
		/// <param name="text">Span of characters to escape</param>
		/// <param name="result">Buffer to receive the escaped string</param>
		public static void Escape(ReadOnlySpan<char> text, StringBuilder result)
		{
			foreach (char character in text)
			{
				result.Append(character);
				if (character == '{' || character == '}')
				{
					result.Append(character);
				}
			}
		}

		/// <summary>
		/// Unescapes a string from a message template
		/// </summary>
		/// <param name="text">The text to unescape</param>
		/// <returns>The unescaped text</returns>
		public static string Unescape(string text)
		{
			StringBuilder result = new();
			Unescape(text.AsSpan(), result);
			return result.ToString();
		}

		/// <summary>
		/// Unescape a string and append the result to a string builder
		/// </summary>
		/// <param name="text">Text to unescape</param>
		/// <param name="result">Receives the unescaped text</param>
		public static void Unescape(ReadOnlySpan<char> text, StringBuilder result)
		{
			char lastChar = '\0';
			foreach (char character in text)
			{
				if ((character != '{' && character != '}') || character != lastChar)
				{
					result.Append(character);
				}
				lastChar = character;
			}
		}

		/// <summary>
		/// Finds locations of property names from the given format string
		/// </summary>
		/// <param name="format">The format string to parse</param>
		/// <returns>List of offset, length pairs for property names. Null if the string does not contain any property references.</returns>
		public static List<(int, int)>? ParsePropertyNames(string format)
		{
			List<(int, int)>? names = null;
			for (int idx = 0; idx < format.Length - 1; idx++)
			{
				if (format[idx] == '{')
				{
					if (format[idx + 1] == '{')
					{
						idx++;
					}
					else
					{
						int startIdx = idx + 1;

						idx = format.IndexOf('}', startIdx);
						if (idx == -1)
						{
							break;
						}
						names ??= [];

						names.Add((startIdx, idx - startIdx));
					}
				}
			}
			return names;
		}

		/// <summary>
		/// Parse the ordered arguments into a dictionary of named properties
		/// </summary>
		/// <param name="format">Format string</param>
		/// <param name="args">Argument list to parse</param>
		/// <param name="properties"></param>
		/// <returns></returns>
		public static void ParsePropertyValues(string format, object[] args, Dictionary<string, object?> properties)
		{
			List<(int, int)>? offsets = ParsePropertyNames(format);
			if (offsets != null)
			{
				for (int idx = 0; idx < offsets.Count; idx++)
				{
					string name = format.Substring(offsets[idx].Item1, offsets[idx].Item2);

					// Parse for format specifier
					int specifierIdx = name.IndexOf(':', 0);
					if (specifierIdx != -1)
					{
						name = name.Substring(0, specifierIdx);
					}

					int number;
					if (Int32.TryParse(name, out number))
					{
						if (number >= 0 && number < args.Length)
						{
							properties[name] = args[number];
						}
					}
					else
					{
						if (idx < args.Length)
						{
							properties[name] = args[idx];
						}
					}
				}
			}
		}

		/// <summary>
		/// Attempts to get a named property value from the given dictionary
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <param name="properties">Sequence of property name/value pairs</param>
		/// <param name="value">On success, receives the property value</param>
		/// <returns>True if the property was found, false otherwise</returns>
		public static bool TryGetPropertyValue(ReadOnlySpan<char> name, IEnumerable<KeyValuePair<string, object?>> properties, out object? value)
		{
			int number;
			if (Int32.TryParse(name, NumberStyles.Integer, null, out number))
			{
				foreach (KeyValuePair<string, object?> property in properties)
				{
					if (number == 0)
					{
						value = property.Value;
						return true;
					}
					number--;
				}
			}
			else
			{
				foreach (KeyValuePair<string, object?> property in properties)
				{
					ReadOnlySpan<char> parameterName = property.Key.AsSpan();
					if (name.Equals(parameterName, StringComparison.Ordinal))
					{
						value = property.Value;
						return true;
					}
				}
			}

			value = null;
			return false;
		}
	}
}
