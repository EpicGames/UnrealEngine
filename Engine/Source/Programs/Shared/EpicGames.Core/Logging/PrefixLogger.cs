// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// ILogger implementation that adds a prefix to the start of each line
	/// </summary>
	public class PrefixLogger : ILogger
	{
		readonly string _prefix;
		readonly ILogger _inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="prefix">Prefix to insert</param>
		/// <param name="inner">Logger to write to</param>
		public PrefixLogger(string prefix, ILogger inner)
		{
			_prefix = prefix;
			_inner = inner;
		}

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _inner.BeginScope<TState>(state);

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => _inner.IsEnabled(logLevel);

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			if (state is IEnumerable<KeyValuePair<string, object>> enumerable)
			{
				List<KeyValuePair<string, object>> copy = [.. enumerable];

				int idx = copy.FindIndex(x => x.Key.Equals("{OriginalFormat}", StringComparison.OrdinalIgnoreCase));
				if (idx != -1 && copy[idx].Value is string format)
				{
					copy[idx] = new KeyValuePair<string, object>(copy[idx].Key, "{_tag} " + format);
					copy.Add(new KeyValuePair<string, object>("_tag", _prefix));
					_inner.Log(logLevel, eventId, copy, exception, (s, e) => $"{_prefix} {formatter(state, exception)}");
					return;
				}
			}
			_inner.Log(logLevel, eventId, state, exception, formatter);
		}
	}
}
