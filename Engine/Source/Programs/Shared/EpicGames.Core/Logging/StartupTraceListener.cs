// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Captures all log output during startup until a log file writer has been created
	/// </summary>
	internal class StartupTraceListener : TraceListener
	{
		readonly StringBuilder _buffer = new();

		/// <summary>
		/// Copy the contents of the buffered output to another trace listener
		/// </summary>
		/// <param name="other">The trace listener to receive the buffered output</param>
		public void CopyTo(TraceListener other)
		{
			foreach (string line in _buffer.ToString().Split("\n"))
			{
				other.WriteLine(line);
			}
		}

		/// <summary>
		/// Write a message to the buffer
		/// </summary>
		/// <param name="message">The message to write</param>
		public override void Write(string? message)
		{
			if (NeedIndent)
			{
				WriteIndent();
			}
			_buffer.Append(message);
		}

		/// <summary>
		/// Write a message to the buffer, followed by a newline
		/// </summary>
		/// <param name="message">The message to write</param>
		public override void WriteLine(string? message)
		{
			Write(message);
			_buffer.Append('\n');
		}
	}
}
