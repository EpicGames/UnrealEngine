// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using Microsoft.Extensions.ObjectPool;
using Microsoft.Win32.SafeHandles;

namespace EpicGames.Core
{
	/// <summary>
	/// Flags for the managed process
	/// </summary>
	[Flags]
	public enum ManagedProcessFlags
	{
		/// <summary>
		/// No flags 
		/// </summary>
		None = 0,

		/// <summary>
		/// Merge stdout and stderr
		/// </summary>
		MergeOutputPipes = 1,
	}

	/// <summary>
	/// Tracks a set of processes, and destroys them when the object is disposed.
	/// </summary>
	public sealed class ManagedProcessGroup : IDisposable
	{
#pragma warning disable IDE0049 // Naming Styles
#pragma warning disable IDE1006 // Naming Styles
		const uint JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE = 0x00002000;
		const uint JOB_OBJECT_LIMIT_BREAKAWAY_OK = 0x00000800;

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_BASIC_LIMIT_INFORMATION
		{
			public Int64 PerProcessUserTimeLimit;
			public Int64 PerJobUserTimeLimit;
			public UInt32 LimitFlags;
			public UIntPtr MinimumWorkingSetSize;
			public UIntPtr MaximumWorkingSetSize;
			public UInt32 ActiveProcessLimit;
			public Int64 Affinity;
			public UInt32 PriorityClass;
			public UInt32 SchedulingClass;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct IO_COUNTERS
		{
			public UInt64 ReadOperationCount;
			public UInt64 WriteOperationCount;
			public UInt64 OtherOperationCount;
			public UInt64 ReadTransferCount;
			public UInt64 WriteTransferCount;
			public UInt64 OtherTransferCount;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_EXTENDED_LIMIT_INFORMATION
		{
			public JOBOBJECT_BASIC_LIMIT_INFORMATION BasicLimitInformation;
			public IO_COUNTERS IoInfo;
			public UIntPtr ProcessMemoryLimit;
			public UIntPtr JobMemoryLimit;
			public UIntPtr PeakProcessMemoryUsed;
			public UIntPtr PeakJobMemoryUsed;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_BASIC_ACCOUNTING_INFORMATION
		{
			public UInt64 TotalUserTime;
			public UInt64 TotalKernelTime;
			public UInt64 ThisPeriodTotalUserTime;
			public UInt64 ThisPeriodTotalKernelTime;
			public UInt32 TotalPageFaultCount;
			public UInt32 TotalProcesses;
			public UInt32 ActiveProcesses;
			public UInt32 TotalTerminatedProcesses;
		}

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern SafeFileHandle CreateJobObject(IntPtr securityAttributes, IntPtr name);

		const int JobObjectBasicAccountingInformation = 1;
		const int JobObjectExtendedLimitInformation = 9;

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int SetInformationJobObject(SafeFileHandle hJob, int jobObjectInfoClass, IntPtr lpJobObjectInfo, int cbJobObjectInfoLength);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool QueryInformationJobObject(SafeFileHandle hJob, int JobObjectInformationClass, out JOBOBJECT_BASIC_ACCOUNTING_INFORMATION lpJobObjectInformation, int cbJobObjectInformationLength, out int lpReturnLength);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool QueryInformationJobObject(SafeFileHandle hJob, int JobObjectInformationClass, out JOBOBJECT_EXTENDED_LIMIT_INFORMATION lpJobObjectInformation, int cbJobObjectInformationLength, out int lpReturnLength);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int AssignProcessToJobObject(SafeFileHandle? hJob, IntPtr hProcess);

		[DllImport("kernel32.dll")]
		static extern bool IsProcessInJob(IntPtr hProcess, IntPtr hJob, out bool result);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern IntPtr GetCurrentProcess();
#pragma warning restore IDE1006 // Naming Styles
#pragma warning restore IDE0049 // Naming Styles

		/// <summary>
		/// Handle to the native job object that this process is added to. This handle is closed by the Dispose() method (and will automatically be closed by the OS on process exit),
		/// resulting in the child process being killed.
		/// </summary>
		public SafeFileHandle? JobHandle
		{
			get;
			private set;
		}

		/// <summary>
		/// Determines support for using job objects
		/// </summary>
		internal static bool SupportsJobObjects => OperatingSystem.IsWindows();

		/// <summary>
		/// Constructor
		/// </summary>
		public ManagedProcessGroup(bool bKillOnJobClose = true)
		{
			if (SupportsJobObjects)
			{
				// Create the job object that the child process will be added to
				JobHandle = CreateJobObject(IntPtr.Zero, IntPtr.Zero);
				if (JobHandle == null)
				{
					throw new Win32Exception();
				}

				// Configure the job object to terminate the processes added to it when the handle is closed
				JOBOBJECT_EXTENDED_LIMIT_INFORMATION limitInformation = new();
				limitInformation.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_BREAKAWAY_OK;
				if (bKillOnJobClose)
				{
					limitInformation.BasicLimitInformation.LimitFlags |= JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE;
				}

				int length = Marshal.SizeOf(typeof(JOBOBJECT_EXTENDED_LIMIT_INFORMATION));
				IntPtr limitInformationPtr = Marshal.AllocHGlobal(length);
				Marshal.StructureToPtr(limitInformation, limitInformationPtr, false);

				if (SetInformationJobObject(JobHandle, JobObjectExtendedLimitInformation, limitInformationPtr, length) == 0)
				{
					throw new Win32Exception();
				}
			}
		}

		/// <summary>
		/// Adds a process to this group
		/// </summary>
		/// <param name="process">Process to add</param>
		public void AddProcess(Process process) => AddProcess(process.Handle);

		/// <summary>
		/// Adds a process to this group
		/// </summary>
		/// <param name="processHandle">Handle to the process to add</param>
		public void AddProcess(IntPtr processHandle)
		{
			if (SupportsJobObjects && AssignProcessToJobObject(JobHandle, processHandle) == 0)
			{
				// Support for nested job objects was only added in Windows 8; prior to that, assigning processes to job objects would fail. Figure out if we're already in a job, and ignore the error if we are.
				int originalError = Marshal.GetLastWin32Error();

				bool bProcessInJob;
				IsProcessInJob(GetCurrentProcess(), IntPtr.Zero, out bProcessInJob);

				if (!bProcessInJob)
				{
					throw new Win32ExceptionWithCode(originalError, "Unable to assign process to job object");
				}
			}
		}

		/// <summary>
		/// Returns the total CPU time usage for this job.
		/// </summary>
		public TimeSpan TotalProcessorTime =>
			SupportsJobObjects && JobHandle != null && QueryInformationJobObject(JobHandle, JobObjectBasicAccountingInformation, out JOBOBJECT_BASIC_ACCOUNTING_INFORMATION accountingInformation, Marshal.SizeOf(typeof(JOBOBJECT_BASIC_ACCOUNTING_INFORMATION)), out _)
				? new TimeSpan((long)accountingInformation.TotalUserTime + (long)accountingInformation.TotalKernelTime)
				: new TimeSpan();

		/// <summary>
		/// Returns the the peak memory for a process for this job.
		/// </summary>
		public long PeakProcessMemoryUsed =>
			SupportsJobObjects && JobHandle != null && QueryInformationJobObject(JobHandle, JobObjectExtendedLimitInformation, out JOBOBJECT_EXTENDED_LIMIT_INFORMATION accountingInformation, Marshal.SizeOf(typeof(JOBOBJECT_EXTENDED_LIMIT_INFORMATION)), out _)
				? (long)accountingInformation.PeakProcessMemoryUsed
				: 0;

		/// <summary>
		/// Returns the the peak memory for this job.
		/// </summary>
		public long PeakJobMemoryUsed =>
			SupportsJobObjects && JobHandle != null && QueryInformationJobObject(JobHandle, JobObjectExtendedLimitInformation, out JOBOBJECT_EXTENDED_LIMIT_INFORMATION accountingInformation, Marshal.SizeOf(typeof(JOBOBJECT_EXTENDED_LIMIT_INFORMATION)), out _)
				? (long)accountingInformation.PeakJobMemoryUsed
				: 0;

		/// <summary>
		/// Dispose of the process group
		/// </summary>
		public void Dispose()
		{
			JobHandle?.Dispose();
			JobHandle = null;
		}
	}

	/// <summary>
	/// Encapsulates a managed child process, from which we can read the console output.
	/// Uses job objects to ensure that the process will be terminated automatically by the O/S if the current process is terminated, and polls pipe reads to avoid blocking on unmanaged code
	/// if the calling thread is terminated. Currently only implemented for Windows; makes heavy use of P/Invoke.
	/// </summary>
	public sealed class ManagedProcess : IDisposable
	{
#pragma warning disable IDE0049 // Naming Styles
#pragma warning disable IDE1006 // Naming Styles
		[StructLayout(LayoutKind.Sequential)]
		class SECURITY_ATTRIBUTES
		{
			public int nLength;
			public IntPtr lpSecurityDescriptor;
			public int bInheritHandle;
		}

		[StructLayout(LayoutKind.Sequential)]
		class PROCESS_INFORMATION
		{
			public IntPtr hProcess;
			public IntPtr hThread;
			public uint dwProcessId;
			public uint dwThreadId;
		}

		[StructLayout(LayoutKind.Sequential)]
		class STARTUPINFO
		{
			public int cb;
			public string? lpReserved;
			public string? lpDesktop;
			public string? lpTitle;
			public uint dwX;
			public uint dwY;
			public uint dwXSize;
			public uint dwYSize;
			public uint dwXCountChars;
			public uint dwYCountChars;
			public uint dwFillAttribute;
			public uint dwFlags;
			public short wShowWindow;
			public short cbReserved2;
			public IntPtr lpReserved2;
			public SafeHandle? hStdInput;
			public SafeHandle? hStdOutput;
			public SafeHandle? hStdError;
		}

		[StructLayout(LayoutKind.Sequential)]
		class STARTUPINFOEX
		{
			public STARTUPINFO? StartupInfo;
			public IntPtr lpAttributeList;
		}

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int CloseHandle(IntPtr hObject);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int CreatePipe(out SafeFileHandle hReadPipe, out SafeFileHandle hWritePipe, SECURITY_ATTRIBUTES lpPipeAttributes, uint nSize);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int SetHandleInformation(SafeFileHandle hObject, int dwMask, int dwFlags);

		const int HANDLE_FLAG_INHERIT = 1;

		const int STARTF_USESTDHANDLES = 0x00000100;

		[Flags]
		enum ProcessCreationFlags : int
		{
			CREATE_NO_WINDOW = 0x08000000,
			CREATE_SUSPENDED = 0x00000004,
			NORMAL_PRIORITY_CLASS = 0x00000020,
			IDLE_PRIORITY_CLASS = 0x00000040,
			HIGH_PRIORITY_CLASS = 0x00000080,
			REALTIME_PRIORITY_CLASS = 0x00000100,
			BELOW_NORMAL_PRIORITY_CLASS = 0x00004000,
			ABOVE_NORMAL_PRIORITY_CLASS = 0x00008000,
			EXTENDED_STARTUPINFO_PRESENT = 0x00080000
		}

#pragma warning disable CA1838 // Avoid 'StringBuilder' parameters for P/Invokes
		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int CreateProcess(/*[MarshalAs(UnmanagedType.LPTStr)]*/ string? lpApplicationName, StringBuilder lpCommandLine, IntPtr lpProcessAttributes, IntPtr lpThreadAttributes, bool bInheritHandles, ProcessCreationFlags dwCreationFlags, IntPtr lpEnvironment, /*[MarshalAs(UnmanagedType.LPTStr)]*/ string? lpCurrentDirectory, STARTUPINFOEX lpStartupInfo, PROCESS_INFORMATION lpProcessInformation);
#pragma warning restore CA1838 // Avoid 'StringBuilder' parameters for P/Invokes

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int ResumeThread(IntPtr hThread);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern IntPtr GetCurrentProcess();

		const int DUPLICATE_SAME_ACCESS = 2;

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int DuplicateHandle(
			IntPtr hSourceProcessHandle,
			SafeHandle hSourceHandle,
			IntPtr hTargetProcess,
			out SafeFileHandle targetHandle,
			int dwDesiredAccess,
			[MarshalAs(UnmanagedType.Bool)] bool bInheritHandle,
			int dwOptions
		);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int GetExitCodeProcess(SafeProcessHandle hProcess, out int lpExitCode);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int TerminateProcess(SafeHandleZeroOrMinusOneIsInvalid hProcess, uint uExitCode);

		const uint INFINITE = 0xFFFFFFFF;
		const uint WAIT_FAILED = 0xFFFFFFFF;

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern uint WaitForSingleObject(SafeHandleZeroOrMinusOneIsInvalid hHandle, uint dwMilliseconds);

		[DllImport("kernel32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool GetProcessTimes(SafeHandleZeroOrMinusOneIsInvalid hProcess,
			out System.Runtime.InteropServices.ComTypes.FILETIME lpCreationTime,
			out System.Runtime.InteropServices.ComTypes.FILETIME lpExitTime,
			out System.Runtime.InteropServices.ComTypes.FILETIME lpKernelTime,
			out System.Runtime.InteropServices.ComTypes.FILETIME lpUserTime);

		[DllImport("kernel32.dll")]
		static extern ushort GetActiveProcessorGroupCount();

		[DllImport("kernel32.dll")]
		static extern uint GetActiveProcessorCount(ushort GroupNumber);

		[StructLayout(LayoutKind.Sequential)]
		class GROUP_AFFINITY
		{
			public ulong Mask;
			public ushort Group;
			public ushort Reserved0;
			public ushort Reserved1;
			public ushort Reserved2;
		}

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int SetThreadGroupAffinity(IntPtr hThread, GROUP_AFFINITY GroupAffinity, GROUP_AFFINITY? PreviousGroupAffinity);

		const int ERROR_ACCESS_DENIED = 5;
#pragma warning restore IDE1006 // Naming Styles
#pragma warning restore IDE0049 // Naming Styles

		/// <summary>
		/// Converts FILETIME to DateTime.
		/// </summary>
		/// <param name="time">Input FILETIME structure</param>
		/// <returns>Converted DateTime</returns>
		static DateTime FileTimeToDateTime(System.Runtime.InteropServices.ComTypes.FILETIME time)
		{
			ulong high = (ulong)time.dwHighDateTime;
			uint low = (uint)time.dwLowDateTime;
			long fileTime = (long)((high << 32) + low);
			try
			{
				return DateTime.FromFileTimeUtc(fileTime);
			}
			catch (ArgumentOutOfRangeException)
			{
				return DateTime.MinValue;
			}
		}

		/// <summary>
		/// The process id
		/// </summary>
		public int Id { get; private set; }

		/// <summary>
		/// Handle for the child process.
		/// </summary>
		SafeProcessHandle? _processHandle;

		/// <summary>
		/// The write end of the child process' stdin pipe.
		/// </summary>
		SafeFileHandle? _stdInWrite;

		/// <summary>
		/// The read end of the child process' stdout pipe.
		/// </summary>
		SafeFileHandle? _stdOutRead;

		/// <summary>
		/// The read end of the child process' stderr pipe.
		/// </summary>
		SafeFileHandle? _stdErrRead;

		/// <summary>
		/// Input stream for the child process.
		/// </summary>
		public Stream StdIn { get; private set; } = null!;

		/// <summary>
		/// Output stream for the child process.
		/// </summary>
		public Stream StdOut { get; private set; } = null!;

		/// <summary>
		/// Reader for the process' output stream.
		/// </summary>
		public StreamReader StdOutText { get; private set; } = null!;

		/// <summary>
		/// Output stream for the child process.
		/// </summary>
		public Stream StdErr { get; private set; } = null!;

		/// <summary>
		/// Reader for the process' output stream.
		/// </summary>
		public StreamReader StdErrText { get; private set; } = null!;

		/// <summary>
		/// Standard process implementation for non-Windows platforms.
		/// </summary>
		Process? _frameworkProcess;

		/// <summary>
		/// Task to read from stdout and stderr
		/// </summary>
		Task? _frameworkOutputTask;

		/// <summary>
		/// Merged output and error write stream for the framework child process. We use a channel rather than a pipe here to support cancellation, which is not supported by async pipes on Windows.
		/// </summary>
		Channel<ChannelBuffer>? _frameworkChannel;

		/// <summary>
		/// Pool of buffer objects output from the framework
		/// </summary>
		ObjectPool<ChannelBuffer>? _frameworkBufferPool;

		/// <summary>
		/// Cancellation token for background threads
		/// </summary>
#pragma warning disable CA2213
		CancellationTokenSource? _frameworkCancellationSource;
#pragma warning restore CA2213

		/// <summary>
		/// Static lock object. This is used to synchronize the creation of child processes - in particular, the inheritance of stdout/stderr write pipes. If processes
		/// inherit pipes meant for other processes, they won't be closed until both terminate.
		/// </summary>
		static readonly Lock s_lockObject = new();

		/// <summary>
		/// Used to perform CPU usage and resource accounting of all children process involved in a single compilation unit.
		/// </summary>
		ManagedProcessGroup? _accountingProcessGroup;

		byte[] _buffer = [];
		int _bufferPos = 0;
		int _bufferLen = 0;
		bool _bCarriageReturn = false;

		class ChannelBuffer
		{
			public byte[] _data = new byte[1024];
			public int _length = 0;
		}

		class ChannelReadStream : Stream
		{
			readonly ChannelReader<ChannelBuffer> _reader;
			readonly ObjectPool<ChannelBuffer> _bufferPool;

			ChannelBuffer? _currentBuf = null;
			int _currentPos = 0;

			public ChannelReadStream(ChannelReader<ChannelBuffer> reader, ObjectPool<ChannelBuffer> bufferPool)
			{
				_reader = reader;
				_bufferPool = bufferPool;
			}

			public override bool CanRead => true;
			public override bool CanSeek => false;
			public override bool CanWrite => false;
			public override long Length => throw new NotSupportedException();
			public override long Position { get => throw new NotSupportedException(); set => throw new NotSupportedException(); }
			public override void Flush() { }
			public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();
			public override void SetLength(long value) => throw new NotSupportedException();
			public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();

			public override int Read(byte[] buffer, int offset, int count)
			{
				// Perform the read on the thread pool to avoid deadlocking the continuation pathway on WinForms apps
				Task<int> readTask = Task.Run(async () => await ReadAsync(buffer.AsMemory(offset, count), CancellationToken.None));
				return readTask.GetAwaiter().GetResult();
			}

			public override async ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken)
			{
				if (_currentBuf != null && _currentPos == _currentBuf._length)
				{
					_bufferPool.Return(_currentBuf);
					_currentBuf = null;
					_currentPos = 0;
				}

				while (_currentBuf == null)
				{
					if (!await _reader.WaitToReadAsync(cancellationToken))
					{
						return 0;
					}
					if (_reader.TryRead(out _currentBuf))
					{
						break;
					}
				}

				int readLen = Math.Min(_currentBuf._length - _currentPos, buffer.Length);
				_currentBuf._data.AsSpan(_currentPos, readLen).CopyTo(buffer.Span);
				_currentPos += readLen;
				return readLen;
			}
		}

		/// <summary>
		/// Spawns a new managed process.
		/// </summary>
		/// <param name="group">The managed process group to add to</param>
		/// <param name="fileName">Path to the executable to be run</param>
		/// <param name="commandLine">Command line arguments for the process</param>
		/// <param name="workingDirectory">Working directory for the new process. May be null to use the current working directory.</param>
		/// <param name="environment">Environment variables for the new process. May be null, in which case the current process' environment is inherited</param>
		/// <param name="input">Text to be passed via stdin to the new process. May be null.</param>
		/// <param name="priority">Priority for the child process</param>
		/// <param name="appContainer">AppContainer to use for process (only available on Windows)</param>
		/// <param name="flags">Flags for the process</param>
		public ManagedProcess(ManagedProcessGroup? group, string fileName, string commandLine, string? workingDirectory, IReadOnlyDictionary<string, string>? environment, byte[]? input, ProcessPriorityClass priority, AppContainer? appContainer = null, ManagedProcessFlags flags = ManagedProcessFlags.MergeOutputPipes)
			: this(group, fileName, commandLine, workingDirectory, environment, priority, appContainer, flags)
		{
			if (input != null)
			{
				StdIn.Write(input, 0, input.Length);
			}
			StdIn.Close();
		}

		/// <summary>
		/// Spawns a new managed process.
		/// </summary>
		/// <param name="group">The managed process group to add to</param>
		/// <param name="fileName">Path to the executable to be run</param>
		/// <param name="commandLine">Command line arguments for the process</param>
		/// <param name="workingDirectory">Working directory for the new process. May be null to use the current working directory.</param>
		/// <param name="environment">Environment variables for the new process. May be null, in which case the current process' environment is inherited</param>
		/// <param name="priority">Priority for the child process</param>
		/// <param name="appContainer">AppContainer to use for process (only available on Windows)</param>
		/// <param name="flags">Flags for the process</param>
		public ManagedProcess(ManagedProcessGroup? group, string fileName, string commandLine, string? workingDirectory, IReadOnlyDictionary<string, string>? environment, ProcessPriorityClass priority, AppContainer? appContainer = null, ManagedProcessFlags flags = ManagedProcessFlags.MergeOutputPipes)
		{
			// Create the child process
			// NOTE: Child process must be created in a separate method to avoid stomping exception callstacks (https://stackoverflow.com/a/2494150)
			try
			{
				if (ManagedProcessGroup.SupportsJobObjects)
				{
					CreateManagedProcessWin32(group, fileName, commandLine, workingDirectory, environment, priority, appContainer, flags);
				}
				else
				{
					CreateManagedProcessPortable(fileName, commandLine, workingDirectory, environment, priority, flags);
				}
			}
			catch (Exception ex)
			{
				ExceptionUtils.AddContext(ex, "while launching {0} {1}", fileName, commandLine);
				throw;
			}
		}

		static readonly int s_processorGroupCount = GetActiveProcessorGroupCount();

		/// <summary>
		/// A counter used to decide which processor group the next process should be assigned to
		/// </summary>
		static int s_processorGroupCounter = 0;

		/// <summary>
		/// Spawns a new managed process using Win32 native functions. 
		/// </summary>
		/// <param name="group">The managed process group to add to</param>
		/// <param name="fileName">Path to the executable to be run</param>
		/// <param name="commandLine">Command line arguments for the process</param>
		/// <param name="workingDirectory">Working directory for the new process. May be null to use the current working directory.</param>
		/// <param name="environment">Environment variables for the new process. May be null, in which case the current process' environment is inherited</param>
		/// <param name="priority">Priority for the child process</param>
		/// <param name="appContainer">AppContainer to use for the new process</param>
		/// <param name="managedFlags">Flags controlling how the new process is created</param>
		private void CreateManagedProcessWin32(ManagedProcessGroup? group, string fileName, string commandLine, string? workingDirectory, IReadOnlyDictionary<string, string>? environment, ProcessPriorityClass priority, AppContainer? appContainer, ManagedProcessFlags? managedFlags)
		{
			IntPtr environmentBlock = IntPtr.Zero;
			try
			{
				// Create the environment block for the child process, if necessary.
				if (environment != null)
				{
					// The native format for the environment block is a sequence of null terminated strings with a final null terminator.
					List<byte> environmentBytes = [];
					foreach (KeyValuePair<string, string> pair in environment)
					{
						environmentBytes.AddRange(Console.OutputEncoding.GetBytes(pair.Key));
						environmentBytes.Add((byte)'=');
						environmentBytes.AddRange(Console.OutputEncoding.GetBytes(pair.Value));
						environmentBytes.Add((byte)0);
					}
					environmentBytes.Add((byte)0);

					// Allocate an unmanaged block of memory to store it.
					environmentBlock = Marshal.AllocHGlobal(environmentBytes.Count);
					Marshal.Copy(environmentBytes.ToArray(), 0, environmentBlock, environmentBytes.Count);
				}

				PROCESS_INFORMATION processInfo = new();
				try
				{
					// Get the flags to create the new process
					ProcessCreationFlags flags = ProcessCreationFlags.CREATE_NO_WINDOW | ProcessCreationFlags.CREATE_SUSPENDED;
					switch (priority)
					{
						case ProcessPriorityClass.Normal:
							flags |= ProcessCreationFlags.NORMAL_PRIORITY_CLASS;
							break;
						case ProcessPriorityClass.Idle:
							flags |= ProcessCreationFlags.IDLE_PRIORITY_CLASS;
							break;
						case ProcessPriorityClass.High:
							flags |= ProcessCreationFlags.HIGH_PRIORITY_CLASS;
							break;
						case ProcessPriorityClass.RealTime:
							flags |= ProcessCreationFlags.REALTIME_PRIORITY_CLASS;
							break;
						case ProcessPriorityClass.BelowNormal:
							flags |= ProcessCreationFlags.BELOW_NORMAL_PRIORITY_CLASS;
							break;
						case ProcessPriorityClass.AboveNormal:
							flags |= ProcessCreationFlags.ABOVE_NORMAL_PRIORITY_CLASS;
							break;
					}

					// Acquire a global lock before creating inheritable handles. If multiple threads create inheritable handles at the same time, and child processes will inherit them all.
					// Since we need to wait for output pipes to be closed (in order to consume all output), this can result in output reads not returning until all processes with the same
					// inherited handles are closed.
					lock (s_lockObject)
					{
						SafeFileHandle? stdInRead = null;
						SafeFileHandle? stdOutWrite = null;
						SafeFileHandle? stdErrWrite = null;
						try
						{
							// Create stdin and stdout pipes for the child process. We'll close the handles for the child process' ends after it's been created.
							SECURITY_ATTRIBUTES securityAttributes = new();
							securityAttributes.nLength = Marshal.SizeOf(securityAttributes);
							securityAttributes.bInheritHandle = 1;

							if (CreatePipe(out stdInRead, out _stdInWrite, securityAttributes, 4 * 1024) == 0 || SetHandleInformation(_stdInWrite, HANDLE_FLAG_INHERIT, 0) == 0)
							{
								throw new Win32ExceptionWithCode("Unable to create stdin pipe");
							}
							if (CreatePipe(out _stdOutRead, out stdOutWrite, securityAttributes, 1024 * 1024) == 0 || SetHandleInformation(_stdOutRead, HANDLE_FLAG_INHERIT, 0) == 0)
							{
								throw new Win32ExceptionWithCode("Unable to create stdout pipe");
							}

							if ((managedFlags & ManagedProcessFlags.MergeOutputPipes) != 0)
							{
								if (DuplicateHandle(GetCurrentProcess(), stdOutWrite, GetCurrentProcess(), out stdErrWrite, 0, true, DUPLICATE_SAME_ACCESS) == 0)
								{
									throw new Win32ExceptionWithCode("Unable to duplicate stdout handle");
								}
							}
							else
							{
								if (CreatePipe(out _stdErrRead, out stdErrWrite, securityAttributes, 1024 * 1024) == 0 || SetHandleInformation(_stdErrRead, HANDLE_FLAG_INHERIT, 0) == 0)
								{
									throw new Win32ExceptionWithCode("Unable to create stderr pipe");
								}
							}

							// Create the new process as suspended, so we can modify it before it starts executing (and potentially preempting us)
							STARTUPINFO startupInfo = new();
							startupInfo.cb = Marshal.SizeOf(typeof(STARTUPINFOEX));
							startupInfo.hStdInput = stdInRead;
							startupInfo.hStdOutput = stdOutWrite;
							startupInfo.hStdError = stdErrWrite;
							startupInfo.dwFlags = STARTF_USESTDHANDLES;

							flags |= ProcessCreationFlags.EXTENDED_STARTUPINFO_PRESENT;
							STARTUPINFOEX startupInfoEx = new() { StartupInfo = startupInfo, lpAttributeList = IntPtr.Zero };

							if (appContainer != null && OperatingSystem.IsWindows())
							{
								startupInfoEx.lpAttributeList = appContainer.GetAttributeList();
							}

							// Under heavy load (ie. spawning large number of processes, typically Clang) we see CreateProcess very occasionally failing with ERROR_ACCESS_DENIED.
							int[] retryDelay = [100, 200, 1000, 5000];
							for (int attemptIdx = 0; ; attemptIdx++)
							{
								if (CreateProcess(null, new StringBuilder("\"" + fileName + "\" " + commandLine), IntPtr.Zero, IntPtr.Zero, true, flags, environmentBlock, workingDirectory, startupInfoEx, processInfo) != 0)
								{
									break;
								}

								if (Marshal.GetLastWin32Error() != ERROR_ACCESS_DENIED || attemptIdx >= retryDelay.Length)
								{
									throw new Win32ExceptionWithCode("Unable to create process");
								}

								Thread.Sleep(retryDelay[attemptIdx]);
							}

							// Save the process id
							Id = (int)processInfo.dwProcessId;
						}
						finally
						{
							// Close the write ends of the handle. We don't want any other process to be able to inherit these.
							stdInRead?.Dispose();
							stdInRead = null;
							stdOutWrite?.Dispose();
							stdOutWrite = null;
							stdErrWrite?.Dispose();
							stdErrWrite = null;
						}
					}

					// Add it to our job object
					group?.AddProcess(processInfo.hProcess);

					// Create a JobObject for each spawned process to do CPU usage accounting of spawned process and all its children
					_accountingProcessGroup = new ManagedProcessGroup(bKillOnJobClose: false);
					_accountingProcessGroup.AddProcess(processInfo.hProcess);

					// On systems with more than one processor group (more than one CPU socket, or more than 64 cores), it is possible
					// that processes launched from here may be scheduled by the operating system in a way that impedes overall throughput.
					//
					// From https://docs.microsoft.com/en-us/windows/win32/procthread/processor-groups
					// > The operating system initially assigns each process to a single group in a round-robin manner across the groups in the system
					//
					// When UBT launches a process here, it is not uncommon for more than one may be created (for example: clang and conhost)
					// If the number of processes created is the same as the number of processor groups in the system, this can lead
					// to high workload processes (like clang) predominantly assigned to one processor group, and lower workload processes
					// (like conhost) to the other.
					//
					// To reduce the chance of pathological process scheduling, we will explicitly distribute processes to each process
					// group. Doing so has been observed to reduce overall compile times for large builds by as much as 10%.
					if (s_processorGroupCount > 1)
					{
						ushort processorGroup = (ushort)(Interlocked.Increment(ref s_processorGroupCounter) % s_processorGroupCount);

						uint groupProcessorCount = GetActiveProcessorCount(processorGroup);

						GROUP_AFFINITY groupAffinity = new();
						groupAffinity.Mask = ~0ul >> (int)(64 - groupProcessorCount);
						groupAffinity.Group = processorGroup;
						if (SetThreadGroupAffinity(processInfo.hThread, groupAffinity, null) == 0)
						{
							throw new Win32Exception();
						}
					}

					// Allow the thread to start running
					if (ResumeThread(processInfo.hThread) == -1)
					{
						throw new Win32ExceptionWithCode("Unable to resume thread in child process");
					}

					// If we have any input text, write it to stdin now
					StdIn = new FileStream(_stdInWrite, FileAccess.Write, 4096, false);

					// Create the stream objects for reading the process output
					StdOut = new FileStream(_stdOutRead, FileAccess.Read, 4096, false);
					StdOutText = new StreamReader(StdOut, Console.OutputEncoding);

					// Do the same for the stderr output
					if (_stdErrRead != null)
					{
						StdErr = new FileStream(_stdErrRead, FileAccess.Read, 4096, false);
						StdErrText = new StreamReader(StdErr, Console.OutputEncoding);
					}
					else
					{
						StdErr = StdOut;
						StdErrText = StdOutText;
					}

					// Wrap the process handle in a SafeFileHandle
					_processHandle = new SafeProcessHandle(processInfo.hProcess, true);
				}
				finally
				{
					if (processInfo.hProcess != IntPtr.Zero && _processHandle == null)
					{
						using SafeFileHandle processHandle = new(processInfo.hProcess, true);
						_ = TerminateProcess(processHandle, UInt32.MaxValue);
					}
					if (processInfo.hThread != IntPtr.Zero)
					{
						_ = CloseHandle(processInfo.hThread);
					}
				}
			}
			finally
			{
				if (environmentBlock != IntPtr.Zero)
				{
					Marshal.FreeHGlobal(environmentBlock);
				}
			}
		}

		/// <summary>
		/// Spawns a new managed process using Win32 native functions. 
		/// </summary>
		/// <param name="fileName">Path to the executable to be run</param>
		/// <param name="commandLine">Command line arguments for the process</param>
		/// <param name="workingDirectory">Working directory for the new process. May be null to use the current working directory.</param>
		/// <param name="environment">Environment variables for the new process. May be null, in which case the current process' environment is inherited</param>
		/// <param name="priority">Priority for the child process</param>
		/// <param name="flags">Flags for the new process</param>
		private void CreateManagedProcessPortable(string fileName, string commandLine, string? workingDirectory, IReadOnlyDictionary<string, string>? environment, ProcessPriorityClass priority, ManagedProcessFlags flags)
		{
			// TODO: Process Arguments follow windows conventions in .NET Core
			// Which means single quotes ' are not considered quotes.
			// see https://github.com/dotnet/runtime/issues/29857
			// also see UE-102580
			// for rules see https://docs.microsoft.com/en-us/cpp/cpp/main-function-command-line-args
			commandLine = commandLine.Replace('\'', '\"');

			// for non-Windows platforms
			_frameworkProcess = new Process();
			_frameworkProcess.StartInfo.FileName = fileName;
			_frameworkProcess.StartInfo.Arguments = commandLine;
			_frameworkProcess.StartInfo.WorkingDirectory = workingDirectory;
			_frameworkProcess.StartInfo.RedirectStandardInput = true;
			_frameworkProcess.StartInfo.RedirectStandardOutput = true;
			_frameworkProcess.StartInfo.RedirectStandardError = true;
			_frameworkProcess.StartInfo.UseShellExecute = false;
			_frameworkProcess.StartInfo.CreateNoWindow = true;
			_frameworkProcess.Exited += (sender, e) =>
			{
				_frameworkExitTime = DateTime.Now;
			};

			if (environment != null)
			{
				foreach (KeyValuePair<string, string> pair in environment)
				{
					_frameworkProcess.StartInfo.EnvironmentVariables[pair.Key] = pair.Value;
				}
			}

			_frameworkStartTime = _frameworkExitTime = DateTime.Now;
			_frameworkProcess.Start();

			try
			{
				_frameworkProcess.PriorityClass = priority;
			}
			catch
			{
			}

			Id = _frameworkProcess.Id;
			StdIn = _frameworkProcess.StandardInput.BaseStream;

			if ((flags & ManagedProcessFlags.MergeOutputPipes) != 0)
			{
				// AnonymousPipes block reading even if the stream has been fully read, until the writer pipe handle is closed.
				_frameworkBufferPool = ObjectPool.Create<ChannelBuffer>();
				_frameworkChannel = Channel.CreateBounded<ChannelBuffer>(new BoundedChannelOptions(128) { FullMode = BoundedChannelFullMode.Wait, SingleReader = true, SingleWriter = false });
				_frameworkCancellationSource = new CancellationTokenSource();

				StdOut = new ChannelReadStream(_frameworkChannel.Reader, _frameworkBufferPool);
				StdOutText = new StreamReader(StdOut, Console.OutputEncoding);

				StdErr = StdOut;
				StdErrText = StdOutText;

				CancellationToken cancellationToken = _frameworkCancellationSource.Token;
				Task stdOutTask = Task.Run(() => CopyPipeAsync(_frameworkProcess.StandardOutput.BaseStream, _frameworkChannel, cancellationToken));
				Task stdErrTask = Task.Run(() => CopyPipeAsync(_frameworkProcess.StandardError.BaseStream, _frameworkChannel, cancellationToken));
				_frameworkOutputTask = Task.WhenAll(stdOutTask, stdErrTask).ContinueWith(x => _frameworkChannel.Writer.Complete(), TaskScheduler.Default);
			}
			else
			{
				StdOut = _frameworkProcess.StandardOutput.BaseStream;
				StdOutText = _frameworkProcess.StandardOutput;
				StdErr = _frameworkProcess.StandardError.BaseStream;
				StdErrText = _frameworkProcess.StandardError;
			}
		}

		/// <summary>
		/// Copy data from one pipe to another.
		/// </summary>
		/// <param name="source"></param>
		/// <param name="target"></param>
		/// <param name="cancellationToken">Cancellation token</param>
		async Task CopyPipeAsync(Stream source, ChannelWriter<ChannelBuffer> target, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				ChannelBuffer buffer = _frameworkBufferPool!.Get();
				int readLen = await source.ReadAsync(buffer._data, 0, buffer._data.Length, cancellationToken);

				if (readLen == 0)
				{
					break;
				}

				buffer._length = readLen;
				await target.WriteAsync(buffer, cancellationToken);
			}
		}

		/// <summary>
		/// Free the managed resources for this process
		/// </summary>
		public void Dispose()
		{
			if (_processHandle != null)
			{
				_ = TerminateProcess(_processHandle, 0);
				_ = WaitForSingleObject(_processHandle, INFINITE);

				_processHandle.Dispose();
				_processHandle = null;
			}

			_stdInWrite?.Dispose();
			_stdInWrite = null;
			_stdOutRead?.Dispose();
			_stdOutRead = null;

			StdIn?.Dispose();
			StdOut?.Dispose();
			StdOutText?.Dispose();
			StdErr?.Dispose();
			StdErrText?.Dispose();
			_accountingProcessGroup?.Dispose();

			if (_frameworkProcess != null)
			{
				_frameworkCancellationSource?.Cancel();

				if (!_frameworkProcess.HasExited)
				{
					// Kill entire process tree
					try
					{
						_frameworkProcess.Kill(true);
					}
					catch { }

					try
					{
						_frameworkProcess.WaitForExit();
					}
					catch { }
				}

				_frameworkProcess.Dispose();
				_frameworkProcess = null;

				_frameworkOutputTask?.ContinueWith(x => _frameworkCancellationSource?.Dispose(), TaskScheduler.Default);
			}
		}

		/// <summary>
		/// Reads data from the process output
		/// </summary>
		/// <param name="buffer">The buffer to receive the data</param>
		/// <param name="offset">Offset within the buffer to write to</param>
		/// <param name="count">Maximum number of bytes to read</param>
		/// <returns>Number of bytes read</returns>
		public int Read(byte[] buffer, int offset, int count)
		{
			// Fill the buffer, reentering managed code every 20ms to allow thread abort exceptions to be thrown
			Task<int> readTask = StdOut!.ReadAsync(buffer, offset, count);
			while (!readTask.Wait(20))
			{
				// Spin through managed code to allow things like ThreadAbortExceptions to be thrown.
			}
			return readTask.Result;
		}

		/// <summary>
		/// Reads data from the process output
		/// </summary>
		/// <param name="buffer">The buffer to receive the data</param>
		/// <param name="offset">Offset within the buffer to write to</param>
		/// <param name="count">Maximum number of bytes to read</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Number of bytes read</returns>
		public async ValueTask<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken)
		{
			return await StdOut!.ReadAsync(buffer, offset, count, cancellationToken);
		}

		/// <summary>
		/// Copy the process output to the given stream
		/// </summary>
		/// <param name="outputStream">The output stream</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns></returns>
		public Task CopyToAsync(Stream outputStream, CancellationToken cancellationToken)
		{
			return StdOut!.CopyToAsync(outputStream, cancellationToken);
		}

		/// <summary>
		/// Copy the process output to the given stream
		/// </summary>
		/// <param name="writeOutput">The output stream</param>
		/// <param name="bufferSize">Size of the buffer for copying</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns></returns>
		public Task CopyToAsync(Action<byte[], int, int> writeOutput, int bufferSize, CancellationToken cancellationToken)
		{
			Task WriteOutputAsync(byte[] buffer, int offset, int length, CancellationToken cancellationToken)
			{
				writeOutput(buffer, offset, length);
				return Task.CompletedTask;
			}
			return CopyToAsync(WriteOutputAsync, bufferSize, cancellationToken);
		}

		/// <summary>
		/// Copy the process output to the given stream
		/// </summary>
		/// <param name="writeOutputAsync">The output stream</param>
		/// <param name="bufferSize">Size of the buffer for copying</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns></returns>
		public async Task CopyToAsync(Func<byte[], int, int, CancellationToken, Task> writeOutputAsync, int bufferSize, CancellationToken cancellationToken)
		{
			TaskCompletionSource<bool> taskCompletionSource = new(TaskCreationOptions.RunContinuationsAsynchronously);
			using (CancellationTokenRegistration registration = cancellationToken.Register(() => taskCompletionSource.SetResult(false)))
			{
				byte[] buffer = new byte[bufferSize];
				for (; ; )
				{
					Task<int> readTask = StdOut.ReadAsync(buffer, 0, bufferSize, cancellationToken);
					_ = readTask.ContinueWith(x => _ = x.Exception, CancellationToken.None, TaskContinuationOptions.OnlyOnFaulted, TaskScheduler.Default);

					Task completedTask = await Task.WhenAny(readTask, taskCompletionSource.Task);
					cancellationToken.ThrowIfCancellationRequested();

					int bytes = await readTask;
					if (bytes == 0)
					{
						break;
					}
					await writeOutputAsync(buffer, 0, bytes, cancellationToken);
				}
			}
		}

		/// <summary>
		/// Reads an individual line from the process
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async ValueTask<string?> ReadLineAsync(CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				// Skip any '\n' that's part of a '\r\n' sequence
				if (_bCarriageReturn && _bufferPos < _bufferLen)
				{
					if (_buffer[_bufferPos] == '\n')
					{
						_bufferPos++;
					}
					_bCarriageReturn = false;
				}

				// Pull out the first complete output line
				for (int idx = _bufferPos; idx < _bufferLen; idx++)
				{
					byte character = _buffer[idx];
					if (character == '\r' || character == '\n')
					{
						string line = Console.OutputEncoding.GetString(_buffer, _bufferPos, idx - _bufferPos);
						_bCarriageReturn = (character == '\r');
						_bufferPos = idx + 1;
						return line;
					}
				}

				// Create some space in the output buffer
				if (_bufferPos > 0)
				{
					Array.Copy(_buffer, _bufferPos, _buffer, 0, _bufferLen - _bufferPos);
					_bufferLen -= _bufferPos;
					_bufferPos = 0;
				}
				else if (_bufferLen == _buffer.Length)
				{
					Array.Resize(ref _buffer, _buffer.Length + (32 * 1024));
				}

				// Read more data into the buffer
				int numBytesRead = await ReadAsync(_buffer, _bufferLen, _buffer.Length - _bufferLen, cancellationToken);
				if (numBytesRead == 0)
				{
					string? line = null;
					if (_bufferPos < _bufferLen)
					{
						line = Console.OutputEncoding.GetString(_buffer, _bufferPos, _bufferLen - _bufferPos);
						_bufferPos = _bufferLen;
					}
					return line;
				}
				_bufferLen += numBytesRead;
			}
		}

		/// <summary>
		/// Read all the output from the process. Does not return until the process terminates.
		/// </summary>
		/// <returns>List of output lines</returns>
		public List<string> ReadAllLines()
		{
			// Manually read all the output lines from the stream. Using ReadToEndAsync() or ReadAsync() on the StreamReader is abysmally slow, especially
			// for high-volume processes. Manually picking out the lines via a buffered ReadAsync() call was found to be 6x faster on 'p4 -ztag have' calls.
			List<string> outputLines = [];
			byte[] buffer = new byte[32 * 1024];
			byte lastCharacter = 0;
			int numBytesInBuffer = 0;
			for (; ; )
			{
				// If we're got a single line larger than 32kb (!), enlarge the buffer to ensure we can handle it
				if (numBytesInBuffer == buffer.Length)
				{
					Array.Resize(ref buffer, buffer.Length + 32 * 1024);
				}

				// Fill the buffer, reentering managed code every 20ms to allow thread abort exceptions to be thrown
				int numBytesRead = Read(buffer, numBytesInBuffer, buffer.Length - numBytesInBuffer);
				if (numBytesRead == 0)
				{
					if (numBytesInBuffer > 0)
					{
						outputLines.Add(Console.OutputEncoding.GetString(buffer, 0, numBytesInBuffer));
					}
					break;
				}

				// Otherwise append to the existing buffer
				numBytesInBuffer += numBytesRead;

				// Pull out all the complete output lines
				int lastStartIdx = 0;
				for (int idx = 0; idx < numBytesInBuffer; idx++)
				{
					if (buffer[idx] == '\r' || buffer[idx] == '\n')
					{
						if (buffer[idx] != '\n' || lastCharacter != '\r')
						{
							outputLines.Add(Console.OutputEncoding.GetString(buffer, lastStartIdx, idx - lastStartIdx));
						}
						lastStartIdx = idx + 1;
					}
					lastCharacter = buffer[idx];
				}

				// Shuffle everything back to the start of the buffer
				Array.Copy(buffer, lastStartIdx, buffer, 0, buffer.Length - lastStartIdx);
				numBytesInBuffer -= lastStartIdx;
			}
			WaitForExit();
			return outputLines;
		}

		/// <summary>
		/// Reads a single line asynchronously
		/// </summary>
		/// <returns>New line</returns>
		public async Task<string?> ReadLineAsync()
		{
			return await StdOutText!.ReadLineAsync();
		}

		/// <summary>
		/// Total amount of time spent by the CPU
		/// </summary>
		public TimeSpan TotalProcessorTime => _accountingProcessGroup != null ? _accountingProcessGroup.TotalProcessorTime : new TimeSpan();

		/// <summary>
		/// Peak memory used
		/// </summary>
		public long PeakMemoryUsed => _accountingProcessGroup != null ? _accountingProcessGroup.PeakJobMemoryUsed : 0;

		/// <summary>
		/// Block until the process outputs a line of text, or terminates.
		/// </summary>
		/// <param name="line">Variable to receive the output line</param>
		/// <param name="token">Cancellation token which can be used to abort the read operation.</param>
		/// <returns>True if a line was read, false if the process terminated without writing another line, or the cancellation token was signaled.</returns>
		public bool TryReadLine([NotNullWhen(true)] out string? line, CancellationToken? token = null)
		{
			try
			{
				// Busy wait for the ReadLine call to finish, so we can get interrupted by thread abort exceptions
				Task<string?> readLineTask = StdOutText!.ReadLineAsync();
				for (; ; )
				{
					const int MillisecondsTimeout = 20;
					if (token.HasValue
							? readLineTask.Wait(MillisecondsTimeout, token.Value)
							: readLineTask.Wait(MillisecondsTimeout))
					{
						line = readLineTask.IsCompleted ? readLineTask.Result : null;
						return line != null;
					}
				}
			}
			catch (OperationCanceledException)
			{
				// If the cancel token is signalled, just return false.
				line = null;
				return false;
			}
		}

		/// <summary>
		/// Waits for the process to exit
		/// </summary>
		/// <returns>Exit code of the process</returns>
		public void WaitForExit()
		{
			if (_frameworkProcess == null)
			{
				if (WaitForSingleObject(_processHandle!, INFINITE) == WAIT_FAILED)
				{
					throw new Win32Exception();
				}
			}
			else
			{
				_frameworkProcess.WaitForExit();
			}
		}

		class ProcessWaitHandle : WaitHandle
		{
			public ProcessWaitHandle(SafeProcessHandle processHandle)
			{
				SafeWaitHandle = new SafeWaitHandle(processHandle.DangerousGetHandle(), false);
			}
		}

		/// <summary>
		/// Waits for the process to exit
		/// </summary>
		public async Task WaitForExitAsync(CancellationToken cancellationToken = default)
		{
			if (_frameworkProcess == null)
			{
				using ProcessWaitHandle waitHandle = new(_processHandle!);
				await waitHandle.WaitOneAsync(cancellationToken);
			}
			else
			{
				await _frameworkProcess.WaitForExitAsync(cancellationToken);
			}
		}

		/// <summary>
		/// The exit code of the process. Throws an exception if the process has not terminated.
		/// </summary>
		public int ExitCode
		{
			get
			{
				if (_frameworkProcess == null)
				{
					int value;
					if (GetExitCodeProcess(_processHandle!, out value) == 0)
					{
						throw new Win32Exception();
					}
					return value;
				}
				else
				{
					return _frameworkProcess.ExitCode;
				}
			}
		}

		private DateTime _frameworkStartTime = DateTime.MinValue;
		private DateTime _frameworkExitTime = DateTime.MinValue;

		/// <summary>
		/// The creation time of the process.
		/// </summary>
		public DateTime StartTime
		{
			get
			{
				if (_frameworkProcess == null)
				{
					System.Runtime.InteropServices.ComTypes.FILETIME creationTime;
					if (!GetProcessTimes(_processHandle!, out creationTime, out _, out _, out _))
					{
						throw new Win32Exception();
					}
					return FileTimeToDateTime(creationTime);
				}
				else
				{
					return _frameworkStartTime;
				}
			}
		}

		/// <summary>
		/// The exit time of the process. Throws an exception if the process has not terminated.
		/// </summary>
		public DateTime ExitTime
		{
			get
			{
				if (_frameworkProcess == null)
				{
					System.Runtime.InteropServices.ComTypes.FILETIME exitTime;
					if (!GetProcessTimes(_processHandle!, out _, out exitTime, out _, out _))
					{
						throw new Win32Exception();
					}
					return FileTimeToDateTime(exitTime);
				}
				else
				{
					return _frameworkExitTime;
				}
			}
		}

		/// <summary>
		/// Returns true if the process has exited
		/// </summary>
		public bool HasExited
		{
			get
			{
				if (_frameworkProcess != null)
				{
					try
					{
						// throws exception if the process is still running
						return _frameworkProcess.HasExited;
					}
					catch (Exception)
					{
						return false;
					}
				}
				else
				{
					// returns non-zero on success and success means the process has exited
					if (GetExitCodeProcess(_processHandle!, out _) != 0)
					{
						return true;
					}
					return false;
				}
			}
		}

		/// <summary>
		/// Gets all the current environment variables
		/// </summary>
		/// <returns></returns>
		public static Dictionary<string, string> GetCurrentEnvVars()
		{
			Dictionary<string, string> newEnvironment = [];
			foreach (object? envVar in Environment.GetEnvironmentVariables())
			{
				System.Collections.DictionaryEntry entry = (System.Collections.DictionaryEntry)envVar!;
				string key = entry.Key.ToString()!;
				if (!newEnvironment.ContainsKey(key))
				{
					newEnvironment[key] = entry.Value!.ToString()!;
				}
			}
			return newEnvironment;
		}

		/// <summary>
		/// Kill the process if it's still running, optionally kill child processes
		/// </summary>
		/// <param name="killDescendants"></param>
		public void Kill(bool killDescendants = true)
		{
			if (_frameworkProcess != null)
			{
				if (_frameworkProcess.HasExited)
				{
					return;
				}

				// Kill entire process tree
				try
				{
					_frameworkProcess.Kill(killDescendants);
				}
				catch { }

				try
				{
					_frameworkProcess.WaitForExit();
				}
				catch { }
			}
			else
			{
				_ = TerminateProcess(_processHandle!, 0);
				_ = WaitForSingleObject(_processHandle!, INFINITE);
			}
		}
	}
}
