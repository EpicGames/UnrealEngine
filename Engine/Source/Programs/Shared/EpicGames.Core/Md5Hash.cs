// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Security.Cryptography;

#pragma warning disable CA5351 // Do Not Use Broken Cryptographic Algorithms

namespace EpicGames.Core
{
	/// <summary>
	/// Struct representing a strongly typed Md5Hash value
	/// </summary>
	[TypeConverter(typeof(Md5HashTypeConverter))]
	public readonly struct Md5Hash : IEquatable<Md5Hash>, IComparable<Md5Hash>
	{
		/// <summary>
		/// Length of an Md5Hash
		/// </summary>
		public const int NumBytes = 16;

		/// <summary>
		/// Length of the hash in bits
		/// </summary>
		public const int NumBits = NumBytes * 8;

		/// <summary>
		/// Memory storing the digest data
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Span for the underlying memory
		/// </summary>
		public ReadOnlySpan<byte> Span => Memory.Span;

		/// <summary>
		/// Hash consisting of zeroes
		/// </summary>
		public static Md5Hash Zero { get; } = new Md5Hash(new byte[NumBytes]);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">Memory to construct from</param>
		public Md5Hash(ReadOnlyMemory<byte> memory)
		{
			if (memory.Length != NumBytes)
			{
				throw new ArgumentException($"Md5Hash must be {NumBytes} bytes long");
			}

			Memory = memory;
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Md5Hash Compute(ReadOnlySpan<byte> data)
		{
			byte[] output = new byte[NumBytes];
			if (!MD5.TryHashData(data, output, out int bytesWritten) || bytesWritten != NumBytes)
			{
				throw new Exception("Unable to hash data");
			}
			return new Md5Hash(output);
		}

		/// <summary>
		/// Creates a content hash for the input Stream object
		/// </summary>
		/// <param name="stream">The Stream object to compoute the has for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Md5Hash Compute(Stream stream)
		{
			using (MD5 hasher = MD5.Create())
			{
				return new Md5Hash(hasher.ComputeHash(stream));
			}
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Md5Hash Parse(string text)
		{
			return new Md5Hash(StringUtils.ParseHexString(text));
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Md5Hash Parse(Utf8String text)
		{
			return new Md5Hash(StringUtils.ParseHexString(text.Span));
		}

		/// <inheritdoc cref="IComparable{T}.CompareTo(T)"/>
		public int CompareTo(Md5Hash other)
		{
			ReadOnlySpan<byte> a = Span;
			ReadOnlySpan<byte> b = other.Span;

			for (int idx = 0; idx < a.Length && idx < b.Length; idx++)
			{
				int compare = a[idx] - b[idx];
				if (compare != 0)
				{
					return compare;
				}
			}
			return a.Length - b.Length;
		}

		/// <inheritdoc/>
		public bool Equals(Md5Hash other) => Span.SequenceEqual(other.Span);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is Md5Hash hash) && hash.Span.SequenceEqual(Span);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32LittleEndian(Span);

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(Memory.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator ==(Md5Hash a, Md5Hash b) => a.Span.SequenceEqual(b.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator !=(Md5Hash a, Md5Hash b) => !(a == b);

		/// <summary>
		/// Tests whether A > B
		/// </summary>
		public static bool operator >(Md5Hash a, Md5Hash b) => a.CompareTo(b) > 0;

		/// <summary>
		/// Tests whether A is less than B
		/// </summary>
		public static bool operator <(Md5Hash a, Md5Hash b) => a.CompareTo(b) < 0;

		/// <summary>
		/// Tests whether A is greater than or equal to B
		/// </summary>
		public static bool operator >=(Md5Hash a, Md5Hash b) => a.CompareTo(b) >= 0;

		/// <summary>
		/// Tests whether A is less than or equal to B
		/// </summary>
		public static bool operator <=(Md5Hash a, Md5Hash b) => a.CompareTo(b) <= 0;
	}

	/// <summary>
	/// Extension methods for dealing with Md5Hash values
	/// </summary>
	public static class Md5HashExtensions
	{
		/// <summary>
		/// Read an <see cref="Md5Hash"/> from a memory reader
		/// </summary>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static Md5Hash ReadMd5Hash(this MemoryReader reader)
		{
			return new Md5Hash(reader.ReadFixedLengthBytes(Md5Hash.NumBytes));
		}

		/// <summary>
		/// Write an <see cref="Md5Hash"/> to a memory writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="hash"></param>
		public static void WriteMd5Hash(this MemoryWriter writer, Md5Hash hash)
		{
			writer.WriteFixedLengthBytes(hash.Span);
		}
	}

	/// <summary>
	/// Type converter from strings to Md5Hash objects
	/// </summary>
	sealed class Md5HashTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return Md5Hash.Parse((string)value);
		}
	}
}
