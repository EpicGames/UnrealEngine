// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.IO.MemoryMappedFiles;
using System.Linq;
using System.Threading;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Manages a cache of memory mapped file handles
	/// </summary>
	public sealed class MemoryMappedFileCache : IDisposable
	{
		// Item which has been opened from the cache using a memory mapped file
		class MappedFile : IDisposable
		{
			public FileReference Key { get; }
			public LinkedListNode<MappedFile> ListNode { get; }

			readonly MemoryMappedFileCache _cache;
			readonly FileInfo _fileInfo;

			MemoryMappedFile? _memoryMappedFile;
			MemoryMappedViewAccessor? _memoryMappedViewAccessor;
			MemoryMappedView? _memoryMappedView;

			int _refCount = 1;
			ReadOnlyMemory<byte> _data;
			bool _deleteOnDispose;

			public int RefCount => _refCount;

			public ulong MappedSize => _memoryMappedViewAccessor?.SafeMemoryMappedViewHandle.ByteLength ?? 0UL;

			public MappedFile(MemoryMappedFileCache cache, FileReference locator, FileInfo fileInfo)
			{
				_cache = cache;

				Key = locator;
				ListNode = new LinkedListNode<MappedFile>(this);

				_fileInfo = fileInfo;

				try
				{
					_memoryMappedFile = MemoryMappedFile.CreateFromFile(fileInfo.FullName, FileMode.Open, null, 0, MemoryMappedFileAccess.Read);
					_memoryMappedViewAccessor = _memoryMappedFile.CreateViewAccessor(0, 0, MemoryMappedFileAccess.Read);
					_memoryMappedView = new MemoryMappedView(_memoryMappedViewAccessor);

					_data = _memoryMappedView.GetMemory(0, (int)Math.Min(fileInfo.Length, Int32.MaxValue));
					Interlocked.Add(ref _cache._mappedSize, _data.Length);
				}
				catch
				{
					Dispose();
					throw;
				}
			}

			public void Dispose()
			{
				Interlocked.Add(ref _cache._mappedSize, -_data.Length);
				_data = ReadOnlyMemory<byte>.Empty;
				_memoryMappedView?.Dispose();
				_memoryMappedView = null;
				_memoryMappedViewAccessor?.Dispose();
				_memoryMappedViewAccessor = null;
				_memoryMappedFile?.Dispose();
				_memoryMappedFile = null;

				if (_deleteOnDispose)
				{
					try
					{
						_fileInfo.Delete();
					}
					catch { }
				}
			}

			public ReadOnlyMemory<byte> GetData(int offset, int? length)
			{
				if (length == null)
				{
					return _data.Slice(offset);
				}
				else
				{
					return _data.Slice(offset, Math.Min(length.Value, _data.Length - offset));
				}
			}

			public void AddRef()
			{
				Interlocked.Increment(ref _refCount);
			}

			public void Release()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					Dispose();
				}
			}

			public void DeleteOnDispose()
			{
				_deleteOnDispose = true;
			}

			public override string ToString() => Key.ToString();
		}

		// Handle to a file in memory
		class MappedFileHandle : IReadOnlyMemoryOwner<byte>
		{
			MappedFile? _mappedFile;
			ReadOnlyMemory<byte> _data;

			public ReadOnlyMemory<byte> Memory => _data;

			public MappedFileHandle(MappedFile? mappedFile, ReadOnlyMemory<byte> data)
			{
				_mappedFile = mappedFile;
				_mappedFile?.AddRef();
				_data = data;
			}

			public MappedFileHandle Clone()
			{
				_mappedFile?.AddRef();
				return new MappedFileHandle(_mappedFile, _data);
			}

			public void Dispose()
			{
				if (_mappedFile != null)
				{
					_mappedFile.Release();
					_mappedFile = null!;
				}

				_data = ReadOnlyMemory<byte>.Empty;
			}
		}

		readonly Lock _lockObject = new();
		readonly Dictionary<FileReference, MappedFile> _pathToMappedFile = [];
		readonly LinkedList<MappedFile> _mappedFiles = new();

		long _mappedSize;

		readonly long _maxMappedSize;
		readonly int _maxMappedCount;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="maxMappedSize">Max amount of data to keep mapped into memory</param>
		/// <param name="maxMappedCount">Max number of memory mapped file handles</param>
		public MemoryMappedFileCache(long maxMappedSize = 1024L * 1024 * 1024, int maxMappedCount = 128)
		{
			_maxMappedSize = maxMappedSize;
			_maxMappedCount = maxMappedCount;
		}

		/// <summary>
		/// Removes any cached files
		/// </summary>
		public void Clear()
		{
			lock (_lockObject)
			{
				UnmapFiles(0, 0);
			}
		}

		/// <inheritdoc/>
		public void Dispose()
			=> Clear();

		/// <summary>
		/// Maps data for a file into memory
		/// </summary>
		/// <param name="file">File to open</param>
		/// <param name="offset">Start offset of the file to map</param>
		/// <param name="length">Length of the mapped data</param>
		public IReadOnlyMemoryOwner<byte> Read(FileReference file, int offset, int? length)
		{
			lock (_lockObject)
			{
				MappedFile? mappedFile = FindOrAddMappedFile(file);
				if (mappedFile == null)
				{
					return ReadOnlyMemoryOwner.Create(ReadOnlyMemory<byte>.Empty, null);
				}
				return new MappedFileHandle(mappedFile, mappedFile.GetData(offset, length));
			}
		}

		/// <summary>
		/// Delete a file from the store
		/// </summary>
		/// <param name="file"></param>
		public void Delete(FileReference file)
		{
			lock (_lockObject)
			{
				MappedFile? mappedFile;
				if (_pathToMappedFile.TryGetValue(file, out mappedFile))
				{
					mappedFile.DeleteOnDispose();

					_pathToMappedFile.Remove(file);
					_mappedFiles.Remove(mappedFile.ListNode);

					mappedFile.Release();
				}
				else
				{
					FileReference.Delete(file);
				}
			}
		}

		/// <summary>
		/// Finds an existing mapped file or adds a new one for the given path
		/// </summary>
		MappedFile? FindOrAddMappedFile(FileReference key)
		{
			MappedFile? mappedFile;
			if (!_pathToMappedFile.TryGetValue(key, out mappedFile))
			{
				FileInfo fileInfo = key.ToFileInfo();
				if (!fileInfo.Exists)
				{
					throw new FileNotFoundException($"Unable to find file '{key}'", key.FullName);
				}
				if (fileInfo.Length == 0)
				{
					return null;
				}

				long maxSize = _maxMappedSize - fileInfo.Length;
				if (_mappedSize > maxSize || _mappedFiles.Count + 1 > _maxMappedCount)
				{
					UnmapFiles(maxSize, _maxMappedCount - 1);
				}

				mappedFile = new MappedFile(this, key, fileInfo);
				_pathToMappedFile.Add(key, mappedFile);
				_mappedFiles.AddFirst(mappedFile.ListNode);
			}
			return mappedFile;
		}

		/// <summary>
		/// Discard mapped files until only a certain size is mapped in memory
		/// </summary>
		/// <param name="maxMappedSize">Maximum mapped size</param>
		/// <param name="maxMappedCount">Maximum number of mapped files</param>
		void UnmapFiles(long maxMappedSize, int maxMappedCount)
		{
			for (LinkedListNode<MappedFile>? listNode = _mappedFiles.Last; listNode != null && (_mappedSize > maxMappedSize || _mappedFiles.Count > maxMappedCount);)
			{
				LinkedListNode<MappedFile>? nextListNode = listNode.Previous;
				if (listNode.Value.RefCount == 1)
				{
					_pathToMappedFile.Remove(listNode.Value.Key);
					_mappedFiles.Remove(listNode);
					listNode.Value.Release();
				}
				listNode = nextListNode;
			}
		}

		/// <summary>
		/// Prints stats about the number of outstanding references to items in the cache
		/// </summary>
		public void WriteRefStats(ILogger logger)
		{
			lock (_lockObject)
			{
				logger.LogInformation("Memory mapped file cache using {NumEntries:n0}/{MaxEntries:n0} ({Size:n0}mb/{MaxSize:n0}mb)", _mappedFiles.Count, _maxMappedCount, _mappedSize, _maxMappedSize);

				int referencedCount = 0;
				ulong referencedSize = 0;
				foreach (MappedFile mappedFile in _mappedFiles.OrderByDescending(x => x.RefCount).ThenBy(x => x.Key).TakeWhile(x => x.RefCount > 1))
				{
					logger.LogInformation("Mapped file {File} has {NumRefs} references", mappedFile.Key, mappedFile.RefCount);
					referencedCount++;
					referencedSize += mappedFile.MappedSize;
				}

				logger.LogInformation("{NumFiles:n0} files have more than one reference count ({SizeMb:n1}mb)", referencedCount, referencedSize / (1024.0 * 1024.0));
			}
		}
	}
}
