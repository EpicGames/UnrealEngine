// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.ComponentModel;
using System.Diagnostics;
using System.IO.MemoryMappedFiles;
using System.Runtime.InteropServices;
using Microsoft.Win32.SafeHandles;

namespace EpicGames.Core
{
	/// <summary>
	/// Implements an unmarshlled view of a memory mapped file
	/// </summary>
	public sealed unsafe class MemoryMappedView : IDisposable
	{
		sealed unsafe class MemoryWrapper : MemoryManager<byte>
		{
			private readonly byte* _pointer;
			private readonly int _length;

			public MemoryWrapper(byte* pointer, int length)
			{
				_pointer = pointer;
				_length = length;
			}

			/// <inheritdoc/>
			public override Span<byte> GetSpan() => new(_pointer, _length);

			/// <inheritdoc/>
			public override MemoryHandle Pin(int elementIndex) => new(_pointer + elementIndex);

			/// <inheritdoc/>
			public override void Unpin() { }

			/// <inheritdoc/>
			protected override void Dispose(bool disposing) { }
		}

		const uint FILE_MAP_COPY = 0x0001;
		const uint FILE_MAP_WRITE = 0x0002;
		const uint FILE_MAP_READ = 0x0004;
		const uint FILE_MAP_EXECUTE = 0x0020;

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern unsafe void* MapViewOfFile(SafeMemoryMappedFileHandle handle, uint desiredAccess, uint fileOffsetHigh, uint fileOffsetLow, long numberOfBytes);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern unsafe bool UnmapViewOfFile(void* ptr);

		byte* _data;
		Action? _disposeMethod;

		/// <summary>
		/// Create a view of a memory mapped file
		/// </summary>
		/// <param name="memoryMappedFile">Handle of the file to map into memory</param>
		/// <param name="offset">Offset within the file to map</param>
		/// <param name="length">Length of the region to map</param>
		/// <param name="access">How to open the file as read only</param>
		public MemoryMappedView(MemoryMappedFile memoryMappedFile, long offset, long length, MemoryMappedFileAccess access = MemoryMappedFileAccess.ReadWrite)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				uint accessFlags = access switch
				{
					MemoryMappedFileAccess.Read => FILE_MAP_READ,
					MemoryMappedFileAccess.ReadWrite => FILE_MAP_READ | FILE_MAP_WRITE,
					MemoryMappedFileAccess.Write => FILE_MAP_WRITE,
					MemoryMappedFileAccess.CopyOnWrite => FILE_MAP_READ | FILE_MAP_WRITE | FILE_MAP_COPY,
					MemoryMappedFileAccess.ReadExecute => FILE_MAP_READ | FILE_MAP_EXECUTE,
					MemoryMappedFileAccess.ReadWriteExecute => FILE_MAP_READ | FILE_MAP_WRITE | FILE_MAP_EXECUTE,
					_ => FILE_MAP_READ | FILE_MAP_WRITE
				};

				_data = (byte*)MapViewOfFile(memoryMappedFile.SafeMemoryMappedFileHandle, accessFlags, (uint)(offset >> 32), (uint)offset, length);
				if (_data == null)
				{
					int status = Marshal.GetLastWin32Error();
					throw new Win32Exception(status, $"Unable to create memory mapped file (GetLastError() == {status})");
				}

				_disposeMethod = () =>
				{
					UnmapViewOfFile(_data);
				};
			}
			else
			{
				MemoryMappedViewAccessor memoryMappedViewAccessor = memoryMappedFile.CreateViewAccessor(offset, length, access);
				memoryMappedViewAccessor.SafeMemoryMappedViewHandle.AcquirePointer(ref _data);
				Trace.Assert(_data != null);

				_disposeMethod = () =>
				{
					memoryMappedViewAccessor.SafeMemoryMappedViewHandle.ReleasePointer();
					memoryMappedViewAccessor.Dispose();
				};
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memoryMappedViewAccessor"></param>
		public MemoryMappedView(MemoryMappedViewAccessor memoryMappedViewAccessor)
		{
			memoryMappedViewAccessor.SafeMemoryMappedViewHandle.AcquirePointer(ref _data);
			_disposeMethod = () => memoryMappedViewAccessor.SafeMemoryMappedViewHandle.ReleasePointer();
		}

		/// <summary>
		/// Gets a pointer to the data
		/// </summary>
		public byte* GetPointer() => _data;

		/// <summary>
		/// Gets a memory object for the given range
		/// </summary>
		/// <param name="offset"></param>
		/// <param name="length"></param>
		/// <returns></returns>
		public Memory<byte> GetMemory(long offset, int length)
		{
			MemoryWrapper wrapper = new(_data + offset, length);
			return wrapper.Memory;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_disposeMethod != null)
			{
				_disposeMethod();
				_disposeMethod = null;
			}
			_data = null;
		}
	}
}
