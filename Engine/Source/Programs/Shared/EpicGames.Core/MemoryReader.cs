// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for reading from data in memory
	/// </summary>
	public interface IMemoryReader
	{
		/// <summary>
		/// Gets a block of memory with at least the given size
		/// </summary>
		/// <param name="minSize">Minimum size of the returned data</param>
		/// <returns>Memory of at least the given size</returns>
		ReadOnlyMemory<byte> GetMemory(int minSize = 1);

		/// <summary>
		/// Updates the current position within the input buffer
		/// </summary>
		/// <param name="length">Number of bytes to advance by</param>
		void Advance(int length);
	}

	/// <summary>
	/// Reads data from a memory buffer
	/// </summary>
	public class MemoryReader : IMemoryReader
	{
		/// <summary>
		/// The memory to read from
		/// </summary>
		public ReadOnlyMemory<byte> RemainingMemory
		{
			get; private set;
		}

		/// <summary>
		/// Returns the memory at the current offset
		/// </summary>
		public ReadOnlySpan<byte> RemainingSpan => RemainingMemory.Span;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The memory to read from</param>
		public MemoryReader(ReadOnlyMemory<byte> memory)
		{
			RemainingMemory = memory;
		}

		/// <summary>
		/// Checks that we've used the exact buffer length
		/// </summary>
		public void CheckEmpty()
		{
			if (RemainingMemory.Length > 0)
			{
				throw new Exception($"Serialization is not at expected offset within the input buffer ({RemainingMemory.Length} bytes unused)");
			}
		}

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> GetMemory(int minSize) => RemainingMemory;

		/// <inheritdoc/>
		public void Advance(int length) => RemainingMemory = RemainingMemory.Slice(length);
	}

	/// <summary>
	/// Extension methods for <see cref="IMemoryReader"/>
	/// </summary>
	public static class MemoryReaderExtensions
	{
		/// <summary>
		/// Gets a span of data with the given length
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="minSize">Minimum size of the returned span</param>
		/// <returns>Span of at least the given size</returns>
		public static ReadOnlySpan<byte> GetSpan(this IMemoryReader reader, int minSize) => reader.GetMemory(minSize).Span;

		/// <summary>
		/// Read a boolean from the buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value read from the buffer</returns>
		public static bool ReadBoolean(this IMemoryReader reader)
		{
			return ReadUInt8(reader) != 0;
		}

		/// <summary>
		/// Reads a byte from the buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value read from the buffer</returns>
		public static sbyte ReadInt8(this IMemoryReader reader)
		{
			sbyte value = (sbyte)reader.GetSpan(1)[0];
			reader.Advance(1);
			return value;
		}

		/// <summary>
		/// Reads a byte from the buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value read from the buffer</returns>
		public static byte ReadUInt8(this IMemoryReader reader)
		{
			byte value = reader.GetSpan(1)[0];
			reader.Advance(1);
			return value;
		}

		/// <summary>
		/// Reads an int16
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value read from the buffer</returns>
		public static short ReadInt16(this IMemoryReader reader)
		{
			short value = BinaryPrimitives.ReadInt16LittleEndian(reader.GetSpan(sizeof(short)));
			reader.Advance(sizeof(short));
			return value;
		}

		/// <summary>
		/// Reads a uint16
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value read from the buffer</returns>
		public static ushort ReadUInt16(this IMemoryReader reader)
		{
			ushort value = BinaryPrimitives.ReadUInt16LittleEndian(reader.GetSpan(sizeof(ushort)));
			reader.Advance(sizeof(ushort));
			return value;
		}

		/// <summary>
		/// Reads an int32
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value that was read from the buffer</returns>
		public static int ReadInt32(this IMemoryReader reader)
		{
			int value = BinaryPrimitives.ReadInt32LittleEndian(reader.GetSpan(sizeof(int)));
			reader.Advance(sizeof(int));
			return value;
		}

		/// <summary>
		/// Reads a uint32
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value that was read from the buffer</returns>
		public static uint ReadUInt32(this IMemoryReader reader)
		{
			uint value = BinaryPrimitives.ReadUInt32LittleEndian(reader.GetSpan(sizeof(uint)));
			reader.Advance(sizeof(uint));
			return value;
		}

		/// <summary>
		/// Reads an int64
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value that was read from the buffer</returns>
		public static long ReadInt64(this IMemoryReader reader)
		{
			long value = BinaryPrimitives.ReadInt64LittleEndian(reader.GetSpan(sizeof(long)));
			reader.Advance(sizeof(long));
			return value;
		}

		/// <summary>
		/// Reads an int64
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The value that was read from the buffer</returns>
		public static ulong ReadUInt64(this IMemoryReader reader)
		{
			ulong value = BinaryPrimitives.ReadUInt64LittleEndian(reader.GetSpan(sizeof(ulong)));
			reader.Advance(sizeof(ulong));
			return value;
		}

		/// <summary>
		/// Reads a DateTime from the memory buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		public static DateTime ReadDateTime(this IMemoryReader reader)
		{
			ulong encoded = reader.ReadUnsignedVarInt();
			return new DateTime((long)encoded >> 2, (DateTimeKind)(encoded & 3UL));
		}

		/// <summary>
		/// Reads a GUID from the memory buffer using NET-ordered serialization.
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		public static Guid ReadGuidNetOrder(this IMemoryReader reader)
		{
			Guid guid = new(reader.GetMemory(16).Slice(0, 16).Span);
			reader.Advance(16);
			return guid;
		}

		/// <summary>
		/// Reads a GUID from the memory buffer using UE-ordered serialization.
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		public static Guid ReadGuidUnrealOrder(this IMemoryReader reader)
		{
			Guid guid = GuidUtils.ReadGuidUnrealOrder(reader.GetSpan(16));
			reader.Advance(16);
			return guid;
		}

		/// <summary>
		/// Reads a sequence of bytes from the buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>Sequence of bytes</returns>
		public static ReadOnlyMemory<byte> ReadVariableLengthBytes(this IMemoryReader reader)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			return ReadFixedLengthBytes(reader, length);
		}

		/// <summary>
		/// Writes a variable length span of bytes
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		public static ReadOnlyMemory<byte> ReadVariableLengthBytesWithInt32Length(this IMemoryReader reader)
		{
			int length = ReadInt32(reader);
			return ReadFixedLengthBytes(reader, length);
		}

		/// <summary>
		/// Reads a sequence of bytes from the buffer
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="length">Number of bytes to read</param>
		/// <returns>Sequence of bytes</returns>
		public static ReadOnlyMemory<byte> ReadFixedLengthBytes(this IMemoryReader reader, int length)
		{
			ReadOnlyMemory<byte> bytes = reader.GetMemory(length).Slice(0, length);
			reader.Advance(length);
			return bytes;
		}

		/// <summary>
		/// Reads a variable length list
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="list">List to receive the items that were read</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static void ReadList<T>(this IMemoryReader reader, List<T> list, Func<T> readItem)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			list.EnsureCapacity(list.Count + length);

			for (int idx = 0; idx < length; idx++)
			{
				list.Add(readItem());
			}
		}

		/// <summary>
		/// Reads a variable length list
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static List<T> ReadList<T>(this IMemoryReader reader, Func<T> readItem)
		{
			List<T> list = [];
			ReadList(reader, list, readItem);
			return list;
		}

		/// <summary>
		/// Reads a variable length list
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static List<T> ReadList<T>(this IMemoryReader reader, Func<IMemoryReader, T> readItem)
		{
			int length = (int)reader.ReadUnsignedVarInt();

			List<T> list = new(length);
			for (int idx = 0; idx < length; idx++)
			{
				list.Add(readItem(reader));
			}

			return list;
		}

		/// <summary>
		/// Reads a variable length array
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static T[] ReadVariableLengthArray<T>(this IMemoryReader reader, Func<T> readItem)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			return ReadFixedLengthArray(reader, length, readItem);
		}

		/// <summary>
		/// Reads a variable length array
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static T[] ReadVariableLengthArray<T>(this IMemoryReader reader, Func<IMemoryReader, T> readItem)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			return ReadFixedLengthArray(reader, length, readItem);
		}

		/// <summary>
		/// Reads a variable length array
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="readItem">Delegate to write an individual item</param>
		public static T[] ReadVariableLengthArrayWithInt32Length<T>(this IMemoryReader reader, Func<T> readItem)
		{
			int length = ReadInt32(reader);
			return ReadFixedLengthArray(reader, length, readItem);
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="length">Length of the array to read</param>
		/// <param name="readItem">Delegate to read an individual item</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public static T[] ReadFixedLengthArray<T>(this IMemoryReader reader, int length, Func<T> readItem)
		{
			if (length == 0)
			{
				return [];
			}

			T[] array = new T[length];
			for (int idx = 0; idx < length; idx++)
			{
				array[idx] = readItem();
			}
			return array;
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="length">Length of the array to read</param>
		/// <param name="readItem">Delegate to read an individual item</param>
		public static T[] ReadFixedLengthArray<T>(this IMemoryReader reader, int length, Func<IMemoryReader, T> readItem)
		{
			if (length == 0)
			{
				return [];
			}

			T[] array = new T[length];
			for (int idx = 0; idx < length; idx++)
			{
				array[idx] = readItem(reader);
			}
			return array;
		}

		/// <summary>
		/// Reads a dictionary from the writer
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		/// <param name="dictionary">The dictionary to read</param>
		/// <param name="readKey">Delegate to write an individual key</param>
		/// <param name="readValue">Delegate to write an individual value</param>
		public static void ReadDictionary<TKey, TValue>(this IMemoryReader reader, Dictionary<TKey, TValue> dictionary, Func<TKey> readKey, Func<TValue> readValue) where TKey : notnull
		{
			int count = (int)reader.ReadUnsignedVarInt();
			dictionary.EnsureCapacity(count);

			for (int idx = 0; idx < count; idx++)
			{
				TKey key = readKey();
				TValue value = readValue();
				dictionary.Add(key, value);
			}
		}

		/// <summary>
		/// Reads a dictionary from the writer
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		/// <param name="dictionary">The dictionary to read</param>
		/// <param name="readKey">Delegate to write an individual key</param>
		/// <param name="readValue">Delegate to write an individual value</param>
		public static void ReadDictionary<TKey, TValue>(this IMemoryReader reader, Dictionary<TKey, TValue> dictionary, Func<IMemoryReader, TKey> readKey, Func<IMemoryReader, TValue> readValue) where TKey : notnull
		{
			int count = (int)reader.ReadUnsignedVarInt();
			dictionary.EnsureCapacity(count);

			for (int idx = 0; idx < count; idx++)
			{
				TKey key = readKey(reader);
				TValue value = readValue(reader);
				dictionary.Add(key, value);
			}
		}

		/// <summary>
		/// Reads a dictionary from the writer
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		/// <param name="readKey">Delegate to write an individual key</param>
		/// <param name="readValue">Delegate to write an individual value</param>
		/// <param name="comparer">Comparer for the new dictionary</param>
		public static Dictionary<TKey, TValue> ReadDictionary<TKey, TValue>(this IMemoryReader reader, Func<TKey> readKey, Func<TValue> readValue, IEqualityComparer<TKey>? comparer = null) where TKey : notnull
		{
			Dictionary<TKey, TValue> dictionary = new(comparer);
			ReadDictionary(reader, dictionary, readKey, readValue);
			return dictionary;
		}

		/// <summary>
		/// Reads a dictionary from the writer
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		/// <param name="readKey">Delegate to write an individual key</param>
		/// <param name="readValue">Delegate to write an individual value</param>
		/// <param name="comparer">Comparer for the new dictionary</param>
		public static Dictionary<TKey, TValue> ReadDictionary<TKey, TValue>(this IMemoryReader reader, Func<IMemoryReader, TKey> readKey, Func<IMemoryReader, TValue> readValue, IEqualityComparer<TKey>? comparer = null) where TKey : notnull
		{
			Dictionary<TKey, TValue> dictionary = new(comparer);
			ReadDictionary(reader, dictionary, readKey, readValue);
			return dictionary;
		}

		/// <summary>
		/// Read a string from memory
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The string that was read</returns>
		public static string ReadString(this IMemoryReader reader) => ReadString(reader, Encoding.UTF8);

		/// <summary>
		/// Read a string from memory
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="encoding">Encoding to use for the string</param>
		/// <returns>The string that was read</returns>
		public static string ReadString(this IMemoryReader reader, Encoding encoding)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			if (length == 0)
			{
				return String.Empty;
			}

			ReadOnlySpan<byte> span = reader.GetSpan(length).Slice(0, length);
			string str = encoding.GetString(span);
			reader.Advance(length);

			return str;
		}

		/// <summary>
		/// Read a string from memory
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The string that was read</returns>
		public static string? ReadOptionalString(this IMemoryReader reader) => ReadOptionalString(reader, Encoding.UTF8);

		/// <summary>
		/// Read a string from memory
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <param name="encoding">Encoding to use for the string</param>
		/// <returns>The string that was read</returns>
		public static string? ReadOptionalString(this IMemoryReader reader, Encoding encoding)
		{
			int length = (int)reader.ReadUnsignedVarInt();
			if (length == 0)
			{
				return null;
			}
			else
			{
				length--;

				ReadOnlySpan<byte> span = reader.GetSpan(length).Slice(0, length);
				string str = encoding.GetString(span);
				reader.Advance(length);

				return str;
			}
		}
	}
}
