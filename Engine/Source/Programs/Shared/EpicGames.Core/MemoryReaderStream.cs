// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Implementation of <see cref="Stream"/> which reads data from a <see cref="IMemoryReader"/>
	/// </summary>
	public class MemoryReaderStream : Stream
	{
		readonly IMemoryReader _reader;

		/// <summary>
		/// Constructor
		/// </summary>
		public MemoryReaderStream(IMemoryReader reader)
		{
			_reader = reader;
		}

		/// <inheritdoc/>
		public override bool CanRead => true;

		/// <inheritdoc/>
		public override bool CanSeek => false;

		/// <inheritdoc/>
		public override bool CanWrite => false;

		/// <inheritdoc/>
		public override long Length => throw new NotSupportedException();

		/// <inheritdoc/>
		public override long Position { get => throw new NotSupportedException(); set => throw new NotSupportedException(); }

		/// <inheritdoc/>
		public override void Flush() { }

		/// <inheritdoc/>
		public override int Read(Span<byte> buffer)
		{
			int sizeRead = 0;
			while (buffer.Length > 0)
			{
				ReadOnlyMemory<byte> memory = _reader.GetMemory();
				if (memory.Length == 0)
				{
					break;
				}

				int copyLength = Math.Min(buffer.Length, memory.Length);
				memory.Slice(0, copyLength).Span.CopyTo(buffer);
				_reader.Advance(copyLength);
				sizeRead += copyLength;

				buffer = buffer.Slice(copyLength);
			}
			return sizeRead;
		}

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => Read(buffer.AsSpan(offset, count));

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();
	}
}
