// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for serializing to a memory buffer
	/// </summary>
	public interface IMemoryWriter : IBufferWriter<byte>
	{
		/// <summary>
		/// Length of the written data
		/// </summary>
		int Length { get; }
	}

	/// <summary>
	/// Writes into a fixed size memory block
	/// </summary>
	public class MemoryWriter : IMemoryWriter
	{
		/// <summary>
		/// The memory block to write to
		/// </summary>
		readonly Memory<byte> _memory;

		/// <summary>
		/// Length of the data that has been written
		/// </summary>
		int _length;

		/// <summary>
		/// Returns the memory that was written to
		/// </summary>
		public ReadOnlyMemory<byte> WrittenMemory => _memory.Slice(0, _length);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">Memory to write to</param>
		public MemoryWriter(Memory<byte> memory)
		{
			_memory = memory;
		}

		/// <summary>
		/// Checks that we've used the exact buffer length
		/// </summary>
		public void CheckEmpty()
		{
			if (_length < _memory.Length)
			{
				throw new Exception($"Serialization is not at expected offset within the output buffer ({_length}/{_memory.Length} bytes used)");
			}
		}

		/// <inheritdoc/>
		public int Length => _length;

		/// <inheritdoc/>
		public Span<byte> GetSpan(int sizeHint) => _memory.Slice(_length).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int sizeHint) => _memory.Slice(_length);

		/// <inheritdoc/>
		public void Advance(int length) => _length += length;
	}

	/// <summary>
	/// Writes into an expandable memory block
	/// </summary>
	public class ArrayMemoryWriter : IMemoryWriter
	{
		byte[] _data;
		int _length;

		/// <summary>
		/// Returns the span that has been written to
		/// </summary>
		public Span<byte> WrittenSpan => _data.AsSpan(0, _length);

		/// <summary>
		/// Returns the memory that has been written to
		/// </summary>
		public Memory<byte> WrittenMemory => _data.AsMemory(0, _length);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="initialSize">Initial size of the allocated data</param>
		public ArrayMemoryWriter(int initialSize)
		{
			_data = new byte[initialSize];
		}

		/// <inheritdoc/>
		public int Length => _length;

		/// <summary>
		/// Resets the current contents of the buffer
		/// </summary>
		public void Clear()
		{
			_length = 0;
		}

		/// <summary>
		/// Resize the underlying buffer, updating the current length.
		/// </summary>
		/// <param name="newLength">New length of the buffer</param>
		public void Resize(int newLength)
		{
			if (newLength > _length)
			{
				Array.Resize(ref _data, newLength);
			}
			_length = newLength;
		}

		/// <inheritdoc/>
		public Span<byte> GetSpan(int sizeHint = 0) => GetMemory(sizeHint).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int minSize = 0)
		{
			int requiredLength = _length + Math.Max(minSize, 1);
			if (_data.Length < requiredLength)
			{
				Array.Resize(ref _data, requiredLength + 4096);
			}
			return _data.AsMemory(_length);
		}

		/// <inheritdoc/>
		public void Advance(int length)
		{
			_length += length;
		}
	}

	/// <summary>
	/// Writes into an expandable memory block
	/// </summary>
	public class PooledMemoryWriter : IMemoryWriter, IDisposable
	{
		readonly MemoryPool<byte> _pool;
		IMemoryOwner<byte> _owner;
		Memory<byte> _memory;
		int _length;

		/// <summary>
		/// Returns the span that has been written to
		/// </summary>
		public Span<byte> WrittenSpan => _memory.Span.Slice(0, _length);

		/// <summary>
		/// Returns the memory that has been written to
		/// </summary>
		public Memory<byte> WrittenMemory => _memory.Slice(0, _length);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="initialSize">Initial size of the allocated data</param>
		public PooledMemoryWriter(int initialSize = 4096)
			: this(MemoryPool<byte>.Shared, initialSize)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="pool">Pool to allocate data from</param>
		/// <param name="initialSize">Initial size of the allocated data</param>
		public PooledMemoryWriter(MemoryPool<byte> pool, int initialSize = 4096)
		{
			_pool = pool;
			_owner = pool.Rent(initialSize);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		/// <param name="disposing">Whether the </param>
		protected virtual void Dispose(bool disposing)
		{
			_owner.Dispose();
		}

		/// <inheritdoc/>
		public int Length => _length;

		/// <summary>
		/// Resets the current contents of the buffer
		/// </summary>
		public void Clear()
		{
			_length = 0;
		}

		/// <summary>
		/// Resets the current contents of the buffer
		/// </summary>
		public void Clear(int minSize)
		{
			Clear();
			if (minSize > _owner.Memory.Length)
			{
				_owner.Dispose();
				_owner = _pool.Rent(minSize);
			}
		}

		/// <inheritdoc/>
		public Span<byte> GetSpan(int sizeHint = 0) => GetMemory(sizeHint).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int minSize = 0)
		{
			int requiredLength = _length + Math.Max(minSize, 1);
			if (_memory.Length < requiredLength)
			{
				IMemoryOwner<byte> newOwner = _pool.Rent(requiredLength + Math.Max(requiredLength / 2, 4096));
				_memory.CopyTo(newOwner.Memory);
				_memory = newOwner.Memory;
				_owner.Dispose();
				_owner = newOwner;
			}
			return _memory.Slice(_length);
		}

		/// <inheritdoc/>
		public void Advance(int length)
		{
			_length += length;
		}
	}

	/// <summary>
	/// Class for building byte sequences, similar to StringBuilder. Allocates memory in chunks to avoid copying data.
	/// </summary>
	public abstract class ChunkedMemoryWriterBase : IMemoryWriter
	{
		/// <summary>
		/// Describes a chunk of data in the output stream
		/// </summary>
		protected class Chunk
		{
			/// <summary>
			/// Start position in the sequence
			/// </summary>
			public int RunningIndex { get; }

			/// <summary>
			/// Data for this chunk
			/// </summary>
			public Memory<byte> Data { get; }

			/// <summary>
			/// Used length of the chunk
			/// </summary>
			public int Length { get; set; }

			/// <summary>
			/// Constructor
			/// </summary>
			public Chunk(int runningIndex, Memory<byte> data)
			{
				RunningIndex = runningIndex;
				Data = data;
			}

			/// <summary>
			/// Release any resources managed by this chunk. Note that ChunkedMemoryWriterBase does not implement IDisposable directly; derived classes should call Clear() on Dispose() if needed.
			/// </summary>
			public virtual void Release() { }

			/// <summary>
			/// Span that has been written to
			/// </summary>
			public ReadOnlySpan<byte> WrittenSpan => WrittenMemory.Span;

			/// <summary>
			/// Memory that has been written to
			/// </summary>
			public ReadOnlyMemory<byte> WrittenMemory => Data.Slice(0, Length);
		}

		readonly List<Chunk> _chunks = [];
		readonly int _chunkSize;
		Chunk _currentChunk;

		/// <summary>
		/// Accessor for the allocated chunks
		/// </summary>
		protected IReadOnlyList<Chunk> Chunks => _chunks;

		/// <summary>
		/// Length of the current sequence
		/// </summary>
		public int Length { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		protected ChunkedMemoryWriterBase(Chunk initialChunk, int chunkSize)
		{
			_currentChunk = initialChunk;
			_chunks.Add(initialChunk);
			_chunkSize = chunkSize;
		}

		/// <summary>
		/// Creates a new chunk for the buffer
		/// </summary>
		protected abstract Chunk CreateChunk(int runningIndex, int size);

		/// <summary>
		/// Clear the current builder
		/// </summary>
		public void Clear()
		{
			foreach (Chunk chunk in _chunks)
			{
				chunk.Release();
			}

			_chunks.Clear();
			Length = 0;
		}

		/// <inheritdoc/>
		public Span<byte> GetSpan(int minSize) => GetMemory(minSize).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int sizeHint)
			=> GetMemory(0, sizeHint);

		/// <summary>
		/// Gets a block of memory to write to, preserving existing (but uncommitted) data in previously returned buffers.
		/// </summary>
		/// <param name="usedSize">Size of data in the buffer which has not been committed via a call to Advance</param>
		/// <param name="desiredSize">Desired size of the buffer to write to</param>
		/// <returns>New block of memory to write to</returns>
		public Memory<byte> GetMemory(int usedSize, int desiredSize)
		{
			int requiredSize = _currentChunk.Length + Math.Max(desiredSize, 1);
			if (requiredSize > _currentChunk.Data.Length)
			{
				int runningIndex = _currentChunk.RunningIndex + _currentChunk.Length;
				Chunk nextChunk = CreateChunk(runningIndex, Math.Max(desiredSize, _chunkSize));

				if (usedSize > 0)
				{
					ReadOnlyMemory<byte> usedData = _currentChunk.Data.Slice(_currentChunk.Length, usedSize);
					usedData.CopyTo(nextChunk.Data);
				}
				if (_currentChunk.Length == 0)
				{
					_currentChunk.Release();
					_chunks.RemoveAt(_chunks.Count - 1);
				}

				_currentChunk = nextChunk;
				_chunks.Add(_currentChunk);
			}
			return _currentChunk.Data.Slice(_currentChunk.Length);
		}

		/// <inheritdoc/>
		public void Advance(int length)
		{
			if (length < 0 || _currentChunk.Length + length > _currentChunk.Data.Length)
			{
				throw new ArgumentException("Length exceeds possible size of data written to output buffer", nameof(length));
			}

			_currentChunk.Length += length;
			Length += length;
		}

		/// <summary>
		/// Appends data in this builder to the given sequence
		/// </summary>
		/// <param name="builder">Sequence builder</param>
		public void AppendTo(ReadOnlySequenceBuilder<byte> builder)
		{
			foreach (Chunk chunk in _chunks)
			{
				builder.Append(chunk.WrittenMemory);
			}
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far
		/// </summary>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence()
		{
			ReadOnlySequenceBuilder<byte> builder = new();
			AppendTo(builder);
			return builder.Construct();
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far, starting at the given offset
		/// </summary>
		/// <param name="offset">Offset to start from</param>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence(int offset)
		{
			// TODO: could do a binary search for offset and work forwards from there
			return AsSequence().Slice(offset);
		}

		/// <summary>
		/// Gets a sequence representing the bytes that have been written so far, starting at the given offset and 
		/// </summary>
		/// <param name="offset">Offset to start from</param>
		/// <param name="length">Length of the sequence to return</param>
		/// <returns>Sequence of bytes</returns>
		public ReadOnlySequence<byte> AsSequence(int offset, int length)
		{
			// TODO: could do a binary search for offset and work fowards from there
			ReadOnlySequence<byte> sequence = AsSequence();
			return sequence.Slice(offset, Math.Min(length, sequence.Length - offset));
		}

		/// <summary>
		/// Copies the data to the given output span
		/// </summary>
		/// <param name="span">Span to write to</param>
		public void CopyTo(Span<byte> span)
		{
			foreach (Chunk chunk in _chunks)
			{
				Span<byte> output = span.Slice(chunk.RunningIndex);
				chunk.WrittenSpan.CopyTo(output);
			}
		}

		/// <summary>
		/// Copies the data to a stream
		/// </summary>
		/// <param name="stream">Stream to write to</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task CopyToAsync(Stream stream, CancellationToken cancellationToken = default)
		{
			foreach (Chunk chunk in _chunks)
			{
				await stream.WriteAsync(chunk.WrittenMemory, cancellationToken);
			}
		}

		/// <summary>
		/// Create a byte array from the sequence
		/// </summary>
		/// <returns>Byte array containing the current buffer data</returns>
		public byte[] ToByteArray()
		{
			byte[] data = new byte[Length];
			CopyTo(data);
			return data;
		}
	}

	/// <summary>
	/// Class for building byte sequences, similar to StringBuilder. Allocates memory in chunks to avoid copying data.
	/// </summary>
	public sealed class ChunkedArrayMemoryWriter : ChunkedMemoryWriterBase
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="chunkSize">Size of each chunk</param>
		public ChunkedArrayMemoryWriter(int chunkSize = 4096)
			: this(chunkSize, chunkSize)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="initialChunkSize">Size of the initial chunk</param>
		/// <param name="nextChunkSize">Size of subsequent chunks</param>
		public ChunkedArrayMemoryWriter(int initialChunkSize, int nextChunkSize)
			: base(new Chunk(0, new byte[initialChunkSize]), nextChunkSize)
		{ }

		/// <inheritdoc/>
		protected override Chunk CreateChunk(int runningIndex, int size) => new(runningIndex, new byte[size]);
	}

	/// <summary>
	/// Class for building byte sequences, similar to StringBuilder. Allocates memory in chunks using a pool allocator to avoid copying data.
	/// </summary>
	public sealed class ChunkedMemoryWriter : ChunkedMemoryWriterBase, IDisposable
	{
		class PooledChunk : Chunk
		{
			public readonly IMemoryOwner<byte> Owner;

			public PooledChunk(int runningIndex, IMemoryOwner<byte> owner)
				: base(runningIndex, owner.Memory)
			{
				Owner = owner;
			}

			public override void Release() => Owner.Dispose();
		}

		readonly IMemoryAllocator<byte> _allocator;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="chunkSize"></param>
		public ChunkedMemoryWriter(int chunkSize)
			: this(chunkSize, chunkSize)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="initialSize">Size of the initial chunk</param>
		/// <param name="chunkSize">Default size for subsequent chunks</param>
		public ChunkedMemoryWriter(int initialSize = 4096, int chunkSize = 4096)
			: this(PoolAllocator.Shared, initialSize, chunkSize)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="allocator">Allocator to draw from</param>
		/// <param name="initialSize">Size of the initial chunk</param>
		/// <param name="chunkSize">Default size for subsequent chunks</param>
		public ChunkedMemoryWriter(IMemoryAllocator<byte> allocator, int initialSize = 4096, int chunkSize = 4096)
			: base(new PooledChunk(0, allocator.Alloc(initialSize, null)), chunkSize)
		{
			_allocator = allocator;
		}

		/// <inheritdoc/>
		protected override Chunk CreateChunk(int runningIndex, int size) => new PooledChunk(runningIndex, _allocator.Alloc(size, null));

		/// <inheritdoc/>
		public void Dispose()
		{
			Clear();
		}
	}

	/// <summary>
	/// Implementation of <see cref="ChunkedMemoryWriterBase"/> which takes pages from a <see cref="IMemoryAllocator{Byte}"/>
	/// </summary>
	public sealed class RefCountedMemoryWriter : ChunkedMemoryWriterBase, IDisposable
	{
		class BufferChunk : Chunk
		{
			public readonly IRefCountedHandle<Memory<byte>> Handle;

			public BufferChunk(int runningIndex, IRefCountedHandle<Memory<byte>> handle)
				: base(runningIndex, handle.Target)
			{
				Handle = handle;
			}

			public override void Release()
			{
				Handle.Dispose();
			}
		}

		class ArrayDisposer : IDisposable
		{
			readonly IDisposable[] _handles;

			public ArrayDisposer(IDisposable[] handles) => _handles = handles;

			public void Dispose()
			{
				for (int idx = 0; idx < _handles.Length; idx++)
				{
					_handles[idx].Dispose();
				}
			}
		}

		readonly IMemoryAllocator<byte> _allocator;
		readonly object? _allocationTag;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="allocator">Allocator for buffers</param>
		/// <param name="chunkSize"></param>
		/// <param name="allocationTag">Tag for allocated memory</param>
		public RefCountedMemoryWriter(IMemoryAllocator<byte> allocator, int chunkSize, object? allocationTag = null)
			: this(allocator, chunkSize, chunkSize, allocationTag)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="allocator">Allocator for buffers</param>
		/// <param name="initialSize">Size of the initial chunk</param>
		/// <param name="chunkSize">Default size for subsequent chunks</param>
		/// <param name="allocationTag">Tag for allocated memory</param>
		public RefCountedMemoryWriter(IMemoryAllocator<byte> allocator, int initialSize, int chunkSize, object? allocationTag)
			: base(new BufferChunk(0, RefCountedHandle.Create(allocator.Alloc(initialSize, allocationTag))), chunkSize)
		{
			_allocator = allocator;
			_allocationTag = allocationTag;
		}

		/// <inheritdoc/>
		protected override Chunk CreateChunk(int runningIndex, int size) => new BufferChunk(runningIndex, RefCountedHandle.Create(_allocator.Alloc(size, _allocationTag)));

		/// <inheritdoc/>
		public void Dispose()
		{
			Clear();
		}

		/// <summary>
		/// Returns a contiguous block of memory containing the written data
		/// </summary>
		public IRefCountedHandle<ReadOnlyMemory<byte>> AsRefCountedMemory() => AsRefCountedMemory(0, Length);

		/// <summary>
		/// Returns a contiguous block of memory containing the written data
		/// </summary>
		public IRefCountedHandle<ReadOnlyMemory<byte>> AsRefCountedMemory(int offset) => AsRefCountedMemory(offset, Length - offset);

		/// <summary>
		/// Returns a contiguous block of memory containing the written data
		/// </summary>
		public IRefCountedHandle<ReadOnlyMemory<byte>> AsRefCountedMemory(int offset, int length)
		{
			IRefCountedHandle<ReadOnlySequence<byte>> sequence = AsRefCountedSequence(offset, length);
			if (sequence.Target.IsSingleSegment)
			{
				return RefCountedHandle.Create(sequence.Target.First, sequence);
			}

			try
			{
				IRefCountedHandle<Memory<byte>> allocation = RefCountedHandle.Create(_allocator.Alloc(length, _allocationTag));
				sequence.Target.CopyTo(allocation.Target.Span);
				return RefCountedHandle.Create<ReadOnlyMemory<byte>>(allocation.Target.Slice(0, length), allocation);
			}
			finally
			{
				sequence.Dispose();
			}
		}

		/// <summary>
		/// Creates a reference counted sequence from the allocated data
		/// </summary>
		public IRefCountedHandle<ReadOnlySequence<byte>> AsRefCountedSequence() => AsRefCountedSequence(0, Length);

		/// <summary>
		/// Creates a reference counted sequence from the allocated data
		/// </summary>
		public IRefCountedHandle<ReadOnlySequence<byte>> AsRefCountedSequence(int offset) => AsRefCountedSequence(offset, Length - offset);

		/// <summary>
		/// Creates a reference counted sequence from the allocated data
		/// </summary>
		public IRefCountedHandle<ReadOnlySequence<byte>> AsRefCountedSequence(int offset, int length)
		{
			int minChunkIdx = 0;
			while (minChunkIdx + 1 < Chunks.Count && offset > Chunks[minChunkIdx + 1].RunningIndex)
			{
				minChunkIdx++;
			}

			int maxChunkIdx = minChunkIdx + 1;
			while (maxChunkIdx < Chunks.Count && offset + length > Chunks[maxChunkIdx].RunningIndex)
			{
				maxChunkIdx++;
			}

			IDisposable[] handles = new IDisposable[maxChunkIdx - minChunkIdx];
			for (int chunkIdx = minChunkIdx; chunkIdx < maxChunkIdx; chunkIdx++)
			{
				handles[chunkIdx - minChunkIdx] = ((BufferChunk)Chunks[chunkIdx]).Handle.AddRef();
			}

			return new RefCountedHandle<ReadOnlySequence<byte>>(AsSequence(offset, length), new ArrayDisposer(handles));
		}
	}

	/// <summary>
	/// Extension methods for <see cref="IMemoryWriter"/>
	/// </summary>
	public static class MemoryWriterExtensions
	{
		/// <summary>
		/// Writes a byte to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="minSize">Minimum size for the returned span</param>
		public static Span<byte> GetSpan(this IMemoryWriter writer, int minSize) => writer.GetMemory(minSize).Span;

		/// <summary>
		/// Writes a byte to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="length">Length of the required span</param>
		public static Memory<byte> GetMemoryAndAdvance(this IMemoryWriter writer, int length)
		{
			Memory<byte> memory = writer.GetMemory(length);
			writer.Advance(length);
			return memory.Slice(0, length); // Returned memory may be larger than requested
		}

		/// <summary>
		/// Writes a byte to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="length">Length of the required span</param>
		public static Span<byte> GetSpanAndAdvance(this IMemoryWriter writer, int length) => GetMemoryAndAdvance(writer, length).Span;

		/// <summary>
		/// Writes a boolean to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteBoolean(this IMemoryWriter writer, bool value)
		{
			WriteUInt8(writer, value ? (byte)1 : (byte)0);
		}

		/// <summary>
		/// Writes a byte to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteInt8(this IMemoryWriter writer, sbyte value)
		{
			WriteUInt8(writer, (byte)value);
		}

		/// <summary>
		/// Writes a byte to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUInt8(this IMemoryWriter writer, byte value)
		{
			writer.GetSpan(1)[0] = (byte)value;
			writer.Advance(1);
		}

		/// <summary>
		/// Writes an int16 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteInt16(this IMemoryWriter writer, short value)
		{
			Span<byte> span = GetSpan(writer, sizeof(short));
			BinaryPrimitives.WriteInt16LittleEndian(span, value);
			writer.Advance(sizeof(short));
		}

		/// <summary>
		/// Writes a uint16 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUInt16(this IMemoryWriter writer, ushort value)
		{
			Span<byte> span = GetSpan(writer, sizeof(ushort));
			BinaryPrimitives.WriteUInt16LittleEndian(span, value);
			writer.Advance(sizeof(ushort));
		}

		/// <summary>
		/// Writes an int32 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteInt32(this IMemoryWriter writer, int value)
		{
			Span<byte> span = GetSpan(writer, sizeof(int));
			BinaryPrimitives.WriteInt32LittleEndian(span, value);
			writer.Advance(sizeof(int));
		}

		/// <summary>
		/// Writes a uint32 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUInt32(this IMemoryWriter writer, uint value)
		{
			Span<byte> span = GetSpan(writer, sizeof(uint));
			BinaryPrimitives.WriteUInt32LittleEndian(span, value);
			writer.Advance(sizeof(uint));
		}

		/// <summary>
		/// Writes an int64 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteInt64(this IMemoryWriter writer, long value)
		{
			Span<byte> span = GetSpan(writer, sizeof(long));
			BinaryPrimitives.WriteInt64LittleEndian(span, value);
			writer.Advance(sizeof(long));
		}

		/// <summary>
		/// Writes a uint64 to the memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUInt64(this IMemoryWriter writer, ulong value)
		{
			Span<byte> span = GetSpan(writer, sizeof(ulong));
			BinaryPrimitives.WriteUInt64LittleEndian(span, value);
			writer.Advance(sizeof(ulong));
		}

		/// <summary>
		/// Writes a DateTime to the memory writer
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteDateTime(this IMemoryWriter writer, DateTime value)
		{
			ulong encoded = ((ulong)value.Ticks << 2) | (ulong)value.Kind;
			writer.WriteUnsignedVarInt(encoded);
		}

		/// <summary>
		/// Writes a Guid to the memory writer using NET order serialization.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="guid">Value to write</param>
		public static void WriteGuidNetOrder(this IMemoryWriter writer, Guid guid)
		{
			Memory<byte> buffer = writer.GetMemory(16);
			if (!guid.TryWriteBytes(buffer.Slice(0, 16).Span))
			{
				throw new InvalidOperationException("Unable to write guid to buffer");
			}
			writer.Advance(16);
		}

		/// <summary>
		/// Writes a Guid to the memory writer (uses UE rather than NET serialization order).
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="guid">Value to write</param>
		public static void WriteGuidUnrealOrder(this IMemoryWriter writer, Guid guid)
		{
			GuidUtils.WriteGuidUnrealOrder(writer.GetSpan(16), guid);
			writer.Advance(16);
		}

		/// <summary>
		/// Appends a span of bytes to the buffer, without a length field.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="span">Bytes to write</param>
		public static void WriteFixedLengthBytes(this IMemoryWriter writer, ReadOnlySpan<byte> span)
		{
			Span<byte> target = GetSpanAndAdvance(writer, span.Length);
			span.CopyTo(target);
		}

		/// <summary>
		/// Appends a sequence of bytes to the buffer, without a length field.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="sequence">Sequence to append</param>
		public static void WriteFixedLengthBytes(this IMemoryWriter writer, ReadOnlySequence<byte> sequence)
		{
			Span<byte> span = GetSpanAndAdvance(writer, (int)sequence.Length);
			sequence.CopyTo(span);
		}

		/// <summary>
		/// Writes a variable length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteList<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<T> writeItem)
		{
			WriteVariableLengthArray(writer, list, writeItem);
		}

		/// <summary>
		/// Writes a variable length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteList<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<IMemoryWriter, T> writeItem)
		{
			WriteVariableLengthArray(writer, list, writeItem);
		}

		/// <summary>
		/// Writes a variable length span of bytes
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="bytes">The bytes to write</param>
		public static void WriteVariableLengthBytes(this IMemoryWriter writer, ReadOnlySpan<byte> bytes)
		{
			int lengthBytes = VarInt.MeasureUnsigned(bytes.Length);
			Span<byte> span = GetSpanAndAdvance(writer, lengthBytes + bytes.Length);
			VarInt.WriteUnsigned(span, bytes.Length);
			bytes.CopyTo(span[lengthBytes..]);
		}

		/// <summary>
		/// Appends a sequence of bytes to the buffer, without a length field.
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="sequence">Sequence to append</param>
		public static void WriteVariableLengthBytes(this IMemoryWriter writer, ReadOnlySequence<byte> sequence)
		{
			int lengthBytes = VarInt.MeasureUnsigned((int)sequence.Length);
			Span<byte> span = GetSpanAndAdvance(writer, (int)(lengthBytes + sequence.Length));
			VarInt.WriteUnsigned(span, (int)sequence.Length);
			sequence.CopyTo(span[lengthBytes..]);
		}

		/// <summary>
		/// Writes a variable length span of bytes
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="bytes">The bytes to write</param>
		public static void WriteVariableLengthBytesWithInt32Length(this IMemoryWriter writer, ReadOnlySpan<byte> bytes)
		{
			writer.WriteInt32(bytes.Length);
			writer.WriteFixedLengthBytes(bytes);
		}

		/// <summary>
		/// Writes a variable length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteVariableLengthArray<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<T> writeItem)
		{
			writer.WriteUnsignedVarInt(list.Count);
			WriteFixedLengthArray(writer, list, writeItem);
		}

		/// <summary>
		/// Writes a variable length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteVariableLengthArray<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<IMemoryWriter, T> writeItem)
		{
			writer.WriteUnsignedVarInt(list.Count);
			WriteFixedLengthArray(writer, list, writeItem);
		}

		/// <summary>
		/// Writes a variable length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="array">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteVariableLengthArrayWithInt32Length<T>(this IMemoryWriter writer, T[] array, Action<T> writeItem)
		{
			WriteInt32(writer, array.Length);
			WriteFixedLengthArray(writer, array, writeItem);
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public static void WriteFixedLengthArray<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<T> writeItem)
		{
			for (int idx = 0; idx < list.Count; idx++)
			{
				writeItem(list[idx]);
			}
		}
		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public static void WriteFixedLengthArray<T>(this IMemoryWriter writer, IReadOnlyList<T> list, Action<IMemoryWriter, T> writeItem)
		{
			for (int idx = 0; idx < list.Count; idx++)
			{
				writeItem(writer, list[idx]);
			}
		}

		/// <summary>
		/// Writes a dictionary to the writer
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="dictionary">The dictionary to write</param>
		/// <param name="writeKey">Delegate to write an individual key</param>
		/// <param name="writeValue">Delegate to write an individual value</param>
		public static void WriteDictionary<TKey, TValue>(this IMemoryWriter writer, IReadOnlyDictionary<TKey, TValue> dictionary, Action<TKey> writeKey, Action<TValue> writeValue) where TKey : notnull
		{
			writer.WriteUnsignedVarInt(dictionary.Count);
			foreach (KeyValuePair<TKey, TValue> kvp in dictionary)
			{
				writeKey(kvp.Key);
				writeValue(kvp.Value);
			}
		}

		/// <summary>
		/// Writes a dictionary to the writer
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="dictionary">The dictionary to write</param>
		/// <param name="writeKey">Delegate to write an individual key</param>
		/// <param name="writeValue">Delegate to write an individual value</param>
		public static void WriteDictionary<TKey, TValue>(this IMemoryWriter writer, IReadOnlyDictionary<TKey, TValue> dictionary, Action<IMemoryWriter, TKey> writeKey, Action<IMemoryWriter, TValue> writeValue) where TKey : notnull
		{
			writer.WriteUnsignedVarInt(dictionary.Count);
			foreach (KeyValuePair<TKey, TValue> kvp in dictionary)
			{
				writeKey(writer, kvp.Key);
				writeValue(writer, kvp.Value);
			}
		}

		/// <summary>
		/// Write a string to memory
		/// </summary>
		/// <param name="writer">Reader to deserialize from</param>
		/// <param name="str">The string to be written</param>
		public static void WriteString(this IMemoryWriter writer, string str) => WriteString(writer, str, Encoding.UTF8);

		/// <summary>
		/// Write a string to memory
		/// </summary>
		/// <param name="writer">Reader to deserialize from</param>
		/// <param name="str">The string to be written</param>
		/// <param name="encoding">Encoding to use for the string</param>
		public static void WriteString(this IMemoryWriter writer, string str, Encoding encoding)
		{
			int stringBytes = encoding.GetByteCount(str);
			int lengthBytes = VarInt.MeasureUnsigned(stringBytes);

			Span<byte> span = writer.GetSpan(lengthBytes + stringBytes);
			VarInt.WriteUnsigned(span, stringBytes);
			encoding.GetBytes(str, span[lengthBytes..]);

			writer.Advance(lengthBytes + stringBytes);
		}

		/// <summary>
		/// Write a nullable string to memory
		/// </summary>
		/// <param name="writer">Reader to deserialize from</param>
		/// <param name="str">The string to be written</param>
		public static void WriteOptionalString(this IMemoryWriter writer, string? str) => WriteOptionalString(writer, str, Encoding.UTF8);

		/// <summary>
		/// Write a nullable string to memory
		/// </summary>
		/// <param name="writer">Reader to deserialize from</param>
		/// <param name="str">The string to be written</param>
		/// <param name="encoding">Encoding to use for the string</param>
		public static void WriteOptionalString(this IMemoryWriter writer, string? str, Encoding encoding)
		{
			if (str == null)
			{
				Span<byte> span = writer.GetSpan(1);
				span[0] = 0;

				writer.Advance(1);
			}
			else
			{
				int stringBytes = encoding.GetByteCount(str);
				int lengthBytes = VarInt.MeasureUnsigned(stringBytes + 1);

				Span<byte> span = writer.GetSpan(lengthBytes + stringBytes);
				VarInt.WriteUnsigned(span, stringBytes + 1);
				encoding.GetBytes(str, span[lengthBytes..]);

				writer.Advance(lengthBytes + stringBytes);
			}
		}
	}
}
