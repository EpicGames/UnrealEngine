// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Implementation of <see cref="Stream"/> which writes data to a <see cref="IMemoryWriter"/>
	/// </summary>
	public class MemoryWriterStream : Stream
	{
		readonly IMemoryWriter _writer;
		long _length;

		/// <summary>
		/// Constructor
		/// </summary>
		public MemoryWriterStream(IMemoryWriter writer)
		{
			_writer = writer;
		}

		/// <inheritdoc/>
		public override bool CanRead => false;

		/// <inheritdoc/>
		public override bool CanSeek => false;

		/// <inheritdoc/>
		public override bool CanWrite => true;

		/// <inheritdoc/>
		public override long Length => _length;

		/// <inheritdoc/>
		public override long Position { get => _length; set => throw new NotSupportedException(); }

		/// <inheritdoc/>
		public override void Flush() { }

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void Write(ReadOnlySpan<byte> buffer)
		{
			while (buffer.Length > 0)
			{
				Memory<byte> current = _writer.GetMemory();

				int copyLength = Math.Min(current.Length, buffer.Length);
				buffer.Slice(0, copyLength).CopyTo(current.Span);
				_writer.Advance(copyLength);
				_length += copyLength;

				buffer = buffer.Slice(copyLength);
			}
		}

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => Write(buffer.AsSpan(offset, count));
	}
}
