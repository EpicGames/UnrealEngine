// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Threading.Tasks.Dataflow;

namespace EpicGames.Core
{
	/// <summary>
	/// Async equivalents for parallel methods
	/// </summary>
	public static class ParallelTask
	{
		/// <summary>
		/// Execute a large number of tasks in parallel over a collection. Parallel.ForEach() method isn't generally compatible with asynchronous programming, because any
		/// exceptions are thrown on the 
		/// </summary>
		/// <param name="fromInclusive">The starting index</param>
		/// <param name="toExclusive">The last index, exclusive</param>
		/// <param name="action">Action to perform for each item in the collection</param>
		/// <returns>Async task</returns>
		public static Task ForAsync(int fromInclusive, int toExclusive, Action<int> action)
		{
			return ForEachAsync(Enumerable.Range(fromInclusive, toExclusive - fromInclusive), action);
		}

		/// <summary>
		/// Execute a large number of tasks in parallel over a collection. Parallel.ForEach() method isn't generally compatible with asynchronous programming, because any
		/// exceptions are thrown on the 
		/// </summary>
		/// <typeparam name="T">The collection type</typeparam>
		/// <param name="collection">The collection to iterate over</param>
		/// <param name="action">Action to perform for each item in the collection</param>
		/// <returns>Async task</returns>
		public static Task ForEachAsync<T>(IEnumerable<T> collection, Action<T> action)
		{
			ExecutionDataflowBlockOptions options = new();
			options.MaxDegreeOfParallelism = DataflowBlockOptions.Unbounded;

			ActionBlock<T> actions = new(action, options);
			foreach (T item in collection)
			{
				actions.Post(item);
			}
			actions.Complete();

			return actions.Completion;
		}

		/// <summary>
		/// Execute a large number of tasks in parallel over a collection. Parallel.ForEach() method isn't generally compatible with asynchronous programming, because any
		/// exceptions are thrown on the 
		/// </summary>
		/// <typeparam name="T">The collection type</typeparam>
		/// <param name="collection">The collection to iterate over</param>
		/// <param name="action">Action to perform for each item in the collection</param>
		/// <param name="maxDegreeOfParallelism">Maximum degree of parallelism</param>
		/// <returns>Async task</returns>
		public static Task ForEachAsync<T>(IEnumerable<T> collection, Func<T, Task> action, int maxDegreeOfParallelism)
		{
			ExecutionDataflowBlockOptions options = new();
			options.MaxDegreeOfParallelism = maxDegreeOfParallelism;

			ActionBlock<T> actions = new(action, options);
			foreach (T item in collection)
			{
				actions.Post(item);
			}
			actions.Complete();

			return actions.Completion;
		}
	}
}
