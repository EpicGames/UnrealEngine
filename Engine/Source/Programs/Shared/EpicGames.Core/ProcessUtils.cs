// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text;
using System.Threading;
using Microsoft.Extensions.Logging;
using Microsoft.Win32.SafeHandles;

namespace EpicGames.Core
{
	/// <summary>
	/// Includes utilities for managing processes
	/// </summary>
	public static class ProcessUtils
	{
		const uint PROCESS_TERMINATE = 0x0001;
		internal const int PROCESS_VM_READ = 0x0010;
		internal const int PROCESS_QUERY_LIMITED_INFORMATION = 0x1000;

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern SafeProcessHandle OpenProcess(uint dwDesiredAccess, bool bInheritHandle, uint processId);

		[DllImport("kernel32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool TerminateProcess(SafeProcessHandle hProcess, uint uExitCode);

		[DllImport("kernel32.dll", SetLastError = true)]
#pragma warning disable CA1838 // Avoid 'StringBuilder' parameters for P/Invokes
		static extern int QueryFullProcessImageName([In] SafeProcessHandle hProcess, [In] int dwFlags, [Out] StringBuilder lpExeName, ref int lpdwSize);
#pragma warning restore CA1838 // Avoid 'StringBuilder' parameters for P/Invokes

		[DllImport("Psapi.dll", SetLastError = true)]
		static extern bool EnumProcesses([MarshalAs(UnmanagedType.LPArray, ArraySubType = UnmanagedType.U4)][In][Out] uint[] processIds, int arraySizeBytes, [MarshalAs(UnmanagedType.U4)] out int bytesCopied);

		const uint WAIT_FAILED = 0xffffffff;

		[DllImport("kernel32.dll")]
		static extern uint WaitForMultipleObjects(int nCount, IntPtr[] lpHandles, bool bWaitAll, uint dwMilliseconds);

		[StructLayout(LayoutKind.Sequential)]
		struct ProcessBasicInformation
		{
			// These members must match PROCESS_BASIC_INFORMATION
			public IntPtr _exitStatus;
			public IntPtr _pebBaseAddress;
			public IntPtr _affinityMask;
			public IntPtr _basePriority;
			public IntPtr _uniqueProcessId;
			public IntPtr _inheritedFromUniqueProcessId;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct UnicodeString
		{
			public ushort _length;
			public ushort _maximumLength;
			public IntPtr _buffer;
		}

		const int ProcessBasicInformationClass = 0;

		[DllImport("ntdll.dll")]
		private static extern int NtQueryInformationProcess(IntPtr processHandle, int processInformationClass, ref ProcessBasicInformation processInformation, int processInformationLength, out int returnLength);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool ReadProcessMemory(
			IntPtr hProcess,
			IntPtr lpBaseAddress,
			byte[] lpBuffer,
			int dwSize,
			out int lpNumberOfBytesRead);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool ReadProcessMemory(
			IntPtr hProcess,
			IntPtr lpBaseAddress,
			ref IntPtr lpBuffer,
			int dwSize,
			out int lpNumberOfBytesRead);

		/// <summary>
		/// Attempts to terminate all processes matching a predicate
		/// </summary>
		/// <param name="predicate">The predicate for whether to terminate a process</param>
		/// <param name="logger">Logging device</param>
		/// <param name="cancellationToken">Cancellation token to abort the search</param>
		/// <returns>True if the process succeeded</returns>
		public static bool TerminateProcesses(Predicate<FileReference> predicate, ILogger logger, CancellationToken cancellationToken)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return TerminateProcessesWin32(predicate, logger, cancellationToken);
			}
			else
			{
				return TerminateProcessesGenericPlatform(predicate, logger, cancellationToken);
			}
		}

		/// <summary>
		/// Attempts to terminate all processes matching a predicate
		/// </summary>
		/// <param name="predicate">The predicate for whether to terminate a process</param>
		/// <param name="logger">Logging device</param>
		/// <param name="cancellationToken">Cancellation token to abort the search</param>
		/// <returns>True if the process succeeded</returns>
		[SupportedOSPlatform("windows")]
		static bool TerminateProcessesWin32(Predicate<FileReference> predicate, ILogger logger, CancellationToken cancellationToken)
		{
			Dictionary<int, int> processToCount = [];
			for (; ; )
			{
				cancellationToken.ThrowIfCancellationRequested();

				// Enumerate the processes
				int numProcessIds;
				uint[] processIds = new uint[512];
				for (; ; )
				{
					int maxBytes = processIds.Length * sizeof(uint);
					int numBytes = 0;
					if (!EnumProcesses(processIds, maxBytes, out numBytes))
					{
						throw new Win32ExceptionWithCode("Unable to enumerate processes");
					}
					if (numBytes < maxBytes)
					{
						numProcessIds = numBytes / sizeof(uint);
						break;
					}
					processIds = new uint[processIds.Length + 256];
				}

				// Find the processes to terminate
				List<SafeProcessHandle> waitHandles = [];
				try
				{
					// Open each process in turn
					foreach (uint processId in processIds)
					{
						SafeProcessHandle handle = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION | PROCESS_TERMINATE, false, processId);
						try
						{
							if (!handle.IsInvalid)
							{
								int characters = 260;
								StringBuilder buffer = new(characters);
								if (QueryFullProcessImageName(handle, 0, buffer, ref characters) > 0)
								{
									FileReference imageFile = new(buffer.ToString(0, characters));
									if (predicate(imageFile))
									{
										logger.LogInformation("Terminating {ImageName} ({ProcessId})", imageFile, processId);
										if (TerminateProcess(handle, 9))
										{
											waitHandles.Add(handle);
										}
										else
										{
											logger.LogInformation("Failed call to TerminateProcess ({Code})", Marshal.GetLastWin32Error());
										}
										handle.SetHandleAsInvalid();
									}
								}
							}
						}
						finally
						{
							handle.Dispose();
						}
					}

					// If there's nothing to do, exit immediately
					if (waitHandles.Count == 0)
					{
						return true;
					}

					// Wait for them all to complete
					if (WaitForMultipleObjects(waitHandles.Count, [.. waitHandles.Select(x => x.DangerousGetHandle())], true, 10 * 1000) == WAIT_FAILED)
					{
						logger.LogInformation("Failed call to WaitForMultipleObjects ({Code})", Marshal.GetLastWin32Error());
					}
				}
				finally
				{
					foreach (SafeProcessHandle waitHandle in waitHandles)
					{
						waitHandle.Close();
					}
				}
			}
		}

		/// <summary>
		/// Attempts to terminate all processes matching a predicate
		/// </summary>
		/// <param name="predicate">The predicate for whether to terminate a process</param>
		/// <param name="logger">Logging device</param>
		/// <param name="cancellationToken">Cancellation token to abort the search</param>
		/// <returns>True if the process succeeded</returns>
		static bool TerminateProcessesGenericPlatform(Predicate<FileReference> predicate, ILogger logger, CancellationToken cancellationToken)
		{
			bool result = true;
			Dictionary<(int, DateTime), int> processToCount = [];
			for (; ; )
			{
				cancellationToken.ThrowIfCancellationRequested();

				bool bNoMatches = true;

				// Enumerate all the processes
				Process[] processes = Process.GetProcesses();
				foreach (Process process in processes)
				{
					// Attempt to get the image file. Ignore exceptions trying to fetch metadata for processes we don't have access to.
					FileReference? imageFile;
					try
					{
						imageFile = new FileReference(process.MainModule!.FileName!);
					}
					catch
					{
						imageFile = null;
					}

					// Test whether to terminate this process
					if (imageFile != null && predicate(imageFile))
					{
						// Get a unique id for the process, given that process ids are recycled
						(int, DateTime) uniqueId;
						try
						{
							uniqueId = (process.Id, process.StartTime);
						}
						catch
						{
							uniqueId = (process.Id, DateTime.MinValue);
						}

						// Figure out whether to try and terminate this process
						const int MaxCount = 5;
						if (!processToCount.TryGetValue(uniqueId, out int count) || count < MaxCount)
						{
							bNoMatches = false;
							try
							{
								logger.LogInformation("Terminating {ImageName} ({ProcessId})", imageFile, process.Id);
								process.Kill(true);
								if (!process.WaitForExit(5 * 1000))
								{
									logger.LogInformation("Termination still pending; will retry...");
								}
							}
							catch (Exception ex)
							{
								count++;
								if (count > 1)
								{
									logger.LogInformation(ex, "Exception while querying basic process info for pid {ProcessId}; will retry.", process.Id);
								}
								else if (count == MaxCount)
								{
									logger.LogWarning(ex, "Unable to terminate process {ImageFile} ({ProcessId}): {Message}", imageFile, process.Id, ex.Message);
									result = false;
								}
								processToCount[uniqueId] = count;
							}
						}
					}
				}

				// Return once we reach this point and haven't found anything else to terminate
				if (bNoMatches)
				{
					return result;
				}
			}
		}

		/// <summary>
		/// Gets the parent process of a specified process
		/// </summary>
		/// <param name="process">The process</param>
		/// <returns>The parent process if running, otherwise null</returns>
		public static Process? GetParentProcess(Process process)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return GetParentProcessWin32(process);
			}
			else
			{
				throw new PlatformNotSupportedException();
			}
		}

		[SupportedOSPlatform("windows")]
		static Process? GetParentProcessWin32(Process process)
		{
			try
			{
				ProcessBasicInformation pbi = new();
				if (NtQueryInformationProcess(process.Handle, 0, ref pbi, Marshal.SizeOf(pbi), out _) == 0)
				{
					return Process.GetProcessById(pbi._inheritedFromUniqueProcessId.ToInt32());
				}
			}
			catch (Exception)
			{
			}
			return null;
		}

		/// <summary>
		/// Gets all ancestor processes of a specified windows process
		/// </summary>
		/// <param name="process">The process</param>
		/// <returns>IEnumerable containing the parent processes</returns>
		public static IEnumerable<Process> GetAncestorProcesses(Process process)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return GetAncestorProcessesWin32(process);
			}
			else
			{
				throw new PlatformNotSupportedException();
			}
		}

		[SupportedOSPlatform("windows")]
		static IEnumerable<Process> GetAncestorProcessesWin32(Process process)
		{
			Process? parentProcess = GetParentProcess(process);
			while (parentProcess != null)
			{
				yield return parentProcess;
				parentProcess = GetParentProcess(parentProcess);
			}
		}

		/// <summary>
		/// Gets the command line for a process by reading its PEB (Windows only)
		/// </summary>
		/// <param name="hProcess">Handle to the process (must have PROCESS_QUERY_LIMITED_INFORMATION and PROCESS_VM_READ access)</param>
		/// <returns>The command line string, or null if it could not be retrieved</returns>
		[SupportedOSPlatform("windows")]
		public static string? GetProcessCommandLine(SafeProcessHandle hProcess)
		{
			if (hProcess == null || hProcess.IsInvalid)
			{
				return null;
			}

			if (!GetProcessBasicInfo(hProcess, out IntPtr pebBaseAddress, out _))
			{
				return null;
			}

			if (pebBaseAddress == IntPtr.Zero)
			{
				return null;
			}

			return GetProcessCommandLineFromPeb(hProcess.DangerousGetHandle(), pebBaseAddress);
		}

		/// <summary>
		/// Gets the command line for a process by process ID (Windows only)
		/// </summary>
		/// <param name="processId">The process ID</param>
		/// <returns>The command line string, or null if it could not be retrieved</returns>
		[SupportedOSPlatform("windows")]
		public static string? GetProcessCommandLine(int processId)
		{
			try
			{
				using SafeProcessHandle hProcess = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION | PROCESS_VM_READ, false, (uint)processId);
				return GetProcessCommandLine(hProcess);
			}
			catch
			{
				return null;
			}
		}

		/// <summary>
		/// Gets the full image path for a process (Windows only)
		/// </summary>
		/// <param name="hProcess">Handle to the process (must have PROCESS_QUERY_LIMITED_INFORMATION access)</param>
		/// <returns>The full path to the process executable, or null if it could not be retrieved</returns>
		[SupportedOSPlatform("windows")]
		public static string? GetProcessImageName(SafeProcessHandle hProcess)
		{
			if (hProcess == null || hProcess.IsInvalid)
			{
				return null;
			}

			int capacity = 260;
			StringBuilder imageNameBuilder = new(capacity);
			if (QueryFullProcessImageName(hProcess, 0, imageNameBuilder, ref capacity) > 0)
			{
				return imageNameBuilder.ToString(0, capacity);
			}
			return null;
		}

		/// <summary>
		/// Gets the full image path for a process by process ID (Windows only)
		/// </summary>
		/// <param name="processId">The process ID</param>
		/// <returns>The full path to the process executable, or null if it could not be retrieved</returns>
		[SupportedOSPlatform("windows")]
		public static string? GetProcessImageName(int processId)
		{
			try
			{
				using SafeProcessHandle hProcess = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, false, (uint)processId);
				return GetProcessImageName(hProcess);
			}
			catch
			{
				return null;
			}
		}

		/// <summary>
		/// Gets the parent process ID for a process (Windows only)
		/// </summary>
		/// <param name="hProcess">Handle to the process (must have PROCESS_QUERY_LIMITED_INFORMATION access)</param>
		/// <returns>The parent process ID, or 0 if it could not be retrieved</returns>
		[SupportedOSPlatform("windows")]
		public static int GetParentProcessId(SafeProcessHandle hProcess)
		{
			if (hProcess == null || hProcess.IsInvalid)
			{
				return 0;
			}

			if (GetProcessBasicInfo(hProcess, out _, out int parentPid))
			{
				return parentPid;
			}
			return 0;
		}

		/// <summary>
		/// Gets basic process information including PEB address and parent PID
		/// </summary>
		[SupportedOSPlatform("windows")]
		private static bool GetProcessBasicInfo(SafeProcessHandle hProcess, out IntPtr pebBaseAddress, out int parentPid)
		{
			pebBaseAddress = IntPtr.Zero;
			parentPid = 0;

			try
			{
				ProcessBasicInformation pbi = new();
				int status = NtQueryInformationProcess(
					hProcess.DangerousGetHandle(),
					ProcessBasicInformationClass,
					ref pbi,
					Marshal.SizeOf(pbi),
					out _);

				if (status != 0)
				{
					return false;
				}

				pebBaseAddress = pbi._pebBaseAddress;
				parentPid = pbi._inheritedFromUniqueProcessId.ToInt32();
				return true;
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Gets the command line for a process by reading its PEB
		/// </summary>
		[SupportedOSPlatform("windows")]
		private static string? GetProcessCommandLineFromPeb(IntPtr hProcess, IntPtr pebBaseAddress)
		{
			try
			{
				// PEB layout differs by architecture:
				// 64-bit: ProcessParameters at offset 0x20
				// 32-bit: ProcessParameters at offset 0x10
				int processParametersOffset = IntPtr.Size == 8 ? 0x20 : 0x10;

				IntPtr processParametersPtr = IntPtr.Zero;
				if (!ReadProcessMemory(
					hProcess,
					pebBaseAddress + processParametersOffset,
					ref processParametersPtr,
					IntPtr.Size,
					out _))
				{
					return null;
				}

				// RTL_USER_PROCESS_PARAMETERS layout:
				// 64-bit: CommandLine UNICODE_STRING at offset 0x70
				// 32-bit: CommandLine UNICODE_STRING at offset 0x40
				int commandLineOffset = IntPtr.Size == 8 ? 0x70 : 0x40;

				// Read the UNICODE_STRING structure
				byte[] unicodeStringBytes = new byte[Marshal.SizeOf<UnicodeString>()];
				if (!ReadProcessMemory(
					hProcess,
					processParametersPtr + commandLineOffset,
					unicodeStringBytes,
					unicodeStringBytes.Length,
					out _))
				{
					return null;
				}

				GCHandle handle = GCHandle.Alloc(unicodeStringBytes, GCHandleType.Pinned);
				try
				{
					UnicodeString unicodeString = Marshal.PtrToStructure<UnicodeString>(handle.AddrOfPinnedObject());

					if (unicodeString._length == 0 || unicodeString._buffer == IntPtr.Zero)
					{
						return null;
					}

					// Read the actual command line string
					byte[] commandLineBytes = new byte[unicodeString._length];
					if (!ReadProcessMemory(
						hProcess,
						unicodeString._buffer,
						commandLineBytes,
						commandLineBytes.Length,
						out _))
					{
						return null;
					}

					return Encoding.Unicode.GetString(commandLineBytes);
				}
				finally
				{
					handle.Free();
				}
			}
			catch
			{
				return null;
			}
		}
	}
}
