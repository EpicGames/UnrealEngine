// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Core
{
	/// <summary>
	/// Keeps track of progress reporting of a child process
	/// </summary>
	public class ProgressValue
	{
		Tuple<string, float> _state = null!;
		readonly Stack<Tuple<float, float>> _ranges = new();

		/// <summary>
		/// CTOR
		/// </summary>
		public ProgressValue()
		{
			Clear();
		}

		/// <summary>
		/// Reset the progress tracking
		/// </summary>
		public void Clear()
		{
			_state = new Tuple<string, float>("Starting...", 0.0f);

			_ranges.Clear();
			_ranges.Push(new Tuple<float, float>(0.0f, 1.0f));
		}

		/// <summary>
		/// The current state
		/// </summary>
		public Tuple<string, float> Current => _state;

		/// <summary>
		/// Add new message
		/// </summary>
		/// <param name="message"></param>
		public void Set(string message)
		{
			if (_ranges.Count == 1)
			{
				_state = new Tuple<string, float>(message, _state.Item2);
			}
		}

		/// <summary>
		/// Add a new message with a fraction of how far it has progressed
		/// </summary>
		/// <param name="message"></param>
		/// <param name="fraction"></param>
		public void Set(string message, float fraction)
		{
			if (_ranges.Count == 1)
			{
				_state = new Tuple<string, float>(message, RelativeToAbsoluteFraction(fraction));
			}
			else
			{
				_state = new Tuple<string, float>(_state.Item1, RelativeToAbsoluteFraction(fraction));
			}
		}

		/// <summary>
		/// Update a new fraction of progress without updating the message
		/// </summary>
		/// <param name="fraction"></param>
		public void Set(float fraction)
		{
			_state = new Tuple<string, float>(_state.Item1, RelativeToAbsoluteFraction(fraction));
		}

		/// <summary>
		/// Increment the current progress
		/// </summary>
		/// <param name="fraction"></param>
		public void Increment(float fraction)
		{
			Set(_state.Item2 + RelativeToAbsoluteFraction(fraction));
		}

		/// <summary>
		/// Push a new range with a max fraction which indicates of far this scope can go
		/// </summary>
		/// <param name="maxFraction"></param>
		public void Push(float maxFraction)
		{
			_ranges.Push(new Tuple<float, float>(_state.Item2, RelativeToAbsoluteFraction(maxFraction)));
		}

		/// <summary>
		/// Pop the latest range as that step is now completed
		/// </summary>
		public void Pop()
		{
			if (_ranges.Count > 1)
			{
				_state = new Tuple<string, float>(_state.Item1, _ranges.Pop().Item2);
			}
		}

		float RelativeToAbsoluteFraction(float fraction)
		{
			Tuple<float, float> range = _ranges.Peek();
			return range.Item1 + (range.Item2 - range.Item1) * fraction;
		}
	}

	/// <summary>
	/// Parses child process output that contains the @progress directives to indicate their progress
	/// </summary>
	public static class ProgressTextReader
	{
		/// <summary>
		/// The directive used to indicate progress
		/// </summary>
		public const string DirectivePrefix = "@progress ";

		/// <summary>
		/// Parse progress value from a text line
		/// </summary>
		/// <param name="line"></param>
		/// <param name="value"></param>
		/// <returns></returns>
		public static string? ParseLine(string line, ProgressValue value)
		{
			string trimLine = line.Trim();
			if (trimLine.StartsWith(DirectivePrefix, StringComparison.Ordinal))
			{
				// Line that just contains a progress directive
				bool skipLine = false;
				ProcessInternal(trimLine.Substring(DirectivePrefix.Length), ref skipLine, value);
				return null;
			}
			else
			{
				bool skipLine = false;
				string remainingLine = line;

				// Look for a progress directive at the end of a line, in square brackets
				if (trimLine.EndsWith(']'))
				{
					for (int lastIdx = trimLine.Length - 2; lastIdx >= 0 && trimLine[lastIdx] != ']'; lastIdx--)
					{
						if (trimLine[lastIdx] == '[')
						{
							string directiveSubstring = trimLine.Substring(lastIdx + 1, trimLine.Length - lastIdx - 2);
							if (directiveSubstring.StartsWith(DirectivePrefix, StringComparison.Ordinal))
							{
								ProcessInternal(directiveSubstring.Substring(DirectivePrefix.Length), ref skipLine, value);
								remainingLine = line.Substring(0, lastIdx).TrimEnd();
							}
							break;
						}
					}
				}

				if (skipLine)
				{
					return null;
				}
				else
				{
					return remainingLine;
				}
			}
		}

		static void ProcessInternal(string line, ref bool skipLine, ProgressValue value)
		{
			List<string> tokens = ParseTokens(line);
			for (int tokenIdx = 0; tokenIdx < tokens.Count;)
			{
				float fraction;
				if (ReadFraction(tokens, ref tokenIdx, out fraction))
				{
					value.Set(fraction);
				}
				else if (tokens[tokenIdx] == "push")
				{
					tokenIdx++;
					if (ReadFraction(tokens, ref tokenIdx, out fraction))
					{
						value.Push(fraction);
					}
				}
				else if (tokens[tokenIdx] == "pop")
				{
					tokenIdx++;
					value.Pop();
				}
				else if (tokens[tokenIdx] == "increment")
				{
					tokenIdx++;
					if (ReadFraction(tokens, ref tokenIdx, out fraction))
					{
						value.Increment(fraction);
					}
				}
				else if (tokens[tokenIdx] == "skipline")
				{
					tokenIdx++;
					skipLine = true;
				}
				else if (tokens[tokenIdx].Length >= 2 && (tokens[tokenIdx][0] == '\'' || tokens[tokenIdx][0] == '\"') && tokens[tokenIdx].Last() == tokens[tokenIdx].First())
				{
					string message = tokens[tokenIdx++];
					value.Set(message.Substring(1, message.Length - 2));
				}
				else
				{
					tokenIdx++;
				}
			}
		}

		static List<string> ParseTokens(string line)
		{
			List<string> tokens = [];
			for (int idx = 0; ;)
			{
				// Skip whitespace
				while (idx < line.Length && Char.IsWhiteSpace(line[idx]))
				{
					idx++;
				}
				if (idx == line.Length)
				{
					break;
				}

				// Read the next token
				if (Char.IsLetterOrDigit(line[idx]))
				{
					int startIdx = idx++;
					while (idx < line.Length && Char.IsLetterOrDigit(line[idx]))
					{
						idx++;
					}
					tokens.Add(line.Substring(startIdx, idx - startIdx));
				}
				else if (line[idx] == '\'' || line[idx] == '\"')
				{
					int startIdx = idx++;
					while (idx < line.Length && line[idx] != line[startIdx])
					{
						idx++;
					}
					tokens.Add(line.Substring(startIdx, ++idx - startIdx));
				}
				else
				{
					tokens.Add(line.Substring(idx++, 1));
				}
			}
			return tokens;
		}

		static bool ReadFraction(List<string> tokens, ref int tokenIdx, out float fraction)
		{
			// Read a fraction in the form x%
			if (tokenIdx + 2 <= tokens.Count && tokens[tokenIdx + 1] == "%")
			{
				int numerator;
				if (Int32.TryParse(tokens[tokenIdx], out numerator))
				{
					fraction = (float)numerator / 100.0f;
					tokenIdx += 2;
					return true;
				}
			}

			// Read a fraction in the form x/y
			if (tokenIdx + 3 <= tokens.Count && tokens[tokenIdx + 1] == "/")
			{
				int numerator, denominator;
				if (Int32.TryParse(tokens[tokenIdx], out numerator) && Int32.TryParse(tokens[tokenIdx + 2], out denominator))
				{
					fraction = (float)numerator / (float)denominator;
					tokenIdx += 3;
					return true;
				}
			}

			fraction = 0.0f;
			return false;
		}
	}
}
