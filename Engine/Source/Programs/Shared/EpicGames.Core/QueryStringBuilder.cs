// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility class for creating Http query strings
	/// </summary>
	public class QueryStringBuilder
	{
		readonly StringBuilder _builder = new();

		/// <summary>
		/// Adds a new argument to the query string
		/// </summary>
		public void Add(string key, string value)
		{
			if (_builder.Length > 0)
			{
				_builder.Append('&');
			}
			_builder.Append(key);
			_builder.Append('=');
			_builder.Append(Uri.EscapeDataString(value));
		}

		/// <summary>
		/// Adds a set of arguments to the query string
		/// </summary>
		public void Add(string key, IEnumerable<string>? values)
		{
			if (values != null)
			{
				foreach (string value in values)
				{
					Add(key, value);
				}
			}
		}

		/// <inheritdoc/>
		public override string ToString() => _builder.ToString();
	}
}
