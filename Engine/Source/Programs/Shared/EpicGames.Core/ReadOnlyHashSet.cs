// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper around the HashSet container that only allows read operations
	/// </summary>
	/// <typeparam name="T">Type of element for the hashset</typeparam>
	[Obsolete("ReadOnlyHashSet<T> is obsolete, please replace with IReadOnlySet<T>")]
	public class ReadOnlyHashSet<T> : IReadOnlyCollection<T>
	{
		/// <summary>
		/// The mutable hashset
		/// </summary>
		readonly HashSet<T> _inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The mutable hashset</param>
		public ReadOnlyHashSet(HashSet<T> inner)
		{
			_inner = inner;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="elements">Elements for the hash set</param>
		public ReadOnlyHashSet(IEnumerable<T> elements)
		{
			_inner = [.. elements];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="elements">Elements for the hash set</param>
		/// <param name="comparer">Comparer for elements in the set</param>
		public ReadOnlyHashSet(IEnumerable<T> elements, IEqualityComparer<T> comparer)
		{
			_inner = new HashSet<T>(elements, comparer);
		}

		/// <summary>
		/// Number of elements in the set
		/// </summary>
		public int Count => _inner.Count;

		/// <summary>
		/// The comparer for elements in the set
		/// </summary>
		public IEqualityComparer<T> Comparer => _inner.Comparer;

		/// <summary>
		/// Tests whether a given item is in the set
		/// </summary>
		/// <param name="item">Item to check for</param>
		/// <returns>True if the item is in the set</returns>
		public bool Contains(T item)
		{
			return _inner.Contains(item);
		}

		/// <summary>
		/// Gets an enumerator for set elements
		/// </summary>
		/// <returns>Enumerator instance</returns>
		public IEnumerator<T> GetEnumerator()
		{
			return _inner.GetEnumerator();
		}

		/// <summary>
		/// Gets an enumerator for set elements
		/// </summary>
		/// <returns>Enumerator instance</returns>
		IEnumerator IEnumerable.GetEnumerator()
		{
			return ((IEnumerable)_inner).GetEnumerator();
		}

		/// <summary>
		/// Implicit conversion operator from hashsets
		/// </summary>
		/// <param name="hashSet"></param>
		public static implicit operator ReadOnlyHashSet<T>(HashSet<T> hashSet)
		{
			return new ReadOnlyHashSet<T>(hashSet);
		}
	}
}
