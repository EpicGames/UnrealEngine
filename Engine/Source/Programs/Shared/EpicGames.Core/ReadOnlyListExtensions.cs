// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for functionality on <see cref="List{T}"/> but missing from <see cref="IReadOnlyList{T}"/>
	/// </summary>
	public static class ReadOnlyListExtensions
	{
		/// <summary>
		/// Performs a binary search on the given list
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="list">The list type</param>
		/// <param name="item">Item to search for</param>
		/// <returns>As List.BinarySearch</returns>
		public static int BinarySearch<T>(this IReadOnlyList<T> list, T item)
		{
			return BinarySearch(list, x => x, item, Comparer<T>.Default);
		}

		/// <summary>
		/// Performs a binary search on the given list
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="list">The list type</param>
		/// <param name="item">Item to search for</param>
		/// <param name="comparer">Comparer for elements in the list</param>
		/// <returns>As List.BinarySearch</returns>
		public static int BinarySearch<T>(this IReadOnlyList<T> list, T item, IComparer<T> comparer)
		{
			return BinarySearch(list, x => x, item, comparer);
		}

		/// <summary>
		/// Binary searches a list based on a projection
		/// </summary>
		/// <typeparam name="TItem">The item in the list</typeparam>
		/// <typeparam name="TField">The field to search on</typeparam>
		/// <param name="list">The list to search</param>
		/// <param name="projection">The projection to apply to each item in the list</param>
		/// <param name="item">The item to find</param>
		/// <returns>As <see cref="List{T}.BinarySearch(T)"/></returns>
		public static int BinarySearch<TItem, TField>(this IReadOnlyList<TItem> list, Func<TItem, TField> projection, TField item)
		{
			return BinarySearch(list, projection, item, Comparer<TField>.Default);
		}

		/// <summary>
		/// Binary searches a list based on a projection
		/// </summary>
		/// <typeparam name="TItem">The item in the list</typeparam>
		/// <typeparam name="TField">The field to search on</typeparam>
		/// <param name="list">The list to search</param>
		/// <param name="projection">The projection to apply to each item in the list</param>
		/// <param name="item">The item to find</param>
		/// <param name="comparer">Comparer for field elements</param>
		/// <returns>As <see cref="List{T}.BinarySearch(T)"/></returns>
		public static int BinarySearch<TItem, TField>(this IReadOnlyList<TItem> list, Func<TItem, TField> projection, TField item, IComparer<TField> comparer)
		{
			int lowerBound = 0;
			int upperBound = list.Count - 1;
			while (lowerBound <= upperBound)
			{
				int idx = lowerBound + (upperBound - lowerBound) / 2;

				int comparison = comparer.Compare(projection(list[idx]), item);
				if (comparison == 0)
				{
					return idx;
				}
				else if (comparison < 0)
				{
					lowerBound = idx + 1;
				}
				else
				{
					upperBound = idx - 1;
				}
			}
			return ~lowerBound;
		}

		/// <summary>
		/// Converts a read only list to a different type
		/// </summary>
		/// <typeparam name="TInput">The input element type</typeparam>
		/// <typeparam name="TOutput">The output element type</typeparam>
		/// <param name="input">Input list</param>
		/// <param name="convert">Conversion function</param>
		/// <returns>New list of items</returns>
		public static List<TOutput> ConvertAll<TInput, TOutput>(this IReadOnlyList<TInput> input, Func<TInput, TOutput> convert)
		{
			List<TOutput> outputs = new(input.Count);
			for (int idx = 0; idx < input.Count; idx++)
			{
				outputs.Add(convert(input[idx]));
			}
			return outputs;
		}

		/// <summary>
		/// Finds the index of the first element matching a predicate
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="list">List to search</param>
		/// <param name="predicate">Predicate for the list</param>
		/// <returns>Index of the element</returns>
		public static int FindIndex<T>(this IReadOnlyList<T> list, Predicate<T> predicate)
		{
			int foundIndex = -1;
			for (int idx = 0; idx < list.Count; idx++)
			{
				if (predicate(list[idx]))
				{
					foundIndex = idx;
					break;
				}
			}
			return foundIndex;
		}
	}
}
