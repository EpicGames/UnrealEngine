// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for an array of <see cref="ReadOnlyMemory{Byte}"/> objects which are embedded within another <see cref="ReadOnlyMemory{Byte}"/> instance.
	/// </summary>
	public interface IReadOnlyMemoryArray : IReadOnlyList<ReadOnlyMemory<byte>>
	{
		/// <summary>
		/// The underlying data storing the memory array
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Gets the offset of an element within the 
		/// </summary>
		/// <param name="index">Index of the element to find</param>
		/// <returns>Offset of the element within the underlying data stream</returns>
		int GetOffset(int index);
	}

	/// <summary>
	/// Represents an array of uniformly sized <see cref="ReadOnlyMemory{Byte}"/> objects in a parent <see cref="ReadOnlyMemory{Byte}"/> instance.
	/// </summary>
	public readonly struct UniformReadOnlyMemoryArray : IReadOnlyMemoryArray
	{
		/// <inheritdoc/>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Size of each element
		/// </summary>
		public int ElementSize { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public UniformReadOnlyMemoryArray(ReadOnlyMemory<byte> data, int elementSize)
		{
			Data = data;
			ElementSize = elementSize;
		}

		/// <summary>
		/// Gets the size of serialized data
		/// </summary>
		public static int Measure(ReadOnlySpan<byte> span, int elementSize) => sizeof(int) + (BinaryPrimitives.ReadInt32LittleEndian(span) * elementSize);

		/// <inheritdoc/>
		public int Count => BinaryPrimitives.ReadInt32LittleEndian(Data.Span);

		/// <inheritdoc/>
		public int GetOffset(int index) => sizeof(int) + (index * ElementSize);

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> this[int index]
		{
			get
			{
				int beginOffset = GetOffset(index);
				int endOffset = GetOffset(index + 1);
				return Data.Slice(beginOffset, endOffset - beginOffset);
			}
		}

		/// <inheritdoc/>
		public IEnumerator<ReadOnlyMemory<byte>> GetEnumerator()
		{
			for (int offset = sizeof(int); offset < Data.Length; offset += ElementSize)
			{
				yield return Data.Slice(offset, ElementSize);
			}
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}

	/// <summary>
	/// Represents an array of variably sized <see cref="ReadOnlyMemory{Byte}"/> objects in a parent <see cref="ReadOnlyMemory{Byte}"/> instance.
	/// </summary>
	public readonly struct JaggedReadOnlyMemoryArray : IReadOnlyMemoryArray
	{
		/// <inheritdoc/>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JaggedReadOnlyMemoryArray(ReadOnlyMemory<byte> data)
		{
			data = data.Slice(0, Measure(data.Span));
			Data = data;
		}

		/// <summary>
		/// Gets the size of serialized data
		/// </summary>
		public static int Measure(ReadOnlySpan<byte> span)
		{
			int offset = BinaryPrimitives.ReadInt32LittleEndian(span);
			return BinaryPrimitives.ReadInt32LittleEndian(span.Slice(offset - sizeof(int)));
		}

		/// <inheritdoc/>
		public int Count => (GetOffset(0) / sizeof(int)) - 1;

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> this[int index]
		{
			get
			{
				int beginOffset = GetOffset(index);
				int endOffset = GetOffset(index + 1);
				return Data.Slice(beginOffset, endOffset - beginOffset);
			}
		}

		/// <inheritdoc/>
		public int GetOffset(int index) => BinaryPrimitives.ReadInt32LittleEndian(Data.Span.Slice(index * 4));

		/// <inheritdoc/>
		public IEnumerator<ReadOnlyMemory<byte>> GetEnumerator()
		{
			int offset = GetOffset(0);
			for (int index = 0; index < Count; index++)
			{
				int nextOffset = GetOffset(index + 1);
				yield return Data.Slice(offset, nextOffset - offset);
				offset = nextOffset;
			}
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}
}
