// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Stream which reads from a <see cref="ReadOnlyMemory{Byte}"/>
	/// </summary>
	public class ReadOnlyMemoryStream : Stream
	{
		/// <summary>
		/// The buffer to read from
		/// </summary>
		readonly ReadOnlyMemory<byte> _memory;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The memory to read from</param>
		public ReadOnlyMemoryStream(ReadOnlyMemory<byte> memory)
		{
			_memory = memory;
		}

		/// <inheritdoc/>
		public override bool CanRead => true;

		/// <inheritdoc/>
		public override bool CanSeek => true;

		/// <inheritdoc/>
		public override bool CanWrite => false;

		/// <inheritdoc/>
		public override long Length => _memory.Length;

		/// <inheritdoc/>
		public override long Position { get; set; }

		/// <inheritdoc/>
		public override void Flush()
		{
		}

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count)
		{
			int copyLength = Math.Min(count, (int)(_memory.Length - Position));
			_memory.Slice((int)Position, copyLength).CopyTo(buffer.AsMemory(offset, copyLength));
			Position += copyLength;
			return copyLength;
		}

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin)
		{
			switch (origin)
			{
				case SeekOrigin.Begin:
					Position = offset;
					break;
				case SeekOrigin.Current:
					Position += offset;
					break;
				case SeekOrigin.End:
					Position = _memory.Length + offset;
					break;
				default:
					throw new ArgumentException(null, nameof(origin));
			}
			return Position;
		}

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => throw new InvalidOperationException();
	}
}
