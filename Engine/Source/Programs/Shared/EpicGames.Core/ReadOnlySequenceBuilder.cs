// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Static utility methods
	/// </summary>
	public static class ReadOnlySequence
	{
		/// <summary>
		/// Create a sequence from a list of segments
		/// </summary>
		/// <param name="segments"></param>
		/// <returns>Sequence for the list of segments</returns>
		public static ReadOnlySequence<T> Create<T>(IReadOnlyList<ReadOnlyMemory<T>> segments) => ReadOnlySequenceBuilder<T>.Create(segments);
	}

	/// <summary>
	/// Utility class to combine buffers into a <see cref="ReadOnlySequence{T}"/>
	/// </summary>
	/// <typeparam name="T">Element type</typeparam>
	public class ReadOnlySequenceBuilder<T>
	{
		class Segment : ReadOnlySequenceSegment<T>
		{
			public Segment(long runningIndex, ReadOnlyMemory<T> memory)
			{
				RunningIndex = runningIndex;
				Memory = memory;
			}

			public void SetNext(Segment next)
			{
				Next = next;
			}
		}

		readonly List<ReadOnlyMemory<T>> _segments = [];

		/// <summary>
		/// Current length of the sequence
		/// </summary>
		public long Length { get; private set; }

		/// <summary>
		/// Create a sequence from a list of segments
		/// </summary>
		/// <param name="segments"></param>
		/// <returns>Sequence for the list of segments</returns>
		internal static ReadOnlySequence<T> Create(IReadOnlyList<ReadOnlyMemory<T>> segments)
		{
			if (segments.Count == 0)
			{
				return ReadOnlySequence<T>.Empty;
			}

			Segment first = new(0, segments[0]);
			Segment last = first;

			for (int idx = 1; idx < segments.Count; idx++)
			{
				Segment next = new(last.RunningIndex + last.Memory.Length, segments[idx]);
				last.SetNext(next);
				last = next;
			}

			return new ReadOnlySequence<T>(first, 0, last, last.Memory.Length);
		}

		/// <summary>
		/// Append a block of memory to the end of the sequence
		/// </summary>
		/// <param name="memory">Memory to append</param>
		public void Append(ReadOnlyMemory<T> memory)
		{
			if (memory.Length > 0)
			{
				_segments.Add(memory);
				Length += memory.Length;
			}
		}

		/// <summary>
		/// Append another sequence to the end of this one
		/// </summary>
		/// <param name="sequence">Sequence to append</param>
		public void Append(ReadOnlySequence<T> sequence)
		{
			foreach (ReadOnlyMemory<T> segment in sequence)
			{
				Append(segment);
			}
		}

		/// <summary>
		/// Construct a sequence from the added blocks
		/// </summary>
		/// <returns>Sequence for the added memory blocks</returns>
		public ReadOnlySequence<T> Construct() => Create(_segments);
	}

	/// <summary>
	/// Extension methods for <see cref="ReadOnlySequence{T}"/>
	/// </summary>
	public static class ReadOnlySequenceExtensions
	{
		/// <summary>
		/// Gets the data from a sequence as a contiguous block of memory
		/// </summary>
		/// <param name="sequence">Sequence to return</param>
		/// <returns>Data for the blob</returns>
		public static ReadOnlyMemory<T> AsSingleSegment<T>(this ReadOnlySequence<T> sequence)
		{
			if (sequence.IsSingleSegment)
			{
				return sequence.First;
			}
			else
			{
				return sequence.ToArray();
			}
		}
	}
}
