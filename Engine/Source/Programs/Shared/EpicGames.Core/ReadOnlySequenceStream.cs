// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;

namespace EpicGames.Core
{
	/// <summary>
	/// Stream which reads from a <see cref="ReadOnlyMemory{Byte}"/>
	/// </summary>
	public class ReadOnlySequenceStream : Stream
	{
		/// <summary>
		/// The buffer to read from
		/// </summary>
		readonly ReadOnlySequence<byte> _source;

		/// <summary>
		/// Remaining sequence to read
		/// </summary>
		ReadOnlySequence<byte> _remaining;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="sequence">The memory to read from</param>
		public ReadOnlySequenceStream(ReadOnlySequence<byte> sequence)
		{
			_source = sequence;
			_remaining = _source;
		}

		/// <inheritdoc/>
		public override bool CanRead => true;

		/// <inheritdoc/>
		public override bool CanSeek => true;

		/// <inheritdoc/>
		public override bool CanWrite => false;

		/// <inheritdoc/>
		public override long Length => _source.Length;

		/// <inheritdoc/>
		public override long Position
		{
			get => _source.Length - _remaining.Length;
			set => _remaining = _source.Slice(value);
		}

		/// <inheritdoc/>
		public override void Flush()
		{
		}

		/// <inheritdoc/>
		public override int Read(Span<byte> buffer)
		{
			ReadOnlySpan<byte> firstSpan = _remaining.FirstSpan;
			int copyLength = Math.Min(firstSpan.Length, buffer.Length);
			firstSpan.Slice(0, copyLength).CopyTo(buffer);
			_remaining = _remaining.Slice(copyLength);
			return copyLength;
		}

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => Read(buffer.AsSpan(offset, count));

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin)
		{
			switch (origin)
			{
				case SeekOrigin.Begin:
					Position = offset;
					break;
				case SeekOrigin.Current:
					Position += offset;
					break;
				case SeekOrigin.End:
					Position = Length + offset;
					break;
				default:
					throw new ArgumentException(null, nameof(origin));
			}
			return Position;
		}

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new InvalidOperationException();

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => throw new InvalidOperationException();
	}
}
