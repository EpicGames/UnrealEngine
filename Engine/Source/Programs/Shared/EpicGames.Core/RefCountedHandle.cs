// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Threading;

namespace EpicGames.Core
{
	/// <summary>
	/// Disposable reference to an underlying resource. The underlying resource is disposed once the last reference to it is disposed.
	/// </summary>
	public interface IRefCountedHandle : IDisposable
	{
		/// <summary>
		/// Increment the reference count and returns a new handle to it
		/// </summary>
		IRefCountedHandle AddRef();
	}

	/// <summary>
	/// Disposable reference to an underlying resource.
	/// </summary>
	/// <typeparam name="T">Type of the target resource</typeparam>
	public interface IRefCountedHandle<T> : IRefCountedHandle
	{
		/// <summary>
		/// Accessor for the current ref count, for debugging purposes.
		/// </summary>
		int RefCount { get; }

		/// <summary>
		/// Target of the reference
		/// </summary>
		T Target { get; }

		/// <inheritdoc cref="IRefCountedHandle.AddRef"/>
		new IRefCountedHandle<T> AddRef();

		/// <inheritdoc/>
		IRefCountedHandle IRefCountedHandle.AddRef() => AddRef();
	}

	/// <summary>
	/// Utility methods for creating ref counted handles
	/// </summary>
	public static class RefCountedHandle
	{
		/// <summary>
		/// Creates a reference counted handle to a disposable resource.
		/// </summary>
		public static IRefCountedHandle<T> Create<T>(T target) where T : IDisposable
		{
			return new RefCountedHandle<T>(target, target);
		}

		/// <summary>
		/// Creates a reference counted handle to a resource with an explicit disposer.
		/// </summary>
		public static IRefCountedHandle<T> Create<T>(T target, IDisposable? owner)
		{
			return new RefCountedHandle<T>(target, owner);
		}

		/// <summary>
		/// Creates a reference counted handle to a memory owner.
		/// </summary>
		public static IRefCountedHandle<Memory<T>> Create<T>(IMemoryOwner<T> target) where T : struct
		{
			return new RefCountedHandle<Memory<T>>(target.Memory, target);
		}
	}

	/// <summary>
	/// Concrete implementation of <see cref="IRefCountedHandle{T}"/>
	/// </summary>
	public sealed class RefCountedHandle<T> : IRefCountedHandle<T>
	{
		readonly RefCountedDisposer? _refCountedDisposer;
		T _target;

		/// <inheritdoc/>
		public int RefCount => _refCountedDisposer?.RefCount ?? throw new ObjectDisposedException(typeof(RefCountedHandle<T>).Name);

		/// <inheritdoc/>
		public T Target => _target ?? throw new ObjectDisposedException(typeof(RefCountedHandle<T>).Name);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="target">Target value</param>
		/// <param name="owner">Owner of the target resource</param>
		public RefCountedHandle(T target, IDisposable? owner)
		{
			_refCountedDisposer = new RefCountedDisposer(owner);
			_target = target;
		}

		private RefCountedHandle(T value, RefCountedDisposer disposer)
		{
			_refCountedDisposer = disposer;
			_target = value;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_refCountedDisposer != null)
			{
				_refCountedDisposer.Release();
				_target = default!;
			}
		}

		/// <inheritdoc/>
		public IRefCountedHandle<T> AddRef()
		{
			ObjectDisposedException.ThrowIf(_refCountedDisposer == null, typeof(RefCountedHandle<T>).Name);

			_refCountedDisposer.AddRef();
			return new RefCountedHandle<T>(Target, _refCountedDisposer);
		}
	}

	/// <summary>
	/// Tracks a ref count value and disposes of a resource once the reference count is zero
	/// </summary>
	public sealed class RefCountedDisposer
	{
		IDisposable? _owner;
		int _value = 1;

		/// <summary>
		/// Current reference count
		/// </summary>
		public int RefCount => _value;

		/// <summary>
		/// Constructor
		/// </summary>
		public RefCountedDisposer(IDisposable? owner, int initialValue = 1)
		{
			_owner = owner;
			_value = initialValue;
		}

		/// <summary>
		/// Increment the current value
		/// </summary>
		/// <returns>The incremented value</returns>
		public int AddRef() => Interlocked.Increment(ref _value);

		/// <summary>
		/// Decrement the current value
		/// </summary>
		/// <returns>The decremented value</returns>
		public int Release()
		{
			int count = Interlocked.Decrement(ref _value);
			if (count == 0)
			{
				_owner?.Dispose();
				_owner = null;
			}
			return count;
		}
	}
}
