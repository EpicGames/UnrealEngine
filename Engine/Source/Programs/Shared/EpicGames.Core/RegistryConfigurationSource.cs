// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using Microsoft.Extensions.Configuration;
using Microsoft.Win32;

namespace EpicGames.Core
{
	/// <summary>
	/// Configuration source which reads values from the Registry on Windows
	/// </summary>
	[SupportedOSPlatform("windows")]
	public class RegistryConfigurationSource : IConfigurationSource
	{
		readonly RegistryKey _baseKey;
		readonly string _keyPath;
		readonly string _baseConfigName;
		readonly Func<string, bool>? _filter;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="baseKey">Hive to resolve keyPath relative to</param>
		/// <param name="keyPath">Path within the registry to enumerate</param>
		/// <param name="baseConfigName">Prefix for returned configuration values</param>
		/// <param name="filter">Optional filter for keys to include</param>
		public RegistryConfigurationSource(RegistryKey baseKey, string keyPath, string baseConfigName, Func<string, bool>? filter = null)
		{
			_baseKey = baseKey;
			_keyPath = keyPath;
			_baseConfigName = baseConfigName;
			_filter = filter;
		}

		/// <summary>
		/// Get the registry key used
		/// </summary>
		/// <returns></returns>
		public string GetRegistryKey() => _baseKey + "\\" + _keyPath;

		/// <inheritdoc/>
		public IConfigurationProvider Build(IConfigurationBuilder builder)
			=> new RegistryConfigProvider(_baseKey, _keyPath, _baseConfigName, _filter);
	}

	class RegistryConfigProvider : ConfigurationProvider
	{
		readonly RegistryKey _baseKey;
		readonly string _keyPath;
		readonly string _baseConfigName;
		readonly Func<string, bool>? _filter;

		public RegistryConfigProvider(RegistryKey baseKey, string keyPath, string baseConfigName, Func<string, bool>? filter)
		{
			_baseKey = baseKey;
			_keyPath = keyPath;
			_baseConfigName = baseConfigName;
			_filter = filter;
		}

		public override void Load()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				Dictionary<string, string?> data = [];
				GetValues(_baseKey, _keyPath, _baseConfigName, _filter, data);
				Data = data;
			}
		}

		[SupportedOSPlatform("windows")]
		static void GetValues(RegistryKey baseKey, string keyPath, string baseConfigName, Func<string, bool>? filter, Dictionary<string, string?> data)
		{
			using RegistryKey? registryKey = baseKey.OpenSubKey(keyPath);
			if (registryKey != null)
			{
				string[] subKeyNames = registryKey.GetSubKeyNames();
				foreach (string subKeyName in subKeyNames)
				{
					GetValues(registryKey, subKeyName, $"{baseConfigName}:{subKeyName}", filter, data);
				}

				string[] valueNames = registryKey.GetValueNames();
				foreach (string valueName in valueNames)
				{
					object? value = registryKey.GetValue(valueName);
					if (value != null)
					{
						string name = $"{baseConfigName}:{valueName}";
						if (filter == null || filter(name))
						{
							data[name] = value.ToString();
						}
					}
				}
			}
		}
	}
}
