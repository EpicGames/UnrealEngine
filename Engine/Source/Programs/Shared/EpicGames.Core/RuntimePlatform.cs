// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;

namespace EpicGames.Core
{
	/// <summary>
	/// Allows testing different platforms
	/// </summary>
	public static class RuntimePlatform
	{
		/// <summary>
		/// Whether we are currently running on Linux.
		/// </summary>
		[Obsolete("Replace with OperatingSystem.IsLinux()")]
		[SupportedOSPlatformGuard("linux")]
		public static bool IsLinux => OperatingSystem.IsLinux();

		/// <summary>
		/// Whether we are currently running on a MacOS platform.
		/// </summary>
		[Obsolete("Replace with OperatingSystem.IsMacOS()")]
		[SupportedOSPlatformGuard("macos")]
		public static bool IsMac => OperatingSystem.IsMacOS();

		/// <summary>
		/// Whether we are currently running a Windows platform.
		/// </summary>
		[Obsolete("Replace with OperatingSystem.IsWindows()")]
		[SupportedOSPlatformGuard("windows")]
		public static bool IsWindows => OperatingSystem.IsWindows();

		/// <summary>
		/// The platform type
		/// </summary>
		public enum Type
		{
			/// <summary>
			/// Windows
			/// </summary>
			Windows,

			/// <summary>
			/// Linux
			/// </summary>
			Linux,

			/// <summary>
			/// Mac
			/// </summary>
			Mac
		};

		/// <summary>
		/// The current runtime platform
		/// </summary>
		public static readonly Type Current = OperatingSystem.IsWindows() ? Type.Windows : OperatingSystem.IsMacOS() ? Type.Mac : Type.Linux;

		/// <summary>
		/// The extension executables have on the current platform
		/// </summary>
		public static readonly string ExeExtension = OperatingSystem.IsWindows() ? ".exe" : "";

		[DllImport("kernel32.dll", CharSet = CharSet.Auto)]
		private static extern IntPtr GetModuleHandle(string lpModuleName);

		[DllImport("kernel32.dll", CharSet = CharSet.Ansi)]
		private static extern IntPtr GetProcAddress(IntPtr hModule, string procName);

		/// <summary>
		/// The current runtime platform is Windows but we're running on Wine
		/// </summary>
		public static bool IsRunningOnWine()
		{
			if (OperatingSystem.IsWindows())
			{
				IntPtr ntdllHandle = GetModuleHandle("ntdll.dll");
				return ntdllHandle.ToInt64() != 0 && GetProcAddress(ntdllHandle, "wine_get_version") != IntPtr.Zero;
			}
			return false;
		}
	}
}
