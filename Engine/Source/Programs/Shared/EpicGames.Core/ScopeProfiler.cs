// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CA1822 // Mark members as static

namespace EpicGames.Core
{
#pragma warning disable IDE0060 // Remove unused parameter

	/// <summary>
	/// A stub/no-op scope-profiler API that can be replaced by another implementation to record execution of
	/// instrumented scopes.
	/// </summary>
	public class ScopeProfiler
	{
		/// <summary>
		/// Static instance of the profiler
		/// </summary>
		public static ScopeProfiler Instance { get; } = new ScopeProfiler();

		/// <summary>
		/// 
		/// </summary>
		/// <param name="programName"></param>
		/// <param name="hostAddress"></param>
		/// <param name="maxThreadCount"></param>
		public void InitializeAndStart(string programName, string hostAddress, int maxThreadCount) { }

		/// <summary>
		/// 
		/// </summary>
		public void StopAndShutdown() { }

		/// <summary>
		/// Start a stacked scope. Must start and end on the same thread
		/// </summary>
		/// <param name="name"></param>
		/// <param name="bIsStall"></param>
		public void StartScope(string name, bool bIsStall) { }

		/// <summary>
		/// End a stacked scope. Must start and end on the same thread.
		/// </summary>
		public void EndScope() { }

		/// <summary>
		/// Record an un-stacked time span at a specified time.
		/// </summary>
		/// <param name="name"></param>
		/// <param name="id"></param>
		/// <param name="bStall"></param>
		/// <param name="beginThreadId"></param>
		/// <param name="endThreadId"></param>
		/// <param name="startTime">start time for the span (use FastTime())</param>
		/// <param name="endTime">end time for the span (use FastTime())</param>
		public void AddSpanAtTime(string name, ulong id, bool bStall, uint beginThreadId, uint endThreadId, ulong startTime, ulong endTime) { }

		/// <summary>
		/// Record a value against a particular name, to be plotted over time
		/// </summary>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public void Plot(string name, double value) { }

		/// <summary>
		/// Generate a timestamp value for the current moment.
		/// </summary>
		/// <returns></returns>
		public ulong FastTime()
		{
			return 0;
		}

		/// <summary>
		/// Retrieve the profiler's identifier for the current thread
		/// </summary>
		/// <returns></returns>
		public uint GetCurrentThreadId()
		{
			return 0;
		}
	}
#pragma warning restore IDE0060 // Remove unused parameter
}

