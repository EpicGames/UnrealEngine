// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper for a semver version string (https://semver.org/)
	/// </summary>
	public readonly struct SemVer : IEquatable<SemVer>, IComparable<SemVer>
	{
		readonly ref struct Field
		{
			public readonly int Pos;
			public readonly ReadOnlySpan<char> Text;

			public Field(int pos, ReadOnlySpan<char> text)
			{
				Pos = pos;
				Text = text;
			}

			public int End => Pos + Text.Length;
		}

		readonly string _text;
		readonly int _length;

		ReadOnlySpan<char> Span => _text.AsSpan(0, _length);

		// Regex taken from official site.
		static readonly Regex s_pattern = new(@"^((0|[1-9]\d*)\.(0|[1-9]\d*)\.(0|[1-9]\d*)(?:-((?:0|[1-9]\d*|\d*[a-zA-Z-][0-9a-zA-Z-]*)(?:\.(?:0|[1-9]\d*|\d*[a-zA-Z-][0-9a-zA-Z-]*))*))?)(?:\+([0-9a-zA-Z-]+(?:\.[0-9a-zA-Z-]+)*))?$");

		/// <summary>
		/// Constructor
		/// </summary>
		private SemVer(string text, int length)
		{
			_text = text;
			_length = length;
		}

		/// <summary>
		/// Parses the given string as a semantic version
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>Resulting semver object</returns>
		public static SemVer Parse(string text)
		{
			SemVer result;
			if (!TryParse(text, out result))
			{
				throw new ArgumentException($"String '{text}' is not a valid semantic version string. See https://semver.org/.", nameof(text));
			}
			return result;
		}

		/// <summary>
		/// Attempt to parse the given string as a semantic version
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="semVer">Resulting version</param>
		/// <returns>True on success</returns>
		public static bool TryParse(string text, out SemVer semVer)
		{
			Match match = s_pattern.Match(text);
			if (match.Success)
			{
				semVer = new SemVer(text, match.Groups[1].Length);
				return true;
			}
			else
			{
				semVer = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override int GetHashCode() => String.GetHashCode(_text.AsSpan(0, _length), StringComparison.Ordinal);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is SemVer other && Equals(other);

		/// <inheritdoc/>
		public bool Equals(SemVer other) => Span.SequenceEqual(other.Span);

		/// <inheritdoc/>
		public int CompareTo(SemVer other) => Compare(Span, other.Span);

		/// <inheritdoc/>
		public override string ToString() => _text;

		/// <summary>
		/// Compare two versions
		/// </summary>
		/// <param name="lhs">First version to compare</param>
		/// <param name="rhs">Second version to compare</param>
		/// <returns>A number indicating the order of the two version strings</returns>
		public static int Compare(SemVer lhs, SemVer rhs) => Compare(lhs.Span, rhs.Span);

		static int Compare(ReadOnlySpan<char> lhs, ReadOnlySpan<char> rhs)
		{
			// Compare major fields
			Field lhsField = GetField(lhs, 0);
			Field rhsField = GetField(rhs, 0);

			int diff = CompareNumericFields(lhsField.Text, rhsField.Text);
			if (diff != 0)
			{
				return diff;
			}

			// Compare minor fields
			lhsField = GetField(lhs, lhsField.End + 1);
			rhsField = GetField(rhs, rhsField.End + 1);

			diff = CompareNumericFields(lhsField.Text, rhsField.Text);
			if (diff != 0)
			{
				return diff;
			}

			// Compare patch fields
			lhsField = GetField(lhs, lhsField.End + 1);
			rhsField = GetField(rhs, rhsField.End + 1);

			diff = CompareNumericFields(lhsField.Text, rhsField.Text);
			if (diff != 0)
			{
				return diff;
			}

			// If either is complete, it takes precedence
			if (lhsField.End == lhs.Length || rhsField.End == rhs.Length)
			{
				return ((lhsField.End == lhs.Length) ? 1 : 0) - ((rhsField.End == rhs.Length) ? 1 : 0);
			}

			// Otherwise compare each pre-release field
			while (lhsField.End < lhs.Length && rhsField.End < rhs.Length)
			{
				lhsField = GetField(lhs, lhsField.End + 1);
				rhsField = GetField(rhs, rhsField.End + 1);

				diff = CompareFields(lhsField.Text, rhsField.Text);
				if (diff != 0)
				{
					return diff;
				}
			}

			// Give precedence to the longer version string
			return ((lhsField.End == lhs.Length) ? 0 : 1) - ((rhsField.End == rhs.Length) ? 0 : 1);
		}

		static Field GetField(ReadOnlySpan<char> text, int pos)
		{
			int end = pos + 1;
			while (end < text.Length && text[end] != '.' && text[end] != '-' && text[end] != '+')
			{
				end++;
			}
			return new Field(pos, text.Slice(pos, end - pos));
		}

		static int CompareFields(ReadOnlySpan<char> lhs, ReadOnlySpan<char> rhs)
		{
			// Rules: 
			// Identifiers consisting of only digits are compared numerically.
			// Identifiers with letters or hyphens are compared lexically in ASCII sort order.
			// Numeric identifiers always have lower precedence than non-numeric identifiers.

			if (IsNumeric(lhs))
			{
				return IsNumeric(rhs) ? CompareNumericFields(lhs, rhs) : -1;
			}
			else
			{
				return IsNumeric(rhs) ? +1 : lhs.SequenceCompareTo(rhs);
			}
		}

		static int CompareNumericFields(ReadOnlySpan<char> lhs, ReadOnlySpan<char> rhs)
		{
			return (lhs.Length == rhs.Length) ? lhs.SequenceCompareTo(rhs) : (lhs.Length - rhs.Length);
		}

		static bool IsNumeric(ReadOnlySpan<char> field)
		{
			for (int i = 0; i < field.Length; i++)
			{
				if (field[i] < '0' || field[i] > '9')
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Compares two versions for equality
		/// </summary>
		public static bool operator ==(SemVer lhs, SemVer rhs) => lhs.Equals(rhs);

		/// <summary>
		/// Compares two versions for inequality
		/// </summary>
		public static bool operator !=(SemVer lhs, SemVer rhs) => !lhs.Equals(rhs);

		/// <summary>
		/// Tests if one version has lower precedence than another
		/// </summary>
		public static bool operator <(SemVer lhs, SemVer rhs) => Compare(lhs.Span, rhs.Span) < 0;

		/// <summary>
		/// Tests if one version has lower or equal precedence to another
		/// </summary>
		public static bool operator <=(SemVer lhs, SemVer rhs) => Compare(lhs.Span, rhs.Span) <= 0;

		/// <summary>
		/// Tests if one version has greater precedence than another
		/// </summary>
		public static bool operator >(SemVer lhs, SemVer rhs) => Compare(lhs.Span, rhs.Span) > 0;

		/// <summary>
		/// Tests if one version has greater or equal precedence to another
		/// </summary>
		public static bool operator >=(SemVer lhs, SemVer rhs) => Compare(lhs.Span, rhs.Span) >= 0;
	}
}
