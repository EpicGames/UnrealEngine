// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension method to wrap a semaphore in a using statement
	/// </summary>
	public static class SemaphoreSlimExtensions
	{
		struct ReleaseWrapper : IDisposable
		{
			private readonly SemaphoreSlim _semaphore;
			private int _disposed;

			public ReleaseWrapper(SemaphoreSlim semaphore)
			{
				_semaphore = semaphore;
				_disposed = 0;
			}

			public void Dispose()
			{
				if (Interlocked.CompareExchange(ref _disposed, 1, 0) == 0)
				{
					_semaphore.Release();
				}
			}
		}

		/// <summary>
		/// Returns a disposable semaphore handle
		/// </summary>
		/// <param name="semaphore">the semaphore to wrap</param>
		/// <param name="cancellationToken">optional cancellation token</param>
		/// <returns></returns>
		public static IDisposable WaitDisposable(this SemaphoreSlim semaphore, CancellationToken cancellationToken = default)
		{
			semaphore.Wait(cancellationToken);
			return new ReleaseWrapper(semaphore);
		}

		/// <summary>
		/// Returns a disposable semaphore handle
		/// </summary>
		/// <param name="semaphore">the semaphore to wrap</param>
		/// <param name="cancellationToken">optional cancellation token</param>
		/// <returns></returns>
		public static async Task<IDisposable> WaitDisposableAsync(this SemaphoreSlim semaphore, CancellationToken cancellationToken = default)
		{
			await semaphore.WaitAsync(cancellationToken).ConfigureAwait(false);
			return new ReleaseWrapper(semaphore);
		}
	}
}
