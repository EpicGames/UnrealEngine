// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.ComponentModel;
using System.Globalization;
using System.Security.Cryptography;
using System.Text.Json;
using System.Text.Json.Serialization;

#pragma warning disable CA5350 // Do Not Use Weak Cryptographic Algorithms

namespace EpicGames.Core
{
	/// <summary>
	/// Struct representing a strongly typed Sha1Hash value.
	/// </summary>
	[JsonConverter(typeof(Sha1HashJsonConverter))]
	[TypeConverter(typeof(Sha1HashTypeConverter))]
	public readonly struct Sha1Hash : IEquatable<Sha1Hash>, IComparable<Sha1Hash>
	{
		/// <summary>
		/// Length of an Sha1Hash
		/// </summary>
		public const int NumBytes = 20;

		/// <summary>
		/// Length of the hash in bits
		/// </summary>
		public const int NumBits = NumBytes * 8;
		readonly ulong _a;
		readonly ulong _b;
		readonly uint _c;

		/// <summary>
		/// Hash consisting of zeroes
		/// </summary>
		public static Sha1Hash Zero { get; } = new Sha1Hash(0, 0, 0);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="span">Memory to construct from</param>
		public Sha1Hash(ReadOnlySpan<byte> span)
			: this(BinaryPrimitives.ReadUInt64BigEndian(span), BinaryPrimitives.ReadUInt64BigEndian(span.Slice(8)), BinaryPrimitives.ReadUInt32BigEndian(span.Slice(16)))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="a">First 64 bits of the hash</param>
		/// <param name="b">Next 64 bits of the hash</param>
		/// <param name="c">Final 32 bits of the hash</param>
		public Sha1Hash(ulong a, ulong b, uint c)
		{
			_a = a;
			_b = b;
			_c = c;
		}

		/// <summary>
		/// Creates a content hash for a block of data, using a given algorithm.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Sha1Hash Compute(ReadOnlySpan<byte> data)
		{
			byte[] output = new byte[20];
			if (!SHA1.TryHashData(data, output, out int bytesWritten) || bytesWritten != NumBytes)
			{
				throw new Exception("Unable to hash data");
			}
			return new Sha1Hash(output);
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Sha1Hash Parse(string text)
		{
			return new Sha1Hash(StringUtils.ParseHexString(text));
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Sha1Hash Parse(ReadOnlySpan<byte> text)
		{
			return new Sha1Hash(StringUtils.ParseHexString(text));
		}

		/// <inheritdoc cref="IComparable{T}.CompareTo(T)"/>
		public int CompareTo(Sha1Hash other)
		{
			if (_a != other._a)
			{
				return (_a < other._a) ? -1 : +1;
			}
			else if (_b != other._b)
			{
				return (_b < other._b) ? -1 : +1;
			}
			else
			{
				return (_c < other._c) ? -1 : +1;
			}
		}

		/// <inheritdoc/>
		public bool Equals(Sha1Hash other) => _a == other._a && _b == other._b && _c == other._c;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is Sha1Hash hash) && Equals(hash);

		/// <inheritdoc/>
		public override int GetHashCode() => (int)_a;

		/// <inheritdoc/>
		public Utf8String ToUtf8String() => StringUtils.FormatUtf8HexString(ToByteArray());

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(ToByteArray());

		/// <summary>
		/// Convert the hash to a byte array
		/// </summary>
		/// <returns></returns>
		public byte[] ToByteArray()
		{
			byte[] data = new byte[NumBytes];
			CopyTo(data);
			return data;
		}

		/// <summary>
		/// Copies this hash into a span
		/// </summary>
		/// <param name="span"></param>
		public void CopyTo(Span<byte> span)
		{
			BinaryPrimitives.WriteUInt64BigEndian(span, _a);
			BinaryPrimitives.WriteUInt64BigEndian(span[8..], _b);
			BinaryPrimitives.WriteUInt32BigEndian(span[16..], _c);
		}

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator ==(Sha1Hash a, Sha1Hash b) => a.Equals(b);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator !=(Sha1Hash a, Sha1Hash b) => !(a == b);

		/// <summary>
		/// Tests whether A > B
		/// </summary>
		public static bool operator >(Sha1Hash a, Sha1Hash b) => a.CompareTo(b) > 0;

		/// <summary>
		/// Tests whether A is less than B
		/// </summary>
		public static bool operator <(Sha1Hash a, Sha1Hash b) => a.CompareTo(b) < 0;

		/// <summary>
		/// Tests whether A is greater than or equal to B
		/// </summary>
		public static bool operator >=(Sha1Hash a, Sha1Hash b) => a.CompareTo(b) >= 0;

		/// <summary>
		/// Tests whether A is less than or equal to B
		/// </summary>
		public static bool operator <=(Sha1Hash a, Sha1Hash b) => a.CompareTo(b) <= 0;
	}

	/// <summary>
	/// Extension methods for dealing with Sha1Hash values
	/// </summary>
	public static class Sha1HashExtensions
	{
		/// <summary>
		/// Read an <see cref="Sha1Hash"/> from a memory reader
		/// </summary>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static Sha1Hash ReadSha1Hash(this IMemoryReader reader)
		{
			ReadOnlySpan<byte> span = reader.GetSpan(Sha1Hash.NumBytes);
			reader.Advance(Sha1Hash.NumBytes);
			return new Sha1Hash(span);
		}

		/// <summary>
		/// Write an <see cref="Sha1Hash"/> to a memory writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="hash"></param>
		public static void WriteSha1Hash(this IMemoryWriter writer, Sha1Hash hash)
		{
			Span<byte> span = writer.GetSpan(Sha1Hash.NumBytes);
			writer.Advance(Sha1Hash.NumBytes);
			hash.CopyTo(span);
		}
	}

	/// <summary>
	/// Type converter for Sha1Hash to and from JSON
	/// </summary>
	sealed class Sha1HashJsonConverter : JsonConverter<Sha1Hash>
	{
		/// <inheritdoc/>
		public override Sha1Hash Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => Sha1Hash.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, Sha1Hash value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToUtf8String().Span);
	}

	/// <summary>
	/// Type converter from strings to Sha1Hash objects
	/// </summary>
	sealed class Sha1HashTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return Sha1Hash.Parse((string)value);
		}
	}
}
