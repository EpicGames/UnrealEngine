// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// System-wide (global) mutex that is thread- and async-safe.
	/// </summary>
	public class SingleInstanceMutex : IDisposable
	{
		/// <summary>
		/// The global mutex instance
		/// </summary>
		private Mutex? _globalMutex;

		/// <summary>
		/// A single-threaded scheduler to run the mutex acquire / release on, to make this class safe for use within an async context.
		/// </summary>
		private readonly SingleThreadTaskScheduler _scheduler = new();

		/// <summary>
		/// Release the mutex and dispose of the object
		/// </summary>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		/// <param name="disposing">Whether the object should be disposed</param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				RunActionOnScheduler(() =>
				{
					_globalMutex?.ReleaseMutex();
					return true;
				});
				_globalMutex?.Dispose();
				_globalMutex = null;
				_scheduler.Dispose();
			}
		}

		private Task<bool> RunActionOnScheduler(Func<bool> action, CancellationToken cancellationToken = default)
		{
			return Task.Factory.StartNew(action, cancellationToken, TaskCreationOptions.None, _scheduler);
		}

		private bool GetMutex(string mutexName, bool wait, CancellationToken cancellationToken = default)
		{
			// Try to create the mutex, with it initially locked
			_globalMutex = new Mutex(true, mutexName, out bool createdMutex);

			// If we didn't create the mutex, we can wait for it or fail immediately
			if (!createdMutex)
			{
				if (wait)
				{
					try
					{
						// Waiting for multiple handles is only supported on Windows
						if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
						{
							WaitHandle[] handles = [cancellationToken.WaitHandle, _globalMutex];
							if (WaitHandle.WaitAny(handles) == 0)
							{
								cancellationToken.ThrowIfCancellationRequested();
							}
						}
						else
						{
							while (!_globalMutex.WaitOne(100))
							{
								if (cancellationToken.IsCancellationRequested)
								{
									cancellationToken.ThrowIfCancellationRequested();
								}
							}
						}
					}
					catch (AbandonedMutexException)
					{
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Acquires a global mutex with the given name.
		/// </summary>
		/// <param name="name">Name of the mutex to acquire.</param>
		/// <returns>Handle to the mutex. Must be disposed when complete.</returns>
		public static IDisposable Acquire(string name)
		{
#pragma warning disable CA2000 // Don't need to dispose of tasks.
			return AcquireAsync(name).GetAwaiter().GetResult();
#pragma warning restore CA2000
		}

		/// <summary>
		/// Acquires a global mutex with the given name.
		/// </summary>
		/// <param name="name">Name of the mutex to acquire.</param>
		/// <param name="cancellationToken">Cancellation token for the wait.</param>
		/// <returns>Handle to the mutex. Must be disposed when complete.</returns>
		public static async Task<IDisposable> AcquireAsync(string name, CancellationToken cancellationToken = default)
		{
			SingleInstanceMutex mutex = new();
			try
			{
				await mutex.RunActionOnScheduler(() => mutex.GetMutex(name, true, cancellationToken), cancellationToken).ConfigureAwait(false);
			}
			catch
			{
				// Calling dispose would try to release the mutex, so don't do that - instead just dispose the bits we need to.
				mutex._globalMutex?.Dispose();
				mutex._scheduler.Dispose();
				throw;
			}
			return mutex;
		}

		/// <summary>
		/// Tries to acquire a global mutex with the given name if it's available.
		/// </summary>
		/// <param name="name">Name of the mutex to acquire.</param>
		/// <param name="mutex">The mutex if acquired, or <see langword="null"/> if not.</param>
		/// <returns>Whether the mutex was acquired.</returns>
		public static bool TryAcquire(string name, [NotNullWhen(true)] out IDisposable? mutex)
		{
			SingleInstanceMutex tempMutex = new();
			try
			{
				bool didAcquire = tempMutex.RunActionOnScheduler(() => tempMutex.GetMutex(name, false)).GetAwaiter().GetResult();
				if (!didAcquire)
				{
					mutex = null;
					tempMutex._globalMutex?.Dispose();
					tempMutex._scheduler.Dispose();
					return false;
				}

				mutex = tempMutex;
				return true;
			}
			catch
			{
				mutex = null;
				tempMutex._globalMutex?.Dispose();
				tempMutex._scheduler.Dispose();
				return false;
			}
		}
	}
}
