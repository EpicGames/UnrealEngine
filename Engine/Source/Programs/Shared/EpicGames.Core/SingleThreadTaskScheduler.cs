// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core;

/// <summary>
/// A task scheduler that runs with a single thread.
/// Allows for use of the TPL whilst running Tasks on a consistent thread, useful for anything that requires thread affinity,
/// such as global mutexes.
/// </summary>
public sealed class SingleThreadTaskScheduler : TaskScheduler, IDisposable
{
	/// <summary>
	/// Creates a single-thread task scheduler.
	/// </summary>
	public SingleThreadTaskScheduler()
	{
		_cancellationTokenSource = new CancellationTokenSource();
		_thread = new Thread(ThreadEntryPoint)
		{
			// The thread should not stop the application from exiting, so set it as background.
			IsBackground = true
		};
		_thread.Start();
	}

	/// <inheritdoc />
	public override int MaximumConcurrencyLevel => 1;

	/// <inheritdoc />
	public void Dispose()
	{
		_cancellationTokenSource.Cancel();
		_cancellationTokenSource.Dispose();
		_tasks.Dispose();
		GC.SuppressFinalize(this);
	}

	/// <inheritdoc />
	protected override IEnumerable<Task> GetScheduledTasks()
	{
		return [.. _tasks];
	}

	/// <inheritdoc />
	protected override void QueueTask(Task task)
	{
		_tasks.Add(task);
	}

	/// <inheritdoc />
	protected override bool TryExecuteTaskInline(Task task, bool taskWasPreviouslyQueued)
	{
		if (Thread.CurrentThread == _thread)
		{
			return TryExecuteTask(task);
		}

		return false;
	}

	private void ThreadEntryPoint()
	{
		while (!_cancellationTokenSource.IsCancellationRequested)
		{
			try
			{
				Task task = _tasks.Take(_cancellationTokenSource.Token);
				TryExecuteTask(task);
			}
			catch (OperationCanceledException)
			{
				// Don't do anything, we'll exit the loop immediately.
			}
			catch (ObjectDisposedException)
			{
				// Don't do anything, we'll exit the loop immediately.
			}
		}
	}

	private readonly CancellationTokenSource _cancellationTokenSource;
	private readonly Thread _thread;
	private readonly BlockingCollection<Task> _tasks = [];
}