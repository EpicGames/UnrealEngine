// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Net.Sockets;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Extension methods for sockets
	/// </summary>
	public static class SocketExtensions
	{
		/// <summary>
		/// Reads a complete buffer from the given socket, retrying reads until the buffer is full.
		/// </summary>
		/// <param name="socket">Socket to read from</param>
		/// <param name="buffer">Buffer to store the data</param>
		/// <param name="socketFlags">Flags for the socket receive call</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task ReceiveMessageAsync(this Socket socket, Memory<byte> buffer, SocketFlags socketFlags, CancellationToken cancellationToken)
		{
			if (!await TryReceiveMessageAsync(socket, buffer, socketFlags, cancellationToken))
			{
				throw new EndOfStreamException();
			}
		}

		/// <summary>
		/// Reads a complete buffer from the given socket, retrying reads until the buffer is full.
		/// </summary>
		/// <param name="socket">Socket to read from</param>
		/// <param name="buffer">Buffer to store the data</param>
		/// <param name="socketFlags">Flags for the socket receive call</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<bool> TryReceiveMessageAsync(this Socket socket, Memory<byte> buffer, SocketFlags socketFlags, CancellationToken cancellationToken)
		{
			for (int offset = 0; offset < buffer.Length;)
			{
				int read = await socket.ReceiveAsync(buffer.Slice(offset), socketFlags, cancellationToken);
				if (read == 0)
				{
					return false;
				}
				offset += read;
			}
			return true;
		}

		/// <summary>
		/// Sends a complete buffer over the given socket.
		/// </summary>
		/// <param name="socket">Socket to write to</param>
		/// <param name="buffer">Buffer to write</param>
		/// <param name="socketFlags">Flags for the socket sent call</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task SendMessageAsync(this Socket socket, ReadOnlyMemory<byte> buffer, SocketFlags socketFlags, CancellationToken cancellationToken)
		{
			while (buffer.Length > 0)
			{
				int written = await socket.SendAsync(buffer, socketFlags, cancellationToken);
				buffer = buffer.Slice(written);
			}
		}
	}
}
