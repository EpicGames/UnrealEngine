// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;

namespace EpicGames.Core
{
	/// <summary>
	/// StringBuffer that can only live on the stack and does not make any allocations as long as initial buffer is big enough
	/// Usage:
	///   Span&lt;char&gt; initial = stackalloc char[512];
	///   using StackStringBuilder sb = new(initial);
	///   sb.Append("Foo");
	/// </summary>
	public ref struct StackStringBuilder
	{
		private Span<char> _chars;
		private int _pos;
		private char[]? _arrayToReturnToPool;

		/// <summary>
		/// Ctor
		/// </summary>
		/// <param name="initialBuffer">Initial buffer</param>
		public StackStringBuilder(Span<char> initialBuffer)
		{
			_chars = initialBuffer;
		}

		/// <summary>
		/// Length of string buffer
		/// </summary>
		public int Length => _pos;

		/// <summary>
		/// Access a character by index
		/// </summary>
		/// <param name="index"></param>
		public ref char this[int index] => ref _chars[index];

		/// <summary>
		/// Resize string buffer
		/// </summary>
		/// <param name="newSize"></param>
		public void Resize(int newSize)
		{
			_pos = newSize;
		}

		/// <summary>
		/// Append character
		/// </summary>
		/// <param name="c"></param>
		public void Append(char c)
		{
			int pos = _pos;
			if ((uint)pos >= (uint)_chars.Length)
			{
				Grow(1);
			}
			_chars[pos] = c;
			_pos = pos + 1;
		}

		/// <summary>
		/// Append span of characters
		/// </summary>
		/// <param name="s"></param>
		public void Append(ReadOnlySpan<char> s)
		{
			int newPos = _pos + s.Length;
			if (newPos > _chars.Length)
			{
				Grow(s.Length);
			}
			s.CopyTo(_chars.Slice(_pos));
			_pos = newPos;
		}

		/// <summary>
		/// Replaces all character of from to to
		/// </summary>
		/// <param name="from"></param>
		/// <param name="to"></param>
		public void Replace(char from, char to)
		{
			_chars.Slice(0, _pos).Replace(from, to);
		}

		/// <summary>
		/// To string
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return new string(_chars.Slice(0, _pos));
		}

		/// <summary>
		/// As span
		/// </summary>
		/// <returns></returns>
		public ReadOnlySpan<char> AsSpan() => _chars.Slice(0, _pos);

		/// <summary>
		/// Dispose buffer
		/// </summary>
		public void Dispose()
		{
			char[]? toReturn = _arrayToReturnToPool;
			this = default;
			if (toReturn is not null)
			{
				ArrayPool<char>.Shared.Return(toReturn);
			}
		}

		private void Grow(int requiredAdditionalCapacity)
		{
			int newSize = Math.Max(_pos + requiredAdditionalCapacity, _chars.Length * 2);
			char[] poolArray = ArrayPool<char>.Shared.Rent(newSize);
			_chars.Slice(0, _pos).CopyTo(poolArray);
			_chars = poolArray;
			_arrayToReturnToPool = poolArray;
		}
	}
}