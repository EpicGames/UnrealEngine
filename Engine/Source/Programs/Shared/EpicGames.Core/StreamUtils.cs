// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for streams
	/// </summary>
	public static class StreamUtils
	{
		/// <summary>
		/// Issues read calls until the given buffer has been filled with data or the stream is at an end
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="buffer">Buffer to receive the output data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<int> ReadGreedyAsync(this Stream stream, Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			int length = 0;
			while (length < buffer.Length)
			{
				int readLength = await stream.ReadAsync(buffer.Slice(length), cancellationToken);
				if (readLength == 0)
				{
					break;
				}
				length += readLength;
			}
			return length;
		}

		/// <summary>
		/// Reads a fixed amount of data from a stream, throwing an exception if the entire buffer cannot be read.
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="buffer">Buffer to receive the output data</param>
		public static void ReadFixedLengthBytes(this Stream stream, Span<byte> buffer)
		{
			for (int offset = 0; offset < buffer.Length;)
			{
				int readLength = stream.Read(buffer.Slice(offset));
				if (readLength == 0)
				{
					throw new EndOfStreamException($"Unexpected end of stream while trying to read {buffer.Length} bytes.");
				}
				offset += readLength;
			}
		}

		/// <summary>
		/// Reads a fixed amount of data from a stream, throwing an exception if the entire buffer cannot be read.
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="buffer">Buffer to receive the output data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task ReadFixedLengthBytesAsync(this Stream stream, Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			for (int offset = 0; offset < buffer.Length;)
			{
				int readLength = await stream.ReadAsync(buffer.Slice(offset), cancellationToken);
				if (readLength == 0)
				{
					throw new EndOfStreamException($"Unexpected end of stream while trying to read {buffer.Length} bytes.");
				}
				offset += readLength;
			}
		}

		/// <summary>
		/// Read the contents of a file async using double buffering
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="cancellationToken">Cancellation token used to terminate processing</param>
		/// <returns>Contents of the stream</returns>
		public static async Task<byte[]> ReadAllBytesAsync(this Stream stream, CancellationToken cancellationToken = default)
		{
			using MemoryStream memoryStream = new();
			await stream.CopyToAsync(memoryStream, cancellationToken);
			return memoryStream.ToArray();
		}

		/// <summary>
		/// Read the contents of a file async using double buffering
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="fileSizeHint">If available, the file size so an appropriate buffer size can be used</param>
		/// <param name="minBufferSize">Minimum size of the buffer</param>
		/// <param name="maxBufferSize">Maximum size of the buffer</param>
		/// <param name="callback">Callback used to send read data back to the caller</param>
		/// <param name="cancellationToken">Cancellation token used to terminate processing</param>
		/// <returns>Contents of the stream</returns>
		public static async Task ReadAllBytesAsync(this Stream stream, long fileSizeHint, int minBufferSize, int maxBufferSize, Func<ReadOnlyMemory<byte>, Task> callback, CancellationToken cancellationToken = default)
		{
			int bufferLength = minBufferSize;
			if (fileSizeHint > minBufferSize)
			{
				bufferLength = (int)(fileSizeHint <= maxBufferSize ? fileSizeHint : maxBufferSize);
			}

			await ReadAllBytesAsync(stream, bufferLength, callback, cancellationToken);
		}

		/// <summary>
		/// Read the contents of a file async using double buffering
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <param name="bufferLength">Size of the internal buffer</param>
		/// <param name="callback">Callback used to send read data back to the caller</param>
		/// <param name="cancellationToken">Cancellation token used to terminate processing</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static async Task ReadAllBytesAsync(this Stream stream, int bufferLength, Func<ReadOnlyMemory<byte>, Task> callback, CancellationToken cancellationToken = default)
		{
			using IMemoryOwner<byte> owner = MemoryPool<byte>.Shared.Rent(bufferLength * 2);
			Memory<byte> buffer = owner.Memory;

			int readBufferOffset = 0;
			Memory<byte> appendBuffer = Memory<byte>.Empty;
			for (; ; )
			{
				// Start a read into memory
				Memory<byte> readBuffer = buffer[readBufferOffset..(readBufferOffset + bufferLength)];
				Task<int> readTask = Task.Run(async () => await stream.ReadAsync(readBuffer, cancellationToken), cancellationToken);

				// In the meantime, append the last data that was read to the tree
				if (appendBuffer.Length > 0)
				{
					await callback(appendBuffer[0..appendBuffer.Length]);
				}

				// Wait for the read to finish
				int numBytes = await readTask;
				if (numBytes == 0)
				{
					break;
				}

				// Switch the buffers around
				appendBuffer = readBuffer[0..numBytes];
				readBufferOffset ^= bufferLength;
			}
		}
	}
}
