// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

#pragma warning disable CA1710

namespace EpicGames.Core
{
	/// <summary>
	/// <see cref="Stream"/> implementation that wraps another stream, allowing derived classes to override some methods of their chosing.
	/// </summary>
	public class StreamWrapper : Stream
	{
		readonly Stream _inner;

		/// <summary>
		/// Accessor for the wrapped stream
		/// </summary>
		protected Stream Inner => _inner;

		/// <inheritdoc/>
		public override bool CanRead => _inner.CanRead;

		/// <inheritdoc/>
		public override bool CanSeek => _inner.CanSeek;

		/// <inheritdoc/>
		public override bool CanTimeout => _inner.CanTimeout;

		/// <inheritdoc/>
		public override bool CanWrite => _inner.CanWrite;

		/// <inheritdoc/>
		public override long Length => _inner.Length;

		/// <inheritdoc/>
		public override long Position { get => _inner.Position; set => _inner.Position = value; }

		/// <inheritdoc/>
		public override int ReadTimeout { get => _inner.ReadTimeout; set => _inner.ReadTimeout = value; }

		/// <inheritdoc/>
		public override int WriteTimeout { get => _inner.WriteTimeout; set => _inner.WriteTimeout = value; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The stream to wrap</param>
		public StreamWrapper(Stream inner) => _inner = inner;

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				_inner.Dispose();
			}
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			await _inner.DisposeAsync();

			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public override IAsyncResult BeginRead(byte[] buffer, int offset, int count, AsyncCallback? callback, object? state) => _inner.BeginRead(buffer, offset, count, callback, state);

		/// <inheritdoc/>
		public override IAsyncResult BeginWrite(byte[] buffer, int offset, int count, AsyncCallback? callback, object? state) => _inner.BeginWrite(buffer, offset, count, callback, state);

		/// <inheritdoc/>
		public override void Close() => _inner.Close();

		/// <inheritdoc/>
		public override void CopyTo(Stream destination, int bufferSize) => _inner.CopyTo(destination, bufferSize);

		/// <inheritdoc/>
		public override Task CopyToAsync(Stream destination, int bufferSize, CancellationToken cancellationToken) => _inner.CopyToAsync(destination, bufferSize, cancellationToken);

		/// <inheritdoc/>
		public override int EndRead(IAsyncResult asyncResult) => _inner.EndRead(asyncResult);

		/// <inheritdoc/>
		public override void EndWrite(IAsyncResult asyncResult) => _inner.EndWrite(asyncResult);

		/// <inheritdoc/>
		public override void Flush() => _inner.Flush();

		/// <inheritdoc/>
		public override Task FlushAsync(CancellationToken cancellationToken) => _inner.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => _inner.Read(buffer, offset, count);

		/// <inheritdoc/>
		public override int Read(Span<byte> buffer) => _inner.Read(buffer);

		/// <inheritdoc/>
		public override Task<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _inner.ReadAsync(buffer, offset, count, cancellationToken);

		/// <inheritdoc/>
		public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => _inner.ReadAsync(buffer, cancellationToken);

		/// <inheritdoc/>
		public override int ReadByte() => _inner.ReadByte();

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => _inner.Seek(offset, origin);

		/// <inheritdoc/>
		public override void SetLength(long value) => _inner.SetLength(value);

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => _inner.Write(buffer, offset, count);

		/// <inheritdoc/>
		public override void Write(ReadOnlySpan<byte> buffer) => _inner.Write(buffer);

		/// <inheritdoc/>
		public override Task WriteAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => _inner.WriteAsync(buffer, offset, count, cancellationToken);

		/// <inheritdoc/>
		public override ValueTask WriteAsync(ReadOnlyMemory<byte> buffer, CancellationToken cancellationToken = default) => _inner.WriteAsync(buffer, cancellationToken);

		/// <inheritdoc/>
		public override void WriteByte(byte value) => _inner.WriteByte(value);
	}

	/// <summary>
	/// Extension methods for <see cref="StreamWrapper"/>
	/// </summary>
	public static class StreamWrapperExtensions
	{
		class StreamResourceOwner : StreamWrapper
		{
			IDisposable? _resource;

			public StreamResourceOwner(Stream inner, IDisposable resource)
				: base(inner)
			{
				_resource = resource;
			}

			/// <inheritdoc/>
			protected override void Dispose(bool disposing)
			{
				if (disposing)
				{
					DisposeResource();
				}

				base.Dispose(disposing);
			}

			/// <inheritdoc/>
			public override async ValueTask DisposeAsync()
			{
				DisposeResource();

				await base.DisposeAsync();
			}

			void DisposeResource()
			{
				_resource?.Dispose();
				_resource = null;
			}
		}

		/// <summary>
		/// Wraps ownership of another resource with a stream
		/// </summary>
		/// <param name="stream">Stream to wrap</param>
		/// <param name="resource">Additional resource to control ownership of</param>
		/// <returns>New stream which will dispose of the given resource when it is disposed</returns>
		public static Stream WrapOwnership(this Stream stream, IDisposable resource) => new StreamResourceOwner(stream, resource);
	}
}
