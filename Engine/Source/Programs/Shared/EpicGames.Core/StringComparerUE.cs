// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// Comparer for String objects.  However, it implements UE style ignore case compare
	/// </summary>
	public class StringComparerUE : IComparer<string>, IEqualityComparer<string>
	{
		/// <summary>
		/// Static instance of an ordinal String comparer
		/// </summary>
		public static StringComparerUE Ordinal { get; } = new StringComparerUE(StringComparison.Ordinal);

		/// <summary>
		/// Static instance of an ordinal String comparer which ignores case
		/// </summary>
		public static StringComparerUE OrdinalIgnoreCase { get; } = new StringComparerUE(StringComparison.OrdinalIgnoreCase);

		/// <summary>
		/// The comparison type
		/// </summary>
		public StringComparison ComparisonType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="comparisonType">Type of comparison to perform</param>
		public StringComparerUE(StringComparison comparisonType)
		{
			ComparisonType = comparisonType;
		}

		/// <inheritdoc/>
		public bool Equals(string? x, string? y)
		{
			return String.Equals(x, y, ComparisonType);
		}

		/// <inheritdoc/>
		public int GetHashCode(string obj)
		{
			return String.GetHashCode(obj, ComparisonType);
		}

		/// <inheritdoc/>
		public int Compare(string? x, string? y)
		{
			if (ComparisonType == StringComparison.OrdinalIgnoreCase)
			{
				if (x == null)
				{
					return y == null ? 0 : -1;
				}
				else if (y == null)
				{
					return 1;
				}

				return StringUtils.CompareIgnoreCaseUe(x.AsSpan(), y.AsSpan());
			}
			else
			{
				return String.Compare(x, y, ComparisonType);
			}
		}
	}
}
