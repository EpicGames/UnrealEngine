// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Utility methods for strings
	/// </summary>
	public static class StringUtils
	{
		/// <summary>
		/// Array mapping from ascii index to hexadecimal digits.
		/// </summary>
		static readonly sbyte[] s_hexDigits = CreateHexDigitsTable();

		/// <summary>
		/// Hex digits to utf8 byte
		/// </summary>
		static readonly byte[] s_hexDigitToUtf8Byte = Encoding.UTF8.GetBytes("0123456789abcdef");

		/// <summary>
		/// Array mapping human readable size of bytes, 1024^x. long max is within the range of Exabytes.
		/// </summary>
		static readonly string[] s_byteSizes = ["B", "KB", "MB", "GB", "TB", "PB", "EB"];

		/// <summary>
		/// Static constructor. Initializes the HexDigits array.
		/// </summary>
		static sbyte[] CreateHexDigitsTable()
		{
			sbyte[] hexDigits = new sbyte[256];
			for (int idx = 0; idx < 256; idx++)
			{
				hexDigits[idx] = -1;
			}
			for (int idx = '0'; idx <= '9'; idx++)
			{
				hexDigits[idx] = (sbyte)(idx - '0');
			}
			for (int idx = 'a'; idx <= 'f'; idx++)
			{
				hexDigits[idx] = (sbyte)(10 + idx - 'a');
			}
			for (int idx = 'A'; idx <= 'F'; idx++)
			{
				hexDigits[idx] = (sbyte)(10 + idx - 'A');
			}
			return hexDigits;
		}

		/// <summary>
		/// Indents a string by a given indent
		/// </summary>
		/// <param name="text">The text to indent</param>
		/// <param name="indent">The indent to add to each line</param>
		/// <returns>The indented string</returns>
		public static string Indent(string text, string indent)
		{
			string result = "";
			if (text.Length > 0)
			{
				result = indent + text.Replace("\n", "\n" + indent, StringComparison.Ordinal);
			}
			return result;
		}

		/// <summary>
		/// Expand all the property references (of the form $(PropertyName)) in a string.
		/// </summary>
		/// <param name="text">The input string to expand properties in</param>
		/// <param name="properties">Dictionary of properties to expand</param>
		/// <returns>The expanded string</returns>
		public static string ExpandProperties(string text, Dictionary<string, string> properties)
		{
			return ExpandProperties(text, name =>
			{
				properties.TryGetValue(name, out string? value);
				return value;
			});
		}

		/// <summary>
		/// Expand all the property references (of the form $(PropertyName)) in a string.
		/// </summary>
		/// <param name="text">The input string to expand properties in</param>
		/// <param name="getPropertyValue">Delegate to retrieve a property value</param>
		/// <returns>The expanded string</returns>
		public static string ExpandProperties(string text, Func<string, string?> getPropertyValue)
		{
			string result = text;
			for (int idx = result.IndexOf("$(", StringComparison.Ordinal); idx != -1; idx = result.IndexOf("$(", idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int endIdx = result.IndexOf(')', idx + 2);
				if (endIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string name = result.Substring(idx + 2, endIdx - (idx + 2));

				// Check if we've got a value for this variable
				string? value = getPropertyValue(name);
				if (value == null)
				{
					// Do not expand it; must be preprocessing the script.
					idx = endIdx;
				}
				else
				{
					// Replace the variable, or skip past it
					result = result.Substring(0, idx) + value + result.Substring(endIdx + 1);

					// Make sure we skip over the expanded variable; we don't want to recurse on it.
					idx += value.Length;
				}
			}
			return result;
		}

		/// <inheritdoc cref="WordWrap(String, Int32, Int32, Int32)"/>
		public static IEnumerable<string> WordWrap(string text, int maxWidth)
		{
			return WordWrap(text, 0, 0, maxWidth);
		}

		/// <summary>
		/// Takes a given sentence and wraps it on a word by word basis so that no line exceeds the set maximum line length. Words longer than a line 
		/// are broken up. Returns the sentence as a list of individual lines.
		/// </summary>
		/// <param name="text">The text to be wrapped</param>
		/// <param name="initialIndent">Indent for the first line</param>
		/// <param name="hangingIndent">Indent for subsequent lines</param>
		/// <param name="maxWidth">The maximum (non negative) length of the returned sentences</param>
		public static IEnumerable<string> WordWrap(string text, int initialIndent, int hangingIndent, int maxWidth)
		{
			StringBuilder builder = new();

			int minIdx = 0;
			for (int lineIdx = 0; minIdx < text.Length; lineIdx++)
			{
				int indent = (lineIdx == 0) ? initialIndent : hangingIndent;
				int maxWidthForLine = maxWidth - indent;
				int maxIdx = GetWordWrapLineEnd(text, minIdx, maxWidthForLine);

				int printMaxIdx = maxIdx;
				while (printMaxIdx > minIdx && Char.IsWhiteSpace(text[printMaxIdx - 1]))
				{
					printMaxIdx--;
				}

				builder.Clear();
				builder.Append(' ', indent);
				builder.Append(text, minIdx, printMaxIdx - minIdx);
				yield return builder.ToString();

				minIdx = maxIdx;
			}
		}

		/// <summary>
		/// Gets the next character index to end a word-wrapped line on
		/// </summary>
		static int GetWordWrapLineEnd(string text, int minIdx, int maxWidth)
		{
			maxWidth = Math.Min(maxWidth, text.Length - minIdx);

			int maxIdx = text.IndexOf('\n', minIdx, maxWidth);
			if (maxIdx == -1)
			{
				maxIdx = minIdx + maxWidth;
			}
			else
			{
				return maxIdx + 1;
			}

			if (maxIdx == text.Length)
			{
				return maxIdx;
			}
			else if (Char.IsWhiteSpace(text[maxIdx]))
			{
				for (; ; maxIdx++)
				{
					if (maxIdx == text.Length)
					{
						return maxIdx;
					}
					if (text[maxIdx] != ' ')
					{
						return maxIdx;
					}
				}
			}
			else
			{
				for (int tryMaxIdx = maxIdx; ; tryMaxIdx--)
				{
					if (tryMaxIdx == minIdx)
					{
						return maxIdx;
					}
					if (text[tryMaxIdx - 1] == ' ')
					{
						return tryMaxIdx;
					}
				}
			}
		}

		/// <summary>
		/// Extension method to allow formatting a string to a stringbuilder and appending a newline
		/// </summary>
		/// <param name="builder">The string builder</param>
		/// <param name="format">Format string, as used for StringBuilder.AppendFormat</param>
		/// <param name="args">Arguments for the format string</param>
		public static void AppendLine(this StringBuilder builder, string format, params object[] args)
		{
			builder.AppendFormat(format, args);
			builder.AppendLine();
		}

		/// <summary>
		/// Formats a list of strings in the style "1, 2, 3 and 4"
		/// </summary>
		/// <param name="arguments">List of strings to format</param>
		/// <param name="conjunction">Conjunction to use between the last two items in the list (eg. "and" or "or")</param>
		/// <returns>Formatted list of strings</returns>
		public static string FormatList(IReadOnlyList<string> arguments, string conjunction = "and")
		{
			StringBuilder result = new();
			if (arguments.Count > 0)
			{
				result.Append(arguments[0]);
				for (int idx = 1; idx < arguments.Count; idx++)
				{
					if (idx == arguments.Count - 1)
					{
						result.AppendFormat(" {0} ", conjunction);
					}
					else
					{
						result.Append(", ");
					}
					result.Append(arguments[idx]);
				}
			}
			return result.ToString();
		}

		/// <summary>
		/// Formats a list of strings in the style "1, 2, 3 and 4"
		/// </summary>
		/// <param name="arguments">List of strings to format</param>
		/// <param name="conjunction">Conjunction to use between the last two items in the list (eg. "and" or "or")</param>
		/// <returns>Formatted list of strings</returns>
		public static string FormatList(IEnumerable<string> arguments, string conjunction = "and")
		{
			return FormatList([.. arguments], conjunction);
		}

		/// <summary>
		/// Formats a list of items
		/// </summary>
		/// <param name="items">Array of items</param>
		/// <param name="maxCount">Maximum number of items to include in the list</param>
		/// <returns>Formatted list of items</returns>
		public static string FormatList(string[] items, int maxCount)
		{
			if (items.Length == 0)
			{
				return "unknown";
			}
			else if (items.Length == 1)
			{
				return items[0];
			}
			else if (items.Length <= maxCount)
			{
				return $"{String.Join(", ", items.Take(items.Length - 1))} and {items.Last()}";
			}
			else
			{
				return $"{String.Join(", ", items.Take(maxCount - 1))} and {items.Length - (maxCount - 1)} others";
			}
		}

		/// <summary>
		/// Generates a string suitable for debugging a list of objects using ToString(). Lists one per line with the prefix string on the first line.
		/// </summary>
		/// <param name="objects">The list of objects to inset into the output string</param>
		/// <param name="prefix">Prefix string to print along with the list of objects</param>
		/// <returns>the resulting debug string</returns>
		public static string CreateObjectList<T>(this IEnumerable<T> objects, string prefix)
		{
			return objects.Aggregate(new StringBuilder(prefix), (sb, obj) => sb.AppendFormat($"\n    {obj}")).ToString();
		}

		/// <summary>
		/// Parses a hexadecimal digit
		/// </summary>
		/// <param name="character">Character to parse</param>
		/// <returns>Value of this digit, or -1 if invalid</returns>
		public static int GetHexDigit(byte character)
		{
			return s_hexDigits[character];
		}

		/// <summary>
		/// Parses a hexadecimal digit
		/// </summary>
		/// <param name="character">Character to parse</param>
		/// <returns>Value of this digit, or -1 if invalid</returns>
		public static int GetHexDigit(char character)
		{
			return s_hexDigits[Math.Min((uint)character, 127)];
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <returns>Array of bytes</returns>
		public static byte[] ParseHexString(string text) => ParseHexString(text.AsSpan());

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <returns>Array of bytes</returns>
		public static byte[] ParseHexString(ReadOnlySpan<char> text)
		{
			byte[]? bytes;
			if (!TryParseHexString(text, out bytes))
			{
				throw new FormatException(String.Format("Invalid hex string: '{0}'", text.ToString()));
			}
			return bytes;
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <returns>Array of bytes</returns>
		public static byte[] ParseHexString(ReadOnlySpan<byte> text)
		{
			byte[]? bytes;
			if (!TryParseHexString(text, out bytes))
			{
				throw new FormatException($"Invalid hex string: '{Encoding.UTF8.GetString(text)}'");
			}
			return bytes;
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="outBytes">Receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParseHexString(ReadOnlySpan<char> text, [NotNullWhen(true)] out byte[]? outBytes)
		{
			byte[] bytes = new byte[text.Length / 2];
			if (TryParseHexString(text, bytes))
			{
				outBytes = bytes;
				return true;
			}
			else
			{
				outBytes = null;
				return false;
			}
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="bytes">Receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParseHexString(ReadOnlySpan<char> text, Span<byte> bytes)
		{
			if ((text.Length & 1) != 0)
			{
				return false;
			}

			for (int idx = 0; idx < text.Length; idx += 2)
			{
				int value = (GetHexDigit(text[idx]) << 4) | GetHexDigit(text[idx + 1]);
				if (value < 0)
				{
					return false;
				}
				bytes[idx / 2] = (byte)value;
			}

			return true;
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="outBytes">Receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParseHexString(ReadOnlySpan<byte> text, [NotNullWhen(true)] out byte[]? outBytes)
		{
			byte[] bytes = new byte[text.Length / 2];
			if (TryParseHexString(text, bytes))
			{
				outBytes = bytes;
				return true;
			}
			else
			{
				outBytes = null;
				return false;
			}
		}

		/// <summary>
		/// Parses a hexadecimal string into an array of bytes
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="bytes">Receives the parsed string</param>
		/// <returns></returns>
		public static bool TryParseHexString(ReadOnlySpan<byte> text, Span<byte> bytes)
		{
			if ((text.Length & 1) != 0)
			{
				return false;
			}

			for (int idx = 0; idx < text.Length; idx += 2)
			{
				int value = ParseHexByte(text, idx);
				if (value < 0)
				{
					return false;
				}
				bytes[idx / 2] = (byte)value;
			}

			return true;
		}

		/// <summary>
		/// Parse a hex byte from the given offset into a span of utf8 characters
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <param name="idx">Index within the text to parse</param>
		/// <returns>The parsed value, or a negative value on error</returns>
		public static int ParseHexByte(ReadOnlySpan<byte> text, int idx)
		{
			return ((int)s_hexDigits[text[idx]] << 4) | ((int)s_hexDigits[text[idx + 1]]);
		}

		/// <summary>
		/// Formats an array of bytes as a hexadecimal string
		/// </summary>
		/// <param name="bytes">An array of bytes</param>
		/// <returns>String representation of the array</returns>
		public static string FormatHexString(byte[] bytes)
		{
			return FormatHexString(bytes.AsSpan());
		}

		/// <summary>
		/// Formats an array of bytes as a hexadecimal string
		/// </summary>
		/// <param name="bytes">An array of bytes</param>
		/// <returns>String representation of the array</returns>
		public static string FormatHexString(ReadOnlySpan<byte> bytes)
		{
			const string HexDigits = "0123456789abcdef";

			char[] characters = new char[bytes.Length * 2];
			for (int idx = 0; idx < bytes.Length; idx++)
			{
				characters[idx * 2 + 0] = HexDigits[bytes[idx] >> 4];
				characters[idx * 2 + 1] = HexDigits[bytes[idx] & 15];
			}
			return new string(characters);
		}

		/// <summary>
		/// Formats an array of bytes as a hexadecimal string
		/// </summary>
		/// <param name="bytes">An array of bytes</param>
		/// <returns>String representation of the array</returns>
		public static Utf8String FormatUtf8HexString(ReadOnlySpan<byte> bytes)
		{
			byte[] characters = new byte[bytes.Length * 2];
			for (int idx = 0; idx < bytes.Length; idx++)
			{
				characters[idx * 2 + 0] = s_hexDigitToUtf8Byte[bytes[idx] >> 4];
				characters[idx * 2 + 1] = s_hexDigitToUtf8Byte[bytes[idx] & 15];
			}
			return new Utf8String(characters);
		}

		/// <summary>
		/// Formats an array of bytes as a hexadecimal string
		/// </summary>
		/// <param name="bytes">An array of bytes</param>
		/// <param name="characters">Buffer to receive the characters</param>
		public static void FormatUtf8HexString(ReadOnlySpan<byte> bytes, Span<byte> characters)
		{
			for (int idx = 0; idx < bytes.Length; idx++)
			{
				characters[idx * 2 + 0] = s_hexDigitToUtf8Byte[bytes[idx] >> 4];
				characters[idx * 2 + 1] = s_hexDigitToUtf8Byte[bytes[idx] & 15];
			}
		}

		/// <summary>
		/// Formats a 32-bit unsigned integer as a hexadecimal string
		/// </summary>
		/// <param name="value">Value to render</param>
		/// <param name="characters">Buffer to receive the characters</param>
		public static void FormatLittleEndianUtf8HexString(uint value, Span<byte> characters)
		{
			characters[0] = s_hexDigitToUtf8Byte[(value >> 4) & 15];
			characters[1] = s_hexDigitToUtf8Byte[value & 15];

			characters[2] = s_hexDigitToUtf8Byte[(value >> 12) & 15];
			characters[3] = s_hexDigitToUtf8Byte[(value >> 8) & 15];

			characters[4] = s_hexDigitToUtf8Byte[(value >> 20) & 15];
			characters[5] = s_hexDigitToUtf8Byte[(value >> 16) & 15];

			characters[6] = s_hexDigitToUtf8Byte[(value >> 28) & 15];
			characters[7] = s_hexDigitToUtf8Byte[(value >> 24) & 15];
		}

		/// <summary>
		/// Formats a 32-bit unsigned integer as a hexadecimal string
		/// </summary>
		/// <param name="value">Value to render</param>
		/// <returns>Hex string</returns>
		public static Utf8String FormatUtf8HexString(uint value)
		{
			byte[] buffer = new byte[8];
			FormatUtf8HexString(value, buffer);
			return new Utf8String(buffer);
		}

		/// <summary>
		/// Formats a 32-bit unsigned integer as a hexadecimal string
		/// </summary>
		/// <param name="value">Value to render</param>
		/// <param name="characters">Buffer to receive the characters</param>
		public static void FormatUtf8HexString(uint value, Span<byte> characters)
		{
			characters[0] = s_hexDigitToUtf8Byte[(value >> 28) & 15];
			characters[1] = s_hexDigitToUtf8Byte[(value >> 24) & 15];
			characters[2] = s_hexDigitToUtf8Byte[(value >> 20) & 15];
			characters[3] = s_hexDigitToUtf8Byte[(value >> 16) & 15];
			characters[4] = s_hexDigitToUtf8Byte[(value >> 12) & 15];
			characters[5] = s_hexDigitToUtf8Byte[(value >> 8) & 15];
			characters[6] = s_hexDigitToUtf8Byte[(value >> 4) & 15];
			characters[7] = s_hexDigitToUtf8Byte[value & 15];
		}

		/// <summary>
		/// Formats a 32-bit unsigned integer as a hexadecimal string
		/// </summary>
		/// <param name="value">Value to render</param>
		/// <param name="characters">Buffer to receive the characters</param>
		public static void FormatUtf8HexString(ulong value, Span<byte> characters)
		{
			FormatUtf8HexString((uint)(value >> 32), characters);
			FormatUtf8HexString((uint)value, characters.Slice(8));
		}

		/// <summary>
		/// Quotes a string as a command line argument
		/// </summary>
		/// <param name="str">The string to quote</param>
		/// <returns>The quoted argument if it contains any spaces, otherwise the original string</returns>
		public static string QuoteArgument(this string str)
		{
			if (str.Contains(' ', StringComparison.Ordinal))
			{
				return $"\"{str}\"";
			}
			else
			{
				return str;
			}
		}

		/// <summary>
		/// Removes the quotes from the beginning and end of a string (if any), can be used to reverse String.QuoteArgument
		/// </summary>
		/// <param name="str">The string to remove the quotes from</param>
		/// <returns>A string without surrounding quotes</returns>
		public static string StripQuoteArgument(this string str)
		{
			if (str.StartsWith('\"') && str.EndsWith('\"'))
			{
				return str.Substring(1, str.Length - 2);
			}
			else
			{
				return str;
			}
		}

		/// <summary>
		/// Formats bytes into a human readable string
		/// </summary>
		/// <param name="bytes">The total number of bytes</param>
		/// <param name="decimalPlaces">The number of decimal places to round the resulting value</param>
		/// <returns>Human readable string based on the value of Bytes</returns>
		public static string FormatBytesString(long bytes, int decimalPlaces = 2)
		{
			if (bytes == 0)
			{
				return $"0 {s_byteSizes[0]}";
			}
			long bytesAbs = Math.Abs(bytes);
			int power = Convert.ToInt32(Math.Floor(Math.Log(bytesAbs, 1024)));
			double value = Math.Round(bytesAbs / Math.Pow(1024, power), decimalPlaces);
			return $"{(Math.Sign(bytes) * value)} {s_byteSizes[power]}";
		}

		/// <summary>
		/// Formats bytes into a human readable string
		/// </summary>
		/// <param name="bytes">The total number of bytes</param>
		/// <param name="decimalPlaces">The number of decimal places to round the resulting value</param>
		/// <returns>Human readable string based on the value of bytes</returns>
		public static string FormatBytesString(ulong bytes, int decimalPlaces = 2)
		{
			if (bytes == 0)
			{
				return $"0 {s_byteSizes[0]}";
			}

			int power = Convert.ToInt32(Math.Floor(Math.Log(bytes, 1024)));
			double value = Math.Round(bytes / Math.Pow(1024, power), decimalPlaces);
			return $"{value} {s_byteSizes[power]}";
		}

		/// <summary>
		/// Converts a bytes string into bytes. E.g 1.2KB -> 1229
		/// </summary>
		/// <param name="bytesString"></param>
		/// <returns></returns>
		public static long ParseBytesString(string bytesString)
		{
			bytesString = bytesString.Trim();

			int power = s_byteSizes.FindIndex(s => (s != s_byteSizes[0]) && bytesString.EndsWith(s, StringComparison.InvariantCultureIgnoreCase)); // need to handle 'B' suffix separately
			if (power == -1 && bytesString.EndsWith(s_byteSizes[0], StringComparison.Ordinal))
			{
				power = 0;
			}
			if (power != -1)
			{
				bytesString = bytesString.Substring(0, bytesString.Length - s_byteSizes[power].Length);
			}

			double value = Double.Parse(bytesString);
			if (power > 0)
			{
				value *= Math.Pow(1024, power);
			}

			return (long)Math.Round(value);
		}

		/// <summary>
		/// Converts a bytes string into bytes. E.g 1.5KB -> 1536
		/// </summary>
		/// <param name="bytesString"></param>
		/// <param name="bytes">Receives the parsed bytes</param>
		/// <returns></returns>
		public static bool TryParseBytesString(string bytesString, out long? bytes)
		{
			try
			{
				bytes = ParseBytesString(bytesString);
				return true;
			}
			catch (Exception)
			{
			}

			bytes = null;
			return false;
		}

		/// <summary>
		/// Parses a string to remove VT100 escape codes
		/// </summary>
		/// <returns></returns>
		public static string ParseEscapeCodes(string line)
		{
			char escapeChar = '\u001b';

			int index = line.IndexOf(escapeChar, StringComparison.Ordinal);
			if (index != -1)
			{
				int lastIndex = 0;

				StringBuilder result = new();
				for (; ; )
				{
					result.Append(line, lastIndex, index - lastIndex);

					while (index < line.Length)
					{
						char character = line[index];
						if ((character >= 'a' && character <= 'z') || (character >= 'A' && character <= 'Z'))
						{
							index++;
							break;
						}
						index++;
					}

					lastIndex = index;

					index = line.IndexOf(escapeChar, index);
					if (index == -1)
					{
						break;
					}
				}
				result.Append(line, lastIndex, line.Length - lastIndex);

				line = result.ToString();
			}

			return line;
		}

		/// <summary>
		/// Truncates the given string to the maximum length, appending an elipsis if it is longer than allowed.
		/// </summary>
		/// <param name="text"></param>
		/// <param name="maxLength"></param>
		/// <returns></returns>
		public static string Truncate(string text, int maxLength)
		{
			if (text.Length > maxLength)
			{
				text = text.Substring(0, maxLength - 3) + "...";
			}
			return text;
		}

		/// <summary>
		/// Compare two strings using UnrealEngine's ignore case algorithm
		/// </summary>
		/// <param name="x">First string to compare</param>
		/// <param name="y">Second string to compare</param>
		/// <returns>Less than zero if X &lt; Y, zero if X == Y, and greater than zero if X &gt; y</returns>
		public static int CompareIgnoreCaseUe(ReadOnlySpan<char> x, ReadOnlySpan<char> y)
		{
			int length = x.Length < y.Length ? x.Length : y.Length;

			for (int index = 0; index < length; ++index)
			{
				char xc = x[index];
				char yc = y[index];
				if (xc == yc)
				{
					continue;
				}
				else if (((xc | yc) & 0xffffff80) == 0) // if (BothAscii)
				{
					if (xc >= 'A' && xc <= 'Z')
					{
						xc += (char)32;
					}
					if (yc >= 'A' && yc <= 'Z')
					{
						yc += (char)32;
					}
					int diff = xc - yc;
					if (diff != 0)
					{
						return diff;
					}
				}
				else
				{
					return xc - yc;
				}
			}

			if (x.Length == length)
			{
				return y.Length == length ? 0 : /* X[Length] */ -y[length];
			}
			else
			{
				return x[length] /* - Y[Length] */;
			}
		}
	}
}
