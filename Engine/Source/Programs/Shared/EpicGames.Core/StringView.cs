// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Core
{
	/// <summary>
	/// View of a character string. Allows comparing/manipulating substrings without unnecessary memory allocations.
	/// </summary>
	public readonly struct StringView : IEquatable<StringView>
	{
		/// <summary>
		/// Memory containing the characters
		/// </summary>
		public ReadOnlyMemory<char> Memory { get; }

		/// <summary>
		/// Span for the sequence of characters
		/// </summary>
		public ReadOnlySpan<char> Span => Memory.Span;

		/// <summary>
		/// Length of the sequence of characters
		/// </summary>
		public int Length => Memory.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">String view to construct from</param>
		public StringView(StringView text)
		{
			Memory = text.Memory;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">String view to construct from</param>
		/// <param name="index">Offset within the string for this view</param>
		public StringView(StringView text, int index)
		{
			Memory = text.Memory.Slice(index);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">String view to construct from</param>
		/// <param name="index">Offset within the string for this view</param>
		/// <param name="count">Number of characters to include</param>
		public StringView(StringView text, int index, int count)
		{
			Memory = text.Memory.Slice(index, count);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The memory containing the characters</param>
		public StringView(ReadOnlyMemory<char> memory)
		{
			Memory = memory;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The memory containing the characters</param>
		/// <param name="index">Offset within the string for this view</param>
		public StringView(ReadOnlyMemory<char> memory, int index)
		{
			Memory = memory.Slice(index);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The memory containing the characters</param>
		/// <param name="index">Offset within the string for this view</param>
		/// <param name="count">Number of characters to include</param>
		public StringView(ReadOnlyMemory<char> memory, int index, int count)
		{
			Memory = memory.Slice(index, count);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">String to construct from</param>
		public StringView(string text)
		{
			Memory = text.AsMemory();
		}

		/// <summary>
		/// Constructs a view onto a substring of the given string
		/// </summary>
		/// <param name="text">String to construct from</param>
		/// <param name="index">Offset within the string for this view</param>
		public StringView(string text, int index)
		{
			Memory = text.AsMemory(index);
		}

		/// <summary>
		/// Constructs a view onto a substring of the given string
		/// </summary>
		/// <param name="text">String to construct from</param>
		/// <param name="index">Offset within the string for this view</param>
		/// <param name="count">Number of characters to include</param>
		public StringView(string text, int index, int count)
		{
			Memory = text.AsMemory(index, count);
		}

		/// <summary>
		/// Equality comparer
		/// </summary>
		/// <param name="x">First string to compare</param>
		/// <param name="y">Second string to compare</param>
		/// <returns>True if the strings are equal</returns>
		public static bool operator ==(StringView x, StringView y)
		{
			return x.Equals(y);
		}

		/// <summary>
		/// Inequality comparer
		/// </summary>
		/// <param name="x">First string to compare</param>
		/// <param name="y">Second string to compare</param>
		/// <returns>True if the strings are equal</returns>
		public static bool operator !=(StringView x, StringView y)
		{
			return !x.Equals(y);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			return obj is StringView view && Equals(view);
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return String.GetHashCode(Memory.Span, StringComparison.Ordinal);
		}

		/// <inheritdoc/>
		public int GetHashCode(StringComparison comparisonType)
		{
			return String.GetHashCode(Memory.Span, comparisonType);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return new string(Memory.Span);
		}

		/// <inheritdoc/>
		public bool Equals(StringView other)
		{
			return Equals(other, StringComparison.CurrentCulture);
		}

		/// <inheritdoc/>
		public bool Equals(StringView? other, StringComparison comparisonType)
		{
			return other.HasValue && Memory.Span.Equals(other.Value.Memory.Span, comparisonType);
		}

		/// <summary>
		/// Implicit conversion operator from a regular string
		/// </summary>
		/// <param name="text">The string to construct from</param>
		public static implicit operator StringView(string text) => new(text);
	}

	/// <summary>
	/// Comparer for StringView objects
	/// </summary>
	public class StringViewComparer : IComparer<StringView>, IEqualityComparer<StringView>
	{
		/// <summary>
		/// Static instance of an ordinal StringView comparer
		/// </summary>
		public static StringViewComparer Ordinal { get; } = new StringViewComparer(StringComparison.Ordinal);

		/// <summary>
		/// Static instance of an ordinal StringView comparer which ignores case
		/// </summary>
		public static StringViewComparer OrdinalIgnoreCase { get; } = new StringViewComparer(StringComparison.OrdinalIgnoreCase);

		/// <summary>
		/// The comparison type
		/// </summary>
		public StringComparison ComparisonType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="comparisonType">Type of comparison to perform</param>
		public StringViewComparer(StringComparison comparisonType)
		{
			ComparisonType = comparisonType;
		}

		/// <inheritdoc/>
		public bool Equals(StringView x, StringView y)
		{
			return x.Span.Equals(y.Span, ComparisonType);
		}

		/// <inheritdoc/>
		public int GetHashCode(StringView obj)
		{
			return String.GetHashCode(obj.Span, ComparisonType);
		}

		/// <inheritdoc/>
		public int Compare(StringView x, StringView y)
		{
			return x.Span.CompareTo(y.Span, ComparisonType);
		}
	}

	/// <summary>
	/// Comparer for StringView objects.  However, it implements UE style ignore case compare
	/// </summary>
	public class StringViewComparerUe : IComparer<StringView>, IEqualityComparer<StringView>
	{
		/// <summary>
		/// Static instance of an ordinal StringView comparer
		/// </summary>
		public static StringViewComparerUe Ordinal { get; } = new StringViewComparerUe(StringComparison.Ordinal);

		/// <summary>
		/// Static instance of an ordinal StringView comparer which ignores case
		/// </summary>
		public static StringViewComparerUe OrdinalIgnoreCase { get; } = new StringViewComparerUe(StringComparison.OrdinalIgnoreCase);

		/// <summary>
		/// The comparison type
		/// </summary>
		public StringComparison ComparisonType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="comparisonType">Type of comparison to perform</param>
		public StringViewComparerUe(StringComparison comparisonType)
		{
			ComparisonType = comparisonType;
		}

		/// <inheritdoc/>
		public bool Equals(StringView x, StringView y)
		{
			return x.Span.Equals(y.Span, ComparisonType);
		}

		/// <inheritdoc/>
		public int GetHashCode(StringView obj)
		{
			return String.GetHashCode(obj.Span, ComparisonType);
		}

		/// <inheritdoc/>
		public int Compare(StringView x, StringView y)
		{
			if (ComparisonType == StringComparison.OrdinalIgnoreCase)
			{
				return StringUtils.CompareIgnoreCaseUe(x.Span, y.Span);
			}
			else
			{
				return x.Span.CompareTo(y.Span, ComparisonType);
			}
		}
	}
}
