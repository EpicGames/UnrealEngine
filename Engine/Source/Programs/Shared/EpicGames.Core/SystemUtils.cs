// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text;

namespace EpicGames.Core;

/// <summary>
/// Utilities for querying system hardware information such as processor counts, memory availability, and CPU utilization.
/// </summary>
public static class SystemUtils
{
	#region Processor Count

	[SupportedOSPlatform("windows")]
	enum LogicalProcessorRelationship
	{
		RelationProcessorCore,
		RelationNumaNode,
		RelationCache,
		RelationProcessorPackage,
		RelationGroup,
		RelationAll = 0xffff
	}

	[SupportedOSPlatform("windows")]
	[DllImport("kernel32.dll", SetLastError = true)]
	static extern bool GetLogicalProcessorInformationEx(LogicalProcessorRelationship relationshipType, IntPtr buffer, ref uint returnedLength);

	// int sysctlbyname(const char *name, void *oldp, size_t *oldlenp, void *newp, size_t newlen); // from man page
	[SupportedOSPlatform("macos")]
	[DllImport("libc")]
	static extern int sysctlbyname(string name, out int oldp, ref ulong oldlenp, IntPtr newp, ulong newlen);

	[SupportedOSPlatform("macos")]
	[DllImport("libc")]
	static extern int sysctlbyname(string name, out ulong oldp, ref ulong oldlenp, IntPtr newp, ulong newlen);

	/// <summary>
	/// Gets the number of logical cores. We use this rather than Environment.ProcessorCount when possible to handle machines with > 64 cores (the single group limit available to the .NET framework).
	/// </summary>
	/// <returns>The number of logical cores.</returns>
	public static int GetLogicalProcessorCount()
	{
		// This function uses Windows P/Invoke calls; if we're not running on Windows, just return the default.
		if (OperatingSystem.IsWindows())
		{
			const int ERROR_INSUFFICIENT_BUFFER = 122;

			// Determine the required buffer size to store the processor information
			uint returnLength = 0;
			if (!GetLogicalProcessorInformationEx(LogicalProcessorRelationship.RelationGroup, IntPtr.Zero, ref returnLength) && Marshal.GetLastWin32Error() == ERROR_INSUFFICIENT_BUFFER)
			{
				// Allocate a buffer for it
				IntPtr ptr = Marshal.AllocHGlobal((int)returnLength);
				try
				{
					if (GetLogicalProcessorInformationEx(LogicalProcessorRelationship.RelationGroup, ptr, ref returnLength))
					{
						int count = 0;
						for (int pos = 0; pos < returnLength;)
						{
							LogicalProcessorRelationship type = (LogicalProcessorRelationship)Marshal.ReadInt16(ptr, pos);
							if (type == LogicalProcessorRelationship.RelationGroup)
							{
								// Read the values from the embedded GROUP_RELATIONSHIP structure
								int groupRelationshipPos = pos + 8;
								int activeGroupCount = Marshal.ReadInt16(ptr, groupRelationshipPos + 2);

								// Read the processor counts from the embedded PROCESSOR_GROUP_INFO structures
								int groupInfoPos = groupRelationshipPos + 24;
								for (int groupIdx = 0; groupIdx < activeGroupCount; groupIdx++)
								{
									count += Marshal.ReadByte(ptr, groupInfoPos + 1);
									groupInfoPos += 40 + IntPtr.Size;
								}
							}
							pos += Marshal.ReadInt32(ptr, pos + 4);
						}
						return count;
					}
				}
				finally
				{
					Marshal.FreeHGlobal(ptr);
				}
			}
		}
		else if (OperatingSystem.IsLinux())
		{
			// query socket/logical core pairings.  There should not be duplicates in this list since each hyperthread
			// will show up as it's own logical "cpu".  Including the socket number allows us to count multi-processor
			// system cores correctly
			string output = RunProcessAndReturnStdOut("lscpu", "-p='SOCKET,CPU'");
			List<string> cpus = [.. output.Split("\n", StringSplitOptions.None).Where(x => !x.StartsWith('#'))];

			return cpus.Count;
		}
		return Environment.ProcessorCount;
	}

	/// <summary>
	/// Gets the number of physical cores, excluding hyper threading.
	/// </summary>
	/// <returns>The number of physical cores, or -1 if it could not be obtained</returns>
	public static int GetPhysicalProcessorCount()
	{
		if (OperatingSystem.IsWindows())
		{
			const int ERROR_INSUFFICIENT_BUFFER = 122;

			// Determine the required buffer size to store the processor information
			uint returnLength = 0;
			if (!GetLogicalProcessorInformationEx(LogicalProcessorRelationship.RelationProcessorCore, IntPtr.Zero,
				ref returnLength) && Marshal.GetLastWin32Error() == ERROR_INSUFFICIENT_BUFFER)
			{
				// Allocate a buffer for it
				IntPtr ptr = Marshal.AllocHGlobal((int)returnLength);
				try
				{
					if (GetLogicalProcessorInformationEx(LogicalProcessorRelationship.RelationProcessorCore, ptr,
						ref returnLength))
					{
						// As per-MSDN, this will return one structure per physical processor. Each SYSTEM_LOGICAL_PROCESSOR_INFORMATION_EX structure is of a variable size, so just skip
						// through the list and count the number of entries.
						int count = 0;
						for (int pos = 0; pos < returnLength;)
						{
							LogicalProcessorRelationship type =
								(LogicalProcessorRelationship)Marshal.ReadInt16(ptr, pos);
							if (type == LogicalProcessorRelationship.RelationProcessorCore)
							{
								count++;
							}

							pos += Marshal.ReadInt32(ptr, pos + 4);
						}

						return count;
					}
				}
				finally
				{
					Marshal.FreeHGlobal(ptr);
				}
			}
		}
		else if (OperatingSystem.IsMacOS())
		{
			ulong size = 4;
			if (0 == sysctlbyname("hw.physicalcpu", out int value, ref size, IntPtr.Zero, 0))
			{
				return value;
			}
		}
		else if (OperatingSystem.IsLinux())
		{
			// query socket/physical core pairings.  There will be duplicates in this if there are hyperthreads
			// using the HashSet ensures that those duplicates are removed and we only count the first "cpu" found
			// for each "core".  Including the socket number allows us to count multi-processor system cores correctly
			string output = RunProcessAndReturnStdOut("lscpu", "-p='SOCKET,CORE'");
			HashSet<string> cpus = [.. output.Split("\n", StringSplitOptions.None).Where(x => !x.StartsWith('#'))];

			return cpus.Count;
		}

		return -1;
	}

	/// <summary>
	/// Gets if the processor has asymmetrical cores (Windows only)
	/// </summary>
	/// <returns></returns>
	public static bool IsAsymmetricalProcessor()
	{
		int logicalCores = GetLogicalProcessorCount();
		int physicalCores = GetPhysicalProcessorCount();

		if (physicalCores <= 0 || physicalCores == logicalCores)
		{
			return false;
		}

		return logicalCores != physicalCores * 2;
	}

	#endregion

	#region Memory

	/// <summary>
	/// Gets the total memory bytes available, based on what is known to the garbage collector.
	/// </summary>
	/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
	/// <returns>The total memory available, in bytes.</returns>
	public static long GetAvailableMemoryBytes()
	{
		GCMemoryInfo memoryInfo = GC.GetGCMemoryInfo();
		// TotalAvailableMemoryBytes will be 0 if garbage collection has not run yet
		return memoryInfo.TotalAvailableMemoryBytes != 0 ? memoryInfo.TotalAvailableMemoryBytes : -1;
	}

	// MEMORYSTATUSEX, from <sysinfoapi.h>
	[SupportedOSPlatform("windows")]
	[StructLayout(LayoutKind.Sequential)]
	struct MemoryStatusEx
	{
		public uint _dwLength;
		public uint _dwMemoryLoad;
		public ulong _ullTotalPhys;
		public ulong _ullAvailPhys;
		public ulong _ullTotalPageFile;
		public ulong _ullAvailPageFile;
		public ulong _ullTotalVirtual;
		public ulong _ullAvailVirtual;
		public ulong _ullAvailExtendedVirtual;
	}

	[SupportedOSPlatform("windows")]
	[DllImport("kernel32.dll", SetLastError = true)]
	static extern bool GlobalMemoryStatusEx(ref MemoryStatusEx lpBuffer);

	// vm_statistics64, based on the definition in <mach/vm_statistics.h>
	[StructLayout(LayoutKind.Sequential)]
	struct VmStatistics64
	{
		/*natural_t*/
		public int _freeCount;              /* # of pages free */
		/*natural_t*/
		public int _activeCount;            /* # of pages active */
		/*natural_t*/
		public int _inactiveCount;          /* # of pages inactive */
		/*natural_t*/
		public int _wireCount;              /* # of pages wired down */
		/*uint64_t */
		public ulong _zeroFillCount;        /* # of zero fill pages */
		/*uint64_t */
		public ulong _reactivations;        /* # of pages reactivated */
		/*uint64_t */
		public ulong _pageins;              /* # of pageins */
		/*uint64_t */
		public ulong _pageouts;             /* # of pageouts */
		/*uint64_t */
		public ulong _faults;               /* # of faults */
		/*uint64_t */
		public ulong _cowFaults;            /* # of copy-on-writes */
		/*uint64_t */
		public ulong _lookups;              /* object cache lookups */
		/*uint64_t */
		public ulong _hits;                 /* object cache hits */
		/*uint64_t */
		public ulong _purges;               /* # of pages purged */
		/*natural_t*/
		public int _purgeableCount;         /* # of pages purgeable */
		/*
          	 * NB: speculative pages are already accounted for in "_freeCount",
          	 * so "_speculativeCount" is the number of "free" pages that are
          	 * used to hold data that was read speculatively from disk but
          	 * haven't actually been used by anyone so far.
          	 */
		/*natural_t*/
		public int _speculativeCount;       /* # of pages speculative */

		/* added for rev1 */
		/*uint64_t */
		public ulong _decompressions;       /* # of pages decompressed */
		/*uint64_t */
		public ulong _compressions;         /* # of pages compressed */
		/*uint64_t */
		public ulong _swapins;              /* # of pages swapped in (via compression segments) */
		/*uint64_t */
		public ulong _swapouts;             /* # of pages swapped out (via compression segments) */
		/*natural_t*/
		public int _compressorPageCount;    /* # of pages used by the compressed pager to hold all the compressed data */
		/*natural_t*/
		public int _throttledCount;         /* # of pages throttled */
		/*natural_t*/
		public int _externalPageCount;      /* # of pages that are file-backed (non-swap) */
		/*natural_t*/
		public int _internalPageCount;      /* # of pages that are anonymous */
		/*uint64_t */
		public ulong _totalUncompressedPagesInCompressor; /* # of pages (uncompressed) held within the compressor. */
	} // __attribute__((aligned(8)));

	// kern_return_t host_statistics64(host_t host_priv, host_flavor_t flavor, host_info64_t host_info64_out, mach_msg_type_number_t *host_info64_outCnt); // from <mach/mach_host.h>
	[SupportedOSPlatform("macos")]
	[DllImport("libc")]
	static extern int host_statistics64(IntPtr hostPriv, int flavor, out VmStatistics64 hostInfo64Out, ref uint hostInfoCount);

	// mach_port_t mach_host_self() // from <mach/mach_init.h>
	[SupportedOSPlatform("macos")]
	[DllImport("libc")]
	static extern IntPtr mach_host_self();

	/// <summary>
	/// Gets the total system memory in bytes based on what is known to the garbage collector.
	/// </summary>
	/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
	/// <returns>The total system memory, in bytes.</returns>
	public static long GetTotalSystemMemoryBytes()
	{
		GCMemoryInfo memoryInfo = GC.GetGCMemoryInfo();
		return memoryInfo.TotalAvailableMemoryBytes;
	}

	/// <summary>
	/// Gets the total memory bytes free, based on what is known to the garbage collector.
	/// </summary>
	/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
	/// <returns>The total memory free, in bytes.</returns>
	public static long GetFreeMemoryBytes()
	{
		long freeMemoryBytes = -1;

		GCMemoryInfo memoryInfo = GC.GetGCMemoryInfo();
		// TotalAvailableMemoryBytes will be 0 if garbage collection has not run yet
		if (memoryInfo.TotalAvailableMemoryBytes != 0)
		{
			freeMemoryBytes = memoryInfo.TotalAvailableMemoryBytes - memoryInfo.MemoryLoadBytes;
		}

		// On Mac, MemoryInfo.MemoryLoadBytes includes memory used to cache disk-backed files ("Cached Files" in
		// Activity Monitor), which can result in a significant over-estimate of memory pressure.
		// We treat memory used for caching of disk-backed files as free for use in compilation tasks.
		if (OperatingSystem.IsMacOS())
		{
			// host_statistics64() flavor, from <mach/host_info.h>
			const int HOST_VM_INFO64 = 4;
			// host_statistics64() count of 32bit values in output struct, from <mach/host_info.h>
			int hostVmInfo64Count = Marshal.SizeOf(typeof(VmStatistics64)) / 4;

			VmStatistics64 vmStats;
			uint structSize = (uint)hostVmInfo64Count;
			IntPtr host = mach_host_self();
			if (host_statistics64(host, HOST_VM_INFO64, out vmStats, ref structSize) != 0)
			{
				// Return the dotnet memory estimate if the OS call fails
				return freeMemoryBytes;
			}

			ulong outSize = 4;
			if (0 != sysctlbyname("hw.pagesize", out int pageSize, ref outSize, IntPtr.Zero, 0))
			{
				pageSize = 4096; // likely result
			}

			freeMemoryBytes += (long)pageSize * (long)vmStats._externalPageCount;
		}
		return freeMemoryBytes;
	}

	/// <summary>
	/// Gets the current amount of committed (virtual) memory in bytes.
	/// On Windows, this is the total page file usage (RAM + page file consumed by all processes).
	/// On Linux, this is the Committed_AS value from /proc/meminfo.
	/// On macOS, this is an approximation based on wired + active + inactive + compressor pages.
	/// </summary>
	/// <returns>Currently committed memory in bytes, or -1 if unavailable.</returns>
	public static long GetCurrentCommittedMemoryBytes()
	{
		if (OperatingSystem.IsWindows())
		{
			MemoryStatusEx status = new() { _dwLength = (uint)Marshal.SizeOf<MemoryStatusEx>() };
			if (GlobalMemoryStatusEx(ref status))
			{
				return (long)(status._ullTotalPageFile - status._ullAvailPageFile);
			}
		}
		else if (OperatingSystem.IsLinux())
		{
			long kb = ReadProcMemInfoKilobytes("Committed_AS");
			if (kb >= 0)
			{
				return kb * 1024L;
			}
		}
		else if (OperatingSystem.IsMacOS())
		{
			const int HOST_VM_INFO64 = 4;
			int hostVmInfo64Count = Marshal.SizeOf(typeof(VmStatistics64)) / 4;
			uint structSize = (uint)hostVmInfo64Count;
			IntPtr host = mach_host_self();
			if (host_statistics64(host, HOST_VM_INFO64, out VmStatistics64 vmStats, ref structSize) == 0)
			{
				ulong outSize = 4;
				if (0 != sysctlbyname("hw.pagesize", out int pageSize, ref outSize, IntPtr.Zero, 0))
				{
					pageSize = 4096;
				}
				long committed = (long)pageSize * ((long)vmStats._wireCount + vmStats._activeCount + vmStats._inactiveCount + vmStats._compressorPageCount);
				return committed;
			}
		}
		return -1;
	}

	/// <summary>
	/// Gets the maximum amount of memory that can be committed in bytes (commit limit).
	/// On Windows, this is the total page file size (RAM + page file capacity).
	/// On Linux, this is the CommitLimit value from /proc/meminfo.
	/// On macOS, this is the total physical RAM (hw.memsize).
	/// </summary>
	/// <returns>Commit limit in bytes, or -1 if unavailable.</returns>
	public static long GetMaxCommittedMemoryBytes()
	{
		if (OperatingSystem.IsWindows())
		{
			MemoryStatusEx status = new() { _dwLength = (uint)Marshal.SizeOf<MemoryStatusEx>() };
			if (GlobalMemoryStatusEx(ref status))
			{
				return (long)status._ullTotalPageFile;
			}
		}
		else if (OperatingSystem.IsLinux())
		{
			long kb = ReadProcMemInfoKilobytes("CommitLimit");
			if (kb >= 0)
			{
				return kb * 1024L;
			}
		}
		else if (OperatingSystem.IsMacOS())
		{
			ulong outSize = sizeof(ulong);
			if (0 == sysctlbyname("hw.memsize", out ulong memSize, ref outSize, IntPtr.Zero, 0))
			{
				return (long)memSize;
			}
		}
		return -1;
	}

	// Parses a "<key>: <value> kB" line from /proc/meminfo and returns the value in kB, or -1.
	[SupportedOSPlatform("linux")]
	static long ReadProcMemInfoKilobytes(string key)
	{
		string prefix = key + ":";
		foreach (string line in File.ReadLines("/proc/meminfo"))
		{
			if (line.StartsWith(prefix, StringComparison.Ordinal))
			{
				ReadOnlySpan<char> span = line.AsSpan(prefix.Length).Trim();
				// Strip trailing " kB"
				if (span.EndsWith(" kB", StringComparison.OrdinalIgnoreCase))
				{
					span = span[..^3];
				}
				if (long.TryParse(span, out long value))
				{
					return value;
				}
			}
		}
		return -1;
	}

	#endregion

	#region CPU Utilization

	[SupportedOSPlatform("windows")]
	[DllImport("pdh.dll", SetLastError = true, CharSet = CharSet.Auto)]
	static extern int PdhOpenQueryW([MarshalAs(UnmanagedType.LPWStr)] string? szDataSource, UIntPtr dwUserData, out IntPtr phQuery);

	[SupportedOSPlatform("windows")]
	[DllImport("pdh.dll", SetLastError = true, CharSet = CharSet.Auto)]
	static extern uint PdhAddCounter(IntPtr hQuery, string szFullCounterPath, IntPtr dwUserData, out IntPtr phCounter);

	[SupportedOSPlatform("windows")]
	[DllImport("pdh.dll", SetLastError = true)]
	static extern uint PdhCollectQueryData(IntPtr phQuery);

	[SupportedOSPlatform("windows")]
	struct PdhFmtCounterValue
	{
		public uint _cStatus;
		public double _doubleValue;
	};

	[SupportedOSPlatform("windows")]
	[DllImport("pdh.dll", SetLastError = true)]
	static extern uint PdhGetFormattedCounterValue(IntPtr phCounter, uint dwFormat, IntPtr lpdwType, out PdhFmtCounterValue pValue);

	static IntPtr s_cpuQuery;
	static IntPtr s_cpuTotal;
	static bool s_cpuInitialized = false;

	/// <summary>
	/// Gives the current CPU utilization.
	/// </summary>
	/// <param name="utilization">Percentage of CPU utilization currently.</param>
	/// <returns>Whether or not it was successful in getting the CPU utilization.</returns>
	public static bool GetTotalCpuUtilization(out float utilization)
	{
		utilization = 0.0f;
		if (OperatingSystem.IsWindows())
		{
			const uint ERROR_SUCCESS = 0;
			if (!s_cpuInitialized)
			{
				if (PdhOpenQueryW(null, UIntPtr.Zero, out s_cpuQuery) == ERROR_SUCCESS)
				{
					if (PdhAddCounter(s_cpuQuery, "\\Processor(_Total)\\% Processor Time", IntPtr.Zero, out s_cpuTotal) != ERROR_SUCCESS)
					{
						return false;
					}

					if (PdhCollectQueryData(s_cpuQuery) != ERROR_SUCCESS)
					{
						return false;
					}

					s_cpuInitialized = true;
				}
			}

			if (s_cpuInitialized)
			{
				const uint PDH_FMT_DOUBLE = 0x00000200;

				if (PdhCollectQueryData(s_cpuQuery) != ERROR_SUCCESS)
				{
					return false;
				}

				if (PdhGetFormattedCounterValue(s_cpuTotal, PDH_FMT_DOUBLE, IntPtr.Zero, out PdhFmtCounterValue counterVal) != ERROR_SUCCESS)
				{
					return false;
				}

				utilization = (float)counterVal._doubleValue;
				return true;
			}
		}

		return false;
	}

	#endregion

	#region Helpers

	static string RunProcessAndReturnStdOut(string command, string args)
	{
		// Process Arguments follow windows conventions in .NET Core
		// Which means single quotes ' are not considered quotes.
		// see https://github.com/dotnet/runtime/issues/29857
		// also see UE-102580
		// for rules see https://docs.microsoft.com/en-us/cpp/cpp/main-function-command-line-args
		args = args.Replace('\'', '\"') ?? String.Empty;

		ProcessStartInfo startInfo = new(command, args)
		{
			UseShellExecute = false,
			RedirectStandardOutput = true,
			RedirectStandardError = true,
			CreateNoWindow = true,
			StandardOutputEncoding = Encoding.UTF8,
		};
		using Process localProcess = Process.Start(startInfo)!;
		string output = localProcess.StandardOutput.ReadToEnd().Trim();
		string error = localProcess.StandardError.ReadToEnd().Trim();
		localProcess.WaitForExit();

		if (error.Length > 0)
		{
			if (output.Length > 0)
			{
				output += Environment.NewLine;
			}
			output += error;
		}
		return output;
	}

	#endregion
}
