// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using System.Timers;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core.Telemetry
{
	/// <summary>
	/// Abstract base class for a data router event
	/// </summary>
	public abstract class DataRouterEvent
	{
		/// <summary>
		/// The app's id
		/// </summary>
		protected string AppId { get; set; } = String.Empty;

		/// <summary>
		/// The app's version
		/// </summary>
		protected string AppVersion { get; set; } = String.Empty;

		/// <summary>
		/// The app's environment
		/// </summary>
		protected string AppEnvironment { get; set; } = String.Empty;

		/// <summary>
		/// The upload type
		/// </summary>
		protected string UploadType { get; set; } = String.Empty;

		/// <summary>
		///  The user's id
		/// </summary>
		protected string UserId { get; set; } = String.Empty;

		/// <summary>
		/// The session's id
		/// </summary>
		protected string SessionId { get; set; } = String.Empty;

		/// <summary>
		/// The TimeStamp of when the event was added to queue UNTIL we serialize so we use the name TimeStamp in code, and serialize as DateOffset
		/// </summary>
		[JsonPropertyName("DateOffset")]
		[JsonConverter(typeof(DateOffsetDataRouterEventConverter))]
		public DateTime TimeStamp
		{
			get;
			protected set;
		}

		/// <summary>
		/// The name of the event
		/// </summary>
		public abstract string EventName
		{
			get;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inTimeStamp"></param>
		protected DataRouterEvent(DateTime? inTimeStamp = null)
		{
			TimeStamp = inTimeStamp ?? DateTime.UtcNow;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inAppId"></param>
		/// <param name="inAppVersion"></param>
		/// <param name="inAppEnvironment"></param>
		/// <param name="inUploadType"></param>
		/// <param name="inUserId"></param>
		/// <param name="inSessionId"></param>
		/// <param name="inTimeStamp"></param>
		protected DataRouterEvent(string inAppId, string inAppVersion, string inAppEnvironment, string inUploadType, string inUserId, string inSessionId, DateTime? inTimeStamp = null)
		{
			AppId = inAppId;
			AppVersion = inAppVersion;
			AppEnvironment = inAppEnvironment;
			UploadType = inUploadType;
			UserId = inUserId;
			SessionId = inSessionId;
			TimeStamp = inTimeStamp ?? DateTime.UtcNow;
		}

		/// <summary>
		/// Generates url parameter strings for this event
		/// </summary>
		/// <returns>The generated url parameters string</returns>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1055:URI-like return values should not be strings", Justification = "Legacy interoperability")]
		public virtual string GetUrlParameters()
		{
			string urlParameters = $"AppID={AppId}&AppVersion={AppVersion}&AppEnvironment={AppEnvironment}&UploadType={UploadType}";
			urlParameters += !String.IsNullOrEmpty(UserId) ? $"&UserID={UserId}" : String.Empty;
			urlParameters += !String.IsNullOrEmpty(SessionId) ? $"&SessionID={SessionId}" : String.Empty;

			return urlParameters;
		}

		/// <summary>
		/// Direct access to override the App ID.
		/// This is used to redirect telemetry data in Databricks.
		/// </summary>
		/// <param name="appId"></param>
		public void OverrideAppId(string appId)
		{
			AppId = appId;
		}
	}

	/// <summary>
	/// Object to post data router events
	/// </summary>
	public class DataRouterPost
	{
		/// <summary>
		/// List of objects to force polymorphic JSON serialization
		/// </summary>
		public List<object> Events
		{
			get;
		} = [];
	}

	/// <summary>
	/// Object to serialize data router post events
	/// </summary>
	public class SerializedDataRouterPost
	{
		/// <summary>
		/// Used in conjunction with a JsonSerializer class to mimic serialization of DataRouterPost
		/// </summary>
		public List<string> EventsSerialized
		{
			get;
		} = [];

	}

	/// <summary>
	/// Implementation of JsonConverter for SerializedDataRouterPost. Matches output of serializing a DataRouterPost.
	/// <see>https://confluence.it.epicgames.com/display/DPE/Data+Router</see>
	/// </summary>
	public class SerializedDataRouterPostConverter : JsonConverter<SerializedDataRouterPost>
	{
		/// <summary>
		/// Reader
		/// </summary>
		/// <param name="reader"></param>
		/// <param name="typeToConvert"></param>
		/// <param name="options"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public override SerializedDataRouterPost? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		/// <param name="options"></param>
		public override void Write(Utf8JsonWriter writer, SerializedDataRouterPost value, JsonSerializerOptions options)
		{
			writer.WriteStartObject();
			writer.WriteStartArray("Events");
			foreach (string serializedEvent in value.EventsSerialized)
			{
				writer.WriteRawValue(serializedEvent);
			}
			writer.WriteEndArray();
			writer.WriteEndObject();
		}
	}

	/// <summary>
	/// DateTime converter for data router events
	/// </summary>
	public class DateOffsetDataRouterEventConverter : JsonConverter<DateTime>
	{
		/// <summary>
		/// Reader
		/// </summary>
		/// <param name="reader"></param>
		/// <param name="typeToConvert"></param>
		/// <param name="options"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public override DateTime Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		/// <param name="options"></param>
		public override void Write(Utf8JsonWriter writer, DateTime value, JsonSerializerOptions options)
		{
			TimeSpan offset = DateTime.UtcNow - value;

			if (offset.Ticks <= 0)
			{
				offset = TimeSpan.Zero;
			}
			else if (offset.TotalDays > 1.0)
			{
				offset = new TimeSpan(23, 59, 59);
			}

			string offsetString = offset.ToString(@"hh\:mm\:ss\.fff");

			writer.WriteStringValue($"+{offsetString}");
		}
	}

	/// <summary>
	/// DateTime converter for data router events to convert to Unix timestamp
	/// </summary>
	public class UnixTimestampDataRouterEventConverter : JsonConverter<DateTime>
	{
		/// <summary>
		/// Reader
		/// </summary>
		/// <param name="reader"></param>
		/// <param name="typeToConvert"></param>
		/// <param name="options"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public override DateTime Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		/// <param name="options"></param>
		public override void Write(Utf8JsonWriter writer, DateTime value, JsonSerializerOptions options)
		{
			TimeSpan diff = value.ToUniversalTime() - DateTime.UnixEpoch;
			writer.WriteNumberValue(diff.TotalSeconds);
		}
	}

	/// <summary>
	/// TimeSpan converter for data router events to convert to seconds
	/// </summary>
	public class DurationDataRouterEventConverter : JsonConverter<TimeSpan>
	{
		/// <summary>
		/// Reader
		/// </summary>
		/// <param name="reader"></param>
		/// <param name="typeToConvert"></param>
		/// <param name="options"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public override TimeSpan Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		/// <param name="options"></param>
		public override void Write(Utf8JsonWriter writer, TimeSpan value, JsonSerializerOptions options)
		{
			writer.WriteNumberValue(value.TotalSeconds);
		}
	}

	/// <summary>
	/// Implementation for TelemetryService using DataRouter.
	/// FlushEvents() should be called before application exits otherwise some events may remain unsent
	/// <see>https://confluence.it.epicgames.com/display/DPE/Data+Router</see>
	/// </summary>
	public class DataRouterTelemetryService : ITelemetryService<DataRouterEvent>
	{
		/// <summary>
		/// Denotes if this is a dry run
		/// </summary>
		public bool IsDryRun
		{
			get;
			set;
		} = false;

		/// <summary>
		/// Logging interface
		/// </summary>
		protected ILogger<DataRouterTelemetryService> Logger { get; }

		/// <summary>
		/// The http client to use for connections
		/// </summary>
		protected HttpClient HttpClient { get; }

		/// <summary>
		/// The queue of pending data router events
		/// </summary>
		protected ConcurrentQueue<DataRouterEvent> EventsQueue { get; } = new ConcurrentQueue<DataRouterEvent>();

		/// <summary>
		/// Interval to flush events, in milliseconds
		/// </summary>
		public int AutoFlushIntervalMilliseconds
		{
			get => _autoFlushIntervalMilliseconds;
			set
			{
				_autoFlushIntervalMilliseconds = value;
				if (_autoFlushTimer != null)
				{
					_autoFlushTimer.Stop();

					if (_autoFlushIntervalMilliseconds <= 0)
					{
						Logger.LogWarning("Invalid value for AutoFlushIntervalMilliseconds {AutoFlushIntervalMs} <= 0 Ms", _autoFlushIntervalMilliseconds);
						return;
					}

					_autoFlushTimer.Interval = _autoFlushIntervalMilliseconds;
					_autoFlushTimer.Start();
				}
			}
		}
		private int _autoFlushIntervalMilliseconds = 60 * 1000;

		/// <summary>
		/// Max size of json body for sending telemetry events
		/// </summary>
		private const int MaxEventSize = 2 * 1000 * 1000;

		/// <summary>
		/// Base url address to post events to
		/// </summary>
		public Uri BaseAddress { get; set; }

		private readonly Timer _autoFlushTimer = new();

		/// <summary>
		/// Length of an empty data router post when serialized to json
		/// </summary>
		public static int DataRouterPostEmptyJsonLength
		{
			get
			{
				if (s_dataRouterPostEmptyJsonLength < 0)
				{
					DataRouterPost emptyPost = new();
					s_dataRouterPostEmptyJsonLength = JsonSerializer.Serialize(emptyPost).Length;
				}
				return s_dataRouterPostEmptyJsonLength;
			}
		}
		private static int s_dataRouterPostEmptyJsonLength = -1;

		/// <summary>
		/// Sets up the default request headers for the http client
		/// </summary>
		private void SetHttpClientHeaders()
		{
			if (HttpClient.DefaultRequestHeaders.UserAgent.Count == 0)
			{
				Assembly? entryAssembly = Assembly.GetEntryAssembly();
				if (entryAssembly != null)
				{
					string productName = String.IsNullOrEmpty(entryAssembly.GetName().Name) ? GetType().Name : entryAssembly.GetName().Name!;
					string productVersion = entryAssembly.GetName().Version == null ? "1.0.0.0" : entryAssembly.GetName().Version!.ToString();
					ProductInfoHeaderValue userAgent = new(productName, productVersion);
					HttpClient.DefaultRequestHeaders.UserAgent.Add(userAgent);
				}
				else
				{
					HttpClient.DefaultRequestHeaders.UserAgent.Add(new ProductInfoHeaderValue(GetType().Name, "1.0.0.0"));
				}
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inHttpClient"></param>
		/// <param name="inBaseAddress"></param>
		public DataRouterTelemetryService(HttpClient inHttpClient, Uri inBaseAddress)
		{
			using (ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
			{
				builder
				.ClearProviders()
				.AddConsole();
			}))
			{
				Logger = loggerFactory.CreateLogger<DataRouterTelemetryService>();
			}

			HttpClient = inHttpClient;
			BaseAddress = inBaseAddress;
			SetHttpClientHeaders();
			Task<HttpResponseMessage> initializeConnectionFireAndForget = HeadRequestAsync();

			_autoFlushTimer.Interval = AutoFlushIntervalMilliseconds;
			_autoFlushTimer.Elapsed += AutoFlushEventsAsync;
		}

		/// <summary>
		/// Async function to automatically flush events
		/// </summary>
		/// <param name="sender"></param>
		/// <param name="e"></param>
		private async void AutoFlushEventsAsync(object? sender, ElapsedEventArgs e)
		{
			if (_prevFlushTask != null && !_prevFlushTask.IsCompleted)
			{
				await _prevFlushTask;
			}

			_prevFlushTask = FlushEventsAsync();
		}
		private Task? _prevFlushTask = null;

		/// <inheritdoc/>
		public virtual void RecordEvent(DataRouterEvent eventData)
		{
			EventsQueue.Enqueue(eventData);
		}

		/// <inheritdoc/>
		public void FlushEvents()
		{
			bool isAutoFlushEnabled = _autoFlushTimer.Enabled;
			_autoFlushTimer.Enabled = false;

			Task flushTask = Task.Run(async () => await FlushEventsAsync());
			flushTask.Wait(millisecondsTimeout: _autoFlushIntervalMilliseconds);

			_autoFlushTimer.Enabled = isAutoFlushEnabled;
		}

		/// <summary>
		/// Creates a batch of post events
		/// </summary>
		/// <param name="inPost"></param>
		/// <param name="postURI"></param>
		/// <param name="outPostsList"></param>
		private void CreateBatchedSerializedDataRouterPosts(DataRouterPost inPost, string postURI, out List<SerializedDataRouterPost> outPostsList)
		{
			outPostsList = [];

			SerializedDataRouterPost currentBatch = new();

			int currentBatchSize = DataRouterPostEmptyJsonLength;

			// remember that we will have an extra comma for all but the last post in a batch
			foreach (object @event in inPost.Events)
			{
				string serializedEvent = JsonSerializer.Serialize(@event);

				int sizeOfEvent = serializedEvent.Length;

				// skip events that are too big on their own
				if (sizeOfEvent > MaxEventSize)
				{
					Logger.LogError("Unable to send DataRouter event with uri '{PostURI}' because event size was over the max of {MaxEventSize} bytes.", postURI, MaxEventSize);
					continue;
				}
				// if we will go over, add current batch to list and start a new batch
				else if (currentBatchSize + sizeOfEvent > MaxEventSize)
				{
					outPostsList.Add(currentBatch);

					currentBatchSize = DataRouterPostEmptyJsonLength;

					currentBatch = new SerializedDataRouterPost();
				}
				else
				{
					currentBatch.EventsSerialized.Add(serializedEvent);
					// account for comma
					currentBatchSize += sizeOfEvent + 1;
				}
			}
			if (currentBatch.EventsSerialized.Count > 0)
			{
				outPostsList.Add(currentBatch);
			}
		}

		/// <summary>
		/// Async function to post a single request
		/// </summary>
		/// <param name="postDataKey"></param>
		/// <param name="jsonBody"></param>
		/// <returns>Http responce</returns>
		private async Task<HttpResponseMessage> PostRequestAsync(string postDataKey, string jsonBody)
		{
			using HttpRequestMessage postRequest = new(HttpMethod.Post, $"{BaseAddress}?{postDataKey}");
			postRequest.Content = new StringContent(jsonBody, System.Text.Encoding.UTF8, "application/json");
			return await HttpClient.SendAsync(postRequest);
		}

		/// <summary>
		/// Async function to issue a single async head request
		/// </summary>
		/// <returns>Http responce</returns>
		private async Task<HttpResponseMessage> HeadRequestAsync()
		{
			using HttpRequestMessage headRequest = new(HttpMethod.Head, $"{BaseAddress}");
			return await Task.Run(async () => await HttpClient.SendAsync(headRequest));
		}

		/// <summary>
		/// Async function to flush events
		/// </summary>
		/// <returns></returns>
		public virtual async Task FlushEventsAsync()
		{
			ConcurrentDictionary<string, DataRouterPost> posts = new();
			DataRouterEvent? eventData;
			while (EventsQueue.TryDequeue(out eventData))
			{
				string requestUri = eventData.GetUrlParameters();
				if (!posts.ContainsKey(requestUri))
				{
					if (!posts.TryAdd(requestUri, new DataRouterPost()))
					{
						Logger.LogWarning("Failed to Flush RequestUri: {RequestUri}", requestUri);
					}
				}

				posts[requestUri].Events.Add((DataRouterEvent)eventData);
			}

			if (IsDryRun)
			{
				foreach (KeyValuePair<string, DataRouterPost> postData in posts)
				{
					Logger.LogInformation("curl -H 'Content-Type: application/json' \\\n-H 'User-Agent:{UserAgent}' \\\n-X POST \\\n -d '{Data}' \\\n{Key}", HttpClient.DefaultRequestHeaders.UserAgent, JsonSerializer.Serialize(postData.Value, new JsonSerializerOptions { WriteIndented = true }), postData.Key);
				}

				return;
			}

			JsonSerializerOptions serializeOptions = new()
			{
				WriteIndented = false,
				Converters =
				{
					new SerializedDataRouterPostConverter()
				}
			};

			List<Task<HttpResponseMessage>> postRequests = [];
			foreach (KeyValuePair<string, DataRouterPost> postData in posts)
			{
				// Break our post into multiple if the DataRouterPost serialized would go over our max event size
				CreateBatchedSerializedDataRouterPosts(postData.Value, postData.Key, out List<SerializedDataRouterPost> serializedPosts);

				foreach (SerializedDataRouterPost serializedPost in serializedPosts)
				{
					string jsonBody = JsonSerializer.Serialize(serializedPost, serializeOptions);

					postRequests.Add(PostRequestAsync(postData.Key, jsonBody));
				}
			}

			try
			{
				HttpResponseMessage[] responses = await Task.WhenAll(postRequests);
				foreach (HttpResponseMessage? postResponse in responses)
				{
					Logger.LogDebug("{RequestUri} finished with status {StatusCode}", postResponse.RequestMessage?.RequestUri, postResponse.StatusCode);
				}
			}
			catch (HttpRequestException ex)
			{
				Logger.LogError("{Exception}", ex.Message);
			}
			catch (TaskCanceledException /*ex*/)
			{
			}
			catch (Exception ex)
			{
				Logger.LogError("{Exception}", ex.ToString());
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Standard Dispose pattern method
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			_autoFlushTimer.Enabled = false;

			// Make sure to flush any remaining events if the service is disposed of
			FlushEvents();

			_autoFlushTimer.Dispose();

			if (disposing)
			{
			}
		}
	}
}
