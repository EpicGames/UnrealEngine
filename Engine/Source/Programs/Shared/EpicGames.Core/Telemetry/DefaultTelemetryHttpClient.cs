// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Http;
using Polly;
using Polly.Extensions.Http;
using System;
using System.Net.Http;

namespace EpicGames.Core.Telemetry
{
	/// <summary>
	/// Static class to set http client defaults
	/// </summary>
	public static class HttpClientDefaults
	{
		/// <summary>
		/// When services can be used, utilize this helper to get this policy that handles retries
		/// ex. ServicesCollection.AddHttpClient&lt;TelemetryService&gt;().AddPolicyHandler(HttpClientDefaults.GetPolicy());
		/// </summary>
		/// <returns>The retry policy</returns>
		public static Polly.Retry.AsyncRetryPolicy<HttpResponseMessage> GetRetryPolicy()
		{
			return HttpPolicyExtensions
				// 408, 5XX responses 
				.HandleTransientHttpError()
				// 504 should have been covered by HandleTransientHttpError but adding here to be safe
				.OrResult(msg => msg.StatusCode == System.Net.HttpStatusCode.NotFound ||
								 msg.StatusCode == System.Net.HttpStatusCode.GatewayTimeout)
				.WaitAndRetryAsync(3, retryAttempt => TimeSpan.FromSeconds(Math.Pow(2, retryAttempt)));
		}

		/// <summary>
		/// Creates a HttpClient with the default retry policy
		/// </summary>
		/// <returns>HttpClient</returns>
		public static HttpClient GetClient()
		{
			// per MS documentation, with a lifetime specified we shouldn't have socket exhaustion issues
			HttpMessageHandler socketHandler = HttpMessageHandlers.CreateDefaultHandler();

#pragma warning disable CA2000 // Dispose objects before losing scope
			PolicyHttpMessageHandler pollyHandler = new(GetRetryPolicy())
			{
				InnerHandler = socketHandler,
			};
#pragma warning restore CA2000 // Dispose objects before losing scope

			return new HttpClient(pollyHandler);
		}
	}
}