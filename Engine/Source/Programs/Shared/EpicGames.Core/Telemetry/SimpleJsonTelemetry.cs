// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Text.Json;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core.Telemetry
{
	/// <summary>
	/// Simple telemetry wrapper for accessing JSON data
	/// </summary>
	public class SimpleJsonTelemetry : IDisposable
	{
		private readonly HttpClient _httpClient;
		private bool _disposed = false;

		/// <summary>
		/// Constructor
		/// </summary>
		public SimpleJsonTelemetry(string server)
		{
			_httpClient = new()
			{
				BaseAddress = new Uri(server),
			};
		}

		/// <summary>
		/// Destructor
		/// </summary>
		~SimpleJsonTelemetry()
		{
			Dispose(disposing: false);
		}

		/// <summary>
		/// Fetch the data at a given endpoint and return it as string
		/// </summary>
		public async Task<T?> GetAsync<T>(string endPoint)
			where T : new()
		{
			try
			{
				using HttpResponseMessage response = await _httpClient.GetAsync(new Uri(endPoint, UriKind.Relative));

				response.EnsureSuccessStatusCode();

				System.IO.Stream responseStream = await response.Content.ReadAsStreamAsync();

				return await JsonSerializer.DeserializeAsync<T>(responseStream)!;
			}
			catch (HttpRequestException e)
			{
				Log.Logger.LogWarning("GetAsync: failed to retrieve {EndPoint}: {Message}", endPoint, e.Message);
				return new T();
			}
		}

		/// <summary>
		/// Implement IDisposable.
		/// </summary>
		public void Dispose()
		{
			Dispose(disposing: true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Implement IDisposable.
		/// </summary>
		/// <param name="disposing">If call comes from Dispose, the value is 'true'. If call comes from finalizer, the value is 'false'.</param>
		protected virtual void Dispose(bool disposing)
		{
			if (!_disposed)
			{
				if (disposing)
				{
					_httpClient.Dispose();
				}

				_disposed = true;
			}
		}
	};
}
