// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core.Telemetry
{
	/// <summary>
	/// Base interface for a TelemetryService.
	/// Derived from IDisposable as we want to make sure implementation handles flushing
	/// remaining events in the case that the service is disposed of. This usually would
	/// happen in a using block with an instance of the ITelemetryService derived class.
	/// If registering as part of a servicescollection, Flush should still be called in shutdown code. 
	/// </summary>
	/// <typeparam name="T">Type of event this service should record</typeparam>
	public interface ITelemetryService<T> : IDisposable where T : class
	{
		/// <summary>
		/// Record an event to be processed
		/// </summary>
		/// <param name="eventData">The event to record</param>
		public void RecordEvent(T eventData);

		/// <summary>
		/// Function to flush pending events
		/// </summary>
		public void FlushEvents();
	}
}
