// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Linq;
using System.Threading;

namespace EpicGames.Core
{
	/// <summary>
	/// Allows queuing a large number of tasks to a thread pool and waiting for them all to complete.
	/// </summary>
	public sealed class ThreadPoolWorkQueue : IDisposable
	{
		/// <summary>
		/// Object used for controlling access to updating _lockObject
		/// </summary>
		readonly Lock _lockObject = new();

		/// <summary>
		/// Number of jobs remaining in the queue. This is updated in an atomic way.
		/// </summary>
		int _numOutstandingJobs;

		/// <summary>
		/// Event which indicates whether the queue is empty.
		/// </summary>
#pragma warning disable CA2213 // Disposable fields should be disposed
		ManualResetEvent? _emptyEvent = new(true);
#pragma warning restore CA2213 // Disposable fields should be disposed

		/// <summary>
		/// Exceptions which occurred while executing tasks
		/// </summary>
		readonly ConcurrentBag<Exception> _exceptions = [];

		/// <summary>
		/// Default constructor
		/// </summary>
		public ThreadPoolWorkQueue()
		{
		}

		/// <summary>
		/// Waits for the queue to drain, and disposes of it
		/// </summary>
		public void Dispose()
		{
			if (_emptyEvent != null) // Check we haven't disposed already
			{
				// Ensure the event state is in sync with the counter before we wait on it. Its state can lag behind the _numOutstandingJobs
				// field because we only acquire the lock and update it after modifying _numOutstandingJobs.
				SetEventState();
				_emptyEvent.WaitOne();

				// Acquire the lock before disposing in case any background threads are about to execute SetEventState.
				lock (_lockObject)
				{
					_emptyEvent.Dispose();
					_emptyEvent = null;
				}

				RethrowExceptions();
			}
		}

		/// <summary>
		/// Returns the number of items remaining in the queue
		/// </summary>
		public int NumRemaining => _numOutstandingJobs;

		/// <summary>
		/// Adds an item to the queue
		/// </summary>
		/// <param name="actionToExecute">The action to add</param>
		public void Enqueue(Action actionToExecute)
		{
			if (Interlocked.Increment(ref _numOutstandingJobs) == 1)
			{
				SetEventState();
			}

#if SINGLE_THREAD
			Execute(ActionToExecute);
#else
			ThreadPool.QueueUserWorkItem(Execute, actionToExecute);
#endif
		}

		/// <summary>
		/// Internal method to execute an action
		/// </summary>
		/// <param name="actionToExecute">The action to execute</param>
		void Execute(object? actionToExecute)
		{
			try
			{
				((Action)actionToExecute!)();
			}
			catch (Exception ex)
			{
				_exceptions.Add(ex);
			}
			finally
			{
				if (Interlocked.Decrement(ref _numOutstandingJobs) == 0)
				{
					SetEventState();
				}
			}
		}

		/// <summary>
		/// Atomically read the contents of <see cref="_emptyEvent"/>, throwing an exception if it's already null (indicating that the object has been disposed)
		/// </summary>
		ManualResetEvent GetEmptyEvent()
		{
			ManualResetEvent? emptyEvent = _emptyEvent ?? throw new ObjectDisposedException(typeof(ThreadPoolWorkQueue).Name);
			return emptyEvent;
		}

		/// <summary>
		/// Waits for all queued tasks to finish
		/// </summary>
		public void Wait()
		{
			GetEmptyEvent().WaitOne();
			RethrowExceptions();
		}

		/// <summary>
		/// Waits for all queued tasks to finish, or the timeout to elapse
		/// </summary>
		/// <param name="millisecondsTimeout">Maximum time to wait</param>
		/// <returns>True if the queue completed, false if the timeout elapsed</returns>
		public bool Wait(int millisecondsTimeout)
		{
			return Wait(TimeSpan.FromMilliseconds(millisecondsTimeout));
		}

		/// <summary>
		/// Waits for all queued tasks to finish, or the timeout to elapse
		/// </summary>
		/// <param name="timeout">Maximum time to wait</param>
		/// <returns>True if the queue completed, false if the timeout elapsed</returns>
		public bool Wait(TimeSpan timeout)
		{
			bool bResult = GetEmptyEvent().WaitOne(timeout);
			if (bResult)
			{
				RethrowExceptions();
			}
			return bResult;
		}

		/// <summary>
		/// Checks for any exceptions which ocurred in queued tasks, and re-throws them on the current thread
		/// </summary>
		public void RethrowExceptions()
		{
			if (!_exceptions.IsEmpty)
			{
				throw new AggregateException(_exceptions.AsEnumerable());
			}
		}

		/// <summary>
		/// When the number of outstanding jobs transitions between 0 and 1 and visa-versa, we need to set the event 
		/// based on the actual counter.  While the adjustment of the counter happens with interlocked instructions,
		/// the setting of the event must be done under a lock to serialize the set/reset
		/// </summary>
		private void SetEventState()
		{
			lock (_lockObject)
			{
				if (_emptyEvent != null)
				{
					if (Interlocked.CompareExchange(ref _numOutstandingJobs, 0, 0) > 0)
					{
						_emptyEvent.Reset();
					}
					else
					{
						_emptyEvent.Set();
					}
				}
			}
		}
	}
}
