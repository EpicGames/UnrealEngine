// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.Reflection;
using System.Text.Json;
using Microsoft.Extensions.Logging;

namespace EpicGames.Core
{
	/// <summary>
	/// Interface for a trace span
	/// </summary>
	public interface ITraceSpan : IDisposable
	{
		/// <summary>
		/// Adds additional metadata to this scope
		/// </summary>
		/// <param name="name">Name of the key</param>
		/// <param name="value">Value for this metadata</param>
		void AddMetadata(string name, string value);
	}

	/// <summary>
	/// Sink for tracing information
	/// </summary>
	public interface ITraceSink
	{
		/// <summary>
		/// Create a new trace span
		/// </summary>
		/// <param name="operation">Name of the operation</param>
		/// <param name="resource">Resource that the operation is being performed on</param>
		/// <param name="service">Name of the service</param>
		/// <returns>New span instance</returns>
		ITraceSpan Create(string operation, string? resource, string? service);

		/// <summary>
		/// Flush all the current trace data
		/// </summary>
		void Flush();
	}

	/// <summary>
	/// Default implementation of ITraceSink. Writes trace information to a JSON file on application exit if the UE_TELEMETRY_DIR environment variable is set.
	/// </summary>
	public class JsonTraceSink : ITraceSink
	{
		class TraceSpanImpl : ITraceSpan
		{
			public string Name { get; }
			public string? Resource { get; }
			public string? Service { get; }
			public DateTimeOffset StartTime { get; }
			public DateTimeOffset? FinishTime { get; private set; }
			public Dictionary<string, string> Metadata { get; } = [];

			public TraceSpanImpl(string name, string? resource, string? service)
			{
				Name = name;
				Resource = resource;
				Service = service;
				StartTime = DateTimeOffset.Now;
			}

			public void AddMetadata(string name, string value)
			{
				Metadata[name] = value;
			}

			public void Dispose()
			{
				if (FinishTime == null)
				{
					FinishTime = DateTimeOffset.Now;
				}
			}
		}

		/// <summary>
		/// Output directory for telemetry info
		/// </summary>
		readonly DirectoryReference _telemetryDir;

		/// <summary>
		/// The current scope provider
		/// </summary>
		readonly List<TraceSpanImpl> _spans = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="telemetryDir">Directory to store telemetry files</param>
		public JsonTraceSink(DirectoryReference telemetryDir)
		{
			_telemetryDir = telemetryDir;
		}

		/// <summary>
		/// Creates a scope using the current provider
		/// </summary>
		public ITraceSpan Create(string name, string? resource = null, string? service = null)
		{
			TraceSpanImpl span = new(name, resource, service);
			_spans.Add(span);
			return span;
		}

		/// <summary>
		/// Saves all the scope information to a file
		/// </summary>
		public void Flush()
		{
			// We've seen this method fail on Horde when running on Mac due to a NullReferenceException.
			// We've been unable to find out what's causing the NRE, so try and catch the whole function
			// if it NREs because not failing the job is more critical than getting the trace file written.
			// We shouldn't broadly catch other exception types as they have not been observed so far.
			try
			{
				FlushImpl();
			}
			catch (NullReferenceException ex)
			{
				Log.Logger.LogInformation(ex, "Caught NullReferenceException when flushing JsonTraceSink: {Exception}", ex);
			}
		}

		private void FlushImpl()
		{
			FileReference file;
			using (Process process = Process.GetCurrentProcess())
			{
				DirectoryReference.CreateDirectory(_telemetryDir);

				string fileName = String.Format("{0}.{1}.{2}.json", Path.GetFileName(Assembly.GetEntryAssembly()!.Location), process.Id, process.StartTime.Ticks);
				file = FileReference.Combine(_telemetryDir, fileName);
			}

			using FileStream stream = new(file.FullName, FileMode.Create, FileAccess.Write, FileShare.Read);
			using Utf8JsonWriter writer = new(stream, new JsonWriterOptions { Indented = true });
			writer.WriteStartObject();
			writer.WriteStartArray("Spans");
			foreach (TraceSpanImpl span in _spans)
			{
				if (span.FinishTime != null)
				{
					writer.WriteStartObject();
					writer.WriteString("Name", span.Name);
					if (span.Resource != null)
					{
						writer.WriteString("Resource", span.Resource);
					}
					if (span.Service != null)
					{
						writer.WriteString("Service", span.Service);
					}
					writer.WriteString("StartTime", span.StartTime.ToString("o", CultureInfo.InvariantCulture));
					writer.WriteString("FinishTime", span.FinishTime.Value.ToString("o", CultureInfo.InvariantCulture));
					writer.WriteStartObject("Metadata");
					foreach (KeyValuePair<string, string> pair in span.Metadata)
					{
						writer.WriteString(pair.Key, pair.Value);
					}
					writer.WriteEndObject();
					writer.WriteEndObject();
				}
			}
			writer.WriteEndArray();
			writer.WriteEndObject();
		}
	}

	/// <summary>
	/// Methods for creating ITraceScope instances
	/// </summary>
	public static class TraceSpan
	{
		class CombinedTraceSpan : ITraceSpan
		{
			readonly ITraceSpan[] _spans;

			public CombinedTraceSpan(ITraceSpan[] spans)
			{
				_spans = spans;
			}

			public void AddMetadata(string name, string value)
			{
				foreach (ITraceSpan span in _spans)
				{
					span.AddMetadata(name, value);
				}
			}

			public void Dispose()
			{
				foreach (ITraceSpan span in _spans)
				{
					span.Dispose();
				}
			}
		}

		/// <summary>
		/// The sinks to use
		/// </summary>
		static readonly List<ITraceSink> s_sinks = GetDefaultSinks();

		/// <summary>
		/// Build a list of default sinks
		/// </summary>
		/// <returns></returns>
		static List<ITraceSink> GetDefaultSinks()
		{
			List<ITraceSink> sinks = [];

			string? telemetryDir = Environment.GetEnvironmentVariable("UE_TELEMETRY_DIR");
			if (telemetryDir != null)
			{
				sinks.Add(new JsonTraceSink(new DirectoryReference(telemetryDir)));
			}

			return sinks;
		}

		/// <summary>
		/// Adds a new sink
		/// </summary>
		/// <param name="sink">The sink to add</param>
		public static void AddSink(ITraceSink sink)
		{
			s_sinks.Add(sink);
		}

		/// <summary>
		/// Remove a sink from the current list
		/// </summary>
		/// <param name="sink">The sink to remove</param>
		public static void RemoveSink(ITraceSink sink)
		{
			s_sinks.Remove(sink);
		}

		/// <summary>
		/// Creates a scope using the current provider
		/// </summary>
		public static ITraceSpan Create(string operation, string? resource = null, string? service = null)
		{
			if (s_sinks.Count == 0)
			{
				return new CombinedTraceSpan([]);
			}
			else if (s_sinks.Count == 1)
			{
				return s_sinks[0].Create(operation, resource, service);
			}
			else
			{
				return new CombinedTraceSpan([.. s_sinks.ConvertAll(x => x.Create(operation, resource, service))]);
			}
		}

		/// <summary>
		/// Saves all the scope information to a file
		/// </summary>
		public static void Flush()
		{
			foreach (ITraceSink sink in s_sinks)
			{
				sink.Flush();
			}
		}
	}
}
