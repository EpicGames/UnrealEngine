// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json.Serialization;

#pragma warning disable CS1591 // Missing documentation
#pragma warning disable CA2227 // Collection properties should be read only
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
#pragma warning disable CA1034 // Nested types should not be visible
#pragma warning disable CA1716 // Identifiers should not match keywords
#pragma warning disable IDE0049 // Use framework type

namespace EpicGames.Core
{
	/// <summary>
	/// Defines the version of the code generation to be used.
	/// 
	/// This MUST be kept in sync with EGeneratedBodyVersion defined in 
	/// Engine\Source\Programs\UnrealHeaderTool\Private\GeneratedCodeVersion.h.
	/// </summary>
	public enum EGeneratedCodeVersion
	{
		/// <summary>
		/// Version not set or the default is to be used.
		/// </summary>
		None,

		/// <summary>
		/// 
		/// </summary>
		V1,

		/// <summary>
		/// 
		/// </summary>
		V2,

		/// <summary>
		/// 
		/// </summary>
		VLatest = V2
	};

	/// <summary>
	/// Build module override type to add additional PKG flags if necessary, mirrored in ModuleRules.cs, enum PackageOverrideType
	/// </summary>
	public enum EPackageOverrideType
	{
		None,
		EditorOnly,
		EngineDeveloper,
		GameDeveloper,
		EngineUncookedOnly,
		GameUncookedOnly,
	}

	/// <summary>
	/// Type of module. This should be sorted by the order in which we expect modules to be built.
	/// </summary>
	public enum UHTModuleType
	{
		Program,
		EngineRuntime,
		EngineUncooked,
		EngineDeveloper,
		EngineEditor,
		EngineThirdParty,
		GameRuntime,
		GameUncooked,
		GameDeveloper,
		GameEditor,
		GameThirdParty,
	}

	/// <summary>
	/// Describes the origin and visibility of Verse code
	/// </summary>
	public enum UHTVerseScope
	{
		/// <summary>
		/// Created by Epic and only public definitions will be visible to public users
		/// </summary>
		PublicAPI,

		/// <summary>
		/// Created by Epic and is entirely hidden from public users
		/// </summary>
		InternalAPI,

		/// <summary>
		/// Created by a public user
		/// </summary>
		PublicUser,

		/// <summary>
		/// Created by an Epic internal user
		/// </summary>
		InternalUser
	}

	/// <summary>
	/// Types of code to generate to initialize compiled-in UObject instances. 
	/// Flags to allow multiple formats to be generated at once for development/debugging.
	/// </summary>
	[Flags]
	public enum UhtCompiledInObjectFormat
	{
		/// <summary>
		/// No flags, should not be configured as a desired output type.
		/// </summary>
		None = 0x0,

		/// <summary>
		/// Generate params structs which will be used to initialize UObjects on the heap at runtime.
		/// </summary>
		Params = 0x1,

		/// <summary>
		/// Generate declarations of constinit variables of UObject-derived types allowing these
		/// objects to be stored in the binary's data segment.
		/// </summary>
		ConstInit = 0x2,

		/// <summary>
		/// Default setting to use
		/// </summary>
		Default = Params,

		/// <summary>
		/// Generate all possible types at once for comparison
		/// </summary>
		All = Params | ConstInit,
	}

	/// <summary>
	/// Per-target settings for UHT to control features that need to be the same across all modules. 
	/// </summary>
	public record struct UHTTargetSettings
	{
		public UHTTargetSettings()
		{
		}

		[SetsRequiredMembers]
		public UHTTargetSettings(BinaryArchiveReader reader) : this()
		{
			CompiledInObjectFormat = Enum.Parse<UhtCompiledInObjectFormat>(reader.ReadString()!);
			GenerateEnumMaxValues = reader.ReadBool();
			ValidateModuleDependencies = reader.ReadBool();
		}

		public readonly void Write(BinaryArchiveWriter writer)
		{
			writer.WriteString(CompiledInObjectFormat.ToString());
			writer.WriteBool(GenerateEnumMaxValues);
			writer.WriteBool(ValidateModuleDependencies);
		}

		public readonly void Write(IMemoryWriter writer)
		{
			writer.WriteString(CompiledInObjectFormat.ToString());
			writer.WriteBoolean(GenerateEnumMaxValues);
			writer.WriteBoolean(ValidateModuleDependencies);
		}

		/// <summary>
		/// What type of code to generate for compiled-on UObject instances such as UClass instances.
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtCompiledInObjectFormat CompiledInObjectFormat { get; set; } = UhtCompiledInObjectFormat.Default;

		/// <summary>
		/// Whether to generate a name for the maximum value of UENUMs where one was not found in the authored code.
		/// If this is false, the UEnum code will generate it at runtime if it can do so without creating a collision. 
		/// </summary>
		public bool GenerateEnumMaxValues { get; set; } = false;

		/// <summary>
		/// Placeholder value to control whether or not UHT validates module dependencies from UBT to find missing 
		/// dependencies which are needed because of generated code (e.g. FStructBaseChain references)
		/// </summary>
		public bool ValidateModuleDependencies { get; set; } = false;

		/// <summary>
		/// If true, enable the input cache read
		/// </summary>
		public bool EnableInputCacheRead { get; set; } = false;

		/// <summary>
		/// If true, enable the input cache write
		/// </summary>
		public bool EnableInputCacheWrite { get; set; } = false;
	}

	public class UHTManifest
	{
		public class Module
		{
			public string Name { get; set; } = "";
			[JsonConverter(typeof(JsonStringEnumConverter))]
			public UHTModuleType ModuleType { get; set; } = UHTModuleType.Program;
			[JsonConverter(typeof(JsonStringEnumConverter))]
			public EPackageOverrideType OverrideModuleType { get; set; } = EPackageOverrideType.None;
			public string BaseDirectory { get; set; } = String.Empty;
			public List<string> IncludePaths { get; set; } = []; // The include paths which all UHT-generated includes should be relative to
			public string OutputDirectory { get; set; } = String.Empty;
			public List<string> ClassesHeaders { get; set; } = [];
			public List<string> PublicHeaders { get; set; } = [];
			public List<string> InternalHeaders { get; set; } = [];
			public List<string> PrivateHeaders { get; set; } = [];
			public List<string> PublicDefines { get; set; } = [];
			// Other modules in the manifest which this module has a link dependency on in UBT
			// Can be used to validate dependencies which are required because of generated code
			public List<string> ModuleDependencies { get; set; } = [];
			public string? GeneratedCPPFilenameBase { get; set; } = null;
			public bool SaveExportedHeaders { get; set; } = false;
			[JsonConverter(typeof(JsonStringEnumConverter))]
			[JsonPropertyName("UHTGeneratedCodeVersion")]
			public EGeneratedCodeVersion GeneratedCodeVersion { get; set; } = EGeneratedCodeVersion.None;
			public string VersePath { get; set; } = "";
			[JsonConverter(typeof(JsonStringEnumConverter))]
			public UHTVerseScope VerseScope { get; set; } = UHTVerseScope.PublicAPI;
			public bool HasVerse { get; set; } = false;
			public string VerseMountPoint { get; set; } = "";
			public string VersePackageName { get; set; } = "";
			public string VerseDirectoryPath { get; set; } = "";
			public List<string> VerseDependencies { get; set; } = [];
			public bool AlwaysExportStructs { get; set; } = true;
			public bool AlwaysExportEnums { get; set; } = true;
			public bool AllowUETypesInNamespaces { get; set; } = false;
			public bool MinimizeGeneratedIncludes { get; set; } = false;

			public override string ToString()
			{
				return Name;
			}

			public bool TryGetDefine(string name, out string? value)
			{
				value = null;
				int length = name.Length;
				foreach (string define in PublicDefines)
				{
					if (!define.StartsWith(name, StringComparison.Ordinal))
					{
						continue;
					}
					if (define.Length > length)
					{
						if (define[length] != '=')
						{
							continue;
						}
						value = define.Substring(length + 1, define.Length - length - 1);
					}
					return true;
				}
				return false;
			}

			public bool TryGetDefine(string name, out int value)
			{
				string? stringValue;
				if (TryGetDefine(name, out stringValue))
				{
					return Int32.TryParse(stringValue, out value);
				}
				value = 0;
				return false;
			}
		}

		/// <summary>
		/// True if the current target is a game target
		/// </summary>
		public bool IsGameTarget { get; set; } = false;

		/// <summary>
		/// The engine path on the local machine
		/// </summary>
		public string RootLocalPath { get; set; } = String.Empty;

		/// <summary>
		/// Name of the target currently being compiled
		/// </summary>
		public string TargetName { get; set; } = String.Empty;

		/// <summary>
		/// File to contain additional dependencies that the generated code depends on
		/// </summary>
		public string ExternalDependenciesFile { get; set; } = String.Empty;

		/// <summary>
		/// Target-wide settings for UHT/code generation. 
		/// </summary>
		public UHTTargetSettings TargetSettings { set; get; } = new();

		/// <summary>
		/// List of modules
		/// </summary>
		public List<Module> Modules { get; set; } = [];

		/// <summary>
		/// List of active UHT plugins.  Only used by the C# version of UHT.  This
		/// list contains plugins from only modules listed above.
		/// </summary>
		public List<string> UhtPlugins { get; set; } = [];
	}
}
