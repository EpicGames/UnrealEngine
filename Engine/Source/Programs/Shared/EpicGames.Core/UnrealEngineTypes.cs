// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;

#pragma warning disable IDE1006 // Naming Styles
#pragma warning disable CS1591 // Missing documentation
#pragma warning disable CA1028 // Enum Storage should be Int32
#pragma warning disable CA2217 // Do not mark enums with FlagsAttribute
#pragma warning disable CA1069 // Enums values should not be duplicated
#pragma warning disable CA1008 // Enums should have zero value

namespace EpicGames.Core
{
	/// <summary>
	/// Standard return codes used by UBT and UHT
	/// 
	/// This MUST be kept in sync with EGeneratedBodyVersion defined in 
	/// Engine\Source\Runtime\Core\Public\Misc\CompilationResult.h
	/// </summary>
	public enum CompilationResult
	{
		/// <summary>
		/// Compilation succeeded
		/// </summary>
		Succeeded = 0,

		/// <summary>
		/// Build was canceled, this is used on the engine side only
		/// </summary>
		Canceled = 1,

		/// <summary>
		/// All targets were up to date, used only with -canskiplink
		/// </summary>
		UpToDate = 2,

		/// <summary>
		/// The process has most likely crashed. This is what UE returns in case of an assert
		/// </summary>
		CrashOrAssert = 3,

		/// <summary>
		/// Compilation failed because generated code changed which was not supported
		/// </summary>
		FailedDueToHeaderChange = 4,

		/// <summary>
		/// Compilation failed due to the engine modules needing to be rebuilt
		/// </summary>
		FailedDueToEngineChange = 5,

		/// <summary>
		/// Compilation failed due to compilation errors
		/// </summary>
		OtherCompilationError = 6,

		/// <summary>
		/// Compilation failed due to live coding action limit being exceeded
		/// </summary>
		LiveCodingLimitError = 7,

		/// <summary>
		/// Compilation failed due to TargetRules or ModuleRules errors
		/// </summary>
		RulesError = 8,

		/// <summary>
		/// Compilation failed due to invalid action graph
		/// </summary>
		ActionGraphInvalid = 9,

		/// <summary>
		/// Another instance of UBT was already running
		/// </summary>
		ConflictingInstance = 10,

		/// <summary>
		/// Compilation is not supported in the current build
		/// </summary>
		Unsupported,

		/// <summary>
		/// Unknown error
		/// </summary>
		Unknown
	}

	/// <summary>
	/// Helper extensions for CompilationResult
	/// </summary>
	public static class CompilationResultExtensions
	{

		/// <summary>
		/// Test to see if the return code is a success
		/// </summary>
		/// <param name="Result">The result to test</param>
		/// <returns>True if the return code is a success, false if it is not.</returns>
		public static bool Succeeded(this CompilationResult Result)
		{
			return Result == CompilationResult.Succeeded || Result == CompilationResult.UpToDate;
		}
	}

	/// <summary>
	/// Flags describing a class.
	///
	/// This MUST be kept in sync with EClassFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EClassFlags : uint
	{
		/// <summary>
		/// No Flags
		/// </summary>
		None = 0x00000000u,

		/// <summary>
		/// Class is abstract and can't be instantiated directly.
		/// </summary>
		Abstract = 0x00000001u,

		/// <summary>
		/// Save object configuration only to Default INIs, never to local INIs. Must be combined with "Config"
		/// </summary>
		DefaultConfig = 0x00000002u,

		/// <summary>
		/// Load object configuration at construction time.
		/// </summary>
		Config = 0x00000004u,

		/// <summary>
		/// This object type can't be saved; null it out at save time.
		/// </summary>
		Transient = 0x00000008u,

		/// <summary>
		/// This object type may not be available in certain context. (i.e. game runtime or in certain configuration). Optional class data is saved separately to other object types. (i.e. might use sidecar files)
		/// </summary>
		Optional = 0x00000010u,

		/// <summary>
		/// 
		/// </summary>
		MatchedSerializers = 0x00000020u,

		/// <summary>
		/// Indicates that the config settings for this class will be saved to Project/User*.ini (similar to "GlobalUserConfig")
		/// </summary>
		ProjectUserConfig = 0x00000040u,

		/// <summary>
		/// Class is a native class - native interfaces will have "Native" set, but not RF_MarkAsNative
		/// </summary>
		Native = 0x00000080u,

		/// <summary>
		/// Class is a partial class
		/// </summary>
		Partial = 0x00000100u,

		/// <summary>
		/// Do not allow users to create in the editor.
		/// </summary>
		NotPlaceable = 0x00000200u,

		/// <summary>
		/// Handle object configuration on a per-object basis, rather than per-class.
		/// </summary>
		PerObjectConfig = 0x00000400u,

		/// <summary>
		/// Whether SetUpRuntimeReplicationData still needs to be called for this class
		/// </summary>
		ReplicationDataIsSetUp = 0x00000800u,

		/// <summary>
		/// Class can be constructed from editinline New button.
		/// </summary>
		EditInlineNew = 0x00001000u,

		/// <summary>
		/// Display properties in the editor without using categories.
		/// </summary>
		CollapseCategories = 0x00002000u,

		/// <summary>
		/// Class is an interface
		/// </summary>
		Interface = 0x00004000u,

		/// <summary>
		/// Config for this class is overridden in platform inis, reload when previewing platforms
		/// </summary>	
		PerPlatformConfig = 0x00008000u,

		/// <summary>
		/// all properties and functions in this class are const and should be exported as const
		/// </summary>
		Const = 0x00010000u,

		/// <summary>
		/// Class flag indicating objects of this class need deferred dependency loading
		/// </summary>
		NeedsDeferredDependencyLoading = 0x00020000u,

		/// <summary>
		/// Indicates that the class was created from blueprint source material
		/// </summary>
		CompiledFromBlueprint = 0x00040000u,

		/// <summary>
		/// Indicates that only the bare minimum bits of this class should be DLL exported/imported
		/// </summary>
		MinimalAPI = 0x00080000u,

		/// <summary>
		/// Indicates this class must be DLL exported/imported (along with all of it's members)
		/// </summary>
		RequiredAPI = 0x00100000u,

		/// <summary>
		/// Indicates that references to this class default to instanced. Used to be subclasses of UComponent, but now can be any UObject
		/// </summary>
		DefaultToInstanced = 0x00200000u,

		/// <summary>
		/// Indicates that the parent token stream has been merged with ours.
		/// </summary>
		TokenStreamAssembled = 0x00400000u,

		/// <summary>
		/// Class has component properties.
		/// </summary>
		HasInstancedReference = 0x00800000u,

		/// <summary>
		/// Don't show this class in the editor class browser or edit inline new menus.
		/// </summary>
		Hidden = 0x01000000u,

		/// <summary>
		/// Don't save objects of this class when serializing
		/// </summary>
		Deprecated = 0x02000000u,

		/// <summary>
		/// Class not shown in editor drop down for class selection
		/// </summary>
		HideDropDown = 0x04000000u,

		/// <summary>
		/// Class settings are saved to [AppData]/..../Blah.ini (as opposed to "DefaultConfig")
		/// </summary>
		GlobalUserConfig = 0x08000000u,

		/// <summary>
		/// Class was declared directly in C++ and has no boilerplate generated by UnrealHeaderTool
		/// </summary>
		Intrinsic = 0x10000000u,

		/// <summary>
		/// Class has already been constructed (maybe in a previous DLL version before hot-reload).
		/// </summary>
		Constructed = 0x20000000u,

		/// <summary>
		/// Indicates that object configuration will not check against ini base/defaults when serialized
		/// </summary>
		ConfigDoNotCheckDefaults = 0x40000000u,

		/// <summary>
		/// Class has been consigned to oblivion as part of a blueprint recompile, and a newer version currently exists.
		/// </summary>
		NewerVersionExists = 0x80000000u,

		/// <summary>
		/// Flags to inherit from base class
		/// </summary>
		Inherit = Transient | Optional | DefaultConfig | Config | Partial | PerObjectConfig | ConfigDoNotCheckDefaults | NotPlaceable | Const | HasInstancedReference |
			Deprecated | DefaultToInstanced | GlobalUserConfig | ProjectUserConfig | NeedsDeferredDependencyLoading,

		/// <summary>
		/// These flags will be cleared by the compiler when the class is parsed during script compilation
		/// </summary>
		RecompilerClear = Inherit | Abstract | Native | Intrinsic | TokenStreamAssembled,

		/// <summary>
		/// These flags will be cleared by the compiler when the class is parsed during script compilation
		/// </summary>
		ShouldNeverBeLoaded = Native | Optional | Intrinsic | TokenStreamAssembled,

		/// <summary>
		/// These flags will be inherited from the base class only for non-intrinsic classes
		/// </summary>
		ScriptInherit = Inherit | EditInlineNew | CollapseCategories,

		/// <summary>
		/// This is used as a mask for the flags put into generated code for "compiled in" classes.
		/// </summary>
		SaveInCompiledInClasses = Abstract | DefaultConfig | GlobalUserConfig | ProjectUserConfig | PerPlatformConfig | Config | Transient | Optional | Native | Partial | NotPlaceable | PerObjectConfig |
			ConfigDoNotCheckDefaults | EditInlineNew | CollapseCategories | Interface | DefaultToInstanced | HasInstancedReference | Hidden | Deprecated |
			HideDropDown | Intrinsic | Const | MinimalAPI | RequiredAPI | MatchedSerializers | NeedsDeferredDependencyLoading,
	};

	/// <summary>
	/// Extension message for EClassFlags
	/// </summary>
	public static class EClassFlagsExtensions
	{
		public static bool HasAnyFlags(this EClassFlags InFlags, EClassFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EClassFlags InFlags, EClassFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EClassFlags InFlags, EClassFlags TestFlags, EClassFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Flags used for quickly casting classes of certain types; all class cast flags are inherited
	///
	/// This MUST be kept in sync with EClassCastFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EClassCastFlags : ulong
	{
		None = 0x0000000000000000,

		UField = 0x0000000000000001,
		FInt8Property = 0x0000000000000002,
		UEnum = 0x0000000000000004,
		UStruct = 0x0000000000000008,
		UScriptStruct = 0x0000000000000010,
		UClass = 0x0000000000000020,
		FByteProperty = 0x0000000000000040,
		FIntProperty = 0x0000000000000080,
		FFloatProperty = 0x0000000000000100,
		FUInt64Property = 0x0000000000000200,
		FClassProperty = 0x0000000000000400,
		FUInt32Property = 0x0000000000000800,
		FInterfaceProperty = 0x0000000000001000,
		FNameProperty = 0x0000000000002000,
		FStrProperty = 0x0000000000004000,
		FProperty = 0x0000000000008000,
		FObjectProperty = 0x0000000000010000,
		FBoolProperty = 0x0000000000020000,
		FUInt16Property = 0x0000000000040000,
		UFunction = 0x0000000000080000,
		FStructProperty = 0x0000000000100000,
		FArrayProperty = 0x0000000000200000,
		FInt64Property = 0x0000000000400000,
		FDelegateProperty = 0x0000000000800000,
		FNumericProperty = 0x0000000001000000,
		FMulticastDelegateProperty = 0x0000000002000000,
		FObjectPropertyBase = 0x0000000004000000,
		FWeakObjectProperty = 0x0000000008000000,
		FLazyObjectProperty = 0x0000000010000000,
		FSoftObjectProperty = 0x0000000020000000,
		FTextProperty = 0x0000000040000000,
		FInt16Property = 0x0000000080000000,
		FDoubleProperty = 0x0000000100000000,
		FSoftClassProperty = 0x0000000200000000,
		UPackage = 0x0000000400000000,
		ULevel = 0x0000000800000000,
		AActor = 0x0000001000000000,
		APlayerController = 0x0000002000000000,
		APawn = 0x0000004000000000,
		USceneComponent = 0x0000008000000000,
		UPrimitiveComponent = 0x0000010000000000,
		USkinnedMeshComponent = 0x0000020000000000,
		USkeletalMeshComponent = 0x0000040000000000,
		UBlueprint = 0x0000080000000000,
		UDelegateFunction = 0x0000100000000000,
		UStaticMeshComponent = 0x0000200000000000,
		FMapProperty = 0x0000400000000000,
		FSetProperty = 0x0000800000000000,
		FEnumProperty = 0x0001000000000000,
		USparseDelegateFunction = 0x0002000000000000,
		FMulticastInlineDelegateProperty = 0x0004000000000000,
		FMulticastSparseDelegateProperty = 0x0008000000000000,
		FFieldPathProperty = 0x0010000000000000,
		// = 0x0020000000000000,
		// = 0x0040000000000000,
		FLargeWorldCoordinatesRealProperty = 0x0080000000000000,
		FOptionalProperty = 0x0100000000000000,
		FVValueProperty = 0x0200000000000000,
		FVRestValueProperty = 0x0400000000000000,
		FVerseStringProperty = 0x0800000000000000,
		FUtf8StrProperty = 0x1000000000000000,
		FAnsiStrProperty = 0x2000000000000000,
		FVCellProperty = 0x4000000000000000,
		// = 0x8000000000000000,
		AllFlags = UInt64.MaxValue,
	};

	/// <summary>
	/// Helper extensions for EClassCastFlags
	/// </summary>
	public static class EClassCastFlagsExtensions
	{
		public static bool HasAnyFlags(this EClassCastFlags InFlags, EClassCastFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EClassCastFlags InFlags, EClassCastFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EClassCastFlags InFlags, EClassCastFlags TestFlags, EClassCastFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Flags describing a UEnum
	///
	/// This MUST be kept in sync with EEnumFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EEnumFlags : uint
	{
		None,
		Flags = 1 << 0,
		NewerVersionExists = 1 << 1,
	}

	/// <summary>
	/// Helper extensions for EEnumFlags
	/// </summary>
	public static class EEnumFlagsExtensions
	{
		public static bool HasAnyFlags(this EEnumFlags InFlags, EEnumFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EEnumFlags InFlags, EEnumFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EEnumFlags InFlags, EEnumFlags TestFlags, EEnumFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Function flags.
	///
	/// This MUST be kept in sync with EFunctionFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\Script.h
	/// </summary>
	[Flags]
	public enum EFunctionFlags : uint
	{
		None = 0x00000000,

		/// <summary>
		/// Function is final (prebindable, non-overridable function).
		/// </summary>
		Final = 0x00000001,

		/// <summary>
		/// Indicates this function is DLL exported/imported.
		/// </summary>
		RequiredAPI = 0x00000002,

		/// <summary>
		/// Function will only run if the object has network authority
		/// </summary>
		BlueprintAuthorityOnly = 0x00000004,

		/// <summary>
		/// Function is cosmetic in nature and should not be invoked on dedicated servers
		/// </summary>
		BlueprintCosmetic = 0x00000008,

		// = 0x00000010,   // unused.
		// = 0x00000020,   // unused.

		/// <summary>
		/// Function is network-replicated.
		/// </summary>
		Net = 0x00000040,

		/// <summary>
		/// Function should be sent reliably on the network.
		/// </summary>
		NetReliable = 0x00000080,

		/// <summary>
		/// Function is sent to a net service
		/// </summary>
		NetRequest = 0x00000100,

		/// <summary>
		/// Executable from command line.
		/// </summary>
		Exec = 0x00000200,

		/// <summary>
		/// Native function.
		/// </summary>
		Native = 0x00000400,

		/// <summary>
		/// Event function.
		/// </summary>
		Event = 0x00000800,

		/// <summary>
		/// Function response from a net service
		/// </summary>
		NetResponse = 0x00001000,

		/// <summary>
		/// Static function.
		/// </summary>
		Static = 0x00002000,

		/// <summary>
		/// Function is networked multicast Server -> All Clients
		/// </summary>
		NetMulticast = 0x00004000,

		/// <summary>
		/// Function is used as the merge 'ubergraph' for a blueprint, only assigned when using the persistent 'ubergraph' frame
		/// </summary>
		UbergraphFunction = 0x00008000,

		/// <summary>
		/// Function is a multi-cast delegate signature (also requires Delegate to be set!)
		/// </summary>
		MulticastDelegate = 0x00010000,

		/// <summary>
		/// Function is accessible in all classes (if overridden, parameters must remain unchanged).
		/// </summary>
		Public = 0x00020000,

		/// <summary>
		/// Function is accessible only in the class it is defined in (cannot be overridden, but function name may be reused in subclasses.  IOW: if overridden, parameters don't need to match, and Super.Func() cannot be accessed since it's private.)
		/// </summary>
		Private = 0x00040000,

		/// <summary>
		/// Function is accessible only in the class it is defined in and subclasses (if overridden, parameters much remain unchanged).
		/// </summary>
		Protected = 0x00080000,

		/// <summary>
		/// Function is delegate signature (either single-cast or multi-cast, depending on whether MulticastDelegate is set.)
		/// </summary>
		Delegate = 0x00100000,

		/// <summary>
		/// Function is executed on servers (set by replication code if passes check)
		/// </summary>
		NetServer = 0x00200000,

		/// <summary>
		/// function has out (pass by reference) parameters
		/// </summary>
		HasOutParms = 0x00400000,

		/// <summary>
		/// function has structs that contain defaults
		/// </summary>
		HasDefaults = 0x00800000,

		/// <summary>
		/// function is executed on clients
		/// </summary>
		NetClient = 0x01000000,

		/// <summary>
		/// function is imported from a DLL
		/// </summary>
		DLLImport = 0x02000000,

		/// <summary>
		/// function can be called from blueprint code
		/// </summary>
		BlueprintCallable = 0x04000000,

		/// <summary>
		/// function can be overridden/implemented from a blueprint
		/// </summary>
		BlueprintEvent = 0x08000000,

		/// <summary>
		/// function can be called from blueprint code, and is also pure (produces no side effects). If you set this, you should set BlueprintCallable as well.
		/// </summary>
		BlueprintPure = 0x10000000,

		/// <summary>
		/// function can only be called from an editor script.
		/// </summary>
		EditorOnly = 0x20000000,

		/// <summary>
		/// function can be called from blueprint code, and only reads state (never writes state)
		/// </summary>
		Const = 0x40000000,

		/// <summary>
		/// function must supply a _Validate implementation
		/// </summary>
		NetValidate = 0x80000000,

		AllFlags = 0xFFFFFFFF,

		// Combinations of flags.
		FuncInherit = Exec | Event | BlueprintCallable | BlueprintEvent | BlueprintAuthorityOnly | BlueprintCosmetic | Const,
		FuncOverrideMatch = Exec | Final | Static | Public | Protected | Private,
		NetFuncFlags = Net | NetReliable | NetServer | NetClient | NetMulticast,
		AccessSpecifiers = Public | Private | Protected,
	}

	/// <summary>
	/// Helper extensions for EFunctionFlags
	/// </summary>
	public static class EFunctionFlagsExtensions
	{
		public static bool HasAnyFlags(this EFunctionFlags InFlags, EFunctionFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EFunctionFlags InFlags, EFunctionFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EFunctionFlags InFlags, EFunctionFlags TestFlags, EFunctionFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Objects flags for internal use(GC, low level UObject code)
	/// 
	/// This MUST be kept in sync with EInternalObjectFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EInternalObjectFlags : int
	{
		None = 0,
		//~ All the other bits are reserved, DO NOT ADD NEW FLAGS HERE!

		/// <summary>
		/// One of the flags used by Garbage Collector to determine UObject's reachability state
		/// </summary>
		ReachabilityFlag0 = 1 << 14,

		/// <summary>
		/// One of the flags used by Garbage Collector to determine UObject's reachability state
		/// </summary>
		ReachabilityFlag1 = 1 << 15,

		/// <summary>
		/// One of the flags used by Garbage Collector to determine UObject's reachability state
		/// </summary>
		ReachabilityFlag2 = 1 << 16,

		/// <summary>
		/// Object was constructed in an AutoRTFM transaction that has been aborted
		/// </summary>
		AutoRTFMConstructionAborted = 1 << 17,

		/// <summary>
		/// Object is ready to be imported by another package during loading
		/// </summary>
		LoaderImport = 1 << 20,

		/// <summary>
		/// Garbage from logical point of view and should not be referenced. This flag is mirrored in EObjectFlags as RF_Garbage for performance
		/// </summary>
		Garbage = 1 << 21,

		/// <summary>
		/// Object is being asynchronously loaded.
		/// </summary>
		AsyncLoadingPhase1 = 1 << 22,

		/// <summary>
		/// External reference to object in cluster exists
		/// </summary>
		ReachableInCluster = 1 << 23,

		/// <summary>
		/// Root of a cluster
		/// </summary>
		ClusterRoot = 1 << 24,

		/// <summary>
		/// Native (UClass only).
		/// </summary>
		Native = 1 << 25,

		/// <summary>
		/// Object exists only on a different thread than the game thread.
		/// </summary>
		Async = 1 << 26,

		/// <summary>
		/// Object is being asynchronously loaded.
		/// </summary>
		AsyncLoadingPhase2 = 1 << 27,

		/// <summary>
		/// Object is not reachable on the object graph.
		/// </summary>
		Unreachable = 1 << 28,

		/// <summary>
		/// Object currently has ref-counts associated with it.
		/// </summary>
		RefCounted = 1 << 29,

		/// <summary>
		/// Object will not be garbage collected, even if unreferenced.
		/// </summary>
		RootSet = 1 << 30,

		/// <summary>
		/// Object didn't have its class constructor called yet (only the UObjectBase one to initialize its most basic members)
		/// </summary>
		PendingConstruction = 1 << 31,
	};

	/// <summary>
	/// Helper extensions for EInternalObjectFlags
	/// </summary>
	public static class EInternalObjectFlagsExtensions
	{
		public static bool HasAnyFlags(this EInternalObjectFlags InFlags, EInternalObjectFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EInternalObjectFlags InFlags, EInternalObjectFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EInternalObjectFlags InFlags, EInternalObjectFlags TestFlags, EInternalObjectFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Package flags, passed into UPackage::SetPackageFlags and related functions
	/// 
	/// This MUST be kept in sync with EPackageFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EPackageFlags : ulong
	{
		/// <summary>
		/// No flags
		/// </summary>
		None = 0x00000000,

		/// <summary>
		/// Newly created package, not saved yet. In editor only.
		/// </summary>
		NewlyCreated = 0x00000001,

		/// <summary>
		/// Purely optional for clients.
		/// </summary>
		ClientOptional = 0x00000002,

		/// <summary>
		/// Only needed on the server side.
		/// </summary>
		ServerSideOnly = 0x00000004,

		/// <summary>
		/// This package is from "compiled in" classes.
		/// </summary>
		CompiledIn = 0x00000010,

		/// <summary>
		/// This package was loaded just for the purposes of diffing
		/// </summary>
		ForDiffing = 0x00000020,

		/// <summary>
		/// This is editor-only package (for example: editor module script package)
		/// </summary>
		EditorOnly = 0x00000040,

		/// <summary>
		/// Developer module
		/// </summary>
		Developer = 0x00000080,

		/// <summary>
		/// Loaded only in uncooked builds (i.e. runtime in editor)
		/// </summary>
		UncookedOnly = 0x00000100,

		/// <summary>
		/// Package is cooked
		/// </summary>
		Cooked = 0x00000200,

		/// <summary>
		/// Package doesn't contain any asset object (although asset tags can be present)
		/// </summary>
		ContainsNoAsset = 0x00000400,

		/// <summary>
		/// Objects in this package cannot be referenced in a different plugin or mount point (i.e /Game -> /Engine)
		/// </summary>
		NotExternallyReferenceable = 0x00000800,

		/// <summary>
		/// Objects in this package can only be referenced in a different plugin or mount point by Epic
		/// </summary>
		AccessSpecifierEpicInternal = 0x00001000,

		/// <summary>
		/// Uses unversioned property serialization instead of versioned tagged property serialization
		/// </summary>
		UnversionedProperties = 0x00002000,

		/// <summary>
		/// Contains map data (UObjects only referenced by a single ULevel) but is stored in a different package
		/// </summary>
		ContainsMapData = 0x00004000,

		/// <summary>
		/// Temporarily set on a package while it is being saved.
		/// </summary>
		IsSaving = 0x00008000,

		/// <summary>
		/// package is currently being compiled
		/// </summary>
		Compiling = 0x00010000,

		/// <summary>
		/// Set if the package contains a ULevel/ UWorld object
		/// </summary>
		ContainsMap = 0x00020000,

		/// <summary>
		/// Set if the package contains any data to be gathered by localization
		/// </summary>
		RequiresLocalizationGather = 0x00040000,

		// Unused = 0x00080000,

		/// <summary>
		/// Set if the package was created for the purpose of PIE
		/// </summary>
		PlayInEditor = 0x00100000,

		/// <summary>
		/// Package is allowed to contain UClass objects
		/// </summary>
		ContainsScript = 0x00200000,

		/// <summary>
		/// Editor should not export asset in this package
		/// </summary>
		DisallowExport = 0x00400000,

		// Unused = 0x00800000,
		// Unused = 0x01000000,	
		// Unused = 0x02000000,	
		// Unused = 0x04000000,

		/// <summary>
		/// This package This package was generated by the cooker and does not exist in the WorkspaceDomain
		/// </summary>
		CookGenerated = 0x08000000,

		/// <summary>
		/// This package should resolve dynamic imports from its export at runtime.
		/// </summary>
		DynamicImports = 0x10000000,

		/// <summary>
		/// This package contains elements that are runtime generated, and may not follow standard loading order rules
		/// </summary>
		RuntimeGenerated = 0x20000000,

		/// <summary>
		/// This package is reloading in the cooker, try to avoid getting data we will never need. We won't save this package.
		/// </summary>
		ReloadingForCooker = 0x40000000,

		/// <summary>
		/// Package has editor-only data filtered out
		/// </summary>
		FilterEditorOnly = 0x80000000,

		/// <summary>
		/// Transient Flags are cleared when serializing to or from PackageFileSummary
		/// </summary>
		TransientFlags = NewlyCreated | IsSaving | ReloadingForCooker,
	}

	/// <summary>
	/// Helper extensions for EPackageFlags
	/// </summary>
	public static class EPackageFlagsExtensions
	{
		public static bool HasAnyFlags(this EPackageFlags InFlags, EPackageFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EPackageFlags InFlags, EPackageFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EPackageFlags InFlags, EPackageFlags TestFlags, EPackageFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Flags associated with each property in a class, overriding the
	/// property's default behavior.
	/// 
	/// This MUST be kept in sync with EPropertyFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\ObjectMacros.h
	/// </summary>
	[Flags]
	public enum EPropertyFlags : ulong
	{
		None = 0,

		/// <summary>
		/// Property is user-settable in the editor.
		/// </summary>
		Edit = 0x0000000000000001,

		/// <summary>
		/// This is a constant function parameter
		/// </summary>
		ConstParm = 0x0000000000000002,

		/// <summary>
		/// This property can be read by blueprint code
		/// </summary>
		BlueprintVisible = 0x0000000000000004,

		/// <summary>
		/// Object can be exported with actor.
		/// </summary>
		ExportObject = 0x0000000000000008,

		/// <summary>
		/// This property cannot be modified by blueprint code
		/// </summary>
		BlueprintReadOnly = 0x0000000000000010,

		/// <summary>
		/// Property is relevant to network replication.
		/// </summary>
		Net = 0x0000000000000020,

		/// <summary>
		/// Indicates that elements of an array can be modified, but its size cannot be changed.
		/// </summary>
		EditFixedSize = 0x0000000000000040,

		/// <summary>
		/// Function/When call parameter.
		/// </summary>
		Parm = 0x0000000000000080,

		/// <summary>
		/// Value is copied out after function call.
		/// </summary>
		OutParm = 0x0000000000000100,

		/// <summary>
		/// memset is fine for construction
		/// </summary>
		ZeroConstructor = 0x0000000000000200,

		/// <summary>
		/// Return value.
		/// </summary>
		ReturnParm = 0x0000000000000400,

		/// <summary>
		/// Disable editing of this property on an archetype/sub-blueprint
		/// </summary>
		DisableEditOnTemplate = 0x0000000000000800,

		/// <summary>
		/// Object property can never be null
		/// </summary>
		NonNullable = 0x0000000000001000,

		/// <summary>
		/// Property is transient: shouldn't be saved or loaded, except for Blueprint CDOs.
		/// </summary>
		Transient = 0x0000000000002000,

		/// <summary>
		/// Property should be loaded/saved as permanent profile.
		/// </summary>
		Config = 0x0000000000004000,

		/// <summary>
		/// Parameter is required in blueprint. Not linking the parameter with a node will result in a compile error.
		/// </summary>
		RequiredParm = 0x0000000000008000,

		/// <summary>
		/// Disable editing on an instance of this class
		/// </summary>
		DisableEditOnInstance = 0x0000000000010000,

		/// <summary>
		/// Property is uneditable in the editor.
		/// </summary>
		EditConst = 0x0000000000020000,

		/// <summary>
		/// Load config from base class, not subclass.
		/// </summary>
		GlobalConfig = 0x0000000000040000,

		/// <summary>
		/// Property is a component references.
		/// </summary>
		InstancedReference = 0x0000000000080000,

		/// <summary>
		/// Property saves objects in separate files, breaks hard links and reload them based on discovery.
		/// </summary>
		ExperimentalExternalObjects = 0x0000000000100000,

		/// <summary>
		/// Property should always be reset to the default value during any type of duplication (copy/paste, binary duplication, etc.)
		/// </summary>
		DuplicateTransient = 0x0000000000200000,

		// Unused = 0x0000000000400000,
		// Unused = 0x0000000000800000,

		/// <summary>
		/// Property should be serialized for save games, this is only checked for game-specific archives with ArIsSaveGame
		/// </summary>
		SaveGame = 0x0000000001000000,

		/// <summary>
		/// Hide clear (and browse) button.
		/// </summary>
		NoClear = 0x0000000002000000,

		// Unused = 0x0000000004000000,

		/// <summary>
		/// Value is passed by reference; "OutParam" and "Param" should also be set.
		/// </summary>
		ReferenceParm = 0x0000000008000000,

		/// <summary>
		/// MC Delegates only.  Property should be exposed for assigning in blueprint code
		/// </summary>
		BlueprintAssignable = 0x0000000010000000,

		/// <summary>
		/// Property is deprecated.  Read it from an archive, but don't save it.
		/// </summary>
		Deprecated = 0x0000000020000000,

		/// <summary>
		/// If this is set, then the property can be memcopied instead of CopyCompleteValue / CopySingleValue
		/// </summary>
		IsPlainOldData = 0x0000000040000000,

		/// <summary>
		/// Not replicated. For non replicated properties in replicated structs 
		/// </summary>
		RepSkip = 0x0000000080000000,

		/// <summary>
		/// Notify actors when a property is replicated
		/// </summary>
		RepNotify = 0x0000000100000000,

		/// <summary>
		/// interpolatable property for use with cinematics
		/// </summary>
		Interp = 0x0000000200000000,

		/// <summary>
		/// Property isn't transacted
		/// </summary>
		NonTransactional = 0x0000000400000000,

		/// <summary>
		/// Property should only be loaded in the editor
		/// </summary>
		EditorOnly = 0x0000000800000000,

		/// <summary>
		/// No destructor
		/// </summary>
		NoDestructor = 0x0000001000000000,

		// Unused = 0x0000002000000000,

		/// <summary>
		/// Only used for weak pointers, means the export type is autoweak
		/// </summary>
		AutoWeak = 0x0000004000000000,

		/// <summary>
		/// Property contains component references.
		/// </summary>
		ContainsInstancedReference = 0x0000008000000000,

		/// <summary>
		/// asset instances will add properties with this flag to the asset registry automatically
		/// </summary>
		AssetRegistrySearchable = 0x0000010000000000,

		/// <summary>
		/// The property is visible by default in the editor details view
		/// </summary>
		SimpleDisplay = 0x0000020000000000,

		/// <summary>
		/// The property is advanced and not visible by default in the editor details view
		/// </summary>
		AdvancedDisplay = 0x0000040000000000,

		/// <summary>
		/// property is protected from the perspective of script
		/// </summary>
		Protected = 0x0000080000000000,

		/// <summary>
		/// MC Delegates only.  Property should be exposed for calling in blueprint code
		/// </summary>
		BlueprintCallable = 0x0000100000000000,

		/// <summary>
		/// MC Delegates only.  This delegate accepts (only in blueprint) only events with BlueprintAuthorityOnly.
		/// </summary>
		BlueprintAuthorityOnly = 0x0000200000000000,

		/// <summary>
		/// Property shouldn't be exported to text format (e.g. copy/paste)
		/// </summary>
		TextExportTransient = 0x0000400000000000,

		/// <summary>
		/// Property should only be copied in PIE
		/// </summary>
		NonPIEDuplicateTransient = 0x0000800000000000,

		/// <summary>
		/// Property is exposed on spawn
		/// </summary>
		ExposeOnSpawn = 0x0001000000000000,

		/// <summary>
		/// A object referenced by the property is duplicated like a component. (Each actor should have an own instance.)
		/// </summary>
		PersistentInstance = 0x0002000000000000,

		/// <summary>
		/// Property was parsed as a wrapper class like TSubclassOf&lt;T&gt;, FScriptInterface etc., rather than a USomething*
		/// </summary>
		UObjectWrapper = 0x0004000000000000,

		/// <summary>
		/// This property can generate a meaningful hash value.
		/// </summary>
		HasGetValueTypeHash = 0x0008000000000000,

		/// <summary>
		/// Public native access specifier
		/// </summary>
		NativeAccessSpecifierPublic = 0x0010000000000000,

		/// <summary>
		/// Protected native access specifier
		/// </summary>
		NativeAccessSpecifierProtected = 0x0020000000000000,

		/// <summary>
		/// Private native access specifier
		/// </summary>
		NativeAccessSpecifierPrivate = 0x0040000000000000,

		/// <summary>
		/// Property shouldn't be serialized, can still be exported to text
		/// </summary>
		SkipSerialization = 0x0080000000000000,

		/// <summary>
		/// Property is a TObjectPtr instead of a USomething*. Need to differentiate between TObjectclassOf and TObjectPtr
		/// </summary>
		TObjectPtr = 0x0100000000000000,

		/// <summary>
		/// TObjectPtr properties are both wrapped and TObjectPtr
		/// </summary>
		TObjectPtrWrapper = TObjectPtr | UObjectWrapper,

		/// <summary>
		/// ****Experimental*** Property will use different logic to serialize knowing what changes are done against its default use the overridable information provided by the overridable manager on the object
		/// </summary>
		ExperimentalOverridableLogic = 0x0200000000000000,

		/// <summary>
		/// ****Experimental*** Property should never inherit from the parent when using overridable serialization
		/// </summary>
		ExperimentalAlwaysOverriden = 0x0400000000000000,

		/// <summary>
		/// ****Experimental*** Property should never be overridden when using overridable serialization
		/// </summary>
		ExperimentalNeverOverriden = 0x0800000000000000,

		/// <summary>
		/// Enables the instancing graph self referencing logic, delegates and verse function are already using this
		/// </summary>
		AllowSelfReference = 0x1000000000000000,

		/// <summary>
		/// Use to force native UCLASS properties to be included in the PostConstructLink chain in a derived Blueprint class, regardless of whether it differs from the native class constructor-initialized value
		/// </summary>
		ForcePostConstructLink = 0x2000000000000000,

		/// <summary>
		/// All Native Access Specifier flags
		/// </summary>
		NativeAccessSpecifiers = NativeAccessSpecifierPublic | NativeAccessSpecifierProtected | NativeAccessSpecifierPrivate,

		/// <summary>
		/// All parameter flags
		/// </summary>
		ParmFlags = Parm | OutParm | ReturnParm | ReferenceParm | ConstParm | RequiredParm,

		/// <summary>
		/// Flags to keep in the inner type when propagating container flags 
		/// </summary>
		PropagateKeepInInner = NonNullable,

		/// <summary>
		/// Flags that are propagated to properties inside array container
		/// </summary>
		PropagateToArrayInner = ExportObject | PersistentInstance | InstancedReference | ContainsInstancedReference | Config | EditConst | Deprecated | EditorOnly | AutoWeak | UObjectWrapper | TObjectPtr,

		/// <summary>
		/// Flags that are propagated to properties inside optional container
		/// </summary>
		PropagateToOptionalInner = ExportObject | PersistentInstance | InstancedReference | ContainsInstancedReference | Config | EditConst | Deprecated | EditorOnly | AutoWeak | UObjectWrapper | Edit,

		/// <summary>
		/// Flags that are propagated to value properties inside map container
		/// </summary>
		PropagateToMapValue = ExportObject | PersistentInstance | InstancedReference | ContainsInstancedReference | Config | EditConst | Deprecated | EditorOnly | AutoWeak | UObjectWrapper | TObjectPtr | Edit,

		/// <summary>
		/// Flags that are propagated to key properties inside map container
		/// </summary>
		PropagateToMapKey = ExportObject | PersistentInstance | InstancedReference | ContainsInstancedReference | Config | EditConst | Deprecated | EditorOnly | AutoWeak | UObjectWrapper | TObjectPtr | Edit,

		/// <summary>
		/// Flags that are propagated to properties inside set container
		/// </summary>
		PropagateToSetElement = ExportObject | PersistentInstance | InstancedReference | ContainsInstancedReference | Config | EditConst | Deprecated | EditorOnly | AutoWeak | UObjectWrapper | TObjectPtr | Edit,

		/// <summary>
		/// The flags that should never be set on interface properties
		/// </summary>
		InterfaceClearMask = ExportObject | InstancedReference | ContainsInstancedReference,

		/// <summary>
		/// All the properties that can be stripped for final release console builds
		/// </summary>
		DevelopmentAssets = EditorOnly,

		/// <summary>
		/// All the properties that should never be loaded or saved
		/// </summary>
		ComputedFlags = IsPlainOldData | NoDestructor | ZeroConstructor | HasGetValueTypeHash,

		/// <summary>
		/// Mask of all property flags
		/// </summary>
		AllFlags = UInt64.MaxValue,
	};

	/// <summary>
	/// Helper extensions for EPropertyFlags
	/// </summary>
	public static class EPropertyFlagsExtensions
	{
		private static readonly Lazy<List<string>> BitNames = new(() =>
		{
			List<string> Out = [];
			FieldInfo[] Fields = typeof(EPropertyFlags).GetFields();
			for (int Bit = 0; Bit < 64; ++Bit)
			{
				bool bFound = false;
				ulong Mask = (ulong)1 << Bit;
				foreach (FieldInfo Field in Fields)
				{
					if (Field.IsSpecialName)
					{
						continue;
					}
					object? Value = Field.GetValue(null);
					if (Value != null)
					{
						if (Mask == (ulong)Value)
						{
							Out.Add($"CPF_{Field.Name}");
							bFound = true;
							break;
						}
					}
				}
				if (!bFound)
				{
					Out.Add($"0x{Mask:X16}");
				}
			}
			return Out;
		});

		public static bool HasAnyFlags(this EPropertyFlags InFlags, EPropertyFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EPropertyFlags InFlags, EPropertyFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EPropertyFlags InFlags, EPropertyFlags TestFlags, EPropertyFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}

		public static List<string> ToStringList(this EPropertyFlags InFlags, bool bIncludePrefix = true)
		{
			List<string> LocalBitNames = BitNames.Value;
			ulong IntFlags = (ulong)InFlags;
			List<string> Out = [];
			for (int Bit = 0; Bit < 64; ++Bit)
			{
				ulong Mask = (ulong)1 << Bit;
				if (Mask > IntFlags)
				{
					break;
				}
				if ((Mask & IntFlags) != 0)
				{
					if (bIncludePrefix)
					{
						Out.Add(LocalBitNames[Bit]);
					}
					else
					{
						Out.Add(LocalBitNames[Bit].Substring(4));
					}
				}
			}
			return Out;
		}
	}

	/// <summary>
	/// Flags describing a struct
	/// 
	/// This MUST be kept in sync with EStructFlags defined in 
	/// Engine\Source\Runtime\CoreUObject\Public\UObject\Class.h
	/// </summary>
	[Flags]
	public enum EStructFlags
	{
		NoFlags = 0x00000000,
		Native = 0x00000001,

		/// <summary>
		/// If set, this struct will be compared using native code
		/// </summary>
		IdenticalNative = 0x00000002,

		HasInstancedReference = 0x00000004,

		NoExport = 0x00000008,

		/// <summary>
		/// Indicates that this struct should always be serialized as a single unit
		/// </summary>
		Atomic = 0x00000010,

		/// <summary>
		/// Indicates that this struct uses binary serialization; it is unsafe to add/remove members from this struct without incrementing the package version
		/// </summary>
		Immutable = 0x00000020,

		/// <summary>
		/// If set, native code needs to be run to find referenced objects
		/// </summary>
		AddStructReferencedObjects = 0x00000040,

		/// <summary>
		/// Indicates that this struct should be exportable/importable at the DLL layer.  Base structs must also be exportable for this to work.
		/// </summary>
		RequiredAPI = 0x00000200,

		/// <summary>
		/// If set, this struct will be serialized using the CPP net serializer
		/// </summary>
		NetSerializeNative = 0x00000400,

		/// <summary>
		/// If set, this struct will be serialized using the CPP serializer
		/// </summary>
		SerializeNative = 0x00000800,

		/// <summary>
		/// If set, this struct will be copied using the CPP operator=
		/// </summary>
		CopyNative = 0x00001000,

		/// <summary>
		/// If set, this struct will be copied using memcpy
		/// </summary>
		IsPlainOldData = 0x00002000,

		/// <summary>
		/// If set, this struct has no destructor and non will be called. IsPlainOldData implies NoDestructor
		/// </summary>
		NoDestructor = 0x00004000,

		/// <summary>
		/// If set, this struct will not be constructed because it is assumed that memory is zero before construction.
		/// </summary>
		ZeroConstructor = 0x00008000,

		/// <summary>
		/// If set, native code will be used to export text
		/// </summary>
		ExportTextItemNative = 0x00010000,

		/// <summary>
		/// If set, native code will be used to export text
		/// </summary>
		ImportTextItemNative = 0x00020000,

		/// <summary>
		/// If set, this struct will have PostSerialize called on it after CPP serializer or tagged property serialization is complete
		/// </summary>
		PostSerializeNative = 0x00040000,

		/// <summary>
		/// If set, this struct will have SerializeFromMismatchedTag called on it if a mismatched tag is encountered.
		/// </summary>
		SerializeFromMismatchedTag = 0x00080000,

		/// <summary>
		/// If set, this struct will be serialized using the CPP net delta serializer
		/// </summary>
		NetDeltaSerializeNative = 0x00100000,

		/// <summary>
		/// If set, this struct will be have PostScriptConstruct called on it after a temporary object is constructed in a running blueprint
		/// </summary>
		PostScriptConstruct = 0x00200000,

		/// <summary>
		/// If set, this struct can share net serialization state across connections
		/// </summary>
		NetSharedSerialization = 0x00400000,

		/// <summary>
		/// If set, this struct has been cleaned and sanitized (trashed) and should not be used
		/// </summary>
		Trashed = 0x00800000,

		/// <summary>
		/// If set, this structure has been replaced via reinstancing
		/// </summary>
		NewerVersionExists = 0x01000000,

		/// <summary>
		/// If set, this struct will have CanEditChange on it in the editor to determine if a child property can be edited
		/// </summary>
		CanEditChange = 0x02000000,

		/// <summary>
		/// Struct flags that are automatically inherited
		/// </summary>
		Inherit = HasInstancedReference | Atomic,

		/// <summary>
		/// Flags that are always computed, never loaded or done with code generation
		/// </summary>
		ComputedFlags = NetDeltaSerializeNative | NetSerializeNative | SerializeNative | PostSerializeNative | CopyNative | IsPlainOldData |
			NoDestructor | ZeroConstructor | IdenticalNative | AddStructReferencedObjects | ExportTextItemNative | ImportTextItemNative |
			SerializeFromMismatchedTag | PostScriptConstruct | NetSharedSerialization
	}

	/// <summary>
	/// Helper extensions for EStructFlags
	/// </summary>
	public static class EStructFlagsExtensions
	{
		public static bool HasAnyFlags(this EStructFlags InFlags, EStructFlags TestFlags)
		{
			return (InFlags & TestFlags) != 0;
		}

		public static bool HasAllFlags(this EStructFlags InFlags, EStructFlags TestFlags)
		{
			return (InFlags & TestFlags) == TestFlags;
		}

		public static bool HasExactFlags(this EStructFlags InFlags, EStructFlags TestFlags, EStructFlags MatchFlags)
		{
			return (InFlags & TestFlags) == MatchFlags;
		}
	}

	/// <summary>
	/// Extra flags for verse classes and structures
	/// 
	/// Keep these flags in sync with VVMVerseClassFlags.h
	/// </summary>
	[Flags]
	public enum EVerseClassFlags : uint
	{
		None = 0x00000000,
		NativeBound = 0x00000001,
		UniversallyAccessible = 0x00000002,   // The class is accessible from any Verse path, and is in a package with a public scope.
		Concrete = 0x00000004,                // The class can be instantiated without explicitly setting any properties
		Module = 0x00000008,                  // This class represents a Verse module
		UHTNative = 0x00000010,               // This class was created by UHT
		Tuple = 0x00000020,                   // This class represents a tuple
		EpicInternal = 0x00000040,            // This class is epic_internal
		HasInstancedSemantics = 0x00000080,   // This class is using explicit instanced reference semantics (backcompat)
		FinalSuper = 0x00000100,              // This class has a <final_super> attribute
		Castable = 0x00000200,                // This class has a <castable> attribute
		EpicInternalConstructor = 0x00000400, // This class' has a <castable> attribute's constructor is epic_internal
		Persistable = 0x00000800u,            // This class has a <persistable> attribute
		Parametric = 0x00001000u,             // This class definition is local to a function, i.e. is parametric
		PersonaConstructible = 0x00004000u,   // This class may be constructed by Persona
		HasVar = 0x00008000u,                 // This class has at least one mutable variable.
		InheritsFromBlueprint = 0x00010000u,  // This class has a blueprint class as (transitive) superclass
		HasTransientSubobjects = 0x00020000u, // This class has a non-serialized reference to an archetype instantiation

		// @TODO: this should be a per-function flag; a class flag is not granular enough
		ErrInoperable = 0x40000000, // One or more of the class's functions contain mis-linked (malformed) bytecode
		ErrIncomplete = 0x80000000, // The class layout is malformed (missing super, illformed data-member, etc.)

		Err = (ErrIncomplete | ErrInoperable)
	};
}
