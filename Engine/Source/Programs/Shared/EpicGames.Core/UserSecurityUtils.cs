// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Security.Principal;

namespace EpicGames.Core;

/// <summary>
/// Provides utility methods for checking user security contexts and permissions
/// </summary>
public static class UserSecurityUtils
{
	/// <summary>
	/// Checks if the current process is running with administrator privileges on Windows
	/// </summary>
	/// <returns>True if process has administrator privileges</returns>
	public static bool IsAdministrator()
	{
		if (!OperatingSystem.IsWindows())
		{
			return false;
		}

		try
		{
			using WindowsIdentity identity = WindowsIdentity.GetCurrent();
			WindowsPrincipal principal = new(identity);
			return principal.IsInRole(WindowsBuiltInRole.Administrator);
		}
		catch
		{
			return false;
		}
	}
}

