// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Represents a memory region which can be treated as a utf-8 string.
	/// </summary>
	public readonly struct Utf8String : IEquatable<Utf8String>, IComparable<Utf8String>
	{
		/// <summary>
		/// An empty string
		/// </summary>
		public static readonly Utf8String Empty = new();

		/// <summary>
		/// The data represented by this string
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Returns read only span for this string
		/// </summary>
		public ReadOnlySpan<byte> Span => Memory.Span;

		/// <summary>
		/// Determines if this string is empty
		/// </summary>
		public bool IsEmpty => Memory.IsEmpty;

		/// <summary>
		/// Returns the length of this string
		/// </summary>
		public int Length => Memory.Length;

		/// <summary>
		/// Allows indexing individual bytes of the data
		/// </summary>
		/// <param name="index">Byte index</param>
		/// <returns>Byte at the given index</returns>
		public byte this[int index] => Span[index];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Text to construct from</param>
		public Utf8String(string text)
		{
			Memory = Encoding.UTF8.GetBytes(text);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Text to construct from</param>
		public Utf8String(ReadOnlySpan<char> text)
		{
			int length = Encoding.UTF8.GetByteCount(text);
			byte[] buffer = new byte[length];
			Encoding.UTF8.GetBytes(text, buffer);
			Memory = buffer;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">The data to construct from</param>
		public Utf8String(ReadOnlyMemory<byte> memory)
		{
			Memory = memory;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buffer">The buffer to construct from</param>
		/// <param name="offset">Offset within the buffer</param>
		/// <param name="length">Length of the string within the buffer</param>
		public Utf8String(byte[] buffer, int offset, int length)
		{
			Memory = new ReadOnlyMemory<byte>(buffer, offset, length);
		}

		/// <summary>
		/// Duplicate this string
		/// </summary>
		/// <returns></returns>
		public Utf8String Clone()
		{
			if (Memory.Length == 0)
			{
				return default;
			}

			byte[] newBuffer = new byte[Memory.Length];
			Memory.CopyTo(newBuffer);
			return new Utf8String(newBuffer);
		}

		/// <summary>
		/// Tests two strings for equality
		/// </summary>
		/// <param name="a">The first string to compare</param>
		/// <param name="b">The second string to compare</param>
		/// <returns>True if the strings are equal</returns>
		public static bool operator ==(Utf8String a, Utf8String b)
		{
			return a.Equals(b);
		}

		/// <summary>
		/// Tests two strings for inequality
		/// </summary>
		/// <param name="a">The first string to compare</param>
		/// <param name="b">The second string to compare</param>
		/// <returns>True if the strings are not equal</returns>
		public static bool operator !=(Utf8String a, Utf8String b)
		{
			return !a.Equals(b);
		}

		/// <inheritdoc/>
		public bool Equals(Utf8String other) => Utf8StringComparer.Ordinal.Equals(Span, other.Span);

		/// <inheritdoc/>
		public int CompareTo(Utf8String other) => Utf8StringComparer.Ordinal.Compare(Span, other.Span);

		/// <inheritdoc/>
		public static bool operator <(Utf8String left, Utf8String right) => left.CompareTo(right) < 0;

		/// <inheritdoc/>
		public static bool operator <=(Utf8String left, Utf8String right) => left.CompareTo(right) <= 0;

		/// <inheritdoc/>
		public static bool operator >(Utf8String left, Utf8String right) => left.CompareTo(right) > 0;

		/// <inheritdoc/>
		public static bool operator >=(Utf8String left, Utf8String right) => left.CompareTo(right) >= 0;

		/// <inheritdoc cref="String.Contains(String)"/>
		public bool Contains(Utf8String str) => IndexOf(str) != -1;

		/// <inheritdoc cref="String.Contains(String, StringComparison)"/>
		public bool Contains(Utf8String str, Utf8StringComparer comparer) => IndexOf(str, comparer) != -1;

		/// <inheritdoc cref="String.IndexOf(Char)"/>
		public int IndexOf(byte character)
		{
			return Span.IndexOf(character);
		}

		/// <inheritdoc cref="String.IndexOf(Char)"/>
		public int IndexOf(char character)
		{
			if (character < 0x80)
			{
				return Span.IndexOf((byte)character);
			}
			else
			{
				return Span.IndexOf(Encoding.UTF8.GetBytes([character]));
			}
		}

		/// <inheritdoc cref="String.IndexOf(Char, Int32)"/>
		public int IndexOf(char character, int index) => IndexOf(character, index, Length - index);

		/// <inheritdoc cref="String.IndexOf(Char, Int32, Int32)"/>
		public int IndexOf(char character, int index, int count)
		{
			int result;
			if (character < 0x80)
			{
				result = Span.Slice(index, count).IndexOf((byte)character);
			}
			else
			{
				result = Span.Slice(index, count).IndexOf(Encoding.UTF8.GetBytes([character]));
			}
			return (result == -1) ? -1 : result + index;
		}

		/// <inheritdoc cref="String.IndexOf(String)"/>
		public int IndexOf(Utf8String str)
		{
			return Span.IndexOf(str.Span);
		}

		/// <inheritdoc cref="String.IndexOf(String, StringComparison)"/>
		public int IndexOf(Utf8String str, Utf8StringComparer comparer)
		{
			for (int idx = 0; idx < Length - str.Length; idx++)
			{
				if (comparer.Equals(str.Slice(idx, str.Length), str))
				{
					return idx;
				}
			}
			return -1;
		}

		/// <inheritdoc cref="String.LastIndexOf(Char)"/>
		public int LastIndexOf(byte character)
		{
			return Span.LastIndexOf(character);
		}

		/// <inheritdoc cref="String.LastIndexOf(Char)"/>
		public int LastIndexOf(char character)
		{
			if (character < 0x80)
			{
				return Span.LastIndexOf((byte)character);
			}
			else
			{
				return Span.LastIndexOf(Encoding.UTF8.GetBytes([character]));
			}
		}

		/// <summary>
		/// Tests if this string starts with another string
		/// </summary>
		/// <param name="other">The string to check against</param>
		/// <returns>True if this string starts with the other string</returns>
		public bool StartsWith(Utf8String other)
		{
			return Span.StartsWith(other.Span);
		}

		/// <summary>
		/// Tests if this string ends with another string
		/// </summary>
		/// <param name="other">The string to check against</param>
		/// <param name="comparer">The string comparer</param>
		/// <returns>True if this string ends with the other string</returns>
		public bool StartsWith(Utf8String other, Utf8StringComparer comparer)
		{
			return Length >= other.Length && comparer.Equals(Slice(0, other.Length), other);
		}

		/// <summary>
		/// Tests if this string ends with another string
		/// </summary>
		/// <param name="other">The string to check against</param>
		/// <returns>True if this string ends with the other string</returns>
		public bool EndsWith(Utf8String other)
		{
			return Span.EndsWith(other.Span);
		}

		/// <summary>
		/// Tests if this string ends with another string
		/// </summary>
		/// <param name="other">The string to check against</param>
		/// <param name="comparer">The string comparer</param>
		/// <returns>True if this string ends with the other string</returns>
		public bool EndsWith(Utf8String other, Utf8StringComparer comparer)
		{
			return Length >= other.Length && comparer.Equals(Slice(Length - other.Length), other);
		}

		/// <inheritdoc cref="Substring(Int32)"/>
		public Utf8String Slice(int start) => Substring(start);

		/// <inheritdoc cref="Substring(Int32, Int32)"/>
		public Utf8String Slice(int start, int count) => Substring(start, count);

		/// <inheritdoc cref="String.Substring(Int32)"/>
		public Utf8String Substring(int start)
		{
			return new Utf8String(Memory.Slice(start));
		}

		/// <inheritdoc cref="String.Substring(Int32, Int32)"/>
		public Utf8String Substring(int start, int count)
		{
			return new Utf8String(Memory.Slice(start, count));
		}

		/// <summary>
		/// Tests if this string is equal to the other object
		/// </summary>
		/// <param name="obj">Object to compare to</param>
		/// <returns>True if the objects are equivalent</returns>
		public override bool Equals(object? obj)
		{
			Utf8String? other = obj as Utf8String?;
			return other != null && Equals(other.Value);
		}

		/// <summary>
		/// Returns the hash code of this string
		/// </summary>
		/// <returns>Hash code for the string</returns>
		public override int GetHashCode() => Utf8StringComparer.Ordinal.GetHashCode(Span);

		/// <summary>
		/// Gets the string represented by this data
		/// </summary>
		/// <returns>The utf-8 string</returns>
		public override string ToString()
		{
			return Encoding.UTF8.GetString(Span);
		}

		/// <summary>
		/// Parse a string as an unsigned integer
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static uint ParseUnsignedInt(Utf8String text)
		{
			ReadOnlySpan<byte> bytes = text.Span;
			if (bytes.Length == 0)
			{
				throw new Exception("Cannot parse empty string as an integer");
			}

			uint value = 0;
			for (int idx = 0; idx < bytes.Length; idx++)
			{
				uint digit = (uint)(bytes[idx] - '0');
				if (digit > 9)
				{
					throw new Exception($"Cannot parse '{text}' as an integer");
				}
				value = (value * 10) + digit;
			}
			return value;
		}

		/// <summary>
		/// Appends two strings
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static Utf8String operator +(Utf8String a, Utf8String b)
		{
			if (a.Length == 0)
			{
				return b;
			}
			if (b.Length == 0)
			{
				return a;
			}

			byte[] buffer = new byte[a.Length + b.Length];
			a.Span.CopyTo(buffer);
			b.Span.CopyTo(buffer.AsSpan(a.Length));
			return new Utf8String(buffer);
		}

		/// <summary>
		/// Implict conversion to a span of bytes
		/// </summary>
		/// <param name="str"></param>
		public static implicit operator ReadOnlySpan<byte>(Utf8String str) => str.Span;
	}

	/// <summary>
	/// Comparison classes for utf8 strings
	/// </summary>
	public abstract class Utf8StringComparer : IEqualityComparer<Utf8String>, IComparer<Utf8String>
	{
		/// <summary>
		/// Ordinal comparer for utf8 strings
		/// </summary>
		sealed class OrdinalComparer : Utf8StringComparer
		{
			/// <inheritdoc/>
			public override bool Equals(ReadOnlySpan<byte> strA, ReadOnlySpan<byte> strB)
			{
				return strA.SequenceEqual(strB);
			}

			/// <inheritdoc/>
			public override int GetHashCode(ReadOnlySpan<byte> str)
			{
				int hash = 5381;
				for (int idx = 0; idx < str.Length; idx++)
				{
					hash += (hash << 5) + str[idx];
				}
				return hash;
			}

			/// <inheritdoc/>
			public override int Compare(ReadOnlySpan<byte> strA, ReadOnlySpan<byte> strB)
			{
				return strA.SequenceCompareTo(strB);
			}
		}

		/// <summary>
		/// Comparison between ReadOnlyUtf8String objects that ignores case for ASCII characters
		/// </summary>
		sealed class OrdinalIgnoreCaseComparer : Utf8StringComparer
		{
			/// <inheritdoc/>
			public override bool Equals(ReadOnlySpan<byte> strA, ReadOnlySpan<byte> strB)
			{
				if (strA.Length != strB.Length)
				{
					return false;
				}

				for (int idx = 0; idx < strA.Length; idx++)
				{
					if (strA[idx] != strB[idx] && ToUpper(strA[idx]) != ToUpper(strB[idx]))
					{
						return false;
					}
				}

				return true;
			}

			/// <inheritdoc/>
			public override int GetHashCode(ReadOnlySpan<byte> str)
			{
				HashCode hashCode = new();
				for (int idx = 0; idx < str.Length; idx++)
				{
					hashCode.Add(ToUpper(str[idx]));
				}
				return hashCode.ToHashCode();
			}

			/// <inheritdoc/>
			public override int Compare(ReadOnlySpan<byte> spanA, ReadOnlySpan<byte> spanB)
			{
				int length = Math.Min(spanA.Length, spanB.Length);
				for (int idx = 0; idx < length; idx++)
				{
					if (spanA[idx] != spanB[idx])
					{
						int upperA = ToUpper(spanA[idx]);
						int upperB = ToUpper(spanB[idx]);
						if (upperA != upperB)
						{
							return upperA - upperB;
						}
					}
				}
				return spanA.Length - spanB.Length;
			}

			/// <summary>
			/// Convert a character to uppercase
			/// </summary>
			/// <param name="character">Character to convert</param>
			/// <returns>The uppercase version of the character</returns>
			static byte ToUpper(byte character)
			{
				return (character >= 'a' && character <= 'z') ? (byte)(character - 'a' + 'A') : character;
			}
		}

		/// <summary>
		/// Static instance of the ordinal utf8 ordinal comparer
		/// </summary>
		public static Utf8StringComparer Ordinal { get; } = new OrdinalComparer();

		/// <summary>
		/// Static instance of the case-insensitive utf8 ordinal string comparer
		/// </summary>
		public static Utf8StringComparer OrdinalIgnoreCase { get; } = new OrdinalIgnoreCaseComparer();

		/// <inheritdoc/>
		public bool Equals(Utf8String strA, Utf8String strB) => Equals(strA.Span, strB.Span);

		/// <inheritdoc/>
		public abstract bool Equals(ReadOnlySpan<byte> strA, ReadOnlySpan<byte> strB);

		/// <inheritdoc/>
		public int GetHashCode(Utf8String str) => GetHashCode(str.Span);

		/// <inheritdoc/>
		public abstract int GetHashCode(ReadOnlySpan<byte> str);

		/// <inheritdoc/>
		public int Compare(Utf8String strA, Utf8String strB) => Compare(strA.Span, strB.Span);

		/// <inheritdoc/>
		public abstract int Compare(ReadOnlySpan<byte> strA, ReadOnlySpan<byte> strB);
	}

	/// <summary>
	/// Extension methods for ReadOnlyUtf8String objects
	/// </summary>
	public static class Utf8StringExtensions
	{
		/// <summary>
		/// Reads a null-terminated utf8 string from the buffer, without copying it
		/// </summary>
		/// <returns>The string data</returns>
		public static Utf8String ReadUtf8String(this IMemoryReader reader)
		{
			return ReadUtf8StringWithoutCopy(reader).Clone();
		}

		/// <summary>
		/// Reads a null-terminated utf8 string from the buffer, without copying it
		/// </summary>
		/// <returns>The string data</returns>
		public static Utf8String ReadUtf8StringWithoutCopy(this IMemoryReader reader)
		{
			return new Utf8String(reader.ReadVariableLengthBytes());
		}

		/// <summary>
		/// Reads a null-terminated utf8 string from the buffer, without copying it
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="str">String to write</param>
		public static void WriteUtf8String(this IMemoryWriter writer, Utf8String str)
		{
			writer.WriteVariableLengthBytes(str.Span);
		}

		/// <summary>
		/// Reads a null-terminated utf8 string from the buffer, without copying it
		/// </summary>
		/// <returns>The string data</returns>
		public static Utf8String ReadNullTerminatedUtf8String(this IMemoryReader reader)
		{
			int minSize = 1;
			for (; ; )
			{
				ReadOnlyMemory<byte> memory = reader.GetMemory(minSize);

				int length = memory.Span.IndexOf((byte)0);
				if (length != -1)
				{
					Utf8String str = new(memory.Slice(0, length));
					reader.Advance(length + 1);
					return str;
				}

				minSize = memory.Length + 1;
			}
		}

		/// <summary>
		/// Writes a null-terminated utf8 string to the buffer
		/// </summary>
		/// <param name="writer">Writer for the output data</param>
		/// <param name="str">String to write</param>
		public static void WriteNullTerminatedUtf8String(this IMemoryWriter writer, Utf8String str)
		{
			Span<byte> span = writer.GetSpan(str.Length + 1);
			str.Span.CopyTo(span);
			span[str.Length] = 0;
			writer.Advance(str.Length + 1);
		}

		/// <summary>
		/// Determines the size of a serialized utf-8 string
		/// </summary>
		/// <param name="str">The string to measure</param>
		/// <returns>Size of the serialized string</returns>
		public static int GetNullTerminatedSize(this Utf8String str)
		{
			return str.Length + 1;
		}
	}
}
