// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;

namespace EpicGames.Core
{
	/// <summary>
	/// Represents a memory region which can be treated as a utf-8 string.
	/// </summary>
	public class Utf8StringBuilder
	{
		readonly ArrayMemoryWriter _writer;

		/// <summary>
		/// Returns the length of this string
		/// </summary>
		public int Length
		{
			get => _writer.Length;
			set
			{
				int prevLength = _writer.Length;
				_writer.Resize(value);

				if (value > prevLength)
				{
					_writer.WrittenSpan.Slice(prevLength).Clear();
				}
			}
		}

		/// <summary>
		/// Accessor for the written span
		/// </summary>
		public Span<byte> WrittenSpan => _writer.WrittenSpan;

		/// <summary>
		/// Accessor for the written memory
		/// </summary>
		public Memory<byte> WrittenMemory => _writer.WrittenMemory;

		/// <summary>
		/// Accessor for the written data
		/// </summary>
		public Utf8String WrittenString => new(_writer.WrittenMemory);

		/// <summary>
		/// Constructor
		/// </summary>
		public Utf8StringBuilder()
			: this(256)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public Utf8StringBuilder(int initialSize)
			: this(new ArrayMemoryWriter(initialSize))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public Utf8StringBuilder(ArrayMemoryWriter writer)
		{
			_writer = writer;
		}

		/// <summary>
		/// Append a character to the end of this builder
		/// </summary>
		/// <param name="ch">Character to append</param>
		public void Append(byte ch)
		{
			_writer.WriteUInt8(ch);
		}

		/// <summary>
		/// Append a character to the end of this builder
		/// </summary>
		/// <param name="ch">Character to append</param>
		public void Append(char ch)
		{
			if (ch < 127)
			{
				Append((byte)ch);
			}
			else
			{
				Append([ch]);
			}
		}

		/// <summary>
		/// Appends a string to the end of this builder
		/// </summary>
		/// <param name="text">Text to append</param>
		public void Append(Utf8String text)
		{
			_writer.WriteFixedLengthBytes(text.Span);
		}

		/// <summary>
		/// Appends an integer to the end of this builder
		/// </summary>
		/// <param name="value">Value to append</param>
		public void Append(int value) => Append((long)value);

		/// <summary>
		/// Appends an integer to the end of this builder
		/// </summary>
		/// <param name="value">Value to append</param>
		public void Append(uint value) => Append((ulong)value);

		/// <summary>
		/// Appends an unsigned long to the end of this builder
		/// </summary>
		/// <param name="value">Value to append</param>
		public void Append(long value)
		{
			if (value < 0)
			{
				_writer.WriteUInt8((byte)'-');
				Append((ulong)-value);
			}
			else
			{
				Append((ulong)value);
			}
		}

		/// <summary>
		/// Appends an unsigned long to the end of this builder
		/// </summary>
		/// <param name="value">Value to append</param>
		public void Append(ulong value)
		{
			Span<byte> span = _writer.GetSpan(32);

			int length = 0;
			for (ulong remaining = value; ; remaining /= 10)
			{
				ulong digit = remaining % 10U;
				span[length++] = (byte)('0' + digit);

				if (remaining == digit)
				{
					break;
				}
			}

			span.Slice(0, length).Reverse();
			_writer.Advance(length);
		}

		/// <summary>
		/// Appends a string to the end of this builder
		/// </summary>
		/// <param name="text">Text to append</param>
		public void Append(ReadOnlySpan<char> text)
		{
			Span<byte> span = _writer.GetSpanAndAdvance(Encoding.UTF8.GetByteCount(text));
			Encoding.UTF8.GetBytes(text, span);
		}

		/// <summary>
		/// Converts the written memory to a utf8 string
		/// </summary>
		/// <returns></returns>
		public Utf8String ToUtf8String()
		{
			return new Utf8String(_writer.WrittenMemory);
		}

		/// <inheritdoc/>
		public override string ToString() => ToUtf8String().ToString();
	}
}
