// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Numerics;

namespace EpicGames.Core
{
	/// <summary>
	/// Methods for reading VarUInt values
	/// </summary>
	public static class VarInt
	{
		/// <summary>
		/// Read a variable-length unsigned integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <returns></returns>
		[Obsolete("Call ReadSigned or ReadUnsigned instead")]
		public static ulong Read(ReadOnlySpan<byte> buffer) => ReadUnsigned(buffer, out _);

		/// <summary>
		/// Read a variable-length unsigned integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <param name="bytesRead">The number of bytes consumed from the input</param>
		/// <returns></returns>
		[Obsolete("Call ReadSigned or ReadUnsigned instead")]
		public static ulong Read(ReadOnlySpan<byte> buffer, out int bytesRead) => ReadUnsigned(buffer, out bytesRead);

		/// <summary>
		/// Read a variable-length signed integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <returns></returns>
		public static long ReadSigned(ReadOnlySpan<byte> buffer) => ReadSigned(buffer, out _);

		/// <summary>
		/// Read a variable-length signed integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <param name="bytesRead">The number of bytes consumed from the input</param>
		/// <returns></returns>
		public static long ReadSigned(ReadOnlySpan<byte> buffer, out int bytesRead)
		{
			ulong unsignedValue = ReadUnsigned(buffer, out bytesRead);
			return DecodeSigned(unsignedValue);
		}

		/// <summary>
		/// Read a variable-length unsigned integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <returns></returns>
		public static ulong ReadUnsigned(ReadOnlySpan<byte> buffer) => ReadUnsigned(buffer, out _);

		/// <summary>
		/// Read a variable-length unsigned integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <param name="bytesRead">The number of bytes consumed from the input</param>
		/// <returns></returns>
		public static ulong ReadUnsigned(ReadOnlySpan<byte> buffer, out int bytesRead)
		{
			bytesRead = (int)Measure(buffer);
			return ReadUnsignedKnownSize(buffer, bytesRead);
		}

		/// <summary>
		/// Read a variable-length unsigned integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an unsigned integer</param>
		/// <param name="numBytes">The number of bytes consumed from the input</param>
		/// <returns></returns>
		internal static ulong ReadUnsignedKnownSize(ReadOnlySpan<byte> buffer, int numBytes)
		{
			ulong value = (ulong)(buffer[0] & (0xff >> numBytes));
			for (int i = 1; i < numBytes; i++)
			{
				value <<= 8;
				value |= buffer[i];
			}
			return value;
		}

		/// <summary>
		/// Measure the length in bytes (1-9) of an encoded variable-length integer.
		/// </summary>
		/// <param name="buffer">A variable-length encoding of an(signed or unsigned) integer.</param>
		/// <returns>The number of bytes used to encode the integer, in the range 1-9.</returns>
		public static int Measure(ReadOnlySpan<byte> buffer)
		{
			byte b = buffer[0];
			b = (byte)~b;
			return BitOperations.LeadingZeroCount(b) - 23;
		}

		/// <summary>
		/// Measure the length in bytes (1-9) of an encoded variable-length integer.
		/// </summary>
		/// <param name="b">First byte of the encoded integer.</param>
		/// <returns>The number of bytes used to encode the integer, in the range 1-9.</returns>
		public static int Measure(byte b)
		{
			b = (byte)~b;
			return BitOperations.LeadingZeroCount(b) - 23;
		}

		/// <summary>
		/// Measure the number of bytes required to encode the input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		[Obsolete("Use MeasureSigned or MeasureUnsigned instead")]
		public static int Measure(uint value) => MeasureUnsigned(value);

		/// <summary>
		/// Measure the number of bytes required to encode the input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		[Obsolete("Use MeasureSigned or MeasureUnsigned instead")]
		public static int Measure(ulong value) => MeasureUnsigned(value);

		/// <summary>
		/// Measure the number of bytes (1-9) required to encode the input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public static int MeasureSigned(long value)
		{
			return MeasureUnsigned(EncodeSigned(value));
		}

		/// <summary>
		/// Measure the number of bytes (1-5) required to encode the 32-bit input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public static int MeasureUnsigned(int value)
		{
			return MeasureUnsigned((ulong)(long)value);
		}

		/// <summary>
		/// Measure the number of bytes (1-5) required to encode the 32-bit input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public static int MeasureUnsigned(uint value)
		{
			return BitOperations.Log2(value) / 7 + 1;
		}

		/// <summary>
		/// Measure the number of bytes (1-9) required to encode the 64-bit input.
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public static int MeasureUnsigned(ulong value)
		{
			return Math.Min(BitOperations.Log2(value) / 7 + 1, 9);
		}

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		[Obsolete("Use WriteUnsigned or WriteSigned instead")]
		public static int Write(Span<byte> buffer, long value) => WriteUnsigned(buffer, (ulong)value);

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		[Obsolete("Use WriteUnsigned or WriteSigned instead")]
		public static int Write(Span<byte> buffer, ulong value) => WriteUnsigned(buffer, value);

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		public static int WriteSigned(Span<byte> buffer, long value)
		{
			ulong unsignedValue = EncodeSigned(value);
			return WriteUnsigned(buffer, unsignedValue);
		}

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		public static int WriteUnsigned(Span<byte> buffer, int value) => WriteUnsigned(buffer, (ulong)value);

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		public static int WriteUnsigned(Span<byte> buffer, long value) => WriteUnsigned(buffer, (ulong)value);

		/// <summary>
		/// Write a variable-length unsigned integer.
		/// </summary>
		/// <param name="value">An unsigned integer to encode</param>
		/// <param name="buffer">A buffer of at least 9 bytes to write the output to.</param>
		/// <returns>The number of bytes used in the output</returns>
		public static int WriteUnsigned(Span<byte> buffer, ulong value)
		{
			int byteCount = MeasureUnsigned(value);

			for (int idx = 1; idx < byteCount; idx++)
			{
				buffer[byteCount - idx] = (byte)value;
				value >>= 8;
			}
			buffer[0] = (byte)((0xff << (9 - (int)byteCount)) | (byte)value);
			return byteCount;
		}

		/// <summary>
		/// Decode a signed VarInt value from an unsigned value
		/// </summary>
		/// <param name="value">Value to decode</param>
		/// <returns>Decoded value</returns>
		public static long DecodeSigned(ulong value)
		{
			return -(long)(value & 1) ^ (long)(value >> 1);
		}

		/// <summary>
		/// Encode a signed VarInt value into an unsigned value
		/// </summary>
		/// <param name="value">Value to encode</param>
		/// <returns>Encoded value</returns>
		public static ulong EncodeSigned(long value)
		{
			return (ulong)((value >> 63) ^ (value << 1));
		}
	}

	/// <summary>
	/// Extension methods for writing VarInt values
	/// </summary>
	public static class VarIntExtensions
	{
		/// <summary>
		/// Read an unsigned VarInt from the given reader
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The deserialized value</returns>
		public static long ReadSignedVarInt(this IMemoryReader reader)
		{
			ulong value = ReadUnsignedVarInt(reader);
			return VarInt.DecodeSigned(value);
		}

		/// <summary>
		/// Read an unsigned VarInt from the given reader
		/// </summary>
		/// <param name="reader">Reader to deserialize from</param>
		/// <returns>The deserialized value</returns>
		public static ulong ReadUnsignedVarInt(this IMemoryReader reader)
		{
			int length = VarInt.Measure(reader.GetSpan(1));
			ReadOnlySpan<byte> span = reader.GetSpan(length);
			reader.Advance(length);
			return VarInt.ReadUnsignedKnownSize(span, length);
		}

		/// <summary>
		/// Writes a signed VarInt to a byte array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteSignedVarInt(this IMemoryWriter writer, int value)
		{
			WriteUnsignedVarInt(writer, VarInt.EncodeSigned(value));
		}

		/// <summary>
		/// Writes a signed VarInt to a byte array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteSignedVarInt(this IMemoryWriter writer, long value)
		{
			WriteUnsignedVarInt(writer, VarInt.EncodeSigned(value));
		}

		/// <summary>
		/// Writes a unsigned VarInt to a byte array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUnsignedVarInt(this IMemoryWriter writer, int value) => WriteUnsignedVarInt(writer, (ulong)value);

		/// <summary>
		/// Writes a unsigned VarInt to a byte array
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		/// <param name="value">Value to write</param>
		public static void WriteUnsignedVarInt(this IMemoryWriter writer, ulong value)
		{
			int length = VarInt.MeasureUnsigned(value);
			VarInt.WriteUnsigned(writer.GetSpan(length), value);
			writer.Advance(length);
		}
	}
}
