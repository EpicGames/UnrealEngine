// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Core
{
	/// <summary>
	/// Stores a numeric version consisting of any number of components.
	/// </summary>
	[Serializable]
	[JsonConverter(typeof(VersionNumberJsonConverter))]
	public class VersionNumber : IComparable<VersionNumber>
	{
		/// <summary>
		/// Set of delimiters for version numbers
		/// </summary>
		static readonly char[] s_delimiters = ['.', ','];

		/// <summary>
		/// The individual version components
		/// </summary>
		public IReadOnlyList<int> Components => _components;

		readonly int[] _components;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="components">The individual version components. At least one value must be given.</param>
		public VersionNumber(params int[] components)
		{
			if (components.Length == 0)
			{
				throw new InvalidOperationException("Version number must have at least one component");
			}
			_components = components;
		}

		/// <summary>
		/// Returns the component at the given index
		/// </summary>
		/// <param name="idx">The zero-based component index to return</param>
		/// <returns>The component at the given index</returns>
		public int GetComponent(int idx)
		{
			return _components[idx];
		}

		/// <summary>
		/// Tests two objects for equality. VersionNumber behaves like a value type.
		/// </summary>
		/// <param name="obj">Object to compare against</param>
		/// <returns>True if the objects are equal, false otherwise.</returns>
		public override bool Equals(object? obj)
		{
			VersionNumber? version = obj as VersionNumber;
			return version is not null && this == version;
		}

		/// <summary>
		/// Returns a hash of the version number.
		/// </summary>
		/// <returns>A hash value for the version number.</returns>
		public override int GetHashCode()
		{
			int result = 5831;
			for (int idx = 0; idx < _components.Length; idx++)
			{
				result = (result * 33) + _components[idx];
			}
			return result;
		}

		/// <summary>
		/// Compares whether two versions are equal.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the versions are equal.</returns>
		public static bool operator ==(VersionNumber? lhs, VersionNumber? rhs)
		{
			return lhs is null ? rhs is null : rhs is not null && Compare(lhs, rhs) == 0;
		}

		/// <summary>
		/// Compares whether two versions are not equal.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the versions are not equal.</returns>
		public static bool operator !=(VersionNumber? lhs, VersionNumber? rhs)
		{
			return !(lhs == rhs);
		}

		/// <summary>
		/// Compares whether one version is less than another.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the first version is less than the second.</returns>
		public static bool operator <(VersionNumber lhs, VersionNumber rhs)
		{
			return Compare(lhs, rhs) < 0;
		}

		/// <summary>
		/// Compares whether one version is less or equal to another.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the first version is less or equal to the second.</returns>
		public static bool operator <=(VersionNumber lhs, VersionNumber rhs)
		{
			return Compare(lhs, rhs) <= 0;
		}

		/// <summary>
		/// Compares whether one version is greater than another.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the first version is greater than the second.</returns>
		public static bool operator >(VersionNumber lhs, VersionNumber rhs)
		{
			return Compare(lhs, rhs) > 0;
		}

		/// <summary>
		/// Compares whether one version is greater or equal to another.
		/// </summary>
		/// <param name="lhs">The first version number</param>
		/// <param name="rhs">The second version number</param>
		/// <returns>True if the first version is greater or equal to the second.</returns>
		public static bool operator >=(VersionNumber lhs, VersionNumber rhs)
		{
			return Compare(lhs, rhs) >= 0;
		}

		/// <summary>
		/// Comparison function for IComparable
		/// </summary>
		/// <param name="other">Other version number to compare to</param>
		/// <returns>A negative value if this version is before Other, a positive value if this version is after Other, and zero otherwise.</returns>
		public int CompareTo(VersionNumber? other)
		{
			return other is null ? 1 : Compare(this, other);
		}

		/// <summary>
		/// Compares two version numbers and returns an integer indicating their order
		/// </summary>
		/// <param name="lhs">The first version to check</param>
		/// <param name="rhs">The second version to check</param>
		/// <returns>A negative value if Lhs is before Rhs, a positive value if Lhs is after Rhs, and zero otherwise.</returns>
		public static int Compare(VersionNumber lhs, VersionNumber rhs)
		{
			for (int idx = 0; ; idx++)
			{
				if (idx == lhs._components.Length)
				{
					if (idx == rhs._components.Length)
					{
						return 0;
					}
					else
					{
						return -1;
					}
				}
				else
				{
					if (idx == rhs._components.Length)
					{
						return +1;
					}
					else if (lhs._components[idx] != rhs._components[idx])
					{
						return lhs._components[idx] - rhs._components[idx];
					}
				}
			}
		}

		/// <summary>
		/// Parses the version number from a string
		/// </summary>
		/// <param name="text">The string to parse</param>
		/// <returns>A version number object</returns>
		public static VersionNumber Parse(string text)
		{
			List<int> components = [];
			foreach (string textElement in text.Split(s_delimiters))
			{
				components.Add(Int32.Parse(textElement));
			}
			return new VersionNumber([.. components]);
		}

		/// <summary>
		/// Parses the version number from a string
		/// </summary>
		/// <param name="text">The string to parse</param>
		/// <param name="outNumber">Variable to receive the parsed version number</param>
		/// <returns>A version number object</returns>
		public static bool TryParse(string text, [NotNullWhen(true)] out VersionNumber? outNumber)
		{
			List<int> components = [];
			foreach (string textElement in text.Split(s_delimiters))
			{
				int component;
				if (!Int32.TryParse(textElement, out component))
				{
					outNumber = null;
					return false;
				}
				components.Add(component);
			}

			outNumber = new VersionNumber([.. components]);
			return true;
		}

		/// <summary>
		/// Returns a string version number, eg. 1.4
		/// </summary>
		/// <returns>The stringized version number</returns>
		public override string ToString()
		{
			StringBuilder result = new();
			if (_components.Length > 0)
			{
				result.Append(_components[0]);
				for (int idx = 1; idx < _components.Length; idx++)
				{
					result.Append('.');
					result.Append(_components[idx]);
				}
			}
			return result.ToString();
		}
	}

	internal class VersionNumberJsonConverter : JsonConverter<VersionNumber>
	{
		public override VersionNumber? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? versionNumber = reader.GetString();
			return versionNumber is null ? null : VersionNumber.Parse(versionNumber);
		}

		public override VersionNumber ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.PropertyName)
			{
				throw new JsonException("VersionNumber keys must be represented as JSON strings");
			}
			return VersionNumber.Parse(reader.GetString()!);
		}

		public override void Write(Utf8JsonWriter writer, VersionNumber value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}

		public override void WriteAsPropertyName(Utf8JsonWriter writer, [DisallowNull] VersionNumber value, JsonSerializerOptions options)
		{
			writer.WritePropertyName(value.ToString());
		}
	}
}
