// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Core
{
	/// <summary>
	/// Range of version numbers
	/// </summary>
	public class VersionNumberRange
	{
		/// <summary>
		/// Minimum version number
		/// </summary>
		public VersionNumber Min { get; }

		/// <summary>
		/// Maximum version number
		/// </summary>
		public VersionNumber Max { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="min"></param>
		/// <param name="max"></param>
		public VersionNumberRange(VersionNumber min, VersionNumber max)
		{
			Min = min;
			Max = max;
		}

		/// <summary>
		/// Tests whether this range contains the given version
		/// </summary>
		/// <param name="version"></param>
		/// <returns></returns>
		public bool Contains(VersionNumber version)
		{
			return version >= Min && version <= Max;
		}

		/// <summary>
		/// Parse a version range from two strings
		/// </summary>
		/// <param name="minText"></param>
		/// <param name="maxText"></param>
		/// <returns></returns>
		public static VersionNumberRange Parse(string minText, string maxText)
		{
			return new VersionNumberRange(VersionNumber.Parse(minText), VersionNumber.Parse(maxText));
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return String.Format("{0}-{1}", Min, Max);
		}
	}
}
