// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;

namespace EpicGames.Core
{
	/// <summary>
	/// Wrapper for Win32Exception which includes the error code in the exception message
	/// </summary>
	class Win32ExceptionWithCode : Win32Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="code">The Windows error code</param>
		public Win32ExceptionWithCode(int code)
			: base(code)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message to display</param>
		public Win32ExceptionWithCode(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="code">The Windows error code</param>
		/// <param name="message">Message to display</param>
		public Win32ExceptionWithCode(int code, string message)
			: base(code, message)
		{
		}

		/// <summary>
		/// Returns the exception message. Overriden to include the error code in the message.
		/// </summary>
		public override string Message => String.Format("{0} (code 0x{1:X8})", base.Message, NativeErrorCode);
	}
}
