// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Text.RegularExpressions;
using System.Xml;

namespace EpicGames.Core
{
	/// <summary>
	/// Cache for XML documentation
	/// </summary>
	public class XmlDocReader
	{
		readonly ConcurrentDictionary<Assembly, XmlDocument?> _cachedDocumentation = new();

		/// <summary>
		/// Adds a parsed file as the documentation for a particular assembly
		/// </summary>
		public void Add(Assembly assembly, XmlDocument? xmlDoc)
		{
			_cachedDocumentation[assembly] = xmlDoc;
		}

		/// <summary>
		/// Gets a property description from Xml documentation file
		/// </summary>
		public string? GetDescription(Type type)
		{
			return GetSummaryFromXmlDoc(type, "T", null);
		}

		/// <summary>
		/// Gets a property description from Xml documentation file
		/// </summary>
		public string? GetDescription(PropertyInfo propertyInfo)
		{
			return GetSummaryFromXmlDoc(propertyInfo.DeclaringType!, "P", propertyInfo.Name);
		}

		/// <summary>
		/// Gets a property description from Xml documentation file
		/// </summary>
		public string? GetDescription(Type type, string field)
		{
			return GetSummaryFromXmlDoc(type, "F", field);
		}

		/// <summary>
		/// Gets a description from Xml documentation file
		/// </summary>
		/// <param name="type">Type to retrieve summary for</param>
		/// <param name="qualifier">Type of element to retrieve</param>
		/// <param name="member">Name of the member</param>
		/// <returns>Summary string, or null if it's not available</returns>
		string? GetSummaryFromXmlDoc(Type type, string qualifier, string? member)
		{
			XmlDocument? xmlDoc;
			if (!TryReadXmlDoc(type.Assembly, out xmlDoc))
			{
				return null;
			}

			string? fullName = type.FullName;
			if (member != null)
			{
				fullName = $"{fullName}.{member}";
			}

			XmlNode? node = xmlDoc.SelectSingleNode($"//member[@name='{qualifier}:{fullName}']");
			if (node == null)
			{
				return null;
			}

			XmlNode? summaryNode = node.SelectSingleNode("summary");
			if (summaryNode == null)
			{
				if (node.SelectSingleNode("inheritdoc") != null)
				{
					foreach (Type baseType in FindBaseTypes(type))
					{
						string? summary = GetSummaryFromXmlDoc(baseType, qualifier, member);
						if (summary != null)
						{
							return summary;
						}
					}
				}
				return null;
			}

			string text = summaryNode.InnerText.Trim().Replace("\r\n", "\n", StringComparison.Ordinal);
			text = Regex.Replace(text, @"[\t ]*\n[\t ]*", "\n");
			text = Regex.Replace(text, @"(?<!\n)\n(?!\n)", " ");
			text = Regex.Replace(text, @"\n+", "\n");
			return text;
		}

		static IEnumerable<Type> FindBaseTypes(Type type)
		{
			if (type.BaseType != null)
			{
				yield return type.BaseType;
			}

			foreach (Type interfaceType in type.GetInterfaces())
			{
				yield return interfaceType;
			}
		}

		bool TryReadXmlDoc(Assembly assembly, [NotNullWhen(true)] out XmlDocument? xmlDoc)
		{
			XmlDocument? documentation;
			if (!_cachedDocumentation.TryGetValue(assembly, out documentation))
			{
				FileReference inputDocumentationFile = new FileReference(assembly.Location).ChangeExtension(".xml");
				if (FileReference.Exists(inputDocumentationFile))
				{
					documentation = new XmlDocument();
					documentation.Load(inputDocumentationFile.FullName);
				}
				_cachedDocumentation.TryAdd(assembly, documentation);
			}

			if (documentation != null)
			{
				xmlDoc = documentation;
				return true;
			}
			else
			{
				xmlDoc = null;
				return false;
			}
		}
	}
}
