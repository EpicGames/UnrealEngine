// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.IO.Compression;
using System.Runtime.InteropServices;

#pragma warning disable CA1707 // Identifiers should not contain underscores

namespace EpicGames.Core
{
	/// <summary>
	/// Additional functionality around <see cref="System.IO.Compression.ZipArchive"/> to support non-Windows filesystems
	/// </summary>
	public static class ZipArchiveExtensions
	{
		/// <summary>
		/// Create a zip archive entry, preserving platform mode bits
		/// </summary>
		/// <param name="destination"></param>
		/// <param name="sourceFileName"></param>
		/// <param name="entryName"></param>
		/// <param name="compressionLevel"></param>
		/// <returns></returns>
		public static ZipArchiveEntry CreateEntryFromFile_CrossPlatform(this ZipArchive destination, string sourceFileName, string entryName, CompressionLevel compressionLevel)
		{
			ZipArchiveEntry entry = ZipFileExtensions.CreateEntryFromFile(destination, sourceFileName, entryName, compressionLevel);
			int result = -1;

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				result = FileUtils.GetFileMode_Linux(sourceFileName);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				result = FileUtils.GetFileMode_Mac(sourceFileName);
			}

			if (result >= 0)
			{
				entry.ExternalAttributes = (int)result << 16;
			}

			return entry;
		}

		/// <summary>
		/// Extract a zip archive entry, preserving platform mode bits
		/// </summary>
		/// <param name="entry"></param>
		/// <param name="targetFileName"></param>
		/// <param name="overwrite"></param>
		public static void ExtractToFile_CrossPlatform(this ZipArchiveEntry entry, string targetFileName, bool overwrite)
		{
			// This seems to be consistently at least 35% faster than ZipFileExtensions.ExtractToFile(Entry, TargetFileName, Overwrite) when the
			// archive contains many small files.
			using (FileStream outputStream = new(targetFileName, overwrite ? FileMode.Create : FileMode.CreateNew, FileAccess.Write))
			{
				using (Stream inputStream = entry.Open())
				{
					inputStream.CopyTo(outputStream);
				}
			}

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				int mode = entry.ExternalAttributes >> 16;
				if (mode != 0)
				{
					FileUtils.SetFileMode_Linux(targetFileName, (ushort)mode);
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				int mode = entry.ExternalAttributes >> 16;
				if (mode != 0)
				{
					FileUtils.SetFileMode_Mac(targetFileName, (ushort)mode);
				}
			}
		}
	}
}
