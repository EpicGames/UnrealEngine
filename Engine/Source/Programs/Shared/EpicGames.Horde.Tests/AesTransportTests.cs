// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.IO;
using System.IO.Pipelines;
using System.Linq;
using System.Net.Sockets;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Transports;
using EpicGames.Horde.Tests.Compute;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests;

[TestClass]
public sealed class AesTransportTests : IAsyncDisposable
{
	private const int SmallBufferThreshold = 1000;
	private const int MinChunkSizeForLargeBuffers = 10;
	private readonly byte[] _key;
	private readonly MemoryComputeTransport _memoryTransport;
	private readonly AesTransport _aesTransport;
	private readonly Random _random = new(42);
	private readonly int[] _bufferSizes = [1, 2, 3, 10, 200000];

	public AesTransportTests()
	{
		_key = AesTransport.CreateKey();
		_memoryTransport = new MemoryComputeTransport();
		_aesTransport = new AesTransport(_memoryTransport, _key);
	}

	public async ValueTask DisposeAsync()
	{
		await _memoryTransport.DisposeAsync();
		await _aesTransport.DisposeAsync();
	}

	[TestMethod]
	public async Task SendAndReceive_VariousBufferSizesOverTcp_SuccessfullyTransferredAsync()
	{
		using CancellationTokenSource cts = new(ComputeSocketTests.TestTimoutMs);
		(Socket clientSocket, Socket serverSocket) = await ComputeSocketTests.CreateSocketsAsync(cts.Token);
		await using TcpTransport clientTransport = new(clientSocket);
		await using TcpTransport serverTransport = new(serverSocket);
		await using AesTransport encryptedClient = new(clientTransport, _key);
		await using AesTransport encryptedServer = new(serverTransport, _key);

		foreach (int bufferSize in _bufferSizes)
		{
			await SendAndReceiveAsync(encryptedClient, encryptedServer, bufferSize, cts.Token);
		}
	}

	[TestMethod]
	public async Task SendAndReceive_VariousBufferSizesInMemory_SuccessfullyTransferredAsync()
	{
		using CancellationTokenSource cts = new(ComputeSocketTests.TestTimoutMs);
		await using MemoryComputeTransport memTransport = new();
		await using AesTransport encryptedClient = new(memTransport, _key);
		await using AesTransport encryptedServer = new(memTransport, _key);

		foreach (int bufferSize in _bufferSizes)
		{
			await SendAndReceiveAsync(encryptedClient, encryptedServer, bufferSize, cts.Token);
		}
	}

	[TestMethod]
	public async Task SendAndReceive_Torture_SuccessfullyTransferredAsync()
	{
		using CancellationTokenSource cts = new(ComputeSocketTests.TestTimoutMs);
		await using MemoryComputeTransport memTransport = new();
		await using AesTransport clientAes = new(memTransport, _key);
		await using AesTransport serverAes = new(memTransport, _key);

		foreach (int bufferSize in _bufferSizes)
		{
			List<int> evilChunkSizes = [1, 2, 3, bufferSize - 1, bufferSize, bufferSize + 1, bufferSize * 5];
			evilChunkSizes = [.. evilChunkSizes
				.Where(size => size > 0)
				.Where(size => bufferSize < SmallBufferThreshold || size >= MinChunkSizeForLargeBuffers)];

			foreach (int sendChunk in evilChunkSizes)
			{
				foreach (int recvChunk in evilChunkSizes)
				{
					await SendAndMultipleReceiveAsync(clientAes, serverAes, bufferSize, () => sendChunk, () => recvChunk, cts.Token);
				}
			}
		}
	}

	private async Task SendAndReceiveAsync(AesTransport client, AesTransport server, int bufferSize, CancellationToken cancellationToken)
	{
		// Arrange
		byte[] sentData = new byte[bufferSize];
		_random.NextBytes(sentData);
		byte[] serverReceivedData = new byte[sentData.Length];
		byte[] clientReceivedData = new byte[sentData.Length];

		// Act
		await client.SendAsync(sentData, cancellationToken);
		await server.SendAsync(sentData, cancellationToken);
		int serverBytesRead = await server.RecvAsync(serverReceivedData, cancellationToken);
		int clientBytesRead = await client.RecvAsync(clientReceivedData, cancellationToken);

		// Assert
		Assert.AreEqual(sentData.Length, serverBytesRead);
		Assert.AreEqual(sentData.Length, clientBytesRead);
		CollectionAssert.AreEqual(sentData, serverReceivedData);
		CollectionAssert.AreEqual(sentData, clientReceivedData);
	}

	private async Task SendAndMultipleReceiveAsync(ComputeTransport client, ComputeTransport server, int bufferSize,
		Func<int> getSendChunkSize, Func<int> getRecvChunkSize, CancellationToken cancellationToken)
	{
		// Arrange
		byte[] testData = new byte[bufferSize];
		_random.NextBytes(testData);
		using MemoryStream received = new(testData.Length);
		int bytesSent = 0;

		// Act
		Task sendTask = Task.Run(async () =>
		{
			while (bytesSent < testData.Length)
			{
				int bytesLeft = testData.Length - bytesSent;
				int bytesToSend = Math.Min(getSendChunkSize(), bytesLeft);
				await client.SendAsync(testData.AsMemory(bytesSent, bytesToSend), cancellationToken);
				bytesSent += bytesToSend;
			}
		}, cancellationToken);

		Task recvTask = Task.Run(async () =>
		{
			while (received.Length < testData.Length)
			{
				byte[] chunk = new byte[getRecvChunkSize()];
				int bytesRead = await server.RecvAsync(chunk, cancellationToken);
				await received.WriteAsync(chunk, 0, bytesRead, cancellationToken);
			}
		}, cancellationToken);

		await Task.WhenAll(sendTask, recvTask);

		// Assert
		Assert.AreEqual(testData.Length, received.Length);
		CollectionAssert.AreEqual(testData, received.ToArray());
	}

	[TestMethod]
	public async Task Constructor_InvalidKeyLength_ThrowsArgumentExceptionAsync()
	{
		await Assert.ThrowsExactlyAsync<ArgumentException>(async () =>
		{
			byte[] invalidKey = new byte[16];
			await using AesTransport _ = new(_memoryTransport, invalidKey);
		});
	}

	[TestMethod]
	public async Task MarkComplete_PropagatedToInnerTransportAsync()
	{
		// Act
		await _aesTransport.MarkCompleteAsync(CancellationToken.None);

		// Assert
		Assert.IsTrue(_memoryTransport.IsCompleted);
	}
}

// Helper class for testing
internal class MemoryComputeTransport : ComputeTransport
{
	private readonly Pipe _pipe;
	public bool IsCompleted { get; private set; }

	public MemoryComputeTransport()
	{
		PipeOptions options = new(pauseWriterThreshold: 10 * 1024 * 1024); // Must fit buffer sizes being tested
		_pipe = new Pipe(options);
	}

	/// <inheritdoc/>
	public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
	{
		ReadResult result = await _pipe.Reader.ReadAsync(cancellationToken);

		if (result.Buffer.IsEmpty && result.IsCompleted)
		{
			return 0;
		}

		int bytesToCopy = (int)Math.Min(buffer.Length, result.Buffer.Length);
		result.Buffer.Slice(0, bytesToCopy).CopyTo(buffer.Span);
		_pipe.Reader.AdvanceTo(result.Buffer.GetPosition(bytesToCopy));

		return bytesToCopy;
	}

	/// <inheritdoc/>
	public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
	{
		foreach (ReadOnlyMemory<byte> segment in buffer)
		{
			FlushResult result = await _pipe.Writer.WriteAsync(segment, cancellationToken);
			if (result.IsCompleted)
			{
				break;
			}
		}
	}

	/// <inheritdoc/>
	public override async ValueTask MarkCompleteAsync(CancellationToken cancellationToken)
	{
		IsCompleted = true;
		await _pipe.Writer.CompleteAsync();
	}

	/// <inheritdoc/>
	public override async ValueTask DisposeAsync()
	{
		await _pipe.Writer.CompleteAsync();
		await _pipe.Reader.CompleteAsync();
		GC.SuppressFinalize(this);
	}
}