// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Commits;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Commits
{
	[TestClass]
	public class CommitTagTests
	{
		[TestMethod]
		public void TestRobomergeTags()
		{
			string input = """
				#ROBOMERGE-OWNER: foo
				#ROBOMERGE-SOURCE: sourceBranch
				#ROBOMERGE-AUTHOR: bar
				#ROBOMERGE-BOT: robo
				""";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("foo", result.RobomergeOwnerTag);
			Assert.AreEqual("sourceBranch", result.RobomergeSourceTag);
			Assert.AreEqual("bar", result.RobomergeAuthorTag);
			Assert.AreEqual("robo", result.RobomergeBotTag);
		}

		[TestMethod]
		public void TestJiraTag()
		{
			string input = "#jira UE-12345 ";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("UE-12345", result.JiraTag);
		}

		[TestMethod]
		public void TestTestTag()
		{
			string input = "#tests SomeTestSuiteHere";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("SomeTestSuiteHere", result.TestTag);
		}

		[TestMethod]
		public void TestPresubmitTag()
		{
			string input = "#preflight CL1234";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("CL1234", result.PresubmitTag);
		}

		[TestMethod]
		public void TestReviewTag()
		{
			string input = "#rb test.user";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("test.user", result.ReviewTag);
		}

		[TestMethod]
		public void TestPreflightTag()
		{
			string input = "#preflight 6927ddf0734f9d0953fc3be5";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("6927ddf0734f9d0953fc3be5", result.PresubmitTag);
		}

		[TestMethod]
		public void TestVirtualizedTag()
		{
			string input = "#virtualized";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("true", result.VirtualizedTag);

			input = "#nonvirtualized";

			result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("false", result.VirtualizedTag);
		}

		[TestMethod]
		public void TestTagsAsSuffix()
		{
			string input = """
				This line should not matter
				Neither should this
				#jira JIRA-999
				""";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("JIRA-999", result.JiraTag);
		}

		[TestMethod]
		public void TestMissingTagsAsNull()
		{
			string input = "#jira JIRA-777";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("JIRA-777", result.JiraTag);
			Assert.IsNull(result.RobomergeOwnerTag);
			Assert.IsNull(result.RobomergeSourceTag);
			Assert.IsNull(result.RobomergeBotTag);
			Assert.IsNull(result.RobomergeAuthorTag);
			Assert.IsNull(result.TestTag);
			Assert.IsNull(result.PresubmitTag);
			Assert.IsNull(result.ReviewTag);
		}

		[TestMethod]
		public void TestMultiTagsMixedOrder()
		{
			string input = """
				Random stuff\n
				#tests TestSuite
				More random stuff
				#ROBOMERGE-AUTHOR: botUser
				#rb test.user
				#jira J-123
				#preflight P123
				#ROBOMERGE-SOURCE: main
				""";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("TestSuite", result.TestTag);
			Assert.AreEqual("botUser", result.RobomergeAuthorTag);
			Assert.AreEqual("test.user", result.ReviewTag);
			Assert.AreEqual("J-123", result.JiraTag);
			Assert.AreEqual("P123", result.PresubmitTag);
			Assert.AreEqual("main", result.RobomergeSourceTag);
		}

		[TestMethod]
		public void TestTagsAsFrontload()
		{
			string input = """
				#ROBOMERGE-OWNER: userA
				#ROBOMERGE-SOURCE: dev
				#ROBOMERGE-AUTHOR: userB
				#ROBOMERGE-BOT: rb
				#jira XYZ-1
				#preflight P1
				#tests T1
				#rb R1
				# This line simulates extra junk after all tags.
				""";

			CommitDescriptionTagContainer result = CommitDescriptionTagContainer.Create(input);

			Assert.AreEqual("userA", result.RobomergeOwnerTag);
			Assert.AreEqual("dev", result.RobomergeSourceTag);
			Assert.AreEqual("userB", result.RobomergeAuthorTag);
			Assert.AreEqual("rb", result.RobomergeBotTag);
			Assert.AreEqual("XYZ-1", result.JiraTag);
			Assert.AreEqual("P1", result.PresubmitTag);
			Assert.AreEqual("T1", result.TestTag);
			Assert.AreEqual("R1", result.ReviewTag);
		}

		/// <summary>
		/// Perfomrance test.
		/// </summary>
		/// <remarks> CachedCommitDoc refers to descriptions being possibly 128kb. This test aims for this.</remarks>
		[TestMethod]
		[Ignore]
		public void TestPerformance()
		{
			string baseLorem =
				"Lorem ipsum dolor sit amet, consectetur adipiscing elit.\n" +
				"Lorem ipsum dolor sit amet, consectetur adipiscing elit.\n" +
				"Sed viverra, justo sed ultrices consectetur, lorem erat posuere nibh,\n" +
				"a luctus turpis mi sed magna. Integer a nisl non tortor tincidunt\n" +
				"sollicitudin. Pellentesque habitant morbi tristique senectus et netus\n" +
				"et malesuada fames ac turpis egestas.\n" +
				"\n" +
				"Suspendisse potenti. Curabitur vel sapien ut erat tempor fermentum.\n" +
				"Maecenas hendrerit augue id nisi pulvinar, nec ultrices sem tincidunt.\n" +
				"Morbi vel malesuada erat. Vestibulum in ultricies tortor. Nunc tempor\n" +
				"arcu sed leo commodo, sed bibendum risus mattis. Donec non enim ipsum.\n" +
				"\n" +
				"Phasellus ac rhoncus lacus. Sed imperdiet, tortor ac accumsan finibus,\n" +
				"nibh nulla sodales tortor, vitae hendrerit lacus lorem non odio.\n" +
				"Vivamus tincidunt nibh non arcu tincidunt, sed feugiat tortor condimentum.\n" +
				"Aenean blandit magna a faucibus viverra. Duis ac eros sed nunc ullamcorper\n" +
				"ultrices quis nec erat.\n" +
				"\n" +
				"Nullam tempor, odio sit amet convallis finibus, sapien lorem ultrices orci,\n" +
				"eu interdum urna lorem eget massa. Sed aliquet faucibus quam, sed cursus\n" +
				"arcu viverra non. Nulla facilisi. In rutrum semper congue. Curabitur\n" +
				"vulputate libero vitae lacinia cursus.\n";

			// Target size in bytes
			int targetSize = 256 * 1024; // 256kb KB - double the current max size for CachedCommitDoc.

			System.Text.StringBuilder sb = new(targetSize);

			while (sb.Length < targetSize)
			{
				sb.Append(baseLorem);
			}

			string largeLorem = sb.ToString();
			string input = largeLorem + """
				Multiline long content
				#ROBOMERGE-OWNER: userA
				#ROBOMERGE-SOURCE: dev
				#ROBOMERGE-AUTHOR: userB
				#ROBOMERGE-BOT: rb
				#jira XYZ-1
				#preflight P1
				#tests T1
				#rb R1
				# This line simulates extra junk after all tags.
				""";

			// Prewarm
			_ = CommitDescriptionTagContainer.Create(input);

			const int Iterations = 100000;

			CommitDescriptionTagContainer? last = null;
			Random rng = new();

			for (int i = 0; i < Iterations; i++)
			{
				// Invalidate cache hits
				string modifiedInput = input + rng.Next();
				last = CommitDescriptionTagContainer.Create(modifiedInput);
			}

			Assert.AreEqual("userA", last!.Value.RobomergeOwnerTag);
			Assert.AreEqual("dev", last!.Value.RobomergeSourceTag);
			Assert.AreEqual("userB", last!.Value.RobomergeAuthorTag);
			Assert.AreEqual("rb", last!.Value.RobomergeBotTag);
			Assert.AreEqual("XYZ-1", last!.Value.JiraTag);
			Assert.AreEqual("P1", last!.Value.PresubmitTag);
			Assert.AreEqual("T1", last!.Value.TestTag);
			Assert.AreEqual("R1", last!.Value.ReviewTag);
		}
	}
}
