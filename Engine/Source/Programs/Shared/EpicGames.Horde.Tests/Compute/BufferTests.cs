// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO.Pipelines;
using System.Linq;
using System.Runtime.InteropServices;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Buffers;
using EpicGames.Horde.Compute.Transports;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute
{
	[TestClass]
	public class BufferTests
	{
		const int ChannelId = 0;

		[TestMethod]
		public async Task TestSimpleBufferAsync()
		{
			using PooledBuffer buffer = new(2, 1024);
			using ComputeBufferWriter bufferWriter = buffer.CreateWriter();
			bufferWriter.AdvanceWritePosition(10);

			using ComputeBufferReader bufferReader = buffer.CreateReader();
			await bufferReader.WaitToReadAsync(9);
			bufferReader.AdvanceReadPosition(9);
			await bufferReader.WaitToReadAsync(1);
		}

		[TestMethod]
		public async Task TestOverflowAsync()
		{
			using PooledBuffer buffer = new(2, 20);
			using ComputeBufferReader bufferReader = buffer.CreateReader();
			using ComputeBufferWriter bufferWriter = buffer.CreateWriter();

			// Fill up the first chunk
			Assert.AreEqual(20, bufferWriter.GetWriteBuffer().Length);
			bufferWriter.AdvanceWritePosition(10);
			Assert.AreEqual(10, bufferWriter.GetWriteBuffer().Length);
			bufferWriter.AdvanceWritePosition(10);
			Assert.AreEqual(0, bufferWriter.GetWriteBuffer().Length);

			Task waitToWriteTask = bufferWriter.WaitToWriteAsync(1).AsTask();
			Assert.IsTrue(waitToWriteTask.IsCompleted);

			// Fill up the second chunk
			Assert.AreEqual(20, bufferWriter.GetWriteBuffer().Length);
			bufferWriter.AdvanceWritePosition(10);
			Assert.AreEqual(10, bufferWriter.GetWriteBuffer().Length);
			bufferWriter.AdvanceWritePosition(10);
			Assert.AreEqual(0, bufferWriter.GetWriteBuffer().Length);

			waitToWriteTask = bufferWriter.WaitToWriteAsync(1).AsTask();
			Assert.IsFalse(waitToWriteTask.IsCompleted);

			// Wait for data to be read
			Assert.AreEqual(20, bufferReader.GetReadBuffer().Length);
			bufferReader.AdvanceReadPosition(10);
			Assert.IsFalse(waitToWriteTask.IsCompleted);

			Assert.AreEqual(10, bufferReader.GetReadBuffer().Length);
			bufferReader.AdvanceReadPosition(10);
			Assert.AreEqual(0, bufferReader.GetReadBuffer().Length);

			Task waitToReadTask = bufferReader.WaitToReadAsync(1).AsTask();
			Assert.IsTrue(waitToReadTask.IsCompleted);

			Assert.AreEqual(20, bufferReader.GetReadBuffer().Length);
			await waitToWriteTask;

			// Make sure both reader and writer have something to work with
			Assert.AreEqual(20, bufferReader.GetReadBuffer().Length);
			Assert.AreEqual(20, bufferWriter.GetWriteBuffer().Length);
		}

		[TestMethod]
		public async Task TestPooledBufferAsync()
		{
			await TestProducerConsumerAsync(length => new PooledBuffer(length), CancellationToken.None);
		}

		[TestMethod]
		public async Task TestSharedMemoryBufferAsync()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				await TestProducerConsumerAsync(length => SharedMemoryBuffer.CreateNew(null, length), CancellationToken.None);
			}
		}

		static async Task TestProducerConsumerAsync(Func<int, ComputeBuffer> createBuffer, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			const int Length = 8000;

			Pipe sourceToTargetPipe = new();
			Pipe targetToSourcePipe = new();

			await using PipeTransport producerTransport = new(targetToSourcePipe.Reader, sourceToTargetPipe.Writer);
			await using PipeTransport consumerTransport = new(sourceToTargetPipe.Reader, targetToSourcePipe.Writer);
			await using RemoteComputeSocket producerSocket = new(producerTransport, ComputeProtocol.Latest, NullLogger.Instance);
			await using RemoteComputeSocket consumerSocket = new(consumerTransport, ComputeProtocol.Latest, NullLogger.Instance);

			using ComputeBuffer consumerBuffer = createBuffer(Length);
			consumerSocket.AttachRecvBuffer(ChannelId, consumerBuffer);

			using ComputeBuffer producerBuffer = createBuffer(Length);
			producerSocket.AttachSendBuffer(ChannelId, producerBuffer);

			byte[] input = RandomNumberGenerator.GetBytes(Length);
			Task producerTask = RunProducerAsync(producerBuffer, input);

			using ComputeBufferReader consumerBufferReader = consumerBuffer.CreateReader();

			byte[] output = new byte[Length];
			await RunConsumerAsync(consumerBufferReader, output);

			await producerTask;
			Assert.IsTrue(input.SequenceEqual(output));
		}

		static async Task RunProducerAsync(ComputeBuffer buffer, ReadOnlyMemory<byte> input)
		{
			using ComputeBufferWriter writer = buffer.CreateWriter();

			int offset = 0;
			while (offset < input.Length)
			{
				int length = Math.Min(input.Length - offset, 100);
				await writer.WriteAsync(input.Slice(offset, length));
				await Task.Delay(10);
				offset += length;
			}

			writer.MarkComplete();
		}

		static async Task RunConsumerAsync(ComputeBufferReader reader, Memory<byte> output)
		{
			int offset = 0;
			while (!reader.IsComplete)
			{
				ReadOnlyMemory<byte> memory = reader.GetReadBuffer();
				if (memory.Length == 0)
				{
					await reader.WaitToReadAsync(1, CancellationToken.None);
					continue;
				}

				int length = Math.Min(memory.Length, 7);
				memory.Slice(0, length).CopyTo(output.Slice(offset));
				reader.AdvanceReadPosition(length);
				offset += length;
			}
		}

		[TestMethod]
		public async Task TestSendBufferCompleteAsync()
		{
			Pipe recvPipe = new();
			Pipe sendPipe = new();
			await using PipeTransport localTransport = new(sendPipe.Reader, recvPipe.Writer);
			await using PipeTransport remoteTransport = new(recvPipe.Reader, sendPipe.Writer);
			await using RemoteComputeSocket localSocket = new(localTransport, ComputeProtocol.Latest, NullLogger.Instance);
			await using RemoteComputeSocket remoteSocket = new(remoteTransport, ComputeProtocol.Latest, NullLogger.Instance);

			using (PooledBuffer remoteBuffer = new(1024))
			{
				remoteSocket.AttachRecvBuffer(1, remoteBuffer);

				using ComputeBufferReader reader = remoteBuffer.CreateReader();

				// Disposing of the buffer should mark the channel as complete
				using (PooledBuffer localBuffer = new(1024))
				{
					localSocket.AttachSendBuffer(1, localBuffer);

					using ComputeBufferWriter localBufferWriter = localBuffer.CreateWriter();
					await localBufferWriter.WriteAsync(new byte[] { 1, 2, 3 });
				}

				Assert.IsTrue(await reader.WaitToReadAsync(3));
				Assert.IsTrue(reader.GetReadBuffer().Slice(0, 3).Span.SequenceEqual(new byte[] { 1, 2, 3 }));
				reader.AdvanceReadPosition(3);

				Assert.IsFalse(await reader.WaitToReadAsync(1));
				Assert.IsTrue(reader.IsComplete);
			}
		}
	}
}
