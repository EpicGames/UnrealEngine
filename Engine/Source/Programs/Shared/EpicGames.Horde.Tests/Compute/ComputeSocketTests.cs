// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO.Pipelines;
using System.Linq;
using System.Net;
using System.Net.Sockets;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Transports;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute
{
	[TestClass]
	[DoNotParallelize]
	public class ComputeSocketTests
	{
		public const int TestTimoutMs = 40000;
		class TestComputeSocket : ComputeSocket, IDisposable
		{
			public Dictionary<int, ComputeBufferWriter> RecvBufferWriters { get; } = [];
			public Dictionary<int, ComputeBufferReader> SendBufferReaders { get; } = [];

			public override ComputeProtocol Protocol => ComputeProtocol.Latest;
			public override ILogger Logger => NullLogger.Instance;

			public void Dispose()
			{
				foreach (ComputeBufferWriter writer in RecvBufferWriters.Values)
				{
					writer.Dispose();
				}
				foreach (ComputeBufferReader reader in SendBufferReaders.Values)
				{
					reader.Dispose();
				}
			}

			public override void AttachRecvBuffer(int channelId, ComputeBuffer recvBuffer)
			{
				RecvBufferWriters.Add(channelId, recvBuffer.CreateWriter());
			}

			public override void AttachSendBuffer(int channelId, ComputeBuffer sendBuffer)
			{
				SendBufferReaders.Add(channelId, sendBuffer.CreateReader());
			}
		}

		class TestLogger(string prefix) : ILogger
		{
			public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null!;

			public bool IsEnabled(LogLevel logLevel) => true;

			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
			{
				Console.WriteLine($"{prefix} {logLevel}: {formatter(state, exception)}");
				Assert.IsFalse(logLevel == LogLevel.Error);
				Assert.IsFalse(logLevel == LogLevel.Warning);
			}
		}

		[TestMethod]
		public async Task TestAgentMessageLoopPipeAsync()
		{
			Pipe recvPipe = new();
			Pipe sendPipe = new();
			await using PipeTransport localTransport = new(sendPipe.Reader, recvPipe.Writer);
			await using PipeTransport agentTransport = new(recvPipe.Reader, sendPipe.Writer);
			await using RemoteComputeSocket localSocket = new(localTransport, ComputeProtocol.Latest, new TestLogger("local"));
			await using RemoteComputeSocket agentSocket = new(agentTransport, ComputeProtocol.Latest, new TestLogger("agent"));

			await RunAgentTestsAsync(localSocket, agentSocket);
		}

		[TestMethod]
		public async Task TestAgentMessageLoopTcpAsync()
		{
			using CancellationTokenSource cts = new(TestTimoutMs);
			(Socket clientSocket, Socket serverSocket) = await CreateSocketsAsync(cts.Token);

			await using TcpTransport clientTransport = new(clientSocket);
			await using TcpTransport serverTransport = new(serverSocket);
			await using RemoteComputeSocket localSocket = new(clientTransport, ComputeProtocol.Latest, new TestLogger("local"));
			await using RemoteComputeSocket agentSocket = new(serverTransport, ComputeProtocol.Latest, new TestLogger("agent"));

			await RunAgentTestsAsync(localSocket, agentSocket, cts.Token);
		}

		[TestMethod]
		[DataRow(Encryption.Ssl)]
		[DataRow(Encryption.SslEcdsaP256)]
		public async Task TestAgentMessageLoopTcpSslAsync(Encryption encryption)
		{
			using CancellationTokenSource cts = new(TestTimoutMs);
			(Socket clientSocket, Socket serverSocket) = await CreateSocketsAsync(cts.Token);

			byte[] certData = TcpSslTransport.GenerateCert(encryption);
			await using TcpSslTransport clientTransport = new(clientSocket, certData, false);
			await using TcpSslTransport serverTransport = new(serverSocket, certData, true);

			Task t1 = clientTransport.AuthenticateAsync(cts.Token);
			Task t2 = serverTransport.AuthenticateAsync(cts.Token);
			await t2;
			await t1;

			await using RemoteComputeSocket localSocket = new(clientTransport, ComputeProtocol.Latest, new TestLogger("local"));
			await using RemoteComputeSocket agentSocket = new(serverTransport, ComputeProtocol.Latest, new TestLogger("agent"));
			await RunAgentTestsAsync(localSocket, agentSocket, cts.Token);
		}

		[TestMethod]
		[Retry(2)] // Retry up to 2 times on failure due to CI load variability
		public async Task TestAgentMessageLoopTcpAesAsync()
		{
			using CancellationTokenSource cts = new(TestTimoutMs);
			(Socket localSocket, Socket agentSocket) = await CreateSocketsAsync(cts.Token);

			byte[] key = AesTransport.CreateKey();
			await using TcpTransport localTcp = new(localSocket);
			await using TcpTransport agentTcp = new(agentSocket);
			await using AesTransport localAes = new(localTcp, key);
			await using AesTransport agentAes = new(agentTcp, key);

			await using IdleTimeoutTransport localIdleTimeout = new(localAes, TimeSpan.FromSeconds(15));
			await using IdleTimeoutTransport agentIdleTimeout = new(agentAes, TimeSpan.FromSeconds(15));
			await using RemoteComputeSocket localComputeSocket = new(localIdleTimeout, ComputeProtocol.Latest, new TestLogger("local"));
			await using RemoteComputeSocket agentComputeSocket = new(agentIdleTimeout, ComputeProtocol.Latest, new TestLogger("agent"));

			await RunAgentTestsAsync(localComputeSocket, agentComputeSocket, cts.Token);
		}

		internal static async Task<(Socket client, Socket server)> CreateSocketsAsync(CancellationToken cancellationToken)
		{
			int port = GetAvailablePort();
			using TcpListener listener = new(IPAddress.Loopback, port);
			listener.Start();
			Socket clientSocket = new(SocketType.Stream, ProtocolType.Tcp);
			Task clientConnectTask = clientSocket.ConnectAsync(IPAddress.Loopback, port, cancellationToken).AsTask();
			Socket serverSocket = await listener.AcceptSocketAsync(cancellationToken);
			await clientConnectTask;
			return (clientSocket, serverSocket);
		}

		static async Task RunAgentTestsAsync(RemoteComputeSocket localSocket, RemoteComputeSocket agentSocket, CancellationToken cancellationToken = default)
		{
			DirectoryReference tempDir = new("test-temp-" + DateTime.UtcNow.Ticks);
			await using (BackgroundTask agentTask = BackgroundTask.StartNew(ctx => RunAgentAsync(agentSocket, tempDir, ctx)))
			{
				const int PrimaryChannelId = 0;
				using (AgentMessageChannel channel = localSocket.CreateAgentMessageChannel(PrimaryChannelId, 4 * 1024 * 1024))
				{
					await channel.WaitForAttachAsync(cancellationToken);

					await channel.PingAsync(cancellationToken);
					using (AgentMessage message = await channel.ReceiveAsync(cancellationToken))
					{
						Assert.AreEqual(AgentMessageType.Ping, message.Type);
						Assert.IsTrue(message.Data.Span.SequenceEqual([]));
					}

					await channel.SendXorRequestAsync(new byte[] { 1, 2, 3 }, 44, cancellationToken);
					using (AgentMessage message = await channel.ReceiveAsync(cancellationToken))
					{
						Assert.AreEqual(AgentMessageType.XorResponse, message.Type);
						Assert.IsTrue(message.Data.Span.SequenceEqual(new byte[] { 1 ^ 44, 2 ^ 44, 3 ^ 44 }));
					}

					const int SecondaryChannelId = 1;
					using (AgentMessageChannel channel2 = localSocket.CreateAgentMessageChannel(SecondaryChannelId, 4 * 1024 * 1024))
					{
						await channel.ForkAsync(SecondaryChannelId, 4 * 1024 * 1024, cancellationToken);

						await channel2.WaitForAttachAsync(cancellationToken);

						await channel2.SendXorRequestAsync(new byte[] { 1, 2, 3 }, 44, cancellationToken);
						using (AgentMessage message = await channel2.ReceiveAsync(cancellationToken))
						{
							Assert.AreEqual(AgentMessageType.XorResponse, message.Type);
							Assert.IsTrue(message.Data.Span.SequenceEqual(new byte[] { 1 ^ 44, 2 ^ 44, 3 ^ 44 }));
						}

						await channel2.CloseAsync(cancellationToken);
					}

					BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
					await using (IBlobWriter blobWriter = storage.CreateBlobWriter(cancellationToken: cancellationToken))
					{
						FileReference file = FileReference.Combine(tempDir, "subdir/hello.txt");
						if (FileReference.Exists(file))
						{
							FileReference.Delete(file);
						}
						Assert.IsFalse(FileReference.Exists(file));

						byte[] data = Encoding.UTF8.GetBytes("Hello world");

						using ChunkedDataWriter writer = new(blobWriter, new ChunkingOptions());
						ChunkedData chunkedData = await writer.CreateAsync(data, cancellationToken);

						DirectoryNode directory = new();
						directory.AddFile("hello.txt", FileEntryFlags.None, chunkedData);

						IHashedBlobRef<DirectoryNode> directoryRef = await blobWriter.WriteBlobAsync(directory, cancellationToken: cancellationToken);

						DirectoryNode root = new();
						root.AddDirectory(new DirectoryEntry("subdir", directory.Length, directoryRef));

						IHashedBlobRef<DirectoryNode> handle = await blobWriter.WriteBlobAsync(root, cancellationToken: cancellationToken);
						await blobWriter.FlushAsync(cancellationToken);

						await channel.UploadFilesAsync("", handle.GetLocator(), storage.Backend, cancellationToken);

						Assert.IsTrue(FileReference.Exists(file));
						byte[] readData = await FileReference.ReadAllBytesAsync(file, cancellationToken);
						Assert.IsTrue(readData.SequenceEqual(data));

						await channel.DeleteFilesAsync(["subdir/hello.txt"], cancellationToken);

						await channel.PingAsync(cancellationToken);
						using (AgentMessage message = await channel.ReceiveAsync(cancellationToken))
						{
							Assert.AreEqual(AgentMessageType.Ping, message.Type);
							Assert.IsTrue(message.Data.Span.SequenceEqual([]));
						}

						Assert.IsFalse(FileReference.Exists(file));
					}
				}
			}

			await localSocket.CloseAsync(CancellationToken.None);
			await agentSocket.CloseAsync(CancellationToken.None);
		}

		private static readonly HashSet<int> s_usedPorts = [];
		private static int GetAvailablePort()
		{
			lock (s_usedPorts)
			{
				for (int i = 0; i < 10; i++)
				{
					using TcpListener listener = new(IPAddress.Loopback, 0);
					try
					{
						listener.Start();
						int port = ((IPEndPoint)listener.LocalEndpoint).Port;
						if (!s_usedPorts.Add(port))
						{
							continue;
						}

						return port;
					}
					finally
					{
						listener.Stop();
					}
				}

				throw new InvalidOperationException("Unable to acquire a locally available IP port");
			}
		}

		static async Task RunAgentAsync(ComputeSocket socket, DirectoryReference tempDir, CancellationToken cancellationToken)
		{
			try
			{
				AgentMessageHandler handler = new(tempDir, null, true, null, null, NullLogger.Instance);
				await handler.RunAsync(socket, cancellationToken);
			}
			catch (Exception e)
			{
				Console.WriteLine("Exception when running agent:\n" + e);
				throw;
			}
		}
	}
}
