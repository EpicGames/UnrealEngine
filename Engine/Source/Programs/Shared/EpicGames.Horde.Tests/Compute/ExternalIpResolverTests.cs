// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Compute;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute;

[TestClass]
public class ExternalIpResolverTests
{
	private class StubMessageHandler : HttpMessageHandler
	{
		private readonly HttpStatusCode _statusCode;
		public string Content { get; set; }

		public StubMessageHandler(HttpStatusCode statusCode, string content)
		{
			_statusCode = statusCode;
			Content = content;
		}

		protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
		{
			return Task.FromResult(new HttpResponseMessage(_statusCode) { Content = new StringContent(Content) });
		}
	}

	[TestMethod]
	public async Task BasicAsync()
	{
		Assert.AreEqual(IPAddress.Parse("192.168.2.3"), await GetIpAsync("192.168.2.3"));
		await Assert.ThrowsExactlyAsync<ExternalIpResolverException>(() => GetIpAsync("bad-ip"));
		await Assert.ThrowsExactlyAsync<ExternalIpResolverException>(() => GetIpAsync("192.168.2.3", HttpStatusCode.NotFound));
	}

	[TestMethod]
	public async Task CacheIpAddressAsync()
	{
		using StubMessageHandler stub = new(HttpStatusCode.OK, "100.200.1.1");
		using HttpClient httpClient = new(stub);
		ExternalIpResolver resolver = new(httpClient);
		IPAddress ip = await resolver.GetExternalIpAddressAsync(CancellationToken.None);
		Assert.AreEqual(IPAddress.Parse("100.200.1.1"), ip);

		// Querying for IP again should never reach the IP resolver's HTTP server, so an invalid IP is returned to ensure this
		stub.Content = "invalid-ip";
		ip = await resolver.GetExternalIpAddressAsync(CancellationToken.None);
		Assert.AreEqual(IPAddress.Parse("100.200.1.1"), ip);
	}

	[TestMethod]
	[Ignore]
	public async Task IntegrationAsync()
	{
		using HttpClient httpClient = new();
		ExternalIpResolver resolver = new(httpClient);
		IPAddress externalIp = await resolver.GetExternalIpAddressAsync();
		Console.WriteLine("External IP: " + externalIp);
	}

	private static async Task<IPAddress> GetIpAsync(string content, HttpStatusCode statusCode = HttpStatusCode.OK, CancellationToken cancellationToken = default)
	{
		using StubMessageHandler handler = new(statusCode, content);
		using HttpClient httpClient = new(handler);
		ExternalIpResolver resolver = new(httpClient);
		return await resolver.GetExternalIpAddressAsync(cancellationToken);
	}
}