// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Horde.Compute;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute;

[TestClass]
public class RequestIdAllocatorTests
{
	[TestMethod]
	public void IdsAreReused()
	{
		RequestIdAllocator ria = new();
		List<string> before = [ria.AllocateId(), ria.AllocateId(), ria.AllocateId()];
		ria.StartBatch();
		List<string> after = [ria.AllocateId(), ria.AllocateId(), ria.AllocateId()];

		CollectionAssert.AreEquivalent(before, after);
	}

	[TestMethod]
	public void AcceptedIdsAreNotReused()
	{
		RequestIdAllocator ria = new();
		List<string> before = [ria.AllocateId(), ria.AllocateId(), ria.AllocateId()];
		ria.MarkAccepted(before[1]);

		ria.StartBatch();
		List<string> after = [ria.AllocateId(), ria.AllocateId(), ria.AllocateId()];
		Assert.IsTrue(after.Remove(before[0]));
		Assert.IsFalse(after.Remove(before[1]));
		Assert.IsTrue(after.Remove(before[2]));
	}
}
