// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Transports;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute
{
	[TestClass]
	public class TransportTests
	{
		[TestMethod]
		public async Task TestStreamTransportAsync()
		{
			byte[] input = new byte[256 * 1024];
			new Random(0).NextBytes(input);

			byte[] payload;
			{
				using MemoryStream memoryStream = new();
				await using StreamTransport streamTransport = new(memoryStream);
				await streamTransport.SendAsync(input, CancellationToken.None);
				payload = memoryStream.ToArray();
			}

			byte[] output = new byte[input.Length];
			{
				using MemoryStream memoryStream = new(payload);
				await using StreamTransport streamTransport = new(memoryStream);
				await CopyToAsync(streamTransport, output);
			}

			for (int idx = 0; idx < input.Length; idx++)
			{
				Assert.IsTrue(input[idx] == output[idx]);
			}
			Assert.IsTrue(input.SequenceEqual(output));

		}

		[TestMethod]
		public async Task TestAesTransportAsync()
		{
			byte[] key = new byte[AesTransport.KeyLength];
			new Random(1).NextBytes(key);

			byte[] input = RandomNumberGenerator.GetBytes(256 * 1024);
			new Random(0).NextBytes(input);

			byte[] encrypted;

			{
				using MemoryStream memoryStream = new();
				await using (StreamTransport streamTransport = new(memoryStream))
				{
					await using (AesTransport aesTransport = new(streamTransport, key))
					{
						await aesTransport.SendAsync(input, CancellationToken.None);
					}
				}
				encrypted = memoryStream.ToArray();
			}

			byte[] output = new byte[input.Length];
			{
				using MemoryStream memoryStream = new(encrypted);
				await using (StreamTransport streamTransport = new(memoryStream))
				{
					await using (AesTransport aesTransport = new(streamTransport, key))
					{
						for (int offset = 0; offset < output.Length;)
						{
							offset += await aesTransport.RecvAsync(output.AsMemory(offset), CancellationToken.None);
						}
					}
				}
			}

			for (int idx = 0; idx < input.Length; idx++)
			{
				Assert.IsTrue(input[idx] == output[idx]);
			}
			Assert.IsTrue(input.SequenceEqual(output));
		}

		public static async Task CopyToAsync(ComputeTransport transport, Memory<byte> buffer)
		{
			for (int offset = 0; offset < buffer.Length;)
			{
				int size = await transport.RecvAsync(buffer.Slice(offset), CancellationToken.None);
				Assert.IsTrue(size > 0);
				offset += size;
			}
		}
	}
}
