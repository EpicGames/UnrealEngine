// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Horde.Compute;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Compute;

[TestClass]
public class UbaCryptoTests
{
	private readonly byte[] _key = [0x3f, 0x58, 0xaa, 0x57, 0x46, 0x6d, 0xb9, 0x99, 0x92, 0x13, 0x45, 0x67, 0x89, 0x12, 0x34, 0x45];

	private const string Decrypted1 = "This is 15 char";
	private const string Encrypted1 = "5B 67 66 7C 2F 66 7C 2F 3E 3A 2F 6C 67 6E 7D";
	private const string Decrypted2 = "This is 16 chars";
	private const string Encrypted2 = "24 DE B6 E3 3B 6A CD D0 A1 BA A8 46 F2 DC B6 CD";
	private const string Decrypted3 = "This is 17 chars!";
	private const string Encrypted3 = "22 42 52 20 03 0F 14 53 AD CA B4 20 A8 1D 30 EA 75";

	[TestMethod]
	[DataRow("", "")]
	[DataRow(Decrypted1, Encrypted1)]
	[DataRow(Decrypted2, Encrypted2)]
	[DataRow(Decrypted3, Encrypted3)]
	public void Encrypt(string data, string expected)
	{
		using UbaCrypto crypto = new(_key);
		byte[] actual = crypto.Encrypt(Encoding.UTF8.GetBytes(data));
		Assert.AreEqual(expected.Replace(" ", "", StringComparison.Ordinal), Convert.ToHexString(actual));
	}

	[TestMethod]
	[DataRow("", "")]
	[DataRow(Encrypted1, Decrypted1)]
	[DataRow(Encrypted2, Decrypted2)]
	[DataRow(Encrypted3, Decrypted3)]
	public void Decrypt(string encryptedHexString, string expected)
	{
		byte[] encryptedData = Convert.FromHexString(encryptedHexString.Replace(" ", "", StringComparison.Ordinal));
		using UbaCrypto crypto = new(_key);
		byte[] actual = crypto.Decrypt(encryptedData);
		Assert.AreEqual(expected, Encoding.UTF8.GetString(actual));
	}
}

