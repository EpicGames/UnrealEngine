<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\EpicGames.OIDC\EpicGames.OIDC.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Compile Update="Properties\Resources.Designer.cs">
      <DesignTime>True</DesignTime>
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <EmbeddedResource Update="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
    </EmbeddedResource>
  </ItemGroup>
</Project>
