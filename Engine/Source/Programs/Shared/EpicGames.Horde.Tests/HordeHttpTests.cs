// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Server;
using EpicGames.OIDC;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests;

public record StubHttpClientFactory(Func<HttpClient> CreateHttpClient) : IHttpClientFactory
{
	public HttpClient CreateClient(string name)
	{
		return CreateHttpClient();
	}
}

public class FakeServerWithAuth : HttpMessageHandler
{
	public const string ServerUrl = "http://fake-server-test";
	public const string SuccessResponse = "fakeServerSuccess";

	public List<HttpRequestMessage> HttpRequests { get; } = [];
	private readonly GetAuthConfigResponse? _authConfigResponse;
	private readonly FakeOidcTokenManager _oidcTokenManager;

	public FakeServerWithAuth(FakeOidcTokenManager oidcTokenManager, GetAuthConfigResponse? authConfigResponse = null)
	{
		_authConfigResponse = authConfigResponse ?? new GetAuthConfigResponse
		{
			Method = AuthMethod.OpenIdConnect,
			ServerUrl = ServerUrl,
			LocalRedirectUrls = [ServerUrl]
		};
		_oidcTokenManager = oidcTokenManager;
	}

	/// <summary>
	/// Get the access token used by a request received in the fake server
	/// </summary>
	/// <returns>Access token</returns>
	public string GetAccessTokenUsed(int requestNum = -1)
	{
		HttpRequestMessage req = requestNum == -1 ? HttpRequests.Last() : HttpRequests[requestNum];
		return req.Headers.Authorization!.Parameter!;
	}

	public IHttpClientFactory GetHttpClientFactory()
	{
		return new StubHttpClientFactory(() => new HttpClient(this) { BaseAddress = new Uri(ServerUrl) });
	}

	/// <inheritdoc/>
	protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
	{
		if (request.RequestUri?.AbsolutePath == "/api/v1/server/auth")
		{
			return Task.FromResult(new HttpResponseMessage(HttpStatusCode.OK) { Content = new StringContent(JsonSerializer.Serialize(_authConfigResponse)) });
		}

		Console.WriteLine($"Saving request {request.RequestUri} {request.Headers.Authorization}");
		HttpRequests.Add(request);
		try
		{
			_oidcTokenManager.ValidateAccessToken(request.Headers.Authorization?.Parameter);
			return Task.FromResult(new HttpResponseMessage(HttpStatusCode.OK) { Content = new StringContent(SuccessResponse) });
		}
		catch (Exception e)
		{
			return Task.FromResult(new HttpResponseMessage(HttpStatusCode.Unauthorized) { Content = new StringContent(e.Message) });
		}
	}
}

[TestClass]
public class HordeHttpAuthHandlerTests
{
	private readonly FakeOidcTokenManager _oidc;
	private readonly StubClock _clock = new();

	public HordeHttpAuthHandlerTests()
	{
		_oidc = new FakeOidcTokenManager(() => _clock.UtcNow);
	}

	[TestMethod]
	public async Task AccessToken_Valid_IsReusedAsync()
	{
		(HttpClient client, FakeServerWithAuth server) = CreateClientServer(_oidc);

		await SendHttpRequestAsync(client);
		Assert.AreEqual(_oidc.AccessToken, server.GetAccessTokenUsed(0));

		await SendHttpRequestAsync(client);
		Assert.AreEqual(_oidc.AccessToken, server.GetAccessTokenUsed(1));
		Assert.AreEqual(server.GetAccessTokenUsed(0), server.GetAccessTokenUsed(1));
	}

	[TestMethod]
	public async Task AccessToken_Expired_IsRefreshedAsync()
	{
		(HttpClient client, FakeServerWithAuth server) = CreateClientServer(_oidc);

		// Send a first request to obtain an access token then advance time so it expires
		await SendHttpRequestAsync(client);
		_clock.Advance(TimeSpan.FromMinutes(30));

		await SendHttpRequestAsync(client);
		Assert.AreEqual(_oidc.AccessToken, server.GetAccessTokenUsed());
		Assert.AreNotEqual(server.GetAccessTokenUsed(0), server.GetAccessTokenUsed(1));
	}

	[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope")]
	private (HttpClient client, FakeServerWithAuth server) CreateClientServer(FakeOidcTokenManager oidcTokenManager)
	{
		FakeServerWithAuth server = new(oidcTokenManager);
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});

		ILogger<OidcAuthState> logger = loggerFactory.CreateLogger<OidcAuthState>();
		IOidcAuthStateConfig oidcAuthStateConfig = new OidcAuthStateConfig("Horde");
		OidcAuthState state = new(oidcTokenManager, new NullStaticAccessTokenProvider(), _clock, oidcAuthStateConfig, logger);
		HttpOidcAuthHandler authHandler = new(server, state, new HttpOidcAuthHandlerConfig("Horde-AllowInteractiveLogin"));

		authHandler.InnerHandler = server;
		HttpClient client = new(authHandler);
		return (client, server);
	}

	private static async Task SendHttpRequestAsync(HttpClient client)
	{
		using HttpRequestMessage req = new(HttpMethod.Get, FakeServerWithAuth.ServerUrl);
		HttpResponseMessage res = await client.SendAsync(req);
		Assert.AreEqual(FakeServerWithAuth.SuccessResponse, await res.Content.ReadAsStringAsync());
		Assert.AreEqual(HttpStatusCode.OK, res.StatusCode);
	}
}
