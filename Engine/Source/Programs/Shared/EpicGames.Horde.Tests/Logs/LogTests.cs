// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Clients;
using EpicGames.Horde.Tests.Properties;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests
{
	[TestClass]
	public sealed class LogTests
	{
		private readonly byte[] _data = Resources.TextFile;

		[TestMethod]
		public void NgramTests()
		{
			NgramSetBuilder builder = new();

			ulong[] values = [1, 2, 3, 5, 8, 13, 21, 34, 55, 89, 144, 233, 377, 610, 987, 1597, 2584, 4181, 6765, 10946, 0xfedcba9876543210UL];
			foreach (ulong value in values)
			{
				builder.Add(value);
			}

			NgramSet trie = builder.ToNgramSet();
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(0, UInt64.MaxValue), values));
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(0, 90), values.Where(x => x <= 90)));
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(2, 89), values.Where(x => x >= 2 && x <= 89)));
		}

		[TestMethod]
		public async Task IndexTestsAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			// Write the test data to the log file in blocks
			LogBuilder builder = new(LogFormat.Text, 1, 1, NullLogger.Instance);

			int readOffset = 0;
			int readLineIndex = 0;
			while (readOffset < _data.Length)
			{
				int length = 0;
				int lineCount = 0;

				for (int idx = readOffset; idx < Math.Min(_data.Length, readOffset + 1883); idx++)
				{
					if (_data[idx] == '\n')
					{
						length = (idx + 1) - readOffset;
						lineCount++;
						break;
					}
				}

				builder.WriteData(_data.AsMemory(readOffset, length));

				readOffset += length;
				readLineIndex += lineCount;
			}

			// Flush it to storage, and read the finished log node
			IHashedBlobRef<LogNode> logRef;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				logRef = await builder.FlushAsync(writer, true, CancellationToken.None);
			}
			LogNode log = await logRef.ReadBlobAsync();

			// Read the data back out and check it's the same
			byte[] readData = new byte[_data.Length];

			int writeOffset = 0;
			await foreach (ReadOnlyMemory<byte> line in log.ReadLogLinesAsync(0))
			{
				line.CopyTo(readData.AsMemory(writeOffset));
				writeOffset += line.Length;
			}

			int equalSize = 0;
			while (equalSize < _data.Length && _data[equalSize] == readData[equalSize])
			{
				equalSize++;
			}
			Assert.AreEqual(equalSize, readOffset);

			// Test some searches
			LogIndexNode index = await log.IndexRef.ReadBlobAsync();
			await SearchLogDataTestAsync(index);
		}

		[TestMethod]
		public async Task PartialTokenTestsAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			// Generate the test data
			string[] lines =
			[
				"abcdefghi\n",
				"jklmno123\n",
				"pqrst99uv\n",
				"wx\n"
			];

			LogBuilder builder = new(LogFormat.Text, 1, 1, NullLogger.Instance);
			for (int lineIdx = 0; lineIdx < lines.Length; lineIdx++)
			{
				builder.WriteData(Encoding.UTF8.GetBytes(lines[lineIdx]));
			}

			IHashedBlobRef<LogNode> rootNodeRef;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				rootNodeRef = await builder.FlushAsync(writer, true, CancellationToken.None);
			}

			// Read it back in and test the index
			LogNode rootNode = await rootNodeRef.ReadBlobAsync();
			LogIndexNode index = await rootNode.IndexRef.ReadBlobAsync();

			for (int lineIdx = 0; lineIdx < lines.Length; lineIdx++)
			{
				for (int strLen = 1; strLen < 7; strLen++)
				{
					for (int strOfs = 0; strOfs + strLen < lines[lineIdx].Length - 1; strOfs++)
					{
						string str = lines[lineIdx].Substring(strOfs, strLen);

						SearchStats stats = new();
						List<int> results = await index.SearchAsync(0, new SearchTerm(str), stats).ToListAsync();
						Assert.AreEqual(1, results.Count);
						Assert.AreEqual(lineIdx, results[0]);

						Assert.AreEqual(1, stats.NumScannedBlocks);
						Assert.AreEqual(3, stats.NumSkippedBlocks);
						Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
					}
				}
			}
		}
		/*
		[TestMethod]
		public async Task AppendIndexTests()
		{
			JobId jobId = JobId.GenerateNewId();
			ILogFile logFile = await _logFileService.CreateLogFileAsync(jobId, null, LogType.Text);

			logFile = await WriteLogDataAsync(logFile, 0, 0, Encoding.UTF8.GetBytes("abc\n"), true);
			logFile = await WriteLogDataAsync(logFile, 4, 1, Encoding.UTF8.GetBytes("def\n"), true);
			logFile = await WriteLogDataAsync(logFile, 8, 2, Encoding.UTF8.GetBytes("ghi\n"), false);

			await ((LogFileService)_logFileService).FlushPendingWritesAsync();

			{
				LogSearchStats stats = new LogSearchStats();
				List<int> results = await _logFileService.SearchLogDataAsync(logFile, "abc", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(0, results[0]);

				Assert.AreEqual(2, stats.NumScannedBlocks); // abc + ghi (no index yet because it hasn't been flushed)
				Assert.AreEqual(1, stats.NumSkippedBlocks); // def
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
			{
				LogSearchStats stats = new LogSearchStats();
				List<int> results = await _logFileService.SearchLogDataAsync(logFile, "def", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(1, results[0]);

				Assert.AreEqual(2, stats.NumScannedBlocks); // def + ghi (no index yet because it hasn't been flushed)
				Assert.AreEqual(1, stats.NumSkippedBlocks); // abc
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
			{
				LogSearchStats stats = new LogSearchStats();
				List<int> results = await _logFileService.SearchLogDataAsync(logFile, "ghi", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(2, results[0]);

				Assert.AreEqual(1, stats.NumScannedBlocks); // ghi
				Assert.AreEqual(2, stats.NumSkippedBlocks); // abc + def
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
		}
*/
		static async Task SearchLogDataTestAsync(LogIndexNode index)
		{
			await SearchLogDataTestAsync(index, "HISPANIOLA", 0, 4, [1503, 1520, 1525, 1595]);
			await SearchLogDataTestAsync(index, "Hispaniola", 0, 4, [1503, 1520, 1525, 1595]);
			await SearchLogDataTestAsync(index, "HizpaniolZ", 0, 4, []);
			await SearchLogDataTestAsync(index, "Pieces of eight!", 0, 100, [2227, 2228, 5840, 5841, 7520]);
			await SearchLogDataTestAsync(index, "NEWSLETTER", 0, 100, [7886]);
		}

		static async Task SearchLogDataTestAsync(LogIndexNode index, string text, int firstLine, int count, int[] expectedLines)
		{
			SearchStats stats = new();
			List<int> lines = await index.SearchAsync(firstLine, new SearchTerm(text), stats, CancellationToken.None).Take(count).ToListAsync();
			Assert.IsTrue(lines.SequenceEqual(expectedLines));
		}
	}
}
