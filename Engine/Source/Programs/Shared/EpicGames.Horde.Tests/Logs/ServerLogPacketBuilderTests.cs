// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Tools;
using Grpc.Core;
using Grpc.Net.Client;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests
{
	[TestClass]
	public class ServerLogPacketBuilderTests
	{
		class JsonLoggerImpl : ILogger
		{
			public List<LogEvent> _lines = [];

			public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null!;

			public bool IsEnabled(LogLevel logLevel) => true;

			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
			{
				JsonLogEvent logEvent = JsonLogEvent.FromLoggerState(logLevel, eventId, state, exception, formatter);
				_lines.Add(LogEvent.Read(logEvent.Data.Span));
			}
		}

		[TestMethod]
		public void JsonLogEventTest()
		{
			JsonLoggerImpl impl = new();
			impl.LogInformation("Hello {Text}", "world");
			impl.LogInformation("Hello {Text}", "world");

			Assert.AreEqual(2, impl._lines.Count);
			Assert.AreEqual("Hello world", impl._lines[0].ToString());
			Assert.AreEqual("world", impl._lines[0].GetProperty("Text")?.ToString());
			Assert.AreEqual("Hello world", impl._lines[1].ToString());
			Assert.AreEqual("world", impl._lines[1].GetProperty("Text")?.ToString());
		}

		static string[] SplitMultiLineMessage(string message)
		{
			JsonLogEvent jsonLogEvent;
			Assert.IsTrue(JsonLogEvent.TryParse(Encoding.UTF8.GetBytes(message), out jsonLogEvent));

			ServerLogPacketBuilder writer = new();
			int count = writer.SanitizeAndWriteEvent(jsonLogEvent);

			string[] result = Encoding.UTF8.GetString(writer.CreatePacket().Item1.Span).Split('\n', StringSplitOptions.RemoveEmptyEntries);
			Assert.AreEqual(count, result.Length);
			return result;
		}

		[TestMethod]
		public void MultiLineFormatTest()
		{
			const string Msg = @"{""level"":""Information"",""message"":""ignored"",""format"":""This\nis a\nmulti-{Line}-end\nlog message {Var1} {Var2}"",""properties"":{""Line"":""line\nsplit\nin\nfour"",""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""}}}";

			string[] result = SplitMultiLineMessage(Msg);
			Assert.AreEqual(7, result.Length);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""This"",""format"":""This"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""}},""line"":0,""lineCount"":7}", result[0]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""is a"",""format"":""is a"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""}},""line"":1,""lineCount"":7}", result[1]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""multi-line"",""format"":""multi-{Line$0}"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""},""Line$0"":""line""},""line"":2,""lineCount"":7}", result[2]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""split"",""format"":""{Line$1}"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""},""Line$1"":""split""},""line"":3,""lineCount"":7}", result[3]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""in"",""format"":""{Line$2}"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""},""Line$2"":""in""},""line"":4,""lineCount"":7}", result[4]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""four-end"",""format"":""{Line$3}-end"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""},""Line$3"":""four""},""line"":5,""lineCount"":7}", result[5]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""log message 123 D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""format"":""log message {Var1} {Var2}"",""properties"":{""Var1"":123,""Var2"":{""$type"":""SourceFile"",""$text"":""D:\\build\\\u002B\u002BUE5\\Sync\\GenerateProjectFiles.bat"",""relativePath"":""GenerateProjectFiles.bat"",""depotPath"":""//UE5/Main/GenerateProjectFiles.bat@20392842""}},""line"":6,""lineCount"":7}", result[6]);
		}

		[TestMethod]
		public void InvalidCharactersText()
		{
			byte[] data = Encoding.UTF8.GetBytes(@"{""level"":""Information"",""message"":""test"",""format"":""GOOD\nBAD""}");
			int idx = data.AsSpan().IndexOf(Encoding.UTF8.GetBytes("BAD"));
			data[idx] = 0xef;
			data[idx + 1] = 0xbb;

			ServerLogPacketBuilder writer = new();
			int count = writer.SanitizeAndWriteEvent(JsonLogEvent.Parse(data));
			Assert.AreEqual(1, count);

			string[] result = Encoding.UTF8.GetString(writer.CreatePacket().Item1.Span).Split('\n', StringSplitOptions.RemoveEmptyEntries);
			Assert.AreEqual(1, result.Length);

			LogEvent logEvent = LogEvent.Read(Encoding.UTF8.GetBytes(result[0]));

			const string ExpectedError = "Invalid json log event: {\"level\":\"Information\",\"message\":\"test\",\"format\":\"GOOD\\n\\xef\\xbbD\"}";
			Assert.AreEqual(ExpectedError, logEvent.ToString());
		}

		[TestMethod]
		public void MultiLineMessageTest()
		{
			const string Msg = @"{""level"":""Information"",""message"":""This is \na multi-line string""}";

			string[] result = SplitMultiLineMessage(Msg);
			Assert.AreEqual(2, result.Length);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""This is "",""line"":0,""lineCount"":2}", result[0]);
			Assert.AreEqual(@"{""level"":""Information"",""message"":""a multi-line string"",""line"":1,""lineCount"":2}", result[1]);
		}

		[TestMethod]
		public void MultiLineDecoyTest()
		{
			const string Msg = @"{""level"":""Information"",""message"":""This is \\\\not a multi-line format string""}";

			string[] result = SplitMultiLineMessage(Msg);
			Assert.AreEqual(1, result.Length);
			Assert.AreEqual(Msg, result[0]);
		}

		class GrpcHelpers
		{
			class StreamReader<T> : IAsyncStreamReader<T> where T : class
			{
				readonly ChannelReader<T> _reader;
				T? _current;

				public T Current => _current ?? throw new InvalidOperationException();

				public StreamReader(ChannelReader<T> reader)
					=> _reader = reader;

				public async Task<bool> MoveNext(CancellationToken cancellationToken)
				{
					while (await _reader.WaitToReadAsync(cancellationToken))
					{
						if (_reader.TryRead(out _current))
						{
							return true;
						}
					}
					return false;
				}
			}

			class StreamWriter<T> : IClientStreamWriter<T>
			{
				readonly ChannelWriter<T> _writer;

				public StreamWriter(ChannelWriter<T> writer)
					=> _writer = writer;

				public WriteOptions? WriteOptions { get => throw new NotImplementedException(); set => throw new NotImplementedException(); }

				public Task CompleteAsync()
				{
					_writer.Complete();
					return Task.CompletedTask;
				}

				public async Task WriteAsync(T message)
					=> await _writer.WriteAsync(message);
			}

			public static AsyncDuplexStreamingCall<TRequest, TResponse> CreateDuplexCall<TRequest, TResponse>(Func<ChannelReader<TRequest>, ChannelWriter<TResponse>, Task> func)
				where TRequest : class
				where TResponse : class
			{
				Channel<TRequest> requests = Channel.CreateUnbounded<TRequest>();
				Channel<TResponse> responses = Channel.CreateUnbounded<TResponse>();

				BackgroundTask runner = BackgroundTask.StartNew(async ctx =>
				{
					await func(requests.Reader, responses.Writer);
					responses.Writer.Complete();
				});

				return new AsyncDuplexStreamingCall<TRequest, TResponse>(new StreamWriter<TRequest>(requests.Writer), new StreamReader<TResponse>(responses.Reader), Task.FromResult<Metadata>(null!), () => Status.DefaultSuccess, () => null!, () => runner.DisposeAsync().AsTask().Wait());
			}
		}

		class FakeLogRpcClient : LogRpc.LogRpcClient
		{
			public Dictionary<LogId, BlobLocator> Logs { get; } = [];

			public override AsyncUnaryCall<RpcUpdateLogResponse> UpdateLogAsync(RpcUpdateLogRequest request, CallOptions options)
			{
				Logs[LogId.Parse(request.LogId)] = new BlobLocator(request.TargetLocator);
				return new AsyncUnaryCall<RpcUpdateLogResponse>(Task.FromResult(new RpcUpdateLogResponse()), Task.FromResult(new Metadata()), () => Status.DefaultSuccess, () => [], () => { });
			}

			public override AsyncDuplexStreamingCall<RpcUpdateLogTailRequest, RpcUpdateLogTailResponse> UpdateLogTail(CallOptions options)
			{
				return GrpcHelpers.CreateDuplexCall<RpcUpdateLogTailRequest, RpcUpdateLogTailResponse>(async (reader, writer) =>
				{
					while (await reader.WaitToReadAsync())
					{
						reader.TryRead(out RpcUpdateLogTailRequest? request);
					}
				});
			}
		}

		class FakeHordeClient : IHordeClient, IAsyncDisposable
		{
			public FakeLogRpcClient LogRpc { get; } = new FakeLogRpcClient();
			public Dictionary<string, BundleStorageNamespace> StorageNamespaces { get; } = [];

			public Uri ServerUrl => throw new NotImplementedException();

			public IArtifactCollection Artifacts => throw new NotImplementedException();
			public IComputeClient Compute => throw new NotImplementedException();
			public IProjectCollection Projects => throw new NotImplementedException();
			public ISecretCollection Secrets => throw new NotImplementedException();
			public IToolCollection Tools => throw new NotImplementedException();

			public event Action? OnAccessTokenStateChanged
			{
				add { }
				remove { }
			}

			public Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken)
				=> throw new NotImplementedException();

			public HordeHttpClient CreateHttpClient()
				=> throw new NotImplementedException();

			public IStorageNamespace GetStorageNamespace(string relativePath, string? accessToken = null)
			{
				BundleStorageNamespace? storageNamespace;
				if (!StorageNamespaces.TryGetValue(relativePath, out storageNamespace))
				{
					storageNamespace = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
					StorageNamespaces.Add(relativePath, storageNamespace);
				}
				return storageNamespace;
			}

			public ValueTask DisposeAsync()
			{
				StorageNamespaces.Clear();
				return default;
			}

			public Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<GrpcChannel> CreateGrpcChannelAsync(CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>
			{
				if (typeof(TClient) == typeof(LogRpc.LogRpcClient))
				{
					return Task.FromResult<TClient>((TClient)(object)LogRpc);
				}
				throw new NotImplementedException();
			}

			public bool HasValidAccessToken()
				=> throw new NotImplementedException();

			public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information)
				=> throw new NotImplementedException();
		}

		[TestMethod]
		public async Task StorageLoggerTestAsync()
		{
			await using BundleCache cache = new();
			await using FakeHordeClient hordeClient = new();

			LogId logId = default;

			const int Count = 20000;
			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < Count; idx++)
				{
					logger.LogInformation("Testing {Number}", idx);
				}
			}

			// Read the log
			IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(logId);
			LogNode file = await storageNamespace.CreateBlobRef(hordeClient.LogRpc.Logs[logId]).ReadBlobAsync<LogNode>();

			// Check the index text
			List<Utf8String> extractedIndexText = [];

			LogIndexNode index = await file.IndexRef.ReadBlobAsync();
			foreach (LogChunkRef block in index.PlainTextChunkRefs)
			{
				LogChunkNode text = await block.Target.ReadBlobAsync();
				extractedIndexText.AddRange(text.Lines);
			}

			Assert.AreEqual(Count, extractedIndexText.Count);
			for (int idx = 0; idx < Count; idx++)
			{
				Assert.AreEqual(extractedIndexText[idx], new Utf8String($"Testing {idx}"));
			}

			// Check the body text
			List<string> extractedBodyText = [];

			foreach (LogChunkRef blockRef in file.TextChunkRefs)
			{
				LogChunkNode blockText = await blockRef.Target.ReadBlobAsync();
				foreach (Utf8String line in blockText.Lines)
				{
					LogEvent logEvent = LogEvent.Read(line.Span);
					extractedBodyText.Add(logEvent.ToString());
				}
			}

			Assert.AreEqual(Count, extractedBodyText.Count);
			for (int idx = 0; idx < Count; idx++)
			{
				Assert.AreEqual(extractedBodyText[idx], $"Testing {idx}");
			}
		}

		[TestMethod]
		public void LongMessageTest()
		{
			DateTime time = new(2023, 1, 1, 0, 0, 0, 0, DateTimeKind.Utc);

			string longLineA = String.Join("\n", Enumerable.Range(0, 5).Select(x => $"A{x}"));
			string longLineB = String.Join("\n", Enumerable.Range(0, 5).Select(x => $"B{x}"));
			string longLineC = String.Join("\n", Enumerable.Range(0, 5).Select(x => $"C{x}"));

			Dictionary<string, object?> properties = new()
			{
				["LongLineA"] = longLineA,
				["LongLineB"] = longLineB,
				["LongLineC"] = longLineC,
			};

			LogEvent baseEvent = new(time, LogLevel.Information, default, $"start {longLineA} x {longLineB} y\nz {longLineC} end", "start {LongLineA} x {LongLineB} y\nz {LongLineC} end", properties, null);

			ServerLogPacketBuilder writer = new();
			writer.SanitizeAndWriteEvent(new JsonLogEvent(baseEvent));

			string[] output = Encoding.UTF8.GetString(writer.CreatePacket().Item1.Span).Split('\n');
			string[] expected =
			[
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""start A0"",""format"":""start {LongLineA$0}"",""properties"":{""LongLineA$0"":""A0""},""line"":0,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""A1"",""format"":""{LongLineA$1}"",""properties"":{""LongLineA$1"":""A1""},""line"":1,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""A2"",""format"":""{LongLineA$2}"",""properties"":{""LongLineA$2"":""A2""},""line"":2,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""A3"",""format"":""{LongLineA$3}"",""properties"":{""LongLineA$3"":""A3""},""line"":3,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""A4 x B0"",""format"":""{LongLineA$4} x {LongLineB$0}"",""properties"":{""LongLineA$4"":""A4"",""LongLineB$0"":""B0""},""line"":4,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""B1"",""format"":""{LongLineB$1}"",""properties"":{""LongLineB$1"":""B1""},""line"":5,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""B2"",""format"":""{LongLineB$2}"",""properties"":{""LongLineB$2"":""B2""},""line"":6,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""B3"",""format"":""{LongLineB$3}"",""properties"":{""LongLineB$3"":""B3""},""line"":7,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""B4 y"",""format"":""{LongLineB$4} y"",""properties"":{""LongLineB$4"":""B4""},""line"":8,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""z C0"",""format"":""z {LongLineC$0}"",""properties"":{""LongLineC$0"":""C0""},""line"":9,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""C1"",""format"":""{LongLineC$1}"",""properties"":{""LongLineC$1"":""C1""},""line"":10,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""C2"",""format"":""{LongLineC$2}"",""properties"":{""LongLineC$2"":""C2""},""line"":11,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""C3"",""format"":""{LongLineC$3}"",""properties"":{""LongLineC$3"":""C3""},""line"":12,""lineCount"":14}",
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""C4 end"",""format"":""{LongLineC$4} end"",""properties"":{""LongLineC$4"":""C4""},""line"":13,""lineCount"":14}",
				""
			];

			for (int idx = 0; idx < output.Length; idx++)
			{
				Assert.AreEqual(expected[idx], output[idx]);
			}
		}

		[TestMethod]
		public void MaxPacketLengthTest()
		{
			DateTime time = new(2023, 1, 1, 0, 0, 0, 0, DateTimeKind.Utc);

			LogEvent baseEvent = new(time, LogLevel.Information, default, "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ", null, null, null);

			ServerLogPacketBuilder writer = new(maxPacketLength: 85);
			writer.SanitizeAndWriteEvent(new JsonLogEvent(baseEvent));
			writer.SanitizeAndWriteEvent(new JsonLogEvent(baseEvent));

			for (int idx = 0; idx < 2; idx++)
			{
				string output = Encoding.UTF8.GetString(writer.CreatePacket().Item1.Span);
				string expected = @"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ""}" + "\n";
				Assert.AreEqual(expected, output);
			}
		}

		[TestMethod]
		public void MaxLineLengthTest()
		{
			DateTime time = new(2023, 1, 1, 0, 0, 0, 0, DateTimeKind.Utc);

			LogEvent baseEvent = new(time, LogLevel.Information, default, "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ", null, null, null);

			ServerLogPacketBuilder writer = new(85);
			writer.SanitizeAndWriteEvent(new JsonLogEvent(baseEvent));

			string[] output = Encoding.UTF8.GetString(writer.CreatePacket().Item1.Span).Split('\n');
			string[] expected =
			[
				@"{""time"":""2023-01-01T00:00:00"",""level"":""Information"",""message"":""abcdefghijklmnopqrstuvwxyzABCDEFGHI [...]""}",
				""
			];

			for (int idx = 0; idx < output.Length; idx++)
			{
				Assert.AreEqual(expected[idx], output[idx]);
			}
		}
	}
}
