// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Tools;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using Grpc.Net.Client;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests
{
	[TestClass]
	public class ServerLoggerTests
	{
		#region Test infrastructure

		class GrpcHelpers
		{
			class StreamReader<T> : IAsyncStreamReader<T> where T : class
			{
				readonly ChannelReader<T> _reader;
				T? _current;

				public T Current => _current ?? throw new InvalidOperationException();

				public StreamReader(ChannelReader<T> reader)
					=> _reader = reader;

				public async Task<bool> MoveNext(CancellationToken cancellationToken)
				{
					while (await _reader.WaitToReadAsync(cancellationToken))
					{
						if (_reader.TryRead(out _current))
						{
							return true;
						}
					}
					return false;
				}
			}

			class StreamWriter<T> : IClientStreamWriter<T>
			{
				readonly ChannelWriter<T> _writer;

				public StreamWriter(ChannelWriter<T> writer)
					=> _writer = writer;

				public WriteOptions? WriteOptions { get => throw new NotImplementedException(); set => throw new NotImplementedException(); }

				public Task CompleteAsync()
				{
					_writer.Complete();
					return Task.CompletedTask;
				}

				public async Task WriteAsync(T message) => await _writer.WriteAsync(message);
			}

			public static AsyncDuplexStreamingCall<TRequest, TResponse> CreateDuplexCall<TRequest, TResponse>(Func<ChannelReader<TRequest>, ChannelWriter<TResponse>, Task> func)
				where TRequest : class
				where TResponse : class
			{
				Channel<TRequest> requests = Channel.CreateUnbounded<TRequest>();
				Channel<TResponse> responses = Channel.CreateUnbounded<TResponse>();

				BackgroundTask runner = BackgroundTask.StartNew(async ctx =>
				{
					await func(requests.Reader, responses.Writer);
					responses.Writer.Complete();
				});

				return new AsyncDuplexStreamingCall<TRequest, TResponse>(new StreamWriter<TRequest>(requests.Writer), new StreamReader<TResponse>(responses.Reader), Task.FromResult<Metadata>(null!), () => Status.DefaultSuccess, () => null!, () => runner.DisposeAsync().AsTask().Wait());
			}
		}

		class FakeLogRpcClient : LogRpc.LogRpcClient
		{
			public Dictionary<LogId, BlobLocator> Logs { get; } = [];
			public Channel<RpcUpdateLogRequest> UpdateLogChannel { get; } = Channel.CreateUnbounded<RpcUpdateLogRequest>();
			public Channel<RpcCreateLogEventsRequest> CreateLogEventsChannel { get; } = Channel.CreateUnbounded<RpcCreateLogEventsRequest>();
			public Channel<RpcUpdateLogTailRequest> UpdateLogTailChannel { get; } = Channel.CreateUnbounded<RpcUpdateLogTailRequest>();

			public override AsyncUnaryCall<RpcUpdateLogResponse> UpdateLogAsync(RpcUpdateLogRequest request, CallOptions options)
			{
				Logs[LogId.Parse(request.LogId)] = new BlobLocator(request.TargetLocator);
				UpdateLogChannel.Writer.TryWrite(request);
				return new AsyncUnaryCall<RpcUpdateLogResponse>(Task.FromResult(new RpcUpdateLogResponse()), Task.FromResult(new Metadata()), () => Status.DefaultSuccess, () => [], () => { });
			}

			public override AsyncUnaryCall<Empty> CreateLogEventsAsync(RpcCreateLogEventsRequest request, CallOptions options)
			{
				CreateLogEventsChannel.Writer.TryWrite(request);
				return new AsyncUnaryCall<Empty>(Task.FromResult(new Empty()), Task.FromResult(new Metadata()), () => Status.DefaultSuccess, () => [], () => { });
			}

			public override AsyncDuplexStreamingCall<RpcUpdateLogTailRequest, RpcUpdateLogTailResponse> UpdateLogTail(CallOptions options)
			{
				return GrpcHelpers.CreateDuplexCall<RpcUpdateLogTailRequest, RpcUpdateLogTailResponse>(async (reader, writer) =>
				{
					while (await reader.WaitToReadAsync())
					{
						if (reader.TryRead(out RpcUpdateLogTailRequest? request))
						{
							UpdateLogTailChannel.Writer.TryWrite(request);
						}
					}
				});
			}
		}

		class FakeHordeClient : IHordeClient, IAsyncDisposable
		{
			public FakeLogRpcClient LogRpc { get; } = new FakeLogRpcClient();
			public Dictionary<string, BundleStorageNamespace> StorageNamespaces { get; } = [];
			public Func<IStorageNamespace, IStorageNamespace>? StorageNamespaceDecorator { get; set; }

			public Uri ServerUrl => throw new NotImplementedException();

			public IArtifactCollection Artifacts => throw new NotImplementedException();
			public IComputeClient Compute => throw new NotImplementedException();
			public IProjectCollection Projects => throw new NotImplementedException();
			public ISecretCollection Secrets => throw new NotImplementedException();
			public IToolCollection Tools => throw new NotImplementedException();

			public event Action? OnAccessTokenStateChanged
			{
				add { }
				remove { }
			}

			public Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken)
				=> throw new NotImplementedException();

			public HordeHttpClient CreateHttpClient()
				=> throw new NotImplementedException();

			public IStorageNamespace GetStorageNamespace(string relativePath, string? accessToken = null)
			{
				BundleStorageNamespace? storageNamespace;
				if (!StorageNamespaces.TryGetValue(relativePath, out storageNamespace))
				{
					storageNamespace = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
					StorageNamespaces.Add(relativePath, storageNamespace);
				}
				IStorageNamespace result = storageNamespace;
				if (StorageNamespaceDecorator != null)
				{
					result = StorageNamespaceDecorator(result);
				}
				return result;
			}

			public ValueTask DisposeAsync()
			{
				StorageNamespaces.Clear();
				return default;
			}

			public Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<GrpcChannel> CreateGrpcChannelAsync(CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>
			{
				if (typeof(TClient) == typeof(LogRpc.LogRpcClient))
				{
					return Task.FromResult<TClient>((TClient)(object)LogRpc);
				}
				throw new NotImplementedException();
			}

			public bool HasValidAccessToken()
				=> throw new NotImplementedException();

			public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information)
				=> throw new NotImplementedException();
		}

		class ThrowOnWriterDisposeStorageNamespace(IStorageNamespace inner) : IStorageNamespace
		{
			public IBlobRef CreateBlobRef(BlobLocator locator) => inner.CreateBlobRef(locator);
			public IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default)
				=> new ThrowOnDisposeBlobWriter(inner.CreateBlobWriter(basePath, serializerOptions, cancellationToken));
			public IStorageWriter CreateWriter(CancellationToken cancellationToken = default) => inner.CreateWriter(cancellationToken);
			public Task<BlobAlias[]> FindAliasesAsync(string name, int? maxResults = null, CancellationToken cancellationToken = default) => inner.FindAliasesAsync(name, maxResults, cancellationToken);
			public Task<IHashedBlobRef?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default) => inner.TryReadRefAsync(name, cacheTime, cancellationToken);
			public void GetStats(StorageStats stats) => inner.GetStats(stats);
		}

		class ThrowOnDisposeBlobWriter(IBlobWriter inner) : IBlobWriter
		{
			public BlobSerializerOptions Options => inner.Options;
			public ReadOnlyMemory<byte> WrittenMemory => inner.WrittenMemory;
			public int Length => inner.Length;
			public void AddAlias(string name, int rank, ReadOnlyMemory<byte> data) => inner.AddAlias(name, rank, data);
			public Task FlushAsync(CancellationToken cancellationToken = default) => inner.FlushAsync(cancellationToken);
			public IBlobWriter Fork() => inner.Fork();
			public ValueTask<IHashedBlobRef> CompleteAsync(BlobType type, CancellationToken cancellationToken = default) => inner.CompleteAsync(type, cancellationToken);
			public ValueTask<IHashedBlobRef<T>> CompleteAsync<T>(BlobType type, CancellationToken cancellationToken = default) => inner.CompleteAsync<T>(type, cancellationToken);
			public void WriteBlobHandleDangerous(IBlobRef handle) => inner.WriteBlobHandleDangerous(handle);
			public void WriteBlobRef(IHashedBlobRef blobRef) => inner.WriteBlobRef(blobRef);
			public Span<byte> GetSpan(int sizeHint = 0) => inner.GetSpan(sizeHint);
			public Memory<byte> GetMemory(int sizeHint = 0) => inner.GetMemory(sizeHint);
			public void Advance(int length) => inner.Advance(length);

			public ValueTask DisposeAsync()
			{
				GC.SuppressFinalize(this);
				return ValueTask.FromException(new IOException("Simulated S3 upload failure during writer disposal"));
			}
		}

		static List<T> DrainChannel<T>(Channel<T> channel)
		{
			List<T> items = [];
			while (channel.Reader.TryRead(out T? item))
			{
				items.Add(item);
			}
			return items;
		}

		static List<RpcCreateLogEventRequest> DrainAllEvents(FakeLogRpcClient rpc)
		{
			return [.. DrainChannel(rpc.CreateLogEventsChannel).SelectMany(r => r.Events)];
		}

		#endregion

		[TestMethod]
		public async Task FinalFlushSetsCompleteFlagAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 10; idx++)
				{
					logger.LogInformation("Line {Number}", idx);
				}
			}

			List<RpcUpdateLogRequest> requests = DrainChannel(hordeClient.LogRpc.UpdateLogChannel);
			Assert.IsTrue(requests.Count >= 1);
			Assert.IsTrue(requests[^1].Complete);
			Assert.AreEqual(10, requests[^1].LineCount);
		}

		[TestMethod]
		public async Task SmallDataPreservedOnDisposeAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				logger.LogInformation("Single line");
			}

			List<RpcUpdateLogRequest> requests = DrainChannel(hordeClient.LogRpc.UpdateLogChannel);
			Assert.AreEqual(1, requests.Count);
			Assert.IsTrue(requests[0].Complete);

			// Verify round-trip through storage
			IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(logId);
			LogNode file = await storageNamespace.CreateBlobRef(hordeClient.LogRpc.Logs[logId]).ReadBlobAsync<LogNode>();
			LogIndexNode index = await file.IndexRef.ReadBlobAsync();

			List<Utf8String> lines = [];
			foreach (LogChunkRef block in index.PlainTextChunkRefs)
			{
				LogChunkNode text = await block.Target.ReadBlobAsync();
				lines.AddRange(text.Lines);
			}

			Assert.AreEqual(1, lines.Count);
			Assert.AreEqual(new Utf8String("Single line"), lines[0]);
		}

		[TestMethod]
		public async Task SizeBasedFlushTriggersIntermediateUpdateAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 10000; idx++)
				{
					logger.LogInformation("This is a sufficiently long line of log output for testing purposes number {Number}", idx);
				}
			}

			List<RpcUpdateLogRequest> requests = DrainChannel(hordeClient.LogRpc.UpdateLogChannel);
			Assert.IsTrue(requests.Count >= 2);
			Assert.IsTrue(requests.Any(r => !r.Complete));
			Assert.IsTrue(requests[^1].Complete);

			for (int i = 1; i < requests.Count; i++)
			{
				Assert.IsTrue(requests[i].LineCount >= requests[i - 1].LineCount);
			}
		}

		[TestMethod]
		public async Task WarningEventsReportedAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 5; idx++)
				{
					logger.LogWarning("Warning {Number}", idx);
				}
			}

			List<RpcCreateLogEventRequest> allEvents = DrainAllEvents(hordeClient.LogRpc);
			Assert.AreEqual(5, allEvents.Count);
			Assert.IsTrue(allEvents.All(e => e.Severity == (int)LogEventSeverity.Warning));
		}

		[TestMethod]
		public async Task ErrorAndCriticalEventsReportedAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 3; idx++)
				{
					logger.LogError("Error {Number}", idx);
				}
				for (int idx = 0; idx < 2; idx++)
				{
					logger.LogCritical("Critical {Number}", idx);
				}
			}

			List<RpcCreateLogEventRequest> allEvents = DrainAllEvents(hordeClient.LogRpc);
			Assert.AreEqual(5, allEvents.Count);
			Assert.IsTrue(allEvents.All(e => e.Severity == (int)LogEventSeverity.Error));
		}

		[TestMethod]
		public Task WarningCapEnforcedAt200Async() => VerifyEventCapAt200Async(LogLevel.Warning);

		[TestMethod]
		public Task ErrorCapEnforcedAt200Async() => VerifyEventCapAt200Async(LogLevel.Error);

		static async Task VerifyEventCapAt200Async(LogLevel level)
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 250; idx++)
				{
					logger.Log(level, "Message {Number}", idx);
				}
			}

			List<RpcCreateLogEventRequest> allEvents = DrainAllEvents(hordeClient.LogRpc);
			Assert.AreEqual(200, allEvents.Count);
		}

		[TestMethod]
		public async Task MixedWarningsAndErrorsHaveIndependentCapsAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 210; idx++)
				{
					logger.LogWarning("Warning {Number}", idx);
				}
				for (int idx = 0; idx < 210; idx++)
				{
					logger.LogError("Error {Number}", idx);
				}
			}

			List<RpcCreateLogEventRequest> allEvents = DrainAllEvents(hordeClient.LogRpc);
			int warningCount = allEvents.Count(e => e.Severity == (int)LogEventSeverity.Warning);
			int errorCount = allEvents.Count(e => e.Severity == (int)LogEventSeverity.Error);

			Assert.AreEqual(200, warningCount);
			Assert.AreEqual(200, errorCount);
		}

		[TestMethod]
		public async Task InformationLinesDoNotCreateEventsAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < 100; idx++)
				{
					logger.LogInformation("Info {Number}", idx);
				}
			}

			List<RpcCreateLogEventRequest> allEvents = DrainAllEvents(hordeClient.LogRpc);
			Assert.AreEqual(0, allEvents.Count);
		}

		[TestMethod]
		public async Task StopAsyncIdempotentAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance);

			for (int idx = 0; idx < 10; idx++)
			{
				logger.LogInformation("Line {Number}", idx);
			}

			// Call StopAsync twice — should not throw
			await logger.StopAsync();
			await logger.StopAsync();

			// DisposeAsync after StopAsync
			await logger.DisposeAsync();

			List<RpcUpdateLogRequest> requests = DrainChannel(hordeClient.LogRpc.UpdateLogChannel);
			int completeCount = requests.Count(r => r.Complete);
			Assert.AreEqual(1, completeCount);
		}

		[TestMethod]
		public async Task StorageRoundTripAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			const int Count = 1000;
			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance))
			{
				for (int idx = 0; idx < Count; idx++)
				{
					logger.LogInformation("Testing {Number}", idx);
				}
			}

			// Read the log back from storage
			IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(logId);
			LogNode file = await storageNamespace.CreateBlobRef(hordeClient.LogRpc.Logs[logId]).ReadBlobAsync<LogNode>();

			// Check index text
			List<Utf8String> extractedIndexText = [];
			LogIndexNode index = await file.IndexRef.ReadBlobAsync();
			foreach (LogChunkRef block in index.PlainTextChunkRefs)
			{
				LogChunkNode text = await block.Target.ReadBlobAsync();
				extractedIndexText.AddRange(text.Lines);
			}

			Assert.AreEqual(Count, extractedIndexText.Count);
			for (int idx = 0; idx < Count; idx++)
			{
				Assert.AreEqual(new Utf8String($"Testing {idx}"), extractedIndexText[idx]);
			}

			// Check body text
			List<string> extractedBodyText = [];
			foreach (LogChunkRef blockRef in file.TextChunkRefs)
			{
				LogChunkNode blockText = await blockRef.Target.ReadBlobAsync();
				foreach (Utf8String line in blockText.Lines)
				{
					LogEvent logEvent = LogEvent.Read(line.Span);
					extractedBodyText.Add(logEvent.ToString());
				}
			}

			Assert.AreEqual(Count, extractedBodyText.Count);
			for (int idx = 0; idx < Count; idx++)
			{
				Assert.AreEqual($"Testing {idx}", extractedBodyText[idx]);
			}
		}

		[TestMethod]
		[Timeout(15000)]
		public async Task TimeBasedFlushTriggersBeforeDisposeAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance, flushInterval: TimeSpan.FromSeconds(1));

			for (int idx = 0; idx < 10; idx++)
			{
				logger.LogInformation("Line {Number}", idx);
			}

			// Wait for an intermediate flush (before dispose)
			using CancellationTokenSource cts = new(TimeSpan.FromSeconds(10));
			RpcUpdateLogRequest request = await hordeClient.LogRpc.UpdateLogChannel.Reader.ReadAsync(cts.Token);
			Assert.IsFalse(request.Complete);
		}

		[TestMethod]
		[Timeout(15000)]
		public async Task EmptyBufferDoesNotTriggerTimeFlushAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using (ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance, flushInterval: TimeSpan.FromSeconds(1)))
			{
				// Write nothing, wait longer than flush interval
				await Task.Delay(TimeSpan.FromSeconds(3));
			}

			List<RpcUpdateLogRequest> requests = DrainChannel(hordeClient.LogRpc.UpdateLogChannel);

			// Only the final flush should be present (Complete == true), and only if data was written.
			// With no data, there should still be a final Complete call from dispose.
			Assert.IsTrue(requests.All(r => r.Complete));
		}

		[TestMethod]
		public async Task MinimumLevelFilteringAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			await using ServerLogger logger = new(hordeClient, logId, LogLevel.Warning, NullLogger.Instance);

			Assert.IsFalse(logger.IsEnabled(LogLevel.Trace));
			Assert.IsFalse(logger.IsEnabled(LogLevel.Debug));
			Assert.IsFalse(logger.IsEnabled(LogLevel.Information));
			Assert.IsTrue(logger.IsEnabled(LogLevel.Warning));
			Assert.IsTrue(logger.IsEnabled(LogLevel.Error));
			Assert.IsTrue(logger.IsEnabled(LogLevel.Critical));
		}

		[TestMethod]
		public async Task LogAfterStopDoesNotThrowAsync()
		{
			await using FakeHordeClient hordeClient = new();
			LogId logId = default;

			ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance);
			logger.LogInformation("Before stop");
			await logger.StopAsync();

			// Should not throw even though the channel is closed
			logger.LogInformation("After stop");

			await logger.DisposeAsync();
		}

		[TestMethod]
		public async Task DisposeAsync_DoesNotThrow_WhenWriterDisposeThrowsAsync()
		{
			await using FakeHordeClient hordeClient = new();
			hordeClient.StorageNamespaceDecorator = ns => new ThrowOnWriterDisposeStorageNamespace(ns);
			LogId logId = default;

			// The logger works normally during its lifecycle...
			ServerLogger logger = new(hordeClient, logId, LogLevel.Information, NullLogger.Instance);
			logger.LogInformation("Line before dispose");

			// ... but DisposeAsync must not propagate the IOException from the writer.
			// Before the fix, this would throw IOException("Simulated S3 upload failure during writer disposal").
			await logger.DisposeAsync();
		}
	}
}
