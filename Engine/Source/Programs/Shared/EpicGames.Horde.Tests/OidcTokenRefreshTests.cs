// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.OIDC;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests;

/// <summary>
/// Testable subclass of OidcTokenClient that overrides DoRefreshAsync (no real OIDC backend)
/// and AcquireRefreshLock (in-process SemaphoreSlim instead of file lock).
/// Optionally blocks mid-refresh until signaled, enabling deterministic cancellation testing.
/// </summary>
sealed class TestableOidcTokenClient(
	string name,
	ITokenStore tokenStore,
	Func<string, TokenRefreshResult> refreshHandler,
	bool asyncGated = false)
	: OidcTokenClient(name, CreateDummyProviderInfo(), TimeSpan.FromMinutes(20), tokenStore), IDisposable
{
	private readonly TaskCompletionSource? _enteredRefresh = asyncGated ? new() : null;
	private readonly TaskCompletionSource? _proceedWithRefresh = asyncGated ? new() : null;
	private readonly SemaphoreSlim _testLock = new(1, 1);
	public int RefreshCallCount { get; private set; }

	/// <summary>Completes when DoRefreshAsync has been entered (only when constructed with async gating).</summary>
	public Task EnteredRefreshTask => _enteredRefresh?.Task ?? Task.CompletedTask;

	/// <summary>Signal the refresh to continue (call after cancelling external CTS).</summary>
	public void AllowRefreshToComplete() => _proceedWithRefresh?.TrySetResult();

	public void Dispose() => _testLock.Dispose();

	protected override async Task<TokenRefreshResult> DoRefreshAsync(
		string refreshToken, CancellationToken cancellationToken)
	{
		RefreshCallCount++;

		if (_enteredRefresh != null)
		{
			_enteredRefresh.TrySetResult();
			await _proceedWithRefresh!.Task;
			cancellationToken.ThrowIfCancellationRequested();
		}

		return await Task.FromResult(refreshHandler(refreshToken));
	}

	protected override async Task<IDisposable> AcquireRefreshLockAsync(CancellationToken cancellationToken)
	{
		await _testLock.WaitAsync(cancellationToken);
		return new SemaphoreReleaser(_testLock);
	}

	private sealed class SemaphoreReleaser(SemaphoreSlim s) : IDisposable
	{
		public void Dispose() => s.Release();
	}

	internal static ProviderInfo CreateDummyProviderInfo() => new()
	{
		ServerUri = new Uri("https://fake-oidc.test"),
		ClientId = "test-client",
		DisplayName = "Test",
		RedirectUri = new Uri("http://localhost:12345/callback"),
		UseDiscoveryDocument = false
	};
}

[TestClass]
public sealed class OidcTokenRefreshTests
{
	private const string Provider = "test";

	private static TokenRefreshResult SuccessResult(string accessToken = "newAccess", string refreshToken = "newRefresh") =>
		new(IsError: false, Error: null, ErrorDescription: null,
			AccessToken: accessToken, RefreshToken: refreshToken,
			AccessTokenExpiration: DateTimeOffset.UtcNow.AddHours(1));

	private static TokenRefreshResult InvalidGrantResult(string description = "Token expired") =>
		new(IsError: true, Error: "invalid_grant", ErrorDescription: description,
			AccessToken: null, RefreshToken: null, AccessTokenExpiration: default);

	private static (InMemoryTokenStore Store, TestableOidcTokenClient Client) CreateClient(
		string initialToken, Func<string, TokenRefreshResult> refreshHandler, bool asyncGated = false)
	{
		InMemoryTokenStore store = new();
		store.AddRefreshToken(Provider, initialToken);
		return (store, new TestableOidcTokenClient(Provider, store, refreshHandler, asyncGated));
	}

	/// <summary>
	/// When the first refresh returns invalid_grant and a different token exists in the store,
	/// the client should re-read the store and retry with the fresh token.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_InvalidGrant_ReadsStoreAndRetriesAsync()
	{
		using InMemoryTokenStore store = new();
		store.AddRefreshToken(Provider, "tokenA");
		using TestableOidcTokenClient client = new(Provider, store, refreshToken =>
		{
			if (refreshToken == "tokenA")
			{
				store.AddRefreshToken(Provider, "tokenB"); // simulate another process rotating the token
				return InvalidGrantResult("Token has been rotated");
			}
			return SuccessResult("newAccess", "tokenC");
		});
		{
			IOidcTokenInfo? result = await client.TryGetAccessTokenAsync(CancellationToken.None);

			Assert.IsNotNull(result);
			Assert.AreEqual("newAccess", result!.AccessToken);
			Assert.IsTrue(store.TryGetRefreshToken(Provider, out string? storedToken));
			Assert.AreEqual("tokenC", storedToken);
			Assert.AreEqual(2, client.RefreshCallCount);
		}
	}

	/// <summary>
	/// When invalid_grant is returned but the store still has the same token,
	/// the client should return null (no fresh token available).
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_InvalidGrant_NoFreshToken_ReturnsNullAsync()
	{
		(InMemoryTokenStore store, TestableOidcTokenClient client) = CreateClient("tokenA", _ => InvalidGrantResult());
		using (store)
		{
			IOidcTokenInfo? result = await client.TryGetAccessTokenAsync(CancellationToken.None);

			Assert.IsNull(result);
			Assert.AreEqual(1, client.RefreshCallCount);
		}
	}

	/// <summary>
	/// A normal successful refresh should work and store the new token.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_ValidRefresh_SucceedsAsync()
	{
		(InMemoryTokenStore store, TestableOidcTokenClient client) = CreateClient("tokenA", _ => SuccessResult("access1", "tokenB"));
		using (store)
		{
			IOidcTokenInfo? result = await client.TryGetAccessTokenAsync(CancellationToken.None);

			Assert.IsNotNull(result);
			Assert.AreEqual("access1", result!.AccessToken);
			Assert.IsTrue(store.TryGetRefreshToken(Provider, out string? storedToken));
			Assert.AreEqual("tokenB", storedToken);
			Assert.AreEqual(1, client.RefreshCallCount);
		}
	}

	/// <summary>
	/// The client should always read the store fresh at refresh time, not use a stale
	/// cached copy from construction.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_AlwaysReadsFromStoreFreshAsync()
	{
		(InMemoryTokenStore store, TestableOidcTokenClient client) = CreateClient("tokenA", refreshToken =>
		{
			if (refreshToken == "tokenA")
			{
				return InvalidGrantResult("Stale");
			}
			return SuccessResult("freshAccess", "tokenC");
		});
		using (store)
		{
			// Externally rotate the token (simulating another process)
			store.AddRefreshToken(Provider, "tokenB");

			IOidcTokenInfo? result = await client.TryGetAccessTokenAsync(CancellationToken.None);

			Assert.IsNotNull(result);
			Assert.AreEqual("freshAccess", result!.AccessToken);
		}
	}

	/// <summary>
	/// Concurrent calls to TryGetAccessTokenAsync should be serialized by the lock.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_ConcurrentCalls_SerializedAsync()
	{
		int concurrentCount = 0;
		int maxConcurrent = 0;

		using InMemoryTokenStore store = new();
		store.AddRefreshToken(Provider, "tokenA");
		using TestableOidcTokenClient client = new(Provider, store, refreshToken =>
		{
			int current = Interlocked.Increment(ref concurrentCount);
			int snapshot;
			do
			{
				snapshot = Volatile.Read(ref maxConcurrent);
			}
			while (current > snapshot && Interlocked.CompareExchange(ref maxConcurrent, current, snapshot) != snapshot);

			Thread.Sleep(50); // small delay to increase chance of overlap detection
			Interlocked.Decrement(ref concurrentCount);

			string newToken = $"token_{Guid.NewGuid():N}";
			store.AddRefreshToken(Provider, newToken);
			return SuccessResult($"access_{refreshToken}", newToken);
		});
		{
			Task<IOidcTokenInfo?> task1 = client.TryGetAccessTokenAsync(CancellationToken.None);
			Task<IOidcTokenInfo?> task2 = client.TryGetAccessTokenAsync(CancellationToken.None);

			IOidcTokenInfo?[] results = await Task.WhenAll(task1, task2);

			Assert.IsNotNull(results[0]);
			Assert.IsNotNull(results[1]);
			Assert.AreEqual(1, maxConcurrent, "Refresh calls should have been serialized by the lock");
		}
	}

	/// <summary>
	/// When external cancellation fires mid-refresh (e.g. process shutdown), the new token
	/// must still be persisted. This is the key bug scenario: Okta rotates the token but
	/// OperationCanceledException skips AddRefreshToken, losing the new token permanently.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_ExternalCancellationDuringRefresh_StillPersistsTokenAsync()
	{
		using CancellationTokenSource externalCts = new();
		(InMemoryTokenStore store, TestableOidcTokenClient client) =
			CreateClient("tokenA", _ => SuccessResult("newAccess", "tokenB"), asyncGated: true);
		using (store)
		{
			Task<IOidcTokenInfo?> refreshTask = client.TryGetAccessTokenAsync(externalCts.Token);
			await client.EnteredRefreshTask;

			// Simulate process shutdown — cancel AFTER Okta has processed the rotation
			await externalCts.CancelAsync();
			client.AllowRefreshToComplete();

			IOidcTokenInfo? result = await refreshTask;

			Assert.IsNotNull(result);
			Assert.IsTrue(store.TryGetRefreshToken(Provider, out string? storedToken));
			Assert.AreEqual("tokenB", storedToken, "New token must be persisted even when external cancellation fires");
		}
	}

	/// <summary>
	/// If cancellation is already requested before refresh starts, the client should
	/// throw without attempting the HTTP call.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_CancelledBeforeRefresh_DoesNotCallRefreshAsync()
	{
		(InMemoryTokenStore store, TestableOidcTokenClient client) =
			CreateClient("tokenA", _ => SuccessResult());
		using (store)
		{
			using CancellationTokenSource cts = new();
			await cts.CancelAsync();

			await Assert.ThrowsExactlyAsync<TaskCanceledException>(
				() => client.TryGetAccessTokenAsync(cts.Token));

			Assert.AreEqual(0, client.RefreshCallCount);
			Assert.IsTrue(store.TryGetRefreshToken(Provider, out string? token));
			Assert.AreEqual("tokenA", token, "Original token must remain on disk");
		}
	}
}
