// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.TestTools.UnitTesting;

[assembly: Parallelize(Workers = 0, Scope = ExecutionScope.MethodLevel)]
