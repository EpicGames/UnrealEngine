// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class BackendTests
	{
		class TempDir : IDisposable
		{
			readonly DirectoryReference _cacheDir;

			public DirectoryReference Location => _cacheDir;

			public TempDir(string name)
			{
				_cacheDir = new DirectoryReference(name);
				DirectoryReference.CreateDirectory(_cacheDir);
				FileUtils.ForceDeleteDirectoryContents(_cacheDir);
			}

			public void Dispose()
			{
				FileUtils.ForceDeleteDirectoryContents(_cacheDir);
				DirectoryReference.Delete(_cacheDir);
			}
		}

		[TestMethod]
		public async Task TestMemoryBackendAsync()
		{
			MemoryStorageBackend backend = new();
			await TestBackendAsync(backend);
		}

		[TestMethod]
		public async Task TestFileBackendAsync()
		{
			using (TempDir tempDir = new("Cache-" + DateTime.UtcNow.Ticks))
			{
				using MemoryMappedFileCache memoryMappedFileCache = new();
				FileStorageBackend backend = new(tempDir.Location, memoryMappedFileCache, NullLogger.Instance);
				await TestBackendAsync(backend);
			}
		}

		[TestMethod]
		public async Task TestFileBackendFileMappingAsync()
		{
			using (TempDir tempDir = new("Cache-" + DateTime.UtcNow.Ticks))
			{
				using MemoryMappedFileCache memoryMappedFileCache = new();
				FileStorageBackend backend = new(tempDir.Location, memoryMappedFileCache, NullLogger.Instance);
				BlobLocator path = await backend.WriteBytesAsync(Encoding.UTF8.GetBytes("hello world"), []);

				using (IReadOnlyMemoryOwner<byte> handle = await backend.ReadBlobAsync(path, 0, null))
				{
					Assert.AreEqual("hello world", Encoding.UTF8.GetString(handle.Memory.Span));
				}
				using (IReadOnlyMemoryOwner<byte> handle = await backend.ReadBlobAsync(path, 0, 5))
				{
					Assert.AreEqual("hello", Encoding.UTF8.GetString(handle.Memory.Span));
				}
				using (IReadOnlyMemoryOwner<byte> handle = await backend.ReadBlobAsync(path, 4, 3))
				{
					Assert.AreEqual("o w", Encoding.UTF8.GetString(handle.Memory.Span));
				}
			}
		}

		[TestMethod]
		public async Task TestCacheBackendAsync()
		{
			using (TempDir tempDir = new("Cache-" + DateTime.UtcNow.Ticks))
			{
				using StorageBackendCache cache = new(tempDir.Location, 12, NullLogger.Instance);

				MemoryStorageBackend memoryBackend = new();
				IStorageBackend cacheBackend = cache.CreateWrapper("", memoryBackend);

				await TestBackendAsync(cacheBackend);

				Assert.AreEqual(1, cache.Items.Count());
				byte[] value = await cacheBackend.ReadBytesAsync(cache.GetLocators().First());
				Assert.IsTrue(value.SequenceEqual(Encoding.UTF8.GetBytes("item 2")));

				byte[] data3 = Encoding.UTF8.GetBytes("3");
				BlobLocator path3 = await cacheBackend.WriteBytesAsync(data3, []);
				await cacheBackend.ReadBytesAsync(path3);

				byte[] data4 = Encoding.UTF8.GetBytes("4");
				BlobLocator path4 = await cacheBackend.WriteBytesAsync(data4, []);
				await cacheBackend.ReadBytesAsync(path4);

				byte[] data5 = Encoding.UTF8.GetBytes("5");
				BlobLocator path5 = await cacheBackend.WriteBytesAsync(data5, []);
				await cacheBackend.ReadBytesAsync(path5);

				HashSet<BlobLocator> paths = [.. cache.GetLocators()];
				Assert.AreEqual(4, paths.Count);
				Assert.IsTrue(paths.Contains(path3));
				Assert.IsTrue(paths.Contains(path4));
				Assert.IsTrue(paths.Contains(path5));

				byte[] data6 = Encoding.UTF8.GetBytes("12345678901");
				BlobLocator path6 = await cacheBackend.WriteBytesAsync(data6, []);
				await cacheBackend.ReadBytesAsync(path6);

				paths = [.. cache.GetLocators()];
				Assert.AreEqual(2, paths.Count);
				Assert.IsTrue(paths.Contains(path5));
				Assert.IsTrue(paths.Contains(path6));

				await cacheBackend.ReadBytesAsync(path3);
				await cacheBackend.ReadBytesAsync(path4);

				paths = [.. cache.GetLocators()];
				Assert.AreEqual(2, paths.Count);
				Assert.IsTrue(paths.Contains(path3));
				Assert.IsTrue(paths.Contains(path4));
			}
		}

		static async Task TestBackendAsync(IStorageBackend backend)
		{
			byte[] data1 = Encoding.UTF8.GetBytes("hello world");
			byte[] data2 = Encoding.UTF8.GetBytes("item 2");

			BlobLocator path1 = await backend.WriteBytesAsync(data1, []);
			byte[] outputData1 = await backend.ReadBytesAsync(path1);

			BlobLocator path2 = await backend.WriteBytesAsync(data2, []);
			byte[] outputData2 = await backend.ReadBytesAsync(path2);

			Assert.IsTrue(data1.SequenceEqual(outputData1));
			Assert.IsTrue(data2.SequenceEqual(outputData2));
		}
	}
}
