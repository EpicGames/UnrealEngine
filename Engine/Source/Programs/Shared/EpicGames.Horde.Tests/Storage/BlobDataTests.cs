// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class BlobDataTests
	{
		class DummyHandle : IBlobRef
		{
			readonly BlobLocator _locator;

			public IBlobRef Innermost => this;

			public DummyHandle(string locator) => _locator = new BlobLocator(new Utf8String(locator));
			public ValueTask FlushAsync(CancellationToken cancellationToken = default) => default;
			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default) => throw new NotImplementedException();

			public bool TryGetLocator(out BlobLocator locator)
			{
				locator = _locator;
				return true;
			}

			public override bool Equals(object? obj) => obj is DummyHandle other && _locator == other._locator;
			public override int GetHashCode() => _locator.GetHashCode();
		}

		[TestMethod]
		public void HeaderSerialization()
		{
			List<IBlobRef> refs = [new DummyHandle("hello"), new DummyHandle("world")];

			using BlobData blobData = new(new BlobType(Guid.NewGuid(), 4), new byte[] { 1, 2, 3 }, refs);

			byte[] data = EncodedBlobData.Create(blobData);
			EncodedBlobData packet = new(data);
			Assert.AreEqual(49, data.Length);

			Assert.AreEqual(blobData.Type.Guid, packet.Type.Guid);
			Assert.AreEqual(blobData.Type.Version, packet.Type.Version);
			Assert.AreEqual(blobData.Imports.Count, packet.Imports.Count);

			for (int idx = 0; idx < packet.Imports.Count; idx++)
			{
				string locator = blobData.Imports[idx].GetLocator().ToString();
				string encoded = packet.Imports[idx].ToString();
				Assert.AreEqual(locator, encoded);
			}
		}
	}
}
