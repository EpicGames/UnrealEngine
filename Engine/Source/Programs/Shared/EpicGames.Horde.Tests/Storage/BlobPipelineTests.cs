// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class BlobPipelineTests
	{
		static readonly BlobType s_blobType = new(Guid.Empty, 1);

		[TestMethod]
		public async Task TestAsync()
		{
			BundleStorageNamespace client = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);

			List<IHashedBlobRef> handles = [];
			await using (IBlobWriter writer = client.CreateBlobWriter())
			{
				for (int idx = 0; idx < 4000; idx++)
				{
					writer.WriteInt32(idx);
					handles.Add(await writer.CompleteAsync(s_blobType));
				}
			}

			await using (BlobPipeline<IoHash> pipeline = new())
			{
				foreach (IHashedBlobRef blobRef in handles)
				{
					pipeline.Add(new BlobRequest<IoHash>(blobRef, blobRef.Hash));
				}
				pipeline.FinishAdding();

				List<BlobResponse<IoHash>> responses = await pipeline.ReadAllAsync().ToListAsync();
				Assert.AreEqual(handles.Count, responses.Count);
			}
		}
	}
}
