// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Bundles.V1;
using EpicGames.Horde.Storage.Clients;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Serialization;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class BundleStoreTests
	{
		[TestMethod]
		public async Task CreateBundlesManuallyV1Async()
		{
			byte[] a = CreateBundleManually();
			byte[] b = await CreateBundleNormalAsync();
			Assert.IsTrue(a.SequenceEqual(b));
		}

		[BlobConverter(typeof(TextNodeConverter))]
		class TextNode
		{
			public string Text { get; }

			public TextNode(string text) => Text = text;
		}

		class TextNodeConverter : BlobConverter<TextNode>
		{
			public static BlobType BlobType { get; } = new BlobType("{F63606D4-4061-5DBB-446F-55A69E22654F}", 1);

			public override TextNode Read(IBlobReader reader, BlobSerializerOptions options)
			{
				string text = reader.ReadString();
				return new TextNode(text);
			}

			public override BlobType Write(IBlobWriter writer, TextNode value, BlobSerializerOptions options)
			{
				writer.WriteString(value.Text);
				return BlobType;
			}
		}

		static async Task<byte[]> CreateBundleNormalAsync()
		{
			MemoryStorageBackend memoryStore = new();
			BundleStorageNamespace store = new(memoryStore, BundleCache.None, new BundleOptions { MaxVersion = BundleVersion.ImportHashes, CompressionFormat = BundleCompressionFormat.None }, NullLogger.Instance);
			await using IBlobWriter writer = store.CreateBlobWriter();

			TextNode node = new("Hello world");
			IHashedBlobRef<TextNode> handle = await writer.WriteBlobAsync(node);
			await writer.FlushAsync();

			using IReadOnlyMemoryOwner<byte> owner = await memoryStore.ReadBlobAsync(handle.GetLocator().BaseLocator, 0, null);
			return owner.Memory.ToArray();
		}

		static byte[] CreateBundleManually()
		{
			ArrayMemoryWriter payloadWriter = new(200);
			payloadWriter.WriteString("Hello world");
			byte[] payload = payloadWriter.WrittenMemory.ToArray();

			List<BlobType> types = [new BlobType("{F63606D4-4061-5DBB-446F-55A69E22654F}", 1)];

			List<BundleExport> exports = [new BundleExport(0, 0, 0, payload.Length, [])];

			List<BundlePacket> packets = [new BundlePacket(BundleCompressionFormat.None, 0, payload.Length, payload.Length)];

			BundleHeader header = new([.. types], [], [.. exports], [.. packets]);

			ReadOnlySequenceBuilder<byte> builder = new();
			header.AppendTo(builder);
			builder.Append(payload);
			return builder.Construct().ToArray();
		}

		[TestMethod]
		public async Task TestTreeAsync()
		{
			MemoryStorageBackend blobStore = new();

			BundleOptions bundleOptions = new() { MaxBlobSize = 1024 * 1024 };
			BundleStorageNamespace bundleStore = new(blobStore, BundleCache.None, bundleOptions, NullLogger.Instance);

			await TestTreeAsync(bundleStore);

			Assert.AreEqual(1, blobStore.Blobs.Count);
			Assert.AreEqual(1, blobStore.Refs.Count);
		}

		[TestMethod]
		public async Task TestTreeSeparateBlobsAsync()
		{
			MemoryStorageBackend blobStore = new();

			BundleOptions bundleOptions = new() { MaxBlobSize = 1 };
			BundleStorageNamespace bundleStore = new(blobStore, BundleCache.None, bundleOptions, NullLogger.Instance);

			await TestTreeAsync(bundleStore);

			Assert.AreEqual(5, blobStore.Blobs.Count);
			Assert.AreEqual(1, blobStore.Refs.Count);
		}

		[BlobConverter(typeof(SimpleNodeConverter))]
		class SimpleNode
		{
			public static BlobType BlobType { get; } = new BlobType("{F63606D4-4061-5DBB-446F-55A69F22654F}", 1);

			public ReadOnlySequence<byte> Data { get; }
			public IReadOnlyList<IHashedBlobRef<SimpleNode>> Refs { get; }

			public SimpleNode(ReadOnlySequence<byte> data, IReadOnlyList<IHashedBlobRef<SimpleNode>> refs)
			{
				Data = data;
				Refs = refs;
			}
		}

		class SimpleNodeConverter : BlobConverter<SimpleNode>
		{
			public override SimpleNode Read(IBlobReader reader, BlobSerializerOptions options)
			{
				ReadOnlyMemory<byte> data = reader.ReadVariableLengthBytes();
				IReadOnlyList<IHashedBlobRef<SimpleNode>> refs = reader.ReadVariableLengthArray(() => reader.ReadBlobRef<SimpleNode>());
				return new SimpleNode(new ReadOnlySequence<byte>(data), refs);
			}

			public override BlobType Write(IBlobWriter writer, SimpleNode value, BlobSerializerOptions options)
			{
				writer.WriteVariableLengthBytes(value.Data);
				writer.WriteVariableLengthArray(value.Refs, x => writer.WriteBlobRef(x));
				return SimpleNode.BlobType;
			}
		}

		static async Task TestTreeAsync(BundleStorageNamespace store)
		{
			// Generate a tree
			{
				await using IBlobWriter writer = store.CreateBlobWriter("test");

				SimpleNode node1 = new(new ReadOnlySequence<byte>([1]), []);
				SimpleNode node2 = new(new ReadOnlySequence<byte>([2]), [await writer.WriteBlobAsync(node1)]);
				SimpleNode node3 = new(new ReadOnlySequence<byte>([3]), [await writer.WriteBlobAsync(node2)]);
				SimpleNode node4 = new(new ReadOnlySequence<byte>([4]), []);

				SimpleNode root = new(new ReadOnlySequence<byte>([5]), [await writer.WriteBlobAsync(node4), await writer.WriteBlobAsync(node3)]);
				IHashedBlobRef<SimpleNode> rootRef = await writer.WriteBlobAsync(root);
				await writer.FlushAsync();

				await store.AddRefAsync(new RefName("test"), rootRef);

				BundleReader reader = new(store, BundleCache.None, NullLogger.Instance);
				await CheckTreeAsync(root);
			}

			// Check we can read it back in
			{
				SimpleNode root = await store.ReadRefTargetAsync<SimpleNode>(new RefName("test"));
				await CheckTreeAsync(root);
			}
		}

		static async Task CheckTreeAsync(SimpleNode root)
		{
			SimpleNode node5 = root;
			byte[] data5 = node5.Data.ToArray();
			Assert.IsTrue(data5.SequenceEqual(new byte[] { 5 }));
			IReadOnlyList<IHashedBlobRef<SimpleNode>> refs5 = node5.Refs;
			Assert.AreEqual(2, refs5.Count);

			SimpleNode node4 = await refs5[0].ReadBlobAsync();
			byte[] data4 = node4.Data.ToArray();
			Assert.IsTrue(data4.SequenceEqual(new byte[] { 4 }));
			IReadOnlyList<IHashedBlobRef<SimpleNode>> refs4 = node4.Refs;
			Assert.AreEqual(0, refs4.Count);

			SimpleNode node3 = await refs5[1].ReadBlobAsync();
			byte[] data3 = node3.Data.ToArray();
			Assert.IsTrue(data3.SequenceEqual(new byte[] { 3 }));
			IReadOnlyList<IHashedBlobRef<SimpleNode>> refs3 = node3.Refs;
			Assert.AreEqual(1, refs3.Count);

			SimpleNode node2 = await refs3[0].ReadBlobAsync();
			byte[] data2 = node2.Data.ToArray();
			Assert.IsTrue(data2.SequenceEqual(new byte[] { 2 }));
			IReadOnlyList<IHashedBlobRef<SimpleNode>> refs2 = node2.Refs;
			Assert.AreEqual(1, refs2.Count);

			SimpleNode node1 = await refs2[0].ReadBlobAsync();
			byte[] data1 = node1.Data.ToArray();
			Assert.IsTrue(data1.SequenceEqual(new byte[] { 1 }));
			IReadOnlyList<IHashedBlobRef<SimpleNode>> refs1 = node1.Refs;
			Assert.AreEqual(0, refs1.Count);
		}

		[TestMethod]
		public async Task SimpleNodeAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			RefName refName = new("test");

			IHashedBlobRef<SimpleNode> inputRef;
			await using (IBlobWriter writer = store.CreateBlobWriter(refName))
			{
				inputRef = await writer.WriteBlobAsync(new SimpleNode(new ReadOnlySequence<byte>([(byte)123]), []));
			}
			await store.AddRefAsync(refName, inputRef);

			SimpleNode node = await store.ReadRefTargetAsync<SimpleNode>(refName);

			Assert.AreEqual(123, node.Data.FirstSpan[0]);
		}

		[TestMethod]
		public async Task DirectoryNodesAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			// Generate a tree
			{
				await using (IBlobWriter writer = store.CreateBlobWriter(new RefName("test")))
				{
					DirectoryNode world = new();
					IHashedBlobRef<DirectoryNode> worldRef = await writer.WriteBlobAsync(world);

					DirectoryNode hello = new();
					hello.AddDirectory(new DirectoryEntry("world", 0, worldRef));
					IHashedBlobRef<DirectoryNode> helloRef = await writer.WriteBlobAsync(hello);

					DirectoryNode root = new(DirectoryFlags.None);
					root.AddDirectory(new DirectoryEntry("hello", 0, helloRef));
					IHashedBlobRef<DirectoryNode> rootRef = await writer.WriteBlobAsync(root);

					await writer.FlushAsync();

					await store.AddRefAsync(new RefName("test"), rootRef);
				}
			}

			// Check we can read it back in
			{
				DirectoryNode root = await store.ReadRefTargetAsync<DirectoryNode>(new RefName("test"));
				await CheckDirectoryTreeAsync(root);
			}
		}

		static async Task CheckDirectoryTreeAsync(DirectoryNode root)
		{
			Assert.AreEqual(1, root.Directories.Count);
			Assert.AreEqual("hello", root.Directories.First().Name);

			DirectoryNode hello = await root.Directories.First().Handle.ReadBlobAsync();
			Assert.AreEqual(1, hello.Directories.Count);
			Assert.AreEqual("world", hello.Directories.First().Name);

			DirectoryNode world = await hello.Directories.First().Handle.ReadBlobAsync();
			Assert.AreEqual(0, world.Directories.Count);
		}

		[TestMethod]
		public async Task FileNodesAsync()
		{
			using IMemoryCache cache = new MemoryCache(new MemoryCacheOptions());
			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
			BundleReader reader = new(store, BundleCache.None, NullLogger.Instance);

			// Generate a tree
			{
				await using IBlobWriter writer = store.CreateBlobWriter();

				using ChunkedDataWriter fileWriter = new(writer, new ChunkingOptions());
				ChunkedData fileHandle = await fileWriter.CreateAsync(Encoding.UTF8.GetBytes("world"), CancellationToken.None);

				List<FileUpdate> fileUpdates = [new FileUpdate("hello/world", FileEntryFlags.None, fileWriter.Length, fileHandle)];

				DirectoryNode root = new();
				await root.UpdateAsync(fileUpdates, writer);

				IHashedBlobRef<DirectoryNode> rootRef = await writer.WriteBlobAsync(root);
				await store.AddRefAsync(new RefName("test"), rootRef);

				await CheckFileTreeAsync(root);
			}

			// Check we can read it back in
			{
				DirectoryNode root = await store.ReadRefTargetAsync<DirectoryNode>(new RefName("test"));
				await CheckFileTreeAsync(root);
			}
		}

		static async Task CheckFileTreeAsync(DirectoryNode root)
		{
			Assert.AreEqual(1, root.Directories.Count);
			Assert.AreEqual("hello", root.Directories.First().Name);

			DirectoryNode hello = await root.Directories.First().Handle.ReadBlobAsync();
			Assert.AreEqual(0, hello.Directories.Count);
			Assert.AreEqual(1, hello.Files.Count);
			Assert.AreEqual("world", hello.Files.First().Name);

			ChunkedDataNode world = await hello.Files.First().Target.ReadBlobAsync();

			byte[] worldData = await GetFileDataAsync(world);
			Assert.IsTrue(worldData.SequenceEqual(Encoding.UTF8.GetBytes("world")));
		}

		[TestMethod]
		public async Task StreamTestAsync()
		{
			BundleOptions bundleOptions = new() { MaxBlobSize = 1024 };
			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(bundleOptions, NullLogger.Instance);

			const int Length = 4096;

			byte[] chunk = new byte[Length];
			new Random(0).NextBytes(chunk);

			// Generate a tree
			ChunkedDataNodeRef nodeRef;
			{
				await using IBlobWriter writer = store.CreateBlobWriter();

				ChunkingOptions options = new();
				options.LeafOptions = new LeafChunkedDataNodeOptions(128, 256, 64 * 1024);

				using ChunkedDataWriter fileWriter = new(writer, options);
				for (int idx = 0; idx < chunk.Length / 16; idx++)
				{
					await fileWriter.AppendAsync(chunk.AsMemory(idx * 16, 16), CancellationToken.None);
				}

				nodeRef = (await fileWriter.FlushAsync(CancellationToken.None)).Root;
			}

			// Check we can read it back in
			{
				ChunkedDataNode newRoot = await nodeRef.ReadBlobAsync();

				using MemoryStream stream = new();
				await newRoot.CopyToStreamAsync(stream);

				byte[] output = stream.ToArray();
				Assert.IsTrue(chunk.SequenceEqual(output));
			}
		}

		[TestMethod]
		public async Task LargeFileTestAsync()
		{
			BundleOptions bundleOptions = new() { MaxBlobSize = 1024 };
			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(bundleOptions, NullLogger.Instance);

			const int Length = 1024;
			const int Copies = 4096;

			byte[] chunk = new byte[Length];
			new Random(0).NextBytes(chunk);

			byte[] data = new byte[chunk.Length * Copies];
			for (int idx = 0; idx < Copies; idx++)
			{
				chunk.CopyTo(data.AsSpan(idx * chunk.Length));
			}

			// Generate a tree
			DirectoryNode root;
			{
				await using DedupeBlobWriter writer = new(store.CreateBlobWriter());

				ChunkingOptions options = new();
				options.LeafOptions = new LeafChunkedDataNodeOptions(128, 256, 64 * 1024);

				using ChunkedDataWriter fileWriter = new(writer, options);
				ChunkedData chunkedData = await fileWriter.CreateAsync(data, CancellationToken.None);

				root = new DirectoryNode(DirectoryFlags.None);
				root.AddFile("test", FileEntryFlags.None, chunkedData);

				IHashedBlobRef<DirectoryNode> rootRef = await writer.WriteBlobAsync(root);
				await store.AddRefAsync(new RefName("test"), rootRef);
			}

			// Verify the data after writer is flushed/disposed
			await CheckLargeFileTreeAsync(root, data);

			// Check we can read it back in
			{
				DirectoryNode newRoot = await store.ReadRefTargetAsync<DirectoryNode>(new RefName("test"));
				await CompareTreesAsync(root, newRoot);
				await CheckLargeFileTreeAsync(root, data);

				FileEntry file = root.GetFileEntry("test");

				Dictionary<BlobLocator, long> locatorToSize = [];
				await GetUniqueBlobsAsync(file.Target, locatorToSize);

				long uniqueSize = locatorToSize.Sum(x => x.Value);
				Assert.IsTrue(uniqueSize < data.Length / 3); // random fraction meaning "lots of dedupe happened"
			}
		}

		static async Task GetUniqueBlobsAsync(IBlobRef handle, Dictionary<BlobLocator, long> locatorToSize)
		{
			using BlobData data = await handle.ReadBlobDataAsync();
			locatorToSize[handle.GetLocator()] = data.Data.Length;

			foreach (IBlobRef import in data.Imports)
			{
				await GetUniqueBlobsAsync(import, locatorToSize);
			}
		}

		static async Task CompareTreesAsync(DirectoryNode oldNode, DirectoryNode newNode)
		{
			Assert.AreEqual(oldNode.Length, newNode.Length);
			Assert.AreEqual(oldNode.Files.Count, newNode.Files.Count);
			Assert.AreEqual(oldNode.Directories.Count, newNode.Directories.Count);

			foreach ((FileEntry oldFileEntry, FileEntry newFileEntry) in oldNode.Files.Zip(newNode.Files))
			{
				ChunkedDataNode oldFile = await oldFileEntry.Target.ReadBlobAsync();
				ChunkedDataNode newFile = await newFileEntry.Target.ReadBlobAsync();
				await CompareTreesAsync(oldFile, newFile);
			}
		}

		static async Task CompareTreesAsync(ChunkedDataNode oldNode, ChunkedDataNode newNode)
		{
			if (oldNode is InteriorChunkedDataNode oldInteriorNode)
			{
				InteriorChunkedDataNode newInteriorNode = (InteriorChunkedDataNode)newNode;
				Assert.AreEqual(oldInteriorNode.Children.Count, newInteriorNode.Children.Count);

				int index = 0;
				foreach ((ChunkedDataNodeRef oldFileRef, ChunkedDataNodeRef newFileRef) in oldInteriorNode.Children.Zip(newInteriorNode.Children))
				{
					ChunkedDataNode oldFile = await oldFileRef.ReadBlobAsync();
					ChunkedDataNode newFile = await newFileRef.ReadBlobAsync();
					await CompareTreesAsync(oldFile, newFile);
					index++;
				}
			}
			else if (oldNode is LeafChunkedDataNode oldLeafNode)
			{
				LeafChunkedDataNode newLeafNode = (LeafChunkedDataNode)newNode;
				Assert.IsTrue(oldLeafNode.Data.Span.SequenceEqual(newLeafNode.Data.Span));
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		static async Task CheckLargeFileTreeAsync(DirectoryNode root, byte[] data)
		{
			Assert.AreEqual(0, root.Directories.Count);
			Assert.AreEqual(1, root.Files.Count);

			ChunkedDataNode world = await root.Files.First().Target.ReadBlobAsync();

			int length = await CheckFileDataAsync(world, data);
			Assert.AreEqual(data.Length, length);
		}

		static async Task<int> CheckFileDataAsync(ChunkedDataNode fileNode, ReadOnlyMemory<byte> data)
		{
			int offset = 0;
			if (fileNode is LeafChunkedDataNode leafNode)
			{
				Assert.IsTrue(leafNode.Data.Span.SequenceEqual(data.Span.Slice(offset, leafNode.Data.Length)));
				offset += leafNode.Data.Length;
			}
			else if (fileNode is InteriorChunkedDataNode interiorFileNode)
			{
				foreach (ChunkedDataNodeRef childRef in interiorFileNode.Children)
				{
					ChunkedDataNode child = await childRef.ReadBlobAsync();
					offset += await CheckFileDataAsync(child, data.Slice(offset));
				}
			}
			else
			{
				throw new NotImplementedException();
			}
			return offset;
		}

		static async Task<byte[]> GetFileDataAsync(ChunkedDataNode fileNode)
		{
			using (MemoryStream stream = new())
			{
				await fileNode.CopyToStreamAsync(stream);
				return stream.ToArray();
			}
		}
	}
}