// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Bundles.V1;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public sealed class BundleTests
	{
		[TestMethod]
		public void BuzHashTests()
		{
			byte[] data = new byte[4096];
			new Random(0).NextBytes(data);

			const int WindowSize = 128;

			uint rollingHash = 0;
			for (int maxIdx = 0; maxIdx < data.Length + WindowSize; maxIdx++)
			{
				int minIdx = maxIdx - WindowSize;

				if (maxIdx < data.Length)
				{
					rollingHash = BuzHash.Add(rollingHash, data[maxIdx]);
				}

				int length = Math.Min(maxIdx + 1, data.Length) - Math.Max(minIdx, 0);
				uint cleanHash = BuzHash.Add(0, data.AsSpan(Math.Max(minIdx, 0), length));
				Assert.AreEqual(rollingHash, cleanHash);

				if (minIdx >= 0)
				{
					rollingHash = BuzHash.Sub(rollingHash, data[minIdx], length);
				}
			}
		}

		[TestMethod]
		public async Task BasicChunkingTestsAsync()
		{
			BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);

			RefName refName = new("test");
			await using IBlobWriter writer = storage.CreateBlobWriter(refName);

			ChunkingOptions options = new();
			options.LeafOptions = new LeafChunkedDataNodeOptions(8, 8, 8);

			using ChunkedDataWriter fileNodeWriter = new(writer, options);

			ChunkedDataNode node;
			ChunkedDataNodeRef nodeRef;
			byte[] data = CreateBuffer(1024);

			nodeRef = (await fileNodeWriter.CreateAsync(data.AsMemory(0, 7), CancellationToken.None)).Root;
			node = await nodeRef.ReadBlobAsync();
			Assert.IsTrue(node is LeafChunkedDataNode);
			Assert.AreEqual(7, ((LeafChunkedDataNode)node).Data.Length);
			await TestBufferlessReadsAsync(nodeRef, data.AsMemory(0, 7));

			nodeRef = (await fileNodeWriter.CreateAsync(data.AsMemory(0, 8), CancellationToken.None)).Root;
			node = await nodeRef.ReadBlobAsync();
			Assert.IsTrue(node is LeafChunkedDataNode);
			Assert.AreEqual(8, ((LeafChunkedDataNode)node).Data.Length);
			await TestBufferlessReadsAsync(nodeRef, data.AsMemory(0, 8));

			nodeRef = (await fileNodeWriter.CreateAsync(data.AsMemory(0, 9), CancellationToken.None)).Root;
			node = await nodeRef.ReadBlobAsync();
			Assert.IsTrue(node is InteriorChunkedDataNode);
			Assert.AreEqual(2, ((InteriorChunkedDataNode)node).Children.Count);
			await TestBufferlessReadsAsync(nodeRef, data.AsMemory(0, 9));

			ChunkedDataNode? childNode1 = await ((InteriorChunkedDataNode)node).Children[0].ReadBlobAsync();
			Assert.IsNotNull(childNode1);
			Assert.IsTrue(childNode1 is LeafChunkedDataNode);
			Assert.AreEqual(8, ((LeafChunkedDataNode)childNode1!).Data.Length);

			ChunkedDataNode? childNode2 = await ((InteriorChunkedDataNode)node).Children[1].ReadBlobAsync();
			Assert.IsNotNull(childNode2);
			Assert.IsTrue(childNode2 is LeafChunkedDataNode);
			Assert.AreEqual(1, ((LeafChunkedDataNode)childNode2!).Data.Length);

			nodeRef = (await fileNodeWriter.CreateAsync(data, CancellationToken.None)).Root;
			node = await nodeRef.ReadBlobAsync();
			Assert.IsTrue(node is InteriorChunkedDataNode);
			await TestBufferlessReadsAsync(nodeRef, data);
		}

		private static byte[] CreateBuffer(int length)
		{
			byte[] output = GC.AllocateUninitializedArray<byte>(length);
			for (int i = 0; i < length; i++)
			{
				output[i] = (byte)i;
			}
			return output;
		}

		private static async Task TestBufferlessReadsAsync(ChunkedDataNodeRef nodeRef, ReadOnlyMemory<byte> expected)
		{
			ReadOnlyMemory<byte> read = await nodeRef.ReadAllBytesAsync();
			Assert.IsTrue(read.Span.SequenceEqual(expected.Span));
		}

		[TestMethod]
		public void SerializationTests()
		{
			BundleHeader oldHeader;
			{
				List<BlobType> types = [new BlobType(Guid.NewGuid(), 0)];

				List<BlobLocator> imports = [new BlobLocator("import1"), new BlobLocator("import2")];

				List<BundleExport> exports =
				[
					new BundleExport(0, 0, 0, 2, [new BundleExportRef(0, 5), new BundleExportRef(0, 6)]),
					new BundleExport(0, 1, 0, 3, [new BundleExportRef(-1, 0)]),
				];

				List<BundlePacket> packets =
				[
					new BundlePacket(BundleCompressionFormat.LZ4, 0, 20, 40),
					new BundlePacket(BundleCompressionFormat.LZ4, 20, 10, 20),
				];

				oldHeader = new BundleHeader([.. types], [.. imports], [.. exports], [.. packets]);
			}

			byte[] serializedData = oldHeader.ToByteArray();

			BundleHeader newHeader = BundleHeader.Read(serializedData);

			Assert.AreEqual(oldHeader.Imports.Count, newHeader.Imports.Count);
			for (int idx = 0; idx < oldHeader.Imports.Count; idx++)
			{
				Assert.AreEqual(oldHeader.Imports[idx], newHeader.Imports[idx]);
			}

			Assert.AreEqual(oldHeader.Exports.Count, newHeader.Exports.Count);
			for (int idx = 0; idx < oldHeader.Exports.Count; idx++)
			{
				BundleExport oldExport = oldHeader.Exports[idx];
				BundleExport newExport = newHeader.Exports[idx];

				Assert.AreEqual(oldExport.Hash, newExport.Hash);
				Assert.AreEqual(oldExport.Length, newExport.Length);
				Assert.IsTrue(oldExport.References.SequenceEqual(newExport.References));
			}

			Assert.AreEqual(oldHeader.Packets.Count, newHeader.Packets.Count);
			for (int idx = 0; idx < oldHeader.Packets.Count; idx++)
			{
				BundlePacket oldPacket = oldHeader.Packets[idx];
				BundlePacket newPacket = newHeader.Packets[idx];
				Assert.AreEqual(oldPacket.DecodedLength, newPacket.DecodedLength);
				Assert.AreEqual(oldPacket.EncodedLength, newPacket.EncodedLength);
			}
		}

		[TestMethod]
		public async Task BasicTestDirectoryAsync()
		{
			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);

			IHashedBlobRef<DirectoryNode> rootRef;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				DirectoryNode world = new();
				IHashedBlobRef<DirectoryNode> worldRef = await writer.WriteBlobAsync(world);

				DirectoryNode hello = new();
				hello.AddDirectory(new DirectoryEntry("world", 0, worldRef));
				IHashedBlobRef<DirectoryNode> helloRef = await writer.WriteBlobAsync(hello);

				DirectoryNode root = new();
				root.AddDirectory(new DirectoryEntry("hello", 0, helloRef));
				rootRef = await writer.WriteBlobAsync(root);

				await writer.FlushAsync();
			}

			RefName refName = new("testref");
			await store.AddRefAsync(refName, rootRef);

			// Should be stored inline
			MemoryStorageBackend memoryStore = (MemoryStorageBackend)store.Backend;
			Assert.AreEqual(1, memoryStore.Refs.Count);
			Assert.AreEqual(1, memoryStore.Blobs.Count);

			// Check the ref
			//			IBlobHandle refTarget =  await store.ReadRefTargetAsync(refName);
			//			IBlobHandle bundleTarget = store.CreateBlobHandle(refTarget.GetLocator().BaseLocator);
			//			using BlobData bundleData = await bundleTarget.ReadBlobDataAsync();

			// This is specific to V1 data
			//			BundleHeader bundleHeader = BundleHeader.Read(bundleData.Data);
			//			Assert.AreEqual(0, bundleHeader.Imports.Count);
			//			Assert.AreEqual(3, bundleHeader.Exports.Count);

			// Create a new bundle and read it back in again
			DirectoryNode newRoot = await store.ReadRefTargetAsync<DirectoryNode>(refName);

			Assert.AreEqual(0, newRoot.Files.Count);
			Assert.AreEqual(1, newRoot.Directories.Count);
			DirectoryNode? outputNode = await newRoot.TryOpenDirectoryAsync("hello");
			Assert.IsNotNull(outputNode);

			Assert.AreEqual(0, outputNode!.Files.Count);
			Assert.AreEqual(1, outputNode!.Directories.Count);
			DirectoryNode? outputNode2 = await outputNode.TryOpenDirectoryAsync("world");
			Assert.IsNotNull(outputNode2);

			Assert.AreEqual(0, outputNode2!.Files.Count);
			Assert.AreEqual(0, outputNode2!.Directories.Count);
		}

		[TestMethod]
		public async Task DedupTestsAsync()
		{
			BundleOptions bundleOptions = new();
			bundleOptions.MaxBlobSize = 1;

			BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(bundleOptions, NullLogger.Instance);

			await using (IBlobWriter writer = new DedupeBlobWriter(storage.CreateBlobWriter()))
			{
				DirectoryNode root = new();
				for (int idx = 1; idx <= 3; idx++)
				{
					DirectoryNode node = new();
					IHashedBlobRef<DirectoryNode> nodeRef = await writer.WriteBlobAsync(node);
					root.AddDirectory(new DirectoryEntry($"node{idx}", 0, nodeRef));
				}

				RefName refName = new("ref");
				IHashedBlobRef<DirectoryNode> rootRef = await writer.WriteBlobAsync(root);
				await storage.AddRefAsync(refName, rootRef);
			}

			MemoryStorageBackend memoryStore = (MemoryStorageBackend)storage.Backend;
			Assert.AreEqual(1, memoryStore.Refs.Count);
			Assert.AreEqual(2, memoryStore.Blobs.Count);
		}

		[TestMethod]
		public async Task ReloadTestsAsync()
		{
			BundleOptions bundleOptions = new();
			bundleOptions.MaxBlobSize = 1;

			BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(bundleOptions, NullLogger.Instance);

			RefName refName = new("ref");

			{
				await using (IBlobWriter writer = storage.CreateBlobWriter())
				{
					IHashedBlobRef<DirectoryNode> rootRef = await writer.WriteBlobAsync(new DirectoryNode());
					for (int idx = 4; idx >= 1; idx--)
					{
						DirectoryNode next = new();
						next.AddDirectory(new DirectoryEntry($"node{idx}", 0, rootRef));
						rootRef = await writer.WriteBlobAsync(next);
					}
					await storage.AddRefAsync(refName, rootRef);
				}

				MemoryStorageBackend memoryStore = (MemoryStorageBackend)storage.Backend;
				Assert.AreEqual(1, memoryStore.Refs.Count);
				Assert.AreEqual(5, memoryStore.Blobs.Count);
			}

			{
				DirectoryNode root = await storage.ReadRefTargetAsync<DirectoryNode>(refName);

				DirectoryNode? newNode1 = await root.TryOpenDirectoryAsync("node1");
				Assert.IsNotNull(newNode1);

				DirectoryNode? newNode2 = await newNode1!.TryOpenDirectoryAsync("node2");
				Assert.IsNotNull(newNode2);

				DirectoryNode? newNode3 = await newNode2!.TryOpenDirectoryAsync("node3");
				Assert.IsNotNull(newNode3);

				DirectoryNode? newNode4 = await newNode3!.TryOpenDirectoryAsync("node4");
				Assert.IsNotNull(newNode4);
			}
		}
	}
}
