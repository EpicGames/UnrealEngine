// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Clients;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class ChunkingTests
	{
		[TestMethod]
		public void BuzHashTests()
		{
			byte[] data = new byte[4096];
			new Random(0).NextBytes(data);

			const int WindowSize = 128;

			uint rollingHash = 0;
			for (int maxIdx = 0; maxIdx < data.Length + WindowSize; maxIdx++)
			{
				int minIdx = maxIdx - WindowSize;

				if (maxIdx < data.Length)
				{
					rollingHash = BuzHash.Add(rollingHash, data[maxIdx]);
				}

				int length = Math.Min(maxIdx + 1, data.Length) - Math.Max(minIdx, 0);
				uint cleanHash = BuzHash.Add(0, data.AsSpan(Math.Max(minIdx, 0), length));
				Assert.AreEqual(rollingHash, cleanHash);

				if (minIdx >= 0)
				{
					rollingHash = BuzHash.Sub(rollingHash, data[minIdx], length);
				}
			}
		}

		[TestMethod]
		public async Task EmptyNodeTestAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			const string RefName = "hello";
			await using (IBlobWriter writer = store.CreateBlobWriter(RefName))
			{
				ChunkingOptions options = new();
				options.LeafOptions = new LeafChunkedDataNodeOptions(64, 64, 64);
				options.InteriorOptions = new InteriorChunkedDataNodeOptions(4, 4, 4);

				using MemoryStream emptyStream = new();
				LeafChunkedData leafChunkedData = await LeafChunkedDataNode.CreateFromStreamAsync(writer, emptyStream, new LeafChunkedDataNodeOptions(64, 64, 64), CancellationToken.None);
				ChunkedData chunkedData = await InteriorChunkedDataNode.CreateTreeAsync(leafChunkedData, new InteriorChunkedDataNodeOptions(4, 4, 4), writer, CancellationToken.None);

				DirectoryNode directory = new();
				directory.AddFile("test.foo", FileEntryFlags.None, chunkedData);

				IHashedBlobRef handle = await writer.WriteBlobAsync(directory);
				await store.AddRefAsync(RefName, handle);
			}
		}

		[TestMethod]
		public async Task FixedSizeChunkingTestsAsync()
		{
			ChunkingOptions options = new();
			options.LeafOptions = new LeafChunkedDataNodeOptions(64, 64, 64);
			options.InteriorOptions = new InteriorChunkedDataNodeOptions(4, 4, 4);

			await TestChunkingAsync(options);
		}

		[TestMethod]
		public async Task VariableSizeChunkingTestsAsync()
		{
			ChunkingOptions options = new();
			options.LeafOptions = new LeafChunkedDataNodeOptions(32, 64, 96);
			options.InteriorOptions = new InteriorChunkedDataNodeOptions(1, 4, 12);

			await TestChunkingAsync(options);
		}

		static async Task TestChunkingAsync(ChunkingOptions options)
		{
			using MemoryCache cache = new(new MemoryCacheOptions());

			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			await using IBlobWriter writer = store.CreateBlobWriter();

			byte[] data = new byte[4096];
			new Random(0).NextBytes(data);

			for (int idx = 0; idx < data.Length; idx++)
			{
				data[idx] = (byte)idx;
			}

			ChunkedDataNodeRef handle;

			const int NumIterations = 100;
			{
				using ChunkedDataWriter fileWriter = new(writer, options);

				for (int idx = 0; idx < NumIterations; idx++)
				{
					await fileWriter.AppendAsync(data, CancellationToken.None);
				}

				handle = (await fileWriter.FlushAsync(CancellationToken.None)).Root;
			}

			ChunkedDataNode root = await handle.ReadBlobAsync();

			byte[] result;
			using (MemoryStream stream = new())
			{
				await root.CopyToStreamAsync(stream);
				result = stream.ToArray();
			}

			Assert.AreEqual(NumIterations * data.Length, result.Length);

			for (int idx = 0; idx < NumIterations; idx++)
			{
				ReadOnlyMemory<byte> spanData = result.AsMemory(idx * data.Length, data.Length);
				Assert.IsTrue(spanData.Span.SequenceEqual(data));
			}

			await CheckSizesAsync(root, options, true);
		}

		static async Task CheckSizesAsync(ChunkedDataNode node, ChunkingOptions options, bool rightmost)
		{
			if (node is LeafChunkedDataNode leafNode)
			{
				Assert.IsTrue(rightmost || leafNode.Data.Length >= options.LeafOptions.MinSize);
				Assert.IsTrue(leafNode.Data.Length <= options.LeafOptions.MaxSize);
			}
			else
			{
				InteriorChunkedDataNode interiorNode = (InteriorChunkedDataNode)node;

				Assert.IsTrue(rightmost || interiorNode.Children.Count >= options.InteriorOptions.MinChildCount);
				Assert.IsTrue(interiorNode.Children.Count <= options.InteriorOptions.MaxChildCount);

				int childCount = interiorNode.Children.Count;
				for (int idx = 0; idx < childCount; idx++)
				{
					ChunkedDataNode childNode = await interiorNode.Children[idx].ReadBlobAsync();
					await CheckSizesAsync(childNode, options, idx == childCount - 1);
				}
			}
		}

		[TestMethod]
		public async Task ChunkingCompatV2Async()
		{
			ChunkingOptions chunkingOptions = new();
			chunkingOptions.LeafOptions = new LeafChunkedDataNodeOptions(2, 2, 2);
			chunkingOptions.InteriorOptions = new InteriorChunkedDataNodeOptions(2, 2, 2);

			BlobSerializerOptions serializerOptions = new();
			serializerOptions.Converters.Add(new InteriorChunkedDataNodeConverter(HordeApiVersion.Initial)); // Does not include length fields in interior nodes

			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();

			byte[] data = Encoding.UTF8.GetBytes("hello world");

			IHashedBlobRef<DirectoryNode> handle;
			await using (IBlobWriter writer = store.CreateBlobWriter(options: serializerOptions))
			{
				using ChunkedDataWriter chunkedWriter = new(writer, chunkingOptions);
				await chunkedWriter.AppendAsync(data, CancellationToken.None);
				ChunkedData chunkedData = await chunkedWriter.FlushAsync(CancellationToken.None);

				DirectoryNode directoryNode = new();
				directoryNode.AddFile("test", FileEntryFlags.None, chunkedData);
				handle = await writer.WriteBlobAsync(directoryNode);
			}

			DirectoryInfo tempDir = new(Path.Combine(Path.GetTempPath(), Guid.NewGuid().ToString()));
			try
			{
				await handle.ExtractAsync(tempDir, new ExtractOptions(), NullLogger.Instance, CancellationToken.None);

				byte[] outputData = await File.ReadAllBytesAsync(Path.Combine(tempDir.FullName, "test"));
				Assert.IsTrue(outputData.SequenceEqual(data));
			}
			finally
			{
				tempDir.Delete(true);
			}
		}

		[TestMethod]
		public async Task HashAsync()
		{
			ChunkingOptions chunkingOptions = new();
			chunkingOptions.LeafOptions = new LeafChunkedDataNodeOptions(64, 128, 92);

			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());
			byte[] data = RandomNumberGenerator.GetBytes(4096);

			ChunkedData chunkedData;
			using (ChunkedDataWriter writer = new(blobWriter, chunkingOptions))
			{
				await writer.AppendAsync(data, default);
				chunkedData = await writer.FlushAsync();
			}

			IoHash hash = IoHash.Compute(data);
			Assert.AreEqual(hash, chunkedData.StreamHash);
		}

		[TestMethod]
		public async Task ChunkOrderAsync()
		{
			InteriorChunkedDataNodeOptions interiorOptions = new(2, 2, 2);

			BlobSerializerOptions serializerOptions = new();
			serializerOptions.Converters.Add(new InteriorChunkedDataNodeConverter());

			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<LeafChunkedDataNode> leafRef = await blobWriter.WriteBlobAsync(new LeafChunkedDataNode(new byte[] { 1, 2, 3 }));
			int leafRefIndex = MemoryBlobWriter.GetIndex((IHashedBlobRef)leafRef);
			ChunkedDataNodeRef leafChunkedRef = new(3, 0, leafRef);

			List<ChunkedDataNodeRef> leafNodeRefs = [.. Enumerable.Repeat(leafChunkedRef, 10000)];
			ChunkedDataNodeRef root = await InteriorChunkedDataNode.CreateTreeAsync(leafNodeRefs, interiorOptions, blobWriter, CancellationToken.None);

			List<IBlobRef> list = [];
			await GetReadOrderAsync(root, list);

			int prevIndex = Int32.MaxValue;
			for (int idx = 0; idx < list.Count; idx++)
			{
				int index = MemoryBlobWriter.GetIndex((IHashedBlobRef)list[idx].Innermost);
				if (index != leafRefIndex)
				{
					Console.WriteLine("{0}", index);
					Assert.IsTrue(index <= prevIndex);
					prevIndex = index;
				}
			}
		}

		static async Task GetReadOrderAsync(IBlobRef handle, List<IBlobRef> list)
		{
			list.Add(handle);

			using BlobData blobData = await handle.ReadBlobDataAsync(CancellationToken.None);
			foreach (IBlobRef childHandle in blobData.Imports)
			{
				await GetReadOrderAsync(childHandle, list);
			}
		}
		private struct ChunkOffsetsAndSizes
		{
			public List<int> _offsets;
			public List<int> _sizes;
		}

		/// <summary>
		/// Utility function to test low-level chunking algorithm with different chunking parameters
		/// </summary>
		private static ChunkOffsetsAndSizes FindChunks(ReadOnlySpan<byte> data, int minSize, int maxSize, int targetSize)
		{
			ChunkOffsetsAndSizes chunks;

			chunks._offsets = [];
			chunks._sizes = [];

			int currentOffset = 0;
			while (currentOffset != data.Length)
			{
				chunks._offsets.Add(currentOffset);
				int chunkLength = BuzHash.FindChunkLength(data.Slice(currentOffset), minSize, maxSize, targetSize, out _);
				currentOffset += chunkLength;
				chunks._sizes.Add(currentOffset);
			}

			Assert.AreEqual(chunks._offsets.Count, chunks._sizes.Count);

			return chunks;
		}

		/// <summary>
		/// Utility function to test low-level chunking algorithm with varying target chunk sizes
		/// </summary>
		private static ChunkOffsetsAndSizes FindChunks(ReadOnlySpan<byte> data, int targetSize)
		{
			int minSize = targetSize / 2;
			int maxSize = targetSize * 4;
			return FindChunks(data, minSize, maxSize, targetSize);
		}

		/// <summary>
		/// Verify that chunking algorithm produces chunks at predetermined offsets,
		/// which is important for generating consistent chunking between different implementations.
		/// </summary>
		[TestMethod]
		public void DeterministicChunkOffsets()
		{
			int dataSize = 1 << 20; // 1MB test buffer
			byte[] data = GenerateChunkingTestBuffer(dataSize, 1234);

			List<int> expectedChunkOffsets =
			[
				0, 34577, 128471, 195115, 238047, 297334, 358754, 396031,
				462359, 508658, 601550, 702021, 754650, 790285, 854987, 887998,
				956848, 1042406
			];

			int targetSize = 65536;
			ChunkOffsetsAndSizes chunks = FindChunks(data, targetSize);

			CollectionAssert.AreEqual(expectedChunkOffsets, chunks._offsets);
		}

		/// <summary>
		/// Verify that chunking algorithm produces chunks of expected sizes for different target chunk size configurations.
		/// This helps validating chunking consistency between different implementations of the algorithm.
		/// </summary>
		[TestMethod]
		public void DeterministicChunkSize()
		{
			int dataSize = 128 << 20; // 128MB test buffer
			byte[] data = GenerateChunkingTestBuffer(dataSize, 1234);

			// Chunk size in KB and expected number of generated chunks
			List<(int, int)> configList =
			[
				(8, 16442),
				(16, 8146),
				(32, 4089),
				(64, 2019),
				(96, 1362),
				(128, 1012),
				(160, 811),
				(192, 681),
				(256, 503),
			];

			Parallel.ForEach(configList, config =>
			{
				(int targetSizeKB, int expectedNumChunks) = config;

				int targetSize = targetSizeKB * 1024;

				ChunkOffsetsAndSizes chunks = FindChunks(data, targetSize);

				int avgChunkSize = dataSize / chunks._offsets.Count;
				int absError = Math.Abs(avgChunkSize - targetSize);
				double absErrorPct = (100.0 * absError) / targetSize;

				Assert.IsTrue(absErrorPct < 5.0, $"Average chunk size {avgChunkSize} is more than 5% different from target {targetSize}");
				Assert.AreEqual(expectedNumChunks, chunks._offsets.Count);
			});
		}

		/// <summary>
		/// Utility function to quickly generate a deterministic byte sequence
		/// that can be used to ensure consistency between multiple implementations 
		/// of the chunking algorithm.
		/// </summary>
		static byte[] GenerateChunkingTestBuffer(int count, uint seed)
		{
			byte[] data = new byte[count];
			uint rngState = seed;

			for (int i = 0; i < count; ++i)
			{
				data[i] = (byte)XorShift32(ref rngState);
			}

			return data;
		}
		static uint XorShift32(ref uint state)
		{
			uint x = state;
			x ^= x << 13;
			x ^= x >> 17;
			x ^= x << 5;
			state = x;
			return x;
		}
	}
}
