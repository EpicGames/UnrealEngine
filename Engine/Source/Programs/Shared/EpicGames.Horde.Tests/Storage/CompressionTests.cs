// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;
using EpicGames.Horde.Storage.Bundles;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class CompressionTests
	{
		[TestMethod]
		[DataRow(BundleCompressionFormat.LZ4)]
		[DataRow(BundleCompressionFormat.Gzip)]
		[DataRow(BundleCompressionFormat.Brotli)]
		[DataRow(BundleCompressionFormat.Zstd)]
		public void SimpleData(BundleCompressionFormat format)
		{
			byte[] input = new byte[2 * 1024 * 1024];
			for (int idx = 0; idx < input.Length; idx++)
			{
				input[idx] = (byte)idx;
			}

			RunTests(format, input);
		}

		[TestMethod]
		[DataRow(BundleCompressionFormat.LZ4)]
		[DataRow(BundleCompressionFormat.Gzip)]
		[DataRow(BundleCompressionFormat.Brotli)]
		[DataRow(BundleCompressionFormat.Zstd)]
		public void RandomData(BundleCompressionFormat format)
		{
			byte[] data = new byte[2 * 1024 * 1024];
			new Random(0).NextBytes(data);

			RunTests(format, data);
		}

		static void RunTests(BundleCompressionFormat format, byte[] input)
		{
			ArrayMemoryWriter writer = new(10 * 1024 * 1024);
			Stopwatch compressTimer = Stopwatch.StartNew();
			BundleData.Compress(format, input, writer);
			compressTimer.Stop();

			byte[] output = new byte[input.Length];
			Stopwatch decompressTimer = Stopwatch.StartNew();
			BundleData.Decompress(format, writer.WrittenMemory, output);
			decompressTimer.Stop();

			Assert.IsTrue(input.SequenceEqual(output));

			Console.WriteLine("Output size: {0:n0} bytes, compression time: {1}ms, decompression time: {2}ms", writer.WrittenMemory.Length, compressTimer.Elapsed.TotalMilliseconds, decompressTimer.Elapsed.TotalMilliseconds);
		}
	}
}
