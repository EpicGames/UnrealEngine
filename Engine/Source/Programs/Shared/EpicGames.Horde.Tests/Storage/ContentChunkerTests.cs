// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class ContentChunkerTests
	{
		record struct SplitPoint(int Index, int Length);

		[TestMethod]
		public async Task SerialTestAsync()
		{
			List<SplitPoint> legacyPoints;
			await using (AsyncPipeline pipeline = new(CancellationToken.None))
			{
				byte[] buffer = new byte[1024 * 1024];
				SerialBuzHashChunker chunker = new(pipeline, buffer, LeafChunkedDataNodeOptions.Default);
				legacyPoints = await TestPipelineAsync(chunker);
			}
		}

		[TestMethod]
		public async Task ParallelTestAsync()
		{
			List<SplitPoint> latestPoints;
			await using (AsyncPipeline pipeline = new(CancellationToken.None))
			{
				byte[] buffer = new byte[1024 * 1024];
				ParallelBuzHashChunker chunker = new(pipeline, buffer, LeafChunkedDataNodeOptions.Default);
				latestPoints = await TestPipelineAsync(chunker);
			}
		}

		[TestMethod]
		public async Task CompareTestAsync()
		{
			List<SplitPoint> legacyPoints;
			await using (AsyncPipeline pipeline = new(CancellationToken.None))
			{
				byte[] buffer = new byte[1024 * 1024];
				SerialBuzHashChunker chunker = new(pipeline, buffer, LeafChunkedDataNodeOptions.Default);
				legacyPoints = await TestPipelineAsync(chunker);
			}

			List<SplitPoint> latestPoints;
			await using (AsyncPipeline pipeline = new(CancellationToken.None))
			{
				byte[] buffer = new byte[1024 * 1024];
				ParallelBuzHashChunker chunker = new(pipeline, buffer, LeafChunkedDataNodeOptions.Default);
				latestPoints = await TestPipelineAsync(chunker);
			}

			Assert.IsTrue(Enumerable.SequenceEqual(legacyPoints, latestPoints));
		}

		static async Task<List<SplitPoint>> TestPipelineAsync(ContentChunker pipeline)
		{
			Random random = new(0);

			int[] lengths = [0, 1, 32 * 1024, 129 * 1024, 256 * 1024, 4 * 1024 * 1024];

			foreach (int length in lengths)
			{
				byte[] data = new byte[length];
				random.NextBytes(data);

				ChunkerSource input = new MemoryChunkerSource(data);
				await pipeline.SourceWriter.WriteAsync(input);
			}

			pipeline.SourceWriter.Complete();

			List<SplitPoint> splitPoints = [];
			for (int idx = 0; idx < lengths.Length; idx++)
			{
				int outputLength = 0;

				while (await pipeline.OutputReader.WaitToReadAsync())
				{
					if (pipeline.OutputReader.TryRead(out ChunkerOutput? output))
					{
						while (await output.MoveNextAsync())
						{
							splitPoints.Add(new SplitPoint(idx, output.Data.Length));
							outputLength += output.Data.Length;
						}
						break;
					}
				}

				Assert.AreEqual(outputLength, lengths[idx]);
			}

			Assert.IsFalse(await pipeline.OutputReader.WaitToReadAsync());
			return splitPoints;
		}
	}
}
