// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class FileMetadataDbTests
	{
		[TestMethod]
		public async Task DirectoryTestsAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();

			int a = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "A");
			int b = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "B");
			int c = await db.AddDirectoryAsync(b, "C");
			int d = await db.AddDirectoryAsync(b, "D");

			{
				List<DirectoryRow> rows = [.. (await db.GetDirectoriesAsync(FileMetadataDb.RootDirectoryId)).OrderBy(x => x.Id)];
				Assert.AreEqual(2, rows.Count);
				Assert.AreEqual(a, rows[0].Id);
				Assert.AreEqual("A", rows[0].Name);
				Assert.AreEqual(b, rows[1].Id);
				Assert.AreEqual("B", rows[1].Name);
			}

			{
				List<DirectoryRow> rows = [.. (await db.GetDirectoriesAsync(a)).OrderBy(x => x.Id)];
				Assert.AreEqual(0, rows.Count);
			}

			{
				List<DirectoryRow> rows = [.. (await db.GetDirectoriesAsync(b)).OrderBy(x => x.Id)];
				Assert.AreEqual(2, rows.Count);
				Assert.AreEqual(c, rows[0].Id);
				Assert.AreEqual("C", rows[0].Name);
				Assert.AreEqual(d, rows[1].Id);
				Assert.AreEqual("D", rows[1].Name);
			}

			Assert.AreEqual("", await db.GetDirectoryPathAsync(FileMetadataDb.RootDirectoryId));
			Assert.AreEqual("A/", await db.GetDirectoryPathAsync(a));
			Assert.AreEqual("B/", await db.GetDirectoryPathAsync(b));
			Assert.AreEqual("B/C/", await db.GetDirectoryPathAsync(c));
			Assert.AreEqual("B/D/", await db.GetDirectoryPathAsync(d));
		}

		[TestMethod]
		public async Task FileTestsAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();

			int a = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "A");

			DateTime time = new(2024, 1, 1, 12, 30, 45, DateTimeKind.Utc);
			int c = await db.AddFileAsync(FileMetadataDb.RootDirectoryId, "C.txt", time, 1234);
			int d = await db.AddFileAsync(FileMetadataDb.RootDirectoryId, "D.txt", time, 2345);
			int e = await db.AddFileAsync(a, "E.txt", time, 3456);
			int f = await db.AddFileAsync(a, "F.txt", time, 4567);

			{
				List<FileRow> files = [.. (await db.FindFilesInDirectoryAsync(FileMetadataDb.RootDirectoryId)).OrderBy(x => x.Id)];
				Assert.AreEqual(2, files.Count);

				Assert.AreEqual(c, files[0].Id);
				Assert.AreEqual(time, files[0].Time);
				Assert.AreEqual(1234, files[0].Length);

				Assert.AreEqual(d, files[1].Id);
				Assert.AreEqual(time, files[1].Time);
				Assert.AreEqual(2345, files[1].Length);
			}

			{
				List<FileRow> files = [.. (await db.FindFilesInDirectoryAsync(a)).OrderBy(x => x.Id)];
				Assert.AreEqual(2, files.Count);

				Assert.AreEqual(e, files[0].Id);
				Assert.AreEqual(time, files[0].Time);
				Assert.AreEqual(3456, files[0].Length);

				Assert.AreEqual(f, files[1].Id);
				Assert.AreEqual(time, files[1].Time);
				Assert.AreEqual(4567, files[1].Length);
			}

			Assert.AreEqual("C.txt", await db.GetFilePathAsync(c));
			Assert.AreEqual("D.txt", await db.GetFilePathAsync(d));
			Assert.AreEqual("A/E.txt", await db.GetFilePathAsync(e));
			Assert.AreEqual("A/F.txt", await db.GetFilePathAsync(f));
		}

		[TestMethod]
		public async Task BatchFileTestsAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();
			int root = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "");

			List<FileRow> files =
			[
				new FileRow(root, "A", DateTime.MinValue, 1),
				new FileRow(root, "B", DateTime.MinValue, 1),
				new FileRow(root, "C", DateTime.MinValue, 1),
			];

			await db.AddFilesAsync(files);
			Assert.AreEqual(3, (await db.FindFilesInDirectoryAsync(root)).Count());
		}

		[TestMethod]
		public async Task BatchDirectoryTestsAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();
			await db.AddDirectoriesAsync([new DirectoryRow(FileMetadataDb.RootDirectoryId, "A"), new DirectoryRow(FileMetadataDb.RootDirectoryId, "B"), new DirectoryRow(FileMetadataDb.RootDirectoryId, "C")]);
			Assert.AreEqual(3, (await db.GetDirectoriesAsync(FileMetadataDb.RootDirectoryId)).Count());
		}

		[TestMethod]
		public async Task ChunkTestsAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();

			int d1 = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "D1");
			int d2 = await db.AddDirectoryAsync(d1, "D2");

			DateTime time = new(2024, 1, 1, 12, 30, 45, DateTimeKind.Utc);
			int f1 = await db.AddFileAsync(d1, "F1.txt", time, 1234);
			int f2 = await db.AddFileAsync(d1, "F2.txt", time, 2345);
			int f3 = await db.AddFileAsync(d2, "F3.txt", time, 3456);
			int f4 = await db.AddFileAsync(d2, "F4.txt", time, 4567);

			IoHash h1 = IoHash.Compute(new byte[] { 1, 2, 3 });
			IoHash h2 = IoHash.Compute(new byte[] { 1, 2, 3, 4 });

			int c1 = await db.AddChunkAsync(f1, 0, 10, h1);
			int c2 = await db.AddChunkAsync(f2, 10, 10, h1);
			int c3 = await db.AddChunkAsync(f3, 20, 10, h1);
			int c4 = await db.AddChunkAsync(f4, 0, 10, h2);

			{
				List<ChunkRow> chunks = [.. (await db.FindChunksAsync(h1, 10)).OrderBy(x => x.Id)];
				Assert.AreEqual(3, chunks.Count);
				Assert.AreEqual(new ChunkRow(c1, f1, 0, 10, h1), chunks[0]);
				Assert.AreEqual(new ChunkRow(c2, f2, 10, 10, h1), chunks[1]);
				Assert.AreEqual(new ChunkRow(c3, f3, 20, 10, h1), chunks[2]);
			}

			{
				List<ChunkRow> chunks = [.. (await db.FindChunksAsync(h2, 10)).OrderBy(x => x.Id)];
				Assert.AreEqual(1, chunks.Count);
				Assert.AreEqual(new ChunkRow(c4, f4, 0, 10, h2), chunks[0]);
			}
		}

		[TestMethod]
		public async Task RemoveTreeTestAsync()
		{
			using FileMetadataDb db = await FileMetadataDb.CreateInMemoryAsync();

			int d1 = await db.AddDirectoryAsync(FileMetadataDb.RootDirectoryId, "D1");
			int d2 = await db.AddDirectoryAsync(d1, "D2");
			int d3 = await db.AddDirectoryAsync(d2, "D3");
			int d4 = await db.AddDirectoryAsync(d2, "D4");
			int d5 = await db.AddDirectoryAsync(d4, "D5");

			DateTime time = new(2024, 1, 1, 12, 30, 45, DateTimeKind.Utc);
			int f1 = await db.AddFileAsync(d1, "F1.txt", time, 1234);
			int f2 = await db.AddFileAsync(d2, "F2.txt", time, 2345);
			int f3 = await db.AddFileAsync(d3, "F3.txt", time, 3456);
			int f4 = await db.AddFileAsync(d4, "F4.txt", time, 4567);
			int f5 = await db.AddFileAsync(d5, "F5.txt", time, 5678);

			IoHash h1 = IoHash.Compute(new byte[] { 1, 2, 3 });

			int c1 = await db.AddChunkAsync(f1, 0, 10, h1);
			int c2 = await db.AddChunkAsync(f2, 10, 10, h1);
			int c3 = await db.AddChunkAsync(f3, 20, 10, h1);
			int c4 = await db.AddChunkAsync(f4, 0, 10, h1);
			int c5 = await db.AddChunkAsync(f5, 10, 10, h1);

			{
				List<ChunkRow> chunks = [.. (await db.FindChunksAsync(h1, 10)).OrderBy(x => x.Id)];
				Assert.AreEqual(5, chunks.Count);
				Assert.AreEqual(new ChunkRow(c1, f1, 0, 10, h1), chunks[0]);
				Assert.AreEqual(new ChunkRow(c2, f2, 10, 10, h1), chunks[1]);
				Assert.AreEqual(new ChunkRow(c3, f3, 20, 10, h1), chunks[2]);
				Assert.AreEqual(new ChunkRow(c4, f4, 0, 10, h1), chunks[3]);
				Assert.AreEqual(new ChunkRow(c5, f5, 10, 10, h1), chunks[4]);
			}

			await db.RemoveDirectoryAsync(d4);

			{
				List<ChunkRow> chunks = [.. (await db.FindChunksAsync(h1, 10)).OrderBy(x => x.Id)];
				Assert.AreEqual(3, chunks.Count);

				Assert.AreEqual(new ChunkRow(c1, f1, 0, 10, h1), chunks[0]);
				Assert.AreEqual(new ChunkRow(c2, f2, 10, 10, h1), chunks[1]);
				Assert.AreEqual(new ChunkRow(c3, f3, 20, 10, h1), chunks[2]);
			}
		}
	}
}
