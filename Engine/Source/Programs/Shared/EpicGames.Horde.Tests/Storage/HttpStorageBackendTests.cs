// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Net;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage;

[TestClass]
public class HttpStorageBackendTests
{
	/// <summary>
	/// HttpResponseMessage subclass that tracks whether Dispose was called.
	/// </summary>
	sealed class TrackableHttpResponseMessage : HttpResponseMessage
	{
		public bool WasDisposed { get; private set; }

		public TrackableHttpResponseMessage(HttpStatusCode statusCode) : base(statusCode) { }

		protected override void Dispose(bool disposing)
		{
			WasDisposed = true;
			base.Dispose(disposing);
		}
	}

	/// <summary>
	/// Mock handler for OpenBlobAsync: returns blob data in a trackable response so we can
	/// verify the response is disposed when the stream is disposed.
	/// </summary>
	sealed class BlobReadHandler : HttpMessageHandler
	{
		public TrackableHttpResponseMessage? LastResponse { get; private set; }

		protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
		{
			TrackableHttpResponseMessage response = new(HttpStatusCode.OK)
			{
				Content = new ByteArrayContent([1, 2, 3, 4, 5])
			};
			LastResponse = response;
			return Task.FromResult<HttpResponseMessage>(response);
		}
	}

	[TestMethod]
	public async Task OpenBlobAsync_DisposesResponseWhenStreamDisposedAsync()
	{
		// Arrange
		using BlobReadHandler handler = new();

		HttpClient CreateApiClient()
		{
			HttpClient client = new(handler, disposeHandler: false);
			client.BaseAddress = new("https://horde.example.com/");
			return client;
		}

		HttpStorageBackend backend = new("api/v1/storage/ns", CreateApiClient, () => new HttpClient(), NullLogger<HttpStorageBackend>.Instance);

		BlobLocator locator = new("test-blob");
		Stream stream = await backend.OpenBlobAsync(locator, 0, null);

		Assert.IsNotNull(handler.LastResponse);
		Assert.IsFalse(handler.LastResponse.WasDisposed, "Response should not be disposed while stream is open");

		// Act: dispose the stream.
		// Before the fix: the raw content stream was returned, and disposing it did NOT dispose the HttpResponseMessage.
		// After the fix: ResponseOwningStream disposes the HttpResponseMessage when the stream is disposed.
		await stream.DisposeAsync();

		// Assert
		Assert.IsTrue(handler.LastResponse.WasDisposed, "Response should be disposed when the stream is disposed");
	}
}
