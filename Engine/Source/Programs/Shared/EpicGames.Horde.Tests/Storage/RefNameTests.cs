// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Text.Json;
using EpicGames.Horde.Storage;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class RefNameTests
	{
		[TestMethod]
		[DataRow("my/nested/ref-name", "my/nested/ref-name", 10)]
		[DataRow("my/nested/ref-name", "my/nested/ref-name", 15)]
		[DataRow("simple", "simple", 5)]
		[DataRow("foo-pcb/baz-main/10200300/editor/a1b2c3d4e5f6a7b8c9d0e1f2/0a1b2c3d4e5f6a7b8c9d0e1f2a3b4c5d_1",
			"foo-pcb/baz-main/10200300/editor/a1b2c3d4e5f6a7b8c9d0e1f2/0a1b2c3d4e5f6a7b8c9d0e1f2a3b4c5d_1", 40)]
		[DataRow("foo-pcb/baz-main/10200300/editor/a1b2c3d4e5f6a7b8c9d0e1f2/0a1b2c3d4e5f6a7b8c9d0e1f2a3b4c5d_1",
			"foo-pcb/baz-main/10200300/editor/a1b2c3d4e5f6a7b8c9d0e1f2/0a1b2c3d4e5f6a7b8c9d0e1f2a3b4c5d_1", 80)]
		public void JsonConverterMultiSegment(string jsonValue, string expectedStr, int splitPoint)
		{
			RefName expected = new(expectedStr);
			string json = $"\"{jsonValue}\"";
			byte[] bytes = Encoding.UTF8.GetBytes(json);
			ReadOnlySequence<byte> sequence = EpicGames.Core.ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			RefName result = JsonSerializer.Deserialize<RefName>(ref reader);
			Assert.AreEqual(expected, result);
		}
	}
}
