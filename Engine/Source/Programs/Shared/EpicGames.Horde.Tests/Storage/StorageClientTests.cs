// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Bundles.V2;
using EpicGames.Horde.Storage.Clients;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class StorageNamespaceTests
	{
		[BlobConverter(typeof(TestNodeConverter))]
		class TestNode
		{
			public static BlobType BlobType = new("{99601905-4F6E-A089-03D6-F187711BAFEE}", 1);

			public int Value { get; }
			public byte[] Padding { get; set; } = [];
			public IHashedBlobRef<TestNode>[] Refs { get; }

			public TestNode(int value, params IHashedBlobRef<TestNode>[] refs)
			{
				Value = value;
				Refs = refs;
			}

			public TestNode(int value, byte[] padding, IHashedBlobRef<TestNode>[] refs)
			{
				Value = value;
				Padding = padding;
				Refs = refs;
			}
		}

		class TestNodeConverter : BlobConverter<TestNode>
		{
			public override TestNode Read(IBlobReader reader, BlobSerializerOptions options)
			{
				int value = reader.ReadInt32();
				byte[] padding = reader.ReadVariableLengthBytes().ToArray();
				IHashedBlobRef<TestNode>[] refs = reader.ReadVariableLengthArray(() => reader.ReadBlobRef<TestNode>());

				return new TestNode(value, padding, refs);
			}

			public override BlobType Write(IBlobWriter writer, TestNode value, BlobSerializerOptions options)
			{
				writer.WriteInt32(value.Value);
				writer.WriteVariableLengthBytes(value.Padding);
				writer.WriteVariableLengthArray(value.Refs, x => writer.WriteBlobRef(x));

				return TestNode.BlobType;
			}
		}

		[TestMethod]
		public async Task TestBasicAsync()
		{
			KeyValueStorageNamespace store = KeyValueStorageNamespace.CreateInMemory();
			await TestBasicAsync(store);
		}

		[TestMethod]
		public async Task TestBasicBundleV2Async()
		{
			BundleStorageNamespace storeV2 = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
			await TestBasicAsync(storeV2);
		}

		static async Task TestBasicAsync(IStorageNamespace store)
		{
			IHashedBlobRef<TestNode> nodeRef;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				nodeRef = await writer.WriteBlobAsync(new TestNode(123));
			}
			await store.AddRefAsync("hello", nodeRef);

			TestNode output = await store.ReadRefTargetAsync<TestNode>("hello");
			Assert.AreEqual(123, output.Value);
		}

		[TestMethod]
		public async Task PendingRefsAsync()
		{
			await using BundleCache cache = new();

			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);

			IHashedBlobRef<TestNode> nodeRef2;
			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				IHashedBlobRef<TestNode> nodeRef1 = await writer.WriteBlobAsync(new TestNode(123));
				nodeRef2 = await writer.WriteBlobAsync(new TestNode(456, nodeRef1));
			}
			await store.AddRefAsync("hello", nodeRef2);

			TestNode output2 = await store.ReadRefTargetAsync<TestNode>("hello");
			Assert.AreEqual(456, output2.Value);
			Assert.AreEqual(1, output2.Refs.Length);

			TestNode output1 = await output2.Refs[0].ReadBlobAsync();
			Assert.AreEqual(0, output1.Refs.Length);
			Assert.AreEqual(123, output1.Value);
		}

		[TestMethod]
		public async Task PacketFlushAsync()
		{
			await using BundleCache cache = new();

			BundleOptions bundleOptions = new() { MinCompressionPacketSize = 100, MaxBlobSize = 1024 * 1024, MaxVersion = BundleVersion.LatestV2 };
			BundleStorageNamespace store = BundleStorageNamespace.CreateInMemory(bundleOptions, NullLogger.Instance);

			await using IBlobWriter writer = store.CreateBlobWriter();
			IHashedBlobRef<TestNode> nodeRef1 = await writer.WriteBlobAsync(new TestNode(123) { Padding = new byte[1024] });
			await writer.FlushAsync();
			IHashedBlobRef<TestNode> nodeRef2 = await writer.WriteBlobAsync(new TestNode(456, nodeRef1) { Padding = new byte[1024] });
			IHashedBlobRef<TestNode> nodeRef3 = await writer.WriteBlobAsync(new TestNode(789, nodeRef2));

			// nodeRef1 is in a flushed bundle
			ExportHandle export1 = (ExportHandle)nodeRef1.Innermost;
			BundleWriter.PendingPacketHandle packet1 = (BundleWriter.PendingPacketHandle)export1.Packet;
			BundleWriter.PendingBundleHandle bundle1 = (BundleWriter.PendingBundleHandle)packet1.Bundle;
			Assert.IsNotNull(packet1.FlushedHandle);
			Assert.IsNotNull(bundle1.FlushedHandle);

			TestNode node1 = await export1.ReadBlobAsync<TestNode>();
			Assert.AreEqual(123, node1.Value);

			// nodeRef2 is in a flushed packet, unflushed bundle
			ExportHandle export2 = (ExportHandle)nodeRef2.Innermost;

			BundleWriter.PendingPacketHandle packet2 = (BundleWriter.PendingPacketHandle)export2.Packet;
			Assert.AreNotEqual(packet1, packet2);
			Assert.IsNotNull(packet2.FlushedHandle);

			BundleWriter.PendingBundleHandle bundle2 = (BundleWriter.PendingBundleHandle)packet2.Bundle;
			Assert.AreNotEqual(bundle1, bundle2);
			Assert.IsNull(bundle2.FlushedHandle);

			TestNode node2 = await export2.ReadBlobAsync<TestNode>();
			Assert.AreEqual(456, node2.Value);

			// nodeRef3 is in an unflushed packet, unflushed bundle
			ExportHandle export3 = (ExportHandle)nodeRef3.Innermost;

			BundleWriter.PendingPacketHandle packet3 = (BundleWriter.PendingPacketHandle)export3.Packet;
			Assert.AreNotEqual(packet2, packet3);
			Assert.IsNull(packet3.FlushedHandle);

			BundleWriter.PendingBundleHandle bundle3 = (BundleWriter.PendingBundleHandle)packet3.Bundle;
			Assert.AreEqual(bundle2, bundle3);

			TestNode node3 = await export3.ReadBlobAsync<TestNode>();
			Assert.AreEqual(789, node3.Value);
		}
	}
}
