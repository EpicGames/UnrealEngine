// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Storage
{
	[TestClass]
	public class WorkspaceTests
	{
		static ReadOnlyMemory<byte> HelloBytes { get; } = Encoding.UTF8.GetBytes("hello");
		static ReadOnlyMemory<byte> WorldBytes { get; } = Encoding.UTF8.GetBytes("world");

		static async Task<IHashedBlobRef<DirectoryNode>> CreateBasicTreeAsync(IBlobWriter blobWriter, CancellationToken cancellationToken)
		{
			DirectoryNode rootDir = new();
			rootDir.AddFile("readme.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes, cancellationToken));
			return await blobWriter.WriteBlobAsync(rootDir, cancellationToken);
		}

		static async Task AssertContentsEqualAsync(IHashedBlobRef<DirectoryNode> dirRefA, IHashedBlobRef<DirectoryNode> dirRefB)
		{
			DirectoryNode dirA = await dirRefA.ReadBlobAsync();
			DirectoryNode dirB = await dirRefB.ReadBlobAsync();
			dirA.DeleteDirectory(".horde");
			dirB.DeleteDirectory(".horde");

			Assert.AreEqual(dirA.Directories.Count, dirB.Directories.Count);
			foreach (DirectoryEntry dirEntryA in dirA.Directories)
			{
				DirectoryEntry dirEntryB = dirB.GetDirectoryEntry(dirEntryA.Name);
				Assert.AreEqual(dirEntryA.Name, dirEntryB.Name, false);
				Assert.AreEqual(dirEntryA.Length, dirEntryB.Length);
				await AssertContentsEqualAsync(dirEntryA.Handle, dirEntryB.Handle);
			}

			Assert.AreEqual(dirA.Files.Count, dirB.Files.Count);
			foreach (FileEntry fileEntryA in dirA.Files)
			{
				FileEntry fileEntryB = dirB.GetFileEntry(fileEntryA.Name);
				Assert.AreEqual(fileEntryA.Name, fileEntryB.Name, false);
				Assert.AreEqual(fileEntryA.Length, fileEntryB.Length);

				byte[] dataA = await fileEntryA.ReadAllBytesAsync();
				byte[] dataB = await fileEntryB.ReadAllBytesAsync();

				Assert.IsTrue(MemoryExtensions.SequenceEqual<byte>(dataA, dataB));
			}
		}

		static DirectoryInfo CreateTempDirectory([CallerMemberName] string? callerName = null)
		{
			DirectoryInfo tempDir = new($"temp/{callerName ?? Guid.NewGuid().ToString()}");
			tempDir.Create();
			FileUtils.ForceDeleteDirectoryContents(tempDir);
			return tempDir;
		}

		[TestMethod]
		public async Task MaterializeManuallyAsync()
		{
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());
			IHashedBlobRef<DirectoryNode> input = await CreateBasicTreeAsync(blobWriter, default);

			DirectoryInfo tempDir = CreateTempDirectory();
			await input.ExtractAsync(tempDir, new ExtractOptions(), NullLogger.Instance, default);

			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(input, output);
		}

		[TestMethod]
		public async Task SyncTestAsync()
		{
			DirectoryInfo tempDir = CreateTempDirectory();
			Workspace ws = await Workspace.CreateAsync(new DirectoryReference(tempDir), NullLogger.Instance);
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<DirectoryNode> initialState;
			{
				DirectoryNode rootDir = new();
				rootDir.AddFile("readme.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				initialState = await blobWriter.WriteBlobAsync(rootDir);
			}

			WorkspaceSyncStats initialStats = new();
			await ws.SyncAsync(WorkspaceLayerId.Default, initialState, null, stats: initialStats);

			WorkspaceSyncStats updateStats = new();
			await ws.SyncAsync(WorkspaceLayerId.Default, initialState, null, stats: updateStats);
			Assert.AreEqual(1, updateStats.NumFilesKept);
			Assert.AreEqual(0, updateStats.NumBytesReused);
			Assert.AreEqual(0, updateStats.NumBytesDownloaded);

			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(initialState, output);
		}

		[TestMethod]
		public async Task MoveTestAsync()
		{
			DirectoryInfo tempDir = CreateTempDirectory();
			Workspace ws = await Workspace.CreateAsync(new DirectoryReference(tempDir), NullLogger.Instance);
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<DirectoryNode> initialState;
			{
				DirectoryNode rootDir = new();
				rootDir.AddFile("readme.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				initialState = await blobWriter.WriteBlobAsync(rootDir);
			}
			await ws.SyncAsync(WorkspaceLayerId.Default, initialState, null);

			IHashedBlobRef<DirectoryNode> updatedState;
			{
				DirectoryNode rootDir = new();
				rootDir.AddFile("file2.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				updatedState = await blobWriter.WriteBlobAsync(rootDir);
			}
			await ws.SyncAsync(WorkspaceLayerId.Default, updatedState, null);

			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(updatedState, output);
		}

		[TestMethod]
		public async Task CopyTestAsync()
		{
			DirectoryInfo tempDir = CreateTempDirectory();
			Workspace ws = await Workspace.CreateAsync(new DirectoryReference(tempDir), NullLogger.Instance);
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<DirectoryNode> initialState;
			{
				DirectoryNode rootDir = new();
				rootDir.AddFile("readme.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				initialState = await blobWriter.WriteBlobAsync(rootDir);
			}
			await ws.SyncAsync(WorkspaceLayerId.Default, initialState, null);

			IHashedBlobRef<DirectoryNode> updatedState;
			{
				DirectoryNode rootDir = new();
				rootDir.AddFile("file2.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				rootDir.AddFile("file3.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				updatedState = await blobWriter.WriteBlobAsync(rootDir);
			}
			await ws.SyncAsync(WorkspaceLayerId.Default, updatedState, null);

			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(updatedState, output);
		}

		[TestMethod]
		public async Task LayerTestAsync()
		{
			DirectoryInfo tempDir = CreateTempDirectory();
			Workspace ws = await Workspace.CreateAsync(new DirectoryReference(tempDir), NullLogger.Instance);
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<DirectoryNode> emptyState;
			{
				DirectoryNode rootDir = new();
				emptyState = await blobWriter.WriteBlobAsync(rootDir);
			}

			IHashedBlobRef<DirectoryNode> initialState;
			{
				DirectoryNode subDirA = new();
				subDirA.AddFile("hello.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				subDirA.AddFile("world.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, WorldBytes));
				IHashedBlobRef<DirectoryNode> subDirRefA = await blobWriter.WriteBlobAsync(subDirA);

				DirectoryNode rootDir = new();
				rootDir.AddDirectory(new DirectoryEntry("A", subDirA.Length, subDirRefA));
				initialState = await blobWriter.WriteBlobAsync(rootDir);
			}

			WorkspaceLayerId layerA = new("layer-a");
			ws.AddLayer(layerA);

			WorkspaceLayerId layerB = new("layer-b");
			ws.AddLayer(layerB);

			// Add the files to layer A
			WorkspaceSyncStats stats = new();
			await ws.SyncAsync(layerA, initialState, stats: stats);
			Assert.AreEqual(0, stats.NumFilesKept);
			Assert.AreEqual(10, stats.NumBytesDownloaded);

			// Add the same files to layer B
			stats = new WorkspaceSyncStats();
			await ws.SyncAsync(layerB, initialState, stats: stats);
			Assert.AreEqual(2, stats.NumFilesKept);
			Assert.AreEqual(0, stats.NumBytesDownloaded);

			// Remove layer B
			stats = new WorkspaceSyncStats();
			await ws.SyncAsync(layerB, null, stats: stats);
			Assert.AreEqual(0, stats.NumFilesKept);
			Assert.AreEqual(0, stats.NumBytesDownloaded);

			// Check it's still the same
			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(initialState, output);

			// Remove layer A
			stats = new WorkspaceSyncStats();
			await ws.SyncAsync(layerA, null, stats: stats);
			Assert.AreEqual(0, stats.NumFilesKept);
			Assert.AreEqual(0, stats.NumBytesDownloaded);

			// Check it's empty
			output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(emptyState, output);
		}

		[TestMethod]
		public async Task SwapTestAsync()
		{
			DirectoryInfo tempDir = CreateTempDirectory();
			Workspace ws = await Workspace.CreateAsync(new DirectoryReference(tempDir), NullLogger.Instance);
			await using MemoryBlobWriter blobWriter = new(new BlobSerializerOptions());

			IHashedBlobRef<DirectoryNode> initialState;
			{
				DirectoryNode subDirA = new();
				subDirA.AddFile("hello.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				subDirA.AddFile("world.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, WorldBytes));
				IHashedBlobRef<DirectoryNode> subDirRefA = await blobWriter.WriteBlobAsync(subDirA);

				DirectoryNode rootDir = new();
				rootDir.AddDirectory(new DirectoryEntry("A", subDirA.Length, subDirRefA));
				rootDir.AddFile(new FileEntry("B", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes)));
				initialState = await blobWriter.WriteBlobAsync(rootDir);
			}
			await ws.SyncAsync(WorkspaceLayerId.Default, initialState, null);

			IHashedBlobRef<DirectoryNode> updatedState;
			{
				DirectoryNode subDirB = new();
				subDirB.AddFile("hello.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes));
				subDirB.AddFile("world.txt", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, WorldBytes));
				IHashedBlobRef<DirectoryNode> subDirRefB = await blobWriter.WriteBlobAsync(subDirB);

				DirectoryNode rootDir = new();
				rootDir.AddFile(new FileEntry("A", FileEntryFlags.None, await ChunkedData.CreateAsync(blobWriter, HelloBytes)));
				rootDir.AddDirectory(new DirectoryEntry("B", subDirB.Length, subDirRefB));
				updatedState = await blobWriter.WriteBlobAsync(rootDir);
			}

			WorkspaceSyncStats syncStats = new();
			await ws.SyncAsync(WorkspaceLayerId.Default, updatedState, stats: syncStats);

			IHashedBlobRef<DirectoryNode> output = await blobWriter.AddFilesAsync(new DirectoryReference(tempDir), null, null, default);
			await AssertContentsEqualAsync(updatedState, output);

			Assert.AreEqual(3, syncStats.NumFilesMoved);
			Assert.AreEqual(0, syncStats.NumBytesReused);
			Assert.AreEqual(0, syncStats.NumBytesDownloaded);
		}
	}
}
