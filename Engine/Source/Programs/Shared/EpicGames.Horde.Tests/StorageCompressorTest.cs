// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests
{
	/// <summary>
	/// Fake compressor used for testing. Simply prepends the data with const prefix.
	/// Allows for easy debugging and is enough to cause a changed hash.
	/// </summary>
	public class FakeStorageCompressor
	{
		private readonly byte[] _prefix = Encoding.UTF8.GetBytes("FAKECOMP-");

		public async Task CompressAsync(Stream input, Stream output, CancellationToken cancellationToken = default)
		{
			await output.WriteAsync(_prefix, cancellationToken);
			await input.CopyToAsync(output, cancellationToken);
		}

		public async Task DecompressAsync(Stream input, Stream output, CancellationToken cancellationToken = default)
		{
			using MemoryStream ms = new();
			using BinaryReader reader = new(input);
			byte[] prefixRead = reader.ReadBytes(_prefix.Length);
			if (!_prefix.SequenceEqual(prefixRead))
			{
				throw new ArgumentException($"Input stream is not compressed. Missing fake prefix!");
			}
			await input.CopyToAsync(output, cancellationToken);
		}
	}

	[TestClass]
	public class FakeStorageCompressorTest
	{
		[TestMethod]
		public async Task CompressAndDecompressAsync()
		{
			FakeStorageCompressor compressor = new();
			byte[] uncompressedData = [0x41, 0x42, 0x43];
			using MemoryStream input = new(uncompressedData);
			using MemoryStream output = new();
			await compressor.CompressAsync(input, output);
			byte[] compressedData = output.ToArray();
			Assert.IsFalse(compressedData.SequenceEqual(uncompressedData));

			using MemoryStream input2 = new(compressedData);
			using MemoryStream output2 = new();
			await compressor.DecompressAsync(input2, output2);
			byte[] decompressedData = output2.ToArray();
			Assert.IsTrue(decompressedData.SequenceEqual(uncompressedData));
		}
	}
}
