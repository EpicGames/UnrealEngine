// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Horde.Utilities;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Utilities
{
	[TestClass]
	public class IntervalJsonConverterTests
	{
		class TestClass
		{
			[JsonConverter(typeof(IntervalJsonConverter))]
			public TimeSpan Time { get; set; }
		}

		readonly JsonSerializerOptions _options = new();

		[TestMethod]
		public void Test()
		{
			TestEqual("2w", TimeSpan.FromDays(14));
			TestEqual("2w1d", TimeSpan.FromDays(15));
			TestEqual("5d", TimeSpan.FromDays(5));
			TestEqual("5d4h", TimeSpan.FromDays(5) + TimeSpan.FromHours(4.0));
			TestEqual("5d3m", TimeSpan.FromDays(5) + TimeSpan.FromMinutes(3.0));
			TestEqual("5d4h3m1s", TimeSpan.FromDays(5) + TimeSpan.FromHours(4) + TimeSpan.FromMinutes(3) + TimeSpan.FromSeconds(1));

			TestEqual("2h04m", TimeSpan.FromHours(2) + TimeSpan.FromMinutes(4.0), "2h4m");
			TestEqual("1m30", TimeSpan.FromSeconds(90), "1m30s");
			TestEqual("90s", TimeSpan.FromSeconds(90), "1m30s");
			TestEqual("120s", TimeSpan.FromMinutes(2), "2m");

			TestEqual("00:00:01", TimeSpan.FromSeconds(1.0), "1s");
			TestEqual("00:02:00", TimeSpan.FromMinutes(2.0), "2m");
			TestEqual("03:00:00", TimeSpan.FromHours(3.0), "3h");

			TestInvalid("3");
			TestInvalid("m");
		}

		void TestEqual(string text, TimeSpan time)
		{
			TestEqual(text, time, text);
		}

		void TestEqual(string text, TimeSpan time, string finalText)
		{
			string json = $"{{\"Time\":\"{text}\"}}";

			TestClass? output = JsonSerializer.Deserialize<TestClass>(json, _options);
			Assert.IsNotNull(output);
			Assert.AreEqual(time, output.Time);

			string serializedText = JsonSerializer.Serialize(output, _options);
			Assert.AreEqual($"{{\"Time\":\"{finalText}\"}}", serializedText);
		}

		void TestInvalid(string text)
		{
			Assert.ThrowsExactly<JsonException>(() => JsonSerializer.Deserialize<TestClass>(text, _options));
		}
	}
}
