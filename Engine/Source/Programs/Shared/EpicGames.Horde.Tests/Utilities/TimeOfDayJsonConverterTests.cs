// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Utilities;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests.Utilities
{
	[TestClass]
	public class TimeOfDayJsonConverterTests
	{
		[TestMethod]
		[DataRow("4:50", 4, 50)]
		[DataRow("4:50am", 4, 50)]
		[DataRow("4:50pm", 16, 50)]
		[DataRow("4.50", 4, 50)]
		[DataRow("4pm", 16, 0)]
		[DataRow("16:50", 16, 50)]
		public void TestTimeSpan(string time, int hours, int minutes)
		{
			TimeSpan span = TimeOfDayJsonConverter.Parse(time);
			Assert.AreEqual(hours, span.Hours);
			Assert.AreEqual(minutes, span.Minutes);
		}
	}
}
