// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.OIDC;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Horde.Tests;

/// <summary>
/// Logger that fails the test if <see cref="LogLevel.Error"/> or higher is logged.
/// This surfaces errors that <c>AddRefreshToken</c>'s catch-all would otherwise swallow.
/// </summary>
sealed class FailOnErrorLogger<T>(Action<string> failAction) : ILogger<T>
{
	public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null;
	public bool IsEnabled(LogLevel logLevel) => true;

	public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
	{
		if (logLevel >= LogLevel.Error)
		{
			failAction($"Unexpected {logLevel}: {formatter(state, exception)}{(exception != null ? $"\n{exception}" : "")}");
		}
	}
}

/// <summary>
/// End-to-end tests for <see cref="WindowsTokenStore"/> including DPAPI encrypt/decrypt
/// and file I/O. These tests are Windows-only because they depend on CryptProtectData.
/// </summary>
[TestClass]
[SupportedOSPlatform("windows")]
public sealed class WindowsTokenStoreTests : IDisposable
{
	private readonly string _tempDir;
	private readonly string _storePath;
	private readonly ILogger<WindowsTokenStore> _logger;

	public WindowsTokenStoreTests()
	{
		_tempDir = Path.Combine(Path.GetTempPath(), $"WindowsTokenStoreTests_{Guid.NewGuid():N}");
		Directory.CreateDirectory(_tempDir);
		_storePath = Path.Combine(_tempDir, "oidcTokenStore.dat");
		_logger = new FailOnErrorLogger<WindowsTokenStore>(msg => Assert.Fail(msg));
	}

	public void Dispose()
	{
		try
		{
			Directory.Delete(_tempDir, true);
		}
		catch
		{
			// Best effort cleanup
		}
	}

	private WindowsTokenStore CreateStore() => new(_logger, _storePath);

	private static bool IsWindows => RuntimeInformation.IsOSPlatform(OSPlatform.Windows);

	/// <summary>
	/// Round-trip: AddRefreshToken persists to disk, TryGetRefreshToken reads it back
	/// through DPAPI decrypt.
	/// </summary>
	[TestMethod]
	public void AddAndGet_RoundTrip()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		using WindowsTokenStore store = CreateStore();
		store.AddRefreshToken("myProvider", "secret-refresh-token-123");

		Assert.IsTrue(File.Exists(_storePath), "Store file should exist after AddRefreshToken.");

		// Read back with a fresh instance (proves it's persisted, not just in memory)
		using WindowsTokenStore store2 = CreateStore();
		Assert.IsTrue(store2.TryGetRefreshToken("myProvider", out string? token));
		Assert.AreEqual("secret-refresh-token-123", token);
	}

	/// <summary>
	/// Multiple providers can coexist in the same store file.
	/// </summary>
	[TestMethod]
	public void MultipleProviders_Coexist()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		using WindowsTokenStore store = CreateStore();
		store.AddRefreshToken("providerA", "tokenA");
		store.AddRefreshToken("providerB", "tokenB");

		using WindowsTokenStore store2 = CreateStore();
		Assert.IsTrue(store2.TryGetRefreshToken("providerA", out string? a));
		Assert.AreEqual("tokenA", a);
		Assert.IsTrue(store2.TryGetRefreshToken("providerB", out string? b));
		Assert.AreEqual("tokenB", b);
	}

	/// <summary>
	/// Overwriting a provider's token replaces the old value.
	/// </summary>
	[TestMethod]
	public void AddRefreshToken_Overwrite_ReplacesOldValue()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		using WindowsTokenStore store = CreateStore();
		store.AddRefreshToken("prov", "old-token");
		store.AddRefreshToken("prov", "new-token");

		using WindowsTokenStore store2 = CreateStore();
		Assert.IsTrue(store2.TryGetRefreshToken("prov", out string? token));
		Assert.AreEqual("new-token", token);
	}

	/// <summary>
	/// TryGetRefreshToken returns false for a provider that was never stored.
	/// </summary>
	[TestMethod]
	public void TryGetRefreshToken_MissingProvider_ReturnsFalse()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		using WindowsTokenStore store = CreateStore();
		Assert.IsFalse(store.TryGetRefreshToken("nonexistent", out _));
	}

	/// <summary>
	/// TryGetRefreshToken returns false when the store file doesn't exist.
	/// </summary>
	[TestMethod]
	public void TryGetRefreshToken_NoFile_ReturnsFalse()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		Assert.IsFalse(File.Exists(_storePath));
		using WindowsTokenStore store = CreateStore();
		Assert.IsFalse(store.TryGetRefreshToken("any", out _));
	}

	/// <summary>
	/// Corrupt JSON in the store file is handled gracefully — TryGetRefreshToken
	/// returns false instead of throwing.
	/// </summary>
	[TestMethod]
	public void TryGetRefreshToken_CorruptJson_ReturnsFalse()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		Directory.CreateDirectory(Path.GetDirectoryName(_storePath)!);
		File.WriteAllText(_storePath, "{{not valid json!!");

		using WindowsTokenStore store = CreateStore();
		Assert.IsFalse(store.TryGetRefreshToken("anyProvider", out _));
	}

	/// <summary>
	/// A store file with one valid and one corrupt Base64 entry still returns
	/// the valid provider's token (the corrupt entry is skipped).
	/// </summary>
	[TestMethod]
	public void TryGetRefreshToken_CorruptBase64Entry_SkipsCorruptProvider()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		// First, write a valid token so we get a real DPAPI-encrypted entry
		using WindowsTokenStore store = CreateStore();
		store.AddRefreshToken("good", "valid-token");

		// Now manually corrupt the store file by adding a bad Base64 entry
		string json = File.ReadAllText(_storePath);
		WindowsTokenStoreState? state = JsonSerializer.Deserialize(json, WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
		Assert.IsNotNull(state);
		state!.Providers["corrupt"] = "not-valid-base64!!!";
		string corrupted = JsonSerializer.Serialize(state, WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
		File.WriteAllText(_storePath, corrupted);

		// The good provider should still work
		using WindowsTokenStore store2 = CreateStore();
		Assert.IsTrue(store2.TryGetRefreshToken("good", out string? token));
		Assert.AreEqual("valid-token", token);

		// The corrupt provider should return false (not throw)
		Assert.IsFalse(store2.TryGetRefreshToken("corrupt", out _));
	}

	/// <summary>
	/// AddRefreshToken does not throw even when the store file is read-only
	/// (simulating a permissions issue). The token is silently lost.
	/// </summary>
	[TestMethod]
	public void AddRefreshToken_ReadOnlyFile_DoesNotThrow()
	{
		if (!IsWindows)
		{
			Assert.Inconclusive("Windows-only test (requires DPAPI).");
		}

		// Create a valid store first
		using WindowsTokenStore store = CreateStore();
		store.AddRefreshToken("prov", "original");

		// Make the file read-only
		File.SetAttributes(_storePath, FileAttributes.ReadOnly);
		try
		{
			// This should not throw — AddRefreshToken catches and logs the error.
			// Use a null logger here since we EXPECT the error to be swallowed.
			using WindowsTokenStore store2 = new(NullLogger<WindowsTokenStore>.Instance, _storePath);
			store2.AddRefreshToken("prov", "updated");

			// The original token should still be there (write failed silently)
			using WindowsTokenStore store3 = CreateStore();
			Assert.IsTrue(store3.TryGetRefreshToken("prov", out string? token));
			Assert.AreEqual("original", token);
		}
		finally
		{
			File.SetAttributes(_storePath, FileAttributes.Normal);
		}
	}
}

/// <summary>
/// Tests for <see cref="WindowsTokenStoreState"/> JSON serialization (cross-platform, no DPAPI).
/// </summary>
[TestClass]
public sealed class WindowsTokenStoreStateTests
{
	[TestMethod]
	public void RoundTrip_SerializeDeserialize()
	{
		byte[] bytesA = [1, 2, 3, 4];
		byte[] bytesB = [10, 20, 30];
		WindowsTokenStoreState state = new(new Dictionary<string, string> { ["providerA"] = Convert.ToBase64String(bytesA), ["providerB"] = Convert.ToBase64String(bytesB) });

		string json = JsonSerializer.Serialize(state, WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
		WindowsTokenStoreState? deserialized = JsonSerializer.Deserialize(json, WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);

		Assert.IsNotNull(deserialized);
		CollectionAssert.AreEqual(bytesA, Convert.FromBase64String(deserialized!.Providers["providerA"]));
		CollectionAssert.AreEqual(bytesB, Convert.FromBase64String(deserialized.Providers["providerB"]));
	}

	[TestMethod]
	public void Deserialize_InvalidJson_ReturnsNull()
	{
		WindowsTokenStoreState? result = null;
		try
		{
			result = JsonSerializer.Deserialize("not valid json{{{", WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
		}
		catch (JsonException)
		{
			// Expected — callers catch this and treat as empty store
		}

		Assert.IsNull(result);
	}
}

/// <summary>
/// Tests that <see cref="OidcTokenClient"/> callers are resilient to silent store failures.
/// </summary>
[TestClass]
public sealed class TokenStoreFailureTests
{
	private const string Provider = "test";

	private static TokenRefreshResult SuccessResult(string accessToken = "newAccess", string refreshToken = "newRefresh") =>
		new(IsError: false, Error: null, ErrorDescription: null,
			AccessToken: accessToken, RefreshToken: refreshToken,
			AccessTokenExpiration: DateTimeOffset.UtcNow.AddHours(1));

	/// <summary>
	/// When the store silently fails to persist (simulating WindowsTokenStore's
	/// catch-and-log on I/O error), the client still returns the access token.
	/// </summary>
	[TestMethod]
	public async Task RefreshToken_StoreWriteFails_StillReturnsTokenAsync()
	{
		using FailingTokenStore store = new();
		store.AddRefreshToken(Provider, "initialToken");

		int callCount = 0;
		using TestableOidcTokenClient client = new(Provider, store, _ =>
		{
			callCount++;
			store.SetDropWrites(true);
			return SuccessResult("access1", "rotatedToken");
		});

		IOidcTokenInfo? result = await client.TryGetAccessTokenAsync(CancellationToken.None);

		Assert.IsNotNull(result);
		Assert.AreEqual("access1", result!.AccessToken);
		Assert.AreEqual(1, callCount);
		Assert.IsTrue(store.TryGetRefreshToken(Provider, out string? storedToken));
		Assert.AreEqual("initialToken", storedToken, "Write was silently dropped; old token should remain.");
	}

	[TestMethod]
	public async Task TryGetAccessToken_EmptyStore_ThrowsNotLoggedInAsync()
	{
		using FailingTokenStore store = new();
		using TestableOidcTokenClient client = new(Provider, store, _ => SuccessResult());

		await Assert.ThrowsExactlyAsync<NotLoggedInException>(
			() => client.TryGetAccessTokenAsync(CancellationToken.None));
	}
}

/// <summary>
/// Token store that silently drops writes when configured, simulating WindowsTokenStore's
/// catch-and-log behavior on persist failure.
/// </summary>
sealed class FailingTokenStore : ITokenStore
{
	private readonly Dictionary<string, string> _tokens = [];
	private bool _dropWrites;

	public void SetDropWrites(bool drop) => _dropWrites = drop;

	public bool TryGetRefreshToken(string oidcProvider, out string refreshToken)
	{
		if (!_tokens.TryGetValue(oidcProvider, out string? result))
		{
			refreshToken = "";
			return false;
		}

		refreshToken = result;
		return true;
	}

	public void AddRefreshToken(string oidcProvider, string refreshToken)
	{
		if (!_dropWrites)
		{
			_tokens[oidcProvider] = refreshToken;
		}
	}

	public void Dispose()
	{
	}
}
