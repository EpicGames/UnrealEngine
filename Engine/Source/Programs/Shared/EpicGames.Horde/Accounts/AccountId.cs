// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Accounts
{
	/// <summary>
	/// Identifier for a user account
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<AccountId, AccountIdConverter>))]
	[BinaryIdConverter(typeof(AccountIdConverter))]
	public readonly record struct AccountId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static AccountId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc cref="BinaryId.TryParse(ReadOnlySpan{Char}, out BinaryId)"/>
		public static bool TryParse(ReadOnlySpan<char> text, out AccountId result)
		{
			BinaryId binaryId;
			if (BinaryId.TryParse(text, out binaryId))
			{
				result = new AccountId(binaryId);
				return true;
			}
			else
			{
				result = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class AccountIdConverter : BinaryIdConverter<AccountId>
	{
		/// <inheritdoc/>
		public override AccountId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(AccountId value) => value.Id;
	}
}
