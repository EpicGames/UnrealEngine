// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Horde.Acls;

#pragma warning disable CA1054 // URI-like parameters should not be strings
#pragma warning disable CA1056 // Change string to URI

namespace EpicGames.Horde.Accounts
{
	/// <summary>
	/// Creates a new user account
	/// </summary>
	/// <param name="Name">Name of the user</param>
	/// <param name="Login">Perforce login identifier</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Email">User's email address</param>
	/// <param name="Password">Password for the user</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record class CreateAccountRequest(string Name, string Login, List<AccountClaimMessage> Claims, string? Description, string? Email, string? Password, bool? Enabled);

	/// <summary>
	/// Creates the admin account
	/// </summary>
	/// <param name="Password">Password for the user</param>
	public record class CreateAdminAccountRequest(string Password);

	/// <summary>
	/// Response from the request to create a new user account
	/// </summary>
	/// <param name="Id">The created account id</param>
	public record class CreateAccountResponse(AccountId Id);

	/// <summary>
	/// Update request for a user account
	/// </summary>
	/// <param name="Name">Name of the user</param>
	/// <param name="Login">Perforce login identifier</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Email">User's email address</param>
	/// <param name="SecretToken">Optional secret token for API access</param>
	/// <param name="Password">Password for the user</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record class UpdateAccountRequest(string? Name, string? Login, List<AccountClaimMessage>? Claims, string? Description, string? Email, string? SecretToken, string? Password, bool? Enabled);

	/// <summary>
	/// Update request for the current user account
	/// </summary>
	/// <param name="OldPassword">Old password for the user</param>
	/// <param name="NewPassword">New password for the user</param>
	public record class UpdateCurrentAccountRequest(string? OldPassword, string? NewPassword);

	/// <summary>
	/// Creates a new user account
	/// </summary>
	/// <param name="Id">Id of the account</param>
	/// <param name="Name">Name of the user</param>
	/// <param name="Login">Perforce login identifier</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Email">User's email address</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record class GetAccountResponse(AccountId Id, string Name, string Login, List<AccountClaimMessage> Claims, string? Description, string? Email, bool Enabled);

	/// <summary>
	/// Message describing a claim for an account
	/// </summary>
	/// <param name="Type">Claim type</param>
	/// <param name="Value">Value of the claim</param>
	public record class AccountClaimMessage(string Type, string Value);

	/// <summary>
	/// Dashboard login request
	/// </summary>
	/// <param name="Username">Username</param>
	/// <param name="Password">Password</param>
	/// <param name="ReturnUrl">Url to return to upon success</param>
	public record class DashboardLoginRequest(string Username, string? Password, string? ReturnUrl);

	/// <summary>
	/// Gets all entitlements for an account
	/// </summary>
	/// <param name="Administrator">Whether the user is an administrator</param>
	/// <param name="Scopes">List of scopes with entitlements</param>
	public record class GetAccountEntitlementsResponse(bool Administrator, List<GetAccountScopeEntitlementsResponse> Scopes);

	/// <summary>
	/// Creates a new user account
	/// </summary>
	/// <param name="Name">Name of the scope</param>
	/// <param name="Actions">Actions for this scope</param>
	public record class GetAccountScopeEntitlementsResponse(string Name, List<AclAction> Actions);
}
