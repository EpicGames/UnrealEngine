// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde.Acls
{
	/// <summary>
	/// Wraps a string used to describe an ACL action
	/// </summary>
	/// <param name="Name">Name of the action</param>
	[JsonSchemaString]
	[JsonConverter(typeof(AclActionJsonConverter))]
	[TypeConverter(typeof(AclActionTypeConverter))]
	public readonly record struct AclAction(string Name)
	{
		/// <inheritdoc/>
		public override string ToString() => Name;
	}

	/// <summary>
	/// Type converter for NamespaceId to and from JSON
	/// </summary>
	sealed class AclActionJsonConverter : JsonConverter<AclAction>
	{
		/// <inheritdoc/>
		public override AclAction Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(reader.GetString()!);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, AclAction value, JsonSerializerOptions options) => writer.WriteStringValue(value.Name);
	}

	/// <summary>
	/// Type converter from strings to NamespaceId objects
	/// </summary>
	sealed class AclActionTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new AclAction((string)value);
	}
}
