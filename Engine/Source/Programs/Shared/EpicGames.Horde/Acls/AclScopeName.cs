// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Horde.Acls
{
	/// <summary>
	/// Name of an ACL scope
	/// </summary>
	[DebuggerDisplay("{Text}")]
	[JsonConverter(typeof(AclScopeNameJsonConverter))]
	[TypeConverter(typeof(AclScopeNameTypeConverter))]
	public record struct AclScopeName(string Text)
	{
		/// <summary>
		/// The root scope name
		/// </summary>
		public static AclScopeName Root { get; } = new AclScopeName("horde");

		/// <summary>
		/// Append another name to this scope
		/// </summary>
		/// <param name="name">Name to append</param>
		/// <returns>New scope name</returns>
		public AclScopeName Append(string name)
			=> String.IsNullOrEmpty(Text) ? new AclScopeName(name) : new AclScopeName($"{Text}/{name}");

		/// <inheritdoc/>
		public bool Equals(AclScopeName other)
			=> String.Equals(Text ?? String.Empty, other.Text ?? String.Empty, StringComparison.Ordinal);

		/// <inheritdoc/>
		public override int GetHashCode() => (Text ?? String.Empty).GetHashCode(StringComparison.Ordinal);

		/// <inheritdoc/>
		public override string ToString() => Text;
	}

	/// <summary>
	/// Serializes <see cref="AclScopeName"/> objects to JSON
	/// </summary>
	class AclScopeNameJsonConverter : JsonConverter<AclScopeName>
	{
		/// <inheritdoc/>
		public override AclScopeName Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(reader.GetString() ?? String.Empty);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, AclScopeName value, JsonSerializerOptions options) => writer.WriteStringValue(value.Text);
	}

	/// <summary>
	/// Converts <see cref="AclScopeName"/> objects to strings
	/// </summary>
	class AclScopeNameTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || base.CanConvertFrom(context, sourceType);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return new AclScopeName(str);
			}
			return base.ConvertFrom(context, culture, value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, [NotNullWhen(true)] Type? destinationType)
		{
			return destinationType == typeof(string) || base.CanConvertTo(context, destinationType);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}
}
