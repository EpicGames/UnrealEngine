// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Acls
{
	/// <summary>
	/// Claim for an ACL
	/// </summary>
	public interface IAclClaim
	{
		/// <summary>
		/// Type of the claim
		/// </summary>
		string Type { get; }

		/// <summary>
		/// Value for the claim
		/// </summary>
		string Value { get; }
	}
}
