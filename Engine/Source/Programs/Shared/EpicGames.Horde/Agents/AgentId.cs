// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde.Agents
{
	/// <summary>
	/// Normalized hostname of an agent
	/// </summary>
	[JsonSchemaString]
	[LogValueType]
	[TypeConverter(typeof(AgentIdTypeConverter))]
	[JsonConverter(typeof(AgentIdJsonConverter))]
	public readonly struct AgentId : IEquatable<AgentId>, IComparable<AgentId>
	{
		/// <summary>
		/// The text representing this id
		/// </summary>
		readonly string _name;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Hostname of the agent</param>
		public AgentId(string name)
		{
			if (name.Length == 0)
			{
				throw new ArgumentException("Agent name may not be empty");
			}

			char[] result = new char[name.Length];
			for (int idx = 0; idx < name.Length; idx++)
			{
				char character = name[idx];
				if ((character >= 'A' && character <= 'Z') || (character >= '0' && character <= '9') || character == '_' || character == '-')
				{
					result[idx] = character;
				}
				else if (character >= 'a' && character <= 'z')
				{
					result[idx] = (char)('A' + (character - 'a'));
				}
				else if (character == '.')
				{
					if (idx == 0)
					{
						throw new ArgumentException("Agent name may not begin with the '.' character");
					}
					else if (idx == name.Length - 1)
					{
						throw new ArgumentException("Agent name may not end with the '.' character");
					}
					else if (name[idx - 1] == '.')
					{
						throw new ArgumentException("Agent name may not contain two consecutive '.' characters");
					}
					else
					{
						result[idx] = character;
					}
				}
				else
				{
					throw new ArgumentException($"Character '{character}' in agent '{name}' is invalid");
				}
			}
			_name = new string(result);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			return obj is AgentId id && Equals(id);
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return (_name ?? String.Empty).GetHashCode(StringComparison.Ordinal);
		}

		/// <inheritdoc/>
		public int CompareTo(AgentId other)
			=> String.Compare(_name, other._name, StringComparison.Ordinal);

		/// <inheritdoc/>
		public bool Equals(AgentId other)
		{
			return String.Equals(_name, other._name, StringComparison.Ordinal);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return _name ?? String.Empty;
		}

#pragma warning disable CS1591
		public static bool operator ==(AgentId left, AgentId right)
			=> left.Equals(right);

		public static bool operator !=(AgentId left, AgentId right)
			=> !left.Equals(right);

		public static bool operator <(AgentId left, AgentId right)
			=> left.CompareTo(right) < 0;

		public static bool operator <=(AgentId left, AgentId right)
			=> left.CompareTo(right) <= 0;

		public static bool operator >(AgentId left, AgentId right)
			=> left.CompareTo(right) > 0;

		public static bool operator >=(AgentId left, AgentId right)
			=> left.CompareTo(right) >= 0;
#pragma warning restore CS1591
	}

	/// <summary>
	/// Type converter from strings to PropertyFilter objects
	/// </summary>
	sealed class AgentIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return new AgentId((string)value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			return destinationType == typeof(string);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			else
			{
				return null;
			}
		}
	}

	/// <summary>
	/// Class which serializes AgentId objects to JSON
	/// </summary>
	public sealed class AgentIdJsonConverter : JsonConverter<AgentId>
	{
		/// <inheritdoc/>
		public override AgentId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(reader.GetString() ?? String.Empty);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, AgentId value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
	}
}
