// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Logs;

namespace EpicGames.Horde.Agents
{
	/// <summary>
	/// Status of an agent. Must match RpcAgentStatus.
	/// </summary>
	public enum AgentStatus
	{
		/// <summary>
		/// Unspecified state.
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Agent is running normally.
		/// </summary>
		Ok = 1,

		/// <summary>
		/// Agent is currently shutting down, and should not be assigned new leases.
		/// </summary>
		Stopping = 2,

		/// <summary>
		/// Agent is in an unhealthy state and should not be assigned new leases.
		/// </summary>
		Unhealthy = 3,

		/// <summary>
		/// Agent is currently stopped.
		/// </summary>
		Stopped = 4,

		/// <summary>
		/// Agent is busy performing other work (eg. serving an interactive user)
		/// </summary>
		Busy = 5,
	}

	/// <summary>
	/// Parameters to update an agent
	/// </summary>
	/// <param name="Enabled"> Whether the agent is currently enabled </param>
	/// <param name="Ephemeral"> Whether the agent is ephemeral </param>
	/// <param name="RequestConform"> Request a conform be performed using the current agent set </param>
	/// <param name="RequestFullConform"> Request that a full conform be performed, removing all intermediate files </param>
	/// <param name="RequestRestart"> Request the machine be restarted </param>
	/// <param name="RequestShutdown"> Request the machine be shut down </param>
	/// <param name="RequestForceRestart"> Request the machine be restarted without waiting for leases to complete </param>
	/// <param name="Pools"> Pools for this agent </param>
	/// <param name="Comment"> New comment </param>
	public record UpdateAgentRequest(bool? Enabled = null, bool? Ephemeral = null, bool? RequestConform = null, bool? RequestFullConform = null, bool? RequestRestart = null, bool? RequestShutdown = null, bool? RequestForceRestart = null, List<string>? Pools = null, string? Comment = null);

	/// <summary>
	/// Response for queries to find a particular lease within an agent
	/// </summary>
	/// <param name="Id"> Identifier for the lease </param>
	/// <param name="ParentId"> Identifier for the parent lease. Used to terminate hierarchies of leases. </param>
	/// <param name="AgentId"> The agent id </param>
	/// <param name="AgentRate"> Cost of this agent, per hour </param>
	/// <param name="Name"> Name of the lease </param>
	/// <param name="LogId"> Log id for this lease </param>
	/// <param name="StartTime"> Time at which the lease started (UTC) </param>
	/// <param name="FinishTime"> Time at which the lease started (UTC) </param>
	/// <param name="Executing"> Whether this lease has started executing on the agent yet </param>
	/// <param name="Details"> Details of the payload being executed </param>
	/// <param name="Outcome"> Outcome of the lease </param>
	/// <param name="OutcomeReasons">List of outcome reasons, in string form</param>
	/// <param name="OutcomeReasonsFlags">Outcome reasons as a uint64 (bit flags)</param>
	/// <param name="State"> State of the lease (for AgentLeases) </param>
	public record GetAgentLeaseResponse(LeaseId Id, LeaseId? ParentId, AgentId? AgentId, double? AgentRate, string Name, LogId? LogId, DateTime StartTime, DateTime? FinishTime, bool Executing, Dictionary<string, string>? Details, LeaseOutcome? Outcome, string[] OutcomeReasons, long? OutcomeReasonsFlags, LeaseState? State);

	/// <summary>
	/// Information about a workspace synced on an agent
	/// </summary>
	/// <param name="Cluster"> The Perforce server and port to connect to </param>
	/// <param name="UserName"> User to log into Perforce with (eg. buildmachine) </param>
	/// <param name="Identifier"> Identifier to distinguish this workspace from other workspaces </param>
	/// <param name="Stream"> The stream to sync </param>
	/// <param name="View"> Custom view for the workspace </param>
	/// <param name="BIncremental"> Whether to use an incremental workspace </param>
	/// <param name="Method"> Method to use when syncing/materializing data from Perforce </param>
	/// <param name="MinScratchSpace">Minimum disk space that must be available *after* syncing this workspace (in megabytes)</param>
	/// <param name="ConformDiskFreeSpace">Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk</param>
	public record GetAgentWorkspaceResponse(string? Cluster, string? UserName, string Identifier, string Stream, List<string>? View, bool BIncremental, string? Method, long? MinScratchSpace, long? ConformDiskFreeSpace);

	/// <summary>
	/// Information about an agent
	/// </summary>
	/// <param name="Id">The agent's unique ID</param>
	/// <param name="Name">Friendly name of the agent</param>
	/// <param name="Enabled">Whether the agent is currently enabled</param>
	/// <param name="Status">Status of the agent</param>
	/// <param name="Rate">Cost estimate per-hour for this agent</param>
	/// <param name="SessionId">The current session id</param>
	/// <param name="Ephemeral">Whether the agent is ephemeral</param>
	/// <param name="Online">Whether the agent is currently online</param>
	/// <param name="Deleted">Whether this agent has expired</param>
	/// <param name="PendingConform">Whether a conform job is pending</param>
	/// <param name="PendingFullConform">Whether a full conform job is pending</param>
	/// <param name="PendingRestart">Whether a restart is pending</param>
	/// <param name="PendingShutdown">Whether a shutdown is pending</param>
	/// <param name="LastShutdownReason">The reason for the last shutdown</param>
	/// <param name="LastConformTime">Last time a conform was attempted</param>
	/// <param name="ConformAttemptCount">Number of times a conform has been attempted</param>
	/// <param name="NextConformTime">Last time a conform was attempted</param>
	/// <param name="Version">The current client version</param>
	/// <param name="Properties">Properties for the agent</param>
	/// <param name="Resources">Resources for the agent</param>
	/// <param name="UpdateTime">Last update time of this agent</param>
	/// <param name="LastOnlineTime">Last time that the agent was online</param>
	/// <param name="Pools">Pools for this agent</param>
	/// <param name="Capabilities">Capabilities of this agent</param>
	/// <param name="Leases">Array of active leases.</param>
	/// <param name="Workspaces">Current workspaces synced on the agent</param>
	/// <param name="Comment">Comment for this agent</param>
	public record GetAgentResponse(AgentId Id, string Name, bool Enabled, AgentStatus Status, double? Rate, SessionId? SessionId, bool Ephemeral, bool Online, bool Deleted, bool PendingConform, bool PendingFullConform, bool PendingRestart, bool PendingShutdown, string LastShutdownReason, DateTime LastConformTime, int? ConformAttemptCount, DateTime? NextConformTime, string? Version, List<string> Properties, Dictionary<string, int> Resources, DateTime? UpdateTime, DateTime? LastOnlineTime, List<string>? Pools, object? Capabilities, List<GetAgentLeaseResponse> Leases, List<GetAgentWorkspaceResponse> Workspaces, string? Comment);

	/// <summary>
	/// Telemetry data for an agent
	/// </summary>
	public record GetAgentTelemetryResponse(List<GetAgentTelemetrySampleResponse> Samples);

	/// <summary>
	/// Telemetry data sample
	/// </summary>
	public record GetAgentTelemetrySampleResponse(DateTime Time, float UserCpu, float IdleCpu, float SystemCpu, int FreeRam, int UsedRam, int TotalRam, long WorkingFreeDisk, long WorkingTotalDisk, long SystemFreeDisk, long SystemTotalDisk);
}
