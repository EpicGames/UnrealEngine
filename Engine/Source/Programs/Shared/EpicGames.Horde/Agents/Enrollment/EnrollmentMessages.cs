// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Horde.Agents.Enrollment
{
	/// <summary>
	/// Updates an existing lease
	/// </summary>
	public record class GetPendingAgentsResponse(List<GetPendingAgentResponse> Agents);

	/// <summary>
	/// Information about an agent pending admission to the farm
	/// </summary>
	/// <param name="Key">Unique key used to identify this agent</param>
	/// <param name="HostName">Agent host name</param>
	/// <param name="Description">Description for the agent</param>
	public record class GetPendingAgentResponse(string Key, string HostName, string Description);

	/// <summary>
	/// Approve an agent for admission to the farm
	/// </summary>
	/// <param name="Agents">Agents to approve</param>
	public record class ApproveAgentsRequest(List<ApproveAgentRequest> Agents);

	/// <summary>
	/// Approve an agent for admission to the farm
	/// </summary>
	/// <param name="Key">Unique key for identifying the machine</param>
	/// <param name="AgentId">Agent id to use for the machine. Set to null to use the default.</param>
	public record class ApproveAgentRequest(string Key, AgentId? AgentId);
}
