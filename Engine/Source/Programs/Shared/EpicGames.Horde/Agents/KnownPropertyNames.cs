// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Agents;

/// <summary>
/// Well-known property names for agents
/// </summary>
public static class KnownPropertyNames
{
	/// <summary>
	/// The agent id
	/// </summary>
	public const string Id = "Id";

	/// <summary>
	/// The UBT platform enum
	/// </summary>
	public const string Platform = "Platform";

	/// <summary>
	/// The UBT platform group enum
	/// </summary>
	public const string PlatformGroup = "PlatformGroup";

	/// <summary>
	/// The operating system (Linux, MacOS, Windows)
	/// </summary>
	public const string OsFamily = "OSFamily";

	/// <summary>
	/// Compatible operating system (mainly for Linux WINE agents to advertise Windows support)
	/// </summary>
	public const string OsFamilyCompatibility = "OSFamilyCompatibility";

	/// <summary>
	/// Whether the agent is a .NET self-contained app
	/// </summary>
	public const string SelfContained = "SelfContained";

	/// <summary>
	/// Self-reported tool ID of an agent. Used by the server when pushing updates to select the correct tool package.
	/// </summary>
	public const string ToolId = "ToolId";

	/// <summary>
	/// Pools that this agent belongs to
	/// </summary>
	public const string Pool = "Pool";

	/// <summary>
	/// Pools requested by the agent to join when registering with server
	/// </summary>
	public const string RequestedPools = "RequestedPools";

	/// <summary>
	/// The total size of the working drive, in bytes
	/// </summary>
	public const string WorkingDiskTotalSize = "WorkingDiskTotalSize";

	/// <summary>
	/// Amount of available free space on the working drive, in bytes
	/// </summary>
	public const string WorkingDiskFreeSpace = "WorkingDiskFreeSpace";

	/// <summary>
	/// The total size of the system drive, in bytes
	/// </summary>
	public const string SystemDiskTotalSize = "SystemDiskTotalSize";

	/// <summary>
	/// Amount of available free space on the system drive, in bytes
	/// </summary>
	public const string SystemDiskFreeSpace = "SystemDiskFreeSpace";

	/// <summary>
	/// IP address used for sending compute task payloads
	/// </summary>
	public const string ComputeIp = "ComputeIp";

	/// <summary>
	/// Port used for sending compute task payloads
	/// </summary>
	public const string ComputePort = "ComputePort";

	/// <summary>
	/// Protocol version for compute task payloads
	/// </summary>
	public const string ComputeProtocol = "ComputeProtocol";

	/// <summary>
	/// AWS: Instance ID
	/// </summary>
	public const string AwsInstanceId = "aws-instance-id";

	/// <summary>
	/// AWS: Instance type
	/// </summary>
	public const string AwsInstanceType = "aws-instance-type";

	/// <summary>
	/// Whether the Wine compatibility layer is enabled (for running Windows applications on Linux)
	/// </summary>
	public const string WineEnabled = "WineEnabled";

	/// <summary>
	/// Whether the agent is trusted (boolean)
	/// </summary>
	public const string Trusted = "Trusted";

	/// <summary>
	/// Whether the agent want to receive software updates from the server (boolean)
	/// In some cases, an external script/process will take care of updating the agent (for example, Unreal Toolbox)
	/// </summary>
	public const string AutoUpdate = "AutoUpdate";

	/// <summary>
	/// ID of current lease
	/// </summary>
	public const string LeaseId = "LeaseId";
}