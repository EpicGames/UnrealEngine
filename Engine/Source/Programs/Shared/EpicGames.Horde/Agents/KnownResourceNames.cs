// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Agents
{
	/// <summary>
	/// Standard resource names
	/// </summary>
	public static class KnownResourceNames
	{
		/// <summary>
		/// Number of logical cores including SMT cores (hyper-threading)
		/// </summary>
		public const string LogicalCores = "LogicalCores";

		/// <summary>
		/// Number of physical cores
		/// </summary>
		public const string PhysicalCores = "PhysicalCores";

		/// <summary>
		/// Amount of RAM, in GB
		/// </summary>
		public const string Ram = "RAM";
	}
}
