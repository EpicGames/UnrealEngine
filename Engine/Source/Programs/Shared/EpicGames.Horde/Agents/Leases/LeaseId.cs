// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Agents.Leases
{
	/// <summary>
	/// Identifier for a user
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<LeaseId, LeaseIdConverter>))]
	[BinaryIdConverter(typeof(LeaseIdConverter))]
	public readonly record struct LeaseId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.TryParse(System.String, out BinaryId)"/>
		public static bool TryParse(string text, out LeaseId leaseId)
		{
			BinaryId objectId;
			if (BinaryId.TryParse(text, out objectId))
			{
				leaseId = new LeaseId(objectId);
				return true;
			}
			else
			{
				leaseId = default;
				return false;
			}
		}

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static LeaseId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class LeaseIdConverter : BinaryIdConverter<LeaseId>
	{
		/// <inheritdoc/>
		public override LeaseId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(LeaseId value) => value.Id;
	}
}
