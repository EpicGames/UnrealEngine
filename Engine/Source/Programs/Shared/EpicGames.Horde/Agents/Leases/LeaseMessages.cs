// Copyright Epic Games, Inc. All Rights Reserved.

using System;

#pragma warning disable CA1027 // Mark enums with FlagsAttribute

namespace EpicGames.Horde.Agents.Leases
{
	/// <summary>
	/// Outcome from a lease. Values must match lease_outcome.proto.
	/// </summary>
	public enum LeaseOutcome
	{
		/// <summary>
		/// Default value.
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// The lease was executed successfully
		/// </summary>
		Success = 1,

		/// <summary>
		/// The lease was not executed succesfully, but cannot be run again.
		/// </summary>
		Failed = 2,

		/// <summary>
		/// The lease was cancelled by request
		/// </summary>
		Cancelled = 4
	}

	/// <summary>
	/// Reason for an outcome of a lease. Values must match lease_outcome.proto.
	/// </summary>
	[Flags]
	public enum LeaseOutcomeReason : long
	{
		/// <summary>No specific reason</summary>
		None = 0,

		/// <summary>Lease finished due to an unhandled or unknown error</summary>
		UnexpectedError = 1,

		/// <summary>Lease finished with a non-zero exit code</summary>
		NonZeroExitCode = 2,

		/// <summary>User initiated</summary>
		UserInitiated = 4,

		/// <summary>Server initiated</summary>
		ServerInitiated = 8,

		/// <summary>Agent initiated</summary>
		AgentInitiated = 16,

		/// <summary>Agent's session was terminated</summary>
		AgentSessionTerminated = 32,

		/// <summary>Agent became busy during lease (e.g non-Horde related work came up running on a workstation)</summary>
		AgentBusy = 64,

		/// <summary>
		/// Lease deemed to be a duplicate
		/// For example two identical jobs executing or too many leases from a single user
		/// </summary>
		Duplicate = 128,

		/// <summary>Lease execution timed out</summary>
		TimedOut = 256,

		/// <summary>Lease no longer needed despite scheduled/started</summary>
		NoLongerNeeded = 512,

		/// <summary>Lease was cancelled but no reason was reported by the agent</summary>
		NoReasonReported = 1024,
	}

	/// <summary>
	/// State of a lease. Values must match lease_state.proto.
	/// </summary>
	public enum LeaseState
	{
		/// <summary>
		/// Default value.
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Set by the server when waiting for an agent to accept the lease. Once processed, the agent should transition the lease state to active.
		/// </summary>
		Pending = 1,

		/// <summary>
		/// The agent is actively working on this lease.
		/// </summary>
		Active = 2,

		/// <summary>
		/// The agent has finished working on this lease.
		/// </summary>
		Completed = 3,

		/// <summary>
		/// Set by the server to indicate that the lease should be cancelled.
		/// </summary>
		Cancelled = 4
	}

	/// <summary>
	/// Updates an existing lease
	/// </summary>
	public class UpdateLeaseRequest
	{
		/// <summary>
		/// Mark this lease as aborted
		/// </summary>
		public bool? Aborted { get; set; }
	}
}
