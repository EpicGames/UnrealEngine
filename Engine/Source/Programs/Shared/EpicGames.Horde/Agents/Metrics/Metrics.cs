// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Agents.Metrics;

/// <summary>
/// Metrics for CPU usage
/// </summary>
public class CpuMetrics
{
	/// <summary>
	/// Percentage of time the CPU was busy executing code in user space
	/// </summary>
	public float User { get; set; }

	/// <summary>
	/// Percentage of time the CPU was busy executing code in kernel space
	/// </summary>
	public float System { get; set; }

	/// <summary>
	/// Percentage of time the CPU was idling
	/// </summary>
	public float Idle { get; set; }

	/// <inheritdoc />
	public override string ToString()
	{
		return $"User={User,5:F1}% System={System,5:F1}% Idle={Idle,5:F1}%";
	}
}

/// <summary>
/// Metrics for memory usage
/// </summary>
public class MemoryMetrics
{
	/// <summary>
	/// Total memory installed (kibibytes)
	/// </summary>
	public uint Total { get; set; }

	/// <summary>
	/// Available memory (kibibytes)
	/// </summary>
	public uint Available { get; set; }

	/// <summary>
	/// Used memory (kibibytes)
	/// </summary>
	public uint Used { get; set; }

	/// <summary>
	/// Used memory (percentage)
	/// </summary>
	public float UsedPercentage { get; set; }

	/// <inheritdoc />
	public override string ToString()
	{
		return $"Total={Total} kB, Available={Available} kB, Used={Used} kB, UsedPct={UsedPercentage * 100.0:F1} %";
	}
}

/// <summary>
/// Metrics for free disk space
/// </summary>
public class DiskMetrics
{
	/// <summary>
	/// Amount of free space on the drive
	/// </summary>
	public long FreeSpace { get; set; }

	/// <summary>
	/// Total size of the drive
	/// </summary>
	public long TotalSize { get; set; }

	/// <inheritdoc />
	public override string ToString()
	{
		return $"Total={TotalSize} kB, Free={FreeSpace}";
	}
}

/// <summary>
/// Metrics for CPU time spent
/// </summary>
public class CpuTimeMetrics
{
	/// <summary>
	/// CPU time spent in user mode
	/// </summary>
	public TimeSpan User { get; set; }

	/// <summary>
	/// CPU time spent in system/kernel mode
	/// </summary>
	public TimeSpan System { get; set; }

	/// <summary>
	/// CPU time spent idle
	/// </summary>
	public TimeSpan Idle { get; set; }

	/// <summary>
	/// Subtract times from other object passed in
	/// </summary>
	public CpuTimeMetrics Subtract(CpuTimeMetrics other)
	{
		return new CpuTimeMetrics { User = User - other.User, System = System - other.System, Idle = Idle - other.Idle };
	}

	/// <inheritdoc />
	public override string ToString()
	{
		return $"User={(int)User.TotalMilliseconds} System={(int)System.TotalMilliseconds} Idle={(int)Idle.TotalMilliseconds}";
	}
}

/// <summary>
/// OS-agnostic interface for retrieving system metrics (CPU, memory etc)
/// </summary>
public interface ISystemMetrics
{
	/// <summary>
	/// Get CPU usage metrics
	/// </summary>
	/// <returns>An object with CPU usage metrics</returns>
	CpuMetrics? GetCpu();

	/// <summary>
	/// Get cumulative CPU time spent since system start
	/// </summary>
	/// <returns>An object with CPU time metrics</returns>
	CpuTimeMetrics? GetCpuTime();

	/// <summary>
	/// Get memory usage metrics
	/// </summary>
	/// <returns>An object with memory usage metrics</returns>
	MemoryMetrics? GetMemory();

	/// <summary>
	/// Gets Working HDD metrics
	/// </summary>
	/// <returns>An object with disk usage metrics</returns>
	DiskMetrics? GetWorkingDisk();

	/// <summary>
	/// Gets System HDD metrics
	/// </summary>
	/// <returns>An object with disk usage metrics</returns>
	DiskMetrics? GetSystemDisk();
}

/// <summary>
/// Enum to handle separate drive types tracked by Horde telemetry
/// </summary>
public enum DriveType
{
	/// <summary>
	/// System-level drive
	/// </summary>
	System = 0,
	/// <summary>
	/// Horde Working drive
	/// </summary>
	Working = 1
}