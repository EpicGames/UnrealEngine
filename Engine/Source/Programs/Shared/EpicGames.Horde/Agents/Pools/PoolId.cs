// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Agents.Pools
{
	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<PoolId, PoolIdConverter>))]
	[StringIdConverter(typeof(PoolIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<PoolId, PoolIdConverter>))]
	public readonly record struct PoolId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public PoolId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class PoolIdConverter : StringIdConverter<PoolId>
	{
		/// <inheritdoc/>
		public override PoolId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(PoolId value) => value.Id;
	}
}
