// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Agents.Sessions
{
	/// <summary>
	/// Identifier for a session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<SessionId, SessionIdConverter>))]
	[BinaryIdConverter(typeof(SessionIdConverter))]
	public readonly record struct SessionId(BinaryId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static SessionId Empty { get; } = default;

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static SessionId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc cref="BinaryId.TryParse(System.String, out BinaryId)"/>
		public static bool TryParse(string text, out SessionId id)
		{
			if (BinaryId.TryParse(text, out BinaryId objectId))
			{
				id = new SessionId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class SessionIdConverter : BinaryIdConverter<SessionId>
	{
		/// <inheritdoc/>
		public override SessionId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(SessionId value) => value.Id;
	}
}
