// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Agents.Sessions
{
	/// <summary>
	/// Information about a session
	/// </summary>
	/// <param name="Id">Unique id for this session</param>
	/// <param name="StartTime">Start time for this session</param>
	/// <param name="FinishTime">Finishing time for this session</param>
	/// <param name="Version"> Version of the software running during this session </param>
	public record GetSessionResponse(SessionId Id, DateTime StartTime, DateTime? FinishTime, string? Version);
}
