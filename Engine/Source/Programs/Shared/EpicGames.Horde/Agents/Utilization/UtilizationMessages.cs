// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Streams;

#pragma warning disable CA2227 // Collection properties should be read only

namespace EpicGames.Horde.Agents.Telemetry
{
	/// <summary>
	/// Response data for a utilization request
	/// </summary>
	public class GetUtilizationDataResponse
	{
		/// <summary>
		/// Start hour
		/// </summary>
		public DateTimeOffset StartTime { get; set; }

		/// <summary>
		/// End hour
		/// </summary>
		public DateTimeOffset FinishTime { get; set; }

		/// <summary>
		/// List of pools
		/// </summary>
		public List<GetUtilizationPoolDataResponse> Pools { get; set; } = [];

		/// <summary>
		/// Total admin time
		/// </summary>
		public double AdminTime { get; set; }

		/// <summary>
		/// Total hibernating time
		/// </summary>
		public double HibernatingTime { get; set; }

		/// <summary>
		/// Total agents
		/// </summary>
		public int NumAgents { get; set; }
	}

	/// <summary>
	/// Representation of an hour of time
	/// </summary>
	public class GetUtilizationPoolDataResponse
	{
		/// <summary>
		/// Pool id
		/// </summary>
		public PoolId PoolId { get; set; }

		/// <summary>
		/// Number of agents in this pool
		/// </summary>
		public int NumAgents { get; set; }

		/// <summary>
		/// Total time spent doing admin work
		/// </summary>
		public double AdminTime { get; set; }

		/// <summary>
		/// Time spent hibernating
		/// </summary>
		public double HibernatingTime { get; set; }

		/// <summary>
		/// Total time agents in this pool were doing work for other pools
		/// </summary>
		public double OtherTime { get; set; }

		/// <summary>
		/// List of streams
		/// </summary>
		public List<GetUtilizationStreamDataResponse> Streams { get; set; } = [];
	}

	/// <summary>
	/// Represents one stream in one pool in one hour of telemetry
	/// </summary>
	public class GetUtilizationStreamDataResponse
	{
		/// <summary>
		/// Stream Id
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// Total time
		/// </summary>
		public double Time { get; set; }
	}
}
