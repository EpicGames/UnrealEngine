// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

#pragma warning disable CA2227

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Class that defines which backend to use to fetch an artifact
	/// </summary>
	public class ArtifactBackend
	{
		/// <summary>
		/// The different types of supported backends
		/// </summary>
		public enum BackendTypeEnum
		{
			/// <summary>
			/// The horde storage backend
			/// </summary>
			Horde,
			/// <summary>
			/// A zen based backend
			/// </summary>
			Zen
		}

		/// <summary>
		/// Enum indicating which type of backend it is and thus how these options should be interpreted
		/// </summary>
		public BackendTypeEnum Type { get; set; } = BackendTypeEnum.Horde;
	}

	/// <summary>
	/// Class which describes an artifact on Horde which can be serialized to JSON.
	/// </summary>
	public class ArtifactDescriptor
	{
		/// <summary>
		/// Name of the artifact
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Type of the artifact
		/// </summary>
		public string? Type { get; set; }

		/// <summary>
		/// Artifact description 
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Base URL for downloading from
		/// </summary>
		public Uri BaseUrl { get; set; }

		/// <summary>
		/// Name of the ref to download
		/// </summary>
		public RefName RefName { get; set; }

		/// <summary>
		/// Optional URL to the job that produced this artifact
		/// </summary>
		public Uri? JobUrl { get; set; }

		/// <summary>
		/// Keys associated with the artifact
		/// </summary>
		public List<string>? Keys { get; set; }

		/// <summary>
		/// Metadata associated with the artifact
		/// </summary>
		public List<string>? Metadata { get; set; }

		/// <summary>
		/// Filter for the files selected for download
		/// </summary>
		public List<string>? Filter { get; set; }

		/// <summary>
		/// Information about which backend system to use. Defaults to Horde.
		/// </summary>
		public ArtifactBackend Backend { get; set; } = new ArtifactBackend();

		/// <summary>
		/// Default constructor
		/// </summary>
		public ArtifactDescriptor()
		{
			BaseUrl = new Uri("http://horde");
			RefName = new RefName("default");
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ArtifactDescriptor(Uri baseUrl, RefName refName, IReadOnlyCollection<string>? filter = null)
		{
			BaseUrl = baseUrl;
			RefName = refName;
			if (filter != null && filter.Count > 0)
			{
				Filter = [.. filter];
			}
		}

		/// <summary>
		/// Deserialize a descriptor from utf8 bytes
		/// </summary>
		/// <param name="data">Data to deserialize from</param>
		/// <returns>New descriptor instance</returns>
		public static ArtifactDescriptor Deserialize(ReadOnlySpan<byte> data)
		{
			return JsonSerializer.Deserialize<ArtifactDescriptor>(data, HordeHttpClient.JsonSerializerOptions)
				?? throw new InvalidOperationException("Cannot deserialize descriptor");
		}

		/// <summary>
		/// Writes this descriptor to storage
		/// </summary>
		/// <param name="file">File to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<ArtifactDescriptor> ReadAsync(FileReference file, CancellationToken cancellationToken)
		{
			byte[] data = await FileReference.ReadAllBytesAsync(file, cancellationToken);
			return Deserialize(data);
		}

		/// <summary>
		/// Serializes a descriptor
		/// </summary>
		/// <returns>Data for the serialized descriptor</returns>
		public byte[] Serialize()
		{
			JsonSerializerOptions serializerOptions = new(HordeHttpClient.JsonSerializerOptions);
			serializerOptions.WriteIndented = true;

			return JsonSerializer.SerializeToUtf8Bytes<ArtifactDescriptor>(this, serializerOptions);
		}

		/// <summary>
		/// Writes this descriptor to storage
		/// </summary>
		/// <param name="file">File to write to</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task WriteAsync(FileReference file, CancellationToken cancellationToken)
		{
			byte[] data = Serialize();
			await FileReference.WriteAllBytesAsync(file, data, cancellationToken);
		}
	}
}
