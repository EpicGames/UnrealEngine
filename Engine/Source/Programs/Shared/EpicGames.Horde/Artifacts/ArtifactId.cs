// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Unique id for an artifact
	/// </summary>
	/// <param name="Id">Identifier for the artifact</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<ArtifactId, ArtifactIdConverter>))]
	[BinaryIdConverter(typeof(ArtifactIdConverter))]
	public readonly record struct ArtifactId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static ArtifactId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override readonly string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter class to and from ObjectId values
	/// </summary>
	class ArtifactIdConverter : BinaryIdConverter<ArtifactId>
	{
		/// <inheritdoc/>
		public override ArtifactId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(ArtifactId value) => value.Id;
	}
}
