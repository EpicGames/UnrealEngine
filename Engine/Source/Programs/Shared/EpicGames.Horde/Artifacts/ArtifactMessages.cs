// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;

#pragma warning disable CA2227

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Creates a new artifact
	/// </summary>
	/// <param name="Name">Name of the artifact</param>
	/// <param name="Type">Additional search keys tagged on the artifact</param>
	/// <param name="Description">Description for the artifact</param>
	/// <param name="StreamId">Stream to create the artifact for</param>
	/// <param name="Keys">Keys used to identify the artifact</param>
	/// <param name="Metadata">Metadata for the artifact</param>
	public record CreateArtifactRequest(ArtifactName Name, ArtifactType Type, string? Description, StreamId? StreamId, List<string> Keys, List<string> Metadata)
	{
		/// <summary>
		/// Legacy Perforce changelist number.
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int Change
		{
			get => _change ?? _commitId?.TryGetPerforceChange() ?? -1;
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// Commit for the new artifact
		/// </summary>
		public CommitId CommitId
		{
			get => _commitId ?? CommitId.FromPerforceChange(_change) ?? CommitId.Empty;
			set => _commitId = value;
		}
		CommitId? _commitId;
	}

	/// <summary>
	/// Information about a created artifact
	/// </summary>
	/// <param name="ArtifactId">Identifier for the new artifact</param>
	/// <param name="CommitId">Resolved commit id for the artifact</param>
	/// <param name="NamespaceId">Namespace that should be written to with artifact data</param>
	/// <param name="RefName">Ref to write to</param>
	/// <param name="PrevRefName">Ref for the artifact at the changelist prior to this one. Can be used to deduplicate against.</param>
	/// <param name="Token">Token which can be used to upload blobs for the artifact, and read blobs from the previous artifact</param>
	public record CreateArtifactResponse(ArtifactId ArtifactId, CommitIdWithOrder CommitId, NamespaceId NamespaceId, RefName RefName, RefName? PrevRefName, string Token);

	/// <summary>
	/// Type of data to download for an artifact
	/// </summary>
	public enum DownloadArtifactFormat
	{
		/// <summary>
		/// Download as a zip file
		/// </summary>
		Zip,

		/// <summary>
		/// Download as a UGS link
		/// </summary>
		Ugs
	}

	/// <summary>
	/// Describes an artifact
	/// </summary>
	public class GetArtifactResponse
	{
		/// <inheritdoc cref="IArtifact.Id"/>
		public ArtifactId Id { get; set; }

		/// <inheritdoc cref="IArtifact.Name"/>
		public ArtifactName Name { get; set; }

		/// <inheritdoc cref="IArtifact.Type"/>
		public ArtifactType Type { get; set; }

		/// <inheritdoc cref="IArtifact.Description"/>
		public string? Description { get; set; }

		/// <inheritdoc cref="IArtifact.StreamId"/>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// Change number
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int Change
		{
			get => _change ?? _commitId?.TryGetPerforceChange() ?? -1;
			set => _change = value;
		}
		int? _change;

		/// <inheritdoc cref="IArtifact.CommitId"/>
		public CommitIdWithOrder CommitId
		{
			get => _commitId ?? CommitIdWithOrder.FromPerforceChange(_change) ?? CommitIdWithOrder.Empty;
			set => _commitId = value;
		}
		CommitIdWithOrder? _commitId;

		/// <inheritdoc cref="IArtifact.Keys"/>
		public IReadOnlyList<string> Keys { get; set; }

		/// <inheritdoc cref="IArtifact.Metadata"/>
		public IReadOnlyList<string> Metadata { get; set; }

		/// <inheritdoc cref="IArtifact.NamespaceId"/>
		public NamespaceId NamespaceId { get; set; }

		/// <inheritdoc cref="IArtifact.RefName"/>
		public RefName RefName { get; set; }

		/// <inheritdoc cref="IArtifact.CreatedAtUtc"/>
		public DateTime CreatedAtUtc { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		[JsonConstructor]
		public GetArtifactResponse()
		{
			Keys = [];
			Metadata = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="artifact"></param>
		public GetArtifactResponse(IArtifact artifact)
		{
			Id = artifact.Id;
			Name = artifact.Name;
			Type = artifact.Type;
			Description = artifact.Description;
			StreamId = artifact.StreamId;
			CommitId = artifact.CommitId;
			Keys = artifact.Keys;
			Metadata = artifact.Metadata;
			NamespaceId = artifact.NamespaceId;
			RefName = artifact.RefName;
			CreatedAtUtc = artifact.CreatedAtUtc;
		}
	}

	/// <summary>
	/// Result of an artifact search
	/// </summary>
	public class FindArtifactsResponse
	{
		/// <summary>
		/// List of artifacts matching the search criteria
		/// </summary>
		public List<GetArtifactResponse> Artifacts { get; set; } = [];
	}

	/// <summary>
	/// Describes a file within an artifact
	/// </summary>
	public class GetArtifactFileEntryResponse
	{
		/// <summary>
		/// Name of this file
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Length of this entry
		/// </summary>
		public long Length { get; }

		/// <summary>
		/// Hash of the target node
		/// </summary>
		public IoHash Hash { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetArtifactFileEntryResponse(string name, long length, IoHash hash)
		{
			Name = name;
			Length = length;
			Hash = hash;
		}
	}

	/// <summary>
	/// Describes a file within an artifact
	/// </summary>
	public class GetArtifactDirectoryEntryResponse : GetArtifactDirectoryResponse
	{
		/// <summary>
		/// Name of this file
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Length of this entry
		/// </summary>
		public long Length { get; }

		/// <summary>
		/// Hash of the target node
		/// </summary>
		public IoHash Hash { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetArtifactDirectoryEntryResponse(string name, long length, IoHash hash)
		{
			Name = name;
			Length = length;
			Hash = hash;
		}
	}

	/// <summary>
	/// Describes a directory within an artifact
	/// </summary>
	public class GetArtifactDirectoryResponse
	{
		/// <summary>
		/// Names of sub-directories
		/// </summary>
		public List<GetArtifactDirectoryEntryResponse>? Directories { get; set; }

		/// <summary>
		/// Files within the directory
		/// </summary>
		public List<GetArtifactFileEntryResponse>? Files { get; set; }
	}

	/// <summary>
	/// Request to create a zip file with artifact data
	/// </summary>
	public class CreateZipRequest
	{
		/// <summary>
		/// Filter lines for the zip. Uses standard <see cref="FileFilter"/> syntax.
		/// </summary>
		public List<string> Filter { get; set; } = [];
	}
}
