// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Name of an artifact
	/// </summary>
	/// <param name="Id">The artifact type</param>
	[JsonSchemaString]
	[StringIdConverter(typeof(ArtifactNameConverter))]
	[TypeConverter(typeof(StringIdTypeConverter<ArtifactName, ArtifactNameConverter>))]
	public readonly record struct ArtifactName(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for the artifact type</param>
		public ArtifactName(string id) : this(new StringId(id)) { }

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class ArtifactNameConverter : StringIdConverter<ArtifactName>
	{
		/// <inheritdoc/>
		public override ArtifactName FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(ArtifactName value) => value.Id;
	}
}
