// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Type of an artifact
	/// </summary>
	/// <param name="Id">The artifact type</param>
	[JsonSchemaString]
	[StringIdConverter(typeof(ArtifactTypeConverter))]
	[TypeConverter(typeof(StringIdTypeConverter<ArtifactType, ArtifactTypeConverter>))]
	public readonly record struct ArtifactType(StringId Id)
	{
		/// <summary>
		/// Default artifact type 
		/// </summary>
		public static ArtifactType Unknown { get; } = default;

		/// <summary>
		/// Output from a build step
		/// </summary>
		public static ArtifactType StepOutput { get; } = new ArtifactType("step-output");

		/// <summary>
		/// Captured state from the machine executing a build step
		/// </summary>
		public static ArtifactType StepSaved { get; } = new ArtifactType("step-saved");

		/// <summary>
		/// Traces from the machine executing a build step
		/// </summary>
		public static ArtifactType StepTrace { get; } = new ArtifactType("step-trace");

		/// <summary>
		/// Test data generated by a machine executing a build step
		/// </summary>
		public static ArtifactType StepTestData { get; } = new ArtifactType("step-testdata");

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for the artifact type</param>
		public ArtifactType(string id) : this(new StringId(id)) { }

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class ArtifactTypeConverter : StringIdConverter<ArtifactType>
	{
		/// <inheritdoc/>
		public override ArtifactType FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(ArtifactType value) => value.Id;
	}

	/// <summary>
	/// Exception thrown to indicate that an artifact type does not exist
	/// </summary>
	public sealed class ArtifactTypeNotFoundException : Exception
	{
		/// <summary>
		/// The stream containing the artifact
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// The missing artifact type
		/// </summary>
		public ArtifactType ArtifactType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ArtifactTypeNotFoundException(StreamId streamId, ArtifactType artifactType)
			: base($"Artifact type {artifactType} not found in stream {streamId}")
		{
			StreamId = streamId;
			ArtifactType = artifactType;
		}
	}
}
