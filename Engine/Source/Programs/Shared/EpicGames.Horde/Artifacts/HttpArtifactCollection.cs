// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Artifacts
{
	class HttpArtifactCollection : IArtifactCollection
	{
		[DebuggerDisplay("{Id}")]
		class Artifact : IArtifact
		{
			readonly HttpArtifactCollection _collection;

			public Artifact(HttpArtifactCollection collection, GetArtifactResponse response)
				: this(collection, response.Id, response.Name, response.Type, response.Description, response.StreamId, response.CommitId, response.Keys, response.Metadata, response.NamespaceId, response.RefName, response.CreatedAtUtc)
			{ }

			public Artifact(HttpArtifactCollection collection, ArtifactId id, ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitIdWithOrder commitId, IReadOnlyList<string> keys, IReadOnlyList<string> metadata, NamespaceId namespaceId, RefName refName, DateTime createdAtUtc)
			{
				_collection = collection;

				Id = id;
				Name = name;
				Type = type;
				Description = description;
				StreamId = streamId;
				CommitId = commitId;
				Keys = keys;
				Metadata = metadata;
				NamespaceId = namespaceId;
				RefName = refName;
				CreatedAtUtc = createdAtUtc;
			}

			public ArtifactId Id { get; }
			public ArtifactName Name { get; }
			public ArtifactType Type { get; }
			public string? Description { get; }
			public StreamId StreamId { get; }
			public CommitIdWithOrder CommitId { get; }
			public IReadOnlyList<string> Keys { get; }
			public IReadOnlyList<string> Metadata { get; }
			public NamespaceId NamespaceId { get; }
			public RefName RefName { get; }
			public DateTime CreatedAtUtc { get; }

			public IBlobRef<DirectoryNode> Content
				=> _collection.Open(this);

			public Task DeleteAsync(CancellationToken cancellationToken)
				=> _collection.DeleteAsync(Id, cancellationToken);
		}

		class ArtifactBuilder : IArtifactBuilder
		{
			record class AliasInfo(string Name, IBlobRef BlobRef, int Rank, ReadOnlyMemory<byte> Data);

			readonly Artifact _artifact;
			readonly CreateArtifactResponse _response;
			readonly IStorageNamespace _namespace;
			readonly List<AliasInfo> _aliases = [];

			public ArtifactId Id => _artifact.Id;
			public ArtifactName Name => _artifact.Name;
			public ArtifactType Type => _artifact.Type;
			public string? Description => _artifact.Description;
			public StreamId StreamId => _artifact.StreamId;
			public CommitIdWithOrder CommitId => _artifact.CommitId;
			public IReadOnlyList<string> Keys => _artifact.Keys;
			public IReadOnlyList<string> Metadata => _artifact.Metadata;
			public NamespaceId NamespaceId => _artifact.NamespaceId;
			public RefName RefName => _artifact.RefName;

			public ArtifactBuilder(HttpArtifactCollection collection, Artifact artifact, CreateArtifactResponse response)
			{
				_artifact = artifact;
				_response = response;
				_namespace = collection._hordeClient.GetStorageNamespace(response.NamespaceId, response.Token);
			}

			/// <inheritdoc/>
			public Task AddAliasAsync(string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
			{
				_aliases.Add(new AliasInfo(name, handle, rank, data.ToArray()));
				return Task.CompletedTask;
			}

			/// <inheritdoc/>
			public async Task<IArtifact> CompleteAsync(IHashedBlobRef blobRef, CancellationToken cancellationToken = default)
			{
				await using (IStorageWriter writer = _namespace.CreateWriter(cancellationToken))
				{
					foreach (AliasInfo aliasInfo in _aliases)
					{
						await writer.AddAliasAsync(aliasInfo.Name, aliasInfo.BlobRef, aliasInfo.Rank, aliasInfo.Data, cancellationToken);
					}
					await writer.AddRefAsync(_artifact.RefName, blobRef, cancellationToken: cancellationToken);
				}
				return _artifact;
			}

			/// <inheritdoc/>
			public IBlobWriter CreateBlobWriter()
				=> _namespace.CreateBlobWriter(_response.RefName);
		}

		readonly IHordeClient _hordeClient;

		public HttpArtifactCollection(IHordeClient hordeClient)
			=> _hordeClient = hordeClient;

		IBlobRef<DirectoryNode> Open(Artifact artifact)
		{
			IStorageNamespace store = _hordeClient.GetStorageNamespace(artifact.Id);
			return store.CreateBlobRef<DirectoryNode>(new RefName("default"));
		}

		/// <inheritdoc/>
		public async Task<IArtifactBuilder> CreateAsync(ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitId commitId, IEnumerable<string> keys, IEnumerable<string> metadata, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			CreateArtifactResponse response = await hordeHttpClient.CreateArtifactAsync(name, type, description, streamId, commitId, keys, metadata, cancellationToken);
			Artifact artifact = new(this, response.ArtifactId, name, type, description, streamId, CommitIdWithOrder.FromPerforceChange(commitId.GetPerforceChange()), [.. keys], [.. metadata], response.NamespaceId, response.RefName, DateTime.UtcNow);
			return new ArtifactBuilder(this, artifact, response);
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<IArtifact> FindAsync(ArtifactId[]? ids = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();

			List<GetArtifactResponse> responses = await hordeHttpClient.FindArtifactsAsync(ids, streamId, minCommitId, maxCommitId, name, type, keys, maxResults, cancellationToken);
			foreach (GetArtifactResponse response in responses)
			{
				yield return new Artifact(this, response);
			}
		}

		async Task DeleteAsync(ArtifactId artifactId, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			await hordeHttpClient.DeleteArtifactAsync(artifactId, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IArtifact?> GetAsync(ArtifactId artifactId, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetArtifactResponse? response = await hordeHttpClient.GetArtifactAsync(artifactId, cancellationToken);
			return new Artifact(this, response);
		}
	}
}
