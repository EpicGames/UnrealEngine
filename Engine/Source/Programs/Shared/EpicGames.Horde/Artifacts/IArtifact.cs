// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Information about an artifact
	/// </summary>
	public interface IArtifact
	{
		/// <summary>
		/// Identifier for the Artifact. Randomly generated.
		/// </summary>
		ArtifactId Id { get; }

		/// <summary>
		/// Name of the artifact
		/// </summary>
		ArtifactName Name { get; }

		/// <summary>
		/// Type of artifact
		/// </summary>
		ArtifactType Type { get; }

		/// <summary>
		/// Description for the artifact
		/// </summary>
		string? Description { get; }

		/// <summary>
		/// Identifier for the stream that produced the artifact
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// Change that the artifact corresponds to
		/// </summary>
		CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// Keys used to collate artifacts
		/// </summary>
		IReadOnlyList<string> Keys { get; }

		/// <summary>
		/// Metadata for the artifact
		/// </summary>
		IReadOnlyList<string> Metadata { get; }

		/// <summary>
		/// Storage namespace containing the data
		/// </summary>
		NamespaceId NamespaceId { get; }

		/// <summary>
		/// Name of the ref containing the root data object
		/// </summary>
		RefName RefName { get; }

		/// <summary>
		/// Time at which the artifact was created
		/// </summary>
		DateTime CreatedAtUtc { get; }

		/// <summary>
		/// Handle to the artifact data
		/// </summary>
		IBlobRef<DirectoryNode> Content { get; }

		/// <summary>
		/// Deletes this artifact
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task DeleteAsync(CancellationToken cancellationToken);
	}

	/// <summary>
	/// Used to write data into an artifact
	/// </summary>
	public interface IArtifactBuilder
	{
		/// <inheritdoc cref="IArtifact.Id"/>
		ArtifactId Id { get; }

		/// <inheritdoc cref="IArtifact.Name"/>
		ArtifactName Name { get; }

		/// <inheritdoc cref="IArtifact.Type"/>
		ArtifactType Type { get; }

		/// <inheritdoc cref="IArtifact.Description"/>
		string? Description { get; }

		/// <inheritdoc cref="IArtifact.StreamId"/>
		StreamId StreamId { get; }

		/// <inheritdoc cref="IArtifact.CommitId"/>
		CommitIdWithOrder CommitId { get; }

		/// <inheritdoc cref="IArtifact.Keys"/>
		IReadOnlyList<string> Keys { get; }

		/// <inheritdoc cref="IArtifact.Metadata"/>
		IReadOnlyList<string> Metadata { get; }

		/// <inheritdoc cref="IArtifact.NamespaceId"/>
		NamespaceId NamespaceId { get; }

		/// <inheritdoc cref="IArtifact.RefName"/>
		RefName RefName { get; }

		/// <summary>
		/// Creates a writer for new artifact blobs
		/// </summary>
		IBlobWriter CreateBlobWriter();

		/// <summary>
		/// Adds an alias to a given blob
		/// </summary>
		/// <param name="name">Alias for the blob</param>
		/// <param name="handle">Locator for the blob</param>
		/// <param name="rank">Rank for this alias. In situations where an alias has multiple mappings, the alias with the highest rank will be returned by default.</param>
		/// <param name="data">Additional data to be stored inline with the alias</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task AddAliasAsync(string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finish writing the artifact data
		/// </summary>
		/// <param name="blobRef">Root blob for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The complete artifact</returns>
		Task<IArtifact> CompleteAsync(IHashedBlobRef blobRef, CancellationToken cancellationToken = default);
	}
}
