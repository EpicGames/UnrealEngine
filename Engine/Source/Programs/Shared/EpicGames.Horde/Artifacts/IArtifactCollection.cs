// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Interface for a collection of artifacts
	/// </summary>
	public interface IArtifactCollection
	{
		/// <summary>
		/// Creates a new artifact
		/// </summary>
		/// <param name="name">Name of the artifact</param>
		/// <param name="type">Type identifier for the artifact</param>
		/// <param name="description">Description for the artifact</param>
		/// <param name="streamId">Stream that the artifact was built from</param>
		/// <param name="commitId">Commit that the artifact was built from</param>
		/// <param name="keys">Keys for the artifact</param>
		/// <param name="metadata">Metadata for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new log file document</returns>
		Task<IArtifactBuilder> CreateAsync(ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitId commitId, IEnumerable<string> keys, IEnumerable<string> metadata, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds artifacts with the given keys.
		/// </summary>
		/// <param name="ids">Identifiers to return</param>
		/// <param name="streamId">Stream to find artifacts for</param>
		/// <param name="minCommitId">Minimum commit for the artifacts (inclusive)</param>
		/// <param name="maxCommitId">Maximum commit for the artifacts (inclusive)</param>
		/// <param name="name">Name of the artifact to search for</param>
		/// <param name="type">The artifact type</param>
		/// <param name="keys">Set of keys, all of which must all be present on any returned artifacts</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sequence of artifacts. Ordered by descending CL order, then by descending order in which they were created.</returns>
		IAsyncEnumerable<IArtifact> FindAsync(ArtifactId[]? ids = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets an artifact by ID
		/// </summary>
		/// <param name="artifactId">Unique id of the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The artifact document</returns>
		Task<IArtifact?> GetAsync(ArtifactId artifactId, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for artifacts
	/// </summary>
	public static class ArtifactCollectionExtensions
	{
		/// <summary>
		/// Finds artifacts with the given keys.
		/// </summary>
		/// <param name="artifactCollection">Collection to operate on</param>
		/// <param name="streamId">Stream to find artifacts for</param>
		/// <param name="minCommitId">Minimum commit for the artifacts (inclusive)</param>
		/// <param name="maxCommitId">Maximum commit for the artifacts (inclusive)</param>
		/// <param name="name">Name of the artifact to search for</param>
		/// <param name="type">The artifact type</param>
		/// <param name="keys">Set of keys, all of which must all be present on any returned artifacts</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sequence of artifacts. Ordered by descending CL order, then by descending order in which they were created.</returns>
		public static IAsyncEnumerable<IArtifact> FindAsync(this IArtifactCollection artifactCollection, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, CancellationToken cancellationToken = default)
			=> artifactCollection.FindAsync(null, streamId, minCommitId, maxCommitId, name, type, keys, maxResults, cancellationToken);
	}
}
