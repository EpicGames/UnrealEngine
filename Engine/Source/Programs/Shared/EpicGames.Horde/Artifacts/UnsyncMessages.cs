// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;

#pragma warning disable CA2227

namespace EpicGames.Horde.Artifacts
{
	/// <summary>
	/// Request to return a set of blobs for Unsync
	/// </summary>
	public class GetUnsyncDataRequest
	{
		/// <summary>
		/// The strong hash algorithm
		/// </summary>
		[JsonPropertyName("hash_strong")]
		public string? HashStrong { get; set; }

		/// <summary>
		/// Files to retrieve
		/// </summary>
		[JsonPropertyName("blocks")]
		public List<string> Blocks { get; set; } = [];

		/// <summary>
		/// Files to retrieve
		/// </summary>
		[JsonPropertyName("files")]
		public List<GetUnsyncFileRequest> Files { get; set; } = [];
	}

	/// <summary>
	/// Requests a set of blobs from a particular file
	/// </summary>
	public class GetUnsyncFileRequest
	{
		/// <summary>
		/// Path to the file
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Blobs to return
		/// </summary>
		public List<GetUnsyncBlockRequest> Blocks { get; set; } = [];
	}

	/// <summary>
	/// Requests a block of data
	/// </summary>
	public class GetUnsyncBlockRequest
	{
		/// <summary>
		/// Hash of the block
		/// </summary>
		[JsonPropertyName("hash_strong")]
		public string? Hash { get; set; }
	}
}
