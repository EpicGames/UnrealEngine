// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel.DataAnnotations;
using System.Net.Http;
using Microsoft.Extensions.DependencyInjection;

namespace EpicGames.Horde.Auth
{
	/// <summary>
	/// Base class for configuring HTTP service clients
	/// </summary>
	public class HttpServiceClientOptions : IOAuthOptions, ITokenAuthOptions
	{
		/// <summary>
		/// Base address for http requests
		/// </summary>
		[Required]
		public Uri Url { get; set; } = null!;

		#region OAuth2

		/// <inheritdoc/>
		public Uri? AuthUrl { get; set; }

		/// <inheritdoc/>
		public string GrantType { get; set; } = String.Empty;

		/// <inheritdoc/>
		public string ClientId { get; set; } = String.Empty;

		/// <inheritdoc/>
		public string ClientSecret { get; set; } = String.Empty;

		/// <inheritdoc/>
		public string Scope { get; set; } = String.Empty;

		#endregion

		#region Bearer token

		/// <inheritdoc/>
		public string Token { get; set; } = String.Empty;

		#endregion
	}

	internal static class AuthExtensions
	{
		public static void AddHttpClientWithAuth<TClient, TImplementation>(this IServiceCollection services, Func<IServiceProvider, HttpServiceClientOptions> getOptions)
			where TClient : class
			where TImplementation : class, TClient
		{
			services.AddScoped<OAuthHandlerFactory>();
			services.AddHttpClient<OAuthHandlerFactory>();
			services.AddScoped<OAuthHandler<TImplementation>>(serviceProvider => serviceProvider.GetRequiredService<OAuthHandlerFactory>().Create<TImplementation>(getOptions(serviceProvider)));

			services.AddHttpClient<TClient, TImplementation>((serviceProvider, client) =>
				{
					HttpServiceClientOptions options = getOptions(serviceProvider);
					client.BaseAddress = options.Url;
				})
				.ConfigurePrimaryHttpMessageHandler(serviceProvider =>
				{
					HttpServiceClientOptions options = getOptions(serviceProvider);
					return CreateMessageHandler<TImplementation>(serviceProvider, options);
				});
		}

		static HttpMessageHandler CreateMessageHandler<TImplementation>(IServiceProvider serviceProvider, HttpServiceClientOptions options)
		{
			if (options.AuthUrl != null)
			{
				return serviceProvider.GetRequiredService<OAuthHandler<TImplementation>>();
			}
			else if (!String.IsNullOrEmpty(options.Token))
			{
				return new TokenHandler<TImplementation>(options);
			}
			else
			{
				return new HttpClientHandler();
			}
		}
	}
}
