// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Net.Http;
using System.Net.Http.Json;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Server;
using EpicGames.OIDC;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace EpicGames.Horde.Auth;

internal sealed class HordeAuthConfigProvider(
	HttpMessageHandler httpMessageHandler,
	Uri serverUri,
	ILogger<HordeAuthConfigProvider> logger
) : IOidcAuthStateConfig, IDisposable
{
	public async Task<string> GetOidcProviderAsync(CancellationToken cancellationToken)
	{
		GetAuthConfigResponse authConfig = await GetAuthConfigResponseAsync(cancellationToken);
		return GetOidcProvider(authConfig);
	}

	public static string GetOidcProvider(GetAuthConfigResponse authConfig) => authConfig.ProfileName ?? "Horde";

	public async Task<GetAuthConfigResponse> GetAuthConfigResponseAsync(CancellationToken cancellationToken)
	{
		const string CacheKey = "auth";
		if (_memoryCache.TryGetValue(CacheKey, out GetAuthConfigResponse? auth))
		{
			return auth!;
		}

		GetAuthConfigResponse? authConfig;
		using (HttpClient httpClient = new(_httpMessageHandler, false))
		{
			httpClient.BaseAddress = _serverUri;
			_logger.LogDebug("Retrieving auth configuration for {Server}", _serverUri);

			JsonSerializerOptions jsonOptions = new();
			HordeHttpClient.ConfigureJsonSerializer(jsonOptions);

			authConfig = await httpClient.GetFromJsonAsync<GetAuthConfigResponse>("api/v1/server/auth", jsonOptions, cancellationToken);
			if (authConfig == null)
			{
				throw new Exception($"Invalid response from server");
			}
		}

		TimeSpan expiresAfter = TimeSpan.FromMinutes(1);
		_memoryCache.Set(CacheKey, authConfig, expiresAfter);
		return authConfig;
	}

	public void Dispose()
	{
		_memoryCache.Dispose();
	}

	[SuppressMessage("Usage", "CA2213:Disposable fields should be disposed", Justification = "Lifetime is managed by caller")]
	private readonly HttpMessageHandler _httpMessageHandler = httpMessageHandler;
	private readonly Uri _serverUri = serverUri;
	private readonly ILogger<HordeAuthConfigProvider> _logger = logger;
	private readonly MemoryCache _memoryCache = new(Options.Create(new MemoryCacheOptions()));
}
