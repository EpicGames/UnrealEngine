// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Server;
using EpicGames.OIDC;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Auth;

internal sealed class HordeOidcTokenManager(
	HordeAuthConfigProvider hordeAuthConfigProvider,
	ITokenStore tokenStore,
	ILogger<HordeOidcTokenManager>? logger = null,
	ILoggerFactory? loggerFactory = null
) : IOidcTokenManager
{
	public async Task<IOidcTokenInfo> GetAccessToken(string providerIdentifier, CancellationToken cancellationToken)
	{
		GetAuthConfigResponse authConfig = await _hordeAuthConfigProvider.GetAuthConfigResponseAsync(cancellationToken);

		if (authConfig.Method == AuthMethod.Anonymous)
		{
			return new AnonymousOidcTokenInfo();
		}

		OidcTokenManager oidcTokenManager = GetOrCreateOidcTokenManager(providerIdentifier, authConfig);
		return await oidcTokenManager.GetAccessToken(providerIdentifier, cancellationToken);
	}

	public async Task<OidcStatus> GetStatusForProvider(string providerIdentifier, CancellationToken cancellationToken)
	{
		GetAuthConfigResponse authConfig = await _hordeAuthConfigProvider.GetAuthConfigResponseAsync(cancellationToken);

		if (authConfig.Method == AuthMethod.Anonymous)
		{
			return OidcStatus.Connected;
		}

		OidcTokenManager oidcTokenManager = GetOrCreateOidcTokenManager(providerIdentifier, authConfig);
		return await oidcTokenManager.GetStatusForProvider(providerIdentifier, cancellationToken);
	}

	public async Task<IOidcTokenInfo> LoginAsync(string providerIdentifier, CancellationToken cancellationToken)
	{
		GetAuthConfigResponse authConfig = await _hordeAuthConfigProvider.GetAuthConfigResponseAsync(cancellationToken);

		if (authConfig.Method == AuthMethod.Anonymous)
		{
			return new AnonymousOidcTokenInfo();
		}

		_logger?.LogDebug("Interactive login requested for OIDC provider '{Provider}'. A browser window will open.", providerIdentifier);
		OidcTokenManager oidcTokenManager = GetOrCreateOidcTokenManager(providerIdentifier, authConfig);
		return await oidcTokenManager.LoginAsync(providerIdentifier, cancellationToken);
	}

	public async Task<IOidcTokenInfo?> TryGetAccessToken(string providerIdentifier, CancellationToken cancellationToken)
	{
		_logger?.LogDebug("Requesting access token for provider '{Provider}'.", providerIdentifier);
		GetAuthConfigResponse authConfig = await _hordeAuthConfigProvider.GetAuthConfigResponseAsync(cancellationToken);

		if (authConfig.Method == AuthMethod.Anonymous)
		{
			return new AnonymousOidcTokenInfo();
		}

		OidcTokenManager oidcTokenManager = GetOrCreateOidcTokenManager(providerIdentifier, authConfig);
		return await oidcTokenManager.TryGetAccessToken(providerIdentifier, cancellationToken);
	}

	private readonly HordeAuthConfigProvider _hordeAuthConfigProvider = hordeAuthConfigProvider;
	private readonly ITokenStore _tokenStore = tokenStore;
	private readonly ILogger<HordeOidcTokenManager>? _logger = logger;
	private readonly ILoggerFactory? _loggerFactory = loggerFactory;
	private readonly ConcurrentDictionary<string, OidcTokenManager> _cachedManagers = new(StringComparer.OrdinalIgnoreCase);

	private OidcTokenManager GetOrCreateOidcTokenManager(string providerIdentifier, GetAuthConfigResponse authConfig)
	{
		return _cachedManagers.GetOrAdd(providerIdentifier, _ => CreateOidcTokenManager(providerIdentifier, authConfig));
	}

	private OidcTokenManager CreateOidcTokenManager(string providerIdentifier, GetAuthConfigResponse authConfig)
	{
		string? localRedirectUrl = authConfig.LocalRedirectUrls?.FirstOrDefault();
		if (String.IsNullOrEmpty(authConfig.ServerUrl) || String.IsNullOrEmpty(localRedirectUrl))
		{
			throw new Exception("No auth server configuration found");
		}
		string oidcProvider = HordeAuthConfigProvider.GetOidcProvider(authConfig);

		if (!String.Equals(oidcProvider, providerIdentifier, StringComparison.Ordinal))
		{
			throw new Exception("Provider identifier mismatch");
		}

		_logger?.LogDebug("Creating OidcTokenManager for provider '{Provider}'.", oidcProvider);

		Dictionary<string, string?> values = [];
		values[$"Providers:{oidcProvider}:DisplayName"] = oidcProvider;
		values[$"Providers:{oidcProvider}:ServerUri"] = authConfig.ServerUrl;
		values[$"Providers:{oidcProvider}:ClientId"] = authConfig.ClientId;
		values[$"Providers:{oidcProvider}:RedirectUri"] = localRedirectUrl;
		if (authConfig.Scopes is { Length: > 0 })
		{
			values[$"Providers:{oidcProvider}:Scopes"] = String.Join(" ", authConfig.Scopes);
		}

		ConfigurationBuilder builder = new();
		builder.AddInMemoryCollection(values);
		IConfiguration configuration = builder.Build();

		return OidcTokenManager.CreateTokenManager(configuration, _tokenStore, [oidcProvider], _loggerFactory);
	}
}
