// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.OIDC;
using Microsoft.Extensions.Options;

namespace EpicGames.Horde.Auth;

internal sealed class HordeStaticAccessTokenProvider(IOptionsSnapshot<HordeOptions> options) : IStaticAccessTokenProvider
{
	public string? AccessToken
	{
		get
		{
			// If an explicit access token is specified, just use that
			if (_options.Value.AccessToken != null)
			{
				return _options.Value.AccessToken;
			}

			// Check environment variables for an access token matching the current server
			string? hordeUrlEnvVar = Environment.GetEnvironmentVariable(HordeHttpClient.HordeUrlEnvVarName);
			if (!String.IsNullOrEmpty(hordeUrlEnvVar))
			{
				Uri hordeUrl = new(hordeUrlEnvVar);
				if (_options.Value.ServerUrl == null || String.Equals(_options.Value.ServerUrl.Host, hordeUrl.Host, StringComparison.OrdinalIgnoreCase))
				{
					string? hordeToken = Environment.GetEnvironmentVariable(HordeHttpClient.HordeTokenEnvVarName);
					if (!String.IsNullOrEmpty(hordeToken))
					{
						return hordeToken;
					}
				}
			}

			// Otherwise we need to find the access token dynamically
			return null;
		}
	}

	private readonly IOptionsSnapshot<HordeOptions> _options = options;
}
