// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Net;
using System.Net.Http;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Auth
{
	/// <summary>
	/// Exception thrown due to failed authorization
	/// </summary>
	public class AuthenticationException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public AuthenticationException(string message, Exception? innerException)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Options for authenticating particular requests
	/// </summary>
	public interface IOAuthOptions
	{
		/// <summary>
		/// Url of the auth server
		/// </summary>
		Uri? AuthUrl { get; }

		/// <summary>
		/// Type of grant
		/// </summary>
		string GrantType { get; }

		/// <summary>
		/// Client id
		/// </summary>
		string ClientId { get; }

		/// <summary>
		/// Client secret
		/// </summary>
		string ClientSecret { get; }

		/// <summary>
		/// Scope of the token
		/// </summary>
		string Scope { get; }
	}

	/// <summary>
	/// Http message handler which adds an OAuth authorization header using a cached/periodically refreshed bearer token
	/// </summary>
	public class OAuthHandler<T> : HttpClientHandler
	{
		[SuppressMessage("Style", "IDE1006:Naming Styles")]
		class ClientCredentialsResponse
		{
			public string? access_token { get; set; }
			public string? token_type { get; set; }
			public int? expires_in { get; set; }
			public string? scope { get; set; }
		}

		readonly HttpClient _client;
		readonly IOAuthOptions _options;
		string _cachedAccessToken = String.Empty;
		DateTime _expiresAt = DateTime.MinValue;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="client"></param>
		/// <param name="options"></param>
		public OAuthHandler(HttpClient client, IOAuthOptions options)
		{
			_client = client;
			_options = options;
		}

		/// <inheritdoc/>
		protected override async Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
		{
			if (DateTime.UtcNow > _expiresAt)
			{
				await UpdateAccessTokenAsync(cancellationToken);
			}

			request.Headers.Add("Authorization", $"Bearer {_cachedAccessToken}");
			return await base.SendAsync(request, cancellationToken);
		}

		/// <summary>
		/// Updates the current access token
		/// </summary>
		/// <returns></returns>
		async Task UpdateAccessTokenAsync(CancellationToken cancellationToken)
		{
			KeyValuePair<string, string>[] content =
			[
				new KeyValuePair<string, string>("grant_type", _options.GrantType),
				new KeyValuePair<string, string>("client_id", _options.ClientId),
				new KeyValuePair<string, string>("client_secret", _options.ClientSecret),
				new KeyValuePair<string, string>("scope", _options.Scope)
			];

			try
			{
				using HttpRequestMessage message = new(HttpMethod.Post, _options.AuthUrl);
				message.Content = new FormUrlEncodedContent(content);

				HttpResponseMessage response = await _client.SendAsync(message, cancellationToken);
				if (!response.IsSuccessStatusCode)
				{
					throw new AuthenticationException($"Authentication failed. Response: {response.Content}", null);
				}

				byte[] responseData = await response.Content.ReadAsByteArrayAsync(cancellationToken);
				ClientCredentialsResponse result = JsonSerializer.Deserialize<ClientCredentialsResponse>(responseData)!;

				string? accessToken = result?.access_token;
				if (String.IsNullOrEmpty(accessToken))
				{
					throw new AuthenticationException("The authentication token received by the server is null or empty. Body received was: " + Encoding.UTF8.GetString(responseData), null);
				}
				_cachedAccessToken = accessToken;
				// renew after half the renewal time
				_expiresAt = DateTime.UtcNow + TimeSpan.FromSeconds((result?.expires_in ?? 3200) / 2.0);
			}
			catch (WebException ex)
			{
				throw new AuthenticationException("Unable to authenticate.", ex);
			}
		}
	}

	/// <summary>
	/// Factory for creating OAuth2AuthProvider instances from a set of options
	/// </summary>
	public class OAuthHandlerFactory
	{
		readonly HttpClient _httpClient;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="httpClient"></param>
		public OAuthHandlerFactory(HttpClient httpClient)
		{
			_httpClient = httpClient;
		}

		/// <summary>
		/// Create an instance of the auth provider
		/// </summary>
		/// <param name="options"></param>
		/// <returns></returns>
		public OAuthHandler<T> Create<T>(IOAuthOptions options) => new(_httpClient, options);
	}
}
