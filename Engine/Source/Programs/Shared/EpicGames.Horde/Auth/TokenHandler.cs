// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Auth
{
	/// <summary>
	/// Options for authenticating particular requests
	/// </summary>
	public interface ITokenAuthOptions
	{
		/// <summary>
		/// Bearer token for auth
		/// </summary>
		string Token { get; }
	}

	/// <summary>
	/// Http message handler which adds an OAuth authorization header using a cached/periodically refreshed bearer token
	/// </summary>
	public class TokenHandler<T> : HttpClientHandler
	{
		readonly ITokenAuthOptions _options;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="options"></param>
		public TokenHandler(ITokenAuthOptions options)
		{
			_options = options;
		}

		/// <inheritdoc/>
		protected override async Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
		{
			request.Headers.Add("Authorization", $"Bearer {_options.Token}");
			return await base.SendAsync(request, cancellationToken);
		}
	}
}
