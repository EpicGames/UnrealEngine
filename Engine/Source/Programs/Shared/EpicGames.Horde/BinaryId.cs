// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.ComponentModel;
using System.Globalization;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde
{
	/// <summary>
	/// Normalized string identifier for a resource
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(BinaryIdJsonConverter))]
	[TypeConverter(typeof(BinaryIdTypeConverter))]
	public readonly struct BinaryId : IEquatable<BinaryId>, IComparable<BinaryId>
	{
		readonly int _a;
		readonly int _b;
		readonly int _c;

		/// <summary>
		/// Number of bytes in the identifier
		/// </summary>
		public const int NumBytes = 12;

		/// <summary>
		/// Number of characters when formatted as a string
		/// </summary>
		public const int NumChars = NumBytes * 2;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="bytes">Bytes to parse</param>
		public BinaryId(ReadOnlySpan<byte> bytes)
		{
			_a = BinaryPrimitives.ReadInt32LittleEndian(bytes);
			_b = BinaryPrimitives.ReadInt32LittleEndian(bytes[4..]);
			_c = BinaryPrimitives.ReadInt32LittleEndian(bytes[8..]);
		}

		/// <summary>
		/// Parse a binary id from a string
		/// </summary>
		public static BinaryId Parse(string text)
		{
			BinaryId id;
			if (!TryParse(text, out id))
			{
				throw new FormatException($"Invalid BinaryId: {text}");
			}
			return id;
		}

		/// <summary>
		/// Parse a binary id from a string
		/// </summary>
		public static BinaryId Parse(Utf8String text) => Parse(text.Span);

		/// <summary>
		/// Parse a binary id from a string
		/// </summary>
		public static BinaryId Parse(ReadOnlySpan<byte> text)
		{
			BinaryId id;
			if (!TryParse(text, out id))
			{
				throw new FormatException($"Invalid BinaryId: {Encoding.UTF8.GetString(text)}");
			}
			return id;
		}

		/// <summary>
		/// Attempt to parse a binary id from a string
		/// </summary>
		public static bool TryParse(string text, out BinaryId result)
		{
			Span<byte> bytes = stackalloc byte[NumBytes];
			if (StringUtils.TryParseHexString(text, bytes))
			{
				result = new BinaryId(bytes);
				return true;
			}
			else
			{
				result = default;
				return false;
			}
		}

		/// <summary>
		/// Attempt to parse a binary id from a string
		/// </summary>
		public static bool TryParse(Utf8String text, out BinaryId result) => TryParse(text.Span, out result);

		/// <summary>
		/// Attempt to parse a binary id from a string
		/// </summary>
		public static bool TryParse(ReadOnlySpan<byte> text, out BinaryId result)
		{
			Span<byte> bytes = stackalloc byte[NumBytes];
			if (StringUtils.TryParseHexString(text, bytes))
			{
				result = new BinaryId(bytes);
				return true;
			}
			else
			{
				result = default;
				return false;
			}
		}

		/// <summary>
		/// Attempt to parse a binary id from a string
		/// </summary>
		public static bool TryParse(ReadOnlySpan<char> text, out BinaryId result)
		{
			Span<byte> bytes = stackalloc byte[NumBytes];
			if (StringUtils.TryParseHexString(text, bytes))
			{
				result = new BinaryId(bytes);
				return true;
			}
			else
			{
				result = default;
				return false;
			}
		}

		/// <summary>
		/// Checks whether this BinaryId is set
		/// </summary>
		public bool IsEmpty => (_a | _b | _c) == 0;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is BinaryId id && Equals(id);

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(_a, _b, _c);

		/// <inheritdoc/>
		public bool Equals(BinaryId other) => _a == other._a && _b == other._b && _c == other._c;

		/// <inheritdoc/>
		public override string ToString()
		{
			Span<byte> bytes = stackalloc byte[NumBytes];
			ToByteArray(bytes);
			return StringUtils.FormatHexString(bytes);
		}

		/// <summary>
		/// Format this id as a sequence of UTF8 characters
		/// </summary>
		public Utf8String ToUtf8String()
		{
			byte[] data = new byte[NumChars];
			ToUtf8String(data.AsSpan());
			return new Utf8String(data);
		}

		/// <summary>
		/// Format this id as a sequence of UTF8 characters
		/// </summary>
		public void ToUtf8String(Span<byte> chars)
		{
			StringUtils.FormatLittleEndianUtf8HexString((uint)_a, chars);
			StringUtils.FormatLittleEndianUtf8HexString((uint)_b, chars[8..]);
			StringUtils.FormatLittleEndianUtf8HexString((uint)_c, chars[16..]);
		}

		/// <summary>
		/// Format this id as a sequence of UTF8 characters
		/// </summary>
		public byte[] ToByteArray()
		{
			byte[] bytes = new byte[NumBytes];
			ToByteArray(bytes);
			return bytes;
		}

		/// <summary>
		/// Format this id as a sequence of UTF8 characters
		/// </summary>
		public void ToByteArray(Span<byte> bytes)
		{
			BinaryPrimitives.WriteInt32LittleEndian(bytes, _a);
			BinaryPrimitives.WriteInt32LittleEndian(bytes[4..], _b);
			BinaryPrimitives.WriteInt32LittleEndian(bytes[8..], _c);
		}

		/// <inheritdoc/>
		public int CompareTo(BinaryId other)
		{
			int result = _a.CompareTo(other._a);
			if (result == 0)
			{
				result = _b.CompareTo(other._b);
				if (result == 0)
				{
					result = _c.CompareTo(other._c);
				}
			}
			return result;
		}

		/// <summary>
		/// Compares two binary ids for equality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are equal</returns>
		public static bool operator ==(BinaryId left, BinaryId right) => left.Equals(right);

		/// <summary>
		/// Compares two binary ids for inequality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are not equal</returns>
		public static bool operator !=(BinaryId left, BinaryId right) => !left.Equals(right);

#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member
		public static bool operator <(BinaryId left, BinaryId right) => left.CompareTo(right) < 0;
		public static bool operator <=(BinaryId left, BinaryId right) => left.CompareTo(right) <= 0;
		public static bool operator >(BinaryId left, BinaryId right) => left.CompareTo(right) > 0;
		public static bool operator >=(BinaryId left, BinaryId right) => left.CompareTo(right) >= 0;
#pragma warning restore CS1591 // Missing XML comment for publicly visible type or member
	}

	/// <summary>
	/// Class which serializes <see cref="BinaryId"/> types
	/// </summary>
	sealed class BinaryIdJsonConverter : JsonConverter<BinaryId>
	{
		/// <inheritdoc/>
		public override BinaryId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => BinaryId.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, BinaryId value, JsonSerializerOptions options)
		{
			Span<byte> bytes = stackalloc byte[BinaryId.NumChars];
			value.ToUtf8String(bytes);
			writer.WriteStringValue(bytes);
		}
	}

	/// <summary>
	/// Class which serializes <see cref="BinaryId"/> types
	/// </summary>
	sealed class BinaryIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => BinaryId.Parse((string)value);

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType) => ((BinaryId)value!).ToString();
	}
}
