// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde
{
	/// <summary>
	/// Base class for converting to and from types containing a <see cref="BinaryId"/>. Useful pattern for reducing boilerplate with strongly typed records.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public abstract class BinaryIdConverter<T> where T : struct
	{
		/// <summary>
		/// Converts a type to a <see cref="BinaryId"/>
		/// </summary>
		public abstract BinaryId ToBinaryId(T value);

		/// <summary>
		/// Constructs a type from a <see cref="BinaryId"/>
		/// </summary>
		public abstract T FromBinaryId(BinaryId id);
	}

	/// <summary>
	/// Attribute declaring a <see cref="BinaryIdConverter{T}"/> for a particular type
	/// </summary>
	[AttributeUsage(AttributeTargets.Struct)]
	public sealed class BinaryIdConverterAttribute : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BinaryIdConverterAttribute(Type converterType) => ConverterType = converterType;
	}

	/// <summary>
	/// Converter to compact binary objects
	/// </summary>
	public sealed class BinaryIdCbConverter<TValue, TConverter> : CbConverter<TValue> where TValue : struct where TConverter : BinaryIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new();

		/// <inheritdoc/>
		public override TValue Read(CbField field)
		{
			return _converter.FromBinaryId(BinaryId.Parse(field.AsUtf8String()));
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, TValue value)
		{
			writer.WriteStringValue(_converter.ToBinaryId(value).ToString());
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, TValue value)
		{
			writer.WriteString(name, _converter.ToBinaryId(value).ToString());
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="BinaryIdConverter{T}"/> to Json
	/// </summary>
	public sealed class BinaryIdTypeConverter<TValue, TConverter> : TypeConverter where TValue : struct where TConverter : BinaryIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new();

		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || sourceType == typeof(BinaryId);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return _converter.FromBinaryId(BinaryId.Parse(str));
			}
			if (value is BinaryId stringId)
			{
				return _converter.FromBinaryId(stringId);
			}
			return null;
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string) || destinationType == typeof(BinaryId);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return _converter.ToBinaryId((TValue)value!).ToString();
			}
			if (destinationType == typeof(BinaryId))
			{
				return _converter.ToBinaryId((TValue)value!);
			}
			return null;
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="BinaryIdConverter{T}"/> to Json
	/// </summary>
	public sealed class BinaryIdJsonConverter<TValue, TConverter> : JsonConverter<TValue> where TValue : struct where TConverter : BinaryIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new();

		/// <inheritdoc/>
		public override TValue Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => _converter.FromBinaryId(BinaryId.Parse(reader.GetUtf8String()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TValue value, JsonSerializerOptions options)
		{
			BinaryId binaryId = _converter.ToBinaryId(value);

			Span<byte> span = stackalloc byte[12 * 2];
			binaryId.ToUtf8String(span);
			writer.WriteStringValue(span);
		}
	}

	/// <summary>
	/// Creates constructors for types with a <see cref="BinaryIdConverter{T}"/> to Json
	/// </summary>
	public sealed class BinaryIdJsonConverterFactory : JsonConverterFactory
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type typeToConvert) => typeToConvert.GetCustomAttribute<BinaryIdConverterAttribute>() != null;

		/// <inheritdoc/>
		public override JsonConverter? CreateConverter(Type type, JsonSerializerOptions options)
		{
			BinaryIdConverterAttribute? attribute = type.GetCustomAttribute<BinaryIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (JsonConverter?)Activator.CreateInstance(typeof(BinaryIdJsonConverter<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}
}
