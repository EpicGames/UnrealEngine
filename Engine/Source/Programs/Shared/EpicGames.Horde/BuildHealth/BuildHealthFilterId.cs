// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.BuildHealth
{
	/// <summary>
	/// Identifier for a build health filter.
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<BuildHealthFilterId, FilterIdConverter>))]
	[BinaryIdConverter(typeof(FilterIdConverter))]
	public readonly record struct BuildHealthFilterId(BinaryId Id)
	{
		/// <summary>
		/// Constant value for an empty build health filter id.
		/// </summary>
		public static BuildHealthFilterId Empty { get; } = default;

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static BuildHealthFilterId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class FilterIdConverter : BinaryIdConverter<BuildHealthFilterId>
	{
		/// <inheritdoc/>
		public override BuildHealthFilterId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(BuildHealthFilterId value) => value.Id;
	}
}
