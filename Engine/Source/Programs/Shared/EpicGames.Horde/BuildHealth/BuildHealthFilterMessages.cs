// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.BuildHealth
{
	/// <summary>
	/// Static class that describes build health filter restrictions.
	/// </summary>
	public static class BuildHealthFilterRestrictions
	{
		/// <summary>
		/// The maximum filter name length.
		/// </summary>
		public const int MaxFilterNameLength = 250;

		/// <summary>
		/// The maximum filter description length.
		/// </summary>
		public const int MaxFilterDescripitonLength = 500;

		/// <summary>
		/// The maximum filter query length.
		/// </summary>
		public const int MaxFilterQueryLength = 4500;
	}

	/// <summary>
	/// DTO used for requests used to add a build health filter.
	/// </summary>
	public record BuildHealthFilterAddRequest
	{
		/// <summary>
		/// The project the filter belongs to.
		/// </summary>
		public ProjectId FilterProject { get; set; } = default!;

		/// <summary>
		/// The name of the filter.
		/// </summary>
		public string FilterName { get; set; } = default!;

		/// <summary>
		/// The description of the filter.
		/// </summary>
		public string? FilterDescription { get; init; }

		/// <summary>
		/// The filter query.
		/// </summary>
		public string FilterQuery { get; set; } = default!;

		/// <summary>
		/// Returns whether the filter name request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterName() => (FilterName != null && FilterName.Length <= BuildHealthFilterRestrictions.MaxFilterNameLength);

		/// <summary>
		/// Returns whether the filter description request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterDescription() => (FilterDescription == null || FilterDescription.Length <= BuildHealthFilterRestrictions.MaxFilterDescripitonLength);

		/// <summary>
		/// Returns whether the filter query request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterQuery() => (FilterQuery != null && FilterQuery.Length <= BuildHealthFilterRestrictions.MaxFilterQueryLength);

		/// <summary>
		/// Returns whether the filter add request is valid.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidAddRequest() => IsValidFilterQuery() && IsValidFilterName() && IsValidFilterDescription();
	}

	/// <summary>
	/// DTO used for requests used to update a build health filter.
	/// </summary>
	public record BuildHealthFilterUpdateRequest
	{
		/// <summary>
		/// The name of the filter.
		/// </summary>
		public string? FilterName { get; set; } = default!;

		/// <summary>
		/// The description of the filter.
		/// </summary>
		public string? FilterDescription { get; init; }

		/// <summary>
		/// The filter query.
		/// </summary>
		public string? FilterQuery { get; set; } = default!;

		/// <summary>
		/// Returns whether the filter name request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterName() => (FilterName == null || FilterName.Length <= BuildHealthFilterRestrictions.MaxFilterNameLength);

		/// <summary>
		/// Returns whether the filter description request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterDescription() => (FilterDescription == null || FilterDescription.Length <= BuildHealthFilterRestrictions.MaxFilterDescripitonLength);

		/// <summary>
		/// Returns whether the filter query request is valid or not.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidFilterQuery() => (FilterQuery == null || FilterQuery.Length <= BuildHealthFilterRestrictions.MaxFilterQueryLength);

		/// <summary>
		/// Returns whether the filter update request is valid.
		/// </summary>
		/// <returns>True if valid, false otherwise.</returns>
		public bool IsValidUpdateRequest() => IsValidFilterQuery() && IsValidFilterName() && IsValidFilterDescription();
	}

	/// <summary>
	/// Response object for build health filters.
	/// </summary>
	public class BuildHealthFilterResponse
	{
		/// <summary>
		/// The id of the filter.
		/// </summary>
		public BuildHealthFilterId Id { get; init; } = default;

		/// <summary>
		/// The owner of the filter.
		/// </summary>
		public GetThinUserInfoResponse? Owner { get; init; } = default;

		/// <summary>
		/// The project the filter belongs to.
		/// </summary>
		public ProjectId FilterProject { get; init; }

		/// <summary>
		/// The filter name.
		/// </summary>
		public string FilterName { get; init; }

		/// <summary>
		/// The description of the filter.
		/// </summary>
		public string? FilterDescription { get; init; }

		/// <summary>
		/// The filter query.
		/// </summary>
		public string FilterQuery { get; init; }

		/// <summary>
		/// The last updated time of the filter.
		/// </summary>
		public DateTime UpdateTimeUtc { get; init; }

		/// <summary>
		/// Creates a build health filter response from a provided build health filter.
		/// </summary>
		/// <param name="filter">The source filter.</param>
		/// <returns>The response object.</returns>
		public BuildHealthFilterResponse(IBuildHealthFilter filter)
		{
			Id = filter.Id;
			Owner = null;
			FilterName = filter.FilterName;
			FilterProject = filter.FilterProject;
			FilterDescription = filter.FilterDescription;
			FilterQuery = filter.FilterQuery;
			UpdateTimeUtc = filter.UpdateTimeUtc;
		}

		/// <summary>
		/// Creates a build health filter response from a provided build health filter, and the ownership object.
		/// </summary>
		/// <param name="filter">The source filter.</param>
		/// <param name="owner">The owner of the filter.</param>
		public BuildHealthFilterResponse(IBuildHealthFilter filter, GetThinUserInfoResponse owner)
		{
			Id = filter.Id;
			Owner = owner;
			FilterName = filter.FilterName;
			FilterProject = filter.FilterProject;
			FilterDescription = filter.FilterDescription;
			FilterQuery = filter.FilterQuery;
			UpdateTimeUtc = filter.UpdateTimeUtc;
		}
	}
}
