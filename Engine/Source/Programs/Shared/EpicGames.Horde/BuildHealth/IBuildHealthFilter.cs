// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.BuildHealth
{
	/// <summary>
	/// Describes a build health filter.
	/// </summary>
	public interface IBuildHealthFilter
	{
		/// <summary>
		/// The id of the filter.
		/// </summary>
		BuildHealthFilterId Id { get; }

		/// <summary>
		/// Owner of the filter.
		/// </summary>
		UserId? Owner { get; }

		/// <summary>
		/// The underyling query of the filter.
		/// </summary>
		string FilterQuery { get; }

		/// <summary>
		/// The name of the filter.
		/// </summary>
		string FilterName { get; }

		/// <summary>
		/// A description of the filter.
		/// </summary>
		string? FilterDescription { get; }

		/// <summary>
		/// The project the filter belongs to.
		/// </summary>
		ProjectId FilterProject { get; }

		/// <summary>
		/// The last time this filter was updated (UTC).
		/// </summary>
		DateTime UpdateTimeUtc { get; }
	}
}
