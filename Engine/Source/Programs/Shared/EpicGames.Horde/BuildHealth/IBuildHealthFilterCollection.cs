// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.BuildHealth
{
	/// <summary>
	/// Collection interace for managing build heatlh filters.
	/// </summary>
	public interface IBuildHealthFilterCollection
	{
		/// <summary>
		/// Creates a new filter.
		/// </summary>
		/// <param name="owner">The owner of the new filter.</param>
		/// <param name="filterRequest">The add request.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The new filter, if applicable.</returns>
		Task<IBuildHealthFilter?> AddBuildHealthFilterAsync(UserId? owner, BuildHealthFilterAddRequest filterRequest, CancellationToken cancellationToken);

		/// <summary>
		/// Updates the build health filter with the provided updates.
		/// </summary>
		/// <param name="filterId">The id of the filter to update.</param>
		/// <param name="owner">The owner of the filter.</param>
		/// <param name="filterRequest">The update request.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The updated filter, if applicable, null if no updates were enacted.</returns>
		/// <remarks>The provided owner must be making the request.</remarks>
		Task<IBuildHealthFilter?> UpdateBuildHealthFilterAsync(BuildHealthFilterId filterId, UserId? owner, BuildHealthFilterUpdateRequest filterRequest, CancellationToken cancellationToken);

		/// <summary>
		/// Gets a filter.
		/// </summary>
		/// <param name="filterId">THe id of the filter to retrieve.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The filter, if found.</returns>
		Task<IBuildHealthFilter?> GetBuildHealthFilterAsync(BuildHealthFilterId filterId, CancellationToken cancellationToken);

		/// <summary>
		/// Gets filters for a given project.
		/// </summary>
		/// <param name="projectId">The project id to search for filters.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>A list of filters, if applicable.</returns>
		Task<IEnumerable<IBuildHealthFilter>> GetBuildHealthFiltersAsync(ProjectId projectId, CancellationToken cancellationToken);

		/// <summary>
		/// Deletes a filter.
		/// </summary>
		/// <param name="filterId">The id of the filter to delete.</param>
		/// <param name="owner">The owner of the filter.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>True if the filter was deleted successfully, false otherwise.</returns>
		/// <remarks>Only the owner of an owned filter can delete it. Unowned filters can be deleted by any user.</remarks>
		Task<bool> DeleteBuildHealthFilterAsync(BuildHealthFilterId filterId, UserId? owner, CancellationToken cancellationToken);
	}
}
