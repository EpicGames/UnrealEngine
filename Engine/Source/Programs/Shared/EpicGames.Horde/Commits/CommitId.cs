// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde.Commits
{
	/// <summary>
	/// Identifier for a commit from an arbitary version control system.
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(CommitIdJsonConverter))]
	[TypeConverter(typeof(CommitIdTypeConverter))]
	public class CommitId : IEquatable<CommitId>
	{
		/// <summary>
		/// Commit id with an empty name
		/// </summary>
		public static CommitId Empty { get; } = new CommitId(String.Empty);

		/// <summary>
		/// Name of this commit. Compared as a case-insensitive string.
		/// </summary>
		[JsonPropertyOrder(0)]
		public string Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CommitId(string name)
			=> Name = name;

		/// <summary>
		/// Creates a commit it from a Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public static CommitId FromPerforceChange(int change)
			=> new($"{change}");

		/// <summary>
		/// Creates a commit it from a Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public static CommitId? FromPerforceChange(int? change)
			=> (change == null) ? null : FromPerforceChange(change.Value);

		/// <summary>
		/// Gets the Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public int GetPerforceChange()
			=> Int32.Parse(Name);

		/// <summary>
		/// Gets the Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public int GetPerforceChangeOrMinusOne()
			=> TryGetPerforceChange() ?? -1;

		/// <summary>
		/// Gets the Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public int? TryGetPerforceChange()
			=> Int32.TryParse(Name, out int value) ? value : null;

		/// <inheritdoc/>
		public override bool Equals(object? obj)
			=> obj is CommitId other && Equals(other);

		/// <inheritdoc/>
		public bool Equals(CommitId? other)
			=> Name.Equals(other?.Name, StringComparison.OrdinalIgnoreCase);

		/// <inheritdoc/>
		public override int GetHashCode()
			=> Name.GetHashCode(StringComparison.OrdinalIgnoreCase);

		/// <inheritdoc/>
		public override string ToString()
			=> Name;

		/// <summary>
		/// Test two commits for equality
		/// </summary>
		public static bool operator ==(CommitId? a, CommitId? b)
			=> a?.Equals(b) ?? (b is null);

		/// <summary>
		/// Test two commits for inequality
		/// </summary>
		public static bool operator !=(CommitId? a, CommitId? b)
			=> !(a == b);
	}

	class CommitIdJsonConverter : JsonConverter<CommitId>
	{
		/// <inheritdoc/>
		public override CommitId? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			=> reader.TokenType switch
			{
				JsonTokenType.Number => CommitId.FromPerforceChange(reader.GetInt32()),
				JsonTokenType.String => new CommitId(reader.GetString()!),
				_ => throw new JsonException("Invalid commit id")
			};

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CommitId value, JsonSerializerOptions options)
			=> writer.WriteStringValue(value.Name);
	}

	class CommitIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
			=> sourceType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
			=> new CommitId((string)value);

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
			=> destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
			=> value?.ToString();
	}

	/// <summary>
	/// Variant of <see cref="CommitId"/> including a value that allows it to be used for ordering.
	/// </summary>
	public class CommitIdWithOrder : CommitId, IEquatable<CommitIdWithOrder>, IComparable<CommitIdWithOrder>
	{
		/// <summary>
		/// Commit id with an empty name
		/// </summary>
		public static new CommitIdWithOrder Empty { get; } = new CommitIdWithOrder(String.Empty, 0);

		/// <summary>
		/// Value used for ordering commits.
		/// </summary>
		[JsonPropertyOrder(1)]
		public int Order { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CommitIdWithOrder(string name, int order)
			: base(name)
		{
			Order = order;
		}

		/// <summary>
		/// Creates a commit it from a Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public static new CommitIdWithOrder FromPerforceChange(int change)
			=> new($"{change}", change);

		/// <summary>
		/// Creates a commit it from a Perforce changelist number. Temporary helper method for migration purposes.
		/// </summary>
		public static new CommitIdWithOrder? FromPerforceChange(int? change)
			=> (change == null) ? null : FromPerforceChange(change.Value);

		/// <summary>
		/// Gets the Perforce change number if the commit name is numeric, otherwise
		/// falls back to using the Order property.
		/// </summary>
		public int GetPerforceChangeOrOrder()
			=> TryGetPerforceChange() ?? Order;

		/// <inheritdoc/>
		public override bool Equals(object? obj)
			=> Equals(obj as CommitIdWithOrder);

		/// <inheritdoc/>
		public bool Equals(CommitIdWithOrder? other)
			=> other is not null && Order == other.Order;

		/// <inheritdoc/>
		public override int GetHashCode()
			=> base.GetHashCode();

		/// <inheritdoc/>
		public int CompareTo(CommitIdWithOrder? other)
		{
			if (other is null)
			{
				return -1;
			}
			else
			{
				return Order.CompareTo(other.Order);
			}
		}

#pragma warning disable CS1591
		public static bool operator ==(CommitIdWithOrder? a, CommitIdWithOrder? b)
			=> a?.Equals(b) ?? b is null;

		public static bool operator !=(CommitIdWithOrder? a, CommitIdWithOrder? b)
			=> !(a == b);

		public static bool operator <(CommitIdWithOrder a, CommitIdWithOrder b)
			=> a.CompareTo(b) < 0;

		public static bool operator <=(CommitIdWithOrder a, CommitIdWithOrder b)
			=> a.CompareTo(b) <= 0;

		public static bool operator >(CommitIdWithOrder a, CommitIdWithOrder b)
			=> a.CompareTo(b) > 0;

		public static bool operator >=(CommitIdWithOrder a, CommitIdWithOrder b)
			=> a.CompareTo(b) >= 0;
#pragma warning restore CS1591
	}
}
