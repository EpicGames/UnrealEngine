// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Users;

#pragma warning disable CA2227

namespace EpicGames.Horde.Commits
{
	/// <summary>
	/// Information about a commit
	/// </summary>
	public class GetCommitResponse
	{
		/// <summary>
		/// The commit id
		/// </summary>
		public CommitIdWithOrder Id
		{
			get => _id ?? CommitIdWithOrder.FromPerforceChange(_number) ?? CommitIdWithOrder.Empty;
			set => _id = value;
		}
		CommitIdWithOrder? _id;

		/// <summary>
		/// The changelist number
		/// </summary>
		[Obsolete("Use Id instead")]
		public int Number
		{
			get => _number ?? _id?.TryGetPerforceChange() ?? -1;
			set => _number = value;
		}
		int? _number;

		/// <summary>
		/// Name of the user that authored this change [DEPRECATED]
		/// </summary>
		public string Author { get; set; }

		/// <summary>
		/// Information about the user that authored this change
		/// </summary>
		public GetThinUserInfoResponse AuthorInfo { get; set; }

		/// <summary>
		/// The description text
		/// </summary>
		public string Description { get; set; }

		/// <summary>
		/// Date/time that change was committed
		/// </summary>
		public DateTime DateUtc { get; set; }

		/// <summary>
		/// Tags for this commit
		/// </summary>
		public List<CommitTag>? Tags { get; set; }

		/// <summary>
		/// List of files that were modified, relative to the stream base
		/// </summary>
		public List<string>? Files { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetCommitResponse(GetThinUserInfoResponse authorInfo, string description, DateTime dateUtc)
		{
			Author = authorInfo.Name;
			AuthorInfo = authorInfo;
			Description = description;
			DateUtc = dateUtc;
		}
	}
}
