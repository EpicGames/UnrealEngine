// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using EpicGames.Core;

namespace EpicGames.Horde.Commits
{
	internal static class RegexExtensions
	{
		public static string? ValueOrNull(this Group group)
		{
			return group.Success ? group.Value : null;
		}

		public static bool PresenceOf(this Group group)
		{
			return group.Success;
		}
	}

	/// <summary>
	/// Convinence data structure that contains all tags associated with a provided commit description.
	/// </summary>
	public partial struct CommitDescriptionTagContainer
	{

		#region -- Compile-time Regex --

		[GeneratedRegex(@"^#ROBOMERGE-OWNER:\s*([^@\s]+)", RegexOptions.Multiline)]
		private static partial Regex RobomergeOwnerRegex();

		[GeneratedRegex(@"^#ROBOMERGE-SOURCE:\s*(.*)", RegexOptions.Multiline)]
		private static partial Regex RobomergeSourceRegex();

		[GeneratedRegex(@"^#ROBOMERGE-AUTHOR:\s*(\S+)", RegexOptions.Multiline)]
		private static partial Regex RobomergeAuthorRegex();

		[GeneratedRegex(@"^#ROBOMERGE-BOT:\s*(.*)", RegexOptions.Multiline)]
		private static partial Regex RobomergeBotRegex();

		[GeneratedRegex(@"^#jira\s*([^@\s]+)", RegexOptions.Multiline)]
		private static partial Regex JiraRegex();

		[GeneratedRegex(@"^#tests\s*(.*)", RegexOptions.Multiline)]
		private static partial Regex TestsRegex();

		[GeneratedRegex(@"^#preflight\s*([^@\s]+)", RegexOptions.Multiline)]
		private static partial Regex PreflightRegex();

		[GeneratedRegex(@"^#rb\s+(.+)$", RegexOptions.Multiline)]
		private static partial Regex ReviewRegex();

		[GeneratedRegex(@"^#virtualized", RegexOptions.Multiline)]
		private static partial Regex Virtualized();

		#endregion -- Compile-time Regex --

		#region -- Public Properties --

		/// <summary>
		/// The source description of which the tags are extracted from.
		/// </summary>
		public string Description { get; init; }

		/// <summary>
		/// The jira tag in the commit description.
		/// </summary>
		public string? JiraTag { get; init; }

		/// <summary>
		/// The review tag in the commit description.
		/// </summary>
		public string? ReviewTag { get; set; }

		/// <summary>
		/// The presubmit tag in the commit description.
		/// </summary>
		public string? PresubmitTag { get; init; }

		/// <summary>
		/// The test tag in the commit description.
		/// </summary>
		public string? TestTag { get; init; }

		/// <summary>
		/// The robomerge owner tag in the commit description.
		/// </summary>
		public string? RobomergeOwnerTag { get; init; }

		/// <summary>
		/// The robomerge author tag in the commit description.
		/// </summary>
		public string? RobomergeAuthorTag { get; set; }

		/// <summary>
		/// The robomerge source tag in the commit description.
		/// </summary>
		public string? RobomergeSourceTag { get; init; }

		/// <summary>
		/// The robomerge bot tag in the commit description.
		/// </summary>
		public string? RobomergeBotTag { get; init; }

		/// <summary>
		/// Whether the change was virtualized or not.
		/// </summary>
		/// <remarks></remarks>
		public string? VirtualizedTag { get; init; }

		#endregion -- Public Properties --

		/// <summary>
		/// Creates a new CommitDescriptionTagContainer from a provided description.
		/// </summary>
		/// <param name="description">The description to use as the source for the CommitDescriptionTagContainer.</param>
		/// <returns>The CommitDescriptionTagContainer as constructed from the input description.</returns>
		public static CommitDescriptionTagContainer Create(string description)
		{
			return new CommitDescriptionTagContainer
			{
				Description = description,
				RobomergeOwnerTag = RobomergeOwnerRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				RobomergeSourceTag = RobomergeSourceRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				RobomergeAuthorTag = RobomergeAuthorRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				RobomergeBotTag = RobomergeBotRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				JiraTag = JiraRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				PresubmitTag = PreflightRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				TestTag = TestsRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				ReviewTag = ReviewRegex().Match(description).Groups[1].ValueOrNull()?.Trim(),
				VirtualizedTag = Virtualized().Match(description).Groups[0].PresenceOf().ToString().ToLower()
			};
		}
	}

	/// <summary>
	/// Constants for known commit tags
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(CommitTagJsonConverter))]
	public readonly struct CommitTag : IEquatable<CommitTag>
	{
		/// <summary>
		/// Predefined filter name for commits containing code
		/// </summary>
		public static CommitTag Code { get; } = new CommitTag("code");

		/// <summary>
		/// Predefined filter name for commits containing content
		/// </summary>
		public static CommitTag Content { get; } = new CommitTag("content");

		/// <summary>
		/// The tag text
		/// </summary>
		public string Text { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text"></param>
		public CommitTag(string text)
		{
			Text = text;
			if (!StringId.TryParse(text, out _, out string? errorMessage))
			{
				throw new ArgumentException(errorMessage, nameof(text));
			}
		}

		/// <summary>
		/// Check if the current tag is empty
		/// </summary>
		public bool IsEmpty() => Text == null;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is CommitTag id && Equals(id);

		/// <inheritdoc/>
		public override int GetHashCode() => String.GetHashCode(Text, StringComparison.Ordinal);

		/// <inheritdoc/>
		public bool Equals(CommitTag other) => String.Equals(Text, other.Text, StringComparison.Ordinal);

		/// <inheritdoc/>
		public override string ToString() => Text;

		/// <summary>
		/// Compares two string ids for equality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are equal</returns>
		public static bool operator ==(CommitTag left, CommitTag right) => left.Equals(right);

		/// <summary>
		/// Compares two string ids for inequality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are not equal</returns>
		public static bool operator !=(CommitTag left, CommitTag right) => !left.Equals(right);
	}

	/// <summary>
	/// Converts <see cref="CommitTag"/> values to and from JSON
	/// </summary>
	public class CommitTagJsonConverter : JsonConverter<CommitTag>
	{
		/// <inheritdoc/>
		public override CommitTag Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return new CommitTag(reader.GetString() ?? String.Empty);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CommitTag value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.Text);
		}
	}
}
