// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.Commits
{
	/// <summary>
	/// Stores metadata about a commit
	/// </summary>
	public interface ICommit
	{
		/// <summary>
		/// Id for this commit
		/// </summary>
		CommitIdWithOrder Id { get; }

		/// <summary>
		/// Stream containing the commit
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The change that this commit originates from
		/// </summary>
		CommitIdWithOrder OriginalCommitId { get; }

		/// <summary>
		/// The author user id
		/// </summary>
		UserId AuthorId { get; }

		/// <summary>
		/// The owner of this change, if different from the author (due to Robomerge)
		/// </summary>
		UserId OwnerId { get; }

		/// <summary>
		/// Changelist description
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Base path for all files in the change
		/// </summary>
		string BasePath { get; }

		/// <summary>
		/// Date/time that change was committed
		/// </summary>
		DateTime DateUtc { get; }

		/// <summary>
		/// Gets the list of tags for the commit
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the commit has the given tag</returns>
		ValueTask<IReadOnlyList<CommitTag>> GetTagsAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Determine if this commit matches the given filter. Prefer using commit tags rather than this method; the results can be cached.
		/// </summary>
		/// <param name="filter">Filter to test</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		ValueTask<bool> MatchesFilterAsync(FileFilter filter, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the files for this change, relative to the root of the stream
		/// </summary>
		/// <param name="minFiles">Minimum number of files to return. The response will include at least this number of files, unless the commit has fewer files.</param>
		/// <param name="maxFiles">Maximum number of files to return. Querying large number of files may cause performance issues with merge commits.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of files modified by this commit</returns>
		ValueTask<IReadOnlyList<string>> GetFilesAsync(int? minFiles, int? maxFiles, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Extension methods for operating on commits
	/// </summary>
	public static class CommitExtensions
	{
		/// <summary>
		/// Gets the files for this change, relative to the root of the stream
		/// </summary>
		/// <param name="commit">Commit to operate on</param>
		/// <param name="files">Number of files to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of files modified by this commit</returns>
		public static ValueTask<IReadOnlyList<string>> GetFilesAsync(this ICommit commit, int files, CancellationToken cancellationToken)
			=> commit.GetFilesAsync(files, files, cancellationToken);
	}
}
