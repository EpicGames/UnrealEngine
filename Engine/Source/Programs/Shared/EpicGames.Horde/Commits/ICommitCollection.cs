// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs.Templates;

namespace EpicGames.Horde.Commits
{
	/// <summary>
	/// Sort order for commit queries
	/// </summary>
	public enum CommitSortOrder
	{
		/// <summary>
		/// Newest commits first (default)
		/// </summary>
		Descending,

		/// <summary>
		/// Oldest commits first
		/// </summary>
		Ascending
	}

	/// <summary>
	/// Represents errors that occur during commit collection operations.
	/// This exception is thrown when operations on a <see cref="ICommitCollection"/> fail.
	///
	/// Serves as a wrapper for underlying VCS-specific exceptions, providing consistent errors for each VCS implementation
	/// </summary>
	public class CommitCollectionException(string? message, Exception? innerException) : Exception(message, innerException);

	/// <summary>
	/// VCS abstraction. Provides information about commits to a particular stream.
	/// </summary>
	public interface ICommitCollection
	{
		/// <summary>
		/// Creates a new change
		/// </summary>
		/// <param name="path">Path to modify in the change</param>
		/// <param name="description">Description of the change</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New commit information</returns>
		Task<CommitIdWithOrder> CreateNewAsync(string path, string description, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a commit by id
		/// </summary>
		/// <param name="commitId">Commit to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Commit details</returns>
		Task<ICommit> GetAsync(CommitId commitId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets an ordered commit id
		/// </summary>
		/// <param name="commitId">The commit to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Numbered commit id</returns>
		ValueTask<CommitIdWithOrder> GetOrderedAsync(CommitId commitId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds changes submitted to a stream.
		/// </summary>
		/// <param name="minCommitId">The minimum changelist number</param>
		/// <param name="includeMinCommit">Whether to include the minimum changelist in the range of enumerated responses</param>
		/// <param name="maxCommitId">The maximum changelist number</param>
		/// <param name="includeMaxCommit">Whether to include the maximum changelist in the range of enumerated responses</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="tags">Tags for the commits to return</param>
		/// <param name="sortOrder">Sort order for results (Descending = newest first, Ascending = oldest first)</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Changelist information</returns>
		IAsyncEnumerable<ICommit> FindAsync(CommitId? minCommitId = null, bool includeMinCommit = true, CommitId? maxCommitId = null, bool includeMaxCommit = true, int? maxResults = null, IReadOnlyList<CommitTag>? tags = null, CommitSortOrder sortOrder = CommitSortOrder.Descending, CancellationToken cancellationToken = default);

		/// <summary>
		/// Subscribes to changes from this commit source
		/// </summary>
		/// <param name="minCommitId">Minimum changelist number (exclusive)</param>
		/// <param name="tags">Tags for the commit to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New change information</returns>
		IAsyncEnumerable<ICommit> SubscribeAsync(CommitId minCommitId, IReadOnlyList<CommitTag>? tags = null, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for <see cref="ICommitCollection"/>
	/// </summary>
	public static class CommitCollectionExtensions
	{
		/// <summary>
		/// Creates a new change for a template
		/// </summary>
		/// <param name="commitCollection">The Perforce service instance</param>
		/// <param name="template">The template being built</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New changelist number</returns>
		public static Task<CommitIdWithOrder> CreateNewAsync(this ICommitCollection commitCollection, ITemplate template, CancellationToken cancellationToken)
		{
			string description = (template.SubmitDescription ?? "[Horde] New change for $(TemplateName)").Replace("$(TemplateName)", template.Name, StringComparison.OrdinalIgnoreCase);
			return commitCollection.CreateNewAsync(template.SubmitNewChange!, description, cancellationToken);
		}

		/// <summary>
		/// Finds changes submitted to a stream, in reverse order.
		/// </summary>
		/// <param name="commitCollection">Collection to operate on</param>
		/// <param name="minCommitId">The minimum changelist number</param>
		/// <param name="maxCommitId">The maximum changelist number</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="tags">Tags for the commits to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Changelist information</returns>
		public static IAsyncEnumerable<ICommit> FindAsync(this ICommitCollection commitCollection, CommitId? minCommitId, CommitId? maxCommitId, int? maxResults = null, IReadOnlyList<CommitTag>? tags = null, CancellationToken cancellationToken = default)
			=> commitCollection.FindAsync(minCommitId: minCommitId, maxCommitId: maxCommitId, maxResults: maxResults, tags: tags, cancellationToken: cancellationToken);

		/// <summary>
		/// Gets the last code code equal or before the given change number
		/// </summary>
		/// <param name="commitCollection">The commit source to query</param>
		/// <param name="maxCommitId">Maximum code change to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The last code change</returns>
		public static async ValueTask<ICommit?> GetLastCodeChangeAsync(this ICommitCollection commitCollection, CommitId? maxCommitId, CancellationToken cancellationToken = default)
		{
			return await commitCollection.FindAsync(minCommitId: null, maxCommitId: maxCommitId, maxResults: 1, tags: [CommitTag.Code], cancellationToken: cancellationToken).FirstOrDefaultAsync(cancellationToken);
		}

		/// <summary>
		/// Finds the latest commit from a source
		/// </summary>
		/// <param name="commitCollection">The commit source to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The latest commit</returns>
		public static async Task<ICommit> GetLatestAsync(this ICommitCollection commitCollection, CancellationToken cancellationToken = default)
		{
			ICommit? commit = await commitCollection.FindAsync(null, null, maxResults: 1, cancellationToken: cancellationToken).FirstOrDefaultAsync(cancellationToken);
			if (commit == null)
			{
				throw new Exception("No changes found for stream.");
			}
			return commit;
		}

		/// <summary>
		/// Finds the latest commit from a source
		/// </summary>
		/// <param name="commitCollection">The commit source to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The latest commit</returns>
		public static async Task<CommitIdWithOrder> GetLastCommitIdAsync(this ICommitCollection commitCollection, CancellationToken cancellationToken = default)
		{
			ICommit commit = await GetLatestAsync(commitCollection, cancellationToken);
			return commit.Id;
		}
	}
}
