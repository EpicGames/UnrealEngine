// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Common
{
	enum TokenType : byte
	{
		Error,
		End,
		Identifier,
		Scalar,
		Not,
		Eq,
		Neq,
		LogicalOr,
		LogicalAnd,
		Lt,
		Lte,
		Gt,
		Gte,
		Lparen,
		Rparen,
		Regex,
		True,
		False,
	}

	class TokenReader
	{
		public string Input { get; private set; }
		public int Offset { get; private set; }
		public int Length { get; private set; }

		public StringView Token => new(Input, Offset, Length);
		public TokenType Type { get; private set; }
		public string? Scalar { get; private set; }

		public TokenReader(string input)
		{
			Input = input;
			MoveNext();
		}

		private void SetCurrent(int length, TokenType type, string? scalar = null)
		{
			Length = length;
			Type = type;
			Scalar = scalar;
		}

		public void MoveNext()
		{
			Offset += Length;

			while (Offset < Input.Length)
			{
				switch (Input[Offset])
				{
					case ' ':
					case '\t':
						Offset++;
						break;
					case '(':
						SetCurrent(1, TokenType.Lparen);
						return;
					case ')':
						SetCurrent(1, TokenType.Rparen);
						return;
					case '!':
						if (Offset + 1 < Input.Length && Input[Offset + 1] == '=')
						{
							SetCurrent(2, TokenType.Neq);
						}
						else
						{
							SetCurrent(1, TokenType.Not);
						}
						return;
					case '&':
						if (RequireCharacter(Input, Offset + 1, '&'))
						{
							SetCurrent(2, TokenType.LogicalAnd);
						}
						return;
					case '|':
						if (RequireCharacter(Input, Offset + 1, '|'))
						{
							SetCurrent(2, TokenType.LogicalOr);
						}
						return;
					case '=':
						if (RequireCharacter(Input, Offset + 1, '='))
						{
							SetCurrent(2, TokenType.Eq);
						}
						return;
					case '~':
						if (RequireCharacter(Input, Offset + 1, '='))
						{
							SetCurrent(2, TokenType.Regex);
						}
						return;
					case '>':
						if (Offset + 1 < Input.Length && Input[Offset + 1] == '=')
						{
							SetCurrent(2, TokenType.Gte);
						}
						else
						{
							SetCurrent(1, TokenType.Gt);
						}
						return;
					case '<':
						if (Offset + 1 < Input.Length && Input[Offset + 1] == '=')
						{
							SetCurrent(2, TokenType.Lte);
						}
						else
						{
							SetCurrent(1, TokenType.Lt);
						}
						return;
					case '\"':
					case '\'':
						ParseString();
						return;
					case char character when IsNumber(character):
						ParseNumber();
						return;
					case char character when IsIdentifier(character):
						int endIdx = Offset + 1;
						while (endIdx < Input.Length && IsIdentifierTail(Input[endIdx]))
						{
							endIdx++;
						}

						TokenType type;
						if (endIdx == Offset + 4 && String.Compare(Input, Offset, "true", 0, 4, StringComparison.OrdinalIgnoreCase) == 0)
						{
							type = TokenType.True;
						}
						else if (endIdx == Offset + 5 && String.Compare(Input, Offset, "false", 0, 5, StringComparison.OrdinalIgnoreCase) == 0)
						{
							type = TokenType.False;
						}
						else
						{
							type = TokenType.Identifier;
						}

						SetCurrent(endIdx - Offset, type);
						return;
					default:
						SetCurrent(1, TokenType.Error, $"Invalid character at offset {Offset}: '{Input[Offset]}'");
						return;
				}
			}

			SetCurrent(0, TokenType.End);
		}

		bool ParseString()
		{
			char quoteChar = Input[Offset];
			if (quoteChar != '\'' && quoteChar != '\"')
			{
				SetCurrent(1, TokenType.Error, $"Invalid quote character '{(char)quoteChar}' at offset {Offset}");
				return false;
			}

			int numEscapeChars = 0;

			int endIdx = Offset + 1;
			for (; ; endIdx++)
			{
				if (endIdx >= Input.Length)
				{
					SetCurrent(endIdx - Offset, TokenType.Error, "Unterminated string in expression");
					return false;
				}
				else if (Input[endIdx] == '\\')
				{
					numEscapeChars++;
					endIdx++;
				}
				else if (Input[endIdx] == quoteChar)
				{
					break;
				}
			}

			char[] copy = new char[endIdx - (Offset + 1) - numEscapeChars];

			int inputIdx = Offset + 1;
			int outputIdx = 0;
			while (outputIdx < copy.Length)
			{
				if (Input[inputIdx] == '\\')
				{
					inputIdx++;
				}
				copy[outputIdx++] = Input[inputIdx++];
			}

			SetCurrent(endIdx + 1 - Offset, TokenType.Scalar, new string(copy));
			return true;
		}

		static readonly Dictionary<StringView, ulong> s_sizeSuffixes = new(StringViewComparer.OrdinalIgnoreCase)
		{
			["kb"] = 1024UL,
			["Mb"] = 1024UL * 1024,
			["Gb"] = 1024UL * 1024 * 1024,
			["Tb"] = 1024UL * 1024 * 1024 * 1024,
		};

		bool ParseNumber()
		{
			ulong value = 0;

			int endIdx = Offset;
			while (endIdx < Input.Length && IsNumber(Input[endIdx]))
			{
				value = (value * 10) + (uint)(Input[endIdx] - '0');
				endIdx++;
			}

			if (endIdx < Input.Length && IsIdentifier(Input[endIdx]))
			{
				int offset = endIdx++;
				while (endIdx < Input.Length && IsIdentifierTail(Input[endIdx]))
				{
					endIdx++;
				}

				StringView suffix = new(Input, offset, endIdx - offset);

				ulong size;
				if (!s_sizeSuffixes.TryGetValue(suffix, out size))
				{
					SetCurrent((endIdx + 1) - offset, TokenType.Error, $"'{suffix}' is not a valid numerical suffix");
					return false;
				}
				value *= size;
			}

			SetCurrent(endIdx - Offset, TokenType.Scalar, value.ToString(CultureInfo.InvariantCulture));
			return true;
		}

		bool RequireCharacter(string text, int idx, char character)
		{
			if (idx == text.Length || text[idx] != character)
			{
				SetCurrent(1, TokenType.Error, $"Invalid character at position {idx}; expected '{character}'.");
				return false;
			}
			return true;
		}

		static bool IsNumber(char character)
		{
			return character >= '0' && character <= '9';
		}

		static bool IsIdentifier(char character)
		{
			return (character >= 'a' && character <= 'z') || (character >= 'A' && character <= 'Z') || character == '_';
		}

		static bool IsIdentifierTail(char character)
		{
			return IsIdentifier(character) || IsNumber(character) || character == '-' || character == '.';
		}
	}

	/// <summary>
	/// Exception thrown when a condition is not valid
	/// </summary>
	public class ConditionException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="error"></param>
		public ConditionException(string error)
			: base(error)
		{
		}
	}

	/// <summary>
	/// A conditional expression that can be evaluated against a particular object
	/// </summary>
	[JsonSchemaString]
	[TypeConverter(typeof(ConditionTypeConverter))]
	[CbConverter(typeof(ConditionCbConverter))]
	[JsonConverter(typeof(ConditionJsonConverter))]
	public class Condition
	{
		[DebuggerDisplay("{Type}")]
		readonly struct Token
		{
			public readonly TokenType Type;
			public readonly byte Index;

			public Token(TokenType type, int index)
			{
				Type = type;
				Index = (byte)index;
			}
		}

		/// <summary>
		/// The condition text
		/// </summary>
		public string Text { get; }

		/// <summary>
		/// Error produced when parsing the condition
		/// </summary>
		public string? Error { get; private set; }

		readonly List<Token> _tokens = [];
		readonly List<string> _strings = [];

		static readonly IEnumerable<string> s_trueScalar = ["true"];
		static readonly IEnumerable<string> s_falseScalar = ["false"];

		private Condition(string text)
		{
			Text = text;

			TokenReader reader = new(text);
			if (reader.Type != TokenType.End)
			{
				Error = ParseOrExpr(reader);

				if (reader.Type == TokenType.Error)
				{
					Error = reader.Scalar;
				}
				else if (Error == null && reader.Type != TokenType.End)
				{
					Error = $"Unexpected token at offset {reader.Offset}: {reader.Token}";
				}
			}
		}

		/// <summary>
		/// Determines if the condition is empty
		/// </summary>
		/// <returns>True if the condition is empty</returns>
		public bool IsEmpty() => _tokens.Count == 0 && IsValid();

		/// <summary>
		/// Checks if the condition has been parsed correctly
		/// </summary>
		public bool IsValid() => Error == null;

		/// <summary>
		/// Parse the given text as a condition
		/// </summary>
		/// <param name="text">Condition text to parse</param>
		/// <returns>The new condition object</returns>
		public static Condition Parse(string text)
		{
			Condition condition = new(text);
			if (!condition.IsValid())
			{
				throw new ConditionException(condition.Error!);
			}
			return condition;
		}

		/// <summary>
		/// Attempts to parse the given text as a condition
		/// </summary>
		/// <param name="text">Condition to parse</param>
		/// <returns>The parsed condition. Does not validate whether the parse completed successfully; call <see cref="Condition.IsValid"/> to verify.</returns>
		public static Condition TryParse(string text)
		{
			return new Condition(text);
		}

		string? ParseOrExpr(TokenReader reader)
		{
			int startCount = _tokens.Count;

			string? error = ParseAndExpr(reader);
			while (error == null && reader.Type == TokenType.LogicalOr)
			{
				_tokens.Insert(startCount++, new Token(TokenType.LogicalOr, 0));
				reader.MoveNext();
				error = ParseAndExpr(reader);
			}

			return error;
		}

		string? ParseAndExpr(TokenReader reader)
		{
			int startCount = _tokens.Count;

			string? error = ParseBooleanExpr(reader);
			while (error == null && reader.Type == TokenType.LogicalAnd)
			{
				_tokens.Insert(startCount++, new Token(TokenType.LogicalAnd, 0));
				reader.MoveNext();
				error = ParseBooleanExpr(reader);
			}

			return error;
		}

		string? ParseBooleanExpr(TokenReader reader)
		{
			switch (reader.Type)
			{
				case TokenType.Not:
					_tokens.Add(new Token(reader.Type, 0));
					reader.MoveNext();
					if (reader.Type != TokenType.Lparen)
					{
						return $"Expected '(' at offset {reader.Offset}";
					}
					return ParseSubExpr(reader);
				case TokenType.Lparen:
					return ParseSubExpr(reader);
				case TokenType.True:
				case TokenType.False:
					_tokens.Add(new Token(reader.Type, 0));
					reader.MoveNext();
					return null;
				default:
					return ParseComparisonExpr(reader);
			}
		}

		string? ParseSubExpr(TokenReader reader)
		{
			reader.MoveNext();

			string? error = ParseOrExpr(reader);
			if (error == null)
			{
				if (reader.Type == TokenType.Rparen)
				{
					reader.MoveNext();
				}
				else
				{
					error = $"Missing ')' at offset {reader.Offset}";
				}
			}
			return error;
		}

		string? ParseComparisonExpr(TokenReader reader)
		{
			int startCount = _tokens.Count;

			string? error = ParseScalarExpr(reader);
			if (error == null)
			{
				switch (reader.Type)
				{
					case TokenType.Lt:
					case TokenType.Lte:
					case TokenType.Gt:
					case TokenType.Gte:
					case TokenType.Eq:
					case TokenType.Neq:
					case TokenType.Regex:
						_tokens.Insert(startCount, new Token(reader.Type, 0));
						reader.MoveNext();
						error = ParseScalarExpr(reader);
						break;
				}
			}

			return error;
		}

		string? ParseScalarExpr(TokenReader reader)
		{
			switch (reader.Type)
			{
				case TokenType.True:
				case TokenType.False:
					_tokens.Add(new Token(reader.Type, 0));
					reader.MoveNext();
					return null;
				case TokenType.Identifier:
					_strings.Add(reader.Token.ToString());
					_tokens.Add(new Token(TokenType.Identifier, _strings.Count - 1));
					reader.MoveNext();
					return null;
				case TokenType.Scalar:
					_strings.Add(reader.Scalar!);
					_tokens.Add(new Token(TokenType.Scalar, _strings.Count - 1));
					reader.MoveNext();
					return null;
				default:
					return $"Unexpected token '{reader.Token}' at offset {reader.Offset}";
			}
		}

		/// <summary>
		/// Evaluate the condition using the given callback to retrieve property values
		/// </summary>
		/// <param name="getPropertyValues"></param>
		/// <returns></returns>
		public bool Evaluate(Func<string, IEnumerable<string>> getPropertyValues)
		{
			if (IsEmpty())
			{
				return true;
			}

			int idx = 0;
			return IsValid() && EvaluateCondition(ref idx, getPropertyValues);
		}

		bool EvaluateCondition(ref int idx, Func<string, IEnumerable<string>> getPropertyValues)
		{
			bool lhsBool;
			bool rhsBool;
			IEnumerable<string> lhsScalar;
			IEnumerable<string> rhsScalar;

			Token token = _tokens[idx++];
			switch (token.Type)
			{
				case TokenType.True:
					return true;
				case TokenType.False:
					return false;
				case TokenType.Not:
					return !EvaluateCondition(ref idx, getPropertyValues);
				case TokenType.Eq:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return lhsScalar.Any(x => rhsScalar.Contains(x, StringComparer.OrdinalIgnoreCase));
				case TokenType.Neq:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return !lhsScalar.Any(x => rhsScalar.Contains(x, StringComparer.OrdinalIgnoreCase));
				case TokenType.LogicalOr:
					lhsBool = EvaluateCondition(ref idx, getPropertyValues);
					rhsBool = EvaluateCondition(ref idx, getPropertyValues);
					return lhsBool || rhsBool;
				case TokenType.LogicalAnd:
					lhsBool = EvaluateCondition(ref idx, getPropertyValues);
					rhsBool = EvaluateCondition(ref idx, getPropertyValues);
					return lhsBool && rhsBool;
				case TokenType.Lt:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return AsIntegers(lhsScalar).Any(x => AsIntegers(rhsScalar).Any(y => x < y));
				case TokenType.Lte:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return AsIntegers(lhsScalar).Any(x => AsIntegers(rhsScalar).Any(y => x <= y));
				case TokenType.Gt:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return AsIntegers(lhsScalar).Any(x => AsIntegers(rhsScalar).Any(y => x > y));
				case TokenType.Gte:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return AsIntegers(lhsScalar).Any(x => AsIntegers(rhsScalar).Any(y => x >= y));
				case TokenType.Regex:
					lhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					rhsScalar = EvaluateScalar(ref idx, getPropertyValues);
					return lhsScalar.Any(x => rhsScalar.Any(y => Regex.IsMatch(x, y, RegexOptions.IgnoreCase)));
				default:
					throw new InvalidOperationException("Invalid token type");
			}
		}

		IEnumerable<string> EvaluateScalar(ref int idx, Func<string, IEnumerable<string>> getPropertyValues)
		{
			Token token = _tokens[idx++];
			return token.Type switch
			{
				TokenType.True => s_trueScalar,
				TokenType.False => s_falseScalar,
				TokenType.Identifier => getPropertyValues(_strings[token.Index]),
				TokenType.Scalar => [_strings[token.Index]],
				_ => throw new InvalidOperationException("Invalid token type")
			};
		}

		static IEnumerable<long> AsIntegers(IEnumerable<string> scalars)
		{
			foreach (string scalar in scalars)
			{
				if (Int64.TryParse(scalar, out long value))
				{
					yield return value;
				}
			}
		}

		/// <summary>
		/// Implicit conversion from string to conditions
		/// </summary>
		/// <param name="text"></param>
		[return: NotNullIfNotNull(nameof(text))]
		public static implicit operator Condition?(string? text)
		{
			if (text == null)
			{
				return null;
			}
			else
			{
				return new Condition(text);
			}
		}

		/// <inheritdoc/>
		public override string ToString() => (Error != null) ? $"[Error] {Text}" : Text;
	}

	/// <summary>
	/// Converter from conditions to compact binary objects
	/// </summary>
	public class ConditionCbConverter : CbConverter<Condition>
	{
		/// <inheritdoc/>
		public override Condition Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}
			else
			{
				return Condition.TryParse(field.AsUtf8String().ToString());
			}
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, Condition value)
		{
			if (value == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.WriteUtf8StringValue(new Utf8String(value.Text));
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, Condition value)
		{
			if (value != null)
			{
				writer.WriteUtf8String(name, new Utf8String(value.Text));
			}
		}
	}

	/// <summary>
	/// Type converter from strings to condition objects
	/// </summary>
	sealed class ConditionTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => Condition.TryParse((string)value);

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type? destinationType) => ((Condition)value!).Text;
	}

	/// <summary>
	/// Type converter from Json strings to condition objects
	/// </summary>
	sealed class ConditionJsonConverter : JsonConverter<Condition>
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type typeToConvert) => typeToConvert == typeof(Condition);

		public override Condition Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? text = reader.GetString();
			if (text == null)
			{
				throw new InvalidOperationException();
			}
			return Condition.Parse(text);
		}

		public override void Write(Utf8JsonWriter writer, Condition value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}
}
