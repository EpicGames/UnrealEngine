// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Represents a remotely executed process managed by the Horde agent
	/// </summary>
	public sealed class AgentManagedProcess : IAsyncDisposable
	{
		readonly Channel<string> _output;
		readonly BackgroundTask _backgroundTask;
		readonly TaskCompletionSource<int> _result = new(TaskCreationOptions.RunContinuationsAsynchronously);

		byte[] _buffer = new byte[1024];
		int _bufferLength;

		/// <inheritdoc/>
		public bool HasExited => _result.Task.IsCompleted;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentManagedProcess(AgentMessageChannel channel)
		{
			_output = Channel.CreateUnbounded<string>();
			_backgroundTask = BackgroundTask.StartNew(ctx => RunAsync(channel, ctx));
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => _backgroundTask.DisposeAsync();

		/// <inheritdoc/>
		public async ValueTask<string?> ReadLineAsync(CancellationToken cancellationToken = default)
		{
			if (!await _output.Reader.WaitToReadAsync(cancellationToken))
			{
				return null;
			}
			return await _output.Reader.ReadAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<int> WaitForExitAsync(CancellationToken cancellationToken) => _result.Task.WaitAsync(cancellationToken);

		async Task RunAsync(AgentMessageChannel channel, CancellationToken cancellationToken)
		{
			try
			{
				await RunInternalAsync(channel, cancellationToken);
				_output.Writer.TryComplete();
			}
			catch (Exception ex)
			{
				_result.TrySetException(ex);
				_output.Writer.TryComplete(ex);
			}
		}

		async Task RunInternalAsync(AgentMessageChannel channel, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				using AgentMessage message = await channel.ReceiveAsync(cancellationToken);
				switch (message.Type)
				{
					case AgentMessageType.Exception:
						ExceptionMessage exception = message.ParseExceptionMessage();
						ComputeExecutionCancelledException.TryThrow(exception);
						throw new ComputeException("Error while executing remote process", new ComputeRemoteException(exception));
					case AgentMessageType.ExecuteOutput:
						AppendData(message.Data.Span);
						break;
					case AgentMessageType.ExecuteResult:
						ExecuteProcessResponseMessage executeProcessResponse = message.ParseExecuteProcessResponse();
						_result.TrySetResult(executeProcessResponse.ExitCode);
						return;
					default:
						message.ThrowIfUnexpectedType();
						return;
				}
			}
		}

		void AppendData(ReadOnlySpan<byte> data)
		{
			for (; ; )
			{
				int lineEnd = data.IndexOf((byte)'\n');
				if (lineEnd == -1)
				{
					AppendToBuffer(data);
					break;
				}

				int lineLen = lineEnd;
				if (lineLen > 0 && data[lineLen - 1] == '\r')
				{
					lineLen--;
				}

				string str;
				if (_bufferLength == 0)
				{
					str = Encoding.UTF8.GetString(data.Slice(0, lineLen));
				}
				else
				{
					AppendToBuffer(data.Slice(0, lineLen));
					str = Encoding.UTF8.GetString(_buffer.AsSpan(0, _bufferLength));
					_bufferLength = 0;
				}

				_output.Writer.TryWrite(str);
				data = data.Slice(lineEnd + 1);
			}
		}

		void AppendToBuffer(ReadOnlySpan<byte> data)
		{
			if (_bufferLength + data.Length > _buffer.Length)
			{
				byte[] newBuffer = new byte[_bufferLength + data.Length + 256];
				_buffer.AsSpan(0, _bufferLength).CopyTo(newBuffer);
				_buffer = newBuffer;
			}
			data.CopyTo(_buffer.AsSpan(_bufferLength));
			_bufferLength += data.Length;
		}
	}
}
