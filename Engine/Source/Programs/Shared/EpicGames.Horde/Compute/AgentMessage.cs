// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;

#pragma warning disable CA1054 // URI-like parameters should not be strings
#pragma warning disable CA1056 // Change string to URI

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Type of a compute message
	/// </summary>
	public enum AgentMessageType
	{
		/// <summary>
		/// No message was received (end of stream)
		/// </summary>
		None = 0x00,

		/// <summary>
		/// No-op message sent to keep the connection alive. Remote should reply with the same message.
		/// </summary>
		Ping = 0x01,

		/// <summary>
		/// Sent in place of a regular response if an error occurs on the remote
		/// </summary>
		Exception = 0x02,

		/// <summary>
		/// Fork the message loop into a new channel
		/// </summary>
		Fork = 0x03,

		/// <summary>
		/// Sent as the first message on a channel to notify the remote that the remote end is attached
		/// </summary>
		Attach = 0x04,

		#region Process Management

		/// <summary>
		/// Extract files on the remote machine (Initiator -> Remote)
		/// </summary>
		WriteFiles = 0x10,

		/// <summary>
		/// Notification that files have been extracted (Remote -> Initiator)
		/// </summary>
		WriteFilesResponse = 0x11,

		/// <summary>
		/// Deletes files on the remote machine (Initiator -> Remote)
		/// </summary>
		DeleteFiles = 0x12,

		/// <summary>
		/// Execute a process in a sandbox (Initiator -> Remote)
		/// </summary>
		ExecuteV1 = 0x16,

		/// <summary>
		/// Execute a process in a sandbox (Initiator -> Remote)
		/// </summary>
		ExecuteV2 = 0x22,

		/// <summary>
		/// Execute a process in a sandbox (Initiator -> Remote)
		/// </summary>
		ExecuteV3 = 0x23,

		/// <summary>
		/// Returns output from the child process to the caller (Remote -> Initiator)
		/// </summary>
		ExecuteOutput = 0x17,

		/// <summary>
		/// Returns the process exit code (Remote -> Initiator)
		/// </summary>
		ExecuteResult = 0x18,

		#endregion

		#region Storage

		/// <summary>
		/// Reads a blob from storage
		/// </summary>
		ReadBlob = 0x20,

		/// <summary>
		/// Response to a <see cref="ReadBlob"/> request.
		/// </summary>
		ReadBlobResponse = 0x21,

		#endregion

		#region Test Requests

		/// <summary>
		/// Xor a block of data with a value
		/// </summary>
		XorRequest = 0xf0,

		/// <summary>
		/// Result from an <see cref="XorRequest"/> request.
		/// </summary>
		XorResponse = 0xf1,

		#endregion
	}

	/// <summary>
	/// Flags describing how to execute a compute task process on the agent
	/// </summary>
	[Flags]
	public enum ExecuteProcessFlags
	{
		/// <summary>
		/// No execute flags set
		/// </summary>
		None = 0,

		/// <summary>
		/// Request execution to be wrapped under Wine when running on Linux.
		/// Agent still reserves the right to refuse it (e.g no Wine executable configured, mismatching OS etc)
		/// </summary>
		UseWine = 1,

		/// <summary>
		/// Use compute process executable as entrypoint for container
		/// If not set, path to the executable is passed as the first parameter to the container invocation
		/// </summary>
		ReplaceContainerEntrypoint = 2,
	}

	/// <summary>
	/// Standard implementation of a message
	/// </summary>
	public sealed class AgentMessage : IMemoryReader, IDisposable
	{
		/// <summary>
		/// Type of the message
		/// </summary>
		public AgentMessageType Type { get; }

		/// <summary>
		/// Data that was read
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		readonly IMemoryOwner<byte> _memoryOwner;
		int _position;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentMessage(AgentMessageType type, ReadOnlyMemory<byte> data)
		{
			_memoryOwner = MemoryPool<byte>.Shared.Rent(data.Length);
			data.CopyTo(_memoryOwner.Memory);

			Type = type;
			Data = _memoryOwner.Memory.Slice(0, data.Length);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_memoryOwner.Dispose();
		}

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> GetMemory(int minSize = 1) => Data.Slice(_position);

		/// <inheritdoc/>
		public void Advance(int length) => _position += length;
	}

	/// <summary>
	/// Exception thrown when an invalid message is received
	/// </summary>
	public sealed class InvalidAgentMessageException : ComputeException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public InvalidAgentMessageException(AgentMessage actualMessage, AgentMessageType? expectedType, ComputeRemoteException? remoteException)
			: base($"Unexpected message {actualMessage.Type}" + (expectedType != null ? $". Wanted {expectedType}" : ""), remoteException)
		{
		}
	}

	/// <summary>
	/// Exception thrown when a compute execution is cancelled
	/// </summary>
	public sealed class ComputeExecutionCancelledException : ComputeException
	{
		private const string Text = "Compute execution cancelled";

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeExecutionCancelledException() : base(Text)
		{
		}

		private ComputeExecutionCancelledException(Exception innerException) : base(Text, innerException)
		{
		}

		/// <summary>
		/// Try constructing and throwing if the exception message matches a cancellation exception
		/// </summary>
		/// <param name="em">Deserialized exception message</param>
		/// <exception cref="ComputeExecutionCancelledException">If message matches</exception>
		public static void TryThrow(ExceptionMessage em)
		{
			if (em.Message == Text)
			{
				throw new ComputeExecutionCancelledException(new ComputeRemoteException(em));
			}
		}
	}

	/// <summary>
	/// Writer for compute messages
	/// </summary>
	public interface IAgentMessageBuilder : IMemoryWriter, IDisposable
	{
		/// <summary>
		/// Sends the current message
		/// </summary>
		void Send();
	}

	/// <summary>
	/// Message for reporting an error
	/// </summary>
	public readonly record struct ExceptionMessage(string Message, string Description);

	/// <summary>
	/// Message requesting that the message loop be forked
	/// </summary>
	/// <param name="ChannelId">New channel to communicate on</param>
	/// <param name="BufferSize">Size of the buffer</param>
	public readonly record struct ForkMessage(int ChannelId, int BufferSize);

	/// <summary>
	/// Extract files from a bundle to a path in the remote sandbox
	/// </summary>
	/// <param name="Name">Path to extract the files to</param>
	/// <param name="Locator">Locator for the tree to extract</param>
	public readonly record struct UploadFilesMessage(string Name, BlobLocator Locator);

	/// <summary>
	/// Deletes files or directories in the remote
	/// </summary>
	/// <param name="Filter">Filter for files to delete</param>
	public readonly record struct DeleteFilesMessage(IReadOnlyList<string> Filter);

	/// <summary>
	/// Message to execute a new child process
	/// </summary>
	/// <param name="Executable">Executable path</param>
	/// <param name="Arguments">Arguments for the executable</param>
	/// <param name="WorkingDir">Working directory to execute in</param>
	/// <param name="EnvVars">Environment variables for the child process. Null values unset variables.</param>
	/// <param name="Flags">Additional execution flags</param>
	/// <param name="ContainerImageUrl">URL to container image. If specified, process will be executed inside this container</param>
	public readonly record struct ExecuteProcessMessage(string Executable, IReadOnlyList<string> Arguments, string? WorkingDir, IReadOnlyDictionary<string, string?> EnvVars, ExecuteProcessFlags Flags, string? ContainerImageUrl);

	/// <summary>
	/// Response from executing a child process
	/// </summary>
	/// <param name="ExitCode">Exit code for the process</param>
	public readonly record struct ExecuteProcessResponseMessage(int ExitCode);

	/// <summary>
	/// Creates a blob read request
	/// </summary>
	public readonly record struct ReadBlobMessage(BlobLocator Locator, int Offset, int Length);

	/// <summary>
	/// Message for running an XOR command
	/// </summary>
	/// <param name="Data">Data to xor</param>
	/// <param name="Value">Value to XOR with</param>
	public readonly record struct XorRequestMessage(ReadOnlyMemory<byte> Data, byte Value);

	/// <summary>
	/// Wraps various requests across compute channels
	/// </summary>
	public static class AgentMessageExtensions
	{
		/// <summary>
		/// Closes the remote message loop
		/// </summary>
		public static async ValueTask CloseAsync(this AgentMessageChannel channel, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.None, cancellationToken);
			message.Send();
		}

		/// <summary>
		/// Sends a ping message to the remote
		/// </summary>
		public static async ValueTask PingAsync(this AgentMessageChannel channel, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.Ping, cancellationToken);
			message.Send();
		}

		/// <summary>
		/// Sends an exception response to the remote
		/// </summary>
		public static ValueTask SendExceptionAsync(this AgentMessageChannel channel, Exception ex, CancellationToken cancellationToken = default) => SendExceptionAsync(channel, ex.Message, ex.ToString(), cancellationToken);

		/// <summary>
		/// Sends an exception response to the remote
		/// </summary>
		public static async ValueTask SendExceptionAsync(this AgentMessageChannel channel, string description, string trace, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.Exception, cancellationToken);
			message.WriteString(description);
			message.WriteString(trace);
			message.Send();
		}

		/// <summary>
		/// Parses a message as an <see cref="ExceptionMessage"/>
		/// </summary>
		public static ExceptionMessage ParseExceptionMessage(this AgentMessage message)
		{
			string msg = message.ReadString();
			string description = message.ReadString();
			return new ExceptionMessage(msg, description);
		}

		/// <summary>
		/// Requests that the remote message loop be forked
		/// </summary>
		public static async ValueTask ForkAsync(this AgentMessageChannel channel, int channelId, int bufferSize, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.Fork, cancellationToken);
			message.WriteInt32(channelId);
			message.WriteInt32(bufferSize);
			message.Send();
		}

		/// <summary>
		/// Parses a fork request message
		/// </summary>
		public static ForkMessage ParseForkMessage(this AgentMessage message)
		{
			int channelId = message.ReadInt32();
			int bufferSize = message.ReadInt32();
			return new ForkMessage(channelId, bufferSize);
		}

		/// <summary>
		/// Notifies the remote that a buffer has been attached
		/// </summary>
		public static async ValueTask AttachAsync(this AgentMessageChannel channel, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.Attach, cancellationToken);
			message.Send();
		}

		/// <summary>
		/// Waits until an attached notification is received along the channel
		/// </summary>
		/// <param name="channel"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public static async ValueTask WaitForAttachAsync(this AgentMessageChannel channel, CancellationToken cancellationToken = default)
		{
			using AgentMessage message = await channel.ReceiveAsync(cancellationToken);
			message.ThrowIfUnexpectedType(AgentMessageType.Attach);
		}

		/// <summary>
		/// Throw an exception if message is not of expected type
		/// </summary>
		/// <param name="message">Agent message to extend</param>
		/// <param name="expectedType">Optional type to expect. If not specified, assume type was unwanted no matter what</param>
		public static void ThrowIfUnexpectedType(this AgentMessage message, AgentMessageType? expectedType = null)
		{
			if (message.Type == expectedType)
			{
				return;
			}

			ComputeRemoteException? cre = message.Type == AgentMessageType.Exception
				? new ComputeRemoteException(message.ParseExceptionMessage())
				: null;

			throw new InvalidAgentMessageException(message, expectedType, cre);
		}

		#region Process

		static async Task<AgentMessage> RunStorageServerAsync(this AgentMessageChannel channel, IStorageBackend storage, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				AgentMessage message = await channel.ReceiveAsync(cancellationToken);
				if (message.Type != AgentMessageType.ReadBlob)
				{
					return message;
				}

				try
				{
					ReadBlobMessage readBlob = message.ParseReadBlobRequest();
					await SendBlobDataAsync(channel, readBlob, storage, cancellationToken);
				}
				finally
				{
					message.Dispose();
				}
			}
		}

		/// <summary>
		/// Creates a sandbox on the remote machine
		/// </summary>
		public static async Task UploadFilesAsync(this AgentMessageChannel channel, string path, BlobLocator locator, IStorageBackend storage, CancellationToken cancellationToken = default)
		{
			using (IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.WriteFiles, cancellationToken))
			{
				request.WriteString(path);
				request.WriteString($"{IoHash.Zero}@{locator}"); // HACK: Currently deployed agents have a hash check in BundleNodeLocator.Parse() which does not check length before checking for the '@' character separating the hash from locator.
				request.Send();
			}

			using AgentMessage response = await RunStorageServerAsync(channel, storage, cancellationToken);
			response.ThrowIfUnexpectedType(AgentMessageType.WriteFilesResponse);
		}

		/// <summary>
		/// Parses a message as a <see cref="UploadFilesMessage"/>
		/// </summary>
		public static UploadFilesMessage ParseUploadFilesMessage(this AgentMessage message)
		{
			string name = message.ReadString();
			string path = message.ReadString();

			int atIdx = path.IndexOf('@', StringComparison.Ordinal);
			if (atIdx != -1)
			{
				path = path.Substring(atIdx + 1);
			}

			BlobLocator locator = new(path);
			return new UploadFilesMessage(name, locator);
		}

		/// <summary>
		/// Destroys a sandbox on the remote machine
		/// </summary>
		/// <param name="channel">Current channel</param>
		/// <param name="paths">Paths of files or directories to delete</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask DeleteFilesAsync(this AgentMessageChannel channel, IReadOnlyList<string> paths, CancellationToken cancellationToken)
		{
			using IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.DeleteFiles, cancellationToken);
			request.WriteList(paths, MemoryWriterExtensions.WriteString);
			request.Send();
		}

		/// <summary>
		/// Parses a message as a <see cref="DeleteFilesMessage"/>
		/// </summary>
		public static DeleteFilesMessage ParseDeleteFilesMessage(this AgentMessage message)
		{
			List<string> files = message.ReadList(MemoryReaderExtensions.ReadString);
			return new DeleteFilesMessage(files);
		}

		/// <summary>
		/// Executes a remote process (using ExecuteV1)
		/// </summary>
		/// <param name="channel">Current channel</param>
		/// <param name="executable">Executable to run, relative to the sandbox root</param>
		/// <param name="arguments">Arguments for the child process</param>
		/// <param name="workingDir">Working directory for the process</param>
		/// <param name="envVars">Environment variables for the child process</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<AgentManagedProcess> ExecuteAsync(this AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, CancellationToken cancellationToken = default)
		{
			using (IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.ExecuteV1, cancellationToken))
			{
				request.WriteString(executable);
				request.WriteList(arguments, MemoryWriterExtensions.WriteString);
				request.WriteOptionalString(workingDir);
				request.WriteDictionary(envVars ?? new Dictionary<string, string?>(), MemoryWriterExtensions.WriteString, MemoryWriterExtensions.WriteOptionalString);
				request.Send();
			}
			return new AgentManagedProcess(channel);
		}

		/// <summary>
		/// Executes a remote process (using ExecuteV2)
		/// </summary>
		/// <param name="channel">Current channel</param>
		/// <param name="executable">Executable to run, relative to the sandbox root</param>
		/// <param name="arguments">Arguments for the child process</param>
		/// <param name="workingDir">Working directory for the process</param>
		/// <param name="envVars">Environment variables for the child process</param>
		/// <param name="flags">Additional execution flags</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<AgentManagedProcess> ExecuteAsync(this AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, ExecuteProcessFlags flags = ExecuteProcessFlags.None, CancellationToken cancellationToken = default)
		{
			using (IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.ExecuteV2, cancellationToken))
			{
				request.WriteString(executable);
				request.WriteList(arguments, MemoryWriterExtensions.WriteString);
				request.WriteOptionalString(workingDir);
				request.WriteDictionary(envVars ?? new Dictionary<string, string?>(), MemoryWriterExtensions.WriteString, MemoryWriterExtensions.WriteOptionalString);
				request.WriteInt32((int)flags);
				request.Send();
			}
			return new AgentManagedProcess(channel);
		}

		/// <summary>
		/// Executes a remote process (using ExecuteV3)
		/// </summary>
		/// <param name="channel">Current channel</param>
		/// <param name="executable">Executable to run, relative to the sandbox root</param>
		/// <param name="arguments">Arguments for the child process</param>
		/// <param name="workingDir">Working directory for the process</param>
		/// <param name="envVars">Environment variables for the child process</param>
		/// <param name="flags">Additional execution flags</param>
		/// <param name="containerImageUrl">Optional container image URL. If set, execution will happen inside this container</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<AgentManagedProcess> ExecuteAsync(this AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, ExecuteProcessFlags flags, string? containerImageUrl, CancellationToken cancellationToken = default)
		{
			using (IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.ExecuteV3, cancellationToken))
			{
				request.WriteString(executable);
				request.WriteList(arguments, MemoryWriterExtensions.WriteString);
				request.WriteOptionalString(workingDir);
				request.WriteDictionary(envVars ?? new Dictionary<string, string?>(), MemoryWriterExtensions.WriteString, MemoryWriterExtensions.WriteOptionalString);
				request.WriteInt32((int)flags);
				request.WriteString(containerImageUrl ?? "");
				request.Send();
			}
			return new AgentManagedProcess(channel);
		}

		/// <summary>
		/// Parses a message as a <see cref="ExecuteProcessMessage"/>
		/// </summary>
		public static ExecuteProcessMessage ParseExecuteProcessV1Message(this AgentMessage message)
		{
			string executable = message.ReadString();
			List<string> arguments = message.ReadList(MemoryReaderExtensions.ReadString);
			string? workingDir = message.ReadOptionalString();
			Dictionary<string, string?> envVars = message.ReadDictionary(MemoryReaderExtensions.ReadString, MemoryReaderExtensions.ReadOptionalString);
			return new ExecuteProcessMessage(executable, arguments, workingDir, envVars, ExecuteProcessFlags.None, null);
		}

		/// <summary>
		/// Parses a message as a <see cref="ExecuteProcessMessage"/>
		/// </summary>
		public static ExecuteProcessMessage ParseExecuteProcessV2Message(this AgentMessage message)
		{
			string executable = message.ReadString();
			List<string> arguments = message.ReadList(MemoryReaderExtensions.ReadString);
			string? workingDir = message.ReadOptionalString();
			Dictionary<string, string?> envVars = message.ReadDictionary(MemoryReaderExtensions.ReadString, MemoryReaderExtensions.ReadOptionalString);
			ExecuteProcessFlags flags = (ExecuteProcessFlags)message.ReadInt32();
			return new ExecuteProcessMessage(executable, arguments, workingDir, envVars, flags, null);
		}

		/// <summary>
		/// Parses a message as a <see cref="ExecuteProcessMessage"/>
		/// </summary>
		public static ExecuteProcessMessage ParseExecuteProcessV3Message(this AgentMessage message)
		{
			string executable = message.ReadString();
			List<string> arguments = message.ReadList(MemoryReaderExtensions.ReadString);
			string? workingDir = message.ReadOptionalString();
			Dictionary<string, string?> envVars = message.ReadDictionary(MemoryReaderExtensions.ReadString, MemoryReaderExtensions.ReadOptionalString);
			ExecuteProcessFlags flags = (ExecuteProcessFlags)message.ReadInt32();
			string containerImageUrl = message.ReadString();
			return new ExecuteProcessMessage(executable, arguments, workingDir, envVars, flags, String.IsNullOrEmpty(containerImageUrl) ? null : containerImageUrl);
		}

		/// <summary>
		/// Sends output from a child process
		/// </summary>
		public static async ValueTask SendExecuteOutputAsync(this AgentMessageChannel channel, ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.ExecuteOutput, data.Length + 20, cancellationToken);
			message.WriteFixedLengthBytes(data.Span);
			message.Send();
		}

		/// <summary>
		/// Sends a response from executing a child process
		/// </summary>
		/// <param name="channel"></param>
		/// <param name="exitCode">Exit code from the process</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask SendExecuteResultAsync(this AgentMessageChannel channel, int exitCode, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder builder = await channel.CreateMessageAsync(AgentMessageType.ExecuteResult, cancellationToken);
			builder.WriteInt32(exitCode);
			builder.Send();
		}

		/// <summary>
		/// Parses a message as a <see cref="ExecuteProcessMessage"/>
		/// </summary>
		public static ExecuteProcessResponseMessage ParseExecuteProcessResponse(this AgentMessage message)
		{
			int exitCode = message.ReadInt32();
			return new ExecuteProcessResponseMessage(exitCode);
		}

		#endregion

		#region Storage

		/// <summary>
		/// 
		/// </summary>
		/// <param name="message"></param>
		/// <returns></returns>
		public static ReadBlobMessage ParseReadBlobRequest(this AgentMessage message)
		{
			BlobLocator locator = new(message.ReadUtf8String());
			int offset = (int)message.ReadUnsignedVarInt();
			int length = (int)message.ReadUnsignedVarInt();
			return new ReadBlobMessage(locator, offset, length);
		}

		/// <summary>
		/// Wraps a compute message containing blob data
		/// </summary>
		sealed class BlobDataStream : ReadOnlyMemoryStream
		{
			readonly AgentMessage _message;

			public BlobDataStream(AgentMessage message)
				: base(message.Data.Slice(8))
			{
				_message = message;
			}

			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				if (disposing)
				{
					_message.Dispose();
				}
			}
		}

		/// <summary>
		/// Reads a blob from the remote
		/// </summary>
		/// <param name="channel">Channel to write to</param>
		/// <param name="path">Path for the blob</param>
		/// <param name="offset">Offset within the blob</param>
		/// <param name="length">Length of data to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Stream containing the blob data</returns>
		public static async Task<ReadOnlyMemory<byte>> ReadBlobAsync(this AgentMessageChannel channel, string path, int offset, int length, CancellationToken cancellationToken = default)
		{
			using (IAgentMessageBuilder request = await channel.CreateMessageAsync(AgentMessageType.ReadBlob, cancellationToken))
			{
				request.WriteString(path);
				request.WriteUnsignedVarInt(offset);
				request.WriteUnsignedVarInt(length);
				request.Send();
			}

			byte[]? buffer = null;
			for (; ; )
			{
				AgentMessage? response = null;
				try
				{
					response = await channel.ReceiveAsync(cancellationToken);
					response.ThrowIfUnexpectedType(AgentMessageType.ReadBlobResponse);

					int chunkOffset = BinaryPrimitives.ReadInt32LittleEndian(response.Data.Span.Slice(0, 4));
					int chunkLength = response.Data.Length - 8;
					int totalLength = BinaryPrimitives.ReadInt32LittleEndian(response.Data.Span.Slice(4, 4));

					buffer ??= new byte[totalLength];
					response.Data.Slice(8).CopyTo(buffer.AsMemory(chunkOffset));

					if (chunkOffset + chunkLength == totalLength)
					{
						break;
					}
				}
				catch
				{
					response?.Dispose();
					throw;
				}
			}

			return buffer;
		}

		/// <summary>
		/// Writes blob data to a compute channel
		/// </summary>
		/// <param name="channel">Channel to write to</param>
		/// <param name="message">The read request</param>
		/// <param name="storage">Storage backend to retrieve the blob from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task SendBlobDataAsync(this AgentMessageChannel channel, ReadBlobMessage message, IStorageBackend storage, CancellationToken cancellationToken = default)
		{
			return SendBlobDataAsync(channel, message.Locator, message.Offset, message.Length, storage, cancellationToken);
		}

		/// <summary>
		/// Writes blob data to a compute channel
		/// </summary>
		/// <param name="channel">Channel to write to</param>
		/// <param name="locator">Locator for the blob to send</param>
		/// <param name="offset">Starting offset of the data</param>
		/// <param name="length">Length of the data</param>
		/// <param name="storage">Storage backend to retrieve the blob from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task SendBlobDataAsync(this AgentMessageChannel channel, BlobLocator locator, int offset, int length, IStorageBackend storage, CancellationToken cancellationToken = default)
		{
			using Stream stream = await storage.OpenBlobAsync(locator, offset, (length == 0) ? null : length, cancellationToken);

			const int MaxChunkSize = 512 * 1024;
			for (int chunkOffset = 0; ;)
			{
				int chunkLength = (int)Math.Min(stream.Length - chunkOffset, MaxChunkSize);
				using (IAgentMessageBuilder response = await channel.CreateMessageAsync(AgentMessageType.ReadBlobResponse, chunkLength + 128, cancellationToken))
				{
					response.WriteInt32(chunkOffset);
					response.WriteInt32((int)stream.Length);

					Memory<byte> memory = response.GetMemoryAndAdvance(chunkLength);
					await stream.ReadFixedLengthBytesAsync(memory, cancellationToken);

					response.Send();
				}

				chunkOffset += chunkLength;
				if (chunkOffset == stream.Length)
				{
					break;
				}
			}
		}

		#endregion

		#region Test Messages

		/// <summary>
		/// Send a message to request that a byte string be xor'ed with a particular value
		/// </summary>
		public static async ValueTask SendXorRequestAsync(this AgentMessageChannel channel, ReadOnlyMemory<byte> data, byte value, CancellationToken cancellationToken = default)
		{
			using IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.XorRequest, cancellationToken);
			message.WriteFixedLengthBytes(data.Span);
			message.WriteUInt8(value);
			message.Send();
		}

		/// <summary>
		/// Parse a message as an XOR request
		/// </summary>
		public static XorRequestMessage AsXorRequest(this AgentMessage message)
		{
			ReadOnlyMemory<byte> data = message.Data;
			return new XorRequestMessage(data[0..^1], data.Span[^1]);
		}

		#endregion
	}
}
