// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute.Buffers;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Implementation of a compute channel
	/// </summary>
	public sealed class AgentMessageChannel : IDisposable
	{
		// Length of a message header. Consists of a 1 byte type field, followed by 4 byte length field.
		const int HeaderLength = 5;

		/// <summary>
		/// Allows creating new messages in rented memory
		/// </summary>
		class MessageBuilder : IAgentMessageBuilder
		{
			readonly AgentMessageChannel _channel;
			readonly ComputeBufferWriter _sendBufferWriter;
			readonly AgentMessageType _type;
			int _length;

			/// <inheritdoc/>
			public int Length => _length;

			public MessageBuilder(AgentMessageChannel channel, ComputeBufferWriter sendBufferWriter, AgentMessageType type)
			{
				_channel = channel;
				_sendBufferWriter = sendBufferWriter;
				_type = type;
				_length = 0;
			}

			public void Dispose()
			{
				if (_channel._currentBuilder == this)
				{
					_channel._currentBuilder = null;
				}
			}

			public void Send()
			{
				Span<byte> header = _sendBufferWriter.GetWriteBuffer().Span;
				header[0] = (byte)_type;
				BinaryPrimitives.WriteInt32LittleEndian(header.Slice(1, 4), _length);

				// if (_channel._logger.IsEnabled(LogLevel.Trace))
				// {
				// 	_channel.LogMessageInfo("SEND", _type, _sendBufferWriter.GetWriteBuffer().Slice(HeaderLength, _length).Span);
				// }

				_sendBufferWriter.AdvanceWritePosition(HeaderLength + _length);
				_length = 0;
			}

			/// <inheritdoc/>
			public void Advance(int count) => _length += count;

			/// <inheritdoc/>
			public Memory<byte> GetMemory(int sizeHint = 0) => _sendBufferWriter.GetWriteBuffer().Slice(HeaderLength + _length);

			/// <inheritdoc/>
			public Span<byte> GetSpan(int sizeHint = 0) => GetMemory(sizeHint).Span;
		}

		readonly int _channelId;
		readonly ComputeProtocol _protocol;
		readonly ComputeBufferReader _recvBufferReader;
		readonly ComputeBufferWriter _sendBufferWriter;

		// Can lock chunked memory writer to acquire pointer
		readonly ILogger _logger;

		MessageBuilder? _currentBuilder;

		/// <summary>
		/// The negotiated compute protocol version number
		/// </summary>
		public ComputeProtocol Protocol => _protocol;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="channelId"></param>
		/// <param name="protocol">Protocol version number</param>
		/// <param name="recvBufferReader"></param>
		/// <param name="sendBufferWriter"></param>
		/// <param name="logger">Logger for diagnostic output</param>
		public AgentMessageChannel(int channelId, ComputeProtocol protocol, ComputeBufferReader recvBufferReader, ComputeBufferWriter sendBufferWriter, ILogger logger)
		{
			_channelId = channelId;
			_protocol = protocol;
			_recvBufferReader = recvBufferReader.AddRef();
			_sendBufferWriter = sendBufferWriter.AddRef();
			_logger = logger;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="socket"></param>
		/// <param name="channelId"></param>
		/// <param name="recvBuffer"></param>
		/// <param name="sendBuffer"></param>
		/// <param name="logger">Logger for diagnostic output</param>
		public AgentMessageChannel(ComputeSocket socket, int channelId, ComputeBuffer recvBuffer, ComputeBuffer sendBuffer, ILogger logger)
		{
			socket.AttachRecvBuffer(channelId, recvBuffer);
			socket.AttachSendBuffer(channelId, sendBuffer);
			_channelId = channelId;
			_protocol = socket.Protocol;
			_recvBufferReader = recvBuffer.CreateReader();
			_sendBufferWriter = sendBuffer.CreateWriter();
			_logger = logger;
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		public void Dispose()
		{
			_currentBuilder?.Dispose();

			_sendBufferWriter.Dispose();
			_recvBufferReader.Dispose();
		}

		/// <summary>
		/// Mark the send buffer as complete
		/// </summary>
		public void MarkComplete()
		{
			_sendBufferWriter.MarkComplete();
		}

		/// <inheritdoc/>
		public async ValueTask<AgentMessage> ReceiveAsync(CancellationToken cancellationToken)
		{
			while (!_recvBufferReader.IsComplete)
			{
				ReadOnlyMemory<byte> memory = _recvBufferReader.GetReadBuffer();
				if (memory.Length < HeaderLength)
				{
					await _recvBufferReader.WaitToReadAsync(HeaderLength, cancellationToken);
					continue;
				}

				int messageLength = BinaryPrimitives.ReadInt32LittleEndian(memory.Span.Slice(1, 4));
				if (memory.Length < HeaderLength + messageLength)
				{
					await _recvBufferReader.WaitToReadAsync(HeaderLength + messageLength, cancellationToken);
					continue;
				}

				AgentMessageType type = (AgentMessageType)memory.Span[0];
				AgentMessage message = new(type, memory.Slice(HeaderLength, messageLength));
				// if (_logger.IsEnabled(LogLevel.Trace))
				// {
				// 	LogMessageInfo("RECV", message.Type, message.Data.Span);
				// }

				_recvBufferReader.AdvanceReadPosition(HeaderLength + messageLength);
				return message;
			}
			return new AgentMessage(AgentMessageType.None, ReadOnlyMemory<byte>.Empty);
		}

		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Log calls disabled")]
		void LogMessageInfo(string verb, AgentMessageType type, ReadOnlySpan<byte> data)
		{
			StringBuilder bytes = new();
			for (int offset = 0; offset < 16 && offset < data.Length; offset++)
			{
				bytes.Append($" {data[offset]:X2}");
			}
			if (data.Length > 16)
			{
				bytes.Append("..");
			}
			_logger.LogTrace("{Verb} {ChannelId} {Type,-18} [{Length,10:n0}] = {Bytes}", verb, _channelId, type, data.Length, bytes.ToString());
		}

		/// <inheritdoc/>
		public async ValueTask<IAgentMessageBuilder> CreateMessageAsync(AgentMessageType type, int maxSize, CancellationToken cancellationToken)
		{
			if (_currentBuilder != null)
			{
				throw new InvalidOperationException("Only one writer can be active at a time. Dispose of the previous writer first.");
			}

			await _sendBufferWriter.WaitToWriteAsync(maxSize, cancellationToken);

			_currentBuilder = new MessageBuilder(this, _sendBufferWriter, type);
			return _currentBuilder;
		}
	}

	/// <summary>
	/// Extension methods to allow creating channels from leases
	/// </summary>
	public static class AgentMessageChannelExtensions
	{
		/// <summary>
		/// Creates a message channel with the given identifier
		/// </summary>
		/// <param name="socket">Socket to create a channel for</param>
		/// <param name="channelId">Identifier for the channel</param>
		public static AgentMessageChannel CreateAgentMessageChannel(this ComputeSocket socket, int channelId)
			=> socket.CreateAgentMessageChannel(channelId, 65536);

		/// <summary>
		/// Creates a message channel with the given identifier
		/// </summary>
		/// <param name="socket">Socket to create a channel for</param>
		/// <param name="channelId">Identifier for the channel</param>
		/// <param name="bufferSize">Size of the send and receive buffer</param>
		public static AgentMessageChannel CreateAgentMessageChannel(this ComputeSocket socket, int channelId, int bufferSize)
			=> socket.CreateAgentMessageChannel(channelId, bufferSize, bufferSize);

		/// <summary>
		/// Creates a message channel with the given identifier
		/// </summary>
		/// <param name="socket">Socket to create a channel for</param>
		/// <param name="channelId">Identifier for the channel</param>
		/// <param name="sendBufferSize">Size of the send buffer</param>
		/// <param name="recvBufferSize">Size of the receive buffer</param>
		public static AgentMessageChannel CreateAgentMessageChannel(this ComputeSocket socket, int channelId, int sendBufferSize, int recvBufferSize)
		{
			using ComputeBuffer sendBuffer = new PooledBuffer(sendBufferSize);
			using ComputeBuffer recvBuffer = new PooledBuffer(recvBufferSize);
			return new AgentMessageChannel(socket, channelId, sendBuffer, recvBuffer, socket.Logger);
		}

		/// <summary>
		/// Reads a message from the channel
		/// </summary>
		/// <param name="channel">Channel to receive on</param>
		/// <param name="type">Expected type of the message</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Data for a message that was read. Must be disposed.</returns>
		public static async ValueTask<AgentMessage> ReceiveAsync(this AgentMessageChannel channel, AgentMessageType type, CancellationToken cancellationToken = default)
		{
			AgentMessage message = await channel.ReceiveAsync(cancellationToken);
			message.ThrowIfUnexpectedType(type);
			return message;
		}

		/// <summary>
		/// Creates a new builder for a message
		/// </summary>
		/// <param name="channel">Channel to send on</param>
		/// <param name="type">Type of the message</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New builder for messages</returns>
		public static ValueTask<IAgentMessageBuilder> CreateMessageAsync(this AgentMessageChannel channel, AgentMessageType type, CancellationToken cancellationToken)
		{
			return channel.CreateMessageAsync(type, 1024, cancellationToken);
		}

		/// <summary>
		/// Forwards an existing message across a channel
		/// </summary>
		/// <param name="channel">Channel to send on</param>
		/// <param name="message">The message to be sent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask SendAsync(this AgentMessageChannel channel, AgentMessage message, CancellationToken cancellationToken)
		{
			using (IAgentMessageBuilder builder = await channel.CreateMessageAsync(message.Type, message.Data.Length, cancellationToken))
			{
				builder.WriteFixedLengthBytes(message.Data.Span);
				builder.Send();
			}
		}
	}
}
