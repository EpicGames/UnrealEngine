// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.ExceptionServices;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute.Buffers;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Implements the remote end of a compute worker. 
	/// </summary>
	public class AgentMessageHandler
	{
		readonly DirectoryReference _sandboxDir;
		readonly Dictionary<string, string?> _envVars;
		readonly bool _executeInProcess;
		readonly string? _wineExecutablePath;
		readonly string? _containerEngineExecutable;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="sandboxDir">Directory to use for reading/writing files</param>
		/// <param name="envVars">Environment variables to set for any child processes</param>
		/// <param name="executeInProcess">Whether to execute any external assemblies in the current process</param>
		/// <param name="wineExecutablePath">Path to Wine executable. If null, execution under Wine is disabled</param>
		/// <param name="containerEngineExecutable">Path to container engine executable, e.g /usr/bin/podman. If null, execution inside a container is disabled</param>
		/// <param name="logger">Logger for diagnostics</param>
		public AgentMessageHandler(DirectoryReference sandboxDir, Dictionary<string, string?>? envVars, bool executeInProcess, string? wineExecutablePath, string? containerEngineExecutable, ILogger logger)
		{
			_sandboxDir = sandboxDir;
			_envVars = envVars ?? [];
			_executeInProcess = executeInProcess;
			_wineExecutablePath = wineExecutablePath;
			_containerEngineExecutable = containerEngineExecutable;
			_logger = logger;
		}

		/// <summary>
		/// Runs the worker using commands sent along the given socket
		/// </summary>
		/// <param name="socket">Socket to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task RunAsync(ComputeSocket socket, CancellationToken cancellationToken)
		{
			// Since we allow forking message channels, we want to ensure that errors on one channel are propagated back here, and terminate the whole connection. 
			// To do that, we take first exception thrown and rethrow it with the original callstack here, while also forcing all other tasks to terminate via a 
			// shared cancellation token.
			using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
			ExceptionDispatchInfo? exceptionInfo = null;

			async Task PostExceptionAsync(Exception ex)
			{
				// Capture stack from call site
				Interlocked.CompareExchange(ref exceptionInfo, ExceptionDispatchInfo.Capture(ex), null);
				await cancellationSource.CancelAsync();
			}

			await RunAsync(socket, 0, 4 * 1024 * 1024, PostExceptionAsync, cancellationSource.Token);

			// Throw the regular cancellation exception if requested
			cancellationToken.ThrowIfCancellationRequested();

			// Otherwise throw any exception posted by a child task
#pragma warning disable CA1508 // Static analyzer doesn't understand how this can be non-null
			exceptionInfo?.Throw();
#pragma warning restore CA1508
		}

		async Task RunAsync(ComputeSocket socket, int channelId, int bufferSize, Func<Exception, Task> postException, CancellationToken cancellationToken)
		{
			List<Task> childTasks = [];
			using AgentMessageChannel channel = socket.CreateAgentMessageChannel(channelId, bufferSize);
			try
			{
				await channel.AttachAsync(cancellationToken);

				for (; ; )
				{
					using AgentMessage message = await channel.ReceiveAsync(cancellationToken);
					_logger.LogDebug("Compute Channel {ChannelId}: {MessageType}", channelId, message.Type);

					switch (message.Type)
					{
						case AgentMessageType.None:
							return;
						case AgentMessageType.Ping:
							await channel.PingAsync(cancellationToken);
							break;
						case AgentMessageType.Fork:
							{
								ForkMessage fork = message.ParseForkMessage();
								childTasks.Add(Task.Run(() => RunAsync(socket, fork.ChannelId, fork.BufferSize, postException, cancellationToken), cancellationToken));
							}
							break;
						case AgentMessageType.WriteFiles:
							{
								UploadFilesMessage writeFiles = message.ParseUploadFilesMessage();
								await WriteFilesAsync(channel, writeFiles.Name, writeFiles.Locator, cancellationToken: cancellationToken);
							}
							break;
						case AgentMessageType.DeleteFiles:
							{
								DeleteFilesMessage deleteFiles = message.ParseDeleteFilesMessage();
								DeleteFiles(deleteFiles.Filter);
							}
							break;
						case AgentMessageType.ExecuteV1:
							{
								ExecuteProcessMessage ep = message.ParseExecuteProcessV1Message();
								await ExecuteProcessAsync(socket, channel, ep.Executable, ep.Arguments, ep.WorkingDir, ep.ContainerImageUrl, ep.EnvVars, ep.Flags, cancellationToken);
							}
							break;
						case AgentMessageType.ExecuteV2:
							{
								ExecuteProcessMessage ep = message.ParseExecuteProcessV2Message();
								await ExecuteProcessAsync(socket, channel, ep.Executable, ep.Arguments, ep.WorkingDir, ep.ContainerImageUrl, ep.EnvVars, ep.Flags, cancellationToken);
							}
							break;
						case AgentMessageType.ExecuteV3:
							{
								ExecuteProcessMessage ep = message.ParseExecuteProcessV3Message();
								await ExecuteProcessAsync(socket, channel, ep.Executable, ep.Arguments, ep.WorkingDir, ep.ContainerImageUrl, ep.EnvVars, ep.Flags, cancellationToken);
							}
							break;
						case AgentMessageType.XorRequest:
							{
								XorRequestMessage xorRequest = message.AsXorRequest();
								await RunXorAsync(channel, xorRequest.Data, xorRequest.Value, cancellationToken);
							}
							break;
						default:
							message.ThrowIfUnexpectedType();
							return;
					}
				}
			}
			catch (OperationCanceledException ex)
			{
				// Ignore cancellations; we will re-throw from the root RunAsync() method.
				_logger.LogDebug(ex, "Compute Channel {ChannelId}: Cancelled.", channelId);
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Compute Channel {ChannelId}: Exception: {Message}", channelId, ex.Message);
				await channel.SendExceptionAsync(ex, cancellationToken);
				await postException(ex);
			}
			finally
			{
				await Task.WhenAll(childTasks);
			}
		}

		static async ValueTask RunXorAsync(AgentMessageChannel channel, ReadOnlyMemory<byte> source, byte value, CancellationToken cancellationToken)
		{
			using IAgentMessageBuilder response = await channel.CreateMessageAsync(AgentMessageType.XorResponse, source.Length, cancellationToken);
			XorData(source.Span, response.GetSpanAndAdvance(source.Length), value);
			response.Send();
		}

		static void XorData(ReadOnlySpan<byte> source, Span<byte> target, byte value)
		{
			for (int idx = 0; idx < source.Length; idx++)
			{
				target[idx] = (byte)(source[idx] ^ value);
			}
		}

		async Task WriteFilesAsync(AgentMessageChannel channel, string path, BlobLocator locator, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default)
		{
			using AgentStorageBackend innerStore = new(channel);
			await using BundleCache cache = new(new BundleCacheOptions { HeaderCacheSize = 10 * 1024 * 1024, PacketCacheSize = 128 * 1024 * 1024 });

			BundleOptions bundleOptions = ComputeProtocolUtilities.GetBundleOptions(channel.Protocol);
			BundleStorageNamespace store = new(innerStore, cache, bundleOptions, _logger);

			IBlobRef<DirectoryNode> directoryRef = store.CreateBlobRef<DirectoryNode>(locator, options);

			DirectoryReference outputDir = DirectoryReference.Combine(_sandboxDir, path);
			if (!outputDir.IsUnderDirectory(_sandboxDir))
			{
				throw new InvalidOperationException("Cannot write files outside sandbox");
			}

			await directoryRef.ExtractAsync(outputDir.ToDirectoryInfo(), new ExtractOptions(), _logger, cancellationToken);
			await VerifyFilesAsync(outputDir, directoryRef, cancellationToken);

			using (IAgentMessageBuilder message = await channel.CreateMessageAsync(AgentMessageType.WriteFilesResponse, cancellationToken))
			{
				message.Send();
			}
		}

		async Task<bool> VerifyFilesAsync(DirectoryReference outputDir, IBlobRef<DirectoryNode> directoryRef, CancellationToken cancellationToken = default)
		{
			bool result = true;

			DirectoryNode directoryNode = await directoryRef.ReadBlobAsync(cancellationToken);
			foreach (FileEntry fileEntry in directoryNode.Files)
			{
				FileReference file = FileReference.Combine(outputDir, fileEntry.Name);
				if (!FileReference.Exists(file))
				{
					_logger.LogError("Extracted file {File} does not exist", file);
					result = false;
				}
				else
				{
					await using FileStream stream = FileReference.Open(file, FileMode.Open, FileAccess.Read);
					IoHash hash = await IoHash.ComputeAsync(stream, cancellationToken);

					if (hash == fileEntry.Hash)
					{
						_logger.LogInformation("Hash of {File} is correct ({Hash})", file, hash);
					}
					if (hash != fileEntry.Hash)
					{
						_logger.LogError("Hash mismatch for {File}; expected {ExpectedHash}, got {ActualHash}", file, fileEntry.Hash, hash);
						result = false;
					}
				}
			}

			foreach (DirectoryEntry directoryEntry in directoryNode.Directories)
			{
				result &= await VerifyFilesAsync(DirectoryReference.Combine(outputDir, directoryEntry.Name), directoryEntry.Handle, cancellationToken);
			}

			return result;
		}

		void DeleteFiles(IReadOnlyList<string> deleteFiles)
		{
			FileFilter filter = new(deleteFiles);

			List<FileReference> files = filter.ApplyToDirectory(_sandboxDir, false);
			foreach (FileReference file in files)
			{
				FileUtils.ForceDeleteFile(file);
			}
		}

		async Task ExecuteProcessAsync(
			ComputeSocket socket,
			AgentMessageChannel channel,
			string executable,
			IReadOnlyList<string> arguments,
			string? workingDir,
			string? containerImageUrl,
			IReadOnlyDictionary<string, string?>? envVars,
			ExecuteProcessFlags flags,
			CancellationToken cancellationToken)
		{
			try
			{
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					await ExecuteProcessWindowsAsync(socket, channel, executable, arguments, workingDir, envVars, flags, cancellationToken);
				}
				else if (containerImageUrl != null)
				{
					await ExecuteProcessInContainerAsync(channel, executable, arguments, workingDir, containerImageUrl, envVars, flags, cancellationToken);
				}
				else
				{
					await ExecuteProcessInternalAsync(channel, executable, arguments, workingDir, envVars, flags, cancellationToken);
				}
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				_logger.LogInformation("Compute process execution cancelled");
				await channel.SendExceptionAsync(new ComputeExecutionCancelledException(), cancellationToken);
			}
			catch (Exception ex)
			{
				await channel.SendExceptionAsync(ex, cancellationToken);
			}
		}

		async Task ExecuteProcessWindowsAsync(ComputeSocket socket, AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, ExecuteProcessFlags flags, CancellationToken cancellationToken)
		{
			Dictionary<string, string?> newEnvVars = new(_envVars);
			if (envVars != null)
			{
				foreach ((string name, string? value) in envVars)
				{
					newEnvVars.Add(name, value);
				}
			}

			await using (WorkerComputeSocketBridge server = await WorkerComputeSocketBridge.CreateAsync(socket, _logger))
			{
				newEnvVars[WorkerComputeSocket.IpcEnvVar] = server.BufferName;

				_logger.LogInformation("Launching {Executable} {Arguments}", CommandLineArguments.Quote(executable), CommandLineArguments.Join(arguments));

				await ExecuteProcessInternalAsync(channel, executable, arguments, workingDir, newEnvVars, flags, cancellationToken);
				_logger.LogInformation("Finished executing process");
			}

			_logger.LogInformation("Child process has shut down");
		}

		internal static async Task ProcessIpcMessagesAsync(ComputeSocket socket, ComputeBufferReader ipcReader, CancellationToken[] cancellationTokens, ILogger logger)
		{
			using CancellationTokenSource cancellationTokenSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationTokens);
			CancellationToken cancellationToken = cancellationTokenSource.Token;

			List<SharedMemoryBuffer> buffers = [];
			try
			{
				List<(int, ComputeBufferWriter)> writers = [];
				while (await ipcReader.WaitToReadAsync(1, cancellationToken))
				{
					ReadOnlyMemory<byte> memory = ipcReader.GetReadBuffer();
					MemoryReader reader = new(memory);

					IpcMessage message = (IpcMessage)reader.ReadUnsignedVarInt();
					try
					{
						switch (message)
						{
							case IpcMessage.AttachSendBuffer:
								{
									int channelId = (int)reader.ReadUnsignedVarInt();
									string name = reader.ReadString();
									logger.LogDebug("Attaching send buffer for channel {ChannelId} to {Name}", channelId, name);

									SharedMemoryBuffer buffer = SharedMemoryBuffer.OpenExisting(name);
									buffers.Add(buffer);

									socket.AttachSendBuffer(channelId, buffer);
								}
								break;
							case IpcMessage.AttachRecvBuffer:
								{
									int channelId = (int)reader.ReadUnsignedVarInt();
									string name = reader.ReadString();
									logger.LogDebug("Attaching recv buffer for channel {ChannelId} to {Name}", channelId, name);

									SharedMemoryBuffer buffer = SharedMemoryBuffer.OpenExisting(name);
									buffers.Add(buffer);

									socket.AttachRecvBuffer(channelId, buffer);
								}
								break;
							default:
								throw new InvalidOperationException($"Invalid IPC message: {message}");
						}
					}
					catch (Exception ex)
					{
						logger.LogError(ex, "Exception while processing messages from child process: {Message}", ex.Message);
					}

					ipcReader.AdvanceReadPosition(memory.Length - reader.RemainingMemory.Length);
				}
			}
			catch (OperationCanceledException)
			{
				logger.LogDebug("Ipc message loop cancelled");
			}
			finally
			{
				foreach (SharedMemoryBuffer buffer in buffers)
				{
					buffer.Dispose();
				}
			}
		}

		// Helper class to take raw UTF8 output and merge it into log lines
		class ProcessOutputWriter
		{
			readonly string _prefix;
			readonly ByteArrayBuilder _lineBuffer = new();
			readonly ILogger _logger;

			public ProcessOutputWriter(string prefix, ILogger logger)
			{
				_prefix = prefix;
				_logger = logger;
			}

			public void WriteBytes(ReadOnlySpan<byte> span)
			{
				for (; ; )
				{
					int newlineIdx = span.IndexOf((byte)'\n');
					if (newlineIdx == -1)
					{
						_lineBuffer.WriteFixedLengthBytes(span);
						break;
					}

					ReadOnlySpan<byte> line = span.Slice(0, newlineIdx);
					if (line.Length > 0 && line[^1] == (byte)'\r')
					{
						line = line.Slice(0, line.Length - 1);
					}

					if (_lineBuffer.Length > 0)
					{
						_lineBuffer.WriteFixedLengthBytes(line);
						_logger.LogInformation("{Prefix}: {Line}", _prefix, Encoding.UTF8.GetString(_lineBuffer.AsMemory().Span));
						_lineBuffer.Clear();
					}
					else
					{
						_logger.LogInformation("{Prefix}: {Line}", _prefix, Encoding.UTF8.GetString(line));
					}

					span = span.Slice(newlineIdx + 1);
				}
			}
		}

		async Task ExecuteProcessAssemblyAsync(AgentMessageChannel channel, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, CancellationToken cancellationToken)
		{
			List<(string, string?)> prevEnvVars = [];
			if (envVars != null)
			{
				foreach ((string key, string? value) in envVars)
				{
					prevEnvVars.Add((key, Environment.GetEnvironmentVariable(key)));
					Environment.SetEnvironmentVariable(key, value);
				}
			}

			string prevWorkingDir = Directory.GetCurrentDirectory();
			Directory.SetCurrentDirectory(GetWorkingDirAbsPath(workingDir));

			try
			{
				string assemblyPath = FileReference.Combine(_sandboxDir, arguments[0]).FullName;
				string[] mainArgs = [.. arguments.Skip(1)];

				_logger.LogWarning("Note: Loading and running {Assembly} in process", assemblyPath);

				TaskCompletionSource<int> resultTcs = new(TaskCreationOptions.RunContinuationsAsynchronously);

				Thread thread = new(() => resultTcs.SetResult(AppDomain.CurrentDomain.ExecuteAssembly(assemblyPath, mainArgs)));
				thread.Start();

				int result = await resultTcs.Task;
				await channel.SendExecuteResultAsync(result, cancellationToken);
			}
			finally
			{
				Directory.SetCurrentDirectory(prevWorkingDir);
				foreach ((string key, string? value) in prevEnvVars)
				{
					Environment.SetEnvironmentVariable(key, value);
				}
			}
		}

		async Task ExecuteProcessInContainerAsync(AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, string containerImageUrl, IReadOnlyDictionary<string, string?>? envVars, ExecuteProcessFlags flags, CancellationToken cancellationToken)
		{
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				throw new Exception("Only Linux is supported for executing a process inside a container");
			}

			if (_containerEngineExecutable == null)
			{
				throw new Exception("Container execution requested but agent has no container engine configured");
			}

			string resolvedExecutable = FileReference.Combine(_sandboxDir, executable).FullName;
			uint linuxUid = getuid();
			uint linuxGid = getgid();

			// Resolve env vars here even if they are resolved later in ExecuteProcessInternalAsync
			// The environment file must be written at this step
			Dictionary<string, string> resolvedEnvVars = ResolveEnvVars(envVars);
			string envFilePath = Path.GetTempFileName();
			StringBuilder sb = new();
			foreach ((string key, string value) in resolvedEnvVars)
			{
				sb.AppendLine($"{key}={value}");
			}
			await File.WriteAllTextAsync(envFilePath, sb.ToString(), cancellationToken);

			List<string> resolvedArguments =
			[
				"run",
				"--tty", // Allocate a pseudo-TTY
				"--rm", // Ensure container is removed after run
				$"--user={linuxUid}:{linuxGid}", // Run container as current user (important for mounted dirs)
				$"--volume={_sandboxDir}:{_sandboxDir}:rw",
				"--env-file=" + envFilePath,
			];

			if (flags.HasFlag(ExecuteProcessFlags.ReplaceContainerEntrypoint))
			{
				resolvedArguments.Add("--entrypoint=" + resolvedExecutable);
				resolvedArguments.Add(containerImageUrl);
			}
			else
			{
				resolvedArguments.Add(containerImageUrl);
				resolvedArguments.Add(resolvedExecutable); // Add executable as first argument and assume the entrypoint inside the container image will handle this
			}

			resolvedArguments.AddRange(arguments);
			_logger.LogInformation("Executing {File} {Arguments} in container", _containerEngineExecutable, arguments);

			// Skip forwarding of env vars as they are explicitly set above as arguments to container run
			await ExecuteProcessInternalAsync(channel, _containerEngineExecutable, resolvedArguments, workingDir, new Dictionary<string, string?>(), flags, cancellationToken);
		}

		async Task ExecuteProcessInternalAsync(AgentMessageChannel channel, string executable, IReadOnlyList<string> arguments, string? workingDir, IReadOnlyDictionary<string, string?>? envVars, ExecuteProcessFlags flags, CancellationToken cancellationToken)
		{
			string resolvedExecutable = GetExecutableAbsPath(executable);
			string resolvedWorkingDir = GetWorkingDirAbsPath(workingDir);

			if (_executeInProcess && Path.GetFileNameWithoutExtension(resolvedExecutable).Equals("dotnet", StringComparison.OrdinalIgnoreCase))
			{
				await ExecuteProcessAssemblyAsync(channel, arguments, workingDir, envVars, cancellationToken);
			}
			else
			{
				string resolvedCommandLine = CommandLineArguments.Join(arguments);

				if (flags.HasFlag(ExecuteProcessFlags.UseWine) && _wineExecutablePath != null)
				{
					// Path to the original Windows executable is prepended to the argument list so Wine can run it
					resolvedCommandLine = CommandLineArguments.Join([resolvedExecutable, .. arguments]);
					resolvedExecutable = _wineExecutablePath;
				}

				Dictionary<string, string> resolvedEnvVars = ResolveEnvVars(envVars);
				if (!File.Exists(resolvedExecutable))
				{
					_logger.LogWarning("Executable {Path} does not exist", resolvedExecutable);
				}

				if (!Directory.Exists(resolvedWorkingDir))
				{
					_logger.LogWarning("Working dir {Path} does not exist", resolvedWorkingDir);
				}

				using ManagedProcessGroup group = new();
				using ManagedProcess process = new(group, resolvedExecutable, resolvedCommandLine, resolvedWorkingDir, resolvedEnvVars, null, ProcessPriorityClass.Normal);
				byte[] buffer = new byte[1024];

				ProcessOutputWriter outputWriter = new($"{Path.GetFileNameWithoutExtension(resolvedExecutable)}> ", _logger);
				for (; ; )
				{
					// Use WaitAsync() as ReadAsync() does not respect the cancellation token when reading
					int length = await process.ReadAsync(buffer, 0, buffer.Length, cancellationToken).AsTask().WaitAsync(cancellationToken);
					if (length == 0)
					{
						await process.WaitForExitAsync(cancellationToken);
						await channel.SendExecuteResultAsync(process.ExitCode, cancellationToken);
						return;
					}

					ReadOnlyMemory<byte> output = buffer.AsMemory(0, length);
					await channel.SendExecuteOutputAsync(output, cancellationToken);

					outputWriter.WriteBytes(output.Span);
				}
			}
		}

		private string GetExecutableAbsPath(string relPath)
		{
			return FileReference.Combine(_sandboxDir, relPath).FullName;
		}

		private string GetWorkingDirAbsPath(string? relPath)
		{
			return DirectoryReference.Combine(_sandboxDir, relPath ?? String.Empty).FullName;
		}

		/// <summary>
		/// Flattens and merges available env vars to be used for compute process execution
		/// </summary>
		/// <param name="envVars">Optional extra env vars</param>
		/// <returns>Merged environment variables</returns>
		private Dictionary<string, string> ResolveEnvVars(IReadOnlyDictionary<string, string?>? envVars)
		{
			Dictionary<string, string> resolvedEnvVars = ManagedProcess.GetCurrentEnvVars();

			foreach ((string key, string? value) in _envVars)
			{
				if (value != null)
				{
					resolvedEnvVars[key] = value;
				}
			}

			if (envVars != null)
			{
				foreach ((string key, string? value) in envVars)
				{
					if (value == null)
					{
						resolvedEnvVars.Remove(key);
					}
					else
					{
						resolvedEnvVars[key] = value;
					}
				}
			}

			return resolvedEnvVars;
		}

		/// <summary>
		/// Get user identity (Linux only)
		/// </summary>
		/// <returns>Real user ID of the calling process</returns>
		[DllImport("libc", SetLastError = true)]
		internal static extern uint getuid();

		/// <summary>
		/// Get group identity (Linux only)
		/// </summary>
		/// <returns>Real group ID of the calling process</returns>
		[DllImport("libc", SetLastError = true)]
		internal static extern uint getgid();
	}
}
