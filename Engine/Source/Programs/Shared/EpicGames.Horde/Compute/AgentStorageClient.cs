// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Storage backend which can read bundles over a compute channel
	/// </summary>
	public sealed class AgentStorageBackend : IStorageBackend, IDisposable
	{
		readonly AgentMessageChannel _channel;
		readonly SemaphoreSlim _semaphore;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="channel"></param>
		public AgentStorageBackend(AgentMessageChannel channel)
		{
			_channel = channel;
			_semaphore = new SemaphoreSlim(1);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_semaphore.Dispose();
		}

		#region Blobs

		/// <inheritdoc/>
		public async Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = await ReadBlobInternalAsync(locator, offset, length, cancellationToken);
			return new ReadOnlyMemoryStream(data);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = await ReadBlobInternalAsync(locator, offset, length, cancellationToken);
			return ReadOnlyMemoryOwner.Create(data);
		}

		async ValueTask<ReadOnlyMemory<byte>> ReadBlobInternalAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			await _semaphore.WaitAsync(cancellationToken);
			try
			{
				// TODO: Want to pass 0 for length to read here (meaning entire blob), but older streams misinterpret this as a 0 byte read.
				return await _channel.ReadBlobAsync(locator.ToString(), offset, length ?? 128 * 1024 * 1024, cancellationToken);
			}
			finally
			{
				_semaphore.Release();
			}
		}

		/// <inheritdoc/>
		public Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException();

		/// <inheritdoc/>
		public Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? basePath = null, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException();

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator>? imports = null, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator>? imports = null, string? prefix = null, CancellationToken cancellationToken = default)
			=> default;

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public Task AddAliasAsync(string name, BlobLocator locator, int rank, ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		/// <inheritdoc/>
		public Task RemoveAliasAsync(string name, BlobLocator locator, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		/// <inheritdoc/>
		public Task<BlobAliasLocator[]> FindAliasesAsync(string name, int? maxResults, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		#endregion

		#region Refs

		/// <inheritdoc/>
		public Task<bool> DeleteRefAsync(RefName name, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		/// <inheritdoc/>
		public Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		/// <inheritdoc/>
		public Task WriteRefAsync(RefName name, HashedBlobRefValue value, RefOptions? options = null, CancellationToken cancellationToken = default) => throw new NotSupportedException();

		#endregion

		/// <inheritdoc/>
		public async Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
		{
			foreach (AddAliasRequest addAlias in request.AddAliases)
			{
				await AddAliasAsync(addAlias.Name, addAlias.Target, addAlias.Rank, addAlias.Data, cancellationToken);
			}
			foreach (RemoveAliasRequest removeAlias in request.RemoveAliases)
			{
				await RemoveAliasAsync(removeAlias.Name, removeAlias.Target, cancellationToken);
			}
			foreach (AddRefRequest addRef in request.AddRefs)
			{
				await WriteRefAsync(addRef.RefName, new HashedBlobRefValue(addRef.Hash, addRef.Target), addRef.Options, cancellationToken);
			}
			foreach (RemoveRefRequest removeRef in request.RemoveRefs)
			{
				await DeleteRefAsync(removeRef.RefName, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
		}
	}
}
