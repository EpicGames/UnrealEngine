// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Compute.Buffers
{
	/// <summary>
	/// In-process buffer used to store compute messages
	/// </summary>
	public sealed class PooledBuffer : ComputeBuffer
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="capacity">Total capacity of the buffer</param>
		public PooledBuffer(int capacity)
			: this(2, capacity / 2)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="numChunks">Number of chunks in the buffer</param>
		/// <param name="chunkLength">Length of each chunk</param>
		/// <param name="numReaders">Number of readers for this buffer</param>
		public PooledBuffer(int numChunks, int chunkLength, int numReaders = 1)
			: base(PooledBufferDetail.Create(numChunks, chunkLength, numReaders))
		{
		}

		private PooledBuffer(ComputeBufferDetail resources)
			: base(resources)
		{
		}

		/// <inheritdoc/>
		public override PooledBuffer AddRef()
		{
			_detail.AddRef();
			return new PooledBuffer(_detail);
		}
	}

	/// <summary>
	/// Core implementation of <see cref="PooledBuffer"/>
	/// </summary>
	class PooledBufferDetail : ComputeBufferDetail
	{
		GCHandle _headerHandle;
		IMemoryOwner<byte>[] _chunkOwners;

		readonly AsyncEvent _writerEvent = new();
		readonly AsyncEvent _readerEvent = new();

		internal PooledBufferDetail(HeaderPtr headerPtr, GCHandle headerHandle, Memory<byte>[] chunks, IMemoryOwner<byte>[] chunkOwners)
			: base(headerPtr, chunks)
		{
			_headerHandle = headerHandle;
			_chunkOwners = chunkOwners;
		}

		public static unsafe PooledBufferDetail Create(int numChunks, int chunkLength, int numReaders)
		{
			byte[] header = new byte[HeaderSize];
			GCHandle headerHandle = GCHandle.Alloc(header, GCHandleType.Pinned);
			HeaderPtr headerPtr = new((ulong*)headerHandle.AddrOfPinnedObject().ToPointer(), numReaders, numChunks, chunkLength);

			Memory<byte>[] chunks = new Memory<byte>[numChunks];
			IMemoryOwner<byte>[] chunkOwners = new IMemoryOwner<byte>[numChunks];

			for (int idx = 0; idx < numChunks; idx++)
			{
				chunkOwners[idx] = MemoryPool<byte>.Shared.Rent(chunkLength);
				chunks[idx] = chunkOwners[idx].Memory.Slice(0, chunkLength);
			}

			return new PooledBufferDetail(headerPtr, headerHandle, chunks, chunkOwners);
		}

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				for (int idx = 0; idx < _chunkOwners.Length; idx++)
				{
					_chunkOwners[idx].Dispose();
				}
				_chunkOwners = [];

				_headerHandle.Free();
				_headerHandle = default;
			}
		}

		/// <inheritdoc/>
		public override void SetReadEvent(int readerIdx) => _readerEvent.Set();

		/// <inheritdoc/>
		public override Task WaitToReadAsync(int readerIdx, CancellationToken cancellationToken) => _readerEvent.Task.WaitAsync(cancellationToken);

		/// <inheritdoc/>
		public override void SetWriteEvent() => _writerEvent.Set();

		/// <inheritdoc/>
		public override Task WaitToWriteAsync(CancellationToken cancellationToken) => _writerEvent.Task.WaitAsync(cancellationToken);
	}
}
