// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.MemoryMappedFiles;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Compute.Buffers
{
	/// <summary>
	/// Core implementation of <see cref="SharedMemoryBuffer"/>
	/// </summary>
	public sealed class SharedMemoryBuffer : ComputeBuffer
	{
		/// <inheritdoc/>
		public string Name => ((SharedMemoryBufferDetail)_detail).Name;

		/// <summary>
		/// Constructor
		/// </summary>
		private SharedMemoryBuffer(ComputeBufferDetail resources)
			: base(resources)
		{
		}

		/// <inheritdoc/>
		public override SharedMemoryBuffer AddRef()
		{
			_detail.AddRef();
			return new SharedMemoryBuffer(_detail);
		}

		/// <summary>
		/// Create a new shared memory buffer
		/// </summary>
		/// <param name="name">Name of the buffer</param>
		/// <param name="capacity">Capacity of the buffer</param>
		public static SharedMemoryBuffer CreateNew(string? name, long capacity)
		{
			int numChunks = (int)Math.Max(4, capacity / Int32.MaxValue);
			return CreateNew(name, numChunks, (int)(capacity / numChunks));
		}

		/// <summary>
		/// Create a new shared memory buffer
		/// </summary>
		/// <param name="name">Name of the buffer</param>
		/// <param name="numChunks">Number of chunks in the buffer</param>
		/// <param name="chunkLength">Length of each chunk</param>
		public static unsafe SharedMemoryBuffer CreateNew(string? name, int numChunks, int chunkLength) => new(SharedMemoryBufferDetail.CreateNew(name, numChunks, chunkLength));

		/// <summary>
		/// Open an existing buffer by name
		/// </summary>
		/// <param name="name">Name of the buffer to open</param>
		public static unsafe SharedMemoryBuffer OpenExisting(string name) => new(SharedMemoryBufferDetail.OpenExisting(name));
	}

	/// <summary>
	/// Core implementation of <see cref="SharedMemoryBuffer"/>
	/// </summary>
	class SharedMemoryBufferDetail : ComputeBufferDetail
	{
		readonly MemoryMappedFile _memoryMappedFile;
		readonly MemoryMappedViewAccessor _memoryMappedViewAccessor;
		readonly MemoryMappedView _memoryMappedView;

		readonly Native.EventHandle _readerEvent;
		readonly Native.EventHandle _writerEvent;

		public string Name { get; }

		internal SharedMemoryBufferDetail(string name, HeaderPtr headerPtr, Memory<byte>[] chunks, MemoryMappedFile memoryMappedFile, MemoryMappedViewAccessor memoryMappedViewAccessor, MemoryMappedView memoryMappedView, Native.EventHandle readerEvent, Native.EventHandle writerEvent)
			: base(headerPtr, chunks)
		{
			Name = name;

			_memoryMappedFile = memoryMappedFile;
			_memoryMappedViewAccessor = memoryMappedViewAccessor;
			_memoryMappedView = memoryMappedView;

			_readerEvent = readerEvent;
			_writerEvent = writerEvent;
		}

		/// <summary>
		/// Create a new shared memory buffer
		/// </summary>
		/// <param name="name">Name of the buffer</param>
		/// <param name="numChunks">Number of chunks in the buffer</param>
		/// <param name="chunkLength">Length of each chunk</param>
		public static unsafe SharedMemoryBufferDetail CreateNew(string? name, int numChunks, int chunkLength)
		{
			long capacity = HeaderSize + (numChunks * sizeof(ulong)) + (numChunks * chunkLength);

			name ??= $"Local\\COMPUTE_{Guid.NewGuid()}";

			MemoryMappedFile memoryMappedFile = MemoryMappedFile.CreateNew($"{name}_M", capacity, MemoryMappedFileAccess.ReadWrite, MemoryMappedFileOptions.None, HandleInheritability.Inheritable);
			MemoryMappedViewAccessor memoryMappedViewAccessor = memoryMappedFile.CreateViewAccessor();
			MemoryMappedView memoryMappedView = new(memoryMappedViewAccessor);

			Native.EventHandle writerEvent = Native.EventHandle.CreateNew($"{name}_W", EventResetMode.AutoReset, true, HandleInheritability.Inheritable);
			Native.EventHandle readerEvent = Native.EventHandle.CreateNew($"{name}_R0", EventResetMode.AutoReset, true, HandleInheritability.Inheritable);

			HeaderPtr headerPtr = new((ulong*)memoryMappedView.GetPointer(), 1, numChunks, chunkLength);
			Memory<byte>[] chunks = CreateChunks(headerPtr.NumChunks, headerPtr.ChunkLength, memoryMappedView);

			return new SharedMemoryBufferDetail(name, headerPtr, chunks, memoryMappedFile, memoryMappedViewAccessor, memoryMappedView, readerEvent, writerEvent);
		}

		/// <summary>
		/// Open an existing buffer by name
		/// </summary>
		/// <param name="name">Name of the buffer to open</param>
		public static unsafe SharedMemoryBufferDetail OpenExisting(string name)
		{
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				throw new NotSupportedException();
			}

			MemoryMappedFile memoryMappedFile = MemoryMappedFile.OpenExisting($"{name}_M");
			MemoryMappedViewAccessor memoryMappedViewAccessor = memoryMappedFile.CreateViewAccessor();
			MemoryMappedView memoryMappedView = new(memoryMappedViewAccessor);

			Native.EventHandle readerEvent = Native.EventHandle.OpenExisting($"{name}_R0");
			Native.EventHandle writerEvent = Native.EventHandle.OpenExisting($"{name}_W");

			HeaderPtr headerPtr = new((ulong*)memoryMappedView.GetPointer());
			Memory<byte>[] chunks = CreateChunks(headerPtr.NumChunks, headerPtr.ChunkLength, memoryMappedView);

			return new SharedMemoryBufferDetail(name, headerPtr, chunks, memoryMappedFile, memoryMappedViewAccessor, memoryMappedView, readerEvent, writerEvent);
		}

		static Memory<byte>[] CreateChunks(int numChunks, int chunkLength, MemoryMappedView memoryMappedView)
		{
			Memory<byte>[] chunks = new Memory<byte>[numChunks];
			for (int chunkIdx = 0; chunkIdx < numChunks; chunkIdx++)
			{
				int chunkOffset = HeaderSize + (chunkLength * chunkIdx);
				chunks[chunkIdx] = memoryMappedView.GetMemory(chunkOffset, chunkLength);
			}
			return chunks;
		}

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				_readerEvent.Dispose();
				_writerEvent.Dispose();

				_memoryMappedView.Dispose();
				_memoryMappedViewAccessor.Dispose();
				_memoryMappedFile.Dispose();
			}
		}

		/// <inheritdoc/>
		public override void SetReadEvent(int readerIdx) => _readerEvent.Set();

		/// <inheritdoc/>
		public override Task WaitToReadAsync(int readerIdx, CancellationToken cancellationToken) => _readerEvent.WaitOneAsync(cancellationToken);

		/// <inheritdoc/>
		public override void SetWriteEvent() => _writerEvent.Set();

		/// <inheritdoc/>
		public override Task WaitToWriteAsync(CancellationToken cancellationToken) => _writerEvent.WaitOneAsync(cancellationToken);
	}
}
