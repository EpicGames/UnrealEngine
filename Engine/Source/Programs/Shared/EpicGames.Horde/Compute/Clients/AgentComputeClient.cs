// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics;
using System.Net;
using System.Net.Sockets;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute.Transports;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute.Clients
{
	/// <summary>
	/// Runs a local Horde Agent process to process compute requests without communicating with a server
	/// </summary>
	public sealed class AgentComputeClient : IComputeClient
	{
		private static readonly ClusterId s_cluster = new("_agent");

		class LeaseImpl : IComputeLease
		{
			readonly IAsyncEnumerator<RemoteComputeSocket> _source;

			/// <inheritdoc/>
			public ClusterId Cluster { get; } = s_cluster;

			/// <inheritdoc/>
			public IReadOnlyList<string> Properties { get; } = [];

			/// <inheritdoc/>
			public IReadOnlyDictionary<string, int> AssignedResources => new Dictionary<string, int>();

			/// <inheritdoc/>
			public RemoteComputeSocket Socket => _source.Current;

			/// <inheritdoc/>
			public string Ip => "127.0.0.1";

			/// <inheritdoc/>
			public UbaConfig? Uba => null;

			/// <inheritdoc/>
			public ConnectionMode ConnectionMode => ConnectionMode.Direct;

			/// <inheritdoc/>
			public IReadOnlyDictionary<string, ConnectionMetadataPort> Ports => new Dictionary<string, ConnectionMetadataPort>();

			public LeaseImpl(IAsyncEnumerator<RemoteComputeSocket> source) => _source = source;

			/// <inheritdoc/>
			public async ValueTask DisposeAsync()
			{
				await _source.MoveNextAsync();
				await _source.DisposeAsync();
			}

			/// <inheritdoc/>
			public ValueTask CloseAsync(CancellationToken cancellationToken) => Socket.CloseAsync(cancellationToken);
		}

		readonly string _hordeAgentAssembly;
		readonly int _port;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="hordeAgentAssembly">Path to the Horde Agent assembly</param>
		/// <param name="port">Loopback port to connect on</param>
		/// <param name="logger">Factory for logger instances</param>
		public AgentComputeClient(string hordeAgentAssembly, int port, ILogger logger)
		{
			_hordeAgentAssembly = hordeAgentAssembly;
			_port = port;
			_logger = logger;
		}

		/// <inheritdoc/>
		public Task<ClusterId> GetClusterAsync(Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, ILogger logger, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(s_cluster);
		}

		/// <inheritdoc/>
		public async Task<IComputeLease?> TryAssignWorkerAsync(ClusterId? clusterId, Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, bool? useUbaCache, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("** CLIENT **");
			logger.LogInformation("Launching {Path} to handle remote", _hordeAgentAssembly);

			// The connection logic is an async enumerator that returns the socket, then shuts down.
			IAsyncEnumerator<RemoteComputeSocket> source = ConnectAsync(logger, cancellationToken).GetAsyncEnumerator(cancellationToken);
			if (!await source.MoveNextAsync())
			{
				await source.DisposeAsync();
				return null;
			}

			return new LeaseImpl(source);
		}

		/// <inheritdoc/>
		public Task DeclareResourceNeedsAsync(ClusterId clusterId, string pool, Dictionary<string, int> resourceNeeds, CancellationToken cancellationToken = default)
		{
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public Task<UbaConfig> AllocateUbaCacheServerAsync(ClusterId clusterId, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		async IAsyncEnumerable<RemoteComputeSocket> ConnectAsync(ILogger logger, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using Socket listener = new(SocketType.Stream, ProtocolType.IP);
			listener.Bind(new IPEndPoint(IPAddress.Loopback, _port));
			listener.Listen();

			await using BackgroundTask agentTask = BackgroundTask.StartNew(ctx => RunAgentAsync(_hordeAgentAssembly, _port, logger, ctx));
			using Socket tcpSocket = await listener.AcceptAsync(cancellationToken);

			await using TcpTransport tcpTransport = new(tcpSocket);
			await using RemoteComputeSocket socket = new(tcpTransport, ComputeProtocol.Latest, _logger);
			yield return socket;

			await socket.CloseAsync(cancellationToken);
		}

		static async Task RunAgentAsync(string hordeAgentAssembly, int port, ILogger logger, CancellationToken cancellationToken)
		{
			using ManagedProcessGroup group = new();
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				List<string> arguments = [hordeAgentAssembly, "computeworker", $"-port={port}"];

				using ManagedProcess process = new(group, "dotnet", CommandLineArguments.Join(arguments), null, null, ProcessPriorityClass.Normal);

				string? line;
				while ((line = await process.ReadLineAsync(cancellationToken)) != null)
				{
					logger.LogInformation("Output: {Line}", line);
				}

				await process.WaitForExitAsync(cancellationToken);
			}
			else
			{
				using Process process = new();
				process.StartInfo.FileName = "dotnet";
				process.StartInfo.ArgumentList.Add(hordeAgentAssembly);
				process.StartInfo.ArgumentList.Add("computeworker");
				process.StartInfo.ArgumentList.Add($"-port={port}");
				process.StartInfo.UseShellExecute = true;
				process.Start();

				await process.WaitForExitAsync(cancellationToken);
			}
		}
	}
}
