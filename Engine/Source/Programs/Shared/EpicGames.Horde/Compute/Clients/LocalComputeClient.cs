// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using System.Net.Sockets;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute.Transports;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute.Clients
{
	/// <summary>
	/// Implementation of <see cref="IComputeClient"/> which marshals data over a loopback connection to a method running on a background task in the same process.
	/// </summary>
	public sealed class LocalComputeClient : IComputeClient, IAsyncDisposable
	{
		private static readonly ClusterId s_cluster = new("_local");

		class LeaseImpl : IComputeLease
		{
			public ClusterId Cluster { get; } = s_cluster;
			public IReadOnlyList<string> Properties { get; } = [];
			public IReadOnlyDictionary<string, int> AssignedResources => new Dictionary<string, int>();
			public RemoteComputeSocket Socket => _socket;
			public string Ip => "127.0.0.1";
			public UbaConfig? Uba => null;
			public ConnectionMode ConnectionMode => ConnectionMode.Direct;
			public IReadOnlyDictionary<string, ConnectionMetadataPort> Ports => new Dictionary<string, ConnectionMetadataPort>();

			readonly RemoteComputeSocket _socket;

			public LeaseImpl(RemoteComputeSocket socket) => _socket = socket;

			/// <inheritdoc/>
			public ValueTask DisposeAsync() => _socket.DisposeAsync();

			/// <inheritdoc/>
			public ValueTask CloseAsync(CancellationToken cancellationToken) => _socket.CloseAsync(cancellationToken);
		}

		readonly BackgroundTask _listenerTask;
		readonly Socket _listener;
		readonly Socket _socket;
		readonly bool _executeInProcess;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="port">Port to connect on</param>
		/// <param name="sandboxDir">Sandbox directory for the worker</param>
		/// <param name="executeInProcess">Whether to run external assemblies in-process. Useful for debugging.</param>
		/// <param name="logger">Logger for diagnostic output</param>
		public LocalComputeClient(int port, DirectoryReference sandboxDir, bool executeInProcess, ILogger logger)
		{
			_executeInProcess = executeInProcess;

			_listener = new Socket(SocketType.Stream, ProtocolType.IP);
			_listener.Bind(new IPEndPoint(IPAddress.Loopback, port));
			_listener.Listen();

			_listenerTask = BackgroundTask.StartNew(ctx => RunListenerAsync(_listener, sandboxDir, _executeInProcess, logger, ctx));

			_socket = new Socket(SocketType.Stream, ProtocolType.IP);
			_socket.Connect(IPAddress.Loopback, port);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			_socket.Dispose();
			await _listenerTask.DisposeAsync();
			_listener.Dispose();
		}

		/// <summary>
		/// Sets up the loopback listener and calls the server method
		/// </summary>
		static async Task RunListenerAsync(Socket listener, DirectoryReference sandboxDir, bool executeInProcess, ILogger logger, CancellationToken cancellationToken)
		{
			using Socket tcpSocket = await listener.AcceptAsync(cancellationToken);
			await using TcpTransport tcpTransport = new(tcpSocket);
			await using RemoteComputeSocket socket = new(tcpTransport, ComputeProtocol.Latest, logger);
			AgentMessageHandler worker = new(sandboxDir, null, executeInProcess, null, null, logger);
			await worker.RunAsync(socket, cancellationToken);
			await socket.CloseAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<ClusterId> GetClusterAsync(Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, ILogger logger, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(s_cluster);
		}

		/// <inheritdoc/>
		public Task<IComputeLease?> TryAssignWorkerAsync(ClusterId? clusterId, Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, bool? useUbaCache, ILogger logger, CancellationToken cancellationToken)
		{
#pragma warning disable CA2000 // Dispose objects before losing scope
			RemoteComputeSocket socket = new(new TcpTransport(_socket), ComputeProtocol.Latest, logger);
			return Task.FromResult<IComputeLease?>(new LeaseImpl(socket));
#pragma warning restore CA2000 // Dispose objects before losing scope
		}

		/// <inheritdoc/>
		public Task DeclareResourceNeedsAsync(ClusterId clusterId, string pool, Dictionary<string, int> resourceNeeds, CancellationToken cancellationToken = default)
		{
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public Task<UbaConfig> AllocateUbaCacheServerAsync(ClusterId clusterId, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}
	}
}
