// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net;
using System.Net.Http;
using System.Net.Http.Json;
using System.Net.Sockets;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Compute.Transports;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute.Clients
{
	/// <summary>
	/// Handshake request message for tunneling server
	/// </summary>
	/// <param name="Host">Target host to relay traffic to/from</param>
	/// <param name="Port">Target port</param>
	public record TunnelHandshakeRequest(string Host, int Port)
	{
		const int Version = 1;
		const string Name = "HANDSHAKE-REQ";

		/// <summary>
		/// Serialize the message
		/// </summary>
		/// <returns>A string based representation</returns>
		public string Serialize()
		{
			return $"{Name}\t{Version}\t{Host}\t{Port}";
		}

		/// <summary>
		/// Deserialize the message
		/// </summary>
		/// <param name="text">A raw string to deserialize</param>
		/// <returns>A request message</returns>
		/// <exception cref="Exception"></exception>
		public static TunnelHandshakeRequest Deserialize(string? text)
		{
			string[] parts = (text ?? "").Split('\t');
			if (parts.Length != 4 || parts[0] != Name || !Int32.TryParse(parts[1], out int _) || !Int32.TryParse(parts[3], out int port))
			{
				throw new Exception("Failed deserializing handshake request. Content: " + text);
			}
			return new TunnelHandshakeRequest(parts[2], port);
		}
	}

	/// <summary>
	/// Handshake response message for tunneling server
	/// </summary>
	/// <param name="IsSuccess">Whether successful or not</param>
	/// <param name="Message">Message with additional information describing the outcome</param>
	public record TunnelHandshakeResponse(bool IsSuccess, string Message)
	{
		const int Version = 1;
		const string Name = "HANDSHAKE-RES";

		/// <summary>
		/// Serialize the message
		/// </summary>
		/// <returns>A string based representation</returns>
		public string Serialize()
		{
			return $"{Name}\t{Version}\t{IsSuccess}\t{Message}";
		}

		/// <summary>
		/// Deserialize the message
		/// </summary>
		/// <param name="text">A raw string to deserialize</param>
		/// <returns>A request message</returns>
		/// <exception cref="Exception"></exception>
		public static TunnelHandshakeResponse Deserialize(string? text)
		{
			string[] parts = (text ?? "").Split('\t');
			if (parts.Length != 4 || !Int32.TryParse(parts[1], out int _) || !Boolean.TryParse(parts[2], out bool isSuccess))
			{
				throw new Exception("Failed deserializing handshake response. Content: " + text);
			}
			return new TunnelHandshakeResponse(isSuccess, parts[3]);
		}
	}

	/// <summary>
	/// Exception for ServerComputeClient
	/// </summary>
	public class ServerComputeClientException : ComputeException
	{
		/// <inheritdoc/>
		public ServerComputeClientException(string message) : base(message)
		{
		}

		/// <inheritdoc/>
		public ServerComputeClientException(string? message, Exception? innerException) : base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Helper class to enlist remote resources to perform compute-intensive tasks.
	/// </summary>
	public sealed class ServerComputeClient : IComputeClient, IDisposable
	{
		/// <summary>
		/// Length of the nonce sent as part of handshaking between initiator and remote
		/// </summary>
		public const int NonceLength = 64;

		record LeaseInfo(
			ClusterId Cluster,
			IReadOnlyList<string> Properties,
			IReadOnlyDictionary<string, int> AssignedResources,
			RemoteComputeSocket Socket,
			string Ip,
			UbaConfig? Uba,
			ConnectionMode ConnectionMode,
			IReadOnlyDictionary<string, ConnectionMetadataPort> Ports);

		class LeaseImpl : IComputeLease
		{
			public ClusterId Cluster => _source.Current.Cluster;
			public IReadOnlyList<string> Properties => _source.Current.Properties;
			public IReadOnlyDictionary<string, int> AssignedResources => _source.Current.AssignedResources;
			public RemoteComputeSocket Socket => _source.Current.Socket;
			public string Ip => _source.Current.Ip;
			public UbaConfig? Uba => _source.Current.Uba;
			public ConnectionMode ConnectionMode => _source.Current.ConnectionMode;
			public IReadOnlyDictionary<string, ConnectionMetadataPort> Ports => _source.Current.Ports;

			private readonly IAsyncEnumerator<LeaseInfo> _source;
			private BackgroundTask? _pingTask;

			public LeaseImpl(IAsyncEnumerator<LeaseInfo> source)
			{
				_source = source;
				_pingTask = BackgroundTask.StartNew(PingAsync);
			}

			/// <inheritdoc/>
			public async ValueTask DisposeAsync()
			{
				if (_pingTask != null)
				{
					await _pingTask.DisposeAsync();
					_pingTask = null;
				}

				await _source.MoveNextAsync();
				await _source.DisposeAsync();
			}

			/// <inheritdoc/>
			public async ValueTask CloseAsync(CancellationToken cancellationToken)
			{
				if (_pingTask != null)
				{
					await _pingTask.DisposeAsync();
					_pingTask = null;
				}

				await Socket.CloseAsync(cancellationToken);
			}

			async Task PingAsync(CancellationToken cancellationToken)
			{
				while (!cancellationToken.IsCancellationRequested)
				{
					await Socket.SendKeepAliveMessageAsync(cancellationToken);
					await Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken);
				}
			}
		}

		readonly HttpClient _httpClient;
		readonly CancellationTokenSource _cancellationSource = new();
		readonly string _sessionId;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="httpClient">Factory for constructing http client instances</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public ServerComputeClient(HttpClient httpClient, ILogger logger) : this(httpClient, null, logger)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="httpClient">Factory for constructing http client instances</param>
		/// <param name="sessionId">Arbitrary ID used for identifying this compute client. If not provided, a random one will be generated</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public ServerComputeClient(HttpClient httpClient, string? sessionId, ILogger logger)
		{
			_httpClient = httpClient;
			_sessionId = sessionId ?? Guid.NewGuid().ToString();
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_cancellationSource.Dispose();
		}

		/// <inheritdoc/>
		public async Task<ClusterId> GetClusterAsync(Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, ILogger logger, CancellationToken cancellationToken = default)
		{
			AssignComputeRequest request = new()
			{
				Requirements = requirements,
				RequestId = requestId,
				Connection = connection,
				Protocol = (int)ComputeProtocol.Latest
			};

			using HttpResponseMessage httpResponse = await HordeHttpRequest.PostAsync(_httpClient, "api/v2/compute/_cluster", request, _cancellationSource.Token);
			if (!httpResponse.IsSuccessStatusCode)
			{
				string body = await httpResponse.Content.ReadAsStringAsync(cancellationToken);
				throw new ComputeClientException($"Unable to find suitable cluster. HTTP status code {httpResponse.StatusCode}: {body}");
			}

			GetClusterResponse? response = await httpResponse.Content.ReadFromJsonAsync<GetClusterResponse>(HordeHttpClient.JsonSerializerOptions, cancellationToken);
			if (response == null)
			{
				throw new InvalidOperationException();
			}

			return response.ClusterId;
		}

		/// <inheritdoc/>
		public async Task<IComputeLease?> TryAssignWorkerAsync(ClusterId? clusterId, Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, bool? useUbaCache, ILogger logger, CancellationToken cancellationToken)
		{
			try
			{
				IAsyncEnumerator<LeaseInfo> source = ConnectAsync(clusterId, requirements, requestId, connection, useUbaCache, logger, cancellationToken).GetAsyncEnumerator(cancellationToken);
				if (!await source.MoveNextAsync())
				{
					await source.DisposeAsync();
					return null;
				}
				return new LeaseImpl(source);
			}
			catch (Polly.Timeout.TimeoutRejectedException ex)
			{
				_logger.LogInformation(ex, "Unable to assign worker from pool {ClusterId} (timeout)", clusterId);
				return null;
			}
		}

		/// <inheritdoc/>
		public async Task DeclareResourceNeedsAsync(ClusterId clusterId, string pool, Dictionary<string, int> resourceNeeds, CancellationToken cancellationToken = default)
		{
			ResourceNeedsMessage request = new() { SessionId = _sessionId, Pool = pool, ResourceNeeds = resourceNeeds };
			using HttpResponseMessage response = await HordeHttpRequest.PostAsync(_httpClient, $"api/v2/compute/{clusterId}/resource-needs", request, _cancellationSource.Token);
			response.EnsureSuccessStatusCode();
		}

		/// <inheritdoc/>
		public async Task<UbaConfig> AllocateUbaCacheServerAsync(ClusterId clusterId, CancellationToken cancellationToken = default)
		{
			AllocateUbaCacheServerRequest request = new();
			using HttpResponseMessage httpResponse = await HordeHttpRequest.PostAsync(_httpClient, $"api/v2/compute/{clusterId}/uba-cache", request, _cancellationSource.Token);
			if (httpResponse.StatusCode == HttpStatusCode.InternalServerError)
			{
				string? content;
				try
				{
					content = await httpResponse.Content.ReadAsStringAsync(cancellationToken);
				}
				catch
				{
					content = "None";
				}
				throw new ComputeClientException($"Error allocating UBA cache server: {content}");
			}

			httpResponse.EnsureSuccessStatusCode();
			UbaConfig? response = await httpResponse.Content.ReadFromJsonAsync<UbaConfig>(HordeHttpClient.JsonSerializerOptions, cancellationToken);
			if (response == null)
			{
				throw new InvalidOperationException();
			}

			return response;
		}

		async IAsyncEnumerable<LeaseInfo> ConnectAsync(ClusterId? clusterId, Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, bool? useUbaCache, ILogger workerLogger, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			_logger.LogDebug("Requesting compute resource");

			// Assign a compute worker
			AssignComputeRequest request = new()
			{
				Requirements = requirements,
				RequestId = requestId,
				Connection = connection,
				UseUbaCache = useUbaCache,
				Protocol = (int)ComputeProtocol.Latest
			};

			AssignComputeResponse? response;
			string path = clusterId == null ? "api/v2/compute" : $"api/v2/compute/{clusterId}";
			using (HttpResponseMessage httpResponse = await HordeHttpRequest.PostAsync(_httpClient, path, request, _cancellationSource.Token))
			{
				if (httpResponse.StatusCode == HttpStatusCode.NotFound)
				{
					throw new NoComputeAgentsFoundException(clusterId ?? new ClusterId("null"), requirements);
				}

				if (httpResponse.StatusCode is HttpStatusCode.ServiceUnavailable or HttpStatusCode.TooManyRequests)
				{
					_logger.LogDebug("No compute resource is available. Reason: {Reason}", await httpResponse.Content.ReadAsStringAsync(cancellationToken));
					yield break;
				}

				if (httpResponse.StatusCode == HttpStatusCode.Unauthorized)
				{
					string? content;
					try
					{
						content = await httpResponse.Content.ReadAsStringAsync(cancellationToken);
					}
					catch
					{
						content = "None";
					}
					throw new ComputeClientException($"Bad authentication credentials. Check or refresh token. (HTTP status {httpResponse.StatusCode}, response: {content})");
				}

				if (httpResponse.StatusCode == HttpStatusCode.Forbidden)
				{
					LogEvent? logEvent = await httpResponse.Content.ReadFromJsonAsync<LogEvent>(HordeHttpClient.JsonSerializerOptions, cancellationToken);
					if (logEvent != null)
					{
						throw new ComputeClientException($"{logEvent.Message} (HTTP status {httpResponse.StatusCode})");
					}
				}

				if (httpResponse.StatusCode == HttpStatusCode.InternalServerError)
				{
					string? content;
					try
					{
						content = await httpResponse.Content.ReadAsStringAsync(cancellationToken);
					}
					catch
					{
						content = "None";
					}
					throw new ComputeClientException($"Error requesting compute resources: {content}");
				}

				httpResponse.EnsureSuccessStatusCode();
				response = await httpResponse.Content.ReadFromJsonAsync<AssignComputeResponse>(HordeHttpClient.JsonSerializerOptions, cancellationToken);
				if (response == null)
				{
					throw new InvalidOperationException();
				}
			}

			string nonce = response.Nonce;
			nonce = nonce.Length <= 8 ? nonce : $"{nonce[..4]}...{nonce[^4..]}"; // Trim large nonce strings

			(string host, int port) agentAddress = (response.Ip, response.Port); // Canonical address of agent not accounting for relays
			(string host, int port) connectionAddress = agentAddress; // De facto address of agent, accounting for relays
			if (response.ConnectionMode == ConnectionMode.Relay && !String.IsNullOrEmpty(response.ConnectionAddress))
			{
				agentAddress.port = response.Ports[ConnectionMetadataPort.ComputeId].AgentPort;
				connectionAddress = (response.ConnectionAddress, response.Ports[ConnectionMetadataPort.ComputeId].Port);
			}

			// Connect to the remote machine
			using Socket socket = new(SocketType.Stream, ProtocolType.Tcp);

			workerLogger.LogDebug(
				"Connecting to {AgentId}. Agent={AgentHost}:{AgentPort} Connection={ConnectionMode}/{ConnectionHost}:{ConnectionPort} Encryption={Encryption} LeaseId={LeaseId} Requirements={Requirements} PublicIp={PublicIp} Nonce={Nonce}",
				response.AgentId,
				agentAddress.host,
				agentAddress.port,
				response.ConnectionMode,
				connectionAddress.host,
				connectionAddress.port,
				response.Encryption,
				response.LeaseId,
				request.Requirements,
				request.Connection?.ClientPublicIp,
				nonce);

			try
			{
				switch (response.ConnectionMode)
				{
					case ConnectionMode.Direct:
						await socket.ConnectAsync(connectionAddress.host, connectionAddress.port, cancellationToken);
						break;

					case ConnectionMode.Tunnel when !String.IsNullOrEmpty(response.ConnectionAddress):
						(connectionAddress.host, connectionAddress.port) = ParseHostPort(response.ConnectionAddress);
						await socket.ConnectAsync(connectionAddress.host, connectionAddress.port, cancellationToken);
						await TunnelHandshakeAsync(socket, response, cancellationToken);
						break;

					case ConnectionMode.Relay when !String.IsNullOrEmpty(response.ConnectionAddress):
						response.Ip = response.ConnectionAddress;
						await ConnectWithRetryAsync(socket, connectionAddress.host, connectionAddress.port, TimeSpan.FromSeconds(5), 3, cancellationToken);
						break;

					default:
						throw new Exception($"Unable to resolve connection mode ({response.ConnectionMode} via {response.ConnectionAddress ?? "none"})");
				}
			}
			catch (SocketException se)
			{
				throw new ServerComputeClientException($"Unable to connect to {connectionAddress.host}:{connectionAddress.port} with mode {response.ConnectionMode}", se);
			}

			// Send the nonce
			byte[] nonceData = StringUtils.ParseHexString(response.Nonce);
			await socket.SendMessageAsync(nonceData, SocketFlags.None, cancellationToken);
			workerLogger.LogInformation("Connected to {AgentId} ({Ip}) under lease {LeaseId} (agent version: {AgentVersion})", response.AgentId, response.Ip, response.LeaseId, response.AgentVersion ?? "unknown");

			response.Properties = [.. response.Properties, $"{KnownPropertyNames.LeaseId}={response.LeaseId}"];
			await using ComputeTransport transport = await CreateTransportAsync(socket, response, cancellationToken);
			await using RemoteComputeSocket computeSocket = new(transport, (ComputeProtocol)response.Protocol, workerLogger);
			yield return new LeaseInfo(response.ClusterId, response.Properties, response.AssignedResources, computeSocket, response.Ip, response.Uba, response.ConnectionMode, response.Ports);
		}

		private async Task ConnectWithRetryAsync(Socket socket, string host, int port, TimeSpan timeout, int maxRetries, CancellationToken cancellationToken)
		{
			TimeSpan retryDelay = TimeSpan.FromSeconds(1);
			for (int attempt = 1; attempt <= maxRetries; attempt++)
			{
				try
				{
					using CancellationTokenSource timeoutCts = new(timeout);
					using CancellationTokenSource linkedCts = CancellationTokenSource.CreateLinkedTokenSource(timeoutCts.Token, cancellationToken);
					await socket.ConnectAsync(host, port, linkedCts.Token);
					return;
				}
				catch (OperationCanceledException)
				{
					if (attempt == maxRetries)
					{
						throw new TimeoutException($"Failed to connect {host}:{port} after {maxRetries} attempts");
					}
				}
				catch (SocketException se)
				{
					if (attempt == maxRetries)
					{
						throw;
					}
					_logger.LogInformation("Unable to connect to {Host}:{Port} within {Timeout} ms. Error code: {Error}. Waiting {RetryDelay} ms before retrying...",
						host, port, (int)timeout.TotalMilliseconds, se.SocketErrorCode, (int)retryDelay.TotalMilliseconds);
				}

				await Task.Delay(retryDelay, cancellationToken);
			}
		}

		private static async Task<ComputeTransport> CreateTransportAsync(Socket socket, AssignComputeResponse response, CancellationToken cancellationToken)
		{
			switch (response.Encryption)
			{
				case Encryption.Ssl:
				case Encryption.SslEcdsaP256:
					TcpSslTransport sslTransport = new(socket, StringUtils.ParseHexString(response.Certificate), false);
					await sslTransport.AuthenticateAsync(cancellationToken);
					return sslTransport;

				case Encryption.Aes:
#pragma warning disable CA2000 // Dispose objects before losing scope
					TcpTransport tcpTransport = new(socket);
					return new AesTransport(tcpTransport, StringUtils.ParseHexString(response.Key));
#pragma warning restore CA2000 // Restore CA2000

				case Encryption.None:
				default:
					return new TcpTransport(socket);
			}
		}

		private static (string host, int port) ParseHostPort(string address)
		{
			try
			{
				string[] parts = address.Split(":");
				string host = parts[0];
				int port = Int32.Parse(parts[1]);
				return (host, port);
			}
			catch (Exception e)
			{
				throw new Exception($"Unable to parse host and port for address: {address}", e);
			}
		}

		private static async Task TunnelHandshakeAsync(Socket socket, AssignComputeResponse response, CancellationToken cancellationToken)
		{
			await using NetworkStream ns = new(socket, false);
			using StreamReader reader = new(ns);
			await using StreamWriter writer = new(ns) { AutoFlush = true };

			string request = new TunnelHandshakeRequest(response.Ip, response.Port).Serialize();
			await writer.WriteLineAsync(request.ToCharArray(), cancellationToken);

			string exceptionMetadata = $"Connection: {response.ConnectionAddress} Target: {response.Ip}:{response.Port}";
#pragma warning disable CA2016 // Forward the 'CancellationToken' parameter to methods
			Task<string?> readTask = reader.ReadLineAsync();
#pragma warning restore CA2016 // Forward the 'CancellationToken' parameter to methods
			Task timeoutTask = Task.Delay(15000, cancellationToken);
			if (await Task.WhenAny(readTask, timeoutTask) == timeoutTask)
			{
				throw new TimeoutException($"Timed out reading tunnel handshake response. {exceptionMetadata}");
			}

			TunnelHandshakeResponse handshakeResponse = TunnelHandshakeResponse.Deserialize(await readTask);
			if (!handshakeResponse.IsSuccess)
			{
				throw new Exception($"Tunnel handshake failed! Reason: {handshakeResponse.Message} {exceptionMetadata}");
			}
		}
	}

	/// <summary>
	/// Exception indicating that no matching compute agents were found
	/// </summary>
	public sealed class NoComputeAgentsFoundException : ComputeClientException
	{
		/// <summary>
		/// The compute cluster requested
		/// </summary>
		public ClusterId ClusterId { get; }

		/// <summary>
		/// Requested agent requirements
		/// </summary>
		public Requirements? Requirements { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public NoComputeAgentsFoundException(ClusterId clusterId, Requirements? requirements)
			: base($"No compute agents found matching '{requirements}' in cluster '{clusterId}'")
		{
			ClusterId = clusterId;
			Requirements = requirements;
		}
	}
}
