// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Identifier for a compute cluster
	/// </summary>
	[LogValueType]
	[JsonSchemaString]
	[CbConverter(typeof(ClusterIdCbConverter))]
	[JsonConverter(typeof(ClusterIdJsonConverter))]
	[TypeConverter(typeof(ClusterIdTypeConverter))]
	public readonly struct ClusterId : IEquatable<ClusterId>
	{
		/// <summary>
		/// The text representing this id
		/// </summary>
		readonly StringId _inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="input">Unique id for the string</param>
		public ClusterId(string input)
		{
			_inner = new StringId(input);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is ClusterId id && _inner.Equals(id._inner);

		/// <inheritdoc/>
		public override int GetHashCode() => _inner.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(ClusterId other) => _inner.Equals(other._inner);

		/// <inheritdoc/>
		public override string ToString() => _inner.ToString();

		/// <inheritdoc cref="StringId.op_Equality"/>
		public static bool operator ==(ClusterId left, ClusterId right) => left._inner == right._inner;

		/// <inheritdoc cref="StringId.op_Inequality"/>
		public static bool operator !=(ClusterId left, ClusterId right) => left._inner != right._inner;
	}

	/// <summary>
	/// Compact binary converter for ClusterId
	/// </summary>
	sealed class ClusterIdCbConverter : CbConverter<ClusterId>
	{
		/// <inheritdoc/>
		public override ClusterId Read(CbField field) => new(field.AsString());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, ClusterId value) => writer.WriteStringValue(value.ToString());

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, ClusterId value) => writer.WriteString(name, value.ToString());
	}

	/// <summary>
	/// Type converter for ClusterId to and from JSON
	/// </summary>
	sealed class ClusterIdJsonConverter : JsonConverter<ClusterId>
	{
		/// <inheritdoc/>
		public override ClusterId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(reader.GetString() ?? String.Empty);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ClusterId value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
	}

	/// <summary>
	/// Type converter from strings to ClusterId objects
	/// </summary>
	sealed class ClusterIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type? sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object? value) => new ClusterId((string)value!);
	}
}

