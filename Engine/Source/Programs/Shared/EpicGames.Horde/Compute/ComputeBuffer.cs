// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// In-process buffer used to store compute messages
	/// </summary>
	public abstract class ComputeBuffer : IDisposable
	{
		/// <summary>
		/// Maximum number of chunks in a buffer
		/// </summary>
		public const int MaxChunks = 16;

		/// <summary>
		/// Maximum number of readers
		/// </summary>
		public const int MaxReaders = 16;

		internal ComputeBufferDetail _detail;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="detail">Resources shared between instances of the buffer</param>
		internal ComputeBuffer(ComputeBufferDetail detail)
		{
			_detail = detail;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		protected virtual void Dispose(bool disposing)
		{
			if (_detail != null)
			{
				_detail.Release();
				_detail = null!;
			}
		}

		/// <summary>
		/// Creates a new reader for this buffer
		/// </summary>
		public ComputeBufferReader CreateReader()
		{
			int readerIdx = _detail.CreateReader();
			_detail.AddRef();
			return new ComputeBufferReader(_detail, readerIdx);
		}

		/// <summary>
		/// Writer for this buffer
		/// </summary>
		public ComputeBufferWriter CreateWriter()
		{
			_detail.CreateWriter();
			_detail.AddRef();
			return new ComputeBufferWriter(_detail);
		}

		/// <summary>
		/// Creates a new reference to the underlying buffer. The underlying resources will only be destroyed once all instances are disposed of.
		/// </summary>
		public abstract ComputeBuffer AddRef();
	}

	/// <summary>
	/// Read interface for a compute buffer
	/// </summary>
	public sealed class ComputeBufferReader : IDisposable
	{
		ComputeBufferDetail _buffer;
		readonly int _readerIdx;

		internal ComputeBufferDetail Detail => _buffer;

		internal ComputeBufferReader(ComputeBufferDetail buffer, int readerIdx)
		{
			_buffer = buffer;
			_readerIdx = readerIdx;
		}

		/// <summary>
		/// Create a new reader instance using the same underlying buffer
		/// </summary>
		public ComputeBufferReader AddRef()
		{
			_buffer.AddRef();
			_buffer.AddReaderRef(_readerIdx);
			return new ComputeBufferReader(_buffer, _readerIdx);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_buffer != null)
			{
				_buffer.ReleaseReaderRef(_readerIdx);
				_buffer.Release();
				_buffer = null!;
			}
		}

		/// <summary>
		/// Detaches this reader from the underlying buffer
		/// </summary>
		public void Detach() => _buffer.DetachReader(_readerIdx);

		/// <summary>
		/// Whether this buffer is complete (no more data will be added)
		/// </summary>
		public bool IsComplete => _buffer.IsComplete(_readerIdx);

		/// <summary>
		/// Updates the read position
		/// </summary>
		/// <param name="length">Size of data that was read</param>
		public void AdvanceReadPosition(int length) => _buffer.AdvanceReadPosition(_readerIdx, length);

		/// <summary>
		/// Gets the next data to read
		/// </summary>
		/// <returns>Memory to read from</returns>
		public ReadOnlyMemory<byte> GetReadBuffer() => _buffer.GetReadBuffer(_readerIdx);

		/// <summary>
		/// Read from a buffer into another buffer
		/// </summary>
		/// <param name="buffer">Memory to receive the read data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Number of bytes read</returns>
		public async ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				ReadOnlyMemory<byte> readMemory = GetReadBuffer();
				if (IsComplete || readMemory.Length > 0)
				{
					int length = Math.Min(readMemory.Length, buffer.Length);
					readMemory.Slice(0, length).CopyTo(buffer);
					AdvanceReadPosition(length);
					return length;
				}
				await WaitToReadAsync(1, cancellationToken);
			}
		}

		/// <summary>
		/// Wait for data to be available, or for the buffer to be marked as complete
		/// </summary>
		/// <param name="minLength">Minimum amount of data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if new data is available, false if the buffer is complete</returns>
		public ValueTask<bool> WaitToReadAsync(int minLength, CancellationToken cancellationToken = default) => _buffer.WaitToReadAsync(_readerIdx, minLength, cancellationToken);
	}

	/// <summary>
	/// Buffer that can receive data from a remote machine.
	/// </summary>
	public sealed class ComputeBufferWriter : IDisposable
	{
		ComputeBufferDetail _detail;

		internal ComputeBufferDetail Detail => _detail;

		internal ComputeBufferWriter(ComputeBufferDetail detail) => _detail = detail;

		/// <summary>
		/// Create a new writer instance using the same underlying buffer
		/// </summary>
		public ComputeBufferWriter AddRef()
		{
			_detail.AddRef();
			_detail.AddWriterRef();
			return new ComputeBufferWriter(_detail);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_detail != null)
			{
				_detail.ReleaseWriterRef();
				_detail.Release();
				_detail = null!;
			}
		}

		/// <inheritdoc/>
		public void AdvanceWritePosition(int size) => _detail.AdvanceWritePosition(size);

		/// <summary>
		/// Gets memory to write to
		/// </summary>
		/// <returns>Memory to be written to</returns>
		public Memory<byte> GetWriteBuffer() => _detail.GetWriteBuffer();

		/// <summary>
		/// Mark the output to this buffer as complete
		/// </summary>
		/// <returns>Whether the writer was marked as complete. False if the writer has already been marked as complete.</returns>
		public bool MarkComplete() => _detail.MarkComplete();

		/// <summary>
		/// Writes data into a buffer from a memory block
		/// </summary>
		/// <param name="buffer">The data to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask WriteAsync(ReadOnlyMemory<byte> buffer, CancellationToken cancellationToken = default)
		{
			while (buffer.Length > 0)
			{
				Memory<byte> writeMemory = GetWriteBuffer();
				if (writeMemory.Length >= buffer.Length)
				{
					buffer.CopyTo(writeMemory);
					AdvanceWritePosition(buffer.Length);
					break;
				}
				await WaitToWriteAsync(buffer.Length, cancellationToken);
			}
		}

		/// <summary>
		/// Gets memory to write to
		/// </summary>
		/// <param name="minLength">Minimum size of the desired write buffer</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Memory to be written to</returns>
		public ValueTask WaitToWriteAsync(int minLength, CancellationToken cancellationToken = default) => _detail.WaitToWriteAsync(minLength, cancellationToken);
	}

	/// <summary>
	/// State shared between buffer instances
	/// </summary>
	[DebuggerTypeProxy(typeof(BufferDebugProxy))]
	internal abstract class ComputeBufferDetail : IDisposable
	{
		internal const int HeaderSize = (2 + ComputeBuffer.MaxReaders + ComputeBuffer.MaxChunks) * sizeof(ulong);

		/// <summary>
		/// Write state for a chunk
		/// </summary>
		protected internal enum WriteState
		{
			/// <summary>
			/// Writer has moved to the next chunk
			/// </summary>
			MovedToNext = 0,

			/// <summary>
			/// Chunk is still being appended to
			/// </summary>
			Writing = 2,

			/// <summary>
			/// This chunk marks the end of the stream
			/// </summary>
			Complete = 3,
		}

		/// <summary>
		/// Stores the state of a chunk in a 64-bit value, which can be updated atomically
		/// </summary>
		protected internal record struct ChunkState(ulong Value)
		{
			// Written length of this chunk
			public readonly int Length => (int)(Value & 0x7fffffff);

			// Set of flags which are set for each reader that still has to read from a chunk
			public readonly int ReaderFlags => (int)((Value >> 31) & 0x7fffffff);

			// State of the writer
			public readonly WriteState WriteState => (WriteState)(Value >> 62);

			// Constructor
			public ChunkState(WriteState writerState, int readerFlags, int length) : this(((ulong)writerState << 62) | ((ulong)readerFlags << 31) | (uint)length) { }

			// Test whether a particular reader is still referencing the chunk
			public readonly bool HasReaderFlag(int readerIdx) => (Value & (1UL << (31 + readerIdx))) != 0;

			/// <inheritdoc/>
			public override readonly string ToString() => $"{WriteState}, Length: {Length}, Readers: {ReaderFlags}";
		}

		/// <summary>
		/// Wraps a pointer to the state of a chunk
		/// </summary>
		protected internal readonly unsafe struct ChunkStatePtr
		{
			readonly ulong* _data;

			public ChunkStatePtr(ulong* data) => _data = data;

			// Current value of the chunk state
			public ChunkState Get() => new(Interlocked.CompareExchange(ref *_data, 0, 0));

			// Set the current state
			public void Set(ChunkState value) => Interlocked.Exchange(ref *_data, value.Value);

			// Attempt to update the chunk state
			public bool TryUpdate(ChunkState prevState, ChunkState nextState) => Interlocked.CompareExchange(ref *_data, nextState.Value, prevState.Value) == prevState.Value;

			// Append data to the chunk
			public void Append(int length) => Interlocked.Add(ref *_data, (ulong)length);

			// Move to the next chunk
			public void MarkComplete() => Interlocked.Or(ref *_data, new ChunkState(WriteState.Complete, 0, 0).Value);

			// Start reading the chunk with the given reader
			public void StartReading(int readerIdx) => Interlocked.Or(ref *_data, new ChunkState(0, 1 << readerIdx, 0).Value);

			// Clear the reader flag
			public void FinishReading(int readerIdx) => Interlocked.And(ref *_data, ~new ChunkState(0, 1 << readerIdx, 0).Value);

			// Move to the next chunk
			public void FinishWriting() => Interlocked.And(ref *_data, ~new ChunkState(WriteState.Writing, 0, 0).Value);

			/// <inheritdoc/>
			public override string ToString() => Get().ToString();
		}

		/// <summary>
		/// State of a reader
		/// </summary>
		protected internal record struct ReaderState(ulong Value)
		{
			public ReaderState(int chunkIdx, int offset, int refCount, bool detached)
				: this((ulong)(uint)offset | ((ulong)(uint)chunkIdx << 32) | ((ulong)(uint)refCount << 40) | ((ulong)((detached ? (1UL << 63) : 0))))
			{ }

			public readonly int Offset => (int)(Value & 0xffffffff);
			public readonly int ChunkIdx => (int)((Value >> 32) & 0xff);
			public readonly int RefCount => (int)((Value >> 40) & 0x7fff);
			public readonly bool Detached => (Value & (1UL << 63)) != 0;

			/// <inheritdoc/>
			public override readonly string ToString() => $"Chunk: {ChunkIdx}, Offset: {Offset}, RefCount: {RefCount}, Detached: {Detached}";
		}

		/// <summary>
		/// Wraps a pointer to the state of a writer
		/// </summary>
		protected internal readonly unsafe struct ReaderStatePtr
		{
			readonly ulong* _data;

			public ReaderStatePtr(ulong* data) => _data = data;

			// Current value of the chunk state
			public ReaderState Get() => new(Interlocked.CompareExchange(ref *_data, 0, 0));

			// Update current state
			public void Set(ReaderState value) => Interlocked.Exchange(ref *_data, value.Value);

			// Compare and swap
			public bool TryUpdate(ReaderState prevState, ReaderState nextState) => Interlocked.CompareExchange(ref *_data, nextState.Value, prevState.Value) == prevState.Value;
		}

		/// <summary>
		/// State of the writer
		/// </summary>
		protected internal record struct WriterState(ulong Value)
		{
			public WriterState(int chunkIdx, int readerFlags, int refCount, bool hasWrapped)
				: this((ulong)(uint)chunkIdx | ((ulong)(uint)readerFlags << 32) | ((ulong)(uint)refCount << 48) | (hasWrapped ? (1UL << 63) : 0))
			{ }

			public readonly int ChunkIdx => (int)(Value & 0x7fffffff);
			public readonly int ReaderFlags => (int)(Value >> 32) & 0xffff;
			public readonly int RefCount => (int)(Value >> 48) & 0x7fff;
			public readonly bool HasWrapped => (Value & (1UL << 63)) != 0;

			/// <inheritdoc/>
			public override readonly string ToString() => $"Chunk: {ChunkIdx}, ReaderFlags: {ReaderFlags}, RefCount: {RefCount}, HasWrapped: {HasWrapped}";
		}

		/// <summary>
		/// Wraps a pointer to the state of a writer
		/// </summary>
		protected internal readonly unsafe struct WriterStatePtr
		{
			readonly ulong* _data;

			public WriterStatePtr(ulong* data) => _data = data;

			// Get the current value
			public WriterState Get() => new(Interlocked.CompareExchange(ref *_data, 0, 0));

			// Set the current value
			public void Set(WriterState state) => Interlocked.Exchange(ref *_data, state.Value);

			// Compare and swap
			public bool TryUpdate(WriterState prevState, WriterState nextState) => Interlocked.CompareExchange(ref *_data, nextState.Value, prevState.Value) == prevState.Value;
		}

		/// <summary>
		/// Tracked state of the buffer
		/// </summary>
		protected internal readonly unsafe struct HeaderPtr
		{
			readonly ulong* _data;

			public HeaderPtr(ulong* data) => _data = data;

			public HeaderPtr(ulong* data, int numReaders, int numChunks, int chunkLength)
			{
				_data = data;
				data[0] = ((ulong)(uint)chunkLength << 32) | ((ulong)(uint)numChunks << 16) | (uint)numReaders;

				GetChunkStatePtr(0).Set(new ChunkState(WriteState.Writing, 0, 0));
			}

			public int NumReaders => (int)(_data[0] & 0xffff);
			public int NumChunks => (int)((_data[0] >> 16) & 0xffff);
			public int ChunkLength => (int)(_data[0] >> 32);

			public WriterStatePtr GetWriterStatePtr() => new(_data + 1);

			public ReaderStatePtr GetReaderStatePtr(int readerIdx) => new(_data + 2 + readerIdx);

			public ChunkStatePtr GetChunkStatePtr(int chunkIdx) => new(_data + 2 + ComputeBuffer.MaxReaders + chunkIdx);
		}

		class BufferDebugProxy
		{
			public int ChunkLength { get; }
			public int RefCount { get; }
			public WriterState Writer { get; }
			public ReaderState[] Readers { get; }
			public ChunkState[] Chunks { get; }

			public BufferDebugProxy(ComputeBufferDetail buffer)
			{
				RefCount = buffer._refCount;

				HeaderPtr headerPtr = buffer._headerPtr;
				ChunkLength = headerPtr.ChunkLength;

				Writer = headerPtr.GetWriterStatePtr().Get();

				Chunks = new ChunkState[headerPtr.NumChunks];
				for (int chunkIdx = 0; chunkIdx < headerPtr.NumChunks; chunkIdx++)
				{
					Chunks[chunkIdx] = headerPtr.GetChunkStatePtr(chunkIdx).Get();
				}

				Readers = new ReaderState[headerPtr.NumReaders];
				for (int readerIdx = 0; readerIdx < headerPtr.NumReaders; readerIdx++)
				{
					Readers[readerIdx] = headerPtr.GetReaderStatePtr(readerIdx).Get();
				}
			}
		}

		HeaderPtr _headerPtr;
		Memory<byte>[] _chunks;
		int _refCount = 1;

		/// <summary>
		/// Constructor
		/// </summary>
		protected ComputeBufferDetail(HeaderPtr headerPtr, Memory<byte>[] chunks)
		{
			_headerPtr = headerPtr;
			_chunks = chunks;
		}

		/// <summary>
		/// Increment the reference count on this object
		/// </summary>
		public void AddRef()
		{
			Interlocked.Increment(ref _refCount);
		}

		/// <summary>
		/// Decrement the reference count on this object, and dispose of it once it reaches zero
		/// </summary>
		public void Release()
		{
			if (Interlocked.Decrement(ref _refCount) == 0)
			{
				Dispose();
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		protected virtual void Dispose(bool disposing)
		{
			_headerPtr = default;
			_chunks = [];
		}

		/// <summary>
		/// Signals a read event
		/// </summary>
		public abstract void SetReadEvent(int readerIdx);

		/// <summary>
		/// Signals read events for every reader
		/// </summary>
		public void SetAllReadEvents()
		{
			for (int readerIdx = 0; readerIdx < _headerPtr.NumReaders; readerIdx++)
			{
				SetReadEvent(readerIdx);
			}
		}

		/// <summary>
		/// Waits for a read event to be signalled
		/// </summary>
		public abstract Task WaitToReadAsync(int readerIdx, CancellationToken cancellationToken);

		/// <summary>
		/// Signals the write event
		/// </summary>
		public abstract void SetWriteEvent();

		/// <summary>
		/// Waits for the write event to be signalled
		/// </summary>
		public abstract Task WaitToWriteAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Allocate a new reader
		/// </summary>
		public int CreateReader()
		{
			for (int readerIdx = 0; readerIdx < _headerPtr.NumReaders; readerIdx++)
			{
				ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
				for (; ; )
				{
					ReaderState readerState = readerStatePtr.Get();
					if (readerState.RefCount > 0)
					{
						break;
					}
					if (readerStatePtr.TryUpdate(readerState, new ReaderState(0, 0, 1, false)))
					{
						WriterStatePtr writerStatePtr = _headerPtr.GetWriterStatePtr();
						for (; ; )
						{
							WriterState writerState = writerStatePtr.Get();
							if (writerState.HasWrapped)
							{
								throw new InvalidOperationException("Cannot create a new reader after writer has wrapped back to the first chunk");
							}
							if (writerStatePtr.TryUpdate(writerState, new WriterState(writerState.ChunkIdx, writerState.ReaderFlags | (1 << readerIdx), writerState.RefCount, writerState.HasWrapped)))
							{
								for (int writeChunkIdx = 0; writeChunkIdx <= writerState.ChunkIdx; writeChunkIdx++)
								{
									_headerPtr.GetChunkStatePtr(writeChunkIdx).StartReading(readerIdx);
								}
								return readerIdx;
							}
						}
					}
				}
			}
			throw new InvalidOperationException("Unable to allocate reader; all available readers are in use.");
		}

		public void AddReaderRef(int readerIdx)
		{
			ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
			for (; ; )
			{
				ReaderState readerState = readerStatePtr.Get();
				if (readerState.RefCount == 0)
				{
					throw new InvalidOperationException("Refcount for reader is zero");
				}
				if (readerStatePtr.TryUpdate(readerState, new ReaderState(readerState.ChunkIdx, readerState.Offset, readerState.RefCount + 1, readerState.Detached)))
				{
					break;
				}
			}
		}

		public void ReleaseReaderRef(int readerIdx)
		{
			ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
			for (; ; )
			{
				ReaderState readerState = readerStatePtr.Get();
				if (readerState.RefCount == 0)
				{
					throw new InvalidOperationException("Refcount for reader is already zero");
				}

				if (readerState.RefCount == 1)
				{
					for (int idx = 0; idx < _headerPtr.NumChunks; idx++)
					{
						_headerPtr.GetChunkStatePtr(idx).FinishReading(readerIdx);
					}
				}

				if (readerStatePtr.TryUpdate(readerState, new ReaderState(readerState.ChunkIdx, readerState.Offset, readerState.RefCount - 1, readerState.Detached)))
				{
					break;
				}
			}
		}

		public void CreateWriter()
		{
			WriterStatePtr writerStatePtr = _headerPtr.GetWriterStatePtr();
			for (; ; )
			{
				WriterState writerState = writerStatePtr.Get();
				if (writerState.RefCount > 0)
				{
					throw new InvalidOperationException("Writer has already been created for this buffer");
				}
				if (writerStatePtr.TryUpdate(writerState, new WriterState(writerState.ChunkIdx, writerState.ReaderFlags, 1, writerState.HasWrapped)))
				{
					ChunkStatePtr chunkStatePtr = _headerPtr.GetChunkStatePtr(writerState.ChunkIdx);
					for (; ; )
					{
						ChunkState chunkState = chunkStatePtr.Get();
						if (chunkStatePtr.TryUpdate(chunkState, new ChunkState(WriteState.Writing, chunkState.ReaderFlags, chunkState.Length)))
						{
							break;
						}
					}
					break;
				}
			}
		}

		public void AddWriterRef()
		{
			WriterStatePtr writerStatePtr = _headerPtr.GetWriterStatePtr();
			for (; ; )
			{
				WriterState writerState = writerStatePtr.Get();
				if (writerState.RefCount == 0)
				{
					throw new InvalidOperationException("Writer does not exist for this buffer");
				}
				if (writerStatePtr.TryUpdate(writerState, new WriterState(writerState.ChunkIdx, writerState.ReaderFlags, writerState.RefCount + 1, writerState.HasWrapped)))
				{
					break;
				}
			}
		}

		public void ReleaseWriterRef()
		{
			WriterStatePtr writerStatePtr = _headerPtr.GetWriterStatePtr();
			for (; ; )
			{
				WriterState writerState = writerStatePtr.Get();
				if (writerState.RefCount == 0)
				{
					throw new InvalidOperationException("Writer does not exist for this buffer");
				}

				if (writerState.RefCount == 1)
				{
					MarkComplete();
				}

				if (writerStatePtr.TryUpdate(writerState, new WriterState(writerState.ChunkIdx, writerState.ReaderFlags, writerState.RefCount - 1, writerState.HasWrapped)))
				{
					break;
				}
			}
		}

		/// <inheritdoc/>
		public bool IsComplete(int readerIdx)
		{
			ReaderState readerState = _headerPtr.GetReaderStatePtr(readerIdx).Get();
			if (readerState.Detached)
			{
				return true;
			}

			ChunkState chunkState = _headerPtr.GetChunkStatePtr(readerState.ChunkIdx).Get();
			return chunkState.WriteState == WriteState.Complete && readerState.Offset == chunkState.Length;
		}

		/// <inheritdoc/>
		public void DetachReader(int readerIdx)
		{
			ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
			for (; ; )
			{
				ReaderState readerState = readerStatePtr.Get();
				if (readerStatePtr.TryUpdate(readerState, new ReaderState(readerState.ChunkIdx, readerState.Offset, readerState.RefCount, true)))
				{
					SetReadEvent(readerIdx);
					break;
				}
			}
		}

		/// <inheritdoc/>
		public void AdvanceReadPosition(int readerIdx, int offset)
		{
			ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
			for (; ; )
			{
				ReaderState readerState = readerStatePtr.Get();
				if (readerStatePtr.TryUpdate(readerState, new ReaderState(readerState.ChunkIdx, readerState.Offset + offset, readerState.RefCount, readerState.Detached)))
				{
					SetReadEvent(readerIdx);
					break;
				}
			}
		}

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> GetReadBuffer(int readerIdx)
		{
			ReaderState readerState = _headerPtr.GetReaderStatePtr(readerIdx).Get();

			ChunkStatePtr chunkStatePtr = _headerPtr.GetChunkStatePtr(readerState.ChunkIdx);
			ChunkState chunkState = chunkStatePtr.Get();

			if (chunkState.HasReaderFlag(readerIdx))
			{
				return _chunks[readerState.ChunkIdx].Slice(readerState.Offset, chunkState.Length - readerState.Offset);
			}
			else
			{
				return ReadOnlyMemory<byte>.Empty;
			}
		}

		/// <inheritdoc/>
		public async ValueTask<bool> WaitToReadAsync(int readerIdx, int minLength, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				ReaderStatePtr readerStatePtr = _headerPtr.GetReaderStatePtr(readerIdx);
				ReaderState readerState = readerStatePtr.Get();
				if (readerState.Detached)
				{
					return false;
				}

				ChunkStatePtr chunkStatePtr = _headerPtr.GetChunkStatePtr(readerState.ChunkIdx);
				ChunkState chunkState = chunkStatePtr.Get();

				if (!chunkState.HasReaderFlag(readerIdx))
				{
					// Wait until the current chunk is readable
					await WaitToReadAsync(readerIdx, cancellationToken);
				}
				else if (readerState.Offset + minLength <= chunkState.Length)
				{
					// We have enough data in the chunk to be able to read a message
					return true;
				}
				else if (chunkState.WriteState == WriteState.Writing)
				{
					// Wait until there is more data in the chunk
					await WaitToReadAsync(readerIdx, cancellationToken);
				}
				else if (readerState.Offset < chunkState.Length || chunkState.WriteState == WriteState.Complete)
				{
					// Cannot read the requested amount of data from this chunk.
					return false;
				}
				else if (chunkState.WriteState == WriteState.MovedToNext)
				{
					// Move to the next chunk
					chunkStatePtr.FinishReading(readerIdx);
					SetWriteEvent();

					int nextChunkIdx = readerState.ChunkIdx + 1;
					if (nextChunkIdx == _headerPtr.NumChunks)
					{
						nextChunkIdx = 0;
					}

					readerStatePtr.TryUpdate(readerState, new ReaderState(nextChunkIdx, 0, readerState.RefCount, readerState.Detached));
				}
				else
				{
					throw new NotImplementedException($"Invalid write state for buffer: {chunkState.WriteState}");
				}
			}
		}

		/// <inheritdoc/>
		public bool MarkComplete()
		{
			WriterState writerState = _headerPtr.GetWriterStatePtr().Get();

			ChunkStatePtr chunkStatePtr = _headerPtr.GetChunkStatePtr(writerState.ChunkIdx);
			if (chunkStatePtr.Get().WriteState != WriteState.Complete)
			{
				chunkStatePtr.MarkComplete();
				SetAllReadEvents();
				return true;
			}

			return false;
		}

		/// <inheritdoc/>
		public void AdvanceWritePosition(int size)
		{
			if (size > 0)
			{
				WriterState writerState = _headerPtr.GetWriterStatePtr().Get();

				ChunkStatePtr chunkStatePtr = _headerPtr.GetChunkStatePtr(writerState.ChunkIdx);
				ChunkState chunkState = chunkStatePtr.Get();

				Debug.Assert(chunkState.WriteState == WriteState.Writing);
				chunkStatePtr.Append(size);

				SetAllReadEvents();
			}
		}

		/// <inheritdoc/>
		public Memory<byte> GetWriteBuffer()
		{
			WriterState writerState = _headerPtr.GetWriterStatePtr().Get();

			ChunkState chunkState = _headerPtr.GetChunkStatePtr(writerState.ChunkIdx).Get();
			if (chunkState.WriteState == WriteState.Writing)
			{
				return _chunks[writerState.ChunkIdx].Slice(chunkState.Length);
			}
			else
			{
				return Memory<byte>.Empty;
			}
		}

		/// <inheritdoc/>
		public async ValueTask WaitToWriteAsync(int minSize, CancellationToken cancellationToken = default)
		{
			if (minSize > _headerPtr.ChunkLength)
			{
				throw new ArgumentException("Requested read size is larger than chunk size.", nameof(minSize));
			}

			// Get the current chunk we're writing to
			WriterState writerState = _headerPtr.GetWriterStatePtr().Get();
			int writeChunkIdx = writerState.ChunkIdx;

			ChunkStatePtr writeChunkStatePtr = _headerPtr.GetChunkStatePtr(writeChunkIdx);

			// Check if we can append to this chunk
			ChunkState chunkState = writeChunkStatePtr.Get();
			if (chunkState.WriteState == WriteState.Writing)
			{
				int length = chunkState.Length;
				if (length + minSize <= _headerPtr.ChunkLength)
				{
					return;
				}

				writeChunkStatePtr.FinishWriting();
				SetAllReadEvents();
			}

			if (chunkState.WriteState == WriteState.Complete)
			{
				return;
			}

			// Otherwise get the next chunk to write to
			int nextWriteChunkIdx = writeChunkIdx + 1;
			if (nextWriteChunkIdx == _chunks.Length)
			{
				nextWriteChunkIdx = 0;
			}

			// Wait until all readers have finished with the chunk, and we can update the writer to match
			ChunkStatePtr nextWriteChunkStatePtr = _headerPtr.GetChunkStatePtr(nextWriteChunkIdx);
			for (; ; )
			{
				ChunkState nextWriteChunkState = nextWriteChunkStatePtr.Get();
				if (nextWriteChunkState.ReaderFlags != 0)
				{
					await WaitToWriteAsync(cancellationToken);
				}
				else if (nextWriteChunkStatePtr.TryUpdate(nextWriteChunkState, new ChunkState(WriteState.Writing, writerState.ReaderFlags, 0)))
				{
					WriterStatePtr writerStatePtr = _headerPtr.GetWriterStatePtr();
					if (writerStatePtr.TryUpdate(writerState, new WriterState(nextWriteChunkIdx, writerState.ReaderFlags, writerState.RefCount, nextWriteChunkIdx == 0)))
					{
						break;
					}
					else
					{
						writerState = writerStatePtr.Get();
					}
				}
			}
		}
	}
}