// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Conventional TCP-like interface for writing data to a socket. Sends are "push", receives are "pull".
	/// </summary>
	public sealed class ComputeChannel : IDisposable
	{
		/// <summary>
		/// Reader for the channel
		/// </summary>
		public ComputeBufferReader Reader { get; }

		/// <summary>
		/// Writer for the channel
		/// </summary>
		public ComputeBufferWriter Writer { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="recvBufferReader"></param>
		/// <param name="sendBufferWriter"></param>
		internal ComputeChannel(ComputeBufferReader recvBufferReader, ComputeBufferWriter sendBufferWriter)
		{
			Reader = recvBufferReader.AddRef();
			Writer = sendBufferWriter.AddRef();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Reader.Dispose();
			Writer.Dispose();
		}

		/// <summary>
		/// Sends data to a remote channel
		/// </summary>
		/// <param name="memory">Memory to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public ValueTask SendAsync(ReadOnlyMemory<byte> memory, CancellationToken cancellationToken = default) => Writer.WriteAsync(memory, cancellationToken);

		/// <summary>
		/// Marks a channel as complete
		/// </summary>
		/// <param name="buffer">Buffer to receive the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => Reader.ReadAsync(buffer, cancellationToken);

		/// <summary>
		/// Reads a complete message from the given socket, retrying reads until the buffer is full.
		/// </summary>
		/// <param name="buffer">Buffer to store the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask RecvMessageAsync(Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			if (!await TryRecvMessageAsync(buffer, cancellationToken))
			{
				throw new EndOfStreamException();
			}
		}

		/// <summary>
		/// Reads either a full message or end of stream from the channel
		/// </summary>
		/// <param name="buffer">Buffer to store the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask<bool> TryRecvMessageAsync(Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			for (int offset = 0; offset < buffer.Length;)
			{
				int read = await RecvAsync(buffer.Slice(offset), cancellationToken);
				if (read == 0)
				{
					return false;
				}
				offset += read;
			}
			return true;
		}

		/// <summary>
		/// Mark the channel as complete (ie. that no more data will be sent)
		/// </summary>
		public void MarkComplete() => Writer.MarkComplete();
	}
}
