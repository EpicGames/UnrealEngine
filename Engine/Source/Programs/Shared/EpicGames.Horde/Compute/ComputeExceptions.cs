// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Generic class for compute errors
	/// </summary>
	public class ComputeException : Exception
	{
		/// <inheritdoc/>
		public ComputeException(string message) : base(message)
		{
		}

		/// <inheritdoc/>
		public ComputeException(string? message, Exception? innerException) : base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Exception thrown for internal reasons
	/// </summary>
	public sealed class ComputeInternalException : ComputeException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		public ComputeInternalException(string message)
			: base(message)
		{
		}
	}

	/// <summary>
	/// Exception thrown on a remote machine
	/// </summary>
	public sealed class ComputeRemoteException : ComputeException
	{
		readonly string _description;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeRemoteException(ExceptionMessage message)
			: this("Remote exception: " + message.Message, message.Description)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeRemoteException(string message, string description)
			: base(message)
		{
			_description = "From remote machine: " + description;
		}

		/// <inheritdoc/>
		public override string ToString() => _description;
	}
}
