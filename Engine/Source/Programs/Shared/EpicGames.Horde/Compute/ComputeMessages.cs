// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;

#pragma warning disable CA2227 // Collection properties should be read only

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Describes port used by a compute task
	/// </summary>
	public class ConnectionMetadataPort
	{
		/// <summary>
		/// Built-in port used for agent and compute task communication
		/// </summary>
		public const string ComputeId = "_horde_compute";

		/// <summary>
		/// Externally visible port that is mapped to agent port
		/// In direct connection mode, these two are identical.
		/// </summary>
		public int Port { get; }

		/// <summary>
		/// Port the local process on the agent is listening on
		/// </summary>
		public int AgentPort { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="port"></param>
		/// <param name="agentPort"></param>
		public ConnectionMetadataPort(int port, int agentPort)
		{
			Port = port;
			AgentPort = agentPort;
		}
	}

	/// <summary>
	/// Request for allocating a UBA cache server/session
	/// </summary>
	public class AllocateUbaCacheServerRequest
	{
	}

	/// <summary>
	/// Configuration settings for Unreal Build Accelerator (UBA) for a compute task invocation
	/// Contains cache server connectivity and other UBA-related settings.
	/// </summary>
	public class UbaConfig(string cacheEndpoint, string cacheSessionKey, bool writeAccess)
	{
		/// <summary>
		/// Network endpoint (IP address and port) for connecting to the UBA cache server
		/// </summary>
		public string CacheEndpoint { get; } = cacheEndpoint;

		/// <summary>
		/// AES encryption key used for secure client-server communication with the UBA cache server
		/// </summary>
		public string CacheSessionKey { get; } = cacheSessionKey;

		/// <summary>
		/// Session has write access to cache
		/// </summary>
		public bool WriteAccess { get; } = writeAccess;
	}

	/// <summary>
	/// Request a machine to execute compute requests
	/// </summary>
	public class AssignComputeRequest
	{
		/// <summary>
		/// Desired protocol version for the client
		/// </summary>
		public int Protocol { get; set; }

		/// <summary>
		/// Condition to identify machines that can execute the request
		/// </summary>
		public Requirements? Requirements { get; set; }

		/// <summary>
		/// Arbitrary ID to correlate the same request over multiple calls.
		/// It's recommended to pick something globally unique, such as a UUID.
		/// </summary>
		public string? RequestId { get; set; }

		/// <summary>
		/// Details for making an agent connection
		/// </summary>
		public ConnectionMetadataRequest? Connection { get; set; }

		/// <summary>
		/// Whether to allocate an Unreal Build Accelerator (UBA) cache server session for this compute request.
		/// If requested, a UBA cache server must be defined for the compute cluster. Null is treated as false.
		/// </summary>
		public bool? UseUbaCache { get; init; }
	}

	/// <summary>
	/// Request details for making an agent connection
	/// </summary>
	public class ConnectionMetadataRequest
	{
		/// <summary>
		/// Public IP of client requesting a compute resource (initiator)
		/// As communication between client and Horde server may be on an internal network,
		/// the client is responsible for resolving and providing this information.
		/// </summary>
		public string? ClientPublicIp { get; set; }

		/// <summary>
		/// TCP/IP ports the compute resource will listen to.
		/// Key = arbitrary name identifying the port
		/// Value = actual port number
		/// Relay connection mode uses this information to set up port forwarding.
		/// </summary>
		public Dictionary<string, int> Ports { get; set; } = [];

		/// <summary>
		/// Type of connection mode that is preferred by the client. Server can still override.
		/// </summary>
		public ConnectionMode? ModePreference { get; set; }

		/// <summary>
		/// Prefer connecting to agent over a public IP even if a more optimal route is available. Server can still override.
		/// This is useful to avoid sending traffic over VPN tunnels.
		/// </summary>
		public bool? PreferPublicIp { get; set; }

		/// <summary>
		/// Encryption mode to request. Server can still override.
		/// </summary>
		public Encryption? Encryption { get; set; }

		/// <summary>
		/// Maximum duration (in milliseconds) that communication can be inactive between
		/// the compute client and task-running agent before the connection is terminated.
		/// </summary>
		public int? InactivityTimeoutMs { get; set; }
	}

	/// <summary>
	/// Response to a cluster lookup request
	/// </summary>
	public class GetClusterResponse
	{
		/// <summary>
		/// Compute cluster ID
		/// </summary>
		public ClusterId ClusterId { get; set; }
	}

	/// <summary>
	/// Response to compute allocation request
	/// </summary>
	public class AssignComputeResponse
	{
		/// <summary>
		/// IP address of the remote agent machine running the compute task
		/// </summary>
		public string Ip { get; set; } = String.Empty;

		/// <summary>
		/// How to establish a connection to the remote machine
		/// </summary>
		public ConnectionMode ConnectionMode { get; set; }

		/// <summary>
		/// An optional address (host:port) to use when connecting to agent via tunnel or relay mode
		/// </summary>
		public string? ConnectionAddress { get; set; }

		/// <summary>
		/// Port number on the remote machine
		/// </summary>
		public int Port { get; set; }

		/// <summary>
		/// Assigned ports (externally visible port -> local port on agent)
		/// Key is an arbitrary name identifying the port (same as was given in <see cref="ConnectionMetadataRequest" />)
		/// When relay mode is used, ports can mapped to a different externally visible port.
		/// If compute task uses and listens to port 7000, that port can be externally represented as something else.
		/// For example, port 32743 can be pointed to port 7000.
		/// This makes no difference for the compute task process, but the client/initiator making connections must
		/// pay attention to this mapping.
		/// </summary>
		public IReadOnlyDictionary<string, ConnectionMetadataPort> Ports { get; set; } = new Dictionary<string, ConnectionMetadataPort>();

		/// <summary>
		/// Encryption used
		/// </summary>
		public Encryption Encryption { get; set; } = Encryption.None;

		/// <summary>
		/// Cryptographic nonce to identify the request, as a hex string
		/// </summary>
		public string Nonce { get; set; } = String.Empty;

		/// <summary>
		/// AES key for the channel, as a hex string
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// X.509 certificate used for SSL/TLS encryption
		/// </summary>
		public string Certificate { get; set; } = String.Empty;

		/// <inheritdoc cref="UbaConfig" />
		public UbaConfig? Uba { get; set; }

		/// <summary>
		/// Which cluster this remote machine belongs to
		/// </summary>
		public ClusterId ClusterId { get; set; }

		/// <summary>
		/// Identifier for the remote machine
		/// </summary>
		public AgentId AgentId { get; set; }

		/// <summary>
		/// Agent version for the remote machine
		/// </summary>
		public string? AgentVersion { get; set; }

		/// <summary>
		/// Identifier for the new lease on the remote machine
		/// </summary>
		public LeaseId LeaseId { get; set; }

		/// <summary>
		/// Resources assigned to this machine
		/// </summary>
		public Dictionary<string, int> AssignedResources { get; set; } = [];

		/// <summary>
		/// Version number for the compute protocol
		/// </summary>
		public int Protocol { get; set; }

		/// <summary>
		/// Properties of the agent assigned to do the work
		/// </summary>
		public IReadOnlyList<string> Properties { get; set; } = [];
	}

	/// <summary>
	/// Describe how to connect to the remote machine
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum ConnectionMode
	{
		/// <summary>
		/// Connection is established directly to remote machine, behaving like a normal TCP/UDP connection
		/// </summary>
		Direct,

		/// <summary>
		/// Connection is tunneled through Horde server.
		/// When connecting, initiator must send a tunnel handshake request indicating which machine/IP to tunnel to.
		/// Once handshake is complete, TCP connection behaves as normal (UDP not supported)
		/// </summary>
		Tunnel,

		/// <summary>
		/// Connection is established to remote machine via a relay.
		/// Forwarding is transparent and behaves like a normal TCP/UDP connection.
		/// </summary>
		Relay
	}

	/// <summary>
	/// Describe encryption for the compute resource connection
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum Encryption
	{
		/// <summary>
		/// No encryption enabled
		/// </summary>
		None,

		/// <summary>
		/// Use custom AES-based encryption transport
		/// </summary>
		Aes,

		/// <summary>
		/// Use SSL/TLS encryption with RSA 2048-bits
		/// </summary>
		Ssl,

		/// <summary>
		/// Use SSL/TLS encryption with ECDSA P-256
		/// </summary>
		SslEcdsaP256
	}

	/// <summary>
	/// Resource needs declaration request
	/// </summary>
	public class ResourceNeedsMessage
	{
		/// <summary>
		/// Unique session ID performing compute resource requests
		/// </summary>
		public string SessionId { get; set; } = String.Empty;

		/// <summary>
		/// Pool of agents requesting resources from
		/// </summary>
		public string Pool { get; set; } = String.Empty;

		/// <summary>
		/// Key/value of resources needed by session (such as CPU or memory, see KnownPropertyNames in Horde.Server)
		/// </summary>
		public Dictionary<string, int> ResourceNeeds { get; set; } = [];
	}

	/// <summary>
	/// Resource needs response
	/// </summary>
	public class GetResourceNeedsResponse
	{
		/// <summary>
		/// List of resource needs
		/// </summary>
		public List<ResourceNeedsMessage> ResourceNeeds { get; set; } = [];
	}
}
