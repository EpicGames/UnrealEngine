// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CA1027 // Use [Flags] attribute.
#pragma warning disable CA1069 // Overlapping constants in enum.

using System;
using EpicGames.Horde.Storage.Bundles;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Version number for the compute protocol
	/// </summary>
	public enum ComputeProtocol
	{
		/// <summary>
		/// No version specified
		/// </summary>
		Unknown,

		/// <summary>
		/// Initial version number
		/// </summary>
		Initial,

		/// <summary>
		/// Set new env vars UE_HORDE_CPU_COUNT and UE_HORDE_CPU_MULTIPLIER
		/// </summary>
		NewCpuEnvVars,

		/// <summary>
		/// Constant for the latest protocol version
		/// </summary>
		Latest = (int)NewCpuEnvVars
	}

	/// <summary>
	/// Helper methods for compute protocol version numbers
	/// </summary>
	public static class ComputeProtocolUtilities
	{
		/// <summary>
		/// Gets the appropriate bundle options for a compute protocol version number
		/// </summary>
		public static BundleOptions GetBundleOptions(ComputeProtocol protocol)
		{
			if (protocol <= ComputeProtocol.Latest)
			{
				return new BundleOptions { MaxVersion = BundleVersion.PacketSequence };
			}
			else
			{
				throw new NotSupportedException($"Unknown compute protocol version ('{(int)protocol}')");
			}
		}
	}
}
