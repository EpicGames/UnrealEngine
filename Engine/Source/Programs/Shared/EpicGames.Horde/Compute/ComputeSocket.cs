// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Net.Sockets;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Compute.Buffers;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Socket for sending and reciving data using a "push" model. The application can attach multiple writers to accept received data.
	/// </summary>
	public abstract class ComputeSocket
	{
		/// <summary>
		/// The current protocol number
		/// </summary>
		public abstract ComputeProtocol Protocol { get; }

		/// <summary>
		/// Logger for diagnostic messages
		/// </summary>
		public abstract ILogger Logger { get; }

		/// <summary>
		/// Attaches a buffer to receive data.
		/// </summary>
		/// <param name="channelId">Channel to receive data on</param>
		/// <param name="recvBuffer">Writer for the buffer to store received data</param>
		public abstract void AttachRecvBuffer(int channelId, ComputeBuffer recvBuffer);

		/// <summary>
		/// Attaches a buffer to send data.
		/// </summary>
		/// <param name="channelId">Channel to receive data on</param>
		/// <param name="sendBuffer">Reader for the buffer to send data from</param>
		public abstract void AttachSendBuffer(int channelId, ComputeBuffer sendBuffer);

		/// <summary>
		/// Creates a channel using a socket and receive buffer
		/// </summary>
		/// <param name="channelId">Channel id to send and receive data</param>
		public ComputeChannel CreateChannel(int channelId)
		{
			using SharedMemoryBuffer recvBuffer = SharedMemoryBuffer.CreateNew(null, 65536);
			using SharedMemoryBuffer sendBuffer = SharedMemoryBuffer.CreateNew(null, 65536);
			return CreateChannel(channelId, recvBuffer, sendBuffer);
		}

		/// <summary>
		/// Creates a channel using a socket and receive buffer
		/// </summary>
		/// <param name="channelId">Channel id to send and receive data</param>
		/// <param name="recvBuffer">Buffer for receiving data</param>
		/// <param name="sendBuffer">Buffer for sending data</param>
		public ComputeChannel CreateChannel(int channelId, ComputeBuffer recvBuffer, ComputeBuffer sendBuffer)
		{
			AttachRecvBuffer(channelId, recvBuffer);
			AttachSendBuffer(channelId, sendBuffer);

			using ComputeBufferReader recvBufferReader = recvBuffer.CreateReader();
			using ComputeBufferWriter sendBufferWriter = sendBuffer.CreateWriter();

			return new ComputeChannel(recvBufferReader, sendBufferWriter);
		}
	}

	internal enum IpcMessage
	{
		AttachRecvBuffer = 0,
		AttachSendBuffer = 1,
	}

	/// <summary>
	/// Provides functionality for attaching buffers for compute workers 
	/// </summary>
	public sealed class WorkerComputeSocket : ComputeSocket, IDisposable
	{
		/// <summary>
		/// Name of the environment variable for passing the name of the compute channel
		/// </summary>
		public const string IpcEnvVar = "UE_HORDE_COMPUTE_IPC";

		readonly ComputeBufferWriter _commandBufferWriter;
		readonly List<ComputeBuffer> _buffers = [];
		readonly ILogger _logger;

		/// <inheritdoc/>
		public override ComputeProtocol Protocol => ComputeProtocol.Unknown;

		/// <inheritdoc/>
		public override ILogger Logger => _logger;

		/// <summary>
		/// Creates a socket for a worker
		/// </summary>
		private WorkerComputeSocket(ComputeBufferWriter commandBufferWriter, ILogger logger)
		{
			_commandBufferWriter = commandBufferWriter;
			_logger = logger;
		}

		/// <summary>
		/// Opens a socket which allows a worker to communicate with the Horde Agent
		/// </summary>
		public static WorkerComputeSocket Open() => Open(NullLogger.Instance);

		/// <summary>
		/// Opens a socket which allows a worker to communicate with the Horde Agent
		/// </summary>
		/// <param name="logger">Logger for diagnostic messages</param>
		public static WorkerComputeSocket Open(ILogger logger)
		{
			string? baseName = Environment.GetEnvironmentVariable(IpcEnvVar);
			if (baseName == null)
			{
				throw new InvalidOperationException($"Environment variable {IpcEnvVar} is not defined; cannot connect as worker.");
			}

			return Open(baseName, logger);
		}

		/// <summary>
		/// Opens a socket which allows a worker to communicate with the Horde Agent
		/// </summary>
		/// <param name="commandBufferName">Name of the command buffer</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public static WorkerComputeSocket Open(string commandBufferName, ILogger logger)
		{
			using SharedMemoryBuffer commandBuffer = SharedMemoryBuffer.OpenExisting(commandBufferName);
			return new WorkerComputeSocket(commandBuffer.CreateWriter(), logger);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_commandBufferWriter.Dispose();
		}

		/// <inheritdoc/>
		public override void AttachRecvBuffer(int channelId, ComputeBuffer buffer)
		{
			_buffers.Add(buffer.AddRef());
			string bufferName = ((SharedMemoryBufferDetail)buffer._detail).Name;
			AttachBuffer(IpcMessage.AttachRecvBuffer, channelId, bufferName);
		}

		/// <inheritdoc/>
		public override void AttachSendBuffer(int channelId, ComputeBuffer buffer)
		{
			_buffers.Add(buffer.AddRef());
			string bufferName = ((SharedMemoryBufferDetail)buffer._detail).Name;
			AttachBuffer(IpcMessage.AttachSendBuffer, channelId, bufferName);
		}

		void AttachBuffer(IpcMessage message, int channelId, string bufferName)
		{
			MemoryWriter writer = new(_commandBufferWriter.GetWriteBuffer());
			writer.WriteUnsignedVarInt((int)message);
			writer.WriteUnsignedVarInt(channelId);
			writer.WriteString(bufferName);

			_commandBufferWriter.AdvanceWritePosition(writer.Length);
		}
	}

	/// <summary>
	/// Operates a server that a child process can open a <see cref="WorkerComputeSocket"/> to.
	/// </summary>
	public sealed class WorkerComputeSocketBridge : IAsyncDisposable
	{
		readonly SharedMemoryBuffer _ipcBuffer;
		readonly ComputeBufferReader _ipcBufferReader;
		readonly BackgroundTask _backgroundTask;
		readonly ILogger _logger;

		/// <summary>
		/// Name of the buffer to pass via <see cref="WorkerComputeSocket.IpcEnvVar"/>
		/// </summary>
		public string BufferName => _ipcBuffer.Name;

		/// <summary>
		/// Constructor
		/// </summary>
		private WorkerComputeSocketBridge(SharedMemoryBuffer ipcBuffer, ComputeBufferReader ipcBufferReader, BackgroundTask backgroundTask, ILogger logger)
		{
			_ipcBuffer = ipcBuffer;
			_ipcBufferReader = ipcBufferReader;
			_backgroundTask = backgroundTask;
			_logger = logger;
		}

		/// <summary>
		/// Creates a new server for <see cref="WorkerComputeSocket"/>
		/// </summary>
		/// <param name="socket">Socket to connect to</param>
		/// <param name="logger">Logger for errors</param>
		/// <returns>New server instance</returns>
		public static async Task<WorkerComputeSocketBridge> CreateAsync(ComputeSocket socket, ILogger logger)
		{
			SharedMemoryBuffer? ipcBuffer = null;
			ComputeBufferReader? ipcBufferReader = null;
			BackgroundTask? backgroundTask = null;
			try
			{
				ipcBuffer = SharedMemoryBuffer.CreateNew(null, 1, 64 * 1024);
				ipcBufferReader = ipcBuffer.CreateReader();
				backgroundTask = BackgroundTask.StartNew(ctx => ProcessIpcMessagesAsync(socket, ipcBufferReader, logger, ctx));
				return new WorkerComputeSocketBridge(ipcBuffer, ipcBufferReader, backgroundTask, logger);
			}
			catch
			{
				if (backgroundTask != null)
				{
					await backgroundTask.DisposeAsync();
				}
				ipcBufferReader?.Dispose();
				ipcBuffer?.Dispose();
				throw;
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
			_logger.LogDebug("Ipc message loop is complete");

			_ipcBufferReader.Dispose();
			_logger.LogDebug("Closed IPC reader");

			_ipcBuffer.Dispose();
			_logger.LogDebug("Closed IPC buffer");
		}

		static async Task ProcessIpcMessagesAsync(ComputeSocket socket, ComputeBufferReader ipcReader, ILogger logger, CancellationToken cancellationToken)
		{
			List<SharedMemoryBuffer> buffers = [];
			try
			{
				List<(int, ComputeBufferWriter)> writers = [];
				while (await ipcReader.WaitToReadAsync(1, cancellationToken))
				{
					ReadOnlyMemory<byte> memory = ipcReader.GetReadBuffer();
					MemoryReader reader = new(memory);

					IpcMessage message = (IpcMessage)reader.ReadUnsignedVarInt();
					try
					{
						switch (message)
						{
							case IpcMessage.AttachSendBuffer:
								{
									int channelId = (int)reader.ReadUnsignedVarInt();
									string name = reader.ReadString();
									logger.LogDebug("Attaching send buffer for channel {ChannelId} to {Name}", channelId, name);

									SharedMemoryBuffer buffer = SharedMemoryBuffer.OpenExisting(name);
									buffers.Add(buffer);

									socket.AttachSendBuffer(channelId, buffer);
								}
								break;
							case IpcMessage.AttachRecvBuffer:
								{
									int channelId = (int)reader.ReadUnsignedVarInt();
									string name = reader.ReadString();
									logger.LogDebug("Attaching recv buffer for channel {ChannelId} to {Name}", channelId, name);

									SharedMemoryBuffer buffer = SharedMemoryBuffer.OpenExisting(name);
									buffers.Add(buffer);

									socket.AttachRecvBuffer(channelId, buffer);
								}
								break;
							default:
								throw new InvalidOperationException($"Invalid IPC message: {message}");
						}
					}
					catch (Exception ex)
					{
						logger.LogError(ex, "Exception while processing messages from child process: {Message}", ex.Message);
					}

					ipcReader.AdvanceReadPosition(memory.Length - reader.RemainingMemory.Length);
				}
			}
			catch (OperationCanceledException)
			{
				logger.LogDebug("Ipc message loop cancelled");
			}
			finally
			{
				foreach (SharedMemoryBuffer buffer in buffers)
				{
					buffer.Dispose();
				}
			}
		}
	}

	/// <summary>
	/// Manages a set of readers and writers to buffers across a transport layer
	/// </summary>
	public class RemoteComputeSocket : ComputeSocket, IAsyncDisposable
	{
		enum ControlMessageType
		{
			Detach = -2,
			KeepAlive = -3,
		}

		class RecvBuffer : IDisposable
		{
			public ComputeBufferWriter? _writer;
			public readonly SemaphoreSlim Semaphore = new(1);
			public int _refCount = 1;

			public RecvBuffer(ComputeBufferWriter writer) => _writer = writer;

			public void AddRef() => Interlocked.Increment(ref _refCount);

			public void Release()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					Dispose();
				}
			}

			public void Dispose()
			{
				_writer?.Dispose();
				Semaphore.Dispose();
			}
		}

		class SendBuffer : IDisposable
		{
			public ComputeBufferReader? _reader;
			public readonly SemaphoreSlim Semaphore = new(1);
			[SuppressMessage("Usage", "CA2213:Disposable fields should be disposed")]
			public readonly BackgroundTask Task;
			int _refCount = 1;

			public SendBuffer(ComputeBufferReader reader, Func<SendBuffer, CancellationToken, Task> func)
			{
				_reader = reader;
				Task = BackgroundTask.StartNew(ctx => func(this, ctx));
			}

			public void AddRef() => Interlocked.Increment(ref _refCount);

			public void Release()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					Dispose();
				}
			}

			public void Dispose()
			{
				_reader?.Dispose();
				Semaphore.Dispose();
			}
		}

		readonly Lock _lockObject = new();

		bool _complete;
		bool _isClosing;

		readonly ComputeTransport _transport;
		readonly ComputeProtocol _protocol;
		readonly ILogger _logger;

		BackgroundTask? _recvTask;
		readonly Dictionary<int, RecvBuffer> _recvBuffers = [];

		readonly SemaphoreSlim _sendSemaphore = new(1, 1);
		readonly Dictionary<int, SendBuffer> _sendBuffers = [];

		/// <inheritdoc/>
		public override ComputeProtocol Protocol => _protocol;

		/// <inheritdoc/>
		public override ILogger Logger => _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="transport">Transport to communicate with the remote</param>
		/// <param name="protocol">The protocol version number</param>
		/// <param name="logger">Logger for trace output</param>
		public RemoteComputeSocket(ComputeTransport transport, ComputeProtocol protocol, ILogger logger)
		{
			_transport = transport;
			_protocol = protocol;
			_logger = logger;

			_recvTask = new BackgroundTask(ctx => RunRecvTaskAsync(_transport, ctx));
		}

		/// <summary>
		/// Attempt to gracefully close the current connection and shutdown both ends of the transport
		/// </summary>
		public async ValueTask CloseAsync(CancellationToken cancellationToken)
		{
			lock (_lockObject)
			{
				if (_isClosing)
				{
					return;
				}

				_isClosing = true;
			}

			try
			{
				// Close the transport layer, freeing the remote end to shutdown.
				await _transport.MarkCompleteAsync(cancellationToken);

				// Close all the buffers
				await DetachAllBuffersAsync(true, true, cancellationToken);

				// Wait for the reader to stop
				if (_recvTask != null)
				{
					await _recvTask.DisposeAsync();
					_recvTask = null;
				}
			}
			catch (SocketException se) when (se.SocketErrorCode is SocketError.ConnectionReset or SocketError.ConnectionAborted)
			{
				_logger.LogInformation("Socket already closed. Ignoring exception");
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await CloseAsync(CancellationToken.None);
			_sendSemaphore.Dispose();
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Sends a keep alive message to the remote machine. Does not wait for a response. Designed to keep a connection open when the remote is eagerly trying to close it.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task SendKeepAliveMessageAsync(CancellationToken cancellationToken)
		{
			_logger.LogDebug("Sending ping message");
			await SendInternalAsync(0, (int)ControlMessageType.KeepAlive, ReadOnlyMemory<byte>.Empty, cancellationToken);
		}

		async Task RunRecvTaskAsync(ComputeTransport transport, CancellationToken cancellationToken)
		{
			try
			{
				_logger.LogDebug("Started socket reader");

				List<Task> detachTasks = [];

				byte[] header = new byte[8];
				try
				{
					Memory<byte> last = Memory<byte>.Empty;

					// Process messages from the remote
					for (; ; )
					{
						detachTasks.RemoveCompleteTasks();

						// Read the next packet header
						if (!await transport.RecvOptionalAsync(header, cancellationToken))
						{
							_logger.LogDebug("End of socket");
							break;
						}

						// Parse the target buffer and packet size
						int id = BinaryPrimitives.ReadInt32LittleEndian(header);
						int size = BinaryPrimitives.ReadInt32LittleEndian(header.AsSpan(4));

						// Dispatch it to the correct place
						if (size >= 0)
						{
							await ReadPacketAsync(transport, id, size, cancellationToken);
						}
						else if (size == (int)ControlMessageType.Detach)
						{
							detachTasks.Add(DetachRecvBufferAsync(id, cancellationToken));
						}
						else if (size == (int)ControlMessageType.KeepAlive)
						{
							_logger.LogDebug("Received ping message");
						}
						else
						{
							_logger.LogDebug("Unrecognized control message: {Message}", size);
						}
					}
				}
				catch (OperationCanceledException)
				{
				}

				// Mark all buffers as complete
				lock (_lockObject)
				{
					_complete = true;
					foreach (int channelIdx in _recvBuffers.Keys)
					{
						detachTasks.Add(DetachRecvBufferAsync(channelIdx, cancellationToken));
					}
				}

				// Wait for all the detach tasks to finish
				if (detachTasks.Count > 0)
				{
					_logger.LogDebug("Waiting for detach tasks to complete...");
					await Task.WhenAll(detachTasks).WaitAsync(cancellationToken);
				}

				_logger.LogDebug("Closing reader");
			}
			catch (Exception e)
			{
				_logger.LogInformation(e, "Exception in background receive");
				throw;
			}
		}

		async Task ReadPacketAsync(ComputeTransport transport, int id, int size, CancellationToken cancellationToken)
		{
			if (!await TryReadPacketAsync(transport, id, size, cancellationToken))
			{
				_logger.LogInformation("Discarding {Size} bytes received on compute channel {Id}; no buffer attached?", size, id);

				int bufferSize = Math.Min(size, 65536);
				using (IMemoryOwner<byte> buffer = MemoryPool<byte>.Shared.Rent(bufferSize))
				{
					for (int remaining = size; remaining > 0;)
					{
						int chunkSize = Math.Min(bufferSize, remaining);
						await transport.RecvAsync(buffer.Memory.Slice(0, chunkSize), cancellationToken);
						remaining -= chunkSize;
					}
				}
			}
		}

		async ValueTask<bool> TryReadPacketAsync(ComputeTransport transport, int id, int size, CancellationToken cancellationToken)
		{
			// Try to get the receive buffer for this channel
			RecvBuffer? recvBuffer;
			lock (_lockObject)
			{
				if (_recvBuffers.TryGetValue(id, out recvBuffer))
				{
					recvBuffer.AddRef();
				}
				else
				{
					return false;
				}
			}

			// Lock it and read a packet
			bool result = false;
			try
			{
				await recvBuffer.Semaphore.WaitAsync(cancellationToken);
				try
				{
					ComputeBufferWriter? writer = recvBuffer._writer;
					if (writer != null)
					{
						Memory<byte> memory = writer.GetWriteBuffer();
						while (memory.Length < size)
						{
							_logger.LogDebug("No space in buffer {Id}, flushing", id);
							await writer.WaitToWriteAsync(size, cancellationToken);
							memory = writer.GetWriteBuffer();
						}

						for (int offset = 0; offset < size;)
						{
							int read = await transport.RecvAsync(memory.Slice(offset, size - offset), cancellationToken);
							if (read == 0)
							{
								// Return true to avoid logic in ReadPacketAsync() from trying to read the whole message from a closed stream
								_logger.LogDebug("Unexpected end of stream while parsing message for channel {Id}; discarding message.", id);
								return true;
							}
							offset += read;
						}

						writer.AdvanceWritePosition(size);
						result = true;
					}
				}
				finally
				{
					recvBuffer.Semaphore.Release();
				}
			}
			finally
			{
				recvBuffer.Release();
			}
			return result;
		}

		class SendSegment : ReadOnlySequenceSegment<byte>
		{
			public void Set(ReadOnlyMemory<byte> memory, ReadOnlySequenceSegment<byte>? next, long runningIndex)
			{
				Memory = memory;
				Next = next;
				RunningIndex = runningIndex;
			}
		}

		readonly byte[] _header = new byte[8];
		readonly SendSegment _headerSegment = new();
		readonly SendSegment _bodySegment = new();

		/// <inheritdoc/>
		async ValueTask SendAsync(int id, ReadOnlyMemory<byte> memory, CancellationToken cancellationToken = default)
		{
			if (memory.Length > 0)
			{
				await SendInternalAsync(id, memory.Length, memory, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public ValueTask MarkCompleteAsync(int id, CancellationToken cancellationToken = default) => SendInternalAsync(id, (int)ControlMessageType.Detach, ReadOnlyMemory<byte>.Empty, cancellationToken);

		async ValueTask SendInternalAsync(int id, int size, ReadOnlyMemory<byte> memory, CancellationToken cancellationToken)
		{
			await _sendSemaphore.WaitAsync(cancellationToken);
			try
			{
				BinaryPrimitives.WriteInt32LittleEndian(_header, id);
				BinaryPrimitives.WriteInt32LittleEndian(_header.AsSpan(4), size);
				_headerSegment.Set(_header, _bodySegment, 0);
				_bodySegment.Set(memory, null, _header.Length);

				ReadOnlySequence<byte> sequence = new(_headerSegment, 0, _bodySegment, memory.Length);
				await _transport.SendAsync(sequence, cancellationToken);
			}
			finally
			{
				_sendSemaphore.Release();
			}
		}

		/// <inheritdoc/>
		public override void AttachRecvBuffer(int channelId, ComputeBuffer recvBuffer)
		{
			_logger.LogDebug("Attaching recv buffer {Id}", channelId);
			lock (_lockObject)
			{
				if (_complete)
				{
					throw new InvalidOperationException($"Cannot attach new buffer to channel {channelId} after socket is closed");
				}
				if (_recvBuffers.ContainsKey(channelId))
				{
					throw new InvalidOperationException($"Buffer is already attached to channel {channelId}");
				}

				_recvBuffers.Add(channelId, new RecvBuffer(recvBuffer.CreateWriter()));

				// Only start the receive task once we have a buffer to receive data, otherwise we discard data from the remote
				if (_recvTask is { Task: null })
				{
					_recvTask.Start();
				}
			}
		}

		[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope")]
		async Task DetachRecvBufferAsync(int id, CancellationToken cancellationToken)
		{
			_logger.LogDebug("Detaching recv buffer {Id}", id);

			// Get the current receive buffer
			RecvBuffer? recvBuffer;
			lock (_lockObject)
			{
				if (!_recvBuffers.TryGetValue(id, out recvBuffer))
				{
					_logger.LogDebug("Buffer {Id} has already been detached", id);
					return;
				}
				recvBuffer.AddRef(); // Note: adding extra ref here
			}

			// Release the writer
			await recvBuffer.Semaphore.WaitAsync(cancellationToken);
			try
			{
				recvBuffer._writer?.Dispose();
				recvBuffer._writer = null;
			}
			finally
			{
				recvBuffer.Semaphore.Release();
				recvBuffer.Release(); // Ref added above
			}

			// Remove the buffer
			lock (_lockObject)
			{
				if (_recvBuffers.Remove(id, out recvBuffer))
				{
					recvBuffer.Release(); // Original ref from _recvBuffers
				}
			}
		}

		async Task DetachAllBuffersAsync(bool recvBuffers, bool sendBuffers, CancellationToken cancellationToken)
		{
			int[] recvChannelIds;
			int[] sendChannelIds;
			lock (_lockObject)
			{
				_complete = true;
				recvChannelIds = [.. _recvBuffers.Keys];
				sendChannelIds = [.. _sendBuffers.Keys];
			}

			List<Task> tasks = [];
			if (recvBuffers)
			{
				foreach (int recvChannelId in recvChannelIds)
				{
					tasks.Add(DetachRecvBufferAsync(recvChannelId, cancellationToken));
				}
			}
			if (sendBuffers)
			{
				foreach (int sendChannelId in sendChannelIds)
				{
					tasks.Add(DetachSendBufferAsync(sendChannelId, cancellationToken));
				}
			}
			await Task.WhenAll(tasks).WaitAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public override void AttachSendBuffer(int channelId, ComputeBuffer sendBuffer)
		{
			_logger.LogDebug("Attaching send buffer {Id}", channelId);
			lock (_lockObject)
			{
				if (_sendBuffers.ContainsKey(channelId))
				{
					throw new InvalidOperationException($"Buffer is already attached to channel {channelId}");
				}

				ComputeBufferReader sendBufferReader = sendBuffer.CreateReader();
				SendBuffer sendBufferInfo = new(sendBufferReader, (buffer, ctx) => SendFromBufferAsync(channelId, buffer, ctx));
				_sendBuffers.Add(channelId, sendBufferInfo);
			}
		}

		[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope")]
		async Task DetachSendBufferAsync(int channelId, CancellationToken cancellationToken)
		{
			SendBuffer? sendBuffer = null;
			try
			{
				// Get the current send buffer state
				lock (_lockObject)
				{
					if (!_sendBuffers.TryGetValue(channelId, out sendBuffer))
					{
						_logger.LogWarning("No buffer is attached to channel {ChannelId}", channelId);
						return;
					}
					sendBuffer.AddRef();
				}

				// Try to stop the send task
				TimeSpan timeout = TimeSpan.FromSeconds(10);
				try
				{
					await sendBuffer.Task.StopAsync(cancellationToken).WaitAsync(timeout, cancellationToken);
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (TimeoutException ex)
				{
					_logger.LogInformation(ex, "Send task did not terminate gracefully in {Timeout} ms. Tearing down socket buffers...", timeout.TotalMilliseconds);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while stopping send buffer task: {Message}", ex.Message);
				}

				// Release the reader
				await sendBuffer.Semaphore.WaitAsync(cancellationToken);
				try
				{
					sendBuffer._reader?.Dispose();
					sendBuffer._reader = null;
				}
				finally
				{
					sendBuffer.Semaphore.Release();
				}
			}
			finally
			{
				sendBuffer?.Release(); // Added above
			}

			// Force the send task to complete
			try
			{
				await sendBuffer.Task.DisposeAsync();
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception disposing send buffer task: {Message}", ex.Message);
			}

			// Remove the buffer from the dictionary
			lock (_lockObject)
			{
				if (_sendBuffers.Remove(channelId, out sendBuffer))
				{
					sendBuffer.Release(); // For _sendBuffers
				}
			}
		}

		async Task SendFromBufferAsync(int channelId, SendBuffer sendBuffer, CancellationToken cancellationToken)
		{
			try
			{
				ComputeBufferReader reader = sendBuffer._reader!;
				while (!cancellationToken.IsCancellationRequested)
				{
					ReadOnlyMemory<byte> memory = reader.GetReadBuffer();
					if (memory.Length > 0 && !_isClosing)
					{
						await SendAsync(channelId, memory, cancellationToken);
						reader.AdvanceReadPosition(memory.Length);
					}
					if (reader.IsComplete)
					{
						await MarkCompleteAsync(channelId, cancellationToken);
						break;
					}
					await reader.WaitToReadAsync(1, cancellationToken);
				}
			}
			catch (OperationCanceledException ex)
			{
				_logger.LogDebug(ex, "Background send task cancelled for channel {ChannelId}", channelId);
			}
			catch (Exception ex)
			{
				if (!_isClosing)
				{
					_logger.LogInformation(ex, "Exception in background send: {Message}", ex.Message);
				}
			}
		}
	}
}
