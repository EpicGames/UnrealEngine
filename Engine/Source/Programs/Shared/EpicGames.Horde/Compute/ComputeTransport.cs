// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Low-level interface for transferring data
	/// </summary>
	public abstract class ComputeTransport : IAsyncDisposable
	{
		/// <summary>
		/// Writes data to the underlying transport
		/// </summary>
		/// <param name="buffer">Buffer to be written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public abstract ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken);

		/// <summary>
		/// Reads data from the underlying transport into an output buffer
		/// </summary>
		/// <param name="buffer">Buffer to read into</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public abstract ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken);

		/// <summary>
		/// Indicate that all data has been read and written to the transport layer, and that there will be no more calls to send/recv
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public abstract ValueTask MarkCompleteAsync(CancellationToken cancellationToken);

		/// <inheritdoc/>
		public abstract ValueTask DisposeAsync();

		/// <summary>
		/// Fill the given buffer with data
		/// </summary>
		/// <param name="buffer">Buffer to read into</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask RecvFullAsync(Memory<byte> buffer, CancellationToken cancellationToken)
		{
			int read = 0;
			while (read < buffer.Length)
			{
				int partialRead = await RecvAsync(buffer.Slice(read, buffer.Length - read), cancellationToken);
				if (partialRead == 0)
				{
					throw new EndOfStreamException();
				}
				read += partialRead;
			}
		}

		/// <summary>
		/// Fill the given buffer with data
		/// </summary>
		/// <param name="buffer">Buffer to read into</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async ValueTask<bool> RecvOptionalAsync(Memory<byte> buffer, CancellationToken cancellationToken)
		{
			try
			{
				int read = await RecvAsync(buffer, cancellationToken);
				if (read == 0)
				{
					return false;
				}
				if (read < buffer.Length)
				{
					await RecvFullAsync(buffer.Slice(read), cancellationToken);
				}
				return true;
			}
			catch (EndOfStreamException)
			{
				return false;
			}
		}

		/// <summary>
		/// Writes data to the underlying transport
		/// </summary>
		/// <param name="buffer">Buffer to be written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public ValueTask SendAsync(ReadOnlyMemory<byte> buffer, CancellationToken cancellationToken)
			=> SendAsync(new ReadOnlySequence<byte>(buffer), cancellationToken);
	}
}

