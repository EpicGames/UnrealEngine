// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute;

/// <summary>
/// Exception for external IP resolver
/// </summary>
public class ExternalIpResolverException : Exception
{
	/// <inheritdoc />
	public ExternalIpResolverException(string? message) : base(message)
	{
	}

	/// <inheritdoc />
	public ExternalIpResolverException(string? message, Exception? innerException) : base(message, innerException)
	{
	}
}

/// <summary>
/// Find public IP address of local machine by querying a third-party IP lookup service
/// </summary>
public class ExternalIpResolver
{
	/// <summary>
	/// Cache of last resolved IP address. Once resolved, the address is cached for the lifetime of this class.
	/// </summary>
	private IPAddress? _resolvedIp;

	private readonly HttpClient _httpClient;
	private readonly List<string> _ipLookupUrls =
	[
		"http://whatismyip.akamai.com", "http://checkip.amazonaws.com", "http://ifconfig.me/ip"
	];

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="httpClient"></param>
	public ExternalIpResolver(HttpClient httpClient)
	{
		_httpClient = httpClient;
	}

	/// <summary>
	/// Get the external, public-facing IP of local machine
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>External IP address</returns>
	/// <exception cref="ExternalIpResolverException">If unable to resolve</exception>
	public async Task<IPAddress> GetExternalIpAddressAsync(CancellationToken cancellationToken = default)
	{
		if (_resolvedIp != null)
		{
			return _resolvedIp;
		}

		ExternalIpResolverException? lastException = null;
		foreach (string lookupUrl in _ipLookupUrls)
		{
			Uri url = new(lookupUrl);
			try
			{
				_resolvedIp = await GetExternalIpAddressAsync(url, cancellationToken);
				return _resolvedIp;
			}
			catch (ExternalIpResolverException e)
			{
				lastException = e;
			}
		}

		throw new ExternalIpResolverException("Exhausted list of all IP resolvers", lastException);
	}

	private async Task<IPAddress> GetExternalIpAddressAsync(Uri url, CancellationToken cancellationToken = default)
	{
		HttpResponseMessage res = await _httpClient.GetAsync(url, cancellationToken);
		if (!res.IsSuccessStatusCode)
		{
			throw new ExternalIpResolverException($"Non-successful response code: {res.StatusCode}");
		}

		string content = await res.Content.ReadAsStringAsync(cancellationToken);
		if (!IPAddress.TryParse(content, out IPAddress? ipAddress))
		{
			throw new ExternalIpResolverException($"Failed parsing HTTP body as an IP address: {content}");
		}

		return ipAddress;
	}
}