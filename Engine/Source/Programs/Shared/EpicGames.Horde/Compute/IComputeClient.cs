// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Interface for uploading compute work to remote machines
	/// </summary>
	public interface IComputeClient
	{
		/// <summary>
		/// Find the most suitable cluster to execute a given compute assignment request
		/// </summary>
		/// <param name="requirements">Requirements for the agent</param>
		/// <param name="requestId">Optional ID identifying the request over multiple calls, such as retrying the same request</param>
		/// <param name="connection">Optional preference of connection details</param>
		/// <param name="logger">Logger for output from this worker</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<ClusterId> GetClusterAsync(Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, ILogger logger, CancellationToken cancellationToken = default);

		/// <summary>
		/// Adds a new remote request
		/// </summary>
		/// <param name="clusterId">Optional cluster ID. If not set, cluster will automatically be resolved by server</param>
		/// <param name="requirements">Requirements for the agent</param>
		/// <param name="requestId">Optional ID identifying the request over multiple calls, such as retrying the same request</param>
		/// <param name="useUbaCache">Whether to allocate an Unreal Build Accelerator (UBA) cache server session for this compute request</param>
		/// <param name="connection">Optional preference of connection details</param>
		/// <param name="logger">Logger for output from this worker</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<IComputeLease?> TryAssignWorkerAsync(ClusterId? clusterId, Requirements? requirements, string? requestId, ConnectionMetadataRequest? connection, bool? useUbaCache, ILogger logger, CancellationToken cancellationToken = default);

		/// <summary>
		/// Declare resource needs for current client
		/// Helps inform the server about current demand.
		/// Can be called as often as necessary to keep needs up-to-date.
		/// </summary>
		/// <param name="clusterId">Cluster to execute the request</param>
		/// <param name="pool">Which pool this applies to</param>
		/// <param name="resourceNeeds">Properties with a target amount of each, such as CPU or RAM</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task DeclareResourceNeedsAsync(ClusterId clusterId, string pool, Dictionary<string, int> resourceNeeds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Allocate a session for the UBA cache associated with the given cluster
		/// </summary>
		/// <param name="clusterId">Cluster to execute the request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<UbaConfig> AllocateUbaCacheServerAsync(ClusterId clusterId, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for <see cref="IComputeClient"/>
	/// </summary>
	public static class ComputeClientExtensions
	{
		/// <inheritdoc cref="IComputeClient.TryAssignWorkerAsync" />
		public static Task<IComputeLease?> TryAssignWorkerAsync(this IComputeClient computeClient, ClusterId clusterId, Requirements? requirements, string? requestId, ConnectionMode? connectionPreference, ILogger logger, CancellationToken cancellationToken = default)
		{
			ConnectionMetadataRequest cmr = new() { ModePreference = connectionPreference };
			return computeClient.TryAssignWorkerAsync(clusterId, requirements, requestId, cmr, null, logger, cancellationToken);
		}

		/// <inheritdoc cref="IComputeClient.TryAssignWorkerAsync" />
		[Obsolete("Prefer taking a requestId parameter")]
		public static Task<IComputeLease?> TryAssignWorkerAsync(this IComputeClient computeClient, ClusterId clusterId, Requirements? requirements, ILogger logger, CancellationToken cancellationToken)
		{
			return computeClient.TryAssignWorkerAsync(clusterId, requirements, null, null, logger, cancellationToken);
		}

		/// <inheritdoc cref="IComputeClient.TryAssignWorkerAsync" />
		[Obsolete("Prefer taking a connection parameter")]
		public static Task<IComputeLease?> TryAssignWorkerAsync(this IComputeClient computeClient, ClusterId clusterId, Requirements? requirements, string? requestId, ILogger logger, CancellationToken cancellationToken = default)
		{
			return computeClient.TryAssignWorkerAsync(clusterId, requirements, requestId, null, logger, cancellationToken);
		}
	}

	/// <summary>
	/// Exception from ComputeClient
	/// </summary>
	public class ComputeClientException : Exception
	{
		/// <inheritdoc/>
		public ComputeClientException(string? message) : base(message)
		{
		}
	}
}
