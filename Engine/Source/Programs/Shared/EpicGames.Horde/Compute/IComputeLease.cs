// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Full-duplex channel for sending and receiving messages
	/// </summary>
	public interface IComputeLease : IAsyncDisposable
	{
		/// <summary>
		/// Compute cluster ID
		/// </summary>
		ClusterId Cluster { get; }

		/// <summary>
		/// Properties of the remote machine
		/// </summary>
		IReadOnlyList<string> Properties { get; }

		/// <summary>
		/// Resources assigned to this lease
		/// </summary>
		IReadOnlyDictionary<string, int> AssignedResources { get; }

		/// <summary>
		/// Socket to communicate with the remote
		/// </summary>
		RemoteComputeSocket Socket { get; }

		/// <summary>
		/// IP address of the remote agent machine running the compute task
		/// When using relay connection mode, this may be the IP of the relay rather than the remote machine itself.
		/// </summary>
		public string Ip { get; }

		/// <inheritdoc cref="UbaConfig" />
		public UbaConfig? Uba { get; }

		/// <summary>
		/// How to establish a connection to the remote machine (when not using the default socket)
		/// </summary>
		public ConnectionMode ConnectionMode { get; }

		/// <summary>
		/// Assigned ports (externally visible port -> local port on agent)
		///
		/// Key is an arbitrary name identifying the port (same as was given when requesting the lease>)
		/// When relay mode is used, ports can mapped to a different externally visible port.
		/// If compute task uses and listens to port 7000, that port can be externally represented as something else.
		/// For example, port 32743 can be pointed to port 7000.
		/// This makes no difference for the compute task process, but the client/initiator making connections must
		/// pay attention to this mapping.
		/// </summary>
		IReadOnlyDictionary<string, ConnectionMetadataPort> Ports { get; }

		/// <summary>
		/// Relinquish the lease gracefully
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		ValueTask CloseAsync(CancellationToken cancellationToken = default);
	}
}
