// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.IO;
using System.Runtime.InteropServices;
using System.Threading;
using Microsoft.Win32.SafeHandles;

namespace EpicGames.Horde.Compute
{
	internal class Native
	{
#pragma warning disable IDE1006 // Naming Styles
		[StructLayout(LayoutKind.Sequential)]
		public class SECURITY_ATTRIBUTES
		{
			public int nLength;
			public IntPtr lpSecurityDescriptor;
			public int bInheritHandle;
		}
#pragma warning restore IDE1006 // Naming Styles

		[DllImport("kernel32.dll")]
		public static extern SafeWaitHandle CreateEvent(SECURITY_ATTRIBUTES lpEventAttributes, bool bManualReset, bool bInitialState, string? lpName);

		[DllImport("Kernel32.dll", SetLastError = true)]
		public static extern SafeWaitHandle OpenEvent(uint dwDesiredAccess, bool bInheritHandle, string lpName);

		[DllImport("kernel32.dll", SetLastError = true)]
		public static extern bool ResetEvent(SafeWaitHandle hEvent);

		[DllImport("kernel32.dll", SetLastError = true)]
		public static extern bool SetEvent(SafeWaitHandle hEvent);

		public const uint DUPLICATE_SAME_ACCESS = 2;

		public const uint SYNCHRONIZE = 0x00100000;
		public const uint EVENT_MODIFY_STATE = 2;

		[DllImport("kernel32.dll", SetLastError = true)]
		public static extern IntPtr GetCurrentProcess();

		[DllImport("kernel32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool DuplicateHandle(IntPtr hSourceProcessHandle, IntPtr hSourceHandle, IntPtr hTargetProcessHandle, out IntPtr lpTargetHandle, uint dwDesiredAccess, [MarshalAs(UnmanagedType.Bool)] bool bInheritHandle, uint dwOptions);

		public class EventHandle : WaitHandle
		{
			public EventHandle(SafeWaitHandle handle)
			{
				SafeWaitHandle = handle;
			}

			public static EventHandle CreateNew(string? name, EventResetMode resetMode, bool initialState, HandleInheritability handleInheritability)
			{
				Native.SECURITY_ATTRIBUTES securityAttributes = new();
				securityAttributes.nLength = Marshal.SizeOf(securityAttributes);
				securityAttributes.bInheritHandle = (handleInheritability == HandleInheritability.Inheritable) ? 1 : 0;

				SafeWaitHandle handle = CreateEvent(securityAttributes, resetMode == EventResetMode.ManualReset, initialState, name);
				if (handle.IsInvalid)
				{
					throw new Win32Exception();
				}
				return new EventHandle(handle);
			}

			public EventHandle(IntPtr handle, bool ownsHandle)
			{
				SafeWaitHandle = new SafeWaitHandle(handle, ownsHandle);
			}

			public static EventHandle OpenExisting(string name)
			{
				SafeWaitHandle handle = OpenEvent(SYNCHRONIZE | EVENT_MODIFY_STATE, true, name);
				if (handle.IsInvalid)
				{
					throw new Win32Exception();
				}
				return new EventHandle(handle);
			}

			public void Set() => SetEvent(SafeWaitHandle);

			public void Reset() => ResetEvent(SafeWaitHandle);
		}
	}
}
