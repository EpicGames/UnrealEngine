// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Cryptography;
using System.Threading;

namespace EpicGames.Horde.Compute;

/// <summary>
/// Manages and hands out request IDs for compute allocation requests
/// </summary>
public class RequestIdAllocator
{
	private readonly Lock _lock = new();
	private readonly List<string> _pendingIds = [];
	private readonly List<string> _batchIds = [];
	private readonly string _allocatorId;
	private int _requestCounter;

	/// <summary>
	/// Constructor
	/// </summary>
	public RequestIdAllocator()
	{
		_allocatorId = GenerateAllocatorId(8);
	}

	/// <summary>
	/// Starts a new batch of requests.
	/// Any requests started during current batch will be reset and marked as unfinished.
	/// </summary>
	public void StartBatch()
	{
		lock (_lock)
		{
			_pendingIds.AddRange(_batchIds);
			_batchIds.Clear();
		}
	}

	/// <summary>
	/// Get or create a request ID and mark it as part of current batch
	/// </summary>
	/// <returns>A request ID</returns>
	public string AllocateId()
	{
		lock (_lock)
		{
			string? reqId;
			if (_pendingIds.Count > 0)
			{
				reqId = _pendingIds[0];
				_pendingIds.RemoveAt(0);
			}
			else
			{
				reqId = $"{_allocatorId}-{_requestCounter++}";
			}

			_batchIds.Add(reqId);
			return reqId;
		}
	}

	/// <summary>
	/// Mark a request Id as accepted. It won't be re-used for any future requests.
	/// </summary>
	/// <param name="reqId">Request ID to mark as finished</param>
	public void MarkAccepted(string reqId)
	{
		lock (_lock)
		{
			_batchIds.Remove(reqId);
		}
	}

	private static string GenerateAllocatorId(int length)
	{
		const string Chars = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789";
		return String.Create(length, Chars, (span, c) =>
		{
			for (int i = 0; i < span.Length; i++)
			{
				span[i] = c[RandomNumberGenerator.GetInt32(c.Length)];
			}
		});
	}
}