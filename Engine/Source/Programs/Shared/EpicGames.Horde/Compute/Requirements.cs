// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Common;

#pragma warning disable CA2227

namespace EpicGames.Horde.Compute
{
	/// <summary>
	/// Requirements for a compute task to be assigned an agent
	/// </summary>
	public class Requirements
	{
		/// <summary>
		/// Pool of machines to draw from
		/// </summary>
		public string? Pool { get; set; }

		/// <summary>
		/// Condition string to be evaluated against the machine spec, eg. cpu-cores >= 10 &amp;&amp; ram.mb >= 200 &amp;&amp; pool == 'worker'
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Properties required from the remote machine
		/// </summary>
		public HashSet<string> Properties { get; set; } = [];

		/// <summary>
		/// Resources used by the process
		/// </summary>
		public Dictionary<string, ResourceRequirements> Resources { get; } = [];

		/// <summary>
		/// Whether we require exclusive access to the device
		/// </summary>
		public bool Exclusive { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public Requirements()
		{
		}

		/// <summary>
		/// Construct a requirements object with a condition
		/// </summary>
		/// <param name="condition">Condition for matching machines to execute the work</param>
		public Requirements(Condition? condition)
		{
			Condition = condition;
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			List<string> list = [];
			if (Pool != null)
			{
				list.Add($"Pool:{Pool}");
			}
			if (Condition != null)
			{
				list.Add($"\"{Condition}\"");
			}
			foreach ((string name, ResourceRequirements allocation) in Resources)
			{
				list.Add($"{name}: {allocation.Min}-{allocation.Max}");
			}
			if (Exclusive)
			{
				list.Add("Exclusive");
			}
			return String.Join(", ", list);
		}
	}

	/// <summary>
	/// Specifies requirements for resource allocation
	/// </summary>
	public class ResourceRequirements
	{
		/// <summary>
		/// Minimum allocation of the requested resource
		/// </summary>
		public int Min { get; set; } = 1;

		/// <summary>
		/// Maximum allocation of the requested resource. Allocates as much as possible unless capped.
		/// </summary>
		public int? Max { get; set; }
	}
}
