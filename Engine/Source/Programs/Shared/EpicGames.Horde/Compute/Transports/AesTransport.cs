// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute.Transports;

/// <summary>
/// Transport layer that adds AES encryption on top of an underlying transport implementation.
/// Key must be exchanged separately (e.g. via the HTTPS request to negotiate a lease with the server).
/// </summary>
public sealed class AesTransport : ComputeTransport
{
	/// <summary>
	/// Length of the required encryption key. 
	/// </summary>
	public const int KeyLength = 32;

	/// <summary>
	/// Length of the nonce. This should be a cryptographically random number, and does not have to be secret.
	/// </summary>
	public const int NonceLength = 12;

	private const int TagSize = 16; // AES-GCM tag size in bytes
	private const int BufferLength = sizeof(int); // Length of buffer stored as int
	private const int HeaderLength = BufferLength + NonceLength;
	private readonly ComputeTransport _inner;
	private readonly bool _leaveInnerOpen;
	private readonly AesGcm _aes;
	private readonly SemaphoreSlim _sendSemaphore = new(1, 1);
	private readonly SemaphoreSlim _recvSemaphore = new(1, 1);
	private readonly MemoryStream _remainingData;
	private bool _disposed;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="inner">The underlying transport implementation that will be encrypted</param>
	/// <param name="key">Encryption key. Should be generated with a cryptographically secure random number generator</param>
	/// <param name="leaveInnerOpen">Whether to dispose the inner transport when this instance is disposed</param>
	/// <param name="readBufferSize">Default receive buffer size for data that needs to buffered for next read. Will automatically grow</param>
	public AesTransport(ComputeTransport inner, ReadOnlySpan<byte> key, bool leaveInnerOpen = false, int readBufferSize = 256 * 1024)
	{
		_inner = inner;
		_leaveInnerOpen = leaveInnerOpen;
		_remainingData = new MemoryStream(readBufferSize);

		if (key.Length != KeyLength)
		{
			throw new ArgumentException("Key must be 32 bytes", nameof(key)); // AES-256
		}

		_aes = new AesGcm(key.ToArray(), TagSize);
	}

	/// <summary>
	/// Create a random key for this transport
	/// </summary>
	/// <returns>A cryptographically random key</returns>
	public static byte[] CreateKey() => RandomNumberGenerator.GetBytes(KeyLength);

	/// <inheritdoc/>
	public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
	{
		await _sendSemaphore.WaitAsync(cancellationToken);
		try
		{
			int bufferLength = (int)buffer.Length;

			// Message format: [length (4 bytes)][nonce (12 bytes)][encrypted data][tag (16 bytes)]
			int messageSize = HeaderLength + bufferLength + TagSize;
			byte[] message = new byte[messageSize];
			BitConverter.TryWriteBytes(message.AsSpan(0, BufferLength), bufferLength);
			RandomNumberGenerator.Fill(message.AsSpan(BufferLength, NonceLength));

			// Copy sequence to contiguous array for encryption
			byte[] plaintext = buffer.ToArray();

			_aes.Encrypt(
				message.AsSpan(BufferLength, NonceLength),
				plaintext,
				message.AsSpan(HeaderLength, bufferLength),
				message.AsSpan(HeaderLength + bufferLength, TagSize));

			await _inner.SendAsync(message, cancellationToken);
		}
		finally
		{
			_sendSemaphore.Release();
		}
	}

	/// <inheritdoc/>
	public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
	{
		await _recvSemaphore.WaitAsync(cancellationToken);
		try
		{
			// Check if we have remaining data from previous recv
			if (_remainingData.Length > 0)
			{
				int bytesToCopyFromPrev = Math.Min((int)(_remainingData.Length - _remainingData.Position), buffer.Length);
				_remainingData.Read(buffer.Span[..bytesToCopyFromPrev]); // TODO: Check actual bytes read

				// If we've consumed all data, reset or dispose the stream
				if (_remainingData.Position >= _remainingData.Length)
				{
					_remainingData.SetLength(0); // Reset the stream for reuse
				}

				return bytesToCopyFromPrev;
			}

			byte[] headerData = new byte[HeaderLength];
			await _inner.RecvFullAsync(headerData, cancellationToken);
			int bufferLength = BitConverter.ToInt32(headerData.AsSpan(0, BufferLength));

			const int MaxMessageSize = 100 * 1024 * 1024; // 100 MB
			if (bufferLength is <= 0 or > MaxMessageSize)
			{
				throw new InvalidDataException($"Invalid or too large frame length: {bufferLength}");
			}

			// Once we know the buffer size, read the encrypted data from underlying transport
			byte[] encryptedBuffer = new byte[bufferLength + TagSize]; // TODO: Use shared buffer
			await _inner.RecvFullAsync(encryptedBuffer, cancellationToken);

			// Decrypt the message
			byte[] plaintext = new byte[bufferLength]; // TODO: Use shared buffer
			try
			{
				_aes.Decrypt(
					headerData.AsSpan(BufferLength, NonceLength),
					encryptedBuffer.AsSpan(0, bufferLength),
					encryptedBuffer.AsSpan(bufferLength, TagSize),
					plaintext);
			}
			catch (CryptographicException)
			{
				throw new InvalidDataException("Decryption failed - data may be corrupted or tampered with");
			}

			// Copy decrypted data to the output buffer which be smaller than actual decrypted buffer
			int bytesToCopy = Math.Min(plaintext.Length, buffer.Length);
			plaintext.AsSpan(0, bytesToCopy).CopyTo(buffer.Span);

			// Store remaining data for next receive call, if any
			if (bytesToCopy < bufferLength)
			{
				_remainingData.SetLength(0); // Clear existing data
				_remainingData.Write(plaintext.AsSpan(bytesToCopy, bufferLength - bytesToCopy));
				_remainingData.Position = 0;
			}

			return bytesToCopy;
		}
		finally
		{
			_recvSemaphore.Release();
		}
	}

	/// <inheritdoc/>
	public override ValueTask MarkCompleteAsync(CancellationToken cancellationToken)
	{
		return _inner.MarkCompleteAsync(cancellationToken);
	}

	/// <inheritdoc/>
	public override async ValueTask DisposeAsync()
	{
		if (!_disposed)
		{
			_disposed = true;
			_aes.Dispose();
			_sendSemaphore.Dispose();
			_recvSemaphore.Dispose();

			if (!_leaveInnerOpen)
			{
				await _inner.DisposeAsync();
			}
		}
	}
}