// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Compute.Transports;

/// <summary>
/// Compute transport which wraps another transport acting as a watchdog timer for inactivity
/// If no send or receive activity has been seen within specified timeout, the cancellation source will be triggered
/// </summary>
public class IdleTimeoutTransport : ComputeTransport
{
	/// <summary>
	/// Timeout before triggering a cancellation
	/// </summary>
	public TimeSpan NoDataTimeout { get; } = TimeSpan.FromSeconds(20);

	private static readonly double s_ticksToSystemTicks = (double)TimeSpan.TicksPerSecond / Stopwatch.Frequency;
	private readonly ComputeTransport _inner;
	private long _lastPingTicks;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="inner">Transport to watch</param>
	/// <param name="noDataTimeout">Timeout before cancelling</param>
	public IdleTimeoutTransport(ComputeTransport inner, TimeSpan? noDataTimeout = null)
	{
		_inner = inner;
		NoDataTimeout = noDataTimeout ?? NoDataTimeout;
		_lastPingTicks = Stopwatch.GetTimestamp();
	}

	/// <inheritdoc/>
	public override ValueTask DisposeAsync()
	{
		GC.SuppressFinalize(this);
		return ValueTask.CompletedTask;
	}

	/// <summary>
	/// Time since last send or receive completed
	/// </summary>
	public TimeSpan TimeSinceActivity => TimeSpan.FromTicks((long)((Stopwatch.GetTimestamp() - Interlocked.CompareExchange(ref _lastPingTicks, 0, 0)) * s_ticksToSystemTicks));

	/// <inheritdoc/>
	public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
	{
		int result = await _inner.RecvAsync(buffer, cancellationToken);
		if (result > 0)
		{
			Interlocked.Exchange(ref _lastPingTicks, Stopwatch.GetTimestamp());
		}
		return result;
	}

	/// <inheritdoc/>
	public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
	{
		await _inner.SendAsync(buffer, cancellationToken);
		Interlocked.Exchange(ref _lastPingTicks, Stopwatch.GetTimestamp());
	}

	/// <inheritdoc/>
	public override ValueTask MarkCompleteAsync(CancellationToken cancellationToken)
	{
		return _inner.MarkCompleteAsync(cancellationToken);
	}

	/// <summary>
	/// Start a loop monitoring activity on the inner transport
	/// </summary>
	/// <param name="cts">Source to cancel when transport times out</param>
	/// <param name="logger">Logger</param>
	/// <param name="cancellationToken">Cancellation token</param>
	public async Task StartWatchdogTimerAsync(CancellationTokenSource cts, ILogger logger, CancellationToken cancellationToken)
	{
		while (!cancellationToken.IsCancellationRequested)
		{
			TimeSpan remainingTime = NoDataTimeout - TimeSinceActivity;
			if (remainingTime < TimeSpan.Zero)
			{
				logger.LogWarning("Terminating compute transport due to timeout ({Timeout} ms with last tick at {Time})", (int)NoDataTimeout.TotalMilliseconds, DateTime.UtcNow - TimeSinceActivity);
				await cts.CancelAsync();
				break;
			}
			await Task.Delay(remainingTime + TimeSpan.FromSeconds(0.2), cancellationToken);
		}
	}
}
