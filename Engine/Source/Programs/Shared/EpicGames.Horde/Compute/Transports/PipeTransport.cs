// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO.Pipelines;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute.Transports
{
	/// <summary>
	/// Implementation of <see cref="ComputeTransport"/> for communicating over a <see cref="Pipe"/>. 
	/// (Note: this uses a .NET in-process pipe, not an IPC pipe).
	/// </summary>
	public class PipeTransport : ComputeTransport
	{
		readonly PipeReader _reader;
		readonly PipeWriter _writer;

		ReadOnlySequence<byte> _readBuffer = ReadOnlySequence<byte>.Empty;
		SequencePosition? _readEnd;

		/// <inheritdoc/>
		public long Position { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="reader">Reader for the pipe</param>
		/// <param name="writer">Writer for the pipe</param>
		public PipeTransport(PipeReader reader, PipeWriter writer)
		{
			_reader = reader;
			_writer = writer;
		}

		/// <inheritdoc/>
		public override ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);
			return ValueTask.CompletedTask;
		}

		/// <inheritdoc/>
		public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
		{
			int sizeRead = 0;
			while (sizeRead < buffer.Length)
			{
				// Try to get more data into the read buffer
				if (_readBuffer.Length == 0)
				{
					if (_readEnd != null)
					{
						_reader.AdvanceTo(_readEnd.Value);
						_readEnd = null;
					}

					ReadResult result;
					if (sizeRead == 0)
					{
						result = await _reader.ReadAsync(cancellationToken);
					}
					else if (!_reader.TryRead(out result) || result.IsCompleted)
					{
						return sizeRead;
					}

					_readBuffer = result.Buffer;
					_readEnd = _readBuffer.End;
				}

				// Copy as much of the next chunk as we can
				int copySize = Math.Min(buffer.Length, _readBuffer.First.Length);
				if (copySize > 0)
				{
					_readBuffer.First.Slice(0, copySize).CopyTo(buffer);
					buffer = buffer.Slice(copySize);
					_readBuffer = _readBuffer.Slice(copySize);
					sizeRead += copySize;
				}
			}
			Position += sizeRead;
			return sizeRead;
		}

		/// <inheritdoc/>
		public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
		{
			foreach (ReadOnlyMemory<byte> memory in buffer)
			{
				await _writer.WriteAsync(memory, cancellationToken);
				Position += memory.Length;
			}
		}

		/// <inheritdoc/>
		public override async ValueTask MarkCompleteAsync(CancellationToken cancellationToken)
		{
			await _writer.CompleteAsync();
		}
	}
}
