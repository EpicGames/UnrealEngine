// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute.Transports
{
	/// <summary>
	/// Compute transport which wraps an underlying stream
	/// </summary>
	class StreamTransport : ComputeTransport
	{
		readonly Stream _stream;
		readonly bool _leaveOpen;
		public long Position { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="stream">Stream to use for the transferring data</param>
		/// <param name="leaveOpen">Whether to leave the inner stream open when disposing</param>
		public StreamTransport(Stream stream, bool leaveOpen = false)
		{
			_stream = stream;
			_leaveOpen = leaveOpen;
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			if (!_leaveOpen)
			{
				await _stream.DisposeAsync();
			}
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
		{
			int length = await _stream.ReadAsync(buffer, cancellationToken);
			Position += length;
			return length;
		}

		/// <inheritdoc/>
		public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
		{
			foreach (ReadOnlyMemory<byte> memory in buffer)
			{
				await _stream.WriteAsync(memory, cancellationToken);
				Position += memory.Length;
			}
		}

		/// <inheritdoc/>
		public override ValueTask MarkCompleteAsync(CancellationToken cancellationToken) => new();
	}
}
