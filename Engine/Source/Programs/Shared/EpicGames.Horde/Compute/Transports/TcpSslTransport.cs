// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Net.Security;
using System.Net.Sockets;
using System.Security.Cryptography;
using System.Security.Cryptography.X509Certificates;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Compute.Transports;

/// <summary>
/// Implementation of <see cref="ComputeTransport"/> for communicating over a socket using SSL/TLS
/// </summary>
public sealed class TcpSslTransport : ComputeTransport
{
	private readonly Socket _socket;
	private readonly X509Certificate2 _cert;
	private readonly bool _isServer;
	private readonly NetworkStream _networkStream;
	private readonly SslStream _sslStream;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="socket">Socket to communicate over</param>
	/// <param name="certData">Certificate used for auth on both server and client</param>
	/// <param name="isServer">Whether socket is acting as a server or client</param>
	public TcpSslTransport(Socket socket, byte[] certData, bool isServer) : this(socket, X509CertificateLoader.LoadPkcs12(certData, null), isServer)
	{
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="socket">Socket to communicate over</param>
	/// <param name="cert">Certificate used for auth on both server and client</param>
	/// <param name="isServer">Whether socket is acting as a server or client</param>
	public TcpSslTransport(Socket socket, X509Certificate2 cert, bool isServer)
	{
		_socket = socket;
		_cert = cert;
		_isServer = isServer;
		_networkStream = new NetworkStream(socket);
		_sslStream = new SslStream(_networkStream, false);
	}

	/// <inheritdoc/>
	public override ValueTask DisposeAsync()
	{
		_networkStream.Dispose();
		_sslStream.Dispose();
		return ValueTask.CompletedTask;
	}

	/// <summary>
	/// Perform SSL authentication
	/// </summary>
	public async Task AuthenticateAsync(CancellationToken cancellationToken)
	{
		if (_isServer)
		{
			await AuthenticateAsServerAsync(cancellationToken);
		}
		else
		{
			await AuthenticateAsClientAsync();
		}
	}

	private async Task AuthenticateAsClientAsync()
	{
		SslClientAuthenticationOptions options = new()
		{
			TargetHost = "horde", // Since cert is self-signed and not CA-based, this is ignored
			ClientCertificates = [_cert],
			CertificateRevocationCheckMode = X509RevocationMode.Offline,
			EncryptionPolicy = EncryptionPolicy.RequireEncryption,
			RemoteCertificateValidationCallback = ValidateCert
		};

		await _sslStream.AuthenticateAsClientAsync(options);
	}

	private async Task AuthenticateAsServerAsync(CancellationToken cancellationToken)
	{
		SslServerAuthenticationOptions opt = new()
		{
			EncryptionPolicy = EncryptionPolicy.RequireEncryption,
			ServerCertificate = _cert,
			AllowRenegotiation = false,
			ClientCertificateRequired = true,
			RemoteCertificateValidationCallback = ValidateCert,
		};
		await _sslStream.AuthenticateAsServerAsync(opt, cancellationToken);
	}

	/// <summary>
	/// Checks the certificate returned by the server is indeed the correct one
	/// </summary>
	/// <param name="sender"></param>
	/// <param name="certificate"></param>
	/// <param name="chain"></param>
	/// <param name="sslPolicyErrors"></param>
	/// <returns>True if it matches</returns>
	private bool ValidateCert(object sender, X509Certificate? certificate, X509Chain? chain, SslPolicyErrors sslPolicyErrors)
	{
		return certificate != null && certificate.GetCertHashString() == _cert.GetCertHashString();
	}

	/// <inheritdoc/>
	public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
	{
		return await _sslStream.ReadAsync(buffer, cancellationToken);
	}

	/// <inheritdoc/>
	public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
	{
		foreach (ReadOnlyMemory<byte> memory in buffer)
		{
			await _sslStream.WriteAsync(memory, cancellationToken);
		}
		await _sslStream.FlushAsync(cancellationToken);
	}

	/// <inheritdoc/>
	public override ValueTask MarkCompleteAsync(CancellationToken cancelationToken)
	{
		_socket.Shutdown(SocketShutdown.Both);
		return new ValueTask();
	}

	/// <summary>
	/// Generate a self-signed certificate to be used for communicating between client and server of this transport
	/// </summary>
	/// <returns>A X509 certificate serialized as bytes</returns>
	public static byte[] GenerateCert(Encryption encryption)
	{
		X509Certificate2 cert;
		DateTimeOffset notBefore = DateTimeOffset.UtcNow - TimeSpan.FromSeconds(10);
		DateTimeOffset notAfter = DateTimeOffset.UtcNow.AddHours(24);
		string subjectName = "cn=horde";

		if (encryption == Encryption.Ssl)
		{
			using RSA rsa = RSA.Create(2048);
			CertificateRequest req = new(subjectName, rsa, HashAlgorithmName.SHA256, RSASignaturePadding.Pkcs1);
			cert = req.CreateSelfSigned(notBefore, notAfter);
		}
		else if (encryption == Encryption.SslEcdsaP256)
		{
			using ECDsa ecdsa = ECDsa.Create(ECCurve.NamedCurves.nistP256);
			CertificateRequest req = new(subjectName, ecdsa, HashAlgorithmName.SHA256);
			cert = req.CreateSelfSigned(notBefore, notAfter);
		}
		else
		{
			throw new ArgumentException($"Cannot generate certificate for encryption {encryption}", nameof(encryption));
		}

		return cert.Export(X509ContentType.Pkcs12); // Note: Need to reimport this to use immediately, otherwise key is ephemeral
	}
}
