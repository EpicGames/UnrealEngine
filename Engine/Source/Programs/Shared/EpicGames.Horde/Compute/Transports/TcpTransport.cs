// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Net.Sockets;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Compute.Transports
{
	/// <summary>
	/// Implementation of <see cref="ComputeTransport"/> for communicating over a socket
	/// </summary>
	public sealed class TcpTransport : ComputeTransport
	{
		readonly Socket _socket;

		/// <inheritdoc/>
		public long Position { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="socket">Socket to communicate over</param>
		public TcpTransport(Socket socket) => _socket = socket;

		/// <inheritdoc/>
		public override ValueTask DisposeAsync()
		{
			return ValueTask.CompletedTask;
		}

		/// <inheritdoc/>
		public override async ValueTask<int> RecvAsync(Memory<byte> buffer, CancellationToken cancellationToken)
		{
			int read;
			try
			{
				read = await _socket.ReceiveAsync(buffer, SocketFlags.None, cancellationToken);
			}
			catch (SocketException ex) when (ex.SocketErrorCode == SocketError.Shutdown)
			{
				read = 0;
			}
			Position += read;
			return read;
		}

		/// <inheritdoc/>
		public override async ValueTask SendAsync(ReadOnlySequence<byte> buffer, CancellationToken cancellationToken)
		{
			int offset = 0;
			foreach (ReadOnlyMemory<byte> memory in buffer)
			{
				_socket.NoDelay = (offset + memory.Length == buffer.Length);
				await _socket.SendMessageAsync(memory, SocketFlags.None, cancellationToken);
				Position += memory.Length;
			}
		}

		/// <inheritdoc/>
		public override ValueTask MarkCompleteAsync(CancellationToken cancelationToken)
		{
			_socket.Shutdown(SocketShutdown.Both);
			return new ValueTask();
		}
	}
}
