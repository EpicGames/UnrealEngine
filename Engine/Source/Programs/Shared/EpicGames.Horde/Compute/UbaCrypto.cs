// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Security.Cryptography;
using System.Threading;

namespace EpicGames.Horde.Compute;

/// <summary>
/// Provides AES encryption and decryption functionality compatible with Unreal Build Accelerator (UBA) communication protocol.
/// Uses AES-128 in CBC mode with zero IV and no padding. Thread-safe through thread-local AES instances.
/// </summary>
public class UbaCrypto : IDisposable
{
	private const int AesBlockSize = 16;
	private static readonly byte[] s_fixedIvLe = [0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00];

	private readonly ThreadLocal<Aes> _aes;
	private bool _disposed;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="key">The 128-bit (16 byte) AES encryption key.</param>
	/// <exception cref="ArgumentException"></exception>
	public UbaCrypto(byte[] key)
	{
		if (key is not { Length: 16 })
		{
			throw new ArgumentException("Key must be exactly 16 bytes");
		}

		_aes = new ThreadLocal<Aes>(() =>
		{
			Aes aes = Aes.Create();
			aes.Key = key;
			aes.Mode = CipherMode.CBC;
			aes.Padding = PaddingMode.None;
			return aes;
		});
	}

	/// <summary>
	/// Encrypts the specified data
	/// </summary>
	/// <param name="data">The data to encrypt. Can be any length.</param>
	/// <returns>
	/// A byte array containing the encrypted data. The array length matches the input length.
	/// Only complete 16-byte blocks are encrypted; remaining bytes are left unmodified.
	/// </returns>
	public byte[] Encrypt(byte[] data) => Transform(data, encrypt: true);

	/// <summary>
	/// Decrypts the specified data.
	/// </summary>
	/// <param name="data">The data to decrypt. Can be any length.</param>
	/// <returns>
	/// A byte array containing the decrypted data. The array length matches the input length.
	/// Only complete 16-byte blocks are decrypted; remaining bytes are left unmodified.
	/// </returns>
	public byte[] Decrypt(byte[] data) => Transform(data, encrypt: false);

	/// <summary>
	/// Decrypt/encrypt method ported from C++ UBA
	/// </summary>
	/// <param name="src">Input buffer to operate on</param>
	/// <param name="encrypt">Whether to encrypt (if false, decrypt)</param>
	/// <returns>A new byte array with encrypted/decrypted data</returns>
	[SuppressMessage("Security", "CA5401:Do not use CreateEncryptor with non-default IV")]
	private byte[] Transform(byte[] src, bool encrypt)
	{
		if (src.Length == 0)
		{
			return [];
		}

		// Avoid modifying caller's buffer
		byte[] buf = (byte[])src.Clone();
		int size = buf.Length;

		// Messages smaller than one block -> XOR with length and return
		if (size < AesBlockSize)
		{
			byte len = (byte)size;
			for (int i = 0; i < size; ++i)
			{
				buf[i] ^= len;
			}
			return buf;
		}

		int aligned = size / AesBlockSize * AesBlockSize;
		int overflow = size - aligned;

		// If encrypting and we have overflowing bytes, XOR them
		if (overflow > 0 && encrypt)
		{
			for (int i = 0; i < overflow; ++i)
			{
				buf[aligned + i] ^= buf[i];
			}
		}

		// AES-CBC on the aligned prefix
		Aes aes = _aes.Value!;
		byte[] clonedId = (byte[])s_fixedIvLe.Clone();
		using ICryptoTransform tr = encrypt ? aes.CreateEncryptor(aes.Key, clonedId) : aes.CreateDecryptor(aes.Key, clonedId);
		byte[] prefix = tr.TransformFinalBlock(buf, 0, aligned);
		Buffer.BlockCopy(prefix, 0, buf, 0, aligned);

		// If decrypting and we have overflowing bytes, XOR them
		if (overflow > 0 && !encrypt)
		{
			for (int i = 0; i < overflow; ++i)
			{
				buf[aligned + i] ^= buf[i];
			}
		}

		return buf;
	}

	/// <inheritdoc />
	public void Dispose()
	{
		Dispose(true);
		GC.SuppressFinalize(this);
	}

	/// <summary>
	/// Dispose
	/// </summary>
	protected virtual void Dispose(bool disposing)
	{
		if (!_disposed)
		{
			if (disposing)
			{
				_aes.Dispose();
			}
			_disposed = true;
		}
	}
}

