// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Common;
using EpicGames.Horde.Server;

#pragma warning disable CA2227 // Collection properties should be read only
#pragma warning disable CA1056 // Change string to URI

namespace EpicGames.Horde.Dashboard
{
	/// <summary>
	/// Setting information required by dashboard
	/// </summary>
	public class GetDashboardConfigResponse
	{
		/// <summary>
		/// The name of the external issue service
		/// </summary>
		public string? ExternalIssueServiceName { get; set; }

		/// <summary>
		/// The url of the external issue service
		/// </summary>
		public string? ExternalIssueServiceUrl { get; set; }

		/// <summary>
		/// The url of the perforce swarm installation
		/// </summary>
		public string? PerforceSwarmUrl { get; set; }

		/// <summary>
		/// Url of Robomergem installation
		/// </summary>
		public string? RobomergeUrl { get; set; }

		/// <summary>
		/// Url of Commits Viewer
		/// </summary>
		public string? CommitsViewerUrl { get; set; }

		/// <summary>
		/// Help email address that users can contact with issues
		/// </summary>
		public string? HelpEmailAddress { get; set; }

		/// <summary>
		/// Help slack channel that users can use for issues
		/// </summary>
		public string? HelpSlackChannel { get; set; }

		/// <summary>
		/// The auth method in use
		/// </summary>
		public AuthMethod AuthMethod { get; set; }

		/// <summary>
		/// Device problem cooldown in minutes
		/// </summary>
		public int DeviceProblemCooldownMinutes { get; set; }

		/// <summary>
		/// Categories to display on the agents page
		/// </summary>
		public List<GetDashboardAgentCategoryResponse> AgentCategories { get; set; } = [];

		/// <summary>
		/// Categories to display on the pools page
		/// </summary>
		public List<GetDashboardPoolCategoryResponse> PoolCategories { get; set; } = [];

		/// <summary>
		/// Configured artifact types
		/// </summary>
		public List<string> ArtifactTypes { get; set; } = [];

		/// <summary>
		/// Configured artifact type info
		/// </summary>
		public List<GetDashboardArtifactTypeResponse> ArtifactTypeInfo { get; set; } = [];

		/// <summary>
		/// The server's global config version number. Maps to the ConfigVersion enum on the server.
		/// </summary>
		public int? ConfigVersion { get; set; }
	}

	/// <summary>
	/// Describes an artifact type
	/// </summary>
	public class GetDashboardArtifactTypeResponse
	{
		/// <summary>
		/// Type of the artifact
		/// </summary>
		public string Type { get; set; } = "UnnamedArtifactType";

		/// <summary>
		/// Number of days to retain artifacts of this type
		/// </summary>
		public int? KeepDays { get; set; }
	}

	/// <summary>
	/// Describes a category for the pools page
	/// </summary>
	public class GetDashboardPoolCategoryResponse
	{
		/// <summary>
		/// Title for the tab
		/// </summary>
		public string Name { get; set; } = "Unnamed";

		/// <summary>
		/// Condition for pools to be included in this category
		/// </summary>
		public Condition? Condition { get; set; }
	}

	/// <summary>
	/// Describes a category for the agents page
	/// </summary>
	public class GetDashboardAgentCategoryResponse
	{
		/// <summary>
		/// Title for the tab
		/// </summary>
		public string Name { get; set; } = "Unnamed";

		/// <summary>
		/// Condition for agents to be included in this category
		/// </summary>
		public Condition? Condition { get; set; }
	}

	/// <summary>
	/// 
	/// </summary>
	public class CreateDashboardPreviewRequest
	{
		/// <summary>
		/// A summary of what the preview item changes
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// The CL the preview was deployed in
		/// </summary>
		public int? DeployedCL { get; set; }

		/// <summary>
		/// An example of the preview site users can view the changes
		/// </summary>
		public string? ExampleLink { get; set; }

		/// <summary>
		/// Optional Link for discussion the preview item
		/// </summary>
		public string? DiscussionLink { get; set; }

		/// <summary>
		/// Optional Link for discussing the preview item
		/// </summary>
		public string? TrackingLink { get; set; }
	}

	/// <summary>
	/// 
	/// </summary>
	public class UpdateDashboardPreviewRequest
	{
		/// <summary>
		/// The preview item to update
		/// </summary>
		public int Id { get; set; }

		/// <summary>
		/// A summary of what the preview item changes
		/// </summary>
		public string? Summary { get; set; }

		/// <summary>
		/// The CL the preview was deployed in
		/// </summary>
		public int? DeployedCL { get; set; }

		/// <summary>
		/// Whather the preview is under consideration, if false the preview item didn't pass muster
		/// </summary>
		public bool? Open { get; set; }

		/// <summary>
		/// An example of the preview site users can view the changes
		/// </summary>
		public string? ExampleLink { get; set; }

		/// <summary>
		/// Optional Link for discussion the preview item
		/// </summary>
		public string? DiscussionLink { get; set; }

		/// <summary>
		/// Optional Link for discussing the preview item
		/// </summary>
		public string? TrackingLink { get; set; }
	}

	/// <summary>
	/// Dashboard preview item response
	/// </summary>
	public class GetDashboardPreviewResponse
	{
		/// <summary>
		/// The unique ID of the preview item
		/// </summary>
		public int Id { get; set; }

		/// <summary>
		/// When the preview item was created
		/// </summary>
		public DateTime CreatedAt { get; set; }

		/// <summary>
		/// A summary of what the preview item changes
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// The CL the preview was deployed in
		/// </summary>
		public int? DeployedCL { get; set; }

		/// <summary>
		/// Whather the preview is under consideration, if false the preview item didn't pass muster
		/// </summary>
		public bool Open { get; set; }

		/// <summary>
		/// An example of the preview site users can view the changes
		/// </summary>
		public string? ExampleLink { get; set; }

		/// <summary>
		/// Optional Link for discussion the preview item
		/// </summary>
		public string? DiscussionLink { get; set; }

		/// <summary>
		/// Optional Link for discussing the preview item
		/// </summary>
		public string? TrackingLink { get; set; }
	}

	/// <summary>
	/// Dashboard challenge response
	/// </summary>
	public class GetDashboardChallengeResponse
	{
		/// <summary>
		/// Whether first time setup needs to run
		/// </summary>
		public bool NeedsFirstTimeSetup { get; set; } = false;

		/// <summary>
		/// Whether the user needs to authorize
		/// </summary>
		public bool NeedsAuthorization { get; set; } = true;

	}
}
