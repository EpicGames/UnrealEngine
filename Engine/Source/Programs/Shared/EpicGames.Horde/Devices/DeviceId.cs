// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Devices
{
	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<DeviceId, DeviceIdConverter>))]
	[StringIdConverter(typeof(DeviceIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<DeviceId, DeviceIdConverter>))]
	public readonly record struct DeviceId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public DeviceId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc cref="StringId.Sanitize(System.String)"/>
		public static DeviceId Sanitize(string text) => new(StringId.Sanitize(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class DeviceIdConverter : StringIdConverter<DeviceId>
	{
		/// <inheritdoc/>
		public override DeviceId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(DeviceId value) => value.Id;
	}
}
