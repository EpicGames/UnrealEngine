// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;

#pragma warning disable CA2227 // Change 'x' to be read-only by removing the property setter

namespace EpicGames.Horde.Devices
{

	/// <summary>
	/// The type of device pool
	/// </summary>
	public enum DevicePoolType
	{
		/// <summary>
		/// Available to CIS jobs
		/// </summary>
		Automation,

		/// <summary>
		/// Shared by users with remote checking and checkouts
		/// </summary>
		Shared
	}

	/// <summary>
	/// Create device platform request
	/// </summary>
	public class CreateDevicePlatformRequest
	{
		/// <summary>
		/// The name of the platform
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

	}

	/// <summary>
	/// Create device platform response object
	/// </summary>
	public class CreateDevicePlatformResponse
	{
		/// <summary>
		/// Id of newly created platform
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CreateDevicePlatformResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Update requesty object for a device platform
	/// </summary>
	public class UpdateDevicePlatformRequest
	{
		/// <summary>
		/// The vendor model ids for the platform
		/// </summary>
		public string[] ModelIds { get; set; } = null!;
	}

	/// <summary>
	/// Get object response which describes a device platform
	/// </summary>
	public class GetDevicePlatformResponse
	{
		/// <summary>
		/// Unique id of device platform
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Friendly name of device platform
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Platform vendor models
		/// </summary>
		public string[] ModelIds { get; set; }

		/// <summary>
		/// Response constructor
		/// </summary>
		public GetDevicePlatformResponse(string id, string name, string[] modelIds)
		{
			Id = id;
			Name = name;
			ModelIds = modelIds;
		}
	}

	/// <summary>
	/// Device pool creation request object
	/// </summary>
	public class CreateDevicePoolRequest
	{
		/// <summary>
		/// The name for the new pool
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The name for the new pool
		/// </summary>
		[Required]
		public DevicePoolType PoolType { get; set; }

		/// <summary>
		/// Projects associated with this device pool
		/// </summary>
		public List<string>? ProjectIds { get; set; }
	}

	/// <summary>
	/// Device pool update request object
	/// </summary>
	public class UpdateDevicePoolRequest
	{
		/// <summary>
		/// Id of the device pool to update
		/// </summary>
		public string Id { get; set; } = null!;

		/// <summary>
		/// Projects associated with this device pool
		/// </summary>
		public List<string>? ProjectIds { get; set; }
	}

	/// <summary>
	/// Device pool creation response object
	/// </summary>
	public class CreateDevicePoolResponse
	{
		/// <summary>
		/// Id of the newly created device pool
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CreateDevicePoolResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Device pool response object
	/// </summary>
	public class GetDevicePoolResponse
	{
		/// <summary>
		/// Id of  the device pool
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Name of the device pool
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Type of the device pool
		/// </summary>
		public DevicePoolType PoolType { get; set; }

		/// <summary>
		/// Whether there is write access to the pool
		/// </summary>
		public bool WriteAccess { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetDevicePoolResponse(string id, string name, DevicePoolType poolType, bool writeAccess)
		{
			Id = id;
			Name = name;
			PoolType = poolType;
			WriteAccess = writeAccess;
		}
	}

	// Devices 

	/// <summary>
	/// Device creation request object
	/// </summary>
	public class CreateDeviceRequest
	{
		/// <summary>
		/// The platform of the device 
		/// </summary>
		[Required]
		public string PlatformId { get; set; } = null!;

		/// <summary>
		/// The pool to assign the device
		/// </summary>
		[Required]
		public string PoolId { get; set; } = null!;

		/// <summary>
		/// The friendly name of the device
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// Whether to create the device in enabled state
		/// </summary>
		public bool? Enabled { get; set; }

		/// <summary>
		/// The network address of the device
		/// </summary>
		public string? Address { get; set; }

		/// <summary>
		/// The vendor model id of the device
		/// </summary>
		public string? ModelId { get; set; }

		/// <summary>
		/// Any tags associated with the device
		/// </summary>
		public List<string>? Tags { get; set; } = null;
	}

	/// <summary>
	/// Device creation response object
	/// </summary>
	public class CreateDeviceResponse
	{
		/// <summary>
		/// The id of the newly created device
		/// </summary>
		[Required]
		public string Id { get; set; } = null!;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id"></param>
		public CreateDeviceResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Get response object which describes a device (DEPRECATED)
	/// </summary>
	public class GetDeviceUtilizationResponse
	{
		/// <summary>
		/// The job id which utilized device
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// The job's step id
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// The time device was reserved
		/// </summary>
		public DateTime ReservationStartUtc { get; set; }

		/// <summary>
		/// The time device was freed
		/// </summary>
		public DateTime? ReservationFinishUtc { get; set; }
	}

	/// <summary>
	/// Get response object which describes a device
	/// </summary>
	public class GetDeviceResponse
	{
		/// <summary>
		/// The unique id of the device
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// The platform of the device
		/// </summary>
		public string PlatformId { get; set; }

		/// <summary>
		/// The pool the device belongs to
		/// </summary>
		public string PoolId { get; set; }

		/// <summary>
		/// The friendly name of the device
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Whether the device is currently enabled
		/// </summary>
		public bool Enabled { get; set; }

		/// <summary>
		///  The address of the device (if it allows network connections)
		/// </summary>
		public string? Address { get; set; }

		/// <summary>
		/// The vendor model id of the device
		/// </summary>
		public string? ModelId { get; set; }

		/// <summary>
		/// Any notes provided for the device
		/// </summary>
		public string? Notes { get; set; }

		/// <summary>
		/// Metadata tags on the device
		/// </summary>
		public List<string>? Tags { get; set; }

		/// <summary>
		/// If the device has a marked problem
		/// </summary>
		public DateTime? ProblemTime { get; set; }

		/// <summary>
		/// If the device is in maintenance mode
		/// </summary>
		public DateTime? MaintenanceTime { get; set; }

		/// <summary>
		/// The user id that has the device checked out
		/// </summary>
		public string? CheckedOutByUserId { get; set; }

		/// <summary>
		/// The last time the device was checked out
		/// </summary>
		public DateTime? CheckOutTime { get; set; }

		/// <summary>
		/// When the checkout will expire
		/// </summary>
		public DateTime? CheckOutExpirationTime { get; set; }

		/// <summary>
		/// The last user to modifiy the device
		/// </summary>
		public string? ModifiedByUser { get; set; }

		/// <summary>
		/// When the clean pass started
		/// </summary>
		public DateTime? CleanStartTime { get; set; }

		/// <summary>
		/// When the clean pass finished
		/// </summary>
		public DateTime? CleanFinishTime { get; set; }

		/// <summary>
		/// Whether the device requires a cleaning
		/// </summary>
		public bool? CleanRequired { get; set; }

		/// <summary>
		///  Device Utilization data
		/// </summary>
		public List<GetDeviceUtilizationResponse>? Utilization { get; set; }

		/// <summary>
		/// Device response constructor
		/// </summary>
		public GetDeviceResponse(string id, string platformId, string poolId, string name, bool enabled, string? address, string? modelId, string? modifiedByUser, string? notes, DateTime? problemTime, DateTime? maintenanceTime, List<GetDeviceUtilizationResponse>? utilization, string? checkedOutByUserId = null, DateTime? checkOutTime = null, DateTime? checkOutExpirationTime = null, bool? cleanRequired = null, DateTime? cleanStartTime = null, DateTime? cleanFinishTime = null, List<string>? tags = null)
		{
			Id = id;
			Name = name;
			PlatformId = platformId;
			PoolId = poolId;
			Enabled = enabled;
			Address = address;
			ModelId = modelId;
			ModifiedByUser = modifiedByUser;
			Notes = notes;
			ProblemTime = problemTime;
			MaintenanceTime = maintenanceTime;
			Utilization = utilization;
			CheckedOutByUserId = checkedOutByUserId;
			CheckOutTime = checkOutTime;
			CheckOutExpirationTime = checkOutExpirationTime;
			CleanRequired = cleanRequired;
			CleanStartTime = cleanStartTime;
			CleanFinishTime = cleanFinishTime;
			Tags = tags;
		}
	}

	/// <summary>
	/// Device update request object
	/// </summary>
	public class UpdateDeviceRequest
	{
		/// <summary>
		/// The device pool id
		/// </summary>
		public string? PoolId { get; set; }

		/// <summary>
		/// The device name
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// IP address or hostname of device
		/// </summary>
		public string? Address { get; set; }

		/// <summary>
		/// Device vendor model id
		/// </summary>
		public string? ModelId { get; set; }

		/// <summary>
		/// Markdown notes
		/// </summary>
		public string? Notes { get; set; }

		/// <summary>
		/// Whether device is enabled
		/// </summary>
		public bool? Enabled { get; set; }

		/// <summary>
		/// Whether the device is in maintenance mode
		/// </summary>
		public bool? Maintenance { get; set; }

		/// <summary>
		/// Whether to set or clear any device problem state
		/// </summary>
		public bool? Problem { get; set; }

		/// <summary>
		/// Additional message to attach to this request
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// Tags on the device
		/// </summary>
		public List<string>? Tags { get; set; }
	}

	/// <summary>
	/// Device error report request object
	/// </summary>
	public class DeviceErrorRequest
	{
		/// <summary>
		/// Additional message to attach to this request
		/// </summary>
		public string? Message { get; set; }
	}

	/// <summary>
	/// Device checkout request object
	/// </summary>
	public class CheckoutDeviceRequest
	{
		/// <summary>
		/// Whether to checkout or in the device
		/// </summary>
		public bool Checkout { get; set; }
	}

	/// <summary>
	/// Device clean request object
	/// </summary>
	public class CleanDeviceRequest
	{
		/// <summary>
		/// Whether to set the device as being cleaned, or not
		/// </summary>
		public bool Clean { get; set; }
	}

	/// <summary>
	/// Device tags request object
	/// </summary>
	public class DeviceTagsRequest
	{
		/// <summary>
		/// Only devices with these tags will be considered for reservation
		/// </summary>
		public List<string>? RequiredTags { get; set; }

		/// <summary>
		/// Devices with these tags will be preferred over devices without them, but are not explicitly required
		/// </summary>
		public List<string>? PreferredTags { get; set; }

		/// <summary>
		/// Devices with these tags will be considered, but only as a last resort
		/// </summary>
		public List<string>? UndesiredTags { get; set; }

		/// <summary>
		/// Devices with these tags will not be considered for reservation
		/// </summary>
		public List<string>? BlockedTags { get; set; }

		/// <summary>
		/// Returns any duplicate tag entries
		/// </summary>
		public HashSet<string> GetDuplicateEntries()
		{
			List<List<string>?> tagSets = [RequiredTags, PreferredTags, UndesiredTags, BlockedTags];
			HashSet<string> seenTags = [];
			HashSet<string> duplicateTags = [];

			foreach (List<string>? set in tagSets)
			{
				if (set == null)
				{
					continue;
				}
				foreach (string tag in set)
				{
					if (!seenTags.Add(tag))
					{
						duplicateTags.Add(tag);
					}
				}
			}

			return duplicateTags;
		}
	}

	/// <summary>
	/// Device reservation request object
	/// </summary>
	public class DeviceReservationRequest
	{
		/// <summary>
		/// Device reservation platform id
		/// </summary>
		[Required]
		public string PlatformId { get; set; } = null!;

		/// <summary>
		/// The optional vendor model ids to include for this device
		/// </summary>
		public List<string>? IncludeModels { get; set; }

		/// <summary>
		/// The optional vendor model ids to exclude for this device
		/// </summary>
		public List<string>? ExcludeModels { get; set; }

		/// <summary>
		/// Desired device tags for this device
		/// </summary>
		public DeviceTagsRequest? Tags { get; set; } = null;
	}

	/// <summary>
	/// Reservation request object
	/// </summary>
	public class CreateDeviceReservationRequest
	{
		/// <summary>
		/// What pool to reserve devices in
		/// </summary>
		[Required]
		public string PoolId { get; set; } = null!;

		/// <summary>
		/// Devices to reserve
		/// </summary>
		[Required]
		public List<DeviceReservationRequest> Devices { get; set; } = null!;
	}

	/// <summary>
	/// Device reservation response object
	/// </summary>
	public class CreateDeviceReservationResponse
	{
		/// <summary>
		/// The reservation id of newly created reservation
		/// </summary>
		public string Id { get; set; } = null!;

		/// <summary>
		/// The devices that were reserved
		/// </summary>
		public List<GetDeviceResponse> Devices { get; set; } = null!;
	}

	/// <summary>
	/// A reservation containing one or more devices
	/// </summary>
	public class GetDeviceReservationResponse
	{
		/// <summary>
		/// Randomly generated unique id for this reservation
		/// </summary>
		[Required]
		public string Id { get; set; } = null!;

		/// <summary>
		/// Which device pool the reservation is in
		/// </summary>
		[Required]
		public string PoolId { get; set; } = null!;

		/// <summary>
		/// The reserved devices
		/// </summary>
		[Required]
		public List<string> Devices { get; set; } = null!;

		/// <summary>
		/// JobID holding reservation
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// Job step id holding reservation
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// Job mame holding reservation
		/// </summary>
		public string? JobName { get; set; }

		/// <summary>
		/// Job step holding reservation
		/// </summary>
		public string? StepName { get; set; }

		/// <summary>
		/// Reservations held by a user, requires a token
		/// </summary>
		public string? UserId { get; set; }

		/// <summary>
		/// The hostname of machine holding reservation
		/// </summary>
		public string? Hostname { get; set; }

		/// <summary>
		/// The optional reservation details 
		/// </summary>
		public string? ReservationDetails { get; set; }

		/// <summary>
		/// The UTC time when the reservation was created
		/// </summary>
		public DateTime CreateTimeUtc { get; set; }

		/// <summary>
		/// The legacy reservation system guid, to be removed once can update Gauntlet client in all streams
		/// </summary>
		public string LegacyGuid { get; set; } = null!;
	}

	/// <summary>
	/// Device telemetry respponse
	/// </summary>
	public class GetTelemetryInfoResponse
	{
		/// <summary>
		/// The UTC time the telemetry data was created
		/// </summary>
		[Required]
		public DateTime CreateTimeUtc { get; set; }

		/// <summary>
		/// The stream id which utilized device
		/// </summary>
		public string? StreamId { get; set; }

		/// <summary>
		/// The job id which utilized device
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// The job name which utilized device
		/// </summary>
		public string? JobName { get; set; }

		/// <summary>
		/// The job's step id
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// The job name which utilized device
		/// </summary>
		public string? StepName { get; set; }

		/// <summary>
		/// If this telemetry has a reservation, the start time of the reservation
		/// </summary>
		public DateTime? ReservationStartUtc { get; set; }

		/// <summary>
		/// If this telemetry has a reservation, the finish time of the reservation
		/// </summary>
		public DateTime? ReservationFinishUtc { get; set; }

		/// <summary>
		/// If this telemetry marks a detected device issue, the time of the issue
		/// </summary>
		public DateTime? ProblemTimeUtc { get; set; }
	}

	/// <summary>
	/// Device telemetry respponse
	/// </summary>
	public class GetDeviceTelemetryResponse
	{
		/// <summary>
		/// The device id for the telemetry data
		/// </summary>
		[Required]
		public string DeviceId { get; set; } = null!;

		/// <summary>
		/// Individual telemetry data points
		/// </summary>
		[Required]
		public List<GetTelemetryInfoResponse> Telemetry { get; set; } = null!;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="deviceId"></param>
		/// <param name="telemetry"></param>
		public GetDeviceTelemetryResponse(string deviceId, List<GetTelemetryInfoResponse> telemetry)
		{
			DeviceId = deviceId;
			Telemetry = telemetry;
		}
	}

	/// <summary>
	/// Stream device telemetry for pool snapshot
	/// </summary>
	public class GetDevicePoolReservationTelemetryResponse
	{
		/// <summary>
		/// Device id for reservation
		/// </summary>
		public string DeviceId { get; set; }

		/// <summary>
		/// Job id associated with reservation
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// The step id of reservation
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// The name of the job holding reservation
		/// </summary>
		public string? JobName { get; set; }

		/// <summary>
		/// The name of the step holding reservation
		/// </summary>
		public string? StepName { get; set; }

		/// <summary>
		/// constructor
		/// </summary>
		/// <param name="deviceId"></param>
		/// <param name="jobId"></param>
		/// <param name="stepId"></param>
		/// <param name="jobName"></param>
		/// <param name="stepName"></param>
		public GetDevicePoolReservationTelemetryResponse(string deviceId, string? jobId, string? stepId, string? jobName, string? stepName)
		{
			DeviceId = deviceId;
			JobId = jobId;
			StepId = stepId;
			JobName = jobName;
			StepName = stepName;
		}
	}

	/// <summary>
	/// Device telemetry respponse
	/// </summary>
	public class GetDevicePlatformTelemetryResponse
	{
		/// <summary>
		/// The corresponding platform id
		/// </summary>
		public string PlatformId { get; set; }

		/// <summary>
		/// Available devices of this platform 
		/// </summary>
		public List<string>? Available { get; set; }

		/// <summary>
		/// Devices in maintenance state
		/// </summary>
		public List<string>? Maintenance { get; set; }

		/// <summary>
		/// Devices in problem state
		/// </summary>
		public List<string>? Problem { get; set; }

		/// <summary>
		/// Number of devices in disabled state
		/// </summary>
		public List<string>? Disabled { get; set; }

		/// <summary>
		/// Reserved devices
		/// </summary>
		public Dictionary<string, List<GetDevicePoolReservationTelemetryResponse>>? Reserved { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetDevicePlatformTelemetryResponse(string platformId, List<string>? available, List<string>? maintenance, List<string>? problem, List<string>? disabled, Dictionary<string, List<GetDevicePoolReservationTelemetryResponse>>? reserved)
		{
			PlatformId = platformId;
			if (available != null && available.Count > 0)
			{
				Available = available;
			}
			if (maintenance != null && maintenance.Count > 0)
			{
				Maintenance = maintenance;
			}
			if (problem != null && problem.Count > 0)
			{
				Problem = problem;
			}
			if (disabled != null && disabled.Count > 0)
			{
				Disabled = disabled;
			}

			if (reserved != null && reserved.Count > 0)
			{
				Reserved = reserved;
			}
		}
	}

	/// <summary>
	/// Device telemetry respponse
	/// </summary>
	public class GetDevicePoolTelemetryResponse
	{
		/// <summary>
		/// The UTC time the telemetry data was created
		/// </summary>
		public DateTime CreateTimeUtc { get; set; }

		/// <summary>
		/// Individual pool telemetry data points
		/// </summary>
		public Dictionary<string, List<GetDevicePlatformTelemetryResponse>> Telemetry { get; set; } = null!;

		/// <summary>
		/// Constructor
		/// </summary>
		public GetDevicePoolTelemetryResponse(DateTime createTimeUtc, Dictionary<string, List<GetDevicePlatformTelemetryResponse>> telemetry)
		{
			CreateTimeUtc = createTimeUtc;
			Telemetry = telemetry;
		}
	}

	// Legacy clients

	/// <summary>
	/// Reservation request for legacy clients
	/// </summary>
	public class LegacyCreateReservationRequest
	{
		/// <summary>
		/// The device types to reserve, these are mapped to platforms
		/// </summary>
		[Required]
		public string[] DeviceTypes { get; set; } = null!;

		/// <summary>
		/// The hostname of machine reserving devices 
		/// </summary>
		[Required]
		public string Hostname { get; set; } = null!;

		/// <summary>
		/// The duration of reservation
		/// </summary>
		[Required]
		public string Duration { get; set; } = null!;

		/// <summary>
		/// Reservation details string
		/// </summary>
		public string? ReservationDetails { get; set; } = null;

		/// <summary>
		/// The PoolId of reservation request 
		/// </summary>
		public string? PoolId { get; set; } = null;

		/// <summary>
		/// The JobId of reservation request 
		/// </summary>
		public string? JobId { get; set; } = null;

		/// <summary>
		/// The StepId of reservation request 
		/// </summary>
		public string? StepId { get; set; } = null;

		/// <summary>
		/// A specific device to reserve
		/// </summary>
		public string? DeviceName { get; set; } = null;

		/// <summary>
		/// Desired metadata of the device
		/// </summary>
		public DeviceTagsRequest? Tags { get; set; } = null;
	}

	/// <summary>
	/// Reservation response for legacy clients
	/// </summary>
	public class GetLegacyReservationResponse
	{
		/// <summary>
		/// The names of the devices that were reserved
		/// </summary>
		[Required]
		public string[] DeviceNames { get; set; } = null!;

		/// <summary>
		/// The corresponding perf specs of the reserved devices
		/// </summary>
		[Required]
		public string[] DevicePerfSpecs { get; set; } = null!;

		/// <summary>
		/// The corresponding perf specs of the reserved devices
		/// </summary>
		[Required]
		public string[] DeviceModels { get; set; } = null!;

		/// <summary>
		/// The host name of the machine making the reservation
		/// </summary>
		[Required]
		public string HostName { get; set; } = null!;

		/// <summary>
		/// The start time of the reservation
		/// </summary>
		[Required]
		public string StartDateTime { get; set; } = null!;

		/// <summary>
		/// The duration of the reservation (before renew)
		/// </summary>
		[Required]
		public string Duration { get; set; } = null!;

		/// <summary>
		/// The JobId of reservation request 
		/// </summary>
		public string? JobId { get; set; } = null;

		/// <summary>
		/// The StepId of reservation request 
		/// </summary>
		public string? StepId { get; set; } = null;

		/// <summary>
		/// The job name of reservation request 
		/// </summary>
		public string? JobName { get; set; } = null;

		/// <summary>
		/// The step name of reservation request 
		/// </summary>
		public string? StepName { get; set; } = null;

		/// <summary>
		/// The legacy guid of the reservation
		/// </summary>
		[Required]
		public string Guid { get; set; } = null!;

		/// <summary>
		/// The step name of reservation request 
		/// </summary>
		public bool? InstallRequired { get; set; } = null;

	}

	/// <summary>
	/// Device response for legacy clients
	/// </summary>
	public class GetLegacyDeviceResponse
	{
		/// <summary>
		/// The  id of the reserved device
		/// </summary>
		[Required]
		public string Id { get; set; } = null!;

		/// <summary>
		/// The  name of the reserved device
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The (legacy) type of the device, mapped from platform id
		/// </summary>
		[Required]
		public string Type { get; set; } = null!;

		/// <summary>
		/// The IP or hostname of device
		/// </summary>
		[Required]
		public string IPOrHostName { get; set; } = null!;

		/// <summary>
		/// The (legacy) perf spec of the device
		/// </summary>
		[Required]
		public string PerfSpec { get; set; } = null!;

		/// <summary>
		/// The device model information
		/// </summary>
		[Required]
		public string Model { get; set; } = null!;

		/// <summary>
		/// The available start time which is parsed client side
		/// </summary>
		[Required]
		public string AvailableStartTime { get; set; } = null!;

		/// <summary>
		/// The available end time which is parsed client side
		/// </summary>
		[Required]
		public string AvailableEndTime { get; set; } = null!;

		/// <summary>
		/// Whether device is enabled
		/// </summary>
		[Required]
		public bool Enabled { get; set; }

		/// <summary>
		/// Associated device data
		/// </summary>
		[Required]
		public string DeviceData { get; set; } = null!;

		/// <summary>
		/// Any tags associated with the device
		/// </summary>
		[Required]
		public List<string> Tags { get; set; } = null!;
	}
}
