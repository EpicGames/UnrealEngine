// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Devices
{
	/// <summary>
	/// Identifier for a device platform
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<DevicePlatformId, DevicePlatformIdConverter>))]
	[StringIdConverter(typeof(DevicePlatformIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<DevicePlatformId, DevicePlatformIdConverter>))]
	public readonly record struct DevicePlatformId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public DevicePlatformId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc cref="StringId.Sanitize(System.String)"/>
		public static DevicePlatformId Sanitize(string text) => new(StringId.Sanitize(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class DevicePlatformIdConverter : StringIdConverter<DevicePlatformId>
	{
		/// <inheritdoc/>
		public override DevicePlatformId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(DevicePlatformId value) => value.Id;
	}
}
