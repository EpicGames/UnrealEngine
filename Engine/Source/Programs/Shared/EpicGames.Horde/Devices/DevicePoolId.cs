// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Devices
{
	/// <summary>
	/// Identifier for a device pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<DevicePoolId, DevicePoolIdConverter>))]
	[StringIdConverter(typeof(DevicePoolIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<DevicePoolId, DevicePoolIdConverter>))]
	public readonly record struct DevicePoolId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public DevicePoolId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc cref="StringId.Sanitize(System.String)"/>
		public static DevicePoolId Sanitize(string text) => new(StringId.Sanitize(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class DevicePoolIdConverter : StringIdConverter<DevicePoolId>
	{
		/// <inheritdoc/>
		public override DevicePoolId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(DevicePoolId value) => value.Id;
	}
}
