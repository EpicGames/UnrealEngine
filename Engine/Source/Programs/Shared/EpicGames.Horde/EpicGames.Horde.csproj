<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
    <AllowUnsafeBlocks>True</AllowUnsafeBlocks>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="BitFaster.Caching" Version="2.4.1" />
    <PackageReference Include="Dapper" Version="2.1.66" />
    <PackageReference Include="Google.Protobuf" Version="3.25.8" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="K4os.Compression.LZ4" Version="1.2.16" />
    <PackageReference Include="Microsoft.Data.Sqlite" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Caching.Memory" Version="10.0.6" />
    <PackageReference Include="Microsoft.VisualStudio.Threading.Analyzers" Version="17.14.15">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="ZstdSharp.Port" Version="0.8.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Analytics.Telemetry\EpicGames.Analytics.Telemetry.csproj" />
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\EpicGames.OIDC\EpicGames.OIDC.csproj" />
    <ProjectReference Include="..\EpicGames.Oodle\EpicGames.Oodle.csproj" />
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Protobuf Include="Protos\horde\log_rpc.proto" ProtoRoot="Protos" GrpcServices="Both" Access="Public" />
    <Protobuf Include="Protos\horde\log_rpc_messages.proto" ProtoRoot="Protos" GrpcServices="None" Access="Public" />
  </ItemGroup>
</Project>
