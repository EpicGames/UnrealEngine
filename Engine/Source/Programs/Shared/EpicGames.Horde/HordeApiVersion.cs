// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json;
using System.Text.Json.Serialization;

#pragma warning disable CA1027
#pragma warning disable CA1069

namespace EpicGames.Horde
{
	/// <summary>
	/// Version number for the Horde public API. Can be retrieved from the /api/v1/server/info endpoint via the
	/// <see cref="Server.GetServerInfoResponse"/> response. Should be serialized as an integer in messages to allow
	/// clients with missing enum names to still parse the result correctly.
	/// </summary>
	public enum HordeApiVersion
	{
		/// <summary>
		/// Unknown version
		/// </summary>
		Unknown = 0,

		/// <summary>
		/// Initial version
		/// </summary>
		Initial = 1,

		/// <summary>
		/// Interior nodes in chunked data now include the length of chunked data to allow seeking.
		/// </summary>
		AddLengthsToInteriorNodes = 2,

		/// <summary>
		/// Add support for last modified timestamps to file entries in directory nodes
		/// </summary>
		AddFileModTimes = 3,

		/// <summary>
		/// Interior nodes in chunked data now include the rolling hash of any leaf nodes
		/// </summary>
		AddRollingHashesForLeafNodes = 4,

		/// <summary>
		/// One past the latest known version number. Add new version numbers above this point.
		/// </summary>
		LatestPlusOne,

		/// <summary>
		/// Latest API version
		/// </summary>
		Latest = (int)(LatestPlusOne - 1),
	}

	/// <summary>
	/// Converter for <see cref="HordeApiVersion"/> which forces serialization as an integer, to override any 
	/// default <see cref="JsonStringEnumConverter"/> that may be enabled.
	/// </summary>
	public class HordeApiVersionConverter : JsonConverter<HordeApiVersion>
	{
		/// <inheritdoc/>
		public override HordeApiVersion Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return (HordeApiVersion)reader.GetInt32();
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, HordeApiVersion value, JsonSerializerOptions options)
		{
			writer.WriteNumberValue((int)value);
		}
	}
}
