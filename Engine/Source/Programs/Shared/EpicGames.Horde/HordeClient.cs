// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Http.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Auth;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Clients;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Server;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Tools;
using EpicGames.OIDC;
using Grpc.Core;
using Grpc.Net.Client;
using Grpc.Net.Client.Configuration;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

#pragma warning disable CA2234 // Use URIs instead of strings

namespace EpicGames.Horde
{
	/// <summary>
	/// Default implementation of <see cref="IHordeClient"/>
	/// </summary>
	public abstract class HordeClient : IHordeClient, IAsyncDisposable
	{
		readonly Uri _serverUrl;
		readonly BundleCache _bundleCache;
		readonly HordeOptions _hordeOptions;
		readonly ILoggerFactory _loggerFactory;
		readonly ILogger _logger;

		ServerComputeClient? _serverComputeClient;
		BackgroundTask<GrpcChannel>? _grpcChannel;

		/// <inheritdoc/>
		public Uri ServerUrl => _serverUrl;

		/// <inheritdoc/>
		public IArtifactCollection Artifacts { get; }

		/// <inheritdoc/>
		public IComputeClient Compute => _serverComputeClient ??= CreateComputeClient();

		/// <inheritdoc/>
		public IProjectCollection Projects { get; }

		/// <inheritdoc/>
		public ISecretCollection Secrets { get; }

		/// <inheritdoc/>
		public IToolCollection Tools { get; }

		/// <inheritdoc/>
		public event Action? OnAccessTokenStateChanged;

		/// <summary>
		/// Accessor for the logger instance
		/// </summary>
		protected ILogger Logger => _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		protected HordeClient(Uri serverUrl, BundleCache bundleCache, IOptionsSnapshot<HordeOptions> hordeOptions, ILoggerFactory loggerFactory)
		{
			_serverUrl = serverUrl;
			_bundleCache = bundleCache;
			_hordeOptions = hordeOptions.Value;
			_loggerFactory = loggerFactory;
			_logger = _loggerFactory.CreateLogger<HordeClient>();

			Artifacts = new HttpArtifactCollection(this);
			Projects = new HttpProjectCollection(this);
			Secrets = new HttpSecretCollection(this);
			Tools = new HttpToolCollection(this);
		}

		/// <inheritdoc/>
		public virtual async ValueTask DisposeAsync()
		{
			_serverComputeClient?.Dispose();

			if (_grpcChannel != null)
			{
				await _grpcChannel.DisposeAsync();
				_grpcChannel = null;
			}

			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Notify listeners that the auth state has changed
		/// </summary>
		protected void NotifyAuthStateChanged()
		{
			OnAccessTokenStateChanged?.Invoke();
		}

		/// <inheritdoc/>
		public abstract Task<bool> LoginAsync(bool interactive, CancellationToken cancellationToken);

		/// <inheritdoc/>
		public abstract bool HasValidAccessToken();

		/// <inheritdoc/>
		public abstract Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken);

		/// <inheritdoc/>
		public async Task<GrpcChannel> CreateGrpcChannelAsync(CancellationToken cancellationToken)
		{
			_grpcChannel ??= BackgroundTask.StartNew(ctx => CreateGrpcChannelInternalAsync(ctx));
			return await _grpcChannel.WaitAsync(cancellationToken);
		}

		async Task<GrpcChannel> CreateGrpcChannelInternalAsync(CancellationToken cancellationToken)
		{
			Uri serverUri = ServerUrl;
			bool useInsecureConnection = serverUri.Scheme.Equals("http", StringComparison.Ordinal);

			// Get the server URL for gRPC traffic. If we're using an unencrypted connection we need to use a different port for HTTP/2,
			// so send a HTTP/1 request to the server to query it.
			if (useInsecureConnection)
			{
				_logger.LogInformation("Querying server {BaseUrl} for rpc port", serverUri);
				using HttpClient httpClient = CreateUnauthenticatedHttpClient();
				httpClient.DefaultRequestHeaders.Add("Accept", "application/json");
				httpClient.Timeout = TimeSpan.FromSeconds(210); // Need to make sure this doesn't cancel any long running gRPC streaming calls (eg. session update)
				using HttpResponseMessage response = await httpClient.GetAsync(new Uri(serverUri, "api/v1/server/ports"), cancellationToken);
				GetPortsResponse? ports = await response.Content.ReadFromJsonAsync<GetPortsResponse>(HordeHttpClient.JsonSerializerOptions, cancellationToken);
				if (ports is { UnencryptedHttp2: not null } && ports.UnencryptedHttp2.Value != 0)
				{
					UriBuilder builder = new(serverUri);
					builder.Port = ports.UnencryptedHttp2.Value;
					serverUri = builder.Uri;
				}
			}

#pragma warning disable CA2000 // Dispose objects before losing scope
			SocketsHttpHandler httpHandler = new();
#pragma warning restore CA2000 // Dispose objects before losing scope

			// Create options for the new channel
			GrpcChannelOptions channelOptions = new();
			channelOptions.MaxReceiveMessageSize = 1024 * 1024 * 1024; // 1 GB 		// Required payloads coming from CAS service can be large
			channelOptions.MaxSendMessageSize = 1024 * 1024 * 1024; // 1 GB
			channelOptions.LoggerFactory = _loggerFactory;
			channelOptions.HttpHandler = httpHandler;
			channelOptions.DisposeHttpClient = true;
			channelOptions.ServiceConfig = new ServiceConfig();
			channelOptions.ServiceConfig.MethodConfigs.Add(new MethodConfig
			{
				Names = { MethodName.Default },
				RetryPolicy = new Grpc.Net.Client.Configuration.RetryPolicy
				{
					MaxAttempts = 3,
					InitialBackoff = TimeSpan.FromSeconds(1),
					MaxBackoff = TimeSpan.FromSeconds(10),
					BackoffMultiplier = 2.0,
					RetryableStatusCodes = { StatusCode.Unavailable },
				}
			});

			// Configure requests to send the bearer token
			string? bearerToken = await GetAccessTokenAsync(_hordeOptions.AllowAuthPrompt, cancellationToken);
			if (!String.IsNullOrEmpty(bearerToken))
			{
				CallCredentials callCredentials = CallCredentials.FromInterceptor((context, metadata) =>
				{
					metadata.Add("Authorization", $"Bearer {bearerToken}");
					return Task.CompletedTask;
				});

				channelOptions.Credentials = ChannelCredentials.Create(useInsecureConnection ? ChannelCredentials.Insecure : ChannelCredentials.SecureSsl, callCredentials);
				channelOptions.UnsafeUseInsecureChannelCallCredentials = useInsecureConnection;
			}

			// Create the channel
			_logger.LogInformation("gRPC channel connecting to {BaseUrl} ...", serverUri);
			return GrpcChannel.ForAddress(serverUri, channelOptions);
		}

		/// <inheritdoc/>
		public async Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default)
			where TClient : ClientBase<TClient>
		{
			GrpcChannel channel = await CreateGrpcChannelAsync(cancellationToken);
			return (TClient)Activator.CreateInstance(typeof(TClient), channel)!;
		}

		/// <inheritdoc/>
		public HordeHttpClient CreateHttpClient()
			=> new(CreateAuthenticatedHttpClient());

		/// <inheritdoc/>
		ServerComputeClient CreateComputeClient()
		{
			string? sessionId = null;
			string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			string? batchId = Environment.GetEnvironmentVariable("UE_HORDE_BATCHID");
			string? stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID");

			if (jobId != null && batchId != null && stepId != null)
			{
				sessionId = $"{jobId}-{batchId}-{stepId}";
			}

			return new ServerComputeClient(CreateAuthenticatedHttpClient(), sessionId, _loggerFactory.CreateLogger<ServerComputeClient>());
		}

		/// <inheritdoc/>
		public IStorageNamespace GetStorageNamespace(string basePath, string? accessToken = null)
		{
			HttpClient CreateClient()
			{
				if (accessToken != null)
				{
					HttpClient httpClient = CreateUnauthenticatedHttpClient();
					httpClient.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Bearer", accessToken);
					return httpClient;
				}
				else
				{
					return CreateAuthenticatedHttpClient();
				}
			}

			// Upload redirect clients talk directly to S3 (or similar) via presigned URLs.
			// Use a bare SocketsHttpHandler without the Polly retry/timeout policies that are
			// designed for Horde server communication. The single outer retry policy in
			// WriteBlobAsync provides adequate retry behavior for transient S3 errors.
			static HttpClient CreateUploadRedirectClient() => new(HttpMessageHandlers.CreateDefaultHandler());

			HttpStorageBackend httpStorageBackend = new(basePath, CreateClient, CreateUploadRedirectClient, _loggerFactory.CreateLogger<HttpStorageBackend>());
			return new BundleStorageNamespace(httpStorageBackend, _bundleCache, _hordeOptions.Bundle, _loggerFactory.CreateLogger<BundleStorageNamespace>());
		}

		/// <inheritdoc/>
		public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information)
			=> new ServerLogger(this, logId, minimumLevel, _logger);

		/// <summary>
		/// Creates an http client for satisfying requests
		/// </summary>
		protected abstract HttpClient CreateAuthenticatedHttpClient();

		/// <summary>
		/// Creates an http client for satisfying requests
		/// </summary>
		protected abstract HttpClient CreateUnauthenticatedHttpClient();
	}

	/// <summary>
	/// Default implementation of <see cref="IHordeClient"/>
	/// </summary>
	class HordeClientWithStaticCredentials : HordeClient
	{
		readonly string? _accessToken;
		readonly IHordeHttpMessageHandler _httpMessageHandler;

		/// <summary>
		/// Constructor
		/// </summary>
		public HordeClientWithStaticCredentials(Uri serverUrl, string? accessToken, IHordeHttpMessageHandler httpMessageHandler, BundleCache bundleCache, IOptionsSnapshot<HordeOptions> hordeOptions, ILoggerFactory loggerFactory)
			: base(serverUrl, bundleCache, hordeOptions, loggerFactory)
		{
			_accessToken = accessToken;
			_httpMessageHandler = httpMessageHandler;
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();
		}

		/// <inheritdoc/>
		public override async Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken)
		{
			using (HttpClient httpClient = CreateAuthenticatedHttpClient())
			{
				using HttpRequestMessage request = new(HttpMethod.Get, "api/v1/dashboard/challenge");
				using HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken);
				return response.IsSuccessStatusCode;
			}
		}

		/// <inheritdoc/>
		public override bool HasValidAccessToken()
			=> true;

		/// <inheritdoc/>
		public override Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken)
			=> Task.FromResult(_accessToken);

		/// <inheritdoc/>
		protected override HttpClient CreateAuthenticatedHttpClient()
		{
			HttpClient httpClient = new(_httpMessageHandler.Instance, false);
			httpClient.BaseAddress = ServerUrl;
			if (!String.IsNullOrEmpty(_accessToken))
			{
				httpClient.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Bearer", _accessToken);
			}
			return httpClient;
		}

		/// <inheritdoc/>
		protected override HttpClient CreateUnauthenticatedHttpClient()
		{
			HttpClient httpClient = new(_httpMessageHandler.Instance, false);
			httpClient.BaseAddress = ServerUrl;
			return httpClient;
		}
	}

	/// <summary>
	/// Default implementation of <see cref="IHordeClient"/>
	/// </summary>
	class HordeClientWithDynamicCredentials : HordeClient
	{
		private const string AllowInteractiveLoginHeaderKey = "Horde-AllowInteractiveLogin";
		private static readonly HttpRequestOptionsKey<bool> s_allowInteractiveLogin = new(AllowInteractiveLoginHeaderKey);

		readonly OidcAuthState _authHandlerState;
		readonly IHordeHttpMessageHandler _baseHttpMessageHandler;
		private readonly ITokenStore _tokenStore;
		private readonly HordeAuthConfigProvider _hordeAuthConfigProvider;

		[SuppressMessage("Usage", "CA2213:Disposable fields should be disposed", Justification = "Lifetime is managed by caller")]
		readonly HttpMessageHandler _authHttpMessageHandler;

		/// <summary>
		/// Constructor
		/// </summary>
		public HordeClientWithDynamicCredentials(Uri serverUrl, BundleCache bundleCache, IHordeHttpMessageHandler baseHttpMessageHandler, IOptionsSnapshot<HordeOptions> hordeOptions, ILoggerFactory loggerFactory)
			: base(serverUrl, bundleCache, hordeOptions, loggerFactory)
		{
			_baseHttpMessageHandler = baseHttpMessageHandler;

			_tokenStore = TokenStoreFactory.CreateTokenStore(loggerFactory);
			_hordeAuthConfigProvider =
				new HordeAuthConfigProvider(_baseHttpMessageHandler.Instance, serverUrl, loggerFactory.CreateLogger<HordeAuthConfigProvider>());
			IOidcTokenManager oidcTokenManager = new HordeOidcTokenManager(_hordeAuthConfigProvider, _tokenStore, loggerFactory.CreateLogger<HordeOidcTokenManager>(), loggerFactory);
			IStaticAccessTokenProvider staticAccessTokenProvider = new HordeStaticAccessTokenProvider(hordeOptions);
			HttpOidcAuthHandlerConfig httpOidcAuthHandlerConfig = HordeOptionsToIHttpOidcAuthHandlerConfig(hordeOptions.Value);
			IClock clock = new DefaultClock();
			_authHandlerState = new OidcAuthState(oidcTokenManager, staticAccessTokenProvider, clock, _hordeAuthConfigProvider, loggerFactory.CreateLogger<OidcAuthState>());
			_authHttpMessageHandler = new HttpOidcAuthHandler(_baseHttpMessageHandler.Instance, _authHandlerState, httpOidcAuthHandlerConfig, loggerFactory.CreateLogger<HttpOidcAuthHandler>());

			_authHandlerState.OnStateChanged += NotifyAuthStateChanged;
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			_authHandlerState.OnStateChanged -= NotifyAuthStateChanged;

			await _authHandlerState.DisposeAsync();
			_hordeAuthConfigProvider.Dispose();
			_tokenStore.Dispose();
			await base.DisposeAsync();
		}

		/// <inheritdoc/>
		public override async Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken)
		{
			// Reset any cached state in the auth handler
			_authHandlerState.Reset();

			// Send a request to the server to log in automatically
			using (HttpClient httpClient = CreateAuthenticatedHttpClient())
			{
				using HttpRequestMessage request = new(HttpMethod.Get, "api/v1/dashboard/challenge");
				request.Options.Set(s_allowInteractiveLogin, allowLogin);

				using HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken);
				return response.IsSuccessStatusCode;
			}
		}

		/// <inheritdoc/>
		public override bool HasValidAccessToken()
		{
			try
			{
				return _authHandlerState.IsLoggedIn;
			}
			catch
			{
				return false;
			}
		}

		/// <inheritdoc/>
		public override Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken)
			=> _authHandlerState.GetAccessTokenAsync(interactive, cancellationToken);

		/// <inheritdoc/>
		protected override HttpClient CreateAuthenticatedHttpClient()
		{
			HttpClient httpClient = new(_authHttpMessageHandler, false);
			httpClient.BaseAddress = ServerUrl;
			return httpClient;
		}

		/// <inheritdoc/>
		protected override HttpClient CreateUnauthenticatedHttpClient()
		{
			HttpClient httpClient = new(_baseHttpMessageHandler.Instance, false);
			httpClient.BaseAddress = ServerUrl;
			return httpClient;
		}

		private static HttpOidcAuthHandlerConfig HordeOptionsToIHttpOidcAuthHandlerConfig(HordeOptions options) =>
			new(AllowInteractiveLoginHeaderKey, options.AllowAuthPrompt);
	}

	/// <summary>
	/// Allows creating <see cref="HordeClient"/> instances
	/// </summary>
	public class HordeClientFactory
	{
		readonly BundleCache _bundleCache;
		readonly IHordeHttpMessageHandler _httpMessageHandler;
		readonly IOptionsSnapshot<HordeOptions> _hordeOptions;
		readonly ILoggerFactory _loggerFactory;

		/// <summary>
		/// Constructor 
		/// </summary>
		public HordeClientFactory(BundleCache bundleCache, IHordeHttpMessageHandler httpMessageHandler, IOptionsSnapshot<HordeOptions> hordeOptions, ILoggerFactory loggerFactory)
		{
			_bundleCache = bundleCache;
			_httpMessageHandler = httpMessageHandler;
			_hordeOptions = hordeOptions;
			_loggerFactory = loggerFactory;
		}

		Uri GetDefaultServerUrl()
		{
			Uri? serverUrl = _hordeOptions.Value.GetServerUrlOrDefault();
			if (serverUrl == null)
			{
				throw new Exception("No Horde server is configured, or can be detected from the environment. Consider specifying a URL when calling AddHordeHttpClient().");
			}
			return serverUrl;
		}

		/// <summary>
		/// Create a client using the user's default access token
		/// </summary>
		public HordeClient Create(Uri? serverUrl = null, string? accessToken = null)
		{
			serverUrl ??= GetDefaultServerUrl();
			if (accessToken == null)
			{
				return new HordeClientWithDynamicCredentials(serverUrl, _bundleCache, _httpMessageHandler, _hordeOptions, _loggerFactory);
			}
			else
			{
				return new HordeClientWithStaticCredentials(serverUrl, accessToken, _httpMessageHandler, _bundleCache, _hordeOptions, _loggerFactory);
			}
		}
	}
}
