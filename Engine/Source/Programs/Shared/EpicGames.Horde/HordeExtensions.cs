// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace EpicGames.Horde
{
	/// <summary>
	/// Extension methods for Horde
	/// </summary>
	public static class HordeExtensions
	{
		/// <summary>
		/// Adds Horde-related services with the default settings
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		public static void AddHorde(this IServiceCollection serviceCollection)
		{
			static StorageBackendCache CreateBackendCache(IServiceProvider serviceProvider)
			{
				StorageBackendCacheOptions options = serviceProvider.GetRequiredService<IOptions<HordeOptions>>().Value.BackendCache;
				DirectoryReference? cacheDir = String.IsNullOrEmpty(options.CacheDir) ? null : new DirectoryReference(options.CacheDir);
				return new StorageBackendCache(cacheDir, options.MaxSize, serviceProvider.GetRequiredService<ILogger<StorageBackendCache>>());
			}

			serviceCollection.AddLogging();

			serviceCollection.AddHttpClient();
			serviceCollection.AddHttpClient(HordeHttpClient.StorageHttpClientName).AddPolicyHandler(HordeHttpMessageHandler.CreateDefaultTransientErrorPolicy());
			serviceCollection.AddSingleton<BundleCache>(sp => new BundleCache(sp.GetRequiredService<IOptions<HordeOptions>>().Value.BundleCache));
			serviceCollection.AddSingleton<StorageBackendCache>(CreateBackendCache);
			serviceCollection.AddSingleton<HttpStorageBackendFactory>();
			serviceCollection.AddSingleton<HttpStorageClient>();
			serviceCollection.AddSingleton<IHordeHttpMessageHandler, HordeHttpMessageHandler>();
			serviceCollection.AddSingleton<IHordeClient>(sp => sp.GetRequiredService<HordeClientFactory>().Create());
			serviceCollection.AddSingleton<HordeClientFactory>();
		}

		/// <summary>
		/// Adds Horde-related services
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		/// <param name="configureHorde">Callback to configure options</param>
		public static void AddHorde(this IServiceCollection serviceCollection, Action<HordeOptions> configureHorde)
		{
			serviceCollection.Configure(configureHorde);
			AddHorde(serviceCollection);
		}
	}
}
