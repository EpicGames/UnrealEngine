// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Http.Json;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Agents.Telemetry;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Dashboard;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Server;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Tools;
using EpicGames.Horde.Ugs;

using static EpicGames.Horde.HordeHttpRequest;

#pragma warning disable CA2234

namespace EpicGames.Horde
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Wraps an Http client which communicates with the Horde server
	/// </summary>
	public sealed class HordeHttpClient : IDisposable
	{
		/// <summary>
		/// Name of an environment variable containing the Horde server URL
		/// </summary>
		public const string HordeUrlEnvVarName = "UE_HORDE_URL";

		/// <summary>
		/// Name of an environment variable containing a token for connecting to the Horde server
		/// </summary>
		public const string HordeTokenEnvVarName = "UE_HORDE_TOKEN";

		/// <summary>
		/// Name of clients created from the http client factory
		/// </summary>
		public const string HttpClientName = "HordeHttpClient";

		/// <summary>
		/// Name of clients used for anonymous requests.
		/// </summary>
		public const string AnonymousHttpClientName = "HordeAnonymousHttpClient";

		/// <summary>
		/// Name of clients created for storage operations
		/// </summary>
		public const string StorageHttpClientName = "HordeStorageHttpClient";

		/// <summary>
		/// Name of clients created from the http client factory for handling upload redirects. Should not contain Horde auth headers.
		/// </summary>
		public const string UploadRedirectHttpClientName = "HordeUploadRedirectHttpClient";

		/// <summary>
		/// Accessor for the inner http client
		/// </summary>
		public HttpClient HttpClient => _httpClient;

		readonly HttpClient _httpClient;

		internal static JsonSerializerOptions JsonSerializerOptions => HordeHttpRequest.JsonSerializerOptions;

		/// <summary>
		/// Base address for the Horde server
		/// </summary>
		public Uri BaseUrl => _httpClient.BaseAddress ?? throw new InvalidOperationException("Expected Horde server base address to be configured");

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="httpClient">The inner HTTP client instance</param>
		public HordeHttpClient(HttpClient httpClient)
		{
			_httpClient = httpClient;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_httpClient.Dispose();
		}

		/// <summary>
		/// Configures a JSON serializer to read Horde responses
		/// </summary>
		/// <param name="options">options for the serializer</param>
		public static void ConfigureJsonSerializer(JsonSerializerOptions options)
			=> HordeHttpRequest.ConfigureJsonSerializer(options);

		#region Connection
		/// <summary>
		/// Check account login status.
		/// </summary>
		public async Task<bool> CheckConnectionAsync(CancellationToken cancellationToken = default)
		{
			HttpResponseMessage response = await _httpClient.GetAsync("account", cancellationToken);

			return response.IsSuccessStatusCode;
		}

		#endregion

		#region Agents

		/// <summary>
		/// Gets a list of agents matching the specified criteria
		/// </summary>
		/// <param name="poolId">Optional pool to filter by</param>
		/// <param name="includeDeleted">Whether to include deleted agents</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<List<GetAgentResponse>> GetAgentsAsync(PoolId? poolId = null, bool includeDeleted = false, CancellationToken cancellationToken = default)
		{
			QueryStringBuilder queryParams = new();
			if (poolId != null)
			{
				queryParams.Add("poolId", poolId.Value.ToString());
			}
			queryParams.Add("includeDeleted", includeDeleted ? "true" : "false");
			return GetAsync<List<GetAgentResponse>>(_httpClient, $"api/v1/agents?{queryParams}", cancellationToken);
		}

		/// <summary>
		/// Gets information about a specific agent
		/// </summary>
		/// <param name="agentId">The agent identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<GetAgentResponse> GetAgentAsync(AgentId agentId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetAgentResponse>(_httpClient, $"api/v1/agents/{agentId}", cancellationToken);
		}

		#endregion

		#region Leases

		/// <summary>
		/// Gets a list of leases matching the specified criteria
		/// </summary>
		/// <param name="agentId">Optional agent to filter by</param>
		/// <param name="sessionId">Optional session to filter by</param>
		/// <param name="startTime">Start of the time window</param>
		/// <param name="finishTime">End of the time window</param>
		/// <param name="minFinishTime">Minimum finish time filter</param>
		/// <param name="maxFinishTime">Maximum finish time filter</param>
		/// <param name="index">Starting index for pagination</param>
		/// <param name="count">Maximum number of results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<List<GetAgentLeaseResponse>> GetLeasesAsync(AgentId? agentId = null, SessionId? sessionId = null, DateTime? startTime = null, DateTime? finishTime = null, DateTime? minFinishTime = null, DateTime? maxFinishTime = null, int index = 0, int count = 1000, CancellationToken cancellationToken = default)
		{
			QueryStringBuilder queryParams = new();
			if (agentId != null)
			{
				queryParams.Add("agentId", agentId.Value.ToString());
			}
			if (sessionId != null)
			{
				queryParams.Add("sessionId", sessionId.Value.ToString());
			}
			if (startTime != null)
			{
				queryParams.Add("startTime", startTime.Value.ToString("o"));
			}
			if (finishTime != null)
			{
				queryParams.Add("finishTime", finishTime.Value.ToString("o"));
			}
			if (minFinishTime != null)
			{
				queryParams.Add("minFinishTime", minFinishTime.Value.ToString("o"));
			}
			if (maxFinishTime != null)
			{
				queryParams.Add("maxFinishTime", maxFinishTime.Value.ToString("o"));
			}
			queryParams.Add("index", index.ToString());
			queryParams.Add("count", count.ToString());
			return GetAsync<List<GetAgentLeaseResponse>>(_httpClient, $"api/v1/leases?{queryParams}", cancellationToken);
		}

		#endregion

		#region Artifacts

		/// <summary>
		/// Creates a new artifact
		/// </summary>
		/// <param name="name">Name of the artifact</param>
		/// <param name="type">Additional search keys tagged on the artifact</param>
		/// <param name="description">Description for the artifact</param>
		/// <param name="streamId">Stream to create the artifact for</param>
		/// <param name="commitId">Commit for the artifact</param>
		/// <param name="keys">Keys used to identify the artifact</param>
		/// <param name="metadata">Metadata for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<CreateArtifactResponse> CreateArtifactAsync(ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitId commitId, IEnumerable<string>? keys = null, IEnumerable<string>? metadata = null, CancellationToken cancellationToken = default)
		{
			return PostAsync<CreateArtifactResponse, CreateArtifactRequest>(_httpClient, $"api/v2/artifacts", new CreateArtifactRequest(name, type, description, streamId, keys?.ToList() ?? [], metadata?.ToList() ?? []) { CommitId = commitId }, cancellationToken);
		}

		/// <summary>
		/// Deletes an artifact
		/// </summary>
		/// <param name="id">Identifier for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task DeleteArtifactAsync(ArtifactId id, CancellationToken cancellationToken = default)
		{
			await DeleteAsync(_httpClient, $"api/v2/artifacts/{id}", cancellationToken);
		}

		/// <summary>
		/// Gets metadata about an artifact object
		/// </summary>
		/// <param name="id">Identifier for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<GetArtifactResponse> GetArtifactAsync(ArtifactId id, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetArtifactResponse>(_httpClient, $"api/v2/artifacts/{id}", cancellationToken);
		}

		/// <summary>
		/// Gets a zip stream for a particular artifact
		/// </summary>
		/// <param name="id">Identifier for the artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<Stream> GetArtifactZipAsync(ArtifactId id, CancellationToken cancellationToken = default)
		{
			return await _httpClient.GetStreamAsync($"api/v2/artifacts/{id}/zip", cancellationToken);
		}

		/// <summary>
		/// Finds artifacts with a certain type with an optional streamId
		/// </summary>
		/// <param name="streamId">Stream to look for the artifact in</param>
		/// <param name="minCommitId">The minimum change number for the artifacts</param>
		/// <param name="maxCommitId">The minimum change number for the artifacts</param>
		/// <param name="name">Name of the artifact</param>
		/// <param name="type">Type to find</param>
		/// <param name="keys">Keys for artifacts to return</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		public Task<List<GetArtifactResponse>> FindArtifactsAsync(StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, CancellationToken cancellationToken = default)
			=> FindArtifactsAsync(null, streamId, minCommitId, maxCommitId, name, type, keys, maxResults, cancellationToken);

		/// <summary>
		/// Finds artifacts with a certain type with an optional streamId
		/// </summary>
		/// <param name="ids">Identifiers to return</param>
		/// <param name="streamId">Stream to look for the artifact in</param>
		/// <param name="minCommitId">The minimum change number for the artifacts</param>
		/// <param name="maxCommitId">The minimum change number for the artifacts</param>
		/// <param name="name">Name of the artifact</param>
		/// <param name="type">Type to find</param>
		/// <param name="keys">Keys for artifacts to return</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		public async Task<List<GetArtifactResponse>> FindArtifactsAsync(IEnumerable<ArtifactId>? ids, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, CancellationToken cancellationToken = default)
		{
			QueryStringBuilder queryParams = new();

			if (ids != null)
			{
				queryParams.Add("id", ids.Select(x => x.ToString()));
			}

			if (streamId != null)
			{
				queryParams.Add("streamId", streamId.ToString()!);
			}

			if (minCommitId != null)
			{
				queryParams.Add("minChange", minCommitId.ToString()!);
			}

			if (maxCommitId != null)
			{
				queryParams.Add("maxChange", maxCommitId.ToString()!);
			}

			if (name != null)
			{
				queryParams.Add("name", name.Value.ToString());
			}

			if (type != null)
			{
				queryParams.Add("type", type.Value.ToString());
			}

			if (keys != null)
			{
				foreach (string key in keys)
				{
					queryParams.Add("key", key);
				}
			}

			queryParams.Add("maxResults", maxResults.ToString());

			FindArtifactsResponse response = await GetAsync<FindArtifactsResponse>(_httpClient, $"api/v2/artifacts?{queryParams}", cancellationToken);
			return response.Artifacts;
		}

		#endregion

		#region Dashboard

		/// <summary>
		/// Create a new dashboard preview item
		/// </summary>
		/// <param name="request">Request to create a new preview item</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Config information needed by the dashboard</returns>
		public Task<GetDashboardPreviewResponse> CreateDashbordPreviewAsync(CreateDashboardPreviewRequest request, CancellationToken cancellationToken = default)
		{
			return PostAsync<GetDashboardPreviewResponse, CreateDashboardPreviewRequest>(_httpClient, $"api/v1/dashboard/preview", request, cancellationToken);
		}

		/// <summary>
		/// Update a dashboard preview item
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		public Task<GetDashboardPreviewResponse> UpdateDashbordPreviewAsync(UpdateDashboardPreviewRequest request, CancellationToken cancellationToken = default)
		{
			return PutAsync<GetDashboardPreviewResponse, UpdateDashboardPreviewRequest>(_httpClient, $"api/v1/dashboard/preview", request, cancellationToken);
		}

		/// <summary>
		/// Query dashboard preview items
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		public Task<List<GetDashboardPreviewResponse>> GetDashbordPreviewsAsync(bool open = true, CancellationToken cancellationToken = default)
		{
			return GetAsync<List<GetDashboardPreviewResponse>>(_httpClient, $"api/v1/dashboard/preview?open={open}", cancellationToken);
		}

		#endregion

		#region Parameters

		/// <summary>
		/// Query parameters for other tools
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Parameters for other tools</returns>
		public Task<JsonObject> GetParametersAsync(CancellationToken cancellationToken = default)
		{
			return GetParametersAsync(null, cancellationToken);
		}

		/// <summary>
		/// Query parameters for other tools
		/// </summary>
		/// <param name="path">Path for properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the projects</returns>
		public Task<JsonObject> GetParametersAsync(string? path, CancellationToken cancellationToken = default)
		{
			string url = "api/v1/parameters";
			if (!String.IsNullOrEmpty(path))
			{
				url = $"{url}/{path}";
			}
			return GetAsync<JsonObject>(_httpClient, url, cancellationToken);
		}

		#endregion

		#region Projects

		/// <summary>
		/// Query all the projects
		/// </summary>
		/// <param name="includeStreams">Whether to include streams in the response</param>
		/// <param name="includeCategories">Whether to include categories in the response</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the projects</returns>
		public Task<List<GetProjectResponse>> GetProjectsAsync(bool includeStreams = false, bool includeCategories = false, CancellationToken cancellationToken = default)
		{
			return GetAsync<List<GetProjectResponse>>(_httpClient, $"api/v1/projects?includeStreams={includeStreams}&includeCategories={includeCategories}", cancellationToken);
		}

		/// <summary>
		/// Retrieve information about a specific project
		/// </summary>
		/// <param name="projectId">Id of the project to get information about</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested project</returns>
		public Task<GetProjectResponse> GetProjectAsync(ProjectId projectId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetProjectResponse>(_httpClient, $"api/v1/projects/{projectId}", cancellationToken);
		}

		#endregion

		#region Secrets

		/// <summary>
		/// Query all the secrets available to the current user
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the projects</returns>
		public Task<GetSecretsResponse> GetSecretsAsync(CancellationToken cancellationToken = default)
		{
			return GetAsync<GetSecretsResponse>(_httpClient, $"api/v1/secrets", cancellationToken);
		}

		/// <summary>
		/// Retrieve information about a specific secret
		/// </summary>
		/// <param name="secretId">Id of the secret to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested secret</returns>
		public Task<GetSecretResponse> GetSecretAsync(SecretId secretId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetSecretResponse>(_httpClient, $"api/v1/secrets/{secretId}", cancellationToken);
		}

		/// <summary>
		/// Retrieve information about a specific secret and property
		/// </summary>
		/// <param name="value">A string representation of a secret to retrieve.
		/// A string that contains the "horde:secret:" prefix followed by secret id and property name e.g. 'horde:secret:my-secret.property' </param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested secret</returns>
		public Task<GetSecretPropertyResponse> ResolveSecretAsync(string value, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetSecretPropertyResponse>(_httpClient, $"api/v1/secrets/resolve/{value}", cancellationToken);
		}

		#endregion

		#region Server

		/// <summary>
		/// Gets information about the currently deployed server version
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the deployed server instance</returns>
		public Task<GetServerInfoResponse> GetServerInfoAsync(CancellationToken cancellationToken = default)
		{
			return GetAsync<GetServerInfoResponse>(_httpClient, "api/v1/server/info", cancellationToken);
		}

		#endregion

		#region Storage

		/// <summary>
		/// Attempts to read a named storage ref from the server
		/// </summary>
		/// <param name="path">Path to the ref</param>
		/// <param name="cacheTime">Max allowed age for a cached value to be returned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ReadRefResponse?> TryReadRefAsync(string path, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			using (HttpRequestMessage request = new(HttpMethod.Get, path))
			{
				if (cacheTime.IsSet())
				{
					request.Headers.CacheControl = new CacheControlHeaderValue { MaxAge = cacheTime.MaxAge };
				}

				using (HttpResponseMessage response = await _httpClient.SendAsync(request, cancellationToken))
				{
					if (response.StatusCode == HttpStatusCode.NotFound)
					{
						return null;
					}
					else if (!response.IsSuccessStatusCode)
					{
						throw new StorageException($"Unable to read ref '{path}' (status: {response.StatusCode}, body: {await response.Content.ReadAsStringAsync(cancellationToken)}");
					}
					else
					{
						return await response.Content.ReadFromJsonAsync<ReadRefResponse>(cancellationToken: cancellationToken);
					}
				}
			}
		}

		#endregion

		#region Telemetry

		/// <summary>
		/// Posts telemetry to Horde.
		/// </summary>
		/// <param name="appId">The id of the App.</param>
		/// <param name="appVersion">The version of the App.</param>
		/// <param name="appEnvironment">The environment.</param>
		/// <param name="payloads">A list of payloads to write as telemetry events.</param>
		/// <param name="sessionId">The session id which this telemetry is a part of.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout.</param>
		/// <returns>The HttpResponseMessage.</returns>
		/// <remarks>BadRequest can be returned under circusmtances where the payload is not serializable (primitive or null payload, <see cref="NotSupportedException"/>, and <see cref="JsonException"/>).</remarks>
		public async Task<HttpResponseMessage> PostTelemetryAsync(string appId, int appVersion, string appEnvironment, object[] payloads, string? sessionId = null, CancellationToken cancellationToken = default)
		{
			QueryStringBuilder queryParams = new();
			queryParams.Add("appId", appId);
			queryParams.Add("appVersion", appVersion.ToString());
			queryParams.Add("appEnvironment", appEnvironment);
			queryParams.Add("uploadType", "EtEventStream");

			if (!String.IsNullOrEmpty(sessionId))
			{
				queryParams.Add("SessionId", sessionId);
			}

			PostTelemetryEventStreamRequest request = new();

			try
			{
				for (int i = 0; i < payloads.Length; ++i)
				{
					object payload = payloads[i];
					JsonElement element = JsonSerializer.SerializeToElement(payload, JsonSerializerOptions);

					if (element.ValueKind == JsonValueKind.Object)
					{
						request.Events.Add(element);
					}
					else
					{
						return new HttpResponseMessage(HttpStatusCode.BadRequest);
					}
				}
			}
			catch (NotSupportedException) // The scenario where a type doesn't contain a JsonConverter.
			{
				return new HttpResponseMessage(HttpStatusCode.BadRequest);
			}
			catch (JsonException) // The scenario where a type encounters a json exception upon serialization.
			{
				return new HttpResponseMessage(HttpStatusCode.BadRequest);
			}

			HttpResponseMessage response = await PostAsync(_httpClient, $"/api/v1/telemetry?{queryParams}", request, cancellationToken);

			return response;
		}

		/// <summary>
		/// Gets telemetry for Horde within a given range
		/// </summary>
		/// <param name="endDate">End date for the range</param>
		/// <param name="range">Number of hours to return</param>
		/// <param name="tzOffset">Timezone offset</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<List<GetUtilizationDataResponse>> GetTelemetryAsync(DateTime endDate, int range, int? tzOffset = null, CancellationToken cancellationToken = default)
		{
			QueryStringBuilder queryParams = new();
			queryParams.Add("Range", range.ToString());
			if (tzOffset != null)
			{
				queryParams.Add("TzOffset", tzOffset.Value.ToString());
			}
			return GetAsync<List<GetUtilizationDataResponse>>(_httpClient, $"api/v1/reports/utilization/{endDate}?{queryParams}", cancellationToken);
		}

		#endregion

		#region Tools

		/// <summary>
		/// Enumerates all the available tools.
		/// </summary>
		public Task<GetToolsSummaryResponse> GetToolsAsync(CancellationToken cancellationToken = default)
		{
			return GetAsync<GetToolsSummaryResponse>(_httpClient, "api/v1/tools", cancellationToken);
		}

		/// <summary>
		/// Gets information about a particular tool
		/// </summary>
		public Task<GetToolResponse> GetToolAsync(ToolId id, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetToolResponse>(_httpClient, $"api/v1/tools/{id}", cancellationToken);
		}

		/// <summary>
		/// Gets information about a particular deployment
		/// </summary>
		public Task<GetToolDeploymentResponse> GetToolDeploymentAsync(ToolId id, ToolDeploymentId deploymentId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetToolDeploymentResponse>(_httpClient, $"api/v1/tools/{id}/deployments/{deploymentId}", cancellationToken);
		}

		/// <summary>
		/// Gets a zip stream for a particular deployment
		/// </summary>
		public async Task<Stream> GetToolDeploymentZipAsync(ToolId id, ToolDeploymentId? deploymentId, CancellationToken cancellationToken = default)
		{
			if (deploymentId == null)
			{
				return await _httpClient.GetStreamAsync($"api/v1/tools/{id}?action=zip", cancellationToken);
			}
			else
			{
				return await _httpClient.GetStreamAsync($"api/v1/tools/{id}/deployments/{deploymentId}?action=zip", cancellationToken);
			}
		}

		/// <summary>
		/// Creates a new tool deployment
		/// </summary>
		/// <param name="id">Id for the tool</param>
		/// <param name="version">Version string for the new deployment</param>
		/// <param name="duration">Duration over which to deploy the tool</param>
		/// <param name="createPaused">Whether to create the deployment, but do not start rolling it out yet</param>
		/// <param name="target">Location of a directory node describing the deployment</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ToolDeploymentId> CreateToolDeploymentAsync(ToolId id, string? version, double? duration, bool? createPaused, HashedBlobRefValue target, CancellationToken cancellationToken = default)
		{
			CreateToolDeploymentRequest request = new(version ?? String.Empty, duration, createPaused, target);
			CreateToolDeploymentResponse response = await PostAsync<CreateToolDeploymentResponse, CreateToolDeploymentRequest>(_httpClient, $"api/v2/tools/{id}/deployments", request, cancellationToken);
			return response.Id;
		}

		/// <summary>
		/// Updates a tool deployment (e.g. changing its state)
		/// </summary>
		/// <param name="id">Id for the tool</param>
		/// <param name="deploymentId">Id of the deployment to update</param>
		/// <param name="request">The update request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task UpdateToolDeploymentAsync(ToolId id, ToolDeploymentId deploymentId, UpdateDeploymentRequest request, CancellationToken cancellationToken = default)
		{
			return PatchAsync(_httpClient, $"api/v1/tools/{id}/deployments/{deploymentId}", request, cancellationToken);
		}

		/// <summary>
		/// Creates a new tool deployment by uploading a zip file via multipart form data (v1 API)
		/// </summary>
		/// <param name="id">Id for the tool</param>
		/// <param name="version">Version string for the new deployment</param>
		/// <param name="content">Stream containing the zip file</param>
		/// <param name="fileName">Name of the zip file</param>
		/// <param name="duration">Duration over which to deploy the tool</param>
		/// <param name="createPaused">Whether to create the deployment in a paused state</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ToolDeploymentId> CreateToolDeploymentFromStreamAsync(ToolId id, string version, Stream content, string fileName, TimeSpan? duration = null, bool createPaused = false, CancellationToken cancellationToken = default)
		{
#pragma warning disable IDE0028 // Collection initialization can be simplified - conditional Add calls prevent collection expression
			using MultipartFormDataContent formContent = new();
#pragma warning restore IDE0028

#pragma warning disable CA2000 // Disposed by MultipartFormDataContent
			formContent.Add(new StreamContent(content), "file", fileName);
			formContent.Add(new StringContent(version), "Version");
			if (duration != null)
			{
				formContent.Add(new StringContent(duration.Value.TotalSeconds.ToString()), "Duration");
			}
			if (createPaused)
			{
				formContent.Add(new StringContent("true"), "CreatePaused");
			}
#pragma warning restore CA2000

			using HttpResponseMessage response = await _httpClient.PostAsync($"api/v1/tools/{id}/deployments", formContent, cancellationToken);
			if (!response.IsSuccessStatusCode)
			{
				string body = await response.Content.ReadAsStringAsync(cancellationToken);
				throw new HttpRequestException(
					$"Failed to create tool deployment for '{id}' (status: {(int)response.StatusCode} {response.StatusCode}, body: {body})",
					inner: null,
					statusCode: response.StatusCode);
			}

			CreateToolDeploymentResponse? result = await response.Content.ReadFromJsonAsync<CreateToolDeploymentResponse>(JsonSerializerOptions, cancellationToken);
			return result?.Id ?? throw new InvalidOperationException("Expected non-null response from tool deployment upload");
		}

		#endregion

		#region Jobs
		/// <summary>
		/// Gets job information for given job ID. Fail response if jobID does not exist.
		/// </summary>
		/// <param name="id">Id of the job to get information for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<GetJobResponse> GetJobAsync(JobId id, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetJobResponse>(_httpClient, $"api/v1/jobs/{id}", cancellationToken);
		}

		/// <summary>
		/// Apply metadata tags to jobs and steps
		/// </summary>
		/// <param name="id"></param>
		/// <param name="jobMetaData"></param>
		/// <param name="stepMetaData"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public Task PutJobMetadataAsync(JobId id, IEnumerable<string>? jobMetaData = null, Dictionary<string, List<string>>? stepMetaData = null, CancellationToken cancellationToken = default)
		{
			return PutAsync(_httpClient, $"api/v1/jobs/{id}/metadata", new PutJobMetadataRequest() { JobMetaData = jobMetaData?.ToList(), StepMetaData = stepMetaData }, cancellationToken);
		}

		/// <summary>
		/// Get a list of job information for given stream ID and template IDs. Returns maximum count results or fewer.
		/// </summary>
		/// <param name="streamId">Stream ID to filter jobs</param>
		/// <param name="templateIds">List of job template IDs to filter by</param>
		/// <param name="filters">A list of filter fields to return for each Job in the results. Defaults to id, name, change, preflightChange, createTime</param>
		/// <param name="minChange">Minimum commit number</param>
		/// <param name="maxChange">Maximum commit number</param>
		/// <param name="minCreateTime">Minimum job creation time</param>
		/// <param name="maxCreateTime">Maximum job creation time</param>
		/// <param name="includePreflight">Whether to include preflight jobs</param>		
		/// <param name="count">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>GetJobResponse List</returns>
		public Task<List<GetJobResponse>> GetJobsAsync(
			StreamId streamId,
			IEnumerable<TemplateId>? templateIds = null,
			List<string>? filters = null,
			CommitId? minChange = null,
			CommitId? maxChange = null,
			DateTimeOffset? minCreateTime = null,
			DateTimeOffset? maxCreateTime = null,
			bool includePreflight = false,
			int count = 100,
			CancellationToken cancellationToken = default)
		{
			StringBuilder queryBuilder = new($"/api/v1/jobs?streamId={streamId}&count={count}&includePreflight={(includePreflight ? "true" : "false")}");

			if (templateIds != null)
			{
				foreach (TemplateId templateId in templateIds)
				{
					queryBuilder.Append($"&t={templateId}");
				}
			}

			if (minChange != null)
			{
				queryBuilder.Append($"&minChange={minChange}");
			}

			if (maxChange != null)
			{
				queryBuilder.Append($"&maxChange={maxChange}");
			}

			if (minCreateTime != null)
			{
				queryBuilder.Append($"&minCreateTime={Uri.EscapeDataString(minCreateTime.Value.ToString("o"))}");
			}

			if (maxCreateTime != null)
			{
				queryBuilder.Append($"&maxCreateTime={Uri.EscapeDataString(maxCreateTime.Value.ToString("o"))}");
			}

			filters ??= ["id", "name", "change", "preflightChange", "createTime"];
			if (filters.Count > 0)
			{
				queryBuilder.Append($"&filter={Uri.EscapeDataString(String.Join(",", filters))}");
			}

			return GetAsync<List<GetJobResponse>>(_httpClient, queryBuilder.ToString(), cancellationToken);
		}

		#endregion

		#region Log
		/// <summary>
		/// Get the given log file 
		/// </summary>
		/// <param name="logId">Id of the log file to retrieve</param>
		/// <param name="searchText">Text to search for in the log</param>
		/// <param name="count">Number of lines to return (default 5)</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<SearchLogResponse> GetSearchLogAsync(LogId logId, string searchText, int count = 5, CancellationToken cancellationToken = default)
		{
			return GetAsync<SearchLogResponse>(_httpClient, $"/api/v1/logs/{logId}/search?Text={Uri.EscapeDataString(searchText)}&count={count}", cancellationToken);
		}

		/// <summary>
		/// Get the requested number of lines from given logFileId, starting at index
		/// </summary>
		/// <param name="logId">Id of log file to retrieve lines from</param>
		/// <param name="startIndex">Start index of lines to retrieve</param>
		/// <param name="count">Number of lines to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<LogLinesResponse> GetLogLinesAsync(LogId logId, int startIndex, int count, CancellationToken cancellationToken = default)
		{
			return GetAsync<LogLinesResponse>(_httpClient, $"/api/v1/logs/{logId}/lines?index={startIndex}&count={count}", cancellationToken);
		}

		/// <summary>
		/// Get events for multiple log files in a single request.
		/// Set <see cref="GetBatchLogEventsRequest.Count"/> to 0 to retrieve only the per-log totals
		/// without downloading any event payloads.
		/// </summary>
		/// <param name="request">Log IDs, optional severity filter, and optional per-log event count limit</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<GetBatchLogEventsResponse> GetBatchLogEventsAsync(GetBatchLogEventsRequest request, CancellationToken cancellationToken = default)
		{
			return PostAsync<GetBatchLogEventsResponse, GetBatchLogEventsRequest>(_httpClient, "/api/v1/logs/events", request, cancellationToken);
		}

		#endregion

		#region Graph

		/// <summary>
		/// Get graph of the given job
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="cancellationToken"></param>
		/// <returns>Contains buildgraph information for the job</returns>
		public Task<GetGraphResponse> GetGraphAsync(JobId jobId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetGraphResponse>(_httpClient, $"/api/v1/jobs/{jobId}/graph", cancellationToken);
		}

		#endregion

		#region UGS
		/// <summary>
		/// 
		/// </summary>
		/// <param name="streamId"></param>
		/// <param name="commitId"></param>
		/// <param name="projectId"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public Task<GetUgsMetadataListResponse> GetUgsMetadataAsync(StreamId streamId, CommitId commitId, ProjectId projectId, CancellationToken cancellationToken = default)
		{
			Regex streamRe = new(@"(.*?)\-(.*)");
			Match streamMatch = streamRe.Match(streamId.ToString());
			if (streamMatch.Success)
			{
				string streamName = $"//{streamMatch.Groups[1]}/{streamMatch.Groups[2]}";
				return GetAsync<GetUgsMetadataListResponse>(_httpClient, $"/ugs/api/metadata?stream={streamName}&change={commitId.GetPerforceChange()}&project={projectId}", cancellationToken);
			}
			else
			{
				throw new ArgumentException($"StreamId '{streamId} is invalid'");
			}
		}
		#endregion

		#region Devices

		/// <summary>
		/// Retrieves information about all devices
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Collection of GetDeviceResponse containing information about all devices</returns>
		public Task<List<GetDeviceResponse>> GetDevicesAsync(CancellationToken cancellationToken = default)
		{
			return GetAsync<List<GetDeviceResponse>>(_httpClient, $"/api/v2/devices", cancellationToken);
		}

		/// <summary>
		/// Retrieves information about the specified device
		/// </summary>
		/// <param name="deviceId">Id of the device to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>GetDeviceResponse containing information about the device</returns>
		public Task<GetDeviceResponse> GetDeviceAsync(string deviceId, CancellationToken cancellationToken = default)
		{
			return GetAsync<GetDeviceResponse>(_httpClient, $"/api/v2/devices/{deviceId}", cancellationToken);
		}

		/// <summary>
		/// Updates an individual device with the requested fields
		/// </summary>
		/// <param name="deviceId">Id of the device to update</param>
		/// <param name="request">Request object containing the fields to update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The response message from the http request</returns>
		public Task<HttpResponseMessage> PutDeviceUpdateAsync(string deviceId, UpdateDeviceRequest request, CancellationToken cancellationToken = default)
		{
			return PutAsync(_httpClient, $"/api/v2/devices/{deviceId}", request, cancellationToken);
		}
		#endregion
	}
}
