// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Net.Http;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Http;
using Microsoft.Extensions.Logging;
using Polly;
using Polly.Extensions.Http;
using Polly.Retry;
using Polly.Timeout;

namespace EpicGames.Horde
{
	/// <summary>
	/// Concrete implementation of <see cref="IHordeHttpMessageHandler"/> which manages the lifetime of the <see cref="HttpMessageHandler"/> instance.
	/// </summary>
	class HordeHttpMessageHandler : IHordeHttpMessageHandler, IDisposable
	{
		readonly HttpMessageHandler _instance;

		/// <inheritdoc/>
		public HttpMessageHandler Instance => _instance;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger"></param>
		public HordeHttpMessageHandler(ILogger<HordeHttpMessageHandler> logger)
		{
			_instance = HttpMessageHandlers.CreateDefaultHandler();
			_instance = new PolicyHttpMessageHandler(request => CreateDefaultTimeoutRetryPolicy(request, logger)) { InnerHandler = _instance };
			_instance = new PolicyHttpMessageHandler(request => CreateHordeTransientErrorPolicy(request, logger)) { InnerHandler = _instance };
		}

		/// <inheritdoc/>
		public void Dispose()
			=> _instance.Dispose();

		/// <summary>
		/// Create a default timeout retry policy
		/// </summary>
		static IAsyncPolicy<HttpResponseMessage> CreateDefaultTimeoutRetryPolicy(HttpRequestMessage request, ILogger logger)
		{
			// Wait 30 seconds for operations to timeout
			Task OnTimeoutAsync(Context context, TimeSpan timespan, Task timeoutTask)
			{
				logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "{Method} {Url} timed out after {Time}s.", request.Method, request.RedactedRequestUri(), (int)timespan.TotalSeconds);
				return Task.CompletedTask;
			}

			AsyncTimeoutPolicy<HttpResponseMessage> timeoutPolicy = Policy.TimeoutAsync<HttpResponseMessage>(60, OnTimeoutAsync);

			// Retry twice after a timeout
			void OnRetry(Exception ex, TimeSpan timespan)
			{
				logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, ex, "{Method} {Url} retrying after {Time}s.", request.Method, request.RedactedRequestUri(), timespan.TotalSeconds);
			}

			TimeSpan[] retryTimes = [TimeSpan.FromSeconds(5.0), TimeSpan.FromSeconds(10.0), TimeSpan.FromSeconds(15.0)];
			AsyncRetryPolicy retryPolicy = Policy.Handle<TimeoutRejectedException>().WaitAndRetryAsync(retryTimes, OnRetry);
			return retryPolicy.WrapAsync(timeoutPolicy);
		}

		/// <summary>
		/// Create an error and timeout retry policy taking into account Horde's use of 503 errors
		/// </summary>
		static IAsyncPolicy<HttpResponseMessage> CreateHordeTransientErrorPolicy(HttpRequestMessage request, ILogger logger)
		{
			Task OnTimeoutAsync(DelegateResult<HttpResponseMessage> outcome, TimeSpan timespan, int retryAttempt, Context context)
			{
				logger.LogInformation(KnownLogEvents.Systemic_Horde_Http, "{Method} {Url} failed ({Result}). Delaying for {DelayMs}ms (attempt #{RetryNum}).", request.Method, request.RedactedRequestUri(), outcome.Result?.StatusCode, timespan.TotalMilliseconds, retryAttempt);
				return Task.CompletedTask;
			}

			TimeSpan[] retryTimes = [TimeSpan.FromSeconds(1.0), TimeSpan.FromSeconds(5.0), TimeSpan.FromSeconds(10.0), TimeSpan.FromSeconds(30.0), TimeSpan.FromSeconds(30.0)];

			// Policy for transient errors is the same as HttpPolicyExtensions.HandleTransientHttpError(), but excludes HttpStatusCode.ServiceUnavailable (which is used as a response
			// when allocating compute resources when none are available). This pathway is handled explicitly on the application side.
			return Policy<HttpResponseMessage>
				.Handle<HttpRequestException>()
				.OrResult(x => (x.StatusCode >= HttpStatusCode.InternalServerError && x.StatusCode != HttpStatusCode.ServiceUnavailable) || x.StatusCode == HttpStatusCode.RequestTimeout)
				.WaitAndRetryAsync(retryTimes, OnTimeoutAsync);
		}

		/// <summary>
		/// Create a default retry policy 
		/// </summary>
		public static IAsyncPolicy<HttpResponseMessage> CreateDefaultTransientErrorPolicy()
		{
			return HttpPolicyExtensions
				.HandleTransientHttpError()
				.Or<TaskCanceledException>(ex =>
					ex.InnerException is TimeoutException &&
					ex.InnerException.Message.Contains("ConnectTimeout", StringComparison.Ordinal))
				.OrResult(response => response.StatusCode == HttpStatusCode.ServiceUnavailable)
				.WaitAndRetryAsync(6, retryAttempt => TimeSpan.FromSeconds(Math.Pow(2, retryAttempt)));
		}
	}
}
