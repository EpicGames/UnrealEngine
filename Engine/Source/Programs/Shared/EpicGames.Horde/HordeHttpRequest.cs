// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Net.Http.Json;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs;

#pragma warning disable CA2234 // Use Uri instead of string

namespace EpicGames.Horde
{
	/// <summary>
	/// Static helper methods for implementing Horde HTTP requests with standard semantics
	/// </summary>
	public static class HordeHttpRequest
	{
		static readonly JsonSerializerOptions s_jsonSerializerOptions = CreateJsonSerializerOptions();
		internal static JsonSerializerOptions JsonSerializerOptions => s_jsonSerializerOptions;

		/// <summary>
		/// Create the shared instance of JSON options for HordeHttpClient instances
		/// </summary>
		static JsonSerializerOptions CreateJsonSerializerOptions()
		{
			JsonSerializerOptions options = new();
			ConfigureJsonSerializer(options);
			return options;
		}

		/// <summary>
		/// Configures a JSON serializer to read Horde responses
		/// </summary>
		/// <param name="options">options for the serializer</param>
		public static void ConfigureJsonSerializer(JsonSerializerOptions options)
		{
			options.AllowTrailingCommas = true;
			options.ReadCommentHandling = JsonCommentHandling.Skip;
			options.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
			options.PropertyNamingPolicy = JsonNamingPolicy.CamelCase;
			options.PropertyNameCaseInsensitive = true;
			options.Converters.Add(new JsonStringEnumConverter());
			options.Converters.Add(new StringIdJsonConverterFactory());
			options.Converters.Add(new BinaryIdJsonConverterFactory());
			options.Converters.Add(new SubResourceIdJsonConverterFactory());
		}

		/// <summary>
		/// Deletes a resource from an HTTP endpoint
		/// </summary>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to retrieve</param>
		/// <param name="cancellationToken">Cancels the request</param>
		public static async Task DeleteAsync(HttpClient httpClient, string relativePath, CancellationToken cancellationToken = default)
		{
			using HttpResponseMessage response = await httpClient.DeleteAsync(relativePath, cancellationToken);
			response.EnsureSuccessStatusCode();
		}

		/// <summary>
		/// Gets a resource from an HTTP endpoint and parses it as a JSON object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to retrieve</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>New instance of the object</returns>
		public static async Task<TResponse> GetAsync<TResponse>(HttpClient httpClient, string relativePath, CancellationToken cancellationToken = default)
		{
			TResponse? response = await httpClient.GetFromJsonAsync<TResponse>(relativePath, s_jsonSerializerOptions, cancellationToken);
			return response ?? throw new InvalidCastException($"Expected non-null response from GET to {relativePath}");
		}

		/// <summary>
		/// Posts an object to an HTTP endpoint as a JSON object, and parses the response object
		/// </summary>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to retrieve</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>The response parsed into the requested type</returns>
		internal static async Task<HttpResponseMessage> PostAsync<TRequest>(HttpClient httpClient, string relativePath, TRequest request, CancellationToken cancellationToken = default)
		{
			return await httpClient.PostAsJsonAsync<TRequest>(relativePath, request, s_jsonSerializerOptions, cancellationToken);
		}

		/// <summary>
		/// Posts an object to an HTTP endpoint as a JSON object, and parses the response object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to retrieve</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>The response parsed into the requested type</returns>
		public static async Task<TResponse> PostAsync<TResponse, TRequest>(HttpClient httpClient, string relativePath, TRequest request, CancellationToken cancellationToken = default)
		{
			using (HttpResponseMessage response = await PostAsync<TRequest>(httpClient, relativePath, request, cancellationToken))
			{
				if (!response.IsSuccessStatusCode)
				{
					string body = await response.Content.ReadAsStringAsync(cancellationToken);
					throw new HttpRequestException($"{(int)response.StatusCode} ({response.StatusCode}) posting to {new Uri(httpClient.BaseAddress!, relativePath)}: {body}", null, response.StatusCode);
				}

				TResponse? responseValue = await response.Content.ReadFromJsonAsync<TResponse>(s_jsonSerializerOptions, cancellationToken);
				return responseValue ?? throw new InvalidCastException($"Expected non-null response from POST to {relativePath}");
			}
		}

		/// <summary>
		/// Puts an object to an HTTP endpoint as a JSON object
		/// </summary>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to write to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>Response message</returns>
		public static async Task<HttpResponseMessage> PutAsync<TRequest>(HttpClient httpClient, string relativePath, TRequest request, CancellationToken cancellationToken)
		{
			return await httpClient.PutAsJsonAsync<TRequest>(relativePath, request, s_jsonSerializerOptions, cancellationToken);
		}

		/// <summary>
		/// Puts an object to an HTTP endpoint as a JSON object
		/// </summary>
		/// <typeparam name="TResponse">The object type to return</typeparam>
		/// <typeparam name="TRequest">The object type to post</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to write to</param>
		/// <param name="request">The object to post</param>
		/// <param name="cancellationToken">Cancels the request</param>
		/// <returns>Response message</returns>
		public static async Task<TResponse> PutAsync<TResponse, TRequest>(HttpClient httpClient, string relativePath, TRequest request, CancellationToken cancellationToken)
		{
			using (HttpResponseMessage response = await httpClient.PutAsJsonAsync<TRequest>(relativePath, request, s_jsonSerializerOptions, cancellationToken))
			{
				if (!response.IsSuccessStatusCode)
				{
					string body = await response.Content.ReadAsStringAsync(cancellationToken);
					throw new HttpRequestException($"{response.StatusCode} put to {new Uri(httpClient.BaseAddress!, relativePath)}: {body}", null, response.StatusCode);
				}

				TResponse? responseValue = await response.Content.ReadFromJsonAsync<TResponse>(s_jsonSerializerOptions, cancellationToken);
				return responseValue ?? throw new InvalidCastException($"Expected non-null response from PUT to {relativePath}");
			}
		}

		/// <summary>
		/// Patches a resource at an HTTP endpoint with a JSON object
		/// </summary>
		/// <typeparam name="TRequest">The object type to send</typeparam>
		/// <param name="httpClient">Http client instance</param>
		/// <param name="relativePath">The url to patch</param>
		/// <param name="request">The object to send</param>
		/// <param name="cancellationToken">Cancels the request</param>
		public static async Task PatchAsync<TRequest>(HttpClient httpClient, string relativePath, TRequest request, CancellationToken cancellationToken = default)
		{
			using HttpResponseMessage response = await httpClient.PatchAsJsonAsync(relativePath, request, s_jsonSerializerOptions, cancellationToken);
			response.EnsureSuccessStatusCode();
		}
	}
}
