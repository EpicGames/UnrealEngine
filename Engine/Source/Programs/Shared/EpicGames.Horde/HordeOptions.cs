// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;
using EpicGames.Horde.Storage.Bundles;
using Microsoft.Win32;

namespace EpicGames.Horde
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Options for configuring the Horde connection
	/// </summary>
	public class HordeOptions
	{
		/// <summary>
		/// Address of the Horde server
		/// </summary>
		public Uri? ServerUrl { get; set; }

		/// <summary>
		/// Access token to use for connecting to the server
		/// </summary>
		public string? AccessToken { get; set; }

		/// <summary>
		/// Whether to allow opening a browser window to prompt for authentication
		/// </summary>
		public bool AllowAuthPrompt { get; set; } = true;

		/// <summary>
		/// Options for creating new bundles
		/// </summary>
		public BundleOptions Bundle { get; } = new BundleOptions();

		/// <summary>
		/// Options for caching bundles 
		/// </summary>
		public BundleCacheOptions BundleCache { get; } = new BundleCacheOptions();

		/// <summary>
		/// Options for the storage backend cache
		/// </summary>
		public StorageBackendCacheOptions BackendCache { get; } = new StorageBackendCacheOptions();

		/// <summary>
		/// Gets the configured server URL, or the default value
		/// </summary>
		public Uri? GetServerUrlOrDefault()
			=> ServerUrl ?? GetServerUrlFromEnvironment() ?? GetDefaultServerUrl();

		/// <summary>
		/// Reads the server URL from the environment
		/// </summary>
		public static Uri? GetServerUrlFromEnvironment()
		{
			string? hordeUrlEnvVar = Environment.GetEnvironmentVariable(HordeHttpClient.HordeUrlEnvVarName);
			if (String.IsNullOrEmpty(hordeUrlEnvVar))
			{
				return null;
			}
			return new Uri(hordeUrlEnvVar);
		}

		/// <summary>
		/// Gets the default server URL for the current user
		/// </summary>
		/// <returns>Default URL</returns>
		public static Uri? GetDefaultServerUrl()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				string? url =
					(Registry.GetValue(@"HKEY_CURRENT_USER\SOFTWARE\Epic Games\Horde", "Url", null) as string) ??
					(Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Epic Games\Horde", "Url", null) as string);

				if (!String.IsNullOrEmpty(url))
				{
					try
					{
						return new Uri(url);
					}
					catch (UriFormatException)
					{
					}
				}
			}
			else
			{
				FileReference? configFile = GetConfigFile();
				if (configFile != null && FileReference.Exists(configFile))
				{
					byte[] data = FileReference.ReadAllBytes(configFile);

					JsonObject? root = JsonNode.Parse(data, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true }) as JsonObject;
					root ??= [];

					JsonNode? value;
					if (root.TryGetPropertyValue("server", out value))
					{
						string? stringValue = (string?)value;
						if (stringValue != null)
						{
							try
							{
								return new Uri(stringValue);
							}
							catch (UriFormatException)
							{
							}
						}
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Sets the default server url for the current user
		/// </summary>
		/// <param name="serverUrl">Horde server URL to use</param>
		public static void SetDefaultServerUrl(Uri serverUrl)
		{
			if (!serverUrl.OriginalString.EndsWith('/'))
			{
				serverUrl = new Uri(serverUrl.OriginalString + "/");
			}

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				string? newServerUrl = serverUrl.ToString();
				string? defaultServerUrl = Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Epic Games\Horde", "Url", null) as string;

				if (String.Equals(newServerUrl, defaultServerUrl, StringComparison.Ordinal))
				{
					using RegistryKey key = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Epic Games\\Horde");
					DeleteRegistryKey(key, "Url");
				}
				else
				{
					Registry.SetValue(@"HKEY_CURRENT_USER\SOFTWARE\Epic Games\Horde", "Url", serverUrl.ToString());
				}
			}
			else
			{
				FileReference? configFile = GetConfigFile();
				if (configFile != null)
				{
					JsonObject? root = null;
					if (FileReference.Exists(configFile))
					{
						byte[] data = FileReference.ReadAllBytes(configFile);
						root = JsonNode.Parse(data, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true }) as JsonObject;
					}

					root ??= [];
					root["server"] = serverUrl.ToString();

					using (FileStream stream = FileReference.Open(configFile, FileMode.Create, FileAccess.ReadWrite, FileShare.Read))
					{
						using Utf8JsonWriter writer = new(stream, new JsonWriterOptions { Indented = true });
						root.WriteTo(writer);
					}
				}
			}
		}

		static FileReference? GetConfigFile()
		{
			DirectoryReference? userFolder = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile);
			userFolder ??= DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
			userFolder ??= DirectoryReference.GetCurrentDirectory();
			return FileReference.Combine(userFolder, ".horde.json");
		}

		[SupportedOSPlatform("windows")]
		static void DeleteRegistryKey(RegistryKey key, string name)
		{
			string[] valueNames = key.GetValueNames();
			if (valueNames.Any(x => String.Equals(x, name, StringComparison.OrdinalIgnoreCase)))
			{
				try
				{
					key.DeleteValue(name);
				}
				catch
				{
				}
			}
		}
	}

	/// <summary>
	/// Options for the storage backend cache
	/// </summary>
	public class StorageBackendCacheOptions
	{
		/// <summary>
		/// Directory to store cached data
		/// </summary>
		public string? CacheDir { get; set; }

		/// <summary>
		/// Maximum size of the cache, in bytes
		/// </summary>
		public long MaxSize { get; set; }
	}
}
