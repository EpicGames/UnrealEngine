// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Tools;
using Grpc.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde
{
	/// <summary>
	/// Interface for Horde functionality.
	/// </summary>
	public interface IHordeClient
	{
		/// <summary>
		/// URL of the horde server
		/// </summary>
		Uri ServerUrl { get; }

		/// <summary>
		/// Accessor for the artifact collection
		/// </summary>
		IArtifactCollection Artifacts { get; }

		/// <summary>
		/// Accessor for the compute client
		/// </summary>
		IComputeClient Compute { get; }

		/// <summary>
		/// Accessor for the project collection
		/// </summary>
		IProjectCollection Projects { get; }

		/// <summary>
		/// Accessor for the secret collection
		/// </summary>
		ISecretCollection Secrets { get; }

		/// <summary>
		/// Accessor for the tools collection
		/// </summary>
		IToolCollection Tools { get; }

		/// <summary>
		/// Event triggered whenever the access token state changes
		/// </summary>
		event Action? OnAccessTokenStateChanged;

		/// <summary>
		/// Connect to the Horde server
		/// </summary>
		/// <param name="interactive">Whether to allow prompting for credentials</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the connection succeded</returns>
		Task<bool> LoginAsync(bool interactive, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the current connection state
		/// </summary>
		bool HasValidAccessToken();

		/// <summary>
		/// Gets an access token for the server
		/// </summary>
		Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a gRPC client interface
		/// </summary>
		Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>;

		/// <summary>
		/// Creates a Horde HTTP client 
		/// </summary>
		HordeHttpClient CreateHttpClient();

		/// <summary>
		/// Creates a storage namespace for the given base path
		/// </summary>
		IStorageNamespace GetStorageNamespace(string relativePath, string? accessToken = null);

		/// <summary>
		/// Creates a logger device that writes data to the server
		/// </summary>
		IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information);
	}

	/// <summary>
	/// Interface for a Horde client with a known lifetime
	/// </summary>
	public interface IHordeClientWithLifetime : IHordeClient, IAsyncDisposable
	{
	}

	/// <summary>
	/// Extension methods for <see cref="IHordeClient"/>
	/// </summary>
	public static class HordeClientExtensions
	{
		/// <summary>
		/// Creates a storage namespace for a particular id
		/// </summary>
		public static IStorageNamespace GetStorageNamespace(this IHordeClient hordeClient, NamespaceId namespaceId, string? accessToken = null)
			=> hordeClient.GetStorageNamespace($"api/v1/storage/{namespaceId}", accessToken);

		/// <summary>
		/// Creates a storage namespace for a particular artifact
		/// </summary>
		public static IStorageNamespace GetStorageNamespace(this IHordeClient hordeClient, ArtifactId artifactId)
			=> hordeClient.GetStorageNamespace($"api/v2/artifacts/{artifactId}");

		/// <summary>
		/// Creates a storage namespace for a particular log
		/// </summary>
		public static IStorageNamespace GetStorageNamespace(this IHordeClient hordeClient, LogId logId)
			=> hordeClient.GetStorageNamespace($"api/v1/logs/{logId}");

		/// <summary>
		/// Creates a storage namespace for a particular tool
		/// </summary>
		public static IStorageNamespace GetStorageNamespace(this IHordeClient hordeClient, ToolId toolId)
			=> hordeClient.GetStorageNamespace($"api/v1/tools/{toolId}");

		/// <summary>
		/// Reads a blob storage ref from a path
		/// </summary>
		public static async Task<IBlobRef?> TryReadRefAsync(this IHordeClient hordeClient, string path, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			ReadRefResponse? response = await hordeClient.CreateHttpClient().TryReadRefAsync(path, cacheTime, cancellationToken);
			if (response == null)
			{
				return null;
			}

			IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(response.BasePath);
			return storageNamespace.CreateBlobRef(response.Target);
		}

		/// <summary>
		/// Reads a typed blob storage ref from a path
		/// </summary>
		public static async Task<IBlobRef<T>?> TryReadRefAsync<T>(this IHordeClient hordeClient, string path, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			IBlobRef? blobRef = await hordeClient.TryReadRefAsync(path, cacheTime, cancellationToken);
			return blobRef?.ForType<T>();
		}
	}
}
