// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http;

namespace EpicGames.Horde
{
	/// <summary>
	/// Provides access to a <see cref="HttpMessageHandler"/> instances for Horde with a default resiliance pipeline.
	/// </summary>
	public interface IHordeHttpMessageHandler
	{
		/// <summary>
		/// Instance of the http message handler
		/// </summary>
		HttpMessageHandler Instance { get; }
	}
}
