// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular BuildGraph script error
	/// </summary>
	[IssueHandler]
	public class BuildGraphIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.AutomationTool_BuildGraphScript;
		}

		static bool IsMaskedEventId(EventId id) => id == KnownLogEvents.ExitCode || id == KnownLogEvents.Systemic_Xge_BuildFailed;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId.HasValue)
			{
				EventId eventId = issueEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					IssueEventGroup issue = new("BuildGraph", "BuildGraph {Severity} in {Files}", IssueChangeFilter.All);
					issue.Events.Add(issueEvent);
					issue.Keys.AddSourceFiles(issueEvent);
					_issues.Add(issue);
					issueEvent.AuditLogger?.LogDebug("BuildGraph issue added for event: '{Event}'", issueEvent.Render());

					return true;
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
