// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler]
	public class CompileIssueHandler : IssueHandler
	{
		/// <summary>
		/// Annotation describing the compile type
		/// </summary>
		const string CompileTypeAnnotation = "CompileType";

		/// <summary>
		/// Annotation specifying a group for compile issues from this node
		/// </summary>
		const string CompileGroupAnnotation = "CompileGroup";

		readonly IssueHandlerContext _context;
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Constructor
		/// </summary>
		public CompileIssueHandler(IssueHandlerContext context)
		{
			_context = context;
		}

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.Compiler || eventId == KnownLogEvents.UHT || eventId == KnownLogEvents.AutomationTool_SourceFileLine || eventId == KnownLogEvents.MSBuild;
		}

		static bool IsMaskedEventId(EventId id) => id == KnownLogEvents.ExitCode || id == KnownLogEvents.Systemic_Xge_BuildFailed || id == KnownLogEvents.Compiler_Summary;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId.HasValue)
			{
				EventId eventId = issueEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					string compileType = "Compile";
					if (_context.NodeAnnotations.TryGetValue(CompileTypeAnnotation, out string? type))
					{
						compileType = type;
					}

					string fingerprintType = "Compile";
					if (_context.NodeAnnotations.TryGetValue(CompileGroupAnnotation, out string? group))
					{
						fingerprintType = $"{fingerprintType}:{group}";
					}

					IssueEventGroup issue = new(fingerprintType, "{Meta:CompileType} {Severity} in {Files}", IssueChangeFilter.Code);
					issue.Events.Add(issueEvent);
					issue.Keys.AddSourceFiles(issueEvent);
					issue.Metadata.Add(CompileTypeAnnotation, compileType);
					_issues.Add(issue);
					issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", fingerprintType, issueEvent.Render());

					return true;
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
