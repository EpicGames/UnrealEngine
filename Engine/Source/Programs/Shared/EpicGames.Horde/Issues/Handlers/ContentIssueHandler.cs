// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler]
	public class ContentIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		static bool IsMatchingEventId(EventId eventId) => eventId == KnownLogEvents.Engine_AssetLog;
		static bool IsMaskedEventId(EventId eventId) => eventId == KnownLogEvents.ExitCode;

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId != null)
			{
				EventId eventId = issueEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					IssueEventGroup issue = new("Content", "{Severity} in {Files}", IssueChangeFilter.Content);
					issue.Events.Add(issueEvent);
					issue.Keys.AddAssets(issueEvent);
					_issues.Add(issue);
					issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());

					return true;
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
