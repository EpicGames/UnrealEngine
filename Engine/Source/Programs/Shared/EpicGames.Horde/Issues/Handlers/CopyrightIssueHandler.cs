// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler]
	public class CopyrightIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId == KnownLogEvents.AutomationTool_MissingCopyright)
			{
				IssueEventGroup issue = new("Copyright", "Missing copyright notice in {Files}", IssueChangeFilter.Code);
				issue.Events.Add(issueEvent);
				issue.Keys.AddSourceFiles(issueEvent);
				_issues.Add(issue);
				issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());

				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
