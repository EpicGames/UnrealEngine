// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Default handler for log events not matched by any other handler
	/// </summary>
	[IssueHandler]
	public class DefaultIssueHandler : IssueHandler
	{
		readonly IssueHandlerContext _context;
		IssueEventGroup? _issue;

		/// <inheritdoc/>
		public override int Priority => 0;

		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultIssueHandler(IssueHandlerContext context) => _context = context;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (_issue == null)
			{
				_issue = new IssueEventGroup("Default", "{Severity} in {Meta:Node}", IssueChangeFilter.All);
				_issue.Metadata.Add("Node", _context.NodeName);
				_issue.Keys.Add(IssueKey.FromStep(_context.StreamId, _context.TemplateId, _context.NodeName));
				issueEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", IssueKey.FromStep(_context.StreamId, _context.TemplateId, _context.NodeName), issueEvent.Render());
			}

			_issue.Events.Add(issueEvent);
			return true;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues()
		{
			if (_issue != null)
			{
				yield return _issue;
			}
		}
	}
}
