// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Handler for log events with issue fingerprints embedded in the structured log data itself
	/// </summary>
	[IssueHandler]
	public class ExternalIssueHandler : IssueHandler
	{
		static readonly JsonNodeOptions s_jsonNodeOptions = new() { PropertyNameCaseInsensitive = true };
		static readonly JsonSerializerOptions? s_jsonSerializerOptions = new() { PropertyNameCaseInsensitive = true };

		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 20;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			foreach (JsonLogEvent line in issueEvent.Lines)
			{
				IssueFingerprint? issueData = ParseIssueFingerprint(line);
				if (issueData != null)
				{
					IssueEventGroup group = new(issueData.Type, issueData.SummaryTemplate, issueData.ChangeFilter);
					if (issueData.Keys != null)
					{
						group.Keys.UnionWith(issueData.Keys);
					}
					if (issueData.Metadata != null)
					{
						group.Metadata.UnionWith(issueData.Metadata);
					}
					group.Events.Add(issueEvent);

					_issues.Add(group);
					issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", group.Type, issueEvent.Render());
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues()
			=> _issues;

		static IssueFingerprint? ParseIssueFingerprint(JsonLogEvent line)
		{
			JsonNode? node = JsonNode.Parse(line.Data.Span, s_jsonNodeOptions);
			if (node != null)
			{
				JsonObject? properties = node["properties"] as JsonObject;
				if (properties != null)
				{
					JsonObject? issue = properties["$issue"] as JsonObject;
					if (issue != null)
					{
						IssueFingerprint? fingerprint = JsonSerializer.Deserialize<IssueFingerprint>(issue, s_jsonSerializerOptions);
						if (fingerprint != null)
						{
							return fingerprint;
						}
					}
				}
			}
			return null;
		}
	}
}
