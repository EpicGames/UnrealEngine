// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular Gauntlet error
	/// </summary>
	[IssueHandler]
	public class GauntletIssueHandler : IssueHandler
	{
		/// <summary>
		/// Prefix for framework keys
		/// </summary>
		const string FrameworkPrefix = "test framework";

		/// <summary>
		/// Prefix for test keys
		/// </summary>
		const string TestPrefix = "test";

		/// <summary>
		/// Prefix for device keys
		/// </summary>
		const string DevicePrefix = "device";

		/// <summary>
		/// Prefix for build drop keys
		/// </summary>
		const string BuildDropPrefix = "build drop";

		/// <summary>
		/// Prefix for fatal failure keys
		/// </summary>
		const string FatalPrefix = "fatal";

		/// <summary>
		/// Callstack log type property
		/// </summary>
		const string CallstackLogType = "Callstack";

		/// <summary>
		/// Summary log type property
		/// </summary>
		const string SummaryLogType = "Summary";

		/// <summary>
		/// Max Number of lines to consider to hash
		/// </summary>
		const int MaxLines = 6;

		/// <summary>
		/// Max Message Length to hash
		/// </summary>
		const int MaxMessageLength = 2000;

		/// <summary>
		/// Whether or not a severe event was reported
		/// </summary>
		bool _wasSevereEventReported = false;

		/// <summary>
		/// Whether or not a test event was reported
		/// </summary>
		bool _wasTestEventReported = false;

		/// <summary>
		/// Whether or not at least one error was reported
		/// </summary>
		bool _wasErrorEventReported = false;

		readonly IssueHandlerContext _context;
		readonly List<IssueEventGroup> _issues = [];

		/// <summary>
		///  Known Gauntlet events
		/// </summary>
		static readonly Dictionary<EventId, string> s_knownGauntletEvents = new()
		{
			{ KnownLogEvents.Gauntlet, FrameworkPrefix},
			{ KnownLogEvents.Gauntlet_TestEvent, TestPrefix},
			{ KnownLogEvents.Gauntlet_DeviceEvent, DevicePrefix},
			{ KnownLogEvents.Gauntlet_UnrealEngineTestEvent, TestPrefix},
			{ KnownLogEvents.Gauntlet_BuildDropEvent, BuildDropPrefix},
			{ KnownLogEvents.Gauntlet_FatalEvent, FatalPrefix}
		};

		/// <summary>
		///  Known Gauntlet events associated with the highest severity
		/// </summary>
		static readonly HashSet<EventId> s_knownSevereGauntletEvents =
		[
			KnownLogEvents.Gauntlet_BuildDropEvent,
			KnownLogEvents.Gauntlet_FatalEvent
		];

		/// <summary>
		///  Known Gauntlet events associated with test context
		/// </summary>
		static readonly HashSet<EventId> s_knownTestGauntletEvents =
		[
			KnownLogEvents.Gauntlet_TestEvent,
			KnownLogEvents.Gauntlet_UnrealEngineTestEvent
		];

		/// <summary>
		///   Gauntlet events to skip issue generation
		/// </summary>
		static readonly HashSet<EventId> s_skipGauntletEvents =
		[
			KnownLogEvents.Gauntlet_DeviceEvent
		];

		/// <summary>
		///  Known summaries that are not specific enough
		/// </summary>
		static readonly string[] s_unspecificSummaries = [
			"Unhandled Exception: EXCEPTION_ACCESS_VIOLATION reading address"
		];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Constructor
		/// </summary>
		public GauntletIssueHandler(IssueHandlerContext context) => _context = context;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return s_knownGauntletEvents.ContainsKey(eventId);
		}

		/// <summary>
		/// Return the prefix string associate with the event id
		/// </summary>
		/// <param name="eventId">The event id to get the information from</param>
		/// <returns>The corresponding prefix as a string</returns>
		public static string GetEventPrefix(EventId eventId)
		{
			return s_knownGauntletEvents[eventId];
		}

		/// <summary>
		/// Produce a hash from error message
		/// </summary>
		/// <param name="issueEvent">The issue event</param>
		/// <param name="keys">Receives a set of the keys</param>
		/// <param name="metadata">Receives a set of metadata</param>
		/// <param name="hasCallstack">Set true if a callstack property was found</param>
		private void GetHash(IssueEvent issueEvent, HashSet<IssueKey> keys, HashSet<IssueMetadata> metadata, bool hasCallstack)
		{
			if (TryGetHash(issueEvent, out Md5Hash hash))
			{
				string key = $"hash:{hash}";
				if (!hasCallstack)
				{
					// add job step salt if no Callstack property was found
					key += $":{_context.NodeName}";
				}
				keys.Add(key, IssueKeyType.None);
				issueEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", key, issueEvent.Render());
				metadata.Add("Hash", hash.ToString());
			}
			else
			{
				// Not enough information, make it an issue associated with only the job step
				keys.Add($"{_context.NodeName}", IssueKeyType.None);
				issueEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", $"{_context.NodeName}", issueEvent.Render());
			}
			metadata.Add("Node", _context.NodeName);
		}

		private static bool TryGetHash(IssueEvent issueEvent, out Md5Hash hash)
		{
			// Use only the summary if one is found instead of the full callstack
			string? summary = GetSummaryProperty(issueEvent);
			// Discard summaries that are too broad
			if (summary != null && s_unspecificSummaries.Any(k => summary.Contains(k, StringComparison.InvariantCultureIgnoreCase)))
			{
				summary = null;
			}
			string sanitized = summary ?? issueEvent.Render();
			sanitized = String.Join("\n", sanitized.Split("\n", MaxLines + 1).Take(MaxLines)); // Limit the number of lines to consider
			sanitized = sanitized.Length > MaxMessageLength ? sanitized.Substring(0, MaxMessageLength) : sanitized;
			sanitized = sanitized.Trim().ToUpperInvariant();
			sanitized = Regex.Replace(sanitized, @"(?:(?<![A-Z])|HTTPS*:)(?:[A-Z]:|[/\\])[\w/\\.:+?=%-]+", "{path}"); // Redact path like reference
			sanitized = Regex.Replace(sanitized, @"0X[0-9A-F]+", "H"); // Redact hex strings
			sanitized = Regex.Replace(sanitized, @"\d[\d.,:]*", "n"); // Redact numbers and timestamp like things

			if (sanitized.Length > 30)
			{
				hash = Md5Hash.Compute(Encoding.UTF8.GetBytes(sanitized));
				return true;
			}
			else
			{
				hash = Md5Hash.Zero;
				return false;
			}
		}

		private static bool EventHasCallstackProperty(IssueEvent issueEvent)
		{
			return issueEvent.Lines.Any(x => FindNestedPropertyOfType(x, CallstackLogType) != null);
		}

		private static string? GetSummaryProperty(IssueEvent issueEvent)
		{
			StringBuilder? summary = null;
			foreach (JsonLogEvent logEvent in issueEvent.Lines)
			{
				JsonProperty? property = FindNestedPropertyOfType(logEvent, SummaryLogType);
				if (property != null)
				{
					summary ??= new();
					JsonElement value = property.Value.Value;
					if (value.ValueKind == JsonValueKind.String
						// handle LogValue type
						|| (value.TryGetProperty(LogEventPropertyName.Text.Span, out value) && value.ValueKind == JsonValueKind.String))
					{
						summary.Append(value.GetString() + '\n');
						continue;
					}
					summary.Append(value.ToString() + '\n');
				}
				else if (summary != null)
				{
					// when property is null but not summary, we early exit since we expect property split to be contiguous
					return summary.ToString();
				}
			}
			return summary?.ToString();
		}

		private static JsonProperty? FindNestedPropertyOfType(JsonLogEvent logEvent, string searchType)
		{
			JsonElement line = JsonDocument.Parse(logEvent.Data).RootElement;
			JsonElement properties;
			if (line.TryGetProperty("properties", out properties) && properties.ValueKind == JsonValueKind.Object)
			{
				foreach (JsonProperty property in properties.EnumerateObject())
				{
					if (property.Name.StartsWith(searchType, StringComparison.OrdinalIgnoreCase))
					{
						// if name is longer, check if it is a split property pattern: {name}${index}
						if (property.Name.Length > searchType.Length && property.Name.Substring(searchType.Length, 1) != "$")
						{
							continue;
						}
						return property;
					}
				}
			}

			return null;
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId != null && IsMatchingEventId(issueEvent.EventId.Value))
			{
				if (s_skipGauntletEvents.Contains(issueEvent.EventId.Value))
				{
					// ignore Device event
					return true;
				}

				bool hasCallstack = EventHasCallstackProperty(issueEvent);

				if (issueEvent.Severity == LogLevel.Warning && !hasCallstack)
				{
					// Do not generate a flood of Build Health issues for smallset of known Gauntlet log event warnings
					// If we identify cases where we want these warnings, they should be defined as new log events, and force their warning in the issue handler, here
					// Though, we have been unable to come up with a single case in exploration					
					return true;
				}

				bool isSevereEvent = s_knownSevereGauntletEvents.Contains(issueEvent.EventId.Value);
				bool isTestEvent = !isSevereEvent && s_knownTestGauntletEvents.Contains(issueEvent.EventId.Value);
				bool isErrorEvent = issueEvent.Severity >= LogLevel.Error;
				if ((_wasSevereEventReported && !isSevereEvent)
					|| (_wasTestEventReported && !isTestEvent && !isSevereEvent)
					|| (_wasErrorEventReported && !isErrorEvent))
				{
					return true;
				}

				bool isEnsure = false;
				if (hasCallstack)
				{
					string rendered = issueEvent.Render();
					if (issueEvent.Render().Contains("Ensure condition failed", StringComparison.OrdinalIgnoreCase))
					{
						isEnsure = true;
					}
				}

				string callstack = isEnsure ? "ensure callstack" : "callstack";

				string summaryTemplate = hasCallstack ? "Automation {Meta:GauntletType} " + callstack + " {Severity} in {Meta:Node}" : "Automation {Meta:GauntletType} {Severity} in {Meta:Node}";

				string gauntletType = GetEventPrefix(issueEvent.EventId!.Value);
				IssueEventGroup issue = new($"Gauntlet:{gauntletType}", summaryTemplate, IssueChangeFilter.All);
				issue.Events.Add(issueEvent);
				GetHash(issueEvent, issue.Keys, issue.Metadata, hasCallstack);
				issue.Metadata.Add("GauntletType", gauntletType);
				if (hasCallstack)
				{
					string? hash = issue.Metadata.FindValues("Hash").FirstOrDefault();
					issue.Type = $"{issue.Type}:with-callstack:{hash}";
				}

				if (isErrorEvent && !_wasErrorEventReported)
				{
					// We've encountered an error event;
					// We can ignore other issues to prevent superfluous issues from being created
					_issues.RemoveAll((issue) => issue.Events.Any(x => x.Severity <= LogLevel.Warning));
					_wasErrorEventReported = true;
				}

				if (isSevereEvent)
				{
					// We've encountered a severe event where either the engine has crashed or a build was not found.
					// We can ignore other issues to prevent superfluous issues from being created
					_issues.RemoveAll((issue) => !issue.Events.Any(x => s_knownSevereGauntletEvents.Contains(x.EventId!.Value)));
					_wasSevereEventReported = true;
				}
				else if (isTestEvent && !_wasTestEventReported)
				{
					// We've encountered a test event;
					// We can ignore other issues to prevent superfluous issues from being created
					_issues.RemoveAll((issue) => !issue.Events.Any(x => s_knownTestGauntletEvents.Contains(x.EventId!.Value)));
					_wasTestEventReported = true;
				}

				_issues.Add(issue);

				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
