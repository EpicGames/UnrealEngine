// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler]
	public class HashedIssueHandler : IssueHandler
	{
		readonly IssueHandlerContext _context;
		readonly List<IssueEvent> _issueEvents = [];

		/// <summary>
		/// Known general events
		/// </summary>
		static readonly HashSet<EventId> s_knownGeneralEvents = [KnownLogEvents.Generic, KnownLogEvents.ExitCode, KnownLogEvents.Horde, KnownLogEvents.Horde_InvalidPreflight];

		/// <summary>
		/// Constructor
		/// </summary>
		public HashedIssueHandler(IssueHandlerContext context) => _context = context;

		/// <summary>
		/// Determines if the given event is general and should be salted to make it unique
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsGeneralEventId(EventId eventId)
		{
			return s_knownGeneralEvents.Contains(eventId) || (eventId.Id >= KnownLogEvents.Systemic.Id && eventId.Id <= KnownLogEvents.Systemic_Max.Id);
		}

		/// <inheritdoc/>
		public override int Priority => 1;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent logEvent)
		{
			_issueEvents.Add(logEvent);
			return true;
		}

		/// <summary>
		/// Get the channel for issue event
		/// </summary>
		/// <returns></returns>
		static string? GetLogChannel(IssueEvent logEvent)
		{
			try
			{

				string? channel = null;

				foreach (JsonLogEvent line in logEvent.Lines)
				{
					using JsonDocument document = JsonDocument.Parse(line.Data);

					string? channelType;
					string? channelText;
					if (document.RootElement.TryGetNestedProperty("properties._channel.$type", out channelType) && document.RootElement.TryGetNestedProperty("properties._channel.$text", out channelText))
					{
						if (channelType != "Channel")
						{
							continue;
						}

						channel = channelText;

						if (!String.IsNullOrEmpty(channel))
						{
							break;
						}
					}
				}

				return channel;

			}
			catch
			{
				return null;
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues()
		{
			List<IssueEventGroup> issues = [];

			// keep hash consistent when only have general, non-unique events
			bool allGeneral = _issueEvents.FirstOrDefault(stepEvent => stepEvent.EventId == null || !IsGeneralEventId(stepEvent.EventId.Value)) == null;

			Dictionary<string, List<IssueEvent>> fingerEvents = [];
			Dictionary<IssueEvent, string?> eventScopes = [];

			foreach (IssueEvent stepEvent in _issueEvents)
			{
				string? channelGroup = null;
				if (_context.ChannelGroups.Count > 0)
				{
					string? channel = GetLogChannel(stepEvent);
					eventScopes.Add(stepEvent, channel);
					if (!String.IsNullOrEmpty(channel))
					{
						_context.ChannelGroups.TryGetValue(channel, out channelGroup);
					}
				}

				string fingerprint = "Hashed";

				if (!String.IsNullOrEmpty(channelGroup))
				{
					fingerprint += $":{channelGroup}";
				}

				List<IssueEvent>? events = null;
				if (!fingerEvents.TryGetValue(fingerprint, out events))
				{
					events = [];
					fingerEvents.Add(fingerprint, events);
				}

				events.Add(stepEvent);
			}

			foreach (KeyValuePair<string, List<IssueEvent>> item in fingerEvents)
			{
				string fingerprintType = item.Key;

				IssueEventGroup? genericFingerprint = null;
				IssueEventGroup? genericErrorsFingerprint = null;
				HashSet<Md5Hash> hashes = [];

				foreach (IssueEvent stepEvent in item.Value)
				{
					string hashSource = stepEvent.Render();

					// For crash callstack events with an assertion/ensure summary, use the
					// summary for platform-agnostic hashing instead of the full callstack render
					string? crashSummary = GetStringProperty(stepEvent, "CrashSummary");
					if (crashSummary != null)
					{
						hashSource = crashSummary;
					}

					string? channel = null;
					eventScopes.TryGetValue(stepEvent, out channel);

					if (!allGeneral && stepEvent.EventId != null)
					{
						// If the event is general, salt the hash with the stream id, template, otherwise it will be aggressively matched.
						// Consider salting with node name, though template id should be enough and have better grouping
						if (IsGeneralEventId(stepEvent.EventId.Value))
						{
							hashSource += $"step:{_context.StreamId}:{_context.TemplateId}";
						}
					}

					string summaryTemplate = channel == null ? "{Severity} in {Meta:Node}" : "{Meta:Channel} {Severity} in {Meta:Node}";

					if (hashes.Count < 25 && TryGetHash(hashSource, out Md5Hash hash))
					{
						hashes.Add(hash);

						IssueEventGroup issue = new(fingerprintType, summaryTemplate, IssueChangeFilter.All);
						issue.Events.Add(stepEvent);
						issue.Keys.AddHash(hash);
						stepEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", hash, stepEvent.Render());
						AddFileMetadata(issue, stepEvent);
						issue.Metadata.Add("Node", _context.NodeName);
						if (!String.IsNullOrEmpty(channel))
						{
							issue.Metadata.Add("Channel", channel);
						}
						issues.Add(issue);
					}
					else
					{
						if (stepEvent.Severity == LogLevel.Error || stepEvent.Severity == LogLevel.Critical)
						{
							if (genericErrorsFingerprint == null)
							{
								genericErrorsFingerprint = new IssueEventGroup(fingerprintType, summaryTemplate, IssueChangeFilter.All);
								genericErrorsFingerprint.Keys.Add(IssueKey.FromStepAndSeverity(_context.StreamId, _context.TemplateId, _context.NodeName, LogLevel.Error));
								stepEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", IssueKey.FromStepAndSeverity(_context.StreamId, _context.TemplateId, _context.NodeName, LogLevel.Error), stepEvent.Render());
								genericErrorsFingerprint.Metadata.Add("Node", _context.NodeName);
								if (!String.IsNullOrEmpty(channel))
								{
									genericErrorsFingerprint.Metadata.Add("Channel", channel);
								}

								issues.Add(genericErrorsFingerprint);
							}
							genericErrorsFingerprint.Events.Add(stepEvent);
							AddFileMetadata(genericErrorsFingerprint, stepEvent);
						}
						else
						{
							if (genericFingerprint == null)
							{
								genericFingerprint = new IssueEventGroup(fingerprintType, summaryTemplate, IssueChangeFilter.All);
								genericFingerprint.Keys.Add(IssueKey.FromStep(_context.StreamId, _context.TemplateId, _context.NodeName));
								stepEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", IssueKey.FromStep(_context.StreamId, _context.TemplateId, _context.NodeName), stepEvent.Render());
								genericFingerprint.Metadata.Add("Node", _context.NodeName);
								if (!String.IsNullOrEmpty(channel))
								{
									genericFingerprint.Metadata.Add("Channel", channel);
								}

								issues.Add(genericFingerprint);
							}
							genericFingerprint.Events.Add(stepEvent);
							AddFileMetadata(genericFingerprint, stepEvent);
						}
					}
				}
			}

			return issues;
		}

		/// <summary>
		/// Gets a string property value from the event's JSON properties
		/// </summary>
		static string? GetStringProperty(IssueEvent logEvent, string propertyName)
		{
			try
			{
				foreach (JsonLogEvent line in logEvent.Lines)
				{
					using JsonDocument document = JsonDocument.Parse(line.Data);
					if (document.RootElement.TryGetProperty("properties", out JsonElement properties) &&
						properties.ValueKind == JsonValueKind.Object &&
						properties.TryGetProperty(propertyName, out JsonElement value) &&
						value.ValueKind == JsonValueKind.String)
					{
						return value.GetString();
					}
				}
			}
			catch
			{
				// Ignore parsing errors
			}
			return null;
		}

		static void AddFileMetadata(IssueEventGroup issue, IssueEvent stepEvent)
		{
			try
			{
				foreach (JsonLogEvent line in stepEvent.Lines)
				{
					using JsonDocument document = JsonDocument.Parse(line.Data);
					JsonElement properties;
					if (document.RootElement.TryGetProperty("properties", out properties) && properties.ValueKind == JsonValueKind.Object)
					{
						foreach (JsonProperty property in properties.EnumerateObject())
						{
							if (property.NameEquals("file") && property.Value.ValueKind == JsonValueKind.String)
							{
								string file = property.Value.GetString()!;
								int endIdx = file.LastIndexOfAny(['/', '\\']) + 1;
								string fileName = file.Substring(endIdx);

								issue.Metadata.Add("File", fileName);
								stepEvent.AuditLogger?.LogDebug("Metadata added for file: '{File}'", fileName);
							}
						}
					}
				}
			}
			catch (System.Exception ex)
			{
				stepEvent.AuditLogger?.LogError("Exception adding file metadata to issue: '{Exception}'", ex);
			}
		}

		static bool TryGetHash(string message, out Md5Hash hash)
		{
			string sanitized = message.ToUpperInvariant();
			sanitized = Regex.Replace(sanitized, @"(?<![a-zA-Z])(?:[A-Z]:|/)[^ :]+[/\\]SYNC[/\\]", "{root}/"); // Redact things that look like workspace roots; may be different between agents
			sanitized = Regex.Replace(sanitized, @"0[xX][0-9a-fA-F]+", "H"); // Redact hex strings
			sanitized = Regex.Replace(sanitized, @"\d[\d.,:]*", "n"); // Redact numbers and timestamp like things
			sanitized = Regex.Replace(sanitized, @"\\", "/");
			sanitized = Regex.Replace(sanitized, @"\[CookWorker \d{1,2}\]", "[CookWorker X]");

			if (sanitized.Length > 30)
			{
				hash = Md5Hash.Compute(Encoding.UTF8.GetBytes(sanitized));
				return true;
			}
			else
			{
				hash = Md5Hash.Zero;
				return false;
			}
		}
	}
}
