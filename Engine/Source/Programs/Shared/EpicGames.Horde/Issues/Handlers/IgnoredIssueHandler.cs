// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular systemic error
	/// </summary>
	[IssueHandler]
	public class IgnoredIssueHandler : IssueHandler
	{
		/// <summary>
		///  Known systemic errors
		/// </summary>
		static readonly HashSet<EventId> s_handledEvents = [KnownLogEvents.Horde_BuildHealth_Ignore];

		/// <inheritdoc/>
		public override int Priority => 100;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return s_handledEvents.Contains(eventId);
		}

		readonly List<IssueEventGroup> _issues = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public IgnoredIssueHandler() { }

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent logEvent)
		{
			if (logEvent.EventId != null)
			{
				if (IsMatchingEventId(logEvent.EventId.Value))
				{
					logEvent.AuditLogger?.LogDebug("IgnoreIssueHandler matched event: '{Event}'", logEvent.Render());
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
