// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of an issue handler for localization warnings/errors
	/// </summary>
	[IssueHandler]
	public class LocalizationIssueHandler : IssueHandler
	{
		private class LocalizationData
		{
			public string _file = "";
			public int _line = -1;
			public string _location = ""; // This can be {pathToFile}({line}), just {pathToFile} or multiple other possibilities including {notAFile}. Let's never assume it is easy to parse.
			public string _localizationNamespace = "";
			public string _localizationKey = "";

			public string _conflictFile = "";
			public int _conflictLine = -1;
			public string _conflictLocation = ""; // This can be {pathToFile}({line}), just {pathToFile} or multiple other possibilities including {notAFile}. Let's never assume it is easy to parse.
			public string _conflictLocalizationNamespace = "";
			public string _conflictLocalizationKey = "";

			public bool HasLocalizationKey() { return !System.String.IsNullOrEmpty(_localizationKey); }
			public string GetNamespaceKeyString() { return HasLocalizationKey() ? _localizationNamespace + "," + _localizationKey : ""; }

			public bool HasFile() { return !System.String.IsNullOrEmpty(_file); }
			public bool HasConflictFile() { return !System.String.IsNullOrEmpty(_conflictFile); }
			public bool HasAnyFile() { return HasFile() || HasConflictFile(); }
		}

		readonly IssueHandlerContext _context;
		readonly IssueEventGroup _issuesWithoutClearSuspects;
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Constructor
		/// </summary>
		public LocalizationIssueHandler(IssueHandlerContext context)
		{
			_context = context;
			_issuesWithoutClearSuspects = CreateNewIssueGroup();
		}

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId? eventId)
		{
			return eventId == KnownLogEvents.Engine_Localization;
		}

		/// <summary>
		/// Determines if an event should be masked by this 
		/// </summary>
		/// <param name="eventId"></param>
		/// <returns></returns>
		static bool IsMaskedEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.ExitCode;
		}

		private IssueEventGroup CreateNewIssueGroup()
		{
			string localizationIssueGroup = "Localization";
			string localizationSummaryTemplate = System.String.Format("{0}Localization {{Severity}}{1}",
				System.String.IsNullOrEmpty(_context.NodeName) ? "" : "{Meta:Node}: ",
				System.String.IsNullOrEmpty(_context.StreamId.ToString()) ? "" : " in {Meta:Stream}");

			IssueEventGroup newIssueGroup = new(localizationIssueGroup, localizationSummaryTemplate, IssueChangeFilter.All);

			if (!System.String.IsNullOrEmpty(_context.NodeName))
			{
				newIssueGroup.Metadata.Add("Node", _context.NodeName);
			}
			if (!System.String.IsNullOrEmpty(_context.StreamId.ToString()))
			{
				newIssueGroup.Metadata.Add("Stream", _context.StreamId.ToString());
			}

			return newIssueGroup;
		}

		/// <summary>
		/// Generate all the IssueKeys with the extracted LocalizationData
		/// </summary>
		/// <param name="keys">Set of keys</param>
		/// <param name="data">The localization data</param>
		private static void AddLocalizationIssueKeys(HashSet<IssueKey> keys, LocalizationData data)
		{
			if (data.HasFile())
			{
				keys.AddSourceFile(data._file, IssueKeyType.File);
			}

			if (data.HasConflictFile())
			{
				keys.AddSourceFile(data._conflictFile, IssueKeyType.File);
			}

			if (data.HasLocalizationKey())
			{
				keys.Add(new IssueKey(data.GetNamespaceKeyString(), IssueKeyType.None));
			}
		}

		/// <summary>
		/// Extracts the localization data from the properties of an IssueEvent
		/// </summary>
		/// <param name="issueEvent">The event data</param>
		private static LocalizationData GetLocalizationData(IssueEvent issueEvent)
		{
			LocalizationData data = new();

			foreach (JsonLogEvent line in issueEvent.Lines)
			{
				JsonDocument document = JsonDocument.Parse(line.Data);

				JsonElement properties;
				if (document.RootElement.TryGetProperty("properties", out properties) && properties.ValueKind == JsonValueKind.Object)
				{
					foreach (JsonProperty property in properties.EnumerateObject())
					{
						if (property.NameEquals("file") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._file = property.Value.GetString()!;
						}
						else if (property.NameEquals("line") && property.Value.ValueKind == JsonValueKind.Number)
						{
							data._line = property.Value.GetInt32();
						}
						else if (property.NameEquals("location") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._location = property.Value.GetString()!;
						}
						else if (property.NameEquals("locNamespace") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._localizationNamespace = property.Value.GetString()!;
						}
						// "locID" is deprecated and will be removed soon
						else if ((property.NameEquals("locKey") || property.NameEquals("locID")) && property.Value.ValueKind == JsonValueKind.String)
						{
							data._localizationKey = property.Value.GetString()!;
						}
						else if (property.NameEquals("conflictFile") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._conflictFile = property.Value.GetString()!;
						}
						else if (property.NameEquals("conflictLine") && property.Value.ValueKind == JsonValueKind.Number)
						{
							data._conflictLine = property.Value.GetInt32();
						}
						else if (property.NameEquals("conflictLocation") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._conflictLocation = property.Value.GetString()!;
						}
						else if (property.NameEquals("conflictLocNamespace") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._conflictLocalizationNamespace = property.Value.GetString()!;
						}
						else if (property.NameEquals("conflictLocKey") && property.Value.ValueKind == JsonValueKind.String)
						{
							data._conflictLocalizationKey = property.Value.GetString()!;
						}
					}
				}
			}

			ConvertLocationToFile(data);
			RemoveFilesExtension(data);

			return data;
		}

		/// <summary>
		/// Location data in LocalizationData may contain file information. We can try to parse it.
		/// </summary>
		/// <param name="data">The localization data</param>
		private static void ConvertLocationToFile(LocalizationData data)
		{
			try
			{
				if (!System.String.IsNullOrEmpty(data._location))
				{
					// If the TextLocation contains a '.', we assume it is a format: "/Path/To/Filename.*" where the filename can be retrieved
					int startExtensionIndex = data._location.IndexOf('.', System.StringComparison.Ordinal);
					if (startExtensionIndex > 0 && System.String.IsNullOrEmpty(data._file))
					{
						data._file = data._location.Substring(0, startExtensionIndex);
					}
				}
				if (!System.String.IsNullOrEmpty(data._conflictLocation))
				{
					// If the TextLocation contains a '.', we assume it is a format: "/Path/To/Filename.*" where the filename can be retrieved
					int startExtensionIndex = data._conflictLocation.IndexOf('.', System.StringComparison.Ordinal);
					if (startExtensionIndex > 0 && System.String.IsNullOrEmpty(data._conflictFile))
					{
						data._conflictFile = data._conflictLocation.Substring(0, startExtensionIndex);
					}
				}
			}
			catch
			{
				data._file = "";
				data._conflictFile = "";
			}
		}

		/// <summary>
		/// LocalizationData file paths may contain extensions, remove them
		/// </summary>
		/// <param name="data">The localization data</param>
		private static void RemoveFilesExtension(LocalizationData data)
		{
			if (!System.String.IsNullOrEmpty(data._file))
			{
				// If the File path contains a '.', we assume it is a format: "/Path/To/Filename.extension" where the filename can be retrieved
				int startExtensionIndex = data._file.IndexOf('.', System.StringComparison.Ordinal);
				if (startExtensionIndex > 0)
				{
					data._file = data._file.Substring(0, startExtensionIndex);
				}
			}
			if (!System.String.IsNullOrEmpty(data._conflictFile))
			{
				// If the ConflictFile contains a '.', we assume it is a format: "/Path/To/Filename.extension" where the filename can be retrieved
				int startExtensionIndex = data._conflictFile.IndexOf('.', System.StringComparison.Ordinal);
				if (startExtensionIndex > 0)
				{
					data._conflictFile = data._conflictFile.Substring(0, startExtensionIndex);
				}
			}
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId != null)
			{
				EventId eventId = issueEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					LocalizationData data = GetLocalizationData(issueEvent);

					IssueEventGroup issueGroup;
					if (data.HasAnyFile())
					{
						// Create a new issue group (might be grouped later based on keys)
						issueGroup = CreateNewIssueGroup();
					}
					else
					{
						// If we have no idea on the file an issue is associated with, then we won't be able to narrow down a suspect.
						// If we can't narrow down a suspect. There is no gain to split the issues amongst different issueGroup.
						// Actually, there is only a risk of creating thousands of issue group if localization is in a weird/broken state.
						issueGroup = _issuesWithoutClearSuspects;
					}

					// Add current event to the chosen issueGroup
					issueGroup.Events.Add(issueEvent);

					AddLocalizationIssueKeys(issueGroup.Keys, data);

					if (issueGroup.Keys.Count > 0 && !_issues.Contains(issueGroup))
					{
						_issues.Add(issueGroup);
						issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issueGroup.Type, issueEvent.Render());
					}

					return true;
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
