// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a Perforce case mismatch error
	/// </summary>
	[IssueHandler]
	public class PerforceCaseIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.AutomationTool_PerforceCase;
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId != null && IsMatchingEventId(issueEvent.EventId.Value))
			{
				IssueEventGroup issue = new("PerforceCase", "Inconsistent case for {Files}", IssueChangeFilter.All);
				issue.Events.Add(issueEvent);
				issue.Keys.AddDepotPaths(issueEvent);
				_issues.Add(issue);
				issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());

				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
