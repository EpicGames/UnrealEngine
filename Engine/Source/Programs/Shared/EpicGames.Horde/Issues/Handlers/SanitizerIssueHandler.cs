// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a specific Thread Sanitizer error
	/// </summary>
	[IssueHandler]
	public class SanitizerIssueHandler : IssueHandler
	{
		static readonly Utf8String s_summaryReasonAnnotation = new("SummaryReason");
		static readonly Utf8String s_summarySourceFileAnnotation = new("SummarySourceFile");
		static readonly Utf8String s_sanitizerNameAnnotation = new("SanitizerName");
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;
		/// <summary>
		/// Log value describing the thread sanitizer error summary reason
		/// </summary>
		public static LogValue SanitizerName => new(s_sanitizerNameAnnotation, "");

		/// <summary>
		/// Log value describing the thread sanitizer error summary reason
		/// </summary>
		public static LogValue SummaryReason => new(s_summaryReasonAnnotation, "");

		/// <summary>
		/// Log value describing the thread sanitizer error summary source file
		/// </summary>
		public static LogValue SummarySourceFile => new(s_summarySourceFileAnnotation, "");

		static bool FindAnnotation(IssueEvent issueEvent, Utf8String annotation, ref string result)
		{
			foreach (JsonProperty property in issueEvent.Lines.SelectMany(x => x.FindPropertiesOfType(annotation)))
			{
				JsonElement value;
				if (property.Value.TryGetProperty(LogEventPropertyName.Text.Span, out value) && value.ValueKind == JsonValueKind.String)
				{
					result = value.GetString() ?? "";
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			int eventIdValue = (issueEvent.EventId.HasValue ? issueEvent.EventId.Value.Id : 0) - KnownLogEvents.Sanitizer.Id;
			bool bIsSanitizerEvent = eventIdValue >= 0 && eventIdValue < 100; // 100 is the max range for unique events within a category
			if (bIsSanitizerEvent)
			{
				string summaryReason = "error";
				string summaryFile = "";
				string sanitizerName = "";

				FindAnnotation(issueEvent, s_summaryReasonAnnotation, ref summaryReason);
				FindAnnotation(issueEvent, s_summarySourceFileAnnotation, ref summaryFile);
				FindAnnotation(issueEvent, s_sanitizerNameAnnotation, ref sanitizerName);
				string fingerprint = $"{sanitizerName}Sanitizer:{summaryReason}:{summaryFile}";

				IssueEventGroup issue = new(fingerprint, $"{{Meta:{s_sanitizerNameAnnotation}}} found '{{Meta:{s_summaryReasonAnnotation}}}' in {{Meta:{s_summarySourceFileAnnotation}}}", IssueChangeFilter.Code);
				issue.Keys.Add(sanitizerName, IssueKeyType.None);
				issue.Metadata.Add(s_sanitizerNameAnnotation.ToString(), sanitizerName);
				issue.Keys.Add(summaryReason, IssueKeyType.Note);
				issue.Metadata.Add(s_summaryReasonAnnotation.ToString(), summaryReason);
				issue.Keys.AddSourceFile(summaryFile, IssueKeyType.File);
				issue.Metadata.Add(s_summarySourceFileAnnotation.ToString(), summaryFile);
				issue.Events.Add(issueEvent);

				_issues.Add(issue);
				issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());
				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
