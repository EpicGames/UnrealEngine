// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler(Tag = "Scoped")]
	public class ScopedIssueHandler : IssueHandler
	{
		const string NodeName = "Node";
		const string ScopeName = "Scope";

		readonly IssueHandlerContext _context;
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 2;
		/// <summary>
		/// Constructor
		/// </summary>
		public ScopedIssueHandler(IssueHandlerContext context) => _context = context;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent logEvent)
		{
			string? scope = null;

			foreach (JsonLogEvent line in logEvent.Lines)
			{
				JsonDocument document = JsonDocument.Parse(line.Data);

				string? channelType;
				string? channelText;
				if (document.RootElement.TryGetNestedProperty("properties.channel.$type", out channelType) && document.RootElement.TryGetNestedProperty("properties.channel.$text", out channelText))
				{
					if (channelType != "Channel" || !channelText.StartsWith("Log", StringComparison.Ordinal))
					{
						scope = null;
						break;
					}

					if (scope != null && scope != channelText)
					{
						scope = null;
						break;
					}

					scope = channelText;
				}
			}

			if (scope == null)
			{
				return false;
			}

			string fingerprintType = $"Scoped:{scope}";

			string hashSource = logEvent.Render();

			if (TryGetHash(hashSource, out Md5Hash hash))
			{
				IssueEventGroup issue = new(fingerprintType, "{Severity} in {Meta:Node} - {Meta:Scope}", IssueChangeFilter.All);
				issue.Events.Add(logEvent);
				issue.Keys.Add(IssueKey.FromHash(hash));
				issue.Metadata.Add(NodeName, _context.NodeName);
				issue.Metadata.Add(ScopeName, scope);
				_issues.Add(issue);
				logEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, logEvent.Render());

				return true;
			}

			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;

		static bool TryGetHash(string message, out Md5Hash hash)
		{
			string sanitized = message.ToUpperInvariant();
			sanitized = Regex.Replace(sanitized, @"(?<![a-zA-Z])(?:[A-Z]:|/)[^ :]+[/\\]SYNC[/\\]", "{root}/"); // Redact things that look like workspace roots; may be different between agents
			sanitized = Regex.Replace(sanitized, @"0[xX][0-9a-fA-F]+", "H"); // Redact hex strings
			sanitized = Regex.Replace(sanitized, @"\d[\d.,:]*", "n"); // Redact numbers and timestamp like things

			hash = Md5Hash.Compute(Encoding.UTF8.GetBytes(sanitized));
			return true;
		}
	}
}
