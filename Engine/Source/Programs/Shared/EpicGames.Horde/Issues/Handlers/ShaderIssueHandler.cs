// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Handles shader compile errors and warnings, creating issue groups for tracking.
	///
	/// Grouping behavior:
	/// - For "Failed to compile Material" warnings: Groups by material path, so all compilation
	///   failures for the same material (across different platforms/shaders) are tracked as one issue.
	/// - For other shader errors: Groups by source file, creating separate issues per file.
	///
	/// The material path is extracted by ShaderEventMatcher and passed as a "material" property.
	/// </summary>
	[IssueHandler]
	public class ShaderIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		static bool IsMatchingEventId(EventId id) => id == KnownLogEvents.Engine_ShaderCompiler;
		static bool IsMaskedEventId(EventId id) => id == KnownLogEvents.Generic || id == KnownLogEvents.ExitCode || id == KnownLogEvents.Systemic_Xge_BuildFailed || id == KnownLogEvents.Engine_Crash || id == KnownLogEvents.AutomationTool_CrashExitCode || id == KnownLogEvents.Engine_AppError;

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent logEvent)
		{
			if (logEvent.EventId.HasValue)
			{
				EventId eventId = logEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					// Check if this event has a material property (set by ShaderEventMatcher for
					// "Failed to compile Material" warnings). If so, group by material path instead
					// of source files, so all failures for the same material are tracked together.
					string? materialPath = TryGetMaterialPath(logEvent);

					IssueEventGroup issue;
					if (materialPath != null)
					{
						// Group by material path - use a different summary template that references the material
						issue = new IssueEventGroup("Shader", "Shader compile {Severity} in {Meta:Material}", IssueChangeFilter.Code);
						issue.Keys.Add(new IssueKey(materialPath, IssueKeyType.File));
						issue.Metadata.Add("Material", GetMaterialDisplayName(materialPath));
						logEvent.AuditLogger?.LogDebug("{IssueType} issue added for material '{Material}': '{Event}'", issue.Type, materialPath, logEvent.Render());
					}
					else
					{
						// Default behavior: group by source files
						issue = new IssueEventGroup("Shader", "Shader compile {Severity} in {Files}", IssueChangeFilter.Code);
						issue.Keys.AddSourceFiles(logEvent);
						logEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, logEvent.Render());
					}

					issue.Events.Add(logEvent);
					_issues.Add(issue);

					return true;
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Extracts the material path from the event's "material" property, if present.
		/// This property is set by ShaderEventMatcher when it detects "Failed to compile Material" warnings.
		/// </summary>
		private static string? TryGetMaterialPath(IssueEvent logEvent)
		{
			foreach (JsonLogEvent line in logEvent.Lines)
			{
				try
				{
					using JsonDocument document = JsonDocument.Parse(line.Data);
					if (document.RootElement.TryGetProperty("properties", out JsonElement properties) &&
						properties.ValueKind == JsonValueKind.Object &&
						properties.TryGetProperty("material", out JsonElement materialElement) &&
						materialElement.ValueKind == JsonValueKind.String)
					{
						return materialElement.GetString();
					}
				}
				catch (JsonException)
				{
					// Ignore parsing errors and continue checking other lines
				}
			}
			return null;
		}

		/// <summary>
		/// Extracts a display-friendly material name from the full material path.
		/// For example: "/Game/Materials/MyMaterial.MyMaterial" -> "MyMaterial"
		/// </summary>
		private static string GetMaterialDisplayName(string materialPath)
		{
			// Material paths are typically "/Package/Path/MaterialName.MaterialName"
			// Extract just the material name (last component after the final slash, before the dot)
			int lastSlash = materialPath.LastIndexOf('/');
			string name = lastSlash >= 0 ? materialPath.Substring(lastSlash + 1) : materialPath;

			int dotIndex = name.IndexOf('.', System.StringComparison.Ordinal);
			if (dotIndex > 0)
			{
				name = name.Substring(0, dotIndex);
			}

			return name;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
