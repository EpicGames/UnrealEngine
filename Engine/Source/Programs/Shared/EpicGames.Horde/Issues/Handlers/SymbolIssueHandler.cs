// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular compile error
	/// </summary>
	[IssueHandler]
	public class SymbolIssueHandler : IssueHandler
	{
		const string EventIdName = "EventId";

		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.Linker_UndefinedSymbol || eventId == KnownLogEvents.Linker_DuplicateSymbol || eventId == KnownLogEvents.Linker;
		}

		/// <summary>
		/// Determines if an event should be masked by this 
		/// </summary>
		/// <param name="eventId"></param>
		/// <returns></returns>
		static bool IsMaskedEventId(EventId eventId)
		{
			return eventId == KnownLogEvents.ExitCode || eventId == KnownLogEvents.Systemic_Xge_BuildFailed;
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId != null)
			{
				EventId eventId = issueEvent.EventId.Value;
				if (IsMatchingEventId(eventId))
				{
					IssueEventGroup issue = new("Symbol", "{LegacySymbolIssueHandler}", IssueChangeFilter.Code);
					issue.Events.Add(issueEvent);
					issue.Keys.AddSymbols(issueEvent);
					issue.Metadata.Add(EventIdName, eventId.Id.ToString());

					if (issue.Keys.Count > 0)
					{
						_issues.Add(issue);
						issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());
						return true;
					}
					if (_issues.Count > 0)
					{
						return true;
					}
				}
				else if (_issues.Count > 0 && IsMaskedEventId(eventId))
				{
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
