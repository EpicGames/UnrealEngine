// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a particular systemic error
	/// </summary>
	[IssueHandler]
	public class SystemicIssueHandler : IssueHandler
	{
		/// <summary>
		///  Known systemic errors
		/// </summary>
		static readonly HashSet<EventId> s_knownSystemic = [KnownLogEvents.Horde, KnownLogEvents.Horde_InvalidPreflight];

		/// <inheritdoc/>
		public override int Priority => 10;

		/// <summary>
		/// Determines if the given event id matches
		/// </summary>
		/// <param name="eventId">The event id to compare</param>
		/// <returns>True if the given event id matches</returns>
		public static bool IsMatchingEventId(EventId eventId)
		{
			return s_knownSystemic.Contains(eventId) || (eventId.Id >= KnownLogEvents.Systemic.Id && eventId.Id <= KnownLogEvents.Systemic_Max.Id);
		}

		bool _nonSystemicError = false;
		readonly IssueHandlerContext _context;
		readonly List<IssueEventGroup> _issues = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public SystemicIssueHandler(IssueHandlerContext context) => _context = context;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent logEvent)
		{
			if (logEvent.EventId != null)
			{
				if (IsMatchingEventId(logEvent.EventId.Value))
				{
					if (_nonSystemicError)
					{
						return true;
					}
					else
					{
						IssueEventGroup issue = new("Systemic", "Systemic {Severity} in {Nodes}", IssueChangeFilter.None);
						issue.Keys.Add(IssueKey.FromStep(_context.StreamId, _context.TemplateId, _context.NodeName));
						issue.Events.Add(logEvent);
						_issues.Add(issue);
						logEvent.AuditLogger?.LogDebug("Fingerprint key: '{Key}' generated from event: '{Event}'", $"{_context.StreamId}:{_context.NodeName}", logEvent.Render());

						return true;
					}
				}
				else if (logEvent.Severity >= LogLevel.Error)
				{
					// We've seen a non-systemic error event, so ignore this systemic event to prevent superfluous issues from being created
					_nonSystemicError = true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
