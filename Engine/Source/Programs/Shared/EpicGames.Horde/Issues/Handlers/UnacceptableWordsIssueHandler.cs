// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// Instance of a Perforce case mismatch error
	/// </summary>
	[IssueHandler]
	public class UnacceptableWordsIssueHandler : IssueHandler
	{
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 8;

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			if (issueEvent.EventId == KnownLogEvents.AutomationTool_UnacceptableWords)
			{
				IssueEventGroup issue = new("UnacceptableWords", "Unacceptable words in {Files}", IssueChangeFilter.Code);
				issue.Events.Add(issueEvent);
				issue.Keys.AddSourceFiles(issueEvent);
				_issues.Add(issue);
				issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", issue.Type, issueEvent.Render());

				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
