// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues.Handlers
{
	/// <summary>
	/// World Leak error issue handler
	/// </summary>
	[IssueHandler]
	public class WorldLeakIssueHandler : IssueHandler
	{
		static readonly Utf8String s_objectFlagsAnnotation = new("ObjectFlags");
		static readonly Utf8String s_objectTextAnnotation = new("ObjectText");
		static readonly Utf8String s_lingeringReferenceAnnotation = new("LingeringReference");
		readonly List<IssueEventGroup> _issues = [];

		/// <inheritdoc/>
		public override int Priority => 10;
		/// <summary>
		/// Log value describing the object reference preventing garbage collection
		/// </summary>
		public static LogValue LingeringReference => new(s_lingeringReferenceAnnotation, "");

		/// <summary>
		/// Log value describing the flag on an object in the leak reference chain
		/// </summary>
		public static LogValue ObjectFlags => new(s_objectFlagsAnnotation, "");

		/// <summary>
		/// Log value describing an object in the leak reference chain
		/// </summary>
		public static LogValue ObjectText => new(s_objectTextAnnotation, "");

		static bool FindAnnotation(IssueEvent issueEvent, Utf8String annotation, ref string result)
		{
			foreach (JsonProperty property in issueEvent.Lines.SelectMany(x => x.FindPropertiesOfType(annotation)))
			{
				JsonElement value;
				if (property.Value.TryGetProperty(LogEventPropertyName.Text.Span, out value) && value.ValueKind == JsonValueKind.String)
				{
					result = value.GetString() ?? "";
					return true;
				}
			}
			return false;
		}

		/// <inheritdoc/>
		public override bool HandleEvent(IssueEvent issueEvent)
		{
			EventId? eventId = issueEvent.EventId;
			if (eventId == KnownLogEvents.WorldLeak)
			{
				string lingeringReference = "";
				string firstObjectFlags = "";
				string firstObjectText = "";

				FindAnnotation(issueEvent, s_lingeringReferenceAnnotation, ref lingeringReference);
				FindAnnotation(issueEvent, s_objectFlagsAnnotation, ref firstObjectFlags);
				FindAnnotation(issueEvent, s_objectTextAnnotation, ref firstObjectText);

				string fingerprint = $"WorldLeak:{lingeringReference}:{firstObjectFlags} {firstObjectText}";

				IssueEventGroup issue = new(fingerprint, $"World Leak Detected: '{{Meta:{s_lingeringReferenceAnnotation}}}' is preventing GC. Reference chain begins from '{{Meta:{s_objectFlagsAnnotation}}} {{Meta:{s_objectTextAnnotation}}}'", IssueChangeFilter.All);
				issue.Keys.Add(lingeringReference, IssueKeyType.Hash);
				issue.Metadata.Add(s_lingeringReferenceAnnotation.ToString(), lingeringReference);
				issue.Keys.Add(firstObjectFlags, IssueKeyType.None);
				issue.Metadata.Add(s_objectFlagsAnnotation.ToString(), firstObjectFlags);
				issue.Keys.Add(firstObjectText, IssueKeyType.None);
				issue.Metadata.Add(s_objectTextAnnotation.ToString(), firstObjectText);
				issue.Events.Add(issueEvent);

				_issues.Add(issue);
				issueEvent.AuditLogger?.LogDebug("{IssueType} issue added for event: '{Event}'", fingerprint, issueEvent.Render());
				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override IEnumerable<IssueEventGroup> GetIssues() => _issues;
	}
}
