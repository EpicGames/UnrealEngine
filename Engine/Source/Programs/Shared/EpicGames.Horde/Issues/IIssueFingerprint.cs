// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Fingerprint for an issue
	/// </summary>
	public interface IIssueFingerprint
	{
		/// <summary>
		/// The type of issue, which defines the handler to use for it
		/// </summary>
		public string Type { get; }

		/// <summary>
		/// Template string for the issue summary
		/// </summary>
		public string SummaryTemplate { get; }

		/// <summary>
		/// List of keys which identify this issue.
		/// </summary>
		public IReadOnlySet<IssueKey> Keys { get; }

		/// <summary>
		/// Set of keys which should trigger a negative match
		/// </summary>
		public IReadOnlySet<IssueKey>? RejectKeys { get; }

		/// <summary>
		/// Collection of additional metadata added by the handler
		/// </summary>
		public IReadOnlySet<IssueMetadata>? Metadata { get; }

		/// <summary>
		/// Filter for changes that should be included in this issue
		/// </summary>
		public string ChangeFilter { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IIssueFingerprint"/>
	/// </summary>
	public static class IssueFingerprintExtensions
	{
		/// <summary>
		/// Checks if a fingerprint matches another fingerprint
		/// </summary>
		/// <param name="fingerprint">The first fingerprint to compare</param>
		/// <param name="other">The other fingerprint to compare to</param>
		/// <returns>True is the fingerprints match</returns>
		public static bool IsMatch(this IIssueFingerprint fingerprint, IIssueFingerprint other)
		{
			if (!fingerprint.Type.Equals(other.Type, StringComparison.Ordinal))
			{
				return false;
			}
			if (!fingerprint.SummaryTemplate.Equals(other.SummaryTemplate, StringComparison.Ordinal))
			{
				return false;
			}
			if (fingerprint.Keys.Count == 0)
			{
				if (other.Keys.Count > 0)
				{
					return false;
				}
			}
			else
			{
				if (!fingerprint.Keys.Any(x => other.Keys.Contains(x)))
				{
					return false;
				}
			}
			if (fingerprint.RejectKeys != null && fingerprint.RejectKeys.Any(x => other.Keys.Contains(x)))
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Checks if a fingerprint matches another fingerprint for creating a new span
		/// </summary>
		/// <param name="fingerprint">The first fingerprint to compare</param>
		/// <param name="other">The other fingerprint to compare to</param>
		/// <returns>True is the fingerprints match</returns>
		public static bool IsMatchForNewSpan(this IIssueFingerprint fingerprint, IIssueFingerprint other)
		{
			if (!fingerprint.Type.Equals(other.Type, StringComparison.Ordinal))
			{
				return false;
			}
			if (fingerprint.RejectKeys != null && fingerprint.RejectKeys.Any(x => other.Keys.Contains(x)))
			{
				return false;
			}
			return true;
		}
	}
}
