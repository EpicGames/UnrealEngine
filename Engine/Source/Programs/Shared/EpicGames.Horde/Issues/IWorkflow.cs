// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Jobs.Graphs;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Configuration for an issue workflow
	/// </summary>
	public interface IWorkflow
	{
		/// <summary>
		/// Identifier for this workflow
		/// </summary>
		WorkflowId Id { get; }

		/// <summary>
		/// Times of day at which to send a report
		/// </summary>
		IReadOnlyList<TimeSpan> ReportTimes { get; }

		/// <summary>
		/// Name of the tab to post summary data to
		/// </summary>
		string? SummaryTab { get; }

		/// <summary>
		/// Channel to post summary information for these templates.
		/// </summary>
		string? ReportChannel { get; }

		/// <summary>
		/// Whether to include issues with a warning status in the summary
		/// </summary>
		bool ReportWarnings { get; }

		/// <summary>
		/// Whether to group issues by template in the report
		/// </summary>
		bool GroupIssuesByTemplate { get; }

		/// <summary>
		/// Channel to post threads for triaging new issues
		/// </summary>
		string? TriageChannel { get; }

		/// <summary>
		/// Prefix for all triage messages
		/// </summary>
		string? TriagePrefix { get; }

		/// <summary>
		/// Suffix for all triage messages
		/// </summary>
		string? TriageSuffix { get; }

		/// <summary>
		/// Instructions posted to triage threads
		/// </summary>
		string? TriageInstructions { get; }

		/// <summary>
		/// User id of a Slack user/alias to ping if there is nobody assigned to an issue by default.
		/// </summary>
		string? TriageAlias { get; }

		/// <summary>
		/// Whether to include issues with an error status in the triage
		/// </summary>
		bool TriageErrors { get; }

		/// <summary>
		/// Whether to include issues with a warning status in the triage
		/// </summary>
		bool TriageWarnings { get; }

		/// <summary>
		/// Slack user/alias to ping for specific issue types (such as Systemic), if there is nobody assigned to an issue by default.
		/// </summary>
		IReadOnlyDictionary<string, string>? TriageTypeAliases { get; }

		/// <summary>
		/// Alias to ping if an issue has not been resolved for a certain amount of time
		/// </summary>
		string? EscalateAlias { get; }

		/// <summary>
		/// Times after an issue has been opened to escalate to the alias above, in minutes. Continues to notify on the last interval once reaching the end of the list.
		/// </summary>
		IReadOnlyList<int> EscalateTimes { get; }

		/// <summary>
		/// Times after a manual root cause analysis has been opened to escalate to the owner, in minutes. Continues to notify on the last interval once reaching the end of the list.
		/// </summary>
		IReadOnlyList<int> EscalateRootCauseAnalysisTimes { get; }

		/// <summary>
		/// Whether to automatically run RCA processing for issues in this workflow
		/// </summary>
		bool AutoRcaEnabled { get; }

		/// <summary>
		/// Whether to post auto-RCA results to the issue's Slack triage thread
		/// </summary>
		bool AutoRcaSlackEnabled { get; }

		/// <summary>
		/// Maximum number of people to mention on a triage thread
		/// </summary>
		int MaxMentions { get; }

		/// <summary>
		/// Whether to mention people on this thread. Useful to disable for testing.
		/// </summary>
		bool AllowMentions { get; }

		/// <summary>
		/// Uses the admin.conversations.invite API to invite users to the channel
		/// </summary>
		bool InviteRestrictedUsers { get; }

		/// <summary>
		/// Skips sending reports when there are no active issues. 
		/// </summary>
		bool SkipWhenEmpty { get; }

		/// <summary>
		/// Whether to show warnings about merging changes into the origin stream.
		/// </summary>
		bool ShowMergeWarnings { get; }

		/// <summary>
		/// Additional node annotations implicit in this workflow
		/// </summary>
		IReadOnlyNodeAnnotations Annotations { get; }

		/// <summary>
		/// External issue tracking configuration for this workflow
		/// </summary>
		IWorkflowExternalIssues? ExternalIssues { get; }

		/// <summary>
		/// Additional issue handlers enabled for this workflow
		/// </summary>
		IReadOnlyList<string>? IssueHandlers { get; }
	}

	/// <summary>
	/// External issue tracking configuration for a workflow
	/// </summary>
	public interface IWorkflowExternalIssues
	{
		/// <summary>
		/// Project key in external issue tracker
		/// </summary>
		string ProjectKey { get; }

		/// <summary>
		/// Default component id for issues using workflow
		/// </summary>
		string DefaultComponentId { get; }

		/// <summary>
		/// Default issue type id for issues using workflow
		/// </summary>
		string DefaultIssueTypeId { get; }
	}
}
