// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Set of rules for filtering the list of suspects for an issue
	/// </summary>
	public static class IssueChangeFilter
	{
		/// <summary>
		/// Filter exclude all changes
		/// </summary>
		public static IReadOnlyList<string> None { get; } = [];

		/// <summary>
		/// Filter including all changes
		/// </summary>
		public static IReadOnlyList<string> All { get; } = ["..."];

		/// <summary>
		/// Set of extensions to treat as code
		/// </summary>
		public static IReadOnlyList<string> Code { get; } =
		[
			"*.c",
			"*.cc",
			"*.cxx",
			"*.cpp",
			"*.inl",
			"*.m",
			"*.mm",
			"*.rc",
			"*.cs",
			"*.csproj",
			"*.h",
			"*.hpp",
			"*.inl",
			"*.usf",
			"*.ush",
			"*.uproject",
			"*.uplugin",
			"*.sln",
			"*.verse"
		];

		/// <summary>
		/// Set of file extensions to treat as content
		/// </summary>
		public static IReadOnlyList<string> Content { get; } =
		[
			"*.uasset",
			"*.umap",
			"*.ini"
		];
	}
}
