// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Wraps a log event and allows it to be tagged by issue handlers
	/// </summary>
	public class IssueEvent
	{
		/// <summary>
		/// Index of the line within this log
		/// </summary>
		public int LineIndex { get; }

		/// <summary>
		/// Severity of the event
		/// </summary>
		public LogLevel Severity { get; }

		/// <summary>
		/// The type of event
		/// </summary>
		public EventId? EventId { get; }

		/// <summary>
		/// Gets this event data as a BSON document
		/// </summary>
		public IReadOnlyList<JsonLogEvent> Lines { get; }

		/// <summary>
		/// <see cref="IssueAuditLogger"/> attached to this event
		/// </summary>
		public IssueAuditLogger? AuditLogger { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueEvent(int lineIndex, LogLevel severity, EventId? eventId, IReadOnlyList<JsonLogEvent> lines)
		{
			LineIndex = lineIndex;
			Severity = severity;
			EventId = eventId;
			Lines = lines;
		}

		/// <summary>
		/// Renders the entire message of this event
		/// </summary>
		public string Render()
			=> String.Join("\n", Lines.Select(x => x.GetRenderedMessage().ToString()));

		/// <inheritdoc/>
		public override string ToString() => $"[{LineIndex}] {Render()}";
	}

	/// <summary>
	/// A group of <see cref="IssueEvent"/> objects with their fingerprint
	/// </summary>
	public class IssueEventGroup
	{
		/// <summary>
		/// The type of issue, which defines the handler to use for it
		/// </summary>
		public string Type { get; set; }

		/// <summary>
		/// Template string for the issue summary
		/// </summary>
		public string SummaryTemplate { get; set; }

		/// <summary>
		/// List of keys which identify this issue.
		/// </summary>
		public HashSet<IssueKey> Keys { get; } = [];

		/// <summary>
		/// Collection of additional metadata added by the handler
		/// </summary>
		public HashSet<IssueMetadata> Metadata { get; } = [];

		/// <summary>
		/// Filter for changes that should be included in this issue
		/// </summary>
		public string ChangeFilter { get; set; }

		/// <summary>
		/// Individual log events
		/// </summary>
		public List<IssueEvent> Events { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The type of issue</param>
		/// <param name="summaryTemplate">Template for the summary string to display for the issue</param>
		/// <param name="changeFilter">Filter for changes covered by this issue</param>
		public IssueEventGroup(string type, string summaryTemplate, string changeFilter)
		{
			Type = type;
			SummaryTemplate = summaryTemplate;
			ChangeFilter = changeFilter;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The type of issue</param>
		/// <param name="summaryTemplate">Template for the summary string to display for the issue</param>
		/// <param name="changeFilter">Filter for changes covered by this issue</param>
		public IssueEventGroup(string type, string summaryTemplate, IReadOnlyList<string> changeFilter)
			: this(type, summaryTemplate, String.Join(";", changeFilter))
		{
		}
	}

	/// <summary>
	/// Temporary log buffer to store information about an <see cref="IssueEvent"/> before it's assigned to a build health issue
	/// </summary>
	public class IssueAuditLogger : ILogger
	{
		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null;

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => true;

		private readonly List<IssueAuditLogEntry> _entries;

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueAuditLogger()
		{
			_entries = [];
		}

		/// <summary>
		/// Gets list of <see cref="IssueAuditLogEntry"/> objects in the buffer
		/// </summary>
		/// <returns>List of log entries</returns>
		public List<IssueAuditLogEntry> GetEntries()
		{
			return _entries;
		}

		/// <summary>
		/// Stores a <see cref="IssueAuditLogEntry"/> in the log buffer
		/// </summary>
		/// <param name="logLevel">Entry will be written on this level.</param>
		/// <param name="eventId">Id of the event.</param>
		/// <param name="state">The entry to be written. Can be also an object.</param>
		/// <param name="exception">The exception related to this entry.</param>
		/// <param name="formatter">Function to create a <see cref="String"/> message of the <paramref name="state"/> and <paramref name="exception"/>.</param>
		/// <typeparam name="TState">The type of the object to be written.</typeparam>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			// extract and store message template and args
			string? messageTemplate = null;
			List<object> args = [];
			IEnumerable<KeyValuePair<string, object>>? innerEnumerable = state as IEnumerable<KeyValuePair<string, object>>;
			if (innerEnumerable != null)
			{
				foreach (KeyValuePair<string, object> pair in innerEnumerable)
				{
					if (pair.Key.Equals("{OriginalFormat}", StringComparison.Ordinal))
					{
						messageTemplate = pair.Value.ToString();
					}
					else
					{
						args.Add(pair.Value);
					}
				}
			}

			IssueAuditLogEntry entry = new(logLevel, messageTemplate ?? "", [.. args]);
			_entries.Add(entry);
		}
	}

	/// <summary>
	/// Buffered log entry stored in a <see cref="IssueAuditLogger"/>
	/// </summary>
	/// <remarks>
	/// Constructor
	/// </remarks>
	public readonly struct IssueAuditLogEntry(LogLevel logLevel, string messageTemplate, object[]? args) : IEquatable<IssueAuditLogEntry>
	{
		/// <summary>
		/// Severity level at which to log the entry
		/// </summary>
		private readonly LogLevel _logLevel = logLevel;

		/// <summary>
		/// Message template for structured log entry
		/// </summary>
		private readonly string _messageTemplate = messageTemplate;

		/// <summary>
		/// Arguments to be injected in the message template of the entry
		/// </summary>
		private readonly object[]? _args = args;

		/// <summary>
		/// Log level accessor
		/// </summary>
		public readonly LogLevel GetLogLevel()
		{
			return _logLevel;
		}

		/// <summary>
		/// Message template accessor
		/// </summary>
		public readonly string? GetMessageTemplate()
		{
			return _messageTemplate;
		}

		/// <summary>
		/// Args accessor
		/// </summary>
		public readonly object[]? GetArgs()
		{
			return _args;
		}

		/// <summary>
		/// Compares against another IssueAuditLogEntry object for equality
		/// </summary>
		/// <param name="otherEntry">Other entry to compare against</param>
		/// <returns>True if the two entries have the same log level, message template, and arguments.</returns>
		public readonly bool Equals(IssueAuditLogEntry otherEntry)
		{
			if (_logLevel != otherEntry._logLevel || _messageTemplate != otherEntry._messageTemplate)
			{
				return false;
			}

			if (_args == null && otherEntry._args == null)
			{
				return true;
			}
			else if (_args == null || otherEntry._args == null)
			{
				return false;
			}
			else
			{
				if (_args.Length != otherEntry._args.Length)
				{
					return false;
				}

				for (int i = 0; i < _args.Length; i++)
				{
					if (_args[i] == null && otherEntry._args[i] == null)
					{
						continue;
					}
					else if (_args[i] == null || otherEntry._args[i] == null)
					{
						return false;
					}
					else if (!_args[i].Equals(otherEntry._args[i]))
					{
						return false;
					}
				}
			}

			return true;
		}

		/// <summary>
		/// Returns the underlying value hashcode
		/// </summary>
		public override readonly int GetHashCode()
		{
			return HashCode.Combine(_logLevel, _messageTemplate);
		}

		/// <inheritdoc/>
		public override readonly bool Equals(object? obj)
		{
			return obj is IssueAuditLogEntry entry && Equals(entry);
		}

		/// <summary>
		/// Compares two IssueAuditLogEntry objects for equality
		/// </summary>
		public static bool operator ==(IssueAuditLogEntry left, IssueAuditLogEntry right)
		{
			return left.Equals(right);
		}

		/// <summary>
		/// Compares two IssueAuditLogEntry objects for inequality
		/// </summary>
		public static bool operator !=(IssueAuditLogEntry left, IssueAuditLogEntry right)
		{
			return !(left == right);
		}
	}
}
