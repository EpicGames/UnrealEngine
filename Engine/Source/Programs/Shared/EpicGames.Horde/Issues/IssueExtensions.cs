// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Extension methods for logging issue data
	/// </summary>
	public static class IssueExtensions
	{
		/// <summary>
		/// Adds an issue fingerprint to a log event
		/// </summary>
		/// <param name="logEvent">Log event to modify</param>
		/// <param name="fingerprint">Fingerprint for the issue</param>
		public static void AddIssueFingerprint(this LogEvent logEvent, IssueFingerprint fingerprint)
			=> logEvent.AddProperty("@$issue", fingerprint);

		/// <summary>
		/// Enters a scope which annotates log messages with the supplied issue fingerprint
		/// </summary>
		/// <param name="logger">Logger device to operate on</param>
		/// <param name="fingerprint">Fingerprint for the issue</param>
		/// <returns>Disposable object for the lifetime of this scope</returns>
		public static IDisposable? BeginIssueScope(this ILogger logger, IssueFingerprint fingerprint)
		{
			Dictionary<string, object> properties = new(1);
			properties.Add("@$issue", fingerprint);

			return logger.BeginScope(properties);
		}
	}
}
