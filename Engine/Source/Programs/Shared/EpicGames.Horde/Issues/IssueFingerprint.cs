// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;

#pragma warning disable CA2227 // Change 'x' to be read-only by removing the property setter

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Fingerprint for an issue. Can be embedded into a structured log event under the "$issue" property to guide how Horde should group the event.
	/// </summary>
	public class IssueFingerprint : IIssueFingerprint
	{
		/// <summary>
		/// The type of issue, which defines the handler to use for it
		/// </summary>
		[JsonPropertyName("type")]
		public string Type { get; set; } = "None";

		/// <summary>
		/// Template string for the issue summary
		/// </summary>
		[JsonPropertyName("summary")]
		public string SummaryTemplate { get; set; } = String.Empty;

		/// <summary>
		/// List of keys which identify this issue.
		/// </summary>
		[JsonIgnore]
		public HashSet<IssueKey> Keys { get; set; } = [];
		IReadOnlySet<IssueKey> IIssueFingerprint.Keys => Keys;

		/// <summary>
		/// Set of keys which should trigger a negative match
		/// </summary>
		[JsonPropertyName("rejectKeys")]
		public HashSet<IssueKey>? RejectKeys { get; set; }
		IReadOnlySet<IssueKey>? IIssueFingerprint.RejectKeys => RejectKeys;

		/// <summary>
		/// Collection of additional metadata added by the handler
		/// </summary>
		[JsonIgnore]
		public HashSet<IssueMetadata> Metadata { get; set; } = [];
		IReadOnlySet<IssueMetadata> IIssueFingerprint.Metadata => Metadata;

		/// <summary>
		/// Filter for changes that touch files which should be included in this issue
		/// </summary>
		[JsonPropertyName("filter")]
		public string ChangeFilter { get; set; } = "...";

		/// <summary>
		/// Keys set which is null when empty
		/// </summary>
		[JsonPropertyName("keys")]
		[JsonIgnore(Condition = JsonIgnoreCondition.WhenWritingNull)]
		public HashSet<IssueKey>? SerializedKeys
		{
			get => (Keys.Count == 0) ? null : Keys;
			set => Keys = value ?? [];
		}

		/// <summary>
		/// Metadata set which is null when empty
		/// </summary>
		[JsonPropertyName("meta")]
		[JsonIgnore(Condition = JsonIgnoreCondition.WhenWritingNull)]
		public HashSet<IssueMetadata>? SerializedMetadata
		{
			get => (Metadata.Count == 0) ? null : Metadata;
			set => Metadata = value ?? [];
		}

		/// <summary>
		/// Default constructor
		/// </summary>
		public IssueFingerprint()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueFingerprint(string type, string summaryTemplate)
		{
			Type = type;
			SummaryTemplate = summaryTemplate;
		}
	}
}
