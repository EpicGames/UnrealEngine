// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Marks an issue handler that should be automatically inserted into the pipeline
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class IssueHandlerAttribute : Attribute
	{
		/// <summary>
		/// Class of handler which can be explicitly enabled via a workflow
		/// </summary>
		public string? Tag { get; set; }
	}

	/// <summary>
	/// Interface for issue matchers
	/// </summary>
	public abstract class IssueHandler
	{
		/// <summary>
		/// Priority value for this issue handler. Handlers are processed in order of decreasing priority.
		/// </summary>
		public abstract int Priority { get; }

		/// <summary>
		/// Attempts to assign a log event to an issue
		/// </summary>
		/// <param name="issueEvent">Events to process</param>
		/// <returns>Issue definition for this log event</returns>
		public abstract bool HandleEvent(IssueEvent issueEvent);

		/// <summary>
		/// Gets all the issues created by this handler
		/// </summary>
		/// <returns></returns>
		public abstract IEnumerable<IssueEventGroup> GetIssues();
	}

	/// <summary>
	/// Context object for issue handlers
	/// </summary>
	public class IssueHandlerContext
	{
		/// <summary>
		/// Identifier for the current stream 
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// Identififer of the template
		/// </summary>
		public TemplateId TemplateId { get; }

		/// <summary>
		/// Identifier for the current node name 
		/// </summary>
		public string NodeName { get; }

		/// <summary>
		/// Log Channel => Channel Grouping
		/// </summary>
		public IReadOnlyDictionary<string, string> ChannelGroups { get; }

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public IReadOnlyDictionary<string, string> NodeAnnotations { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueHandlerContext(StreamId streamId, TemplateId templateId, string nodeName, IReadOnlyDictionary<string, string> channelGroups, IReadOnlyDictionary<string, string> nodeAnnotations)
		{
			StreamId = streamId;
			TemplateId = templateId;
			NodeName = nodeName;
			NodeAnnotations = nodeAnnotations;
			ChannelGroups = channelGroups;
		}
	}
}
