// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Type of a key in an issue, used for grouping.
	/// </summary>
	public enum IssueKeyType
	{
		/// <summary>
		/// Unknown type
		/// </summary>
		None,

		/// <summary>
		/// Filename
		/// </summary>
		File,

		/// <summary>
		/// Secondary file
		/// </summary>
		Note,

		/// <summary>
		/// Name of a symbol
		/// </summary>
		Symbol,

		/// <summary>
		/// Hash of a particular error
		/// </summary>
		Hash,

		/// <summary>
		/// Identifier for a particular step
		/// </summary>
		Step,
	}

	/// <summary>
	/// Defines a key which can be used to group an issue with other issues
	/// </summary>
	public class IssueKey : IEquatable<IssueKey>
	{
		/// <summary>
		/// Name of the key
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Type of the key
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public IssueKeyType Type { get; }

		/// <summary>
		/// Arbitrary string that can be used to discriminate between otherwise identical keys, limiting the issues that it can merge with.
		/// </summary>
		[JsonIgnore(Condition = JsonIgnoreCondition.WhenWritingNull)]
		public string? Scope { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueKey(string name, IssueKeyType type)
		{
			Name = name;
			Type = type;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		[JsonConstructor]
		public IssueKey(string name, IssueKeyType type, string? scope = null)
		{
			Name = name;
			Type = type;
			Scope = scope;
		}

		/// <summary>
		/// Creates an issue key for a file
		/// </summary>
		public static IssueKey FromFile(string file, bool note = false) => new(file, note ? IssueKeyType.Note : IssueKeyType.File);

		/// <summary>
		/// Creates an issue key for a file
		/// </summary>
		public static IssueKey FromSymbol(string name) => new(name, IssueKeyType.Symbol);

		/// <summary>
		/// Creates an issue key for a particular hash
		/// </summary>
		public static IssueKey FromHash(Md5Hash hash) => new(hash.ToString(), IssueKeyType.Hash);

		/// <summary>
		/// Creates an issue key for a particular step
		/// </summary>
		public static IssueKey FromStep(StreamId streamId, TemplateId templateId, string nodeName) => new($"{streamId}:{templateId}:{nodeName}", IssueKeyType.Step);

		/// <summary>
		/// Creates an issue key for a particular step and severity
		/// </summary>
		public static IssueKey FromStepAndSeverity(StreamId streamId, TemplateId templateId, string nodeName, LogLevel severity) => new($"{streamId}:{templateId}:{nodeName}:{severity}", IssueKeyType.Step);

		/// <inheritdoc/>
		public bool Equals(IssueKey? other) => other is not null && other.Name.Equals(Name, StringComparison.OrdinalIgnoreCase) && other.Type == Type && String.Equals(Scope, other.Scope, StringComparison.Ordinal);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is IssueKey other && Equals(other);

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(String.GetHashCode(Name, StringComparison.OrdinalIgnoreCase), Type, Scope?.GetHashCode(StringComparison.Ordinal) ?? 0);

		/// <inheritdoc/>
		public override string ToString() => Name;

		/// <inheritdoc/>
		public static bool operator ==(IssueKey left, IssueKey right) => left.Equals(right);

		/// <inheritdoc/>
		public static bool operator !=(IssueKey left, IssueKey right) => !left.Equals(right);
	}

	/// <summary>
	/// Extension methods for issue keys
	/// </summary>
	public static class IssueKeyExtensions
	{
		/// <summary>
		/// Adds a new entry to a set
		/// </summary>
		public static void Add(this HashSet<IssueKey> entries, string name, IssueKeyType type, string? scope = null)
		{
			entries.Add(new IssueKey(name, type, scope));
		}

		/// <summary>
		/// Adds all the assets from the given log event
		/// </summary>
		/// <param name="keys">Set of keys</param>
		/// <param name="issueEvent">The log event to parse</param>
		public static void AddAssets(this HashSet<IssueKey> keys, IssueEvent issueEvent)
		{
			foreach (JsonLogEvent line in issueEvent.Lines)
			{
				JsonDocument document = JsonDocument.Parse(line.Data);

				string? relativePath;
				if (document.RootElement.TryGetNestedProperty("properties.asset.relativePath", out relativePath) || document.RootElement.TryGetNestedProperty("properties.asset.$text", out relativePath))
				{
					int endIdx = relativePath.LastIndexOfAny(['/', '\\']) + 1;
					string fileName = relativePath.Substring(endIdx);
					IssueKey issueKey = new(fileName, IssueKeyType.File);
					keys.Add(issueKey);
				}
			}
		}

		/// <summary>
		/// Extracts a list of source files from an event
		/// </summary>
		/// <param name="keys">Set of keys</param>
		/// <param name="issueEvent">The event data</param>
		public static void AddDepotPaths(this HashSet<IssueKey> keys, IssueEvent issueEvent)
		{
			foreach (JsonProperty property in issueEvent.Lines.SelectMany(x => x.FindPropertiesOfType(LogValueType.DepotPath)))
			{
				JsonElement value;
				if (property.Value.TryGetProperty(LogEventPropertyName.Text.Span, out value) && value.ValueKind == JsonValueKind.String)
				{
					string path = value.GetString() ?? String.Empty;
					string fileName = path.Substring(path.LastIndexOf('/') + 1);
					keys.Add(new IssueKey(fileName, IssueKeyType.File));
				}
			}
		}

		/// <summary>
		/// Extracts a list of source files from an event
		/// </summary>
		/// <param name="keys">Set of keys</param>
		/// <param name="hash">The event hash</param>
		/// <param name="scope">Scope for merging this hash value</param>
		public static void AddHash(this HashSet<IssueKey> keys, Md5Hash hash, string? scope = null)
		{
			keys.Add(hash.ToString(), IssueKeyType.Hash, scope);
		}

		/// <summary>
		/// Extracts a list of source files from an event
		/// </summary>
		/// <param name="keys">Set of keys</param>
		/// <param name="issueEvent">The event data</param>
		/// <param name="scope">The scope of the key to add</param>
		public static void AddSourceFiles(this HashSet<IssueKey> keys, IssueEvent issueEvent, string? scope = null)
		{
			foreach (JsonLogEvent line in issueEvent.Lines)
			{
				JsonDocument document = JsonDocument.Parse(line.Data);

				JsonElement properties;
				if (document.RootElement.TryGetProperty("properties", out properties) && properties.ValueKind == JsonValueKind.Object)
				{
					IssueKeyType type = IssueKeyType.File;
					if (properties.TryGetProperty("note", out JsonElement noteElement) && noteElement.GetBoolean())
					{
						type = IssueKeyType.Note;
					}

					foreach (JsonProperty property in properties.EnumerateObject())
					{
						if (property.NameEquals("file") && property.Value.ValueKind == JsonValueKind.String)
						{
							keys.AddSourceFile(property.Value.GetString()!, type, scope);
						}
						if (property.Value.HasStringProperty("$type", "SourceFile") && property.Value.TryGetStringProperty("relativePath", out string? value))
						{
							keys.AddSourceFile(value, type, scope);
						}
					}
				}
			}
		}

		/// <summary>
		/// Add a new source file to a list of unique source files
		/// </summary>
		/// <param name="keys">List of source files</param>
		/// <param name="relativePath">File to add</param>
		/// <param name="type">Type of key to add</param>
		/// <param name="scope">The scope of the key to add</param>
		public static void AddSourceFile(this HashSet<IssueKey> keys, string relativePath, IssueKeyType type, string? scope = null)
		{
			int endIdx = relativePath.LastIndexOfAny(['/', '\\']) + 1;

			string fileName = relativePath.Substring(endIdx);
			IssueKey key = new(fileName, type, scope);

			keys.Add(key);
		}

		/// <summary>
		/// Parses symbol names from a log event
		/// </summary>
		/// <param name="keys">List of source files</param>
		/// <param name="eventData">The log event data</param>
		public static void AddSymbols(this HashSet<IssueKey> keys, IssueEvent eventData)
		{
			foreach (JsonLogEvent line in eventData.Lines)
			{
				JsonDocument document = JsonDocument.Parse(line.Data);

				string? identifier;
				if (document.RootElement.TryGetNestedProperty("properties.symbol.identifier", out identifier))
				{
					IssueKey key = new(identifier, IssueKeyType.Symbol);
					keys.Add(key);
				}
			}
		}
	}
}
