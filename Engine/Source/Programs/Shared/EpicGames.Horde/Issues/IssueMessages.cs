// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;

#pragma warning disable CA2227 // Change collections to be read-only

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// The severity of an issue
	/// </summary>
	public enum IssueSeverity
	{
		/// <summary>
		/// Unspecified severity
		/// </summary>
		Unspecified,

		/// <summary>
		/// This error represents a warning
		/// </summary>
		Warning,

		/// <summary>
		/// This issue represents an error
		/// </summary>
		Error,
	}

	/// <summary>
	/// Identifies a particular changelist and job
	/// </summary>
	public class GetIssueStepResponse
	{
		/// <summary>
		/// The changelist number
		/// </summary>
		[Obsolete("Use Commit instead")]
		public int Change
		{
			get => _change ?? _commitId?.TryGetPerforceChange() ?? -1;
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// The commit for this step
		/// </summary>
		public CommitIdWithOrder CommitId
		{
			get => _commitId ?? CommitIdWithOrder.FromPerforceChange(_change) ?? CommitIdWithOrder.Empty;
			set => _commitId = value;
		}
		CommitIdWithOrder? _commitId;

		/// <summary>
		/// Severity of the issue in this step
		/// </summary>
		public IssueSeverity Severity { get; set; }

		/// <summary>
		/// Name of the job containing this step
		/// </summary>
		public string JobName { get; set; } = String.Empty;

		/// <summary>
		/// The unique job id
		/// </summary>
		public JobId JobId { get; set; }

		/// <summary>
		/// The unique batch id
		/// </summary>
		public JobStepBatchId BatchId { get; set; }

		/// <summary>
		/// The unique step id
		/// </summary>
		public JobStepId StepId { get; set; }

		/// <summary>
		/// Time at which the step ran
		/// </summary>
		public DateTime StepTime { get; set; }

		/// <summary>
		/// The unique log id
		/// </summary>
		public LogId? LogId { get; set; }
	}

	/// <summary>
	/// Trace of a set of node failures across multiple steps
	/// </summary>
	public class GetIssueSpanResponse
	{
		/// <summary>
		/// Unique id of this span
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The template containing this step
		/// </summary>
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// Name of the step
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Workflow that this span belongs to
		/// </summary>
		public WorkflowId? WorkflowId { get; set; }

		/// <summary>
		/// The previous build 
		/// </summary>
		public GetIssueStepResponse? LastSuccess { get; set; }

		/// <summary>
		/// The failing builds for a particular event
		/// </summary>
		public List<GetIssueStepResponse> Steps { get; set; } = [];

		/// <summary>
		/// The following successful build
		/// </summary>
		public GetIssueStepResponse? NextSuccess { get; set; }
	}

	/// <summary>
	/// Information about a particular step
	/// </summary>
	public class GetIssueStreamResponse
	{
		/// <summary>
		/// Unique id of the stream
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// Minimum commit affected by this issue (ie. last successful build)
		/// </summary>
		[Obsolete("Use MinCommitId instead")]
		public int? MinChange
		{
			get => _minChange ?? _minCommitId?.GetPerforceChangeOrMinusOne();
			set => _minChange = value;
		}
		int? _minChange;

		/// <summary>
		/// Maximum commit affected by this issue (ie. next successful build)
		/// </summary>
		[Obsolete("Use MaxCommitId instead")]
		public int? MaxChange
		{
			get => _maxChange ?? _maxCommitId?.GetPerforceChangeOrMinusOne();
			set => _maxChange = value;
		}
		int? _maxChange;

		/// <summary>
		/// Minimum changelist affected by this issue (ie. last successful build)
		/// </summary>
		public CommitIdWithOrder? MinCommitId
		{
			get => _minCommitId ?? CommitIdWithOrder.FromPerforceChange(_minChange);
			set => _minCommitId = value;
		}
		CommitIdWithOrder? _minCommitId;

		/// <summary>
		/// Maximum changelist affected by this issue (ie. next successful build)
		/// </summary>
		public CommitIdWithOrder? MaxCommitId
		{
			get => _maxCommitId ?? CommitIdWithOrder.FromPerforceChange(_maxChange);
			set => _maxCommitId = value;
		}
		CommitIdWithOrder? _maxCommitId;

		/// <summary>
		/// Map of steps to (event signature id -> trace id)
		/// </summary>
		public List<GetIssueSpanResponse> Nodes { get; set; } = [];
	}

	/// <summary>
	/// Outcome of a particular build
	/// </summary>
	public enum IssueBuildOutcome
	{
		/// <summary>
		/// Unknown outcome
		/// </summary>
		Unknown,

		/// <summary>
		/// Build succeeded
		/// </summary>
		Success,

		/// <summary>
		/// Build failed
		/// </summary>
		Error,

		/// <summary>
		/// Build finished with warnings
		/// </summary>
		Warning,
	}

	/// <summary>
	/// Information about a template affected by an issue
	/// </summary>
	public class GetIssueAffectedTemplateResponse
	{
		/// <summary>
		/// The template id
		/// </summary>
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// The template name
		/// </summary>
		public string TemplateName { get; set; } = String.Empty;

		/// <summary>
		/// Whether it has been resolved or not
		/// </summary>
		public bool Resolved { get; set; }

		/// <summary>
		/// The issue severity of the affected template
		/// </summary>
		public IssueSeverity Severity { get; set; }
	}

	/// <summary>
	/// Summary for the state of a stream in an issue
	/// </summary>
	public class GetIssueAffectedStreamResponse
	{
		/// <summary>
		/// Id of the stream
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// Name of the stream
		/// </summary>
		public string StreamName { get; set; } = String.Empty;

		/// <summary>
		/// Whether the issue has been resolved in this stream
		/// </summary>
		public bool Resolved { get; set; }

		/// <summary>
		/// The affected templates
		/// </summary>
		public List<GetIssueAffectedTemplateResponse> AffectedTemplates { get; set; } = [];

		/// <summary>
		/// List of affected template ids
		/// </summary>
		public List<string> TemplateIds { get; set; } = [];

		/// <summary>
		/// List of resolved template ids
		/// </summary>
		public List<string> ResolvedTemplateIds { get; set; } = [];

		/// <summary>
		/// List of unresolved template ids
		/// </summary>
		public List<string> UnresolvedTemplateIds { get; set; } = [];
	}

	/// <summary>
	/// Stores information about a build health issue
	/// </summary>
	public class GetIssueResponse
	{
		/// <summary>
		/// The unique object id
		/// </summary>
		public int Id { get; set; }

		/// <summary>
		/// Time at which the issue was created
		/// </summary>
		public DateTime CreatedAt { get; set; }

		/// <summary>
		/// Time at which the issue was retrieved
		/// </summary>
		public DateTime RetrievedAt { get; set; }

		/// <summary>
		/// The associated project for the issue
		/// </summary>
		public string? Project { get; set; }

		/// <summary>
		/// The summary text for this issue
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// Detailed description text
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Description of the current fingerprint used for issue identification
		/// </summary>
		public string? FingerprintDescription { get; set; }

		/// <summary>
		/// Severity of this issue
		/// </summary>
		public IssueSeverity Severity { get; set; }

		/// <summary>
		/// Whether the issue is promoted
		/// </summary>
		public bool Promoted { get; set; }

		/// <summary>
		/// Owner of the issue [DEPRECATED]
		/// </summary>
		public string? Owner { get; set; }

		/// <summary>
		/// User id of the owner [DEPRECATED]
		/// </summary>
		public string? OwnerId { get; set; }

		/// <summary>
		/// Owner of the issue
		/// </summary>
		public GetThinUserInfoResponse? OwnerInfo { get; set; }

		/// <summary>
		/// User that nominated the current owner [DEPRECATED]
		/// </summary>
		public string? NominatedBy { get; set; }

		/// <summary>
		/// Owner of the issue
		/// </summary>
		public GetThinUserInfoResponse? NominatedByInfo { get; set; }

		/// <summary>
		/// Time that the issue was acknowledged
		/// </summary>
		public DateTime? AcknowledgedAt { get; set; }

		/// <summary>
		/// Perforce changelist that fixed this issue
		/// </summary>
		public int? FixChange
		{
			get => _fixChange ?? _fixCommitId?.GetPerforceChangeOrMinusOne();
			set => _fixChange = value;
		}
		int? _fixChange;

		/// <summary>
		/// Commit that fixed this issue
		/// </summary>
		public CommitId? FixCommitId
		{
			get => _fixCommitId ?? CommitId.FromPerforceChange(_fixChange);
			set => _fixCommitId = value;
		}
		CommitId? _fixCommitId;

		/// <summary>
		/// Whether the issue is marked fixed as a systemic issue
		/// </summary>
		public bool FixSystemic
		{
			get => _fixSystemic ?? (_fixChange < 0);
			set => _fixSystemic = value;
		}
		bool? _fixSystemic;

		/// <summary>
		/// Commit that introduced this issue
		/// </summary>
		public CommitId? RootCommitId
		{
			get => _rootCommitId;
			set => _rootCommitId = value;
		}
		CommitId? _rootCommitId;

		/// <summary>
		/// User that provided the root cause analysis
		/// </summary>
		public GetThinUserInfoResponse? RootCauseOwnerInfo { get; set; }

		/// <summary>
		/// Root Cause Category for this issue
		/// </summary>
		public string? RootCauseCategory { get; set; }

		/// <summary>
		/// Root Cause Summary for this issue
		/// </summary>
		public string? RootCauseSummary { get; set; }

		/// <summary>
		/// Duplicate issue id
		/// </summary>
		public int? DuplicateIssueId { get; set; }

		/// <summary>
		/// Time at which the issue was resolved
		/// </summary>
		public DateTime? ResolvedAt { get; set; }

		/// <summary>
		/// Name of the user that resolved the issue [DEPRECATED]
		/// </summary>
		public string? ResolvedBy { get; set; }

		/// <summary>
		/// User id of the person that resolved the issue [DEPRECATED]
		/// </summary>
		public string? ResolvedById { get; set; }

		/// <summary>
		/// User that resolved the issue
		/// </summary>
		public GetThinUserInfoResponse? ResolvedByInfo { get; set; }

		/// <summary>
		/// Time at which the issue was verified
		/// </summary>
		public DateTime? VerifiedAt { get; set; }

		/// <summary>
		/// Time that the issue was last seen
		/// </summary>
		public DateTime LastSeenAt { get; set; }

		/// <summary>
		/// List of stream paths affected by this issue
		/// </summary>
		public List<string> Streams { get; set; } = [];

		/// <summary>
		/// List of affected stream ids
		/// </summary>
		public List<string> ResolvedStreams { get; set; } = [];

		/// <summary>
		/// List of unresolved streams
		/// </summary>
		public List<string> UnresolvedStreams { get; set; } = [];

		/// <summary>
		/// List of affected streams
		/// </summary>
		public List<GetIssueAffectedStreamResponse> AffectedStreams { get; set; } = [];

		/// <summary>
		/// Most likely suspects for causing this issue [DEPRECATED]
		/// </summary>
		public List<string>? PrimarySuspects { get; set; }

		/// <summary>
		/// User ids of the most likely suspects [DEPRECATED]
		/// </summary>
		public List<string>? PrimarySuspectIds { get; set; }

		/// <summary>
		/// Most likely suspects for causing this issue
		/// </summary>
		public List<GetThinUserInfoResponse> PrimarySuspectsInfo { get; set; } = [];

		/// <summary>
		/// Whether to show alerts for this issue
		/// </summary>
		public bool ShowDesktopAlerts { get; set; }

		/// <summary>
		/// Key for this issue in external issue tracker
		/// </summary>
		public string? ExternalIssueKey { get; set; }

		/// <summary>
		/// User who quarantined the issue
		/// </summary>
		public GetThinUserInfoResponse? QuarantinedByUserInfo { get; set; }

		/// <summary>
		/// The UTC time when the issue was quarantined
		/// </summary>
		public DateTime? QuarantineTimeUtc { get; set; }

		/// <summary>
		/// User who force closed the issue
		/// </summary>
		public GetThinUserInfoResponse? ForceClosedByUserInfo { get; set; }

		/// <summary>
		/// The workflow thread url for this issue
		/// </summary>
		public Uri? WorkflowThreadUrl { get; set; }
	}

	/// <summary>
	/// Information about a span within an issue
	/// </summary>
	public class FindIssueSpanResponse
	{
		/// <summary>
		/// Unique id of this span
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The template containing this step
		/// </summary>
		public string TemplateId { get; set; } = String.Empty;

		/// <summary>
		/// Name of the step
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Workflow for this span
		/// </summary>
		public WorkflowId? WorkflowId { get; set; }

		/// <summary>
		/// The previous build 
		/// </summary>
		public GetIssueStepResponse? LastSuccess { get; set; }

		/// <summary>
		/// The following successful build
		/// </summary>
		public GetIssueStepResponse? NextSuccess { get; set; }
	}

	/// <summary>
	/// Stores information about a build health issue
	/// </summary>
	public class FindIssueResponse
	{
		/// <summary>
		/// The unique object id
		/// </summary>
		public int Id { get; set; }

		/// <summary>
		/// Time at which the issue was created
		/// </summary>
		public DateTime CreatedAt { get; set; }

		/// <summary>
		/// Time at which the issue was retrieved
		/// </summary>
		public DateTime RetrievedAt { get; set; }

		/// <summary>
		/// The associated project for the issue
		/// </summary>
		public string? Project { get; set; }

		/// <summary>
		/// The summary text for this issue
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// Detailed description text
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Severity of this issue
		/// </summary>
		public IssueSeverity Severity { get; set; }

		/// <summary>
		/// Severity of this issue in the stream
		/// </summary>
		public IssueSeverity? StreamSeverity { get; set; }

		/// <summary>
		/// Whether the issue is promoted
		/// </summary>
		public bool Promoted { get; set; }

		/// <summary>
		/// Owner of the issue
		/// </summary>
		public GetThinUserInfoResponse? Owner { get; set; }

		/// <summary>
		/// Owner of the issue
		/// </summary>
		public GetThinUserInfoResponse? NominatedBy { get; set; }

		/// <summary>
		/// Time that the issue was acknowledged
		/// </summary>
		public DateTime? AcknowledgedAt { get; set; }

		/// <summary>
		/// Changelist that fixed this issue
		/// </summary>
		public int? FixChange
		{
			get => _fixChange ?? ((_fixSystemic ?? false) ? -1 : _fixCommitId?.GetPerforceChangeOrMinusOne());
			set => _fixChange = value;
		}
		int? _fixChange;

		/// <summary>
		/// Changelist that fixed this issue
		/// </summary>
		public CommitId? FixCommitId
		{
			get => _fixCommitId ?? CommitId.FromPerforceChange(_fixChange);
			set => _fixCommitId = value;
		}
		CommitId? _fixCommitId;

		/// <summary>
		/// User that provided the root cause analysis
		/// </summary>
		public GetThinUserInfoResponse? RootCauseOwner { get; set; }

		/// <summary>
		/// Root Cause Category for this issue
		/// </summary>
		public string? RootCauseCategory { get; set; }

		/// <summary>
		/// Root Cause Summary for this issue
		/// </summary>
		public string? RootCauseSummary { get; set; }

		/// <summary>
		/// Duplcate issue id
		/// </summary>
		public int? DuplicateIssueId { get; set; }

		/// <summary>
		/// Whether the issue is marked fixed as a systemic issue
		/// </summary>
		public bool FixSystemic
		{
			get => _fixSystemic ?? (_fixChange < 0);
			set => _fixSystemic = value;
		}
		bool? _fixSystemic;

		/// <summary>
		/// Changelist that introduced this issue
		/// </summary>
		public CommitId? RootCommitId
		{
			get => _rootCommitId;
			set => _rootCommitId = value;
		}
		CommitId? _rootCommitId;

		/// <summary>
		/// Time at which the issue was resolved
		/// </summary>
		public DateTime? ResolvedAt { get; set; }

		/// <summary>
		/// User that resolved the issue
		/// </summary>
		public GetThinUserInfoResponse? ResolvedBy { get; set; }

		/// <summary>
		/// Time at which the issue was verified
		/// </summary>
		public DateTime? VerifiedAt { get; set; }

		/// <summary>
		/// Time that the issue was last seen
		/// </summary>
		public DateTime LastSeenAt { get; set; }

		/// <summary>
		/// Spans for this issue
		/// </summary>
		public List<FindIssueSpanResponse>? Spans { get; set; }

		/// <summary>
		/// Key for this issue in external issue tracker
		/// </summary>
		public string? ExternalIssueKey { get; set; }

		/// <summary>
		/// User who quarantined the issue
		/// </summary>
		public GetThinUserInfoResponse? QuarantinedBy { get; set; }

		/// <summary>
		/// The UTC time when the issue was quarantined
		/// </summary>
		public DateTime? QuarantineTimeUtc { get; set; }

		/// <summary>
		/// The workflow thread url for this issue
		/// </summary>
		public Uri? WorkflowThreadUrl { get; set; }

		/// <summary>
		/// Workflows for which this issue is open
		/// </summary>
		public List<WorkflowId>? OpenWorkflows { get; set; }
	}

	/// <summary>
	/// Request an issue to be updated
	/// </summary>
	public class UpdateIssueRequest
	{
		/// <summary>
		/// Summary of the issue
		/// </summary>
		public string? Summary { get; set; }

		/// <summary>
		/// Description of the issue
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Whether the issue is promoted or not
		/// </summary>
		public bool? Promoted { get; set; }

		/// <summary>
		/// New user id for owner of the issue, can be cleared by passing empty string
		/// </summary>
		public string? OwnerId { get; set; }

		/// <summary>
		/// User id that nominated the new owner
		/// </summary>
		public string? NominatedById { get; set; }

		/// <summary>
		/// Whether the issue has been acknowledged
		/// </summary>
		public bool? Acknowledged { get; set; }

		/// <summary>
		/// Whether the user has declined this issue
		/// </summary>
		public bool? Declined { get; set; }

		/// <summary>
		/// The change at which the issue is claimed fixed. 0 = not fixed, -1 = systemic issue.
		/// </summary>
		[Obsolete("Use FixCommitId and FixSystemic instead")]
		public int? FixChange
		{
			get => _fixChange ?? ((_fixSystemic ?? false) ? -1 : _fixCommitId?.GetPerforceChangeOrMinusOne());
			set => _fixChange = value;
		}
		int? _fixChange;

		/// <summary>
		/// The change at which the issue is claimed fixed. """" = not fixed.
		/// </summary>
		public CommitId? FixCommitId
		{
			get => _fixCommitId ?? ((_fixChange < 0) ? null : (_fixChange == 0) ? CommitId.Empty : CommitId.FromPerforceChange(_fixChange));
			set => _fixCommitId = value;
		}
		CommitId? _fixCommitId;

		/// <summary>
		/// Set to mark the issue as fixed systemically
		/// </summary>
		public bool FixSystemic
		{
			get => _fixSystemic ?? (_fixChange < 0);
			set => _fixSystemic = value;
		}
		bool? _fixSystemic;

		/// <summary>
		/// The change at which the issue is claimed to have been introduced.
		/// </summary>
		public CommitId? RootCommitId
		{
			get => _rootCommitId == null ? null : String.IsNullOrWhiteSpace(_rootCommitId.Name) ? CommitId.Empty : _rootCommitId;
			set => _rootCommitId = value;
		}
		CommitId? _rootCommitId;

		/// <summary>
		/// New user id for owner of the root cause analysis, can be cleared by passing empty string
		/// </summary>
		public string? RootCauseOwnerId { get; set; }

		/// <summary>
		/// Root Cause Category for this issue
		/// </summary>
		public string? RootCauseCategory { get; set; }

		/// <summary>
		/// Root Cause Summary for this issue
		/// </summary>
		public string? RootCauseSummary { get; set; }

		/// <summary>
		/// Duplicate Issue Id
		/// </summary>
		public int? DuplicateIssueId { get; set; }

		/// <summary>
		/// Whether the issue should be marked as resolved
		/// </summary>
		public bool? Resolved { get; set; }

		/// <summary>
		/// List of spans to add to this issue
		/// </summary>
		public List<string>? AddSpans { get; set; }

		/// <summary>
		/// List of spans to remove from this issue
		/// </summary>
		public List<string>? RemoveSpans { get; set; }

		/// <summary>
		/// A key to issue in external tracker
		/// </summary>
		public string? ExternalIssueKey { get; set; }

		/// <summary>
		/// Id of user quarantining issue
		/// </summary>
		public string? QuarantinedById { get; set; }

		/// <summary>
		/// Id of user who is forcibly closing this issue, skipping verification checks.  This is useful for when a failing step has been removed for example
		/// </summary>
		public string? ForceClosedById { get; set; }
	}

	/// <summary>
	/// External issue project information
	/// </summary>
	public class GetExternalIssueProjectResponse
	{
		/// <summary>
		/// The project key
		/// </summary>
		public string ProjectKey { get; set; } = String.Empty;

		/// <summary>
		/// The name of the project
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The id of the project
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// component id => name
		/// </summary>
		public Dictionary<string, string> Components { get; set; } = [];

		/// <summary>
		/// IssueType id => name
		/// </summary>
		public Dictionary<string, string> IssueTypes { get; set; } = [];
	}

	/// <summary>
	/// Marks an issue as fixed by another user. Designed for use from a Perforce trigger.
	/// </summary>
	public class MarkFixedViaPerforceRequest
	{
		/// <summary>
		/// Name of the user that fixed the issue
		/// </summary>
		public string UserName { get; set; } = String.Empty;

		/// <summary>
		/// Change that fixed the issue
		/// </summary>
		public int FixChange { get; set; }
	}

	/// <summary>
	/// Request an issue to be created on external issue tracking system
	/// </summary>
	public class CreateExternalIssueRequest
	{
		/// <summary>
		/// Horde issue which is linked to external issue
		/// </summary>
		public int IssueId { get; set; } = 0;

		/// <summary>
		/// StreamId of a stream with this issue
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// Summary text for external issue
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// External issue project id
		/// </summary>
		public string ProjectId { get; set; } = String.Empty;

		/// <summary>
		/// External issue component id
		/// </summary>
		public string ComponentId { get; set; } = String.Empty;

		/// <summary>
		/// External issue type id
		/// </summary>
		public string IssueTypeId { get; set; } = String.Empty;

		/// <summary>
		/// Optional description text for external issue
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Optional link to issue on Horde
		/// </summary>
		public string? HordeIssueLink { get; set; }
	}

	/// <summary>
	/// Response for externally created issue
	/// </summary>
	public class CreateExternalIssueResponse
	{
		/// <summary>
		/// External issue key 
		/// </summary>
		public string Key { get; set; }

		/// <summary>
		/// Link to issue on external tracking site
		/// </summary>
		public string? Link { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="key"></param>
		/// <param name="link"></param>
		public CreateExternalIssueResponse(string key, string? link)
		{
			Key = key;
			Link = link;
		}
	}

	/// <summary>
	/// External issue response object
	/// </summary>
	public class GetExternalIssueResponse
	{
		/// <summary>
		/// The external issue key
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The issue link on external tracking site
		/// </summary>
		public string? Link { get; set; }

		/// <summary>
		/// The issue status name, "To Do", "In Progress", etc
		/// </summary>
		public string? StatusName { get; set; }

		/// <summary>
		/// The issue resolution name, "Fixed", "Closed", etc
		/// </summary>
		public string? ResolutionName { get; set; }

		/// <summary>
		/// The issue priority name, "1 - Critical", "2 - Major", etc
		/// </summary>
		public string? PriorityName { get; set; }

		/// <summary>
		/// The current assignee's user name
		/// </summary>
		public string? AssigneeName { get; set; }

		/// <summary>
		/// The current assignee's display name
		/// </summary>
		public string? AssigneeDisplayName { get; set; }

		/// <summary>
		/// The current assignee's email address
		/// </summary>
		public string? AssigneeEmailAddress { get; set; }
	}
}
