// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Entry in an issue's metadata collection. Implemented as a case-insensitive key and value
	/// </summary>
	public class IssueMetadata : IEquatable<IssueMetadata>
	{
		/// <summary>
		/// Key for the metadata item
		/// </summary>
		public string Key { get; }

		/// <summary>
		/// Value for the metadata
		/// </summary>
		public string Value { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		[JsonConstructor]
		public IssueMetadata(string key, string value)
		{
			Key = key;
			Value = value;
		}

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(Key.GetHashCode(StringComparison.OrdinalIgnoreCase), Value.GetHashCode(StringComparison.OrdinalIgnoreCase));

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is IssueMetadata entry && Equals(entry);

		/// <inheritdoc/>
		public bool Equals(IssueMetadata? entry) => entry is not null && Key.Equals(entry.Key, StringComparison.OrdinalIgnoreCase) && Value.Equals(entry.Value, StringComparison.OrdinalIgnoreCase);

		/// <inheritdoc/>
		public override string ToString() => $"{Key}={Value}";
	}

	/// <summary>
	/// Extension methods for <see cref="IssueMetadata"/>
	/// </summary>
	public static class IssueMetadataEntryExtensions
	{
		/// <summary>
		/// Adds a new entry to a set
		/// </summary>
		public static void Add(this HashSet<IssueMetadata> entries, string key, string value)
		{
			entries.Add(new IssueMetadata(key, value));
		}

		/// <summary>
		/// Gets all the metadata values with a given key
		/// </summary>
		/// <param name="entries">Set of entries to search</param>
		/// <param name="key">Key name to search for</param>
		/// <returns>All values with the given key</returns>
		public static IEnumerable<string> FindValues(this IReadOnlySet<IssueMetadata> entries, string key)
		{
			foreach (IssueMetadata entry in entries)
			{
				if (entry.Key.Equals(key, StringComparison.OrdinalIgnoreCase))
				{
					yield return entry.Value;
				}
			}
		}
	}
}
