// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Identifier for a workflow
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<WorkflowId, WorkflowIdConverter>))]
	[StringIdConverter(typeof(WorkflowIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<WorkflowId, WorkflowIdConverter>))]
	public readonly record struct WorkflowId(StringId Id)
	{
		/// <summary>
		/// Empty workflow id constant
		/// </summary>
		public static WorkflowId Empty { get; } = default;

		/// <summary>
		/// Constructor
		/// </summary>
		public WorkflowId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class WorkflowIdConverter : StringIdConverter<WorkflowId>
	{
		/// <inheritdoc/>
		public override WorkflowId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(WorkflowId value) => value.Id;
	}
}
