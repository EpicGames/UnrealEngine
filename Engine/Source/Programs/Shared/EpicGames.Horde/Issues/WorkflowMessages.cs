// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Horde.Jobs.Graphs;

#pragma warning disable CA2227 //  Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Issues
{
	/// <summary>
	/// Configuration for an issue workflow
	/// </summary>
	public class GetWorkflowResponse : IWorkflow
	{
		/// <inheritdoc/>
		public WorkflowId Id { get; set; }

		/// <inheritdoc cref="IWorkflow.ReportTimes"/>
		public List<TimeSpan> ReportTimes { get; set; }
		IReadOnlyList<TimeSpan> IWorkflow.ReportTimes => ReportTimes;

		/// <inheritdoc/>
		public string? SummaryTab { get; set; }

		/// <inheritdoc/>
		public string? ReportChannel { get; set; }

		/// <inheritdoc/>
		public bool ReportWarnings { get; set; } = true;

		/// <inheritdoc/>
		public bool GroupIssuesByTemplate { get; set; } = true;

		/// <inheritdoc/>
		public string? TriageChannel { get; set; }

		/// <inheritdoc/>
		public string? TriagePrefix { get; set; }

		/// <inheritdoc/>
		public string? TriageSuffix { get; set; }

		/// <inheritdoc/>
		public string? TriageInstructions { get; set; }

		/// <inheritdoc/>
		public string? TriageAlias { get; set; }

		/// <inheritdoc/>
		public bool TriageErrors { get; set; } = true;

		/// <inheritdoc/>
		public bool TriageWarnings { get; set; } = true;

		/// <inheritdoc cref="IWorkflow.TriageTypeAliases"/>
		public Dictionary<string, string>? TriageTypeAliases { get; set; }
		IReadOnlyDictionary<string, string>? IWorkflow.TriageTypeAliases => TriageTypeAliases;

		/// <inheritdoc/>
		public string? EscalateAlias { get; set; }

		/// <inheritdoc cref="IWorkflow.EscalateTimes"/>
		public List<int> EscalateTimes { get; set; }
		IReadOnlyList<int> IWorkflow.EscalateTimes => EscalateTimes;

		/// <inheritdoc cref="IWorkflow.EscalateRootCauseAnalysisTimes"/>
		public List<int> EscalateRootCauseAnalysisTimes { get; set; }

		IReadOnlyList<int> IWorkflow.EscalateRootCauseAnalysisTimes => EscalateRootCauseAnalysisTimes;

		/// <inheritdoc/>
		public bool AutoRcaEnabled { get; set; }

		/// <inheritdoc/>
		public bool AutoRcaSlackEnabled { get; set; }

		/// <inheritdoc/>
		public int MaxMentions { get; set; }

		/// <inheritdoc/>
		public bool AllowMentions { get; set; }

		/// <inheritdoc/>
		public bool InviteRestrictedUsers { get; set; }

		/// <inheritdoc/>
		public bool SkipWhenEmpty { get; set; }

		/// <inheritdoc/>
		public bool ShowMergeWarnings { get; set; }

		/// <inheritdoc cref="IWorkflow.Annotations"/>
		public NodeAnnotations Annotations { get; set; } = [];
		IReadOnlyNodeAnnotations IWorkflow.Annotations => Annotations;

		/// <inheritdoc cref="IWorkflow.ExternalIssues"/>
		public GetWorkflowExternalIssuesResponse? ExternalIssues { get; set; }
		IWorkflowExternalIssues? IWorkflow.ExternalIssues => ExternalIssues;

		/// <inheritdoc cref="IWorkflow.IssueHandlers"/>
		public List<string>? IssueHandlers { get; set; }
		IReadOnlyList<string>? IWorkflow.IssueHandlers => IssueHandlers;

		/// <summary>
		/// Constructor
		/// </summary>
		public GetWorkflowResponse()
		{
			ReportTimes = [];
			EscalateTimes = [];
			EscalateRootCauseAnalysisTimes = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetWorkflowResponse(IWorkflow workflow)
		{
			Id = workflow.Id;
			ReportTimes = [.. workflow.ReportTimes];
			SummaryTab = workflow.SummaryTab;
			ReportChannel = workflow.ReportChannel;
			ReportWarnings = workflow.ReportWarnings;
			GroupIssuesByTemplate = workflow.GroupIssuesByTemplate;
			TriageChannel = workflow.TriageChannel;
			TriagePrefix = workflow.TriagePrefix;
			TriageSuffix = workflow.TriageSuffix;
			TriageInstructions = workflow.TriageInstructions;
			TriageAlias = workflow.TriageAlias;
			TriageTypeAliases = workflow.TriageTypeAliases?.ToDictionary();
			TriageWarnings = workflow.TriageWarnings;
			TriageErrors = workflow.TriageErrors;
			EscalateAlias = workflow.EscalateAlias;
			EscalateTimes = [.. workflow.EscalateTimes];
			EscalateRootCauseAnalysisTimes = [.. workflow.EscalateRootCauseAnalysisTimes];
			AutoRcaEnabled = workflow.AutoRcaEnabled;
			AutoRcaSlackEnabled = workflow.AutoRcaSlackEnabled;
			MaxMentions = workflow.MaxMentions;
			AllowMentions = workflow.AllowMentions;
			InviteRestrictedUsers = workflow.InviteRestrictedUsers;
			SkipWhenEmpty = workflow.SkipWhenEmpty;
			ShowMergeWarnings = workflow.ShowMergeWarnings;
			Annotations = new NodeAnnotations(workflow.Annotations);
			ExternalIssues = (workflow.ExternalIssues == null) ? null : new GetWorkflowExternalIssuesResponse(workflow.ExternalIssues);
			IssueHandlers = workflow.IssueHandlers?.ToList();
		}
	}

	/// <summary>
	/// External issue tracking configuration for a workflow
	/// </summary>
	public class GetWorkflowExternalIssuesResponse : IWorkflowExternalIssues
	{
		/// <inheritdoc/>
		public string ProjectKey { get; set; } = String.Empty;

		/// <inheritdoc/>
		public string DefaultComponentId { get; set; } = String.Empty;

		/// <inheritdoc/>
		public string DefaultIssueTypeId { get; set; } = String.Empty;

		/// <summary>
		/// Constructor
		/// </summary>
		public GetWorkflowExternalIssuesResponse()
		{
			ProjectKey = String.Empty;
			DefaultComponentId = String.Empty;
			DefaultIssueTypeId = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetWorkflowExternalIssuesResponse(IWorkflowExternalIssues issues)
		{
			ProjectKey = issues.ProjectKey;
			DefaultComponentId = issues.DefaultComponentId;
			DefaultIssueTypeId = issues.DefaultIssueTypeId;
		}
	}
}
