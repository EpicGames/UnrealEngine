// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Jobs.Bisect
{
	/// <summary>
	/// Identifier for a job
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<BisectTaskId, BisectTaskIdConverter>))]
	[BinaryIdConverter(typeof(BisectTaskIdConverter))]
	public readonly record struct BisectTaskId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static BisectTaskId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class BisectTaskIdConverter : BinaryIdConverter<BisectTaskId>
	{
		/// <inheritdoc/>
		public override BisectTaskId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(BisectTaskId value) => value.Id;
	}
}
