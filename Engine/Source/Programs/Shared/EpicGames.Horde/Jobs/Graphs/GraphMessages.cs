// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

#pragma warning disable CA2227 // Change 'InputDependencies' to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.Graphs
{
	/// <summary>
	/// Information required to create a node
	/// </summary>
	public class GetNodeResponse
	{
		/// <summary>
		/// The name of this node 
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Inputs for this node
		/// </summary>
		public List<string> Inputs { get; set; } = [];

		/// <summary>
		/// Optional inputs for this node
		/// </summary>
		public List<string> OptionalInputs { get; set; } = [];

		/// <summary>
		/// Output from this node
		/// </summary>
		public List<string> Outputs { get; set; } = [];

		/// <summary>
		/// Indices of nodes which must have succeeded for this node to run
		/// </summary>
		public List<string> InputDependencies { get; set; } = [];

		/// <summary>
		/// Indices of nodes which may have succeeded for this node to run
		/// </summary>
		public List<string> OptionalInputDependencies { get; set; } = [];

		/// <summary>
		/// Indices of nodes which must have completed for this node to run
		/// </summary>
		public List<string> OrderDependencies { get; set; } = [];

		/// <summary>
		/// The priority of this node
		/// </summary>
		public Priority Priority { get; set; }

		/// <summary>
		/// Whether this node can be retried
		/// </summary>
		public bool AllowRetry { get; set; }

		/// <summary>
		/// This node can start running early, before dependencies of other nodes in the same group are complete
		/// </summary>
		public bool RunEarly { get; set; }

		/// <summary>
		/// Whether to include warnings in diagnostic output
		/// </summary>
		public bool Warnings { get; set; }

		/// <summary>
		/// Average time to execute this node based on historical trends
		/// </summary>
		public float? AverageDuration { get; set; }

		/// <summary>
		/// Credentials required for this node to run. This dictionary maps from environment variable names to a credential property in the format 'CredentialName.PropertyName'.
		/// </summary>
		public IReadOnlyDictionary<string, string>? Credentials { get; set; }

		/// <summary>
		/// Properties for this node
		/// </summary>
		public IReadOnlyDictionary<string, string>? Properties { get; set; }

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public IReadOnlyDictionary<string, string>? Annotations { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		public GetNodeResponse(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Information about a group of nodes
	/// </summary>
	public class GetGroupResponse
	{
		/// <summary>
		/// The type of agent to execute this group
		/// </summary>
		public string AgentType { get; set; }

		/// <summary>
		/// Nodes in the group
		/// </summary>
		public List<GetNodeResponse> Nodes { get; set; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public GetGroupResponse(string agentType)
		{
			AgentType = agentType;
		}
	}

	/// <summary>
	/// Information about an aggregate
	/// </summary>
	public class GetAggregateResponse
	{
		/// <summary>
		/// Name of the aggregate
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Nodes which must be part of the job for the aggregate to be shown
		/// </summary>
		public List<string> Nodes { get; set; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public GetAggregateResponse(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Information about a label
	/// </summary>
	public class GetLabelResponse
	{
		/// <summary>
		/// Category of the aggregate
		/// </summary>
		public string? Category => DashboardCategory;

		/// <summary>
		/// Label for this aggregate
		/// </summary>
		public string? Name => DashboardName;

		/// <summary>
		/// Name to show for this label on the dashboard
		/// </summary>
		public string? DashboardName { get; set; }

		/// <summary>
		/// Category to show this label in on the dashboard
		/// </summary>
		public string? DashboardCategory { get; set; }

		/// <summary>
		/// Name to show for this label in UGS
		/// </summary>
		public string? UgsName { get; set; }

		/// <summary>
		/// Project to display this label for in UGS
		/// </summary>
		public string? UgsProject { get; set; }

		/// <summary>
		/// Nodes which must be part of the job for the aggregate to be shown
		/// </summary>
		public List<string> RequiredNodes { get; set; } = [];

		/// <summary>
		/// Nodes to include in the status of this aggregate, if present in the job
		/// </summary>
		public List<string> IncludedNodes { get; set; } = [];
	}

	/// <summary>
	/// Information about a graph
	/// </summary>
	public class GetGraphResponse
	{
		/// <summary>
		/// The hash of the graph
		/// </summary>
		public string Hash { get; set; }

		/// <summary>
		/// Array of nodes for this job
		/// </summary>
		public List<GetGroupResponse>? Groups { get; set; }

		/// <summary>
		/// List of aggregates
		/// </summary>
		public List<GetAggregateResponse>? Aggregates { get; set; }

		/// <summary>
		/// List of labels for the graph
		/// </summary>
		public List<GetLabelResponse>? Labels { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetGraphResponse(string hash)
		{
			Hash = hash;
		}
	}
}
