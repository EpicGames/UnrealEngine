// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.Graphs
{
	/// <summary>
	/// A unique dependency graph instance
	/// </summary>
	public interface IGraph
	{
		/// <summary>
		/// Hash of this graph
		/// </summary>
		public ContentHash Id { get; }

		/// <summary>
		/// Schema version for this document
		/// </summary>
		public int Schema { get; }

		/// <summary>
		/// List of groups for this graph
		/// </summary>
		public IReadOnlyList<INodeGroup> Groups { get; }

		/// <summary>
		/// List of aggregates for this graph
		/// </summary>
		public IReadOnlyList<IAggregate> Aggregates { get; }

		/// <summary>
		/// Status labels for this graph
		/// </summary>
		public IReadOnlyList<ILabel> Labels { get; }

		/// <summary>
		/// Artifacts for this graph
		/// </summary>
		public IReadOnlyList<IGraphArtifact> Artifacts { get; }
	}

	/// <summary>
	/// Extension methods for graphs
	/// </summary>
	public static class GraphExtensions
	{
		/// <summary>
		/// Gets the node from a node reference
		/// </summary>
		/// <param name="graph">The graph instance</param>
		/// <param name="nodeRef">The node reference</param>
		/// <returns>The node for the given reference</returns>
		public static INode GetNode(this IGraph graph, NodeRef nodeRef)
		{
			return graph.Groups[nodeRef.GroupIdx].Nodes[nodeRef.NodeIdx];
		}

		/// <summary>
		/// Tries to find a node by name
		/// </summary>
		/// <param name="graph">The graph to search</param>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="nodeRef">Receives the node reference</param>
		/// <returns>True if the node was found, false otherwise</returns>
		public static bool TryFindNode(this IGraph graph, string nodeName, out NodeRef nodeRef)
		{
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
				{
					INode node = group.Nodes[nodeIdx];
					if (String.Equals(node.Name, nodeName, StringComparison.OrdinalIgnoreCase))
					{
						nodeRef = new NodeRef(groupIdx, nodeIdx);
						return true;
					}
				}
			}

			nodeRef = new NodeRef(0, 0);
			return false;
		}

		/// <summary>
		/// Tries to find a node by name
		/// </summary>
		/// <param name="graph">The graph to search</param>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="node">Receives the node</param>
		/// <returns>True if the node was found, false otherwise</returns>
		public static bool TryFindNode(this IGraph graph, string nodeName, [NotNullWhen(true)] out INode? node)
		{
			NodeRef nodeRef;
			if (TryFindNode(graph, nodeName, out nodeRef))
			{
				node = graph.Groups[nodeRef.GroupIdx].Nodes[nodeRef.NodeIdx];
				return true;
			}
			else
			{
				node = null;
				return false;
			}
		}

		/// <summary>
		/// Tries to find a node by name
		/// </summary>
		/// <param name="graph">The graph to search</param>
		/// <param name="name">Name of the node</param>
		/// <param name="aggregateIdx">Receives the aggregate index</param>
		/// <returns>True if the node was found, false otherwise</returns>
		public static bool TryFindAggregate(this IGraph graph, string name, out int aggregateIdx)
		{
			aggregateIdx = graph.Aggregates.FindIndex(x => x.Name.Equals(name, StringComparison.OrdinalIgnoreCase));
			return aggregateIdx != -1;
		}

		/// <summary>
		/// Tries to find a node by name
		/// </summary>
		/// <param name="graph">The graph to search</param>
		/// <param name="name">Name of the node</param>
		/// <param name="aggregate">Receives the aggregate</param>
		/// <returns>True if the node was found, false otherwise</returns>
		public static bool TryFindAggregate(this IGraph graph, string name, [NotNullWhen(true)] out IAggregate? aggregate)
		{
			int aggregateIdx;
			if (TryFindAggregate(graph, name, out aggregateIdx))
			{
				aggregate = graph.Aggregates[aggregateIdx];
				return true;
			}
			else
			{
				aggregate = null;
				return false;
			}
		}

		/// <summary>
		/// Gets a list of dependencies for the given node
		/// </summary>
		/// <param name="graph">The graph instance</param>
		/// <param name="node">The node to return dependencies for</param>
		/// <returns>List of dependencies</returns>
		public static IEnumerable<INode> GetDependencies(this IGraph graph, INode node)
		{
			return Enumerable.Concat(node.InputDependencies, node.OrderDependencies).Concat(node.OptionalInputDependencies).Select(x => graph.GetNode(x));
		}
	}

	/// <summary>
	/// Represents a node in the graph
	/// </summary>
	public interface INode
	{
		/// <summary>
		/// The name of this node 
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// References to inputs for this node
		/// </summary>
		public IReadOnlyList<NodeOutputRef> Inputs { get; }

		/// <summary>
		/// References to optional inputs for this node
		/// </summary>
		public IReadOnlyList<NodeOutputRef> OptionalInputs { get; }

		/// <summary>
		/// List of output names
		/// </summary>
		public IReadOnlyList<string> OutputNames { get; }

		/// <summary>
		/// Indices of nodes which must have succeeded for this node to run
		/// </summary>
		public NodeRef[] InputDependencies { get; }

		/// <summary>
		/// Indices of nodes which may have succeeded for this node to run
		/// </summary>
		public NodeRef[] OptionalInputDependencies { get; }

		/// <summary>
		/// Indices of nodes which must have completed for this node to run
		/// </summary>
		public NodeRef[] OrderDependencies { get; }

		/// <summary>
		/// The priority that this node should be run at, within this job
		/// </summary>
		public Priority Priority { get; }

		/// <summary>
		/// Whether this node can be run multiple times
		/// </summary>
		public bool AllowRetry { get; }

		/// <summary>
		/// This node can start running early, before dependencies of other nodes in the same group are complete
		/// </summary>
		public bool RunEarly { get; }

		/// <summary>
		/// Whether to include warnings in the output (defaults to true)
		/// </summary>
		public bool Warnings { get; }

		/// <summary>
		/// List of credentials required for this node. Each entry maps an environment variable name to a credential in the form "CredentialName.PropertyName".
		/// </summary>
		public IReadOnlyDictionary<string, string>? Credentials { get; }

		/// <summary>
		/// Properties for this node
		/// </summary>
		public IReadOnlyDictionary<string, string>? Properties { get; }

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public IReadOnlyNodeAnnotations Annotations { get; }
	}

	/// <summary>
	/// Information about a sequence of nodes which can execute on a single agent
	/// </summary>
	public interface INodeGroup
	{
		/// <summary>
		/// The type of agent to execute this group
		/// </summary>
		public string AgentType { get; }

		/// <summary>
		/// Nodes in this group
		/// </summary>
		public IReadOnlyList<INode> Nodes { get; }
	}

	/// <summary>
	/// Reference to a node within another grup
	/// </summary>
	[DebuggerDisplay("Group: {GroupIdx}, Node: {NodeIdx}")]
	public class NodeRef
	{
		/// <summary>
		/// The group index of the referenced node
		/// </summary>
		public int GroupIdx { get; set; }

		/// <summary>
		/// The node index of the referenced node
		/// </summary>
		public int NodeIdx { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private NodeRef()
		{
			GroupIdx = 0;
			NodeIdx = 0;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="groupIdx">Index of thr group containing the node</param>
		/// <param name="nodeIdx">Index of the node within the group</param>
		public NodeRef(int groupIdx, int nodeIdx)
		{
			GroupIdx = groupIdx;
			NodeIdx = nodeIdx;
		}

		/// <inheritdoc/>
		public override bool Equals(object? other)
		{
			NodeRef? otherNodeRef = other as NodeRef;
			return otherNodeRef != null && otherNodeRef.GroupIdx == GroupIdx && otherNodeRef.NodeIdx == NodeIdx;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return HashCode.Combine(GroupIdx, NodeIdx);
		}

		/// <summary>
		/// Converts this reference to a node name
		/// </summary>
		/// <param name="groups">List of groups that this reference points to</param>
		/// <returns>Name of the referenced node</returns>
		public INode ToNode(IReadOnlyList<INodeGroup> groups)
		{
			return groups[GroupIdx].Nodes[NodeIdx];
		}
	}

	/// <summary>
	/// Output from a node
	/// </summary>
	[DebuggerDisplay("{NodeRef}, Output: {OutputIdx}")]
	public class NodeOutputRef
	{
		/// <summary>
		/// Node producing the output
		/// </summary>
		public NodeRef NodeRef { get; set; }

		/// <summary>
		/// Index of the output
		/// </summary>
		public int OutputIdx { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public NodeOutputRef(NodeRef nodeRef, int outputIdx)
		{
			NodeRef = nodeRef;
			OutputIdx = outputIdx;
		}

		/// <inheritdoc/>
		public override bool Equals(object? other) => other is NodeOutputRef otherRef && otherRef.NodeRef == NodeRef && otherRef.OutputIdx == OutputIdx;

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(NodeRef.GetHashCode(), OutputIdx);
	}

	/// <summary>
	/// An collection of node references
	/// </summary>
	public interface IAggregate
	{
		/// <summary>
		/// Name of the aggregate
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// List of nodes for the aggregate to be valid
		/// </summary>
		public IReadOnlyList<NodeRef> Nodes { get; }
	}

	/// <summary>
	/// Change at which to display a label
	/// </summary>
	public enum LabelChange
	{
		/// <summary>
		/// The current changelist
		/// </summary>
		Current = 0,

		/// <summary>
		/// The last code changelist
		/// </summary>
		Code = 1
	}

	/// <summary>
	/// Label indicating the status of a set of nodes
	/// </summary>
	public interface ILabel
	{
		/// <summary>
		/// Label to show in the dashboard. Null if does not need to be shown.
		/// </summary>
		public string? DashboardName { get; }

		/// <summary>
		/// Category for the label. May be null.
		/// </summary>
		public string? DashboardCategory { get; }

		/// <summary>
		/// Name to display for this label in UGS
		/// </summary>
		public string? UgsName { get; }

		/// <summary>
		/// Project which this label applies to, for UGS
		/// </summary>
		public string? UgsProject { get; }

		/// <summary>
		/// Which change to display the label on
		/// </summary>
		public LabelChange Change { get; }

		/// <summary>
		/// List of required nodes for the aggregate to be valid
		/// </summary>
		public List<NodeRef> RequiredNodes { get; }

		/// <summary>
		/// List of optional nodes to include in the aggregate state
		/// </summary>
		public List<NodeRef> IncludedNodes { get; }
	}

	/// <summary>
	/// Extension methods for ILabel
	/// </summary>
	public static class LabelExtensions
	{
		/// <summary>
		/// Enumerate all the required dependencies of this node group
		/// </summary>
		/// <param name="label">The label instance</param>
		/// <param name="groups">List of groups for the job containing this aggregate</param>
		/// <returns>Sequence of nodes</returns>
		public static IEnumerable<INode> GetDependencies(this ILabel label, IReadOnlyList<INodeGroup> groups)
		{
			foreach (NodeRef requiredNode in label.RequiredNodes)
			{
				yield return requiredNode.ToNode(groups);
			}
			foreach (NodeRef includedNode in label.IncludedNodes)
			{
				yield return includedNode.ToNode(groups);
			}
		}
	}

	/// <summary>
	/// Artifact produced by a graph
	/// </summary>
	public interface IGraphArtifact
	{
		/// <summary>
		/// Name of the artifact
		/// </summary>
		public ArtifactName Name { get; }

		/// <summary>
		/// Type of the artifact
		/// </summary>
		public ArtifactType Type { get; }

		/// <summary>
		/// Description for the artifact
		/// </summary>
		public string Description { get; }

		/// <summary>
		/// Base path for files in the artifact
		/// </summary>
		public string BasePath { get; }

		/// <summary>
		/// Keys for finding the artifact
		/// </summary>
		public IReadOnlyList<string> Keys { get; }

		/// <summary>
		/// Metadata for the artifact
		/// </summary>
		public IReadOnlyList<string> Metadata { get; }

		/// <summary>
		/// Name of the node producing this artifact
		/// </summary>
		public string? NodeName { get; }

		/// <summary>
		/// Tag for the artifact files
		/// </summary>
		public string? OutputName { get; }
	}

	/// <summary>
	/// Information required to create a node
	/// </summary>
	public class NewNode
	{
		/// <summary>
		/// The name of this node 
		/// </summary>
		public string Name { get; set; } = null!;

		/// <summary>
		/// Input names
		/// </summary>
		public List<string>? Inputs { get; set; }

		/// <summary>
		/// Optional input names
		/// </summary>
		public List<string>? OptionalInputs { get; set; }

		/// <summary>
		/// Output names
		/// </summary>
		public List<string>? Outputs { get; set; }

		/// <summary>
		/// List of nodes which must succeed for this node to run
		/// </summary>
		public List<string>? InputDependencies { get; set; }

		/// <summary>
		/// List of nodes which may have succeeded when this node is run
		/// </summary>
		public List<string>? OptionalInputDependencies { get; set; }

		/// <summary>
		/// List of nodes which must have completed for this node to run
		/// </summary>
		public List<string>? OrderDependencies { get; set; }

		/// <summary>
		/// The priority of this node
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// This node can be run multiple times
		/// </summary>
		public bool? AllowRetry { get; set; }

		/// <summary>
		/// This node can start running early, before dependencies of other nodes in the same group are complete
		/// </summary>
		public bool? RunEarly { get; set; }

		/// <summary>
		/// Whether to include warnings in the diagnostic output
		/// </summary>
		public bool? Warnings { get; set; }

		/// <summary>
		/// Credentials required for this node to run. This dictionary maps from environment variable names to a credential property in the format 'CredentialName.PropertyName'.
		/// </summary>
		public Dictionary<string, string>? Credentials { get; set; }

		/// <summary>
		/// Properties for this node
		/// </summary>
		public Dictionary<string, string>? Properties { get; set; }

		/// <summary>
		/// Additional user annotations for this node
		/// </summary>
		public NodeAnnotations? Annotations { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the node</param>
		/// <param name="inputs">List of inputs for the node</param>
		/// <param name="optionalInputs">List of optional inputs for the node</param>
		/// <param name="outputs">List of output names for the node</param>
		/// <param name="inputDependencies">List of nodes which must have completed successfully for this node to run</param>
		/// <param name="optionalInputDependencies">List of nodes which could have completed successfully for this node to run</param>
		/// <param name="orderDependencies">List of nodes which must have completed for this node to run</param>
		/// <param name="priority">Priority of this node</param>
		/// <param name="allowRetry">Whether the node can be run multiple times</param>
		/// <param name="runEarly">Whether the node can run early, before dependencies of other nodes in the same group complete</param>
		/// <param name="warnings">Whether to include warnings in the diagnostic output (defaults to true)</param>
		/// <param name="credentials">Credentials required for this node to run</param>
		/// <param name="properties">Properties for the node</param>
		/// <param name="annotations">User annotations for this node</param>
		public NewNode(string name, List<string>? inputs = null, List<string>? optionalInputs = null, List<string>? outputs = null, List<string>? inputDependencies = null, List<string>? optionalInputDependencies = null, List<string>? orderDependencies = null, Priority? priority = null, bool? allowRetry = null, bool? runEarly = null, bool? warnings = null, Dictionary<string, string>? credentials = null, Dictionary<string, string>? properties = null, IReadOnlyNodeAnnotations? annotations = null)
		{
			Name = name;
			Inputs = inputs;
			OptionalInputs = optionalInputs;
			Outputs = outputs;
			InputDependencies = inputDependencies;
			OptionalInputDependencies = optionalInputDependencies;
			OrderDependencies = orderDependencies;
			Priority = priority;
			AllowRetry = allowRetry;
			RunEarly = runEarly;
			Warnings = warnings;
			Credentials = credentials;
			Properties = properties;
			if (annotations != null)
			{
				Annotations = new NodeAnnotations(annotations);
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="graph">Existing graph containing a node</param>
		/// <param name="node">Node to copy</param>
		public NewNode(IGraph graph, INode node)
			: this(node.Name, [.. node.Inputs.Select(x => graph.GetNode(x.NodeRef).Name)], [.. node.OptionalInputs.Select(x => graph.GetNode(x.NodeRef).Name)], [.. node.OutputNames], [.. node.InputDependencies.Select(x => graph.GetNode(x).Name)], [.. node.OptionalInputDependencies.Select(x => graph.GetNode(x).Name)], [.. node.OrderDependencies.Select(x => graph.GetNode(x).Name)], node.Priority, node.AllowRetry, node.RunEarly, node.Warnings, node.Credentials?.ToDictionary(x => x.Key, x => x.Value), node.Properties?.ToDictionary(x => x.Key, x => x.Value), node.Annotations)
		{
		}
	}

	/// <summary>
	/// Information about a group of nodes
	/// </summary>
	public class NewGroup
	{
		/// <summary>
		/// The type of agent to execute this group
		/// </summary>
		public string AgentType { get; set; }

		/// <summary>
		/// Nodes in the group
		/// </summary>
		public List<NewNode> Nodes { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="agentType">The type of agent to execute this group</param>
		/// <param name="nodes">Nodes in this group</param>
		public NewGroup(string agentType, List<NewNode> nodes)
		{
			AgentType = agentType;
			Nodes = nodes;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="graph">Graph containing the node group</param>
		/// <param name="group">Node group to copy</param>
		public NewGroup(IGraph graph, INodeGroup group)
			: this(group.AgentType, [.. group.Nodes.Select(x => new NewNode(graph, x))])
		{
		}
	}

	/// <summary>
	/// Information about a group of nodes
	/// </summary>
	public class NewLabel
	{
		/// <summary>
		/// Category for this label
		/// </summary>
		[Obsolete("Use DashboardCategory instead")]
		public string? Category => DashboardCategory;

		/// <summary>
		/// Name of the aggregate
		/// </summary>
		[Obsolete("Use DashboardName instead")]
		public string? Name => DashboardName;

		/// <summary>
		/// Name of the aggregate
		/// </summary>
		public string? DashboardName { get; set; }

		/// <summary>
		/// Category for this label
		/// </summary>
		public string? DashboardCategory { get; set; }

		/// <summary>
		/// Name of the badge in UGS
		/// </summary>
		public string? UgsName { get; set; }

		/// <summary>
		/// Project to show this label for in UGS
		/// </summary>
		public string? UgsProject { get; set; }

		/// <summary>
		/// Which change the label applies to
		/// </summary>
		public LabelChange Change { get; set; }

		/// <summary>
		/// Nodes which must be part of the job for the aggregate to be valid
		/// </summary>
		public List<string> RequiredNodes { get; set; } = [];

		/// <summary>
		/// Nodes which must be part of the job for the aggregate to be valid
		/// </summary>
		public List<string> IncludedNodes { get; set; } = [];
	}

	/// <summary>
	/// Information about a group of nodes
	/// </summary>
	public class NewAggregate
	{
		/// <summary>
		/// Name of the aggregate
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Nodes which must be part of the job for the aggregate to be valid
		/// </summary>
		public List<string> Nodes { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of this aggregate</param>
		/// <param name="nodes">Nodes which must be part of the job for the aggregate to be shown</param>
		public NewAggregate(string name, List<string> nodes)
		{
			Name = name;
			Nodes = nodes;
		}
	}

	/// <summary>
	/// Information about an artifact
	/// </summary>
	public record class NewGraphArtifact(ArtifactName Name, ArtifactType Type, string Description, string BasePath, List<string> Keys, List<string> Metadata, string? NodeName, string? OutputName);
}
