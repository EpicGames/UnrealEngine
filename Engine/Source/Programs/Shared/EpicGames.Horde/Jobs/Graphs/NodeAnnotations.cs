// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Issues;

namespace EpicGames.Horde.Jobs.Graphs
{
	/// <summary>
	/// Interface which wraps a generic key/value dictionary to provide specific node annotations
	/// </summary>
#pragma warning disable CA1710 // Identifiers should have correct suffix
	public interface IReadOnlyNodeAnnotations : IReadOnlyDictionary<string, string>
#pragma warning restore CA1710 // Identifiers should have correct suffix
	{
		/// <summary>
		/// Workflow to use for triaging issues from this node
		/// </summary>
		WorkflowId? WorkflowId { get; }

		/// <summary>
		/// Whether to create issues for this node
		/// </summary>
		bool? CreateIssues { get; }

		/// <summary>
		/// Whether to create warning issues for this node
		/// </summary>
		bool? CreateWarningIssues { get; }

		/// <summary>
		/// Whether to automatically assign issues that could only be caused by one user, or have a well defined correlation with a modified file.
		/// </summary>
		bool? AutoAssign { get; }

		/// <summary>
		/// Automatically assign any issues to the given user
		/// </summary>
		string? AutoAssignToUser { get; }

		/// <summary>
		/// Whether to notify all submitters between a build suceeding and failing, allowing them to step forward and take ownership of an issue.
		/// </summary>
		bool? NotifySubmitters { get; }

		/// <summary>
		/// Key to use for grouping issues together, preventing them being merged with other groups
		/// </summary>
		string? IssueGroup { get; }

		/// <summary>
		/// Whether failures in this node should be flagged as build blockers
		/// </summary>
		bool? BuildBlocker { get; }
	}

	/// <summary>
	/// Set of annotations for a node
	/// </summary>
	public class NodeAnnotations : Dictionary<string, string>, IReadOnlyNodeAnnotations
	{
		/// <summary>
		/// Empty annotation dictionary
		/// </summary>
		public static IReadOnlyNodeAnnotations Empty { get; } = new NodeAnnotations();

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.WorkflowId"/>
		public const string WorkflowKeyName = "Workflow";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.CreateIssues"/>
		public const string CreateIssuesKeyName = "CreateIssues";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.CreateWarningIssues"/>
		public const string CreateWarningIssuesKeyName = "CreateWarningIssues";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.AutoAssign"/>
		public const string AutoAssignKeyName = "AutoAssign";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.AutoAssignToUser"/>
		public const string AutoAssignToUserKeyName = "AutoAssignToUser";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.NotifySubmitters"/>
		public const string NotifySubmittersKeyName = "NotifySubmitters";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.IssueGroup"/>
		public const string IssueGroupKeyName = "IssueGroup";

		/// <inheritdoc cref="IReadOnlyNodeAnnotations.BuildBlocker"/>
		public const string BuildBlockerKeyName = "BuildBlocker";

		/// <summary>
		/// Constructor
		/// </summary>
		public NodeAnnotations()
			: base(StringComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="annotations"></param>
		public NodeAnnotations(IReadOnlyDictionary<string, string> annotations)
			: this()
		{
			Merge(annotations);
		}

		/// <inheritdoc/>
		public WorkflowId? WorkflowId
		{
			get
			{
				string? workflowName;
				if (!TryGetValue(WorkflowKeyName, out workflowName))
				{
					return null;
				}

				try
				{
					return new WorkflowId(workflowName);
				}
				catch
				{
					return null;
				}
			}
			set => SetStringValue(WorkflowKeyName, value?.ToString());
		}

		/// <inheritdoc/>
		public bool? CreateIssues
		{
			get => GetBoolValue(CreateIssuesKeyName);
			set => SetBoolValue(CreateIssuesKeyName, value);
		}

		/// <inheritdoc/>
		public bool? CreateWarningIssues
		{
			get => GetBoolValue(CreateWarningIssuesKeyName);
			set => SetBoolValue(CreateWarningIssuesKeyName, value);
		}

		/// <inheritdoc/>
		public bool? AutoAssign
		{
			get => GetBoolValue(AutoAssignKeyName);
			set => SetBoolValue(AutoAssignKeyName, value);
		}

		/// <inheritdoc/>
		public string? AutoAssignToUser
		{
			get => GetStringValue(AutoAssignToUserKeyName);
			set => SetStringValue(AutoAssignToUserKeyName, value);
		}

		/// <inheritdoc/>
		public bool? NotifySubmitters
		{
			get => GetBoolValue(NotifySubmittersKeyName);
			set => SetBoolValue(NotifySubmittersKeyName, value);
		}

		/// <inheritdoc/>
		public string? IssueGroup
		{
			get => GetStringValue(IssueGroupKeyName);
			set => SetStringValue(IssueGroupKeyName, value);
		}

		/// <inheritdoc/>
		public bool? BuildBlocker
		{
			get => GetBoolValue(BuildBlockerKeyName);
			set => SetBoolValue(BuildBlockerKeyName, value);
		}

		private bool? GetBoolValue(string key)
		{
			string? value = GetStringValue(key);
			if (value != null)
			{
				if (value.Equals("0", StringComparison.Ordinal) || value.Equals("false", StringComparison.OrdinalIgnoreCase))
				{
					return false;
				}
				if (value.Equals("1", StringComparison.Ordinal) || value.Equals("true", StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}
			return null;
		}

		private void SetBoolValue(string key, bool? value)
		{
			if (value == null)
			{
				Remove(key);
			}
			else
			{
				SetStringValue(key, value.Value ? "1" : "0");
			}
		}

		private string? GetStringValue(string key)
		{
			TryGetValue(key, out string? value);
			return value;
		}

		private void SetStringValue(string key, string? value)
		{
			if (value == null)
			{
				Remove(key);
			}
			else
			{
				this[key] = value;
			}
		}

		/// <summary>
		/// Merge in entries from another set of annotation
		/// </summary>
		/// <param name="other"></param>
		public void Merge(IReadOnlyDictionary<string, string> other)
		{
			foreach ((string key, string value) in other)
			{
				this[key] = value;
			}
		}
	}
}
