// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.Timing;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Ugs;
using EpicGames.Horde.Users;
using Microsoft.Extensions.Logging;

#pragma warning disable CA1716

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Document describing a job
	/// </summary>
	public interface IJob
	{
		/// <summary>
		/// Job argument indicating a target that should be built
		/// </summary>
		public const string TargetArgumentPrefix = "-Target=";

		/// <summary>
		/// Name of the node which parses the buildgraph script
		/// </summary>
		public const string SetupNodeName = "Setup Build";

		/// <summary>
		/// Identifier for the job. Randomly generated.
		/// </summary>
		public JobId Id { get; }

		/// <summary>
		/// The stream that this job belongs to
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// The template ref id
		/// </summary>
		public TemplateId TemplateId { get; }

		/// <summary>
		/// The template that this job was created from
		/// </summary>
		public ContentHash? TemplateHash { get; }

		/// <summary>
		/// Hash of the graph definition
		/// </summary>
		public ContentHash GraphHash { get; }

		/// <summary>
		/// Graph for this job
		/// </summary>
		public IGraph Graph { get; }

		/// <summary>
		/// Id of the user that started this job
		/// </summary>
		public UserId? StartedByUserId { get; }

		/// <summary>
		/// Id of the user that aborted this job. Set to null if the job is not aborted.
		/// </summary>
		public UserId? AbortedByUserId { get; }

		/// <summary>
		/// Optional reason for why the job was canceled
		/// </summary>
		public string? CancellationReason { get; }

		/// <summary>
		/// Identifier of the bisect task that started this job
		/// </summary>
		public BisectTaskId? StartedByBisectTaskId { get; }

		/// <summary>
		/// Name of the job.
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// The commit to build
		/// </summary>
		public CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// The code commit for this build
		/// </summary>
		public CommitIdWithOrder? CodeCommitId { get; }

		/// <summary>
		/// The preflight changelist number
		/// </summary>
		public CommitId? PreflightCommitId { get; }

		/// <summary>
		/// Description for the shelved change if running a preflight
		/// </summary>
		public string? PreflightDescription { get; }

		/// <summary>
		/// Priority of this job
		/// </summary>
		public Priority Priority { get; }

		/// <summary>
		/// For preflights, submit the change if the job is successful
		/// </summary>
		public bool AutoSubmit { get; }

		/// <summary>
		/// The submitted changelist number
		/// </summary>
		public int? AutoSubmitChange { get; }

		/// <summary>
		/// Message produced by trying to auto-submit the change
		/// </summary>
		public string? AutoSubmitMessage { get; }

		/// <summary>
		/// Whether to update issues based on the outcome of this job
		/// </summary>
		public bool UpdateIssues { get; }

		/// <summary>
		/// Whether to promote issues by default based on the outcome of this job
		/// </summary>
		public bool PromoteIssuesByDefault { get; }

		/// <summary>
		/// Time that the job was created (in UTC)
		/// </summary>
		public DateTime CreateTimeUtc { get; }

		/// <summary>
		/// Options for executing the job
		/// </summary>
		public JobOptions? JobOptions { get; }

		/// <summary>
		/// Claims inherited from the user that started this job
		/// </summary>
		public IReadOnlyList<IAclClaim> Claims { get; }

		/// <summary>
		/// Array of jobstep runs
		/// </summary>
		public IReadOnlyList<IJobStepBatch> Batches { get; }

		/// <summary>
		/// Parameters for the job
		/// </summary>
		public IReadOnlyDictionary<ParameterId, string> Parameters { get; }

		/// <summary>
		/// Optional user-defined properties for this job
		/// </summary>
		public IReadOnlyList<string> Arguments { get; }

		/// <summary>
		/// Custom list of targets for the job. If null or empty, the list of targets is determined from the command line.
		/// </summary>
		public IReadOnlyList<string>? Targets { get; }

		/// <summary>
		/// Additional arguments for the job, when a set of parameters are applied.
		/// </summary>
		public IReadOnlyList<string> AdditionalArguments { get; }

		/// <summary>
		/// Environment variables for the job
		/// </summary>
		public IReadOnlyDictionary<string, string> Environment { get; }

		/// <summary>
		/// Issues associated with this job
		/// </summary>
		public IReadOnlyList<int> Issues { get; }

		/// <summary>
		/// Unique id for notifications
		/// </summary>
		public NotificationTriggerId? NotificationTriggerId { get; }

		/// <summary>
		/// Whether to show badges in UGS for this job
		/// </summary>
		public bool ShowUgsBadges { get; }

		/// <summary>
		/// Whether to show alerts in UGS for this job
		/// </summary>
		public bool ShowUgsAlerts { get; }

		/// <summary>
		/// Notification channel for this job.
		/// </summary>
		public string? NotificationChannel { get; }

		/// <summary>
		/// Notification channel filter for this job.
		/// </summary>
		public string? NotificationChannelFilter { get; }

		/// <summary>
		/// Mapping of label ids to notification trigger ids for notifications
		/// </summary>
		public IReadOnlyDictionary<int, NotificationTriggerId> LabelIdxToTriggerId { get; }

		/// <summary>
		/// List of reports for this step
		/// </summary>
		public IReadOnlyList<IJobReport>? Reports { get; }

		/// <summary>
		/// List of downstream job triggers
		/// </summary>
		public IReadOnlyList<IChainedJob> ChainedJobs { get; }

		/// <summary>
		/// The job which spawned this job
		/// </summary>
		public JobId? ParentJobId { get; }

		/// <summary>
		/// The job step which spawned this job
		/// </summary>
		public JobStepId? ParentJobStepId { get; }

		/// <summary>
		/// The last update time
		/// </summary>
		public DateTime UpdateTimeUtc { get; }

		/// <summary>
		/// The job meta data
		/// </summary>
		public List<string> Metadata { get; }

		/// <summary>
		/// Update counter for this document. Any updates should compare-and-swap based on the value of this counter, or increment it in the case of server-side updates.
		/// </summary>
		public int UpdateIndex { get; }

		/// <summary>
		/// Gets the latest job state
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IJob?> RefreshAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to get a batch with the given id
		/// </summary>
		/// <param name="batchId">The job batch id</param>
		/// <param name="batch">Receives the batch interface on success</param>
		/// <returns>True if the batch was found</returns>
		bool TryGetBatch(JobStepBatchId batchId, [NotNullWhen(true)] out IJobStepBatch? batch);

		/// <summary>
		/// Attempt to get a step with the given id
		/// </summary>
		/// <param name="stepId">The job step id</param>
		/// <param name="step">Receives the step interface on success</param>
		/// <returns>True if the step was found</returns>
		bool TryGetStep(JobStepId stepId, [NotNullWhen(true)] out IJobStep? step);

		/// <summary>
		/// Attempt to delete the job
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was deleted. False if the job is not the latest revision.</returns>
		Task<bool> TryDeleteAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Removes a job from the dispatch queue. Ignores the state of any batches still remaining to execute. Should only be used to correct for inconsistent state.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IJob?> TryRemoveFromDispatchQueueAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates a new job
		/// </summary>
		/// <param name="name">Name of the job</param>
		/// <param name="priority">Priority of the job</param>
		/// <param name="autoSubmit">Automatically submit the job on completion</param>
		/// <param name="autoSubmitChange">Changelist that was automatically submitted</param>
		/// <param name="autoSubmitMessage"></param>
		/// <param name="abortedByUserId">Name of the user that aborted the job</param>
		/// <param name="notificationTriggerId">Id for a notification trigger</param>
		/// <param name="reports">New reports</param>
		/// <param name="arguments">New arguments for the job</param>
		/// <param name="labelIdxToTriggerId">New trigger ID for a label in the job</param>
		/// <param name="jobTrigger">New downstream job id</param>
		/// <param name="cancellationReason">Optional reason why the job was canceled</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IJob?> TryUpdateJobAsync(string? name = null, Priority? priority = null, bool? autoSubmit = null, int? autoSubmitChange = null, string? autoSubmitMessage = null, UserId? abortedByUserId = null, NotificationTriggerId? notificationTriggerId = null, List<JobReport>? reports = null, List<string>? arguments = null, KeyValuePair<int, NotificationTriggerId>? labelIdxToTriggerId = null, KeyValuePair<TemplateId, JobId>? jobTrigger = null, string? cancellationReason = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates the state of a batch
		/// </summary>
		/// <param name="batchId">Unique id of the batch to update</param>
		/// <param name="newLogId">The new log file id</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="newError">Error code for the batch</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was updated, false if it was deleted</returns>
		Task<IJob?> TryUpdateBatchAsync(JobStepBatchId batchId, LogId? newLogId, JobStepBatchState? newState, JobStepBatchError? newError, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update a jobstep state
		/// </summary>
		/// <param name="batchId">Unique id of the batch containing the step</param>
		/// <param name="stepId">Unique id of the step to update</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="newOutcome">New outcome of the jobstep</param>
		/// <param name="newError">New error annotation for this jobstep</param>
		/// <param name="newAbortRequested">New state of request abort</param>
		/// <param name="newAbortByUserId">New name of user that requested the abort</param>
		/// <param name="newLogId">New log id for the jobstep</param>
		/// <param name="newNotificationTriggerId">New id for a notification trigger</param>
		/// <param name="newRetryByUserId">Whether the step should be retried</param>
		/// <param name="newPriority">New priority for this step</param>
		/// <param name="newReports">New report documents</param>
		/// <param name="newProperties">Property changes. Any properties with a null value will be removed.</param>
		/// <param name="newCancellationReason">The reason the job step was canceled</param>
		/// <param name="newSpawnedJob">JobId of any job this step spawned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was updated, false if it was deleted in the meantime</returns>
		Task<IJob?> TryUpdateStepAsync(JobStepBatchId batchId, JobStepId stepId, JobStepState newState = default, JobStepOutcome newOutcome = default, JobStepError? newError = null, bool? newAbortRequested = null, UserId? newAbortByUserId = null, LogId? newLogId = null, NotificationTriggerId? newNotificationTriggerId = null, UserId? newRetryByUserId = null, Priority? newPriority = null, List<JobReport>? newReports = null, Dictionary<string, string?>? newProperties = null, string? newCancellationReason = null, JobId? newSpawnedJob = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to update the node groups to be executed for a job. Fails if another write happens in the meantime.
		/// </summary>
		/// <param name="newGraph">New graph for this job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the groups were updated to the given list. False if another write happened first.</returns>
		Task<IJob?> TryUpdateGraphAsync(IGraph newGraph, CancellationToken cancellationToken = default);

		/// <summary>
		/// Marks a job as skipped
		/// </summary>
		/// <param name="reason">Reason for this batch being failed</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated version of the job</returns>
		Task<IJob?> TrySkipAllBatchesAsync(JobStepBatchError reason, CancellationToken cancellationToken = default);

		/// <summary>
		/// Marks a batch as skipped
		/// </summary>
		/// <param name="batchId">The batch to mark as skipped</param>
		/// <param name="reason">Reason for this batch being failed</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated version of the job</returns>
		Task<IJob?> TrySkipBatchAsync(JobStepBatchId batchId, JobStepBatchError reason, CancellationToken cancellationToken = default);

		/// <summary>
		/// Abort an agent's lease, and update the payload accordingly
		/// </summary>
		/// <param name="batchIdx">Index of the batch to cancel</param>
		/// <param name="reason">Reason for this batch being failed</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job is updated</returns>
		Task<IJob?> TryFailBatchAsync(int batchIdx, JobStepBatchError reason, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to assign a lease to execute a batch
		/// </summary>
		/// <param name="batchIdx">Index of the batch</param>
		/// <param name="poolId">The pool id</param>
		/// <param name="agentId">New agent to execute the batch</param>
		/// <param name="sessionId">Session of the agent that is to execute the batch</param>
		/// <param name="leaseId">The lease unique id</param>
		/// <param name="logId">Unique id of the log for the batch</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the batch is updated</returns>
		Task<IJob?> TryAssignLeaseAsync(int batchIdx, PoolId poolId, AgentId agentId, SessionId sessionId, LeaseId leaseId, LogId logId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Cancel a lease reservation on a batch (before it has started)
		/// </summary>
		/// <param name="batchIdx">Index of the batch to cancel</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job is updated</returns>
		Task<IJob?> TryCancelLeaseAsync(int batchIdx, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for jobs
	/// </summary>
	public static class JobExtensions
	{
		/// <summary>
		/// Gets the current job state
		/// </summary>
		/// <param name="job">The job document</param>
		/// <returns>Job state</returns>
		public static JobState GetState(this IJob job)
		{
			bool waiting = false;
			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep step in batch.Steps)
				{
					if (step.State == JobStepState.Running)
					{
						return JobState.Running;
					}
					else if (step.State == JobStepState.Ready || step.State == JobStepState.Waiting)
					{
						if (batch.State == JobStepBatchState.Starting || batch.State == JobStepBatchState.Running)
						{
							return JobState.Running;
						}
						else
						{
							waiting = true;
						}
					}
				}
			}
			return waiting ? JobState.Waiting : JobState.Complete;
		}

		/// <summary>
		/// Gets the outcome for a particular named target. May be an aggregate or node name.
		/// </summary>
		/// <param name="job">The job to check</param>
		/// <returns>The step outcome</returns>
		public static (JobStepState, JobStepOutcome) GetTargetState(this IJob job)
		{
			IReadOnlyDictionary<NodeRef, IJobStep> nodeToStep = GetStepForNodeMap(job);
			return GetTargetState(nodeToStep.Values);
		}

		/// <summary>
		/// Gets the outcome for a particular named target. May be an aggregate or node name.
		/// </summary>
		/// <param name="job">The job to check</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="target">Target to find an outcome for</param>
		/// <returns>The step outcome</returns>
		public static (JobStepState, JobStepOutcome)? GetTargetState(this IJob job, IGraph graph, string? target)
		{
			if (target == null)
			{
				return GetTargetState(job);
			}

			NodeRef nodeRef;
			if (graph.TryFindNode(target, out nodeRef))
			{
				IJobStep? step;
				if (job.TryGetStepForNode(nodeRef, out step))
				{
					return (step.State, step.Outcome);
				}
				else
				{
					return null;
				}
			}

			IAggregate? aggregate;
			if (graph.TryFindAggregate(target, out aggregate))
			{
				IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = GetStepForNodeMap(job);

				List<IJobStep> steps = [];
				foreach (NodeRef aggregateNodeRef in aggregate.Nodes)
				{
					IJobStep? step;
					if (!stepForNode.TryGetValue(aggregateNodeRef, out step))
					{
						return null;
					}
					steps.Add(step);
				}

				return GetTargetState(steps);
			}

			return null;
		}

		/// <summary>
		/// Gets the outcome for a particular named target. May be an aggregate or node name.
		/// </summary>
		/// <param name="steps">Steps to include</param>
		/// <returns>The step outcome</returns>
		public static (JobStepState, JobStepOutcome) GetTargetState(IEnumerable<IJobStep> steps)
		{
			bool anySkipped = false;
			bool anyWarnings = false;
			bool anyFailed = false;
			bool anyPending = false;
			foreach (IJobStep step in steps)
			{
				anyPending |= step.IsPending();
				anySkipped |= step.State == JobStepState.Aborted || step.State == JobStepState.Skipped;
				anyFailed |= (step.Outcome == JobStepOutcome.Failure);
				anyWarnings |= (step.Outcome == JobStepOutcome.Warnings);
			}

			JobStepState newState = anyPending ? JobStepState.Running : JobStepState.Completed;
			JobStepOutcome newOutcome = anyFailed ? JobStepOutcome.Failure : anyWarnings ? JobStepOutcome.Warnings : anySkipped ? JobStepOutcome.Unspecified : JobStepOutcome.Success;
			return (newState, newOutcome);
		}

		/// <summary>
		/// Gets the outcome for a particular named target. May be an aggregate or node name.
		/// </summary>
		/// <param name="job">The job to check</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="target">Target to find an outcome for</param>
		/// <returns>The step outcome</returns>
		public static JobStepOutcome GetTargetOutcome(this IJob job, IGraph graph, string target)
		{
			NodeRef nodeRef;
			if (graph.TryFindNode(target, out nodeRef))
			{
				IJobStep? step;
				if (job.TryGetStepForNode(nodeRef, out step))
				{
					return step.Outcome;
				}
				else
				{
					return JobStepOutcome.Unspecified;
				}
			}

			IAggregate? aggregate;
			if (graph.TryFindAggregate(target, out aggregate))
			{
				IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = GetStepForNodeMap(job);

				bool warnings = false;
				foreach (NodeRef aggregateNodeRef in aggregate.Nodes)
				{
					IJobStep? step;
					if (!stepForNode.TryGetValue(aggregateNodeRef, out step))
					{
						return JobStepOutcome.Unspecified;
					}
					if (step.Outcome == JobStepOutcome.Failure)
					{
						return JobStepOutcome.Failure;
					}
					warnings |= (step.Outcome == JobStepOutcome.Warnings);
				}
				return warnings ? JobStepOutcome.Warnings : JobStepOutcome.Success;
			}

			return JobStepOutcome.Unspecified;
		}

		/// <summary>
		/// Gets the job step for a particular node
		/// </summary>
		/// <param name="job">The job to search</param>
		/// <param name="nodeRef">The node ref</param>
		/// <param name="jobStep">Receives the jobstep on success</param>
		/// <returns>True if the jobstep was founds</returns>
		public static bool TryGetStepForNode(this IJob job, NodeRef nodeRef, [NotNullWhen(true)] out IJobStep? jobStep)
		{
			jobStep = null;
			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.GroupIdx == nodeRef.GroupIdx)
				{
					foreach (IJobStep batchStep in batch.Steps)
					{
						if (batchStep.NodeIdx == nodeRef.NodeIdx)
						{
							jobStep = batchStep;
						}
					}
				}
			}
			return jobStep != null;
		}

		/// <summary>
		/// Gets a dictionary that maps <see cref="NodeRef"/> objects to their associated
		/// <see cref="IJobStep"/> objects on a <see cref="IJob"/>.
		/// </summary>
		/// <param name="job">The job document</param>
		/// <returns>Map of <see cref="NodeRef"/> to <see cref="IJobStep"/></returns>
		public static IReadOnlyDictionary<NodeRef, IJobStep> GetStepForNodeMap(this IJob job)
		{
			Dictionary<NodeRef, IJobStep> stepForNode = [];
			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep batchStep in batch.Steps)
				{
					NodeRef batchNodeRef = new(batch.GroupIdx, batchStep.NodeIdx);
					stepForNode[batchNodeRef] = batchStep;
				}
			}
			return stepForNode;
		}

		/// <summary>
		/// Find the latest step executing the given node
		/// </summary>
		/// <param name="job">The job being run</param>
		/// <param name="nodeRef">Node to find</param>
		/// <returns>The retried step information</returns>
		public static JobStepRefId? FindLatestStepForNode(this IJob job, NodeRef nodeRef)
		{
			for (int batchIdx = job.Batches.Count - 1; batchIdx >= 0; batchIdx--)
			{
				IJobStepBatch batch = job.Batches[batchIdx];
				if (batch.GroupIdx == nodeRef.GroupIdx)
				{
					for (int stepIdx = batch.Steps.Count - 1; stepIdx >= 0; stepIdx--)
					{
						IJobStep step = batch.Steps[stepIdx];
						if (step.NodeIdx == nodeRef.NodeIdx)
						{
							return new JobStepRefId(job.Id, batch.Id, step.Id);
						}
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Gets the estimated timing info for all nodes in the job
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="graph">Graph for this job</param>
		/// <param name="jobTiming">Job timing information</param>
		/// <param name="logger">Logger for any diagnostic messages</param>
		/// <returns>Map of node to expected timing info</returns>
		public static Dictionary<INode, TimingInfo> GetTimingInfo(this IJob job, IGraph graph, IJobTiming jobTiming, ILogger logger)
		{
#pragma warning disable IDE0054 // Use compound assignment
			TimeSpan currentTime = DateTime.UtcNow - job.CreateTimeUtc;

			Dictionary<INode, TimingInfo> nodeToTimingInfo = graph.Groups.SelectMany(x => x.Nodes).ToDictionary(x => x, x => new TimingInfo());
			foreach (IJobStepBatch batch in job.Batches)
			{
				INodeGroup group = graph.Groups[batch.GroupIdx];

				// Step through the batch, keeping track of the time that things finish.
				TimingInfo timingInfo = new();

				// Wait for the dependencies for the batch to start
				HashSet<INode> dependencyNodes = batch.GetStartDependencies(graph.Groups);
				timingInfo.WaitForAll(dependencyNodes.Select(x => nodeToTimingInfo[x]));

				// If the batch has actually started, correct the expected time to use this instead
				if (batch.StartTimeUtc != null)
				{
					timingInfo.TotalTimeToComplete = batch.StartTimeUtc - job.CreateTimeUtc;
				}

				// Get the average times for this batch
				TimeSpan? averageWaitTime = GetAverageWaitTime(graph, batch, jobTiming, logger);
				TimeSpan? averageInitTime = GetAverageInitTime(graph, batch, jobTiming, logger);

				// Update the wait times and initialization times along this path
				timingInfo.TotalWaitTime = timingInfo.TotalWaitTime + (batch.GetWaitTime() ?? averageWaitTime);
				timingInfo.TotalInitTime = timingInfo.TotalInitTime + (batch.GetInitTime() ?? averageInitTime);

				// Update the average wait and initialization times too
				timingInfo.AverageTotalWaitTime = timingInfo.AverageTotalWaitTime + averageWaitTime;
				timingInfo.AverageTotalInitTime = timingInfo.AverageTotalInitTime + averageInitTime;

				// Step through the batch, updating the expected times as we go
				foreach (IJobStep step in batch.Steps)
				{
					INode node = group.Nodes[step.NodeIdx];

					// Get the timing for this step
					IJobStepTiming? stepTimingInfo;
					jobTiming.TryGetStepTiming(node.Name, logger, out stepTimingInfo);

					// If the step has already started, update the actual time to reach this point
					if (step.StartTimeUtc != null)
					{
						timingInfo.TotalTimeToComplete = step.StartTimeUtc.Value - job.CreateTimeUtc;
					}

					// If the step hasn't started yet, make sure the start time is later than the current time
					if (step.StartTimeUtc == null && currentTime > timingInfo.TotalTimeToComplete)
					{
						timingInfo.TotalTimeToComplete = currentTime;
					}

					// Wait for all the node dependencies to complete
					timingInfo.WaitForAll(graph.GetDependencies(node).Select(x => nodeToTimingInfo[x]));

					// If the step has actually finished, correct the time to use that instead
					if (step.FinishTimeUtc != null)
					{
						timingInfo.TotalTimeToComplete = step.FinishTimeUtc.Value - job.CreateTimeUtc;
					}
					else
					{
						timingInfo.TotalTimeToComplete = timingInfo.TotalTimeToComplete + NullableTimeSpanFromSeconds(stepTimingInfo?.AverageDuration);
					}

					// If the step hasn't finished yet, make sure the start time is later than the current time
					if (step.FinishTimeUtc == null && currentTime > timingInfo.TotalTimeToComplete)
					{
						timingInfo.TotalTimeToComplete = currentTime;
					}

					// Update the average time to complete
					timingInfo.AverageTotalTimeToComplete = timingInfo.AverageTotalTimeToComplete + NullableTimeSpanFromSeconds(stepTimingInfo?.AverageDuration);

					// Add it to the lookup
					TimingInfo nodeTimingInfo = new(timingInfo);
					nodeTimingInfo.StepTiming = stepTimingInfo;
					nodeToTimingInfo[node] = nodeTimingInfo;
				}
			}
			return nodeToTimingInfo;
#pragma warning restore IDE0054 // Use compound assignment
		}

		/// <summary>
		/// Gets the step completion info of the job.
		/// </summary>
		/// <param name="job">The job document</param>
		/// <returns>The job's step completion info.</returns>
		/// <remarks>Any steps not in a terminal state (<see cref="JobStepState.Completed"/>, <see cref="JobStepState.Aborted"/>, <see cref="JobStepState.Skipped"/>) are excluded in total counts except <see cref="JobStepsCompletionInfo.StepTotalCount"/>.</remarks>
		public static JobStepsCompletionInfo GetStepCompletionInfo(this IJob job)
		{
			JobStepsCompletionInfo jobStepsCompletionInfo = new();

			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep step in batch.Steps)
				{
					jobStepsCompletionInfo.StepTotalCount++;

					if (step.State != JobStepState.Completed && step.State != JobStepState.Aborted && step.State != JobStepState.Skipped)
					{
						continue;
					}

					if (step.FinishTimeUtc != null && step.StartTimeUtc != null)
					{
						TimeSpan? stepDuration = step.FinishTimeUtc.Value - step.StartTimeUtc;

						if (stepDuration != null)
						{
							jobStepsCompletionInfo.JobStepsTotalTime += (float)stepDuration.Value.TotalSeconds;
						}
					}

					switch (step.Outcome)
					{
						case JobStepOutcome.Success:
							jobStepsCompletionInfo.StepPassCount++;
							break;
						case JobStepOutcome.Warnings:
							jobStepsCompletionInfo.StepWarningCount++;
							break;
						case JobStepOutcome.Failure:
							jobStepsCompletionInfo.StepFailureCount++;
							break;
					}
				}
			}
			return jobStepsCompletionInfo;
		}

		/// <summary>
		/// Gets the average wait time for this batch
		/// </summary>
		/// <param name="graph">Graph for the job</param>
		/// <param name="batch">The batch to get timing info for</param>
		/// <param name="jobTiming">The job timing information</param>
		/// <param name="logger">Logger for diagnostic info</param>
		/// <returns>Wait time for the batch</returns>
		public static TimeSpan? GetAverageWaitTime(IGraph graph, IJobStepBatch batch, IJobTiming jobTiming, ILogger logger)
		{
			TimeSpan? waitTime = null;
			foreach (IJobStep step in batch.Steps)
			{
				INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
				if (jobTiming.TryGetStepTiming(node.Name, logger, out IJobStepTiming? timingInfo))
				{
					if (timingInfo.AverageWaitTime != null)
					{
						TimeSpan stepWaitTime = TimeSpan.FromSeconds(timingInfo.AverageWaitTime.Value);
						if (waitTime == null || stepWaitTime > waitTime.Value)
						{
							waitTime = stepWaitTime;
						}
					}
				}
			}
			return waitTime;
		}

		/// <summary>
		/// Gets the average initialization time for this batch
		/// </summary>
		/// <param name="graph">Graph for the job</param>
		/// <param name="batch">The batch to get timing info for</param>
		/// <param name="jobTiming">The job timing information</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		/// <returns>Initialization time for this batch</returns>
		public static TimeSpan? GetAverageInitTime(IGraph graph, IJobStepBatch batch, IJobTiming jobTiming, ILogger logger)
		{
			TimeSpan? initTime = null;
			foreach (IJobStep step in batch.Steps)
			{
				INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
				if (jobTiming.TryGetStepTiming(node.Name, logger, out IJobStepTiming? timingInfo))
				{
					if (timingInfo.AverageInitTime != null)
					{
						TimeSpan stepInitTime = TimeSpan.FromSeconds(timingInfo.AverageInitTime.Value);
						if (initTime == null || stepInitTime > initTime.Value)
						{
							initTime = stepInitTime;
						}
					}
				}
			}
			return initTime;
		}

		/// <summary>
		/// Creates a nullable timespan from a nullable number of seconds
		/// </summary>
		/// <param name="seconds">The number of seconds to construct from</param>
		/// <returns>TimeSpan object</returns>
		static TimeSpan? NullableTimeSpanFromSeconds(float? seconds)
		{
			if (seconds == null)
			{
				return null;
			}
			else
			{
				return TimeSpan.FromSeconds(seconds.Value);
			}
		}

		/// <summary>
		/// Attempts to get a batch with the given id
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="batchId">The batch id</param>
		/// <param name="stepId">The step id</param>
		/// <param name="step">On success, receives the step object</param>
		/// <returns>True if the batch was found</returns>
		public static bool TryGetStep(this IJob job, JobStepBatchId batchId, JobStepId stepId, [NotNullWhen(true)] out IJobStep? step)
		{
			if (!job.TryGetStep(stepId, out step) || step.Batch.Id != batchId)
			{
				step = null;
				return false;
			}
			return true;
		}

		/// <summary>
		/// Finds the set of nodes affected by a label
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="graph">Graph definition for the job</param>
		/// <param name="labelIdx">Index of the label. -1 or Graph.Labels.Count are treated as referring to the default lable.</param>
		/// <returns>Set of nodes affected by the given label</returns>
		public static HashSet<NodeRef> GetNodesForLabel(this IJob job, IGraph graph, int labelIdx)
		{
			if (labelIdx != -1 && labelIdx != graph.Labels.Count)
			{
				// Return all the nodes included by the label
				return [.. graph.Labels[labelIdx].IncludedNodes];
			}
			else
			{
				// Set of nodes which are not covered by an existing label, initially containing everything
				HashSet<NodeRef> unlabeledNodes = [];
				for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
				{
					INodeGroup group = graph.Groups[groupIdx];
					for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
					{
						unlabeledNodes.Add(new NodeRef(groupIdx, nodeIdx));
					}
				}

				// Remove all the nodes that are part of an active label
				IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = job.GetStepForNodeMap();
				foreach (ILabel label in graph.Labels)
				{
					if (label.RequiredNodes.Any(x => stepForNode.ContainsKey(x)))
					{
						unlabeledNodes.ExceptWith(label.IncludedNodes);
					}
				}
				return unlabeledNodes;
			}
		}

		/// <summary>
		/// Create a list of aggregate responses, combining the graph definitions with the state of the job
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="graph">Graph definition for the job</param>
		/// <param name="responses">List to receive all the responses</param>
		/// <returns>The default label state</returns>
		public static GetDefaultLabelStateResponse? GetLabelStateResponses(this IJob job, IGraph graph, List<GetLabelStateResponse> responses)
		{
			// Create a lookup from noderef to step information
			IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = job.GetStepForNodeMap();

			// Set of nodes which are not covered by an existing label, initially containing everything
			HashSet<NodeRef> unlabeledNodes = [];
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
				{
					unlabeledNodes.Add(new NodeRef(groupIdx, nodeIdx));
				}
			}

			// Create the responses
			foreach (ILabel label in graph.Labels)
			{
				// Refresh the state for this label
				LabelState newState = LabelState.Unspecified;
				foreach (NodeRef requiredNodeRef in label.RequiredNodes)
				{
					if (stepForNode.ContainsKey(requiredNodeRef))
					{
						newState = LabelState.Complete;
						break;
					}
				}

				// Refresh the outcome
				LabelOutcome newOutcome = LabelOutcome.Success;
				if (newState == LabelState.Complete)
				{
					GetLabelState(label.IncludedNodes, stepForNode, out newState, out newOutcome);
					unlabeledNodes.ExceptWith(label.IncludedNodes);
				}

				// Create the response
				GetLabelStateResponse response = new();
				response.DashboardName = label.DashboardName;
				response.DashboardCategory = label.DashboardCategory;
				response.UgsName = label.UgsName;
				response.UgsProject = label.UgsProject;
				response.State = newState;
				response.Outcome = newOutcome;

				foreach (NodeRef includedNodeRef in label.IncludedNodes)
				{
					IJobStep? includedStep;
					if (stepForNode.TryGetValue(includedNodeRef, out includedStep))
					{
						response.Steps.Add(includedStep.Id);
					}
				}

				responses.Add(response);
			}

			// Remove all the nodes that don't have a step
			unlabeledNodes.RemoveWhere(x => !stepForNode.ContainsKey(x));

			// Remove successful "setup build" nodes from the list
			if (graph.Groups.Count > 1 && graph.Groups[0].Nodes.Count > 0)
			{
				INode node = graph.Groups[0].Nodes[0];
				if (node.Name == IJob.SetupNodeName)
				{
					NodeRef nodeRef = new(0, 0);
					if (unlabeledNodes.Contains(nodeRef))
					{
						IJobStep step = stepForNode[nodeRef];
						if (step.State == JobStepState.Completed && step.Outcome == JobStepOutcome.Success && responses.Count > 0)
						{
							unlabeledNodes.Remove(nodeRef);
						}
					}
				}
			}

			// Add a response for everything not included elsewhere.
			GetLabelState(unlabeledNodes, stepForNode, out LabelState otherState, out LabelOutcome otherOutcome);

			GetDefaultLabelStateResponse defaultResponse = new();
			defaultResponse.State = otherState;
			defaultResponse.Outcome = otherOutcome;
			defaultResponse.Nodes.AddRange(unlabeledNodes.Select(x => graph.GetNode(x).Name));
			defaultResponse.Steps.AddRange(unlabeledNodes.Select(x => stepForNode[x].Id));
			return defaultResponse;
		}

		/// <summary>
		/// Get the states of all labels for this job
		/// </summary>
		/// <param name="job">The job to get states for</param>
		/// <param name="graph">The graph for this job</param>
		/// <returns>Collection of label states by label index</returns>
		public static IReadOnlyList<(LabelState, LabelOutcome)> GetLabelStates(this IJob job, IGraph graph)
		{
			IReadOnlyDictionary<NodeRef, IJobStep> stepForNodeRef = job.GetStepForNodeMap();

			List<(LabelState, LabelOutcome)> states = [];
			for (int idx = 0; idx < graph.Labels.Count; idx++)
			{
				ILabel label = graph.Labels[idx];

				// Default the label to the unspecified state
				LabelState newState = LabelState.Unspecified;
				LabelOutcome newOutcome = LabelOutcome.Unspecified;

				// Check if the label should be included
				if (label.RequiredNodes.Any(x => stepForNodeRef.ContainsKey(x)))
				{
					// Combine the state of the steps contributing towards this label
					bool anySkipped = false;
					bool anyWarnings = false;
					bool anyFailed = false;
					bool anyPending = false;
					foreach (NodeRef includedNode in label.IncludedNodes)
					{
						IJobStep? step;
						if (stepForNodeRef.TryGetValue(includedNode, out step))
						{
							anyPending |= step.IsPending();
							anySkipped |= step.State == JobStepState.Aborted || step.State == JobStepState.Skipped;
							anyFailed |= (step.Outcome == JobStepOutcome.Failure);
							anyWarnings |= (step.Outcome == JobStepOutcome.Warnings);
						}
					}

					// Figure out the overall label state
					newState = anyPending ? LabelState.Running : LabelState.Complete;
					newOutcome = anyFailed ? LabelOutcome.Failure : anyWarnings ? LabelOutcome.Warnings : anySkipped ? LabelOutcome.Unspecified : LabelOutcome.Success;
				}

				states.Add((newState, newOutcome));
			}
			return states;
		}

		/// <summary>
		/// Get the states of all UGS badges for this job
		/// </summary>
		/// <param name="job">The job to get states for</param>
		/// <param name="graph">The graph for this job</param>
		/// <returns>List of badge states</returns>
		public static Dictionary<int, UgsBadgeState> GetUgsBadgeStates(this IJob job, IGraph graph)
		{
			IReadOnlyList<(LabelState, LabelOutcome)> labelStates = GetLabelStates(job, graph);
			return job.GetUgsBadgeStates(graph, labelStates);
		}

		/// <summary>
		/// Get the states of all UGS badges for this job
		/// </summary>
		/// <param name="job">The job to get states for</param>
		/// <param name="graph">The graph for this job</param>
		/// <param name="labelStates">The existing label states to get the UGS badge states from</param>
		/// <returns>List of badge states</returns>
#pragma warning disable IDE0060 // Remove unused parameter
		public static Dictionary<int, UgsBadgeState> GetUgsBadgeStates(this IJob job, IGraph graph, IReadOnlyList<(LabelState, LabelOutcome)> labelStates)
#pragma warning restore IDE0060 // Remove unused parameter
		{
			Dictionary<int, UgsBadgeState> ugsBadgeStates = [];
			for (int labelIdx = 0; labelIdx < labelStates.Count; ++labelIdx)
			{
				if (graph.Labels[labelIdx].UgsName == null)
				{
					continue;
				}

				(LabelState state, LabelOutcome outcome) = labelStates[labelIdx];
				switch (state)
				{
					case LabelState.Complete:
						{
							switch (outcome)
							{
								case LabelOutcome.Success:
									{
										ugsBadgeStates.Add(labelIdx, UgsBadgeState.Success);
										break;
									}

								case LabelOutcome.Warnings:
									{
										ugsBadgeStates.Add(labelIdx, UgsBadgeState.Warning);
										break;
									}

								case LabelOutcome.Failure:
									{
										ugsBadgeStates.Add(labelIdx, UgsBadgeState.Failure);
										break;
									}

								case LabelOutcome.Unspecified:
									{
										ugsBadgeStates.Add(labelIdx, UgsBadgeState.Skipped);
										break;
									}
							}
							break;
						}

					case LabelState.Running:
						{
							ugsBadgeStates.Add(labelIdx, UgsBadgeState.Starting);
							break;
						}

					case LabelState.Unspecified:
						{
							ugsBadgeStates.Add(labelIdx, UgsBadgeState.Skipped);
							break;
						}
				}
			}
			return ugsBadgeStates;
		}

		/// <summary>
		/// Gets the state of a job, as a label that includes all steps
		/// </summary>
		/// <param name="job">The job to query</param>
		/// <param name="stepForNode">Map from node to step</param>
		/// <param name="newState">Receives the state of the label</param>
		/// <param name="newOutcome">Receives the outcome of the label</param>
		public static void GetJobState(this IJob job, IReadOnlyDictionary<NodeRef, IJobStep> stepForNode, out LabelState newState, out LabelOutcome newOutcome)
		{
			List<NodeRef> nodes = [];
			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep step in batch.Steps)
				{
					nodes.Add(new NodeRef(batch.GroupIdx, step.NodeIdx));
				}
			}
			GetLabelState(nodes, stepForNode, out newState, out newOutcome);
		}

		/// <summary>
		/// Gets the state of a label
		/// </summary>
		/// <param name="includedNodes">Nodes to include in this label</param>
		/// <param name="stepForNode">Map from node to step</param>
		/// <param name="newState">Receives the state of the label</param>
		/// <param name="newOutcome">Receives the outcome of the label</param>
		public static void GetLabelState(IEnumerable<NodeRef> includedNodes, IReadOnlyDictionary<NodeRef, IJobStep> stepForNode, out LabelState newState, out LabelOutcome newOutcome)
		{
			newState = LabelState.Complete;
			newOutcome = LabelOutcome.Success;
			foreach (NodeRef includedNodeRef in includedNodes)
			{
				IJobStep? includedStep;
				if (stepForNode.TryGetValue(includedNodeRef, out includedStep))
				{
					// Update the state
					if (includedStep.State != JobStepState.Completed && includedStep.State != JobStepState.Skipped && includedStep.State != JobStepState.Aborted)
					{
						newState = LabelState.Running;
					}

					// Update the outcome
					if (includedStep.State == JobStepState.Skipped || includedStep.State == JobStepState.Aborted || includedStep.Outcome == JobStepOutcome.Failure)
					{
						newOutcome = LabelOutcome.Failure;
					}
					else if (includedStep.Outcome == JobStepOutcome.Warnings && newOutcome == LabelOutcome.Success)
					{
						newOutcome = LabelOutcome.Warnings;
					}
				}
			}
		}

		/// <summary>
		/// Gets a key attached to all artifacts produced for a job
		/// </summary>
		public static string GetArtifactKey(this IJob job)
		{
			return $"job:{job.Id}";
		}

		/// <summary>
		/// Gets a key attached to all artifacts produced for a job step
		/// </summary>
		public static string GetArtifactKey(this IJob job, IJobStep jobStep)
		{
			return $"job:{job.Id}/step:{jobStep.Id}";
		}

		/// <inheritdoc cref="IJob.TrySkipBatchAsync(JobStepBatchId, JobStepBatchError, CancellationToken)"/>
		public static async Task<IJob?> SkipBatchAsync(this IJob job, JobStepBatchId batchId, JobStepBatchError error, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				IJob? newJob = await job.TrySkipBatchAsync(batchId, error, cancellationToken);
				if (newJob != null)
				{
					return newJob;
				}

				newJob = await job.RefreshAsync(cancellationToken);
				if (newJob == null)
				{
					return null;
				}

				job = newJob;
			}
		}

		/// <inheritdoc cref="IJob.TrySkipAllBatchesAsync(JobStepBatchError, CancellationToken)"/>
		public static async Task<IJob?> SkipAllBatchesAsync(this IJob job, JobStepBatchError reason, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				IJob? newJob = await job.TrySkipAllBatchesAsync(reason, cancellationToken);
				if (newJob != null)
				{
					return newJob;
				}

				newJob = await job.RefreshAsync(cancellationToken);
				if (newJob == null)
				{
					return null;
				}

				job = newJob;
			}
		}
	}

	/// <summary>
	/// Stores information about a batch of job steps
	/// </summary>
	public interface IJobStepBatch
	{
		/// <summary>
		/// Job that this batch belongs to
		/// </summary>
		public IJob Job { get; }

		/// <summary>
		/// Unique id for this group
		/// </summary>
		public JobStepBatchId Id { get; }

		/// <summary>
		/// The type of agent to execute this group
		/// </summary>
		public string AgentType { get; }

		/// <summary>
		/// The log file id for this batch
		/// </summary>
		public LogId? LogId { get; }

		/// <summary>
		/// The node group for this batch
		/// </summary>
		public INodeGroup Group { get; }

		/// <summary>
		/// Index of the group being executed
		/// </summary>
		public int GroupIdx { get; }

		/// <summary>
		/// The state of this group
		/// </summary>
		public JobStepBatchState State { get; }

		/// <summary>
		/// Error associated with this group
		/// </summary>
		public JobStepBatchError Error { get; }

		/// <summary>
		/// Steps within this run
		/// </summary>
		public IReadOnlyList<IJobStep> Steps { get; }

		/// <summary>
		/// The pool that this agent was taken from
		/// </summary>
		public PoolId? PoolId { get; }

		/// <summary>
		/// The agent assigned to execute this group
		/// </summary>
		public AgentId? AgentId { get; }

		/// <summary>
		/// The agent session that is executing this group
		/// </summary>
		public SessionId? SessionId { get; }

		/// <summary>
		/// The lease that's executing this group
		/// </summary>
		public LeaseId? LeaseId { get; }

		/// <summary>
		/// The weighted priority of this batch for the scheduler
		/// </summary>
		public int SchedulePriority { get; }

		/// <summary>
		/// Time at which the group became ready (UTC).
		/// </summary>
		public DateTime? ReadyTimeUtc { get; }

		/// <summary>
		/// Time at which the group started (UTC).
		/// </summary>
		public DateTime? StartTimeUtc { get; }

		/// <summary>
		/// Time at which the group finished (UTC)
		/// </summary>
		public DateTime? FinishTimeUtc { get; }
	}

	/// <summary>
	/// Extension methods for IJobStepBatch
	/// </summary>
	public static class JobStepBatchExtensions
	{
		/// <summary>
		/// Attempts to get a step with the given id
		/// </summary>
		/// <param name="batch">The batch to search</param>
		/// <param name="stepId">The step id</param>
		/// <param name="step">On success, receives the step object</param>
		/// <returns>True if the step was found</returns>
		public static bool TryGetStep(this IJobStepBatch batch, JobStepId stepId, [NotNullWhen(true)] out IJobStep? step)
		{
			step = batch.Steps.FirstOrDefault(x => x.Id == stepId);
			return step != null;
		}

		/// <summary>
		/// Determines if new steps can be appended to this batch. We do not allow this after the last step has been completed, because the agent is shutting down.
		/// </summary>
		/// <param name="batch">The batch to search</param>
		/// <returns>True if new steps can be appended to this batch</returns>
		public static bool CanBeAppendedTo(this IJobStepBatch batch)
		{
			return batch.State <= JobStepBatchState.Running;
		}

		/// <summary>
		/// Gets the wait time for this batch
		/// </summary>
		/// <param name="batch">The batch to search</param>
		/// <returns>Wait time for the batch</returns>
		public static TimeSpan? GetWaitTime(this IJobStepBatch batch)
		{
			if (batch.StartTimeUtc == null || batch.ReadyTimeUtc == null)
			{
				return null;
			}
			else
			{
				return batch.StartTimeUtc.Value - batch.ReadyTimeUtc.Value;
			}
		}

		/// <summary>
		/// Gets the initialization time for this batch
		/// </summary>
		/// <param name="batch">The batch to search</param>
		/// <returns>Initialization time for this batch</returns>
		public static TimeSpan? GetInitTime(this IJobStepBatch batch)
		{
			if (batch.StartTimeUtc != null)
			{
				foreach (IJobStep step in batch.Steps)
				{
					if (step.StartTimeUtc != null)
					{
						return step.StartTimeUtc - batch.StartTimeUtc.Value;
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Get the dependencies required for this batch to start, taking run-early nodes into account
		/// </summary>
		/// <param name="batch">The batch to search</param>
		/// <param name="groups">List of node groups</param>
		/// <returns>Set of nodes that must have completed for this batch to start</returns>
		public static HashSet<INode> GetStartDependencies(this IJobStepBatch batch, IReadOnlyList<INodeGroup> groups)
		{
			List<INode> batchNodes = batch.Steps.ConvertAll(x => groups[batch.GroupIdx].Nodes[x.NodeIdx]);
			return GetStartDependencies(batchNodes, groups);
		}

		/// <summary>
		/// Get the dependencies required for this batch to start, taking run-early nodes into account
		/// </summary>
		/// <param name="batchNodes">Nodes in the batch to search</param>
		/// <param name="groups">List of node groups</param>
		/// <returns>Set of nodes that must have completed for this batch to start</returns>
		public static HashSet<INode> GetStartDependencies(IEnumerable<INode> batchNodes, IReadOnlyList<INodeGroup> groups)
		{
			// Find all the nodes that this group will start with.
			List<INode> nodes = [.. batchNodes];
			if (nodes.Any(x => x.RunEarly))
			{
				nodes.RemoveAll(x => !x.RunEarly);
			}

			// Find all their dependencies
			HashSet<INode> dependencies = [];
			foreach (INode node in nodes)
			{
				dependencies.UnionWith(node.InputDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx]));
				dependencies.UnionWith(node.OptionalInputDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx]));
				dependencies.UnionWith(node.OrderDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx]));
			}

			// Exclude all the dependencies within the same group
			dependencies.ExceptWith(batchNodes);
			return dependencies;
		}
	}

	/// <summary>
	/// Embedded jobstep document
	/// </summary>
	public interface IJobStep
	{
		/// <summary>
		/// Job that this step belongs to
		/// </summary>
		public IJob Job { get; }

		/// <summary>
		/// Batch that this step belongs to
		/// </summary>
		public IJobStepBatch Batch { get; }

		/// <summary>
		/// Unique ID assigned to this jobstep. A new id is generated whenever a jobstep's order is changed.
		/// </summary>
		public JobStepId Id { get; }

		/// <summary>
		/// The node for this step
		/// </summary>
		public INode Node { get; }

		/// <summary>
		/// Index of the node which this jobstep is to execute
		/// </summary>
		public int NodeIdx { get; }

		/// <summary>
		/// The name of this node 
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// References to inputs for this node
		/// </summary>
		public IReadOnlyList<JobStepOutputRef> Inputs { get; }

		/// <summary>
		/// References to optional inputs for this node
		/// </summary>
		public IReadOnlyList<JobStepOutputRef> OptionalInputs { get; }

		/// <summary>
		/// List of output names
		/// </summary>
		public IReadOnlyList<string> OutputNames { get; }

		/// <summary>
		/// Indices of nodes which must have succeeded for this node to run
		/// </summary>
		public IReadOnlyList<JobStepId> InputDependencies { get; }

		/// <summary>
		/// Indices of nodes which could have succeeded for this node to run
		/// </summary>
		public IReadOnlyList<JobStepId> OptionalInputDependencies { get; }

		/// <summary>
		/// Indices of nodes which must have completed for this node to run
		/// </summary>
		public IReadOnlyList<JobStepId> OrderDependencies { get; }

		/// <summary>
		/// Whether this node can be run multiple times
		/// </summary>
		public bool AllowRetry { get; }

		/// <summary>
		/// This node can start running early, before dependencies of other nodes in the same group are complete
		/// </summary>
		public bool RunEarly { get; }

		/// <summary>
		/// Whether to include warnings in the output (defaults to true)
		/// </summary>
		public bool Warnings { get; }

		/// <summary>
		/// List of credentials required for this node. Each entry maps an environment variable name to a credential in the form "CredentialName.PropertyName".
		/// </summary>
		public IReadOnlyDictionary<string, string>? Credentials { get; }

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public IReadOnlyNodeAnnotations Annotations { get; }

		/// <summary>
		/// Metadata for this node
		/// </summary>
		public IReadOnlyList<string> Metadata { get; }

		/// <summary>
		/// Current state of the job step. This is updated automatically when runs complete.
		/// </summary>
		public JobStepState State { get; }

		/// <summary>
		/// Current outcome of the jobstep
		/// </summary>
		public JobStepOutcome Outcome { get; }

		/// <summary>
		/// Error from executing this step
		/// </summary>
		public JobStepError Error { get; }

		/// <summary>
		/// The log id for this step
		/// </summary>
		public LogId? LogId { get; }

		/// <summary>
		/// Unique id for notifications
		/// </summary>
		public NotificationTriggerId? NotificationTriggerId { get; }

		/// <summary>
		/// Time at which the batch transitioned to the ready state (UTC).
		/// </summary>
		public DateTime? ReadyTimeUtc { get; }

		/// <summary>
		/// Time at which the batch transitioned to the executing state (UTC).
		/// </summary>
		public DateTime? StartTimeUtc { get; }

		/// <summary>
		/// Time at which the run finished (UTC)
		/// </summary>
		public DateTime? FinishTimeUtc { get; }

		/// <summary>
		/// Override for the priority of this step
		/// </summary>
		public Priority? Priority { get; }

		/// <summary>
		/// If a retry is requested, stores the name of the user that requested it
		/// </summary>
		public UserId? RetriedByUserId { get; }

		/// <summary>
		/// Signal if a step should be aborted
		/// </summary>
		public bool AbortRequested { get; }

		/// <summary>
		/// If an abort is requested, stores the id of the user that requested it
		/// </summary>
		public UserId? AbortedByUserId { get; }

		/// <summary>
		/// Optional reason for why the job step was canceled
		/// </summary>
		public string? CancellationReason { get; }

		/// <summary>
		/// List of reports for this step
		/// </summary>
		public IReadOnlyList<IJobReport>? Reports { get; }

		/// <summary>
		/// List of jobs this step has spawned
		/// </summary>
		public IReadOnlyList<JobId>? SpawnedJobs { get; }

		/// <summary>
		/// Reports for this jobstep.
		/// </summary>
		public IReadOnlyDictionary<string, string>? Properties { get; }
	}

	/// <summary>
	/// Extension methods for job steps
	/// </summary>
	public static class JobStepExtensions
	{
		/// <summary>
		/// Determines if a jobstep state is completed, skipped, or aborted.
		/// </summary>
		/// <returns>True if the step is completed, skipped, or aborted</returns>
		public static bool IsPendingState(JobStepState state)
		{
			return state != JobStepState.Aborted && state != JobStepState.Completed && state != JobStepState.Skipped;
		}

		/// <summary>
		/// Determines if a jobstep is done by checking to see if it is completed, skipped, or aborted.
		/// </summary>
		/// <returns>True if the step is completed, skipped, or aborted</returns>
		public static bool IsPending(this IJobStep step)
			=> IsPendingState(step.State);

		/// <summary>
		/// Determine if a step should be timed out
		/// </summary>
		/// <param name="step"></param>
		/// <param name="utcNow"></param>
		/// <returns></returns>
		public static bool HasTimedOut(this IJobStep step, DateTime utcNow)
		{
			if (step.State == JobStepState.Running && step.StartTimeUtc != null)
			{
				TimeSpan elapsed = utcNow - step.StartTimeUtc.Value;
				if (elapsed > TimeSpan.FromHours(24.0))
				{
					return true;
				}
			}
			return false;
		}
	}

	/// <summary>
	/// Cumulative timing information to reach a certain point in a job
	/// </summary>
	public class TimingInfo
	{
		/// <summary>
		/// Wait time on the critical path
		/// </summary>
		public TimeSpan? TotalWaitTime { get; set; }

		/// <summary>
		/// Sync time on the critical path
		/// </summary>
		public TimeSpan? TotalInitTime { get; set; }

		/// <summary>
		/// Duration to this point
		/// </summary>
		public TimeSpan? TotalTimeToComplete { get; set; }

		/// <summary>
		/// Average wait time to this point
		/// </summary>
		public TimeSpan? AverageTotalWaitTime { get; set; }

		/// <summary>
		/// Average sync time to this point
		/// </summary>
		public TimeSpan? AverageTotalInitTime { get; set; }

		/// <summary>
		/// Average duration to this point
		/// </summary>
		public TimeSpan? AverageTotalTimeToComplete { get; set; }

		/// <summary>
		/// Individual step timing information
		/// </summary>
		public IJobStepTiming? StepTiming { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public TimingInfo()
		{
			TotalWaitTime = TimeSpan.Zero;
			TotalInitTime = TimeSpan.Zero;
			TotalTimeToComplete = TimeSpan.Zero;

			AverageTotalWaitTime = TimeSpan.Zero;
			AverageTotalInitTime = TimeSpan.Zero;
			AverageTotalTimeToComplete = TimeSpan.Zero;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">The timing info object to copy from</param>
		public TimingInfo(TimingInfo other)
		{
			TotalWaitTime = other.TotalWaitTime;
			TotalInitTime = other.TotalInitTime;
			TotalTimeToComplete = other.TotalTimeToComplete;

			AverageTotalWaitTime = other.AverageTotalWaitTime;
			AverageTotalInitTime = other.AverageTotalInitTime;
			AverageTotalTimeToComplete = other.AverageTotalTimeToComplete;
		}

		/// <summary>
		/// Modifies this timing to wait for another timing
		/// </summary>
		/// <param name="other">The other node to wait for</param>
		public void WaitFor(TimingInfo other)
		{
			if (TotalTimeToComplete != null)
			{
				if (other.TotalTimeToComplete == null || other.TotalTimeToComplete.Value > TotalTimeToComplete.Value)
				{
					TotalInitTime = other.TotalInitTime;
					TotalWaitTime = other.TotalWaitTime;
					TotalTimeToComplete = other.TotalTimeToComplete;
				}
			}

			if (AverageTotalTimeToComplete != null)
			{
				if (other.AverageTotalTimeToComplete == null || other.AverageTotalTimeToComplete.Value > AverageTotalTimeToComplete.Value)
				{
					AverageTotalInitTime = other.AverageTotalInitTime;
					AverageTotalWaitTime = other.AverageTotalWaitTime;
					AverageTotalTimeToComplete = other.AverageTotalTimeToComplete;
				}
			}
		}

		/// <summary>
		/// Waits for all the given timing info objects to complete
		/// </summary>
		/// <param name="others">Other timing info objects to wait for</param>
		public void WaitForAll(IEnumerable<TimingInfo> others)
		{
			foreach (TimingInfo other in others)
			{
				WaitFor(other);
			}
		}

		/// <summary>
		/// Constructs a new TimingInfo object which represents the last TimingInfo to finish
		/// </summary>
		/// <param name="others">TimingInfo objects to wait for</param>
		/// <returns>New TimingInfo instance</returns>
		public static TimingInfo Max(IEnumerable<TimingInfo> others)
		{
			TimingInfo timingInfo = new();
			timingInfo.WaitForAll(others);
			return timingInfo;
		}

		/// <summary>
		/// Copies this info to a repsonse object
		/// </summary>
		public void CopyToResponse(GetTimingInfoResponse response)
		{
			response.TotalWaitTime = (float?)TotalWaitTime?.TotalSeconds;
			response.TotalInitTime = (float?)TotalInitTime?.TotalSeconds;
			response.TotalTimeToComplete = (float?)TotalTimeToComplete?.TotalSeconds;

			response.AverageTotalWaitTime = (float?)AverageTotalWaitTime?.TotalSeconds;
			response.AverageTotalInitTime = (float?)AverageTotalInitTime?.TotalSeconds;
			response.AverageTotalTimeToComplete = (float?)AverageTotalTimeToComplete?.TotalSeconds;
		}
	}

	/// <summary>
	/// Step completion information
	/// </summary>
	public class JobStepsCompletionInfo
	{
		/// <summary>
		/// The total count of steps passed within a job.
		/// </summary>
		public int StepPassCount { get; set; }

		/// <summary>
		/// The total count of steps with a warning result within a job.
		/// </summary>
		public int StepWarningCount { get; set; }

		/// <summary>
		/// The total count of steps with a failure result within a job.
		/// </summary>
		public int StepFailureCount { get; set; }

		/// <summary>
		/// The total number of steps within a job.
		/// </summary>
		public int StepTotalCount { get; set; }

		/// <summary>
		/// The total step time for the job, in seconds.
		/// </summary>
		public float JobStepsTotalTime { get; set; }

		/// <summary>
		/// The pass count as compared to the total step count.
		/// </summary>
		public float PassRatio => StepTotalCount == 0 ? 0 : StepPassCount / (float)StepTotalCount;

		/// <summary>
		/// The pass and warning count as compared to the total step count.
		/// </summary>
		public float PassWithWarningRatio => StepTotalCount == 0 ? 0 : (StepPassCount + StepWarningCount) / (float)StepTotalCount;

		/// <summary>
		/// The warning count as compared to the total step count.
		/// </summary>
		public float WarningRatio => StepTotalCount == 0 ? 0 : StepWarningCount / (float)StepTotalCount;

		/// <summary>
		/// The failure count as compared to the total step count.
		/// </summary>
		public float FailureRatio => StepTotalCount == 0 ? 0 : StepFailureCount / (float)StepTotalCount;
	}

	/// <summary>
	/// Information about a chained job trigger
	/// </summary>
	public interface IChainedJob
	{
		/// <summary>
		/// The target to monitor
		/// </summary>
		public string Target { get; }

		/// <summary>
		/// The template to trigger on success
		/// </summary>
		public TemplateId TemplateRefId { get; }

		/// <summary>
		/// The triggered job id
		/// </summary>
		public JobId? JobId { get; }

		/// <summary>
		/// Whether to run the latest change, or default change for the template, when starting the new job. Uses same change as the triggering job by default.
		/// </summary>
		public bool UseDefaultChangeForTemplate { get; }
	}

	/// <summary>
	/// Report for a job or jobstep
	/// </summary>
	public interface IJobReport
	{
		/// <summary>
		/// Name of the report
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Where to render the report
		/// </summary>
		JobReportPlacement Placement { get; }

		/// <summary>
		/// The artifact id
		/// </summary>
		ArtifactId? ArtifactId { get; }

		/// <summary>
		/// Inline data for the report
		/// </summary>
		string? Content { get; }

		/// <summary>
		/// The job/step message
		/// </summary>
		string? Message { get; }

		/// <summary>
		/// The message severity 
		/// </summary>
		string? Severity { get; }

		/// <summary>
		/// The message color
		/// </summary>
		string? Color { get; }
	}

	/// <summary>
	/// Implementation of IReport
	/// </summary>
	public class JobReport : IJobReport
	{
		/// <inheritdoc/>
		public string Name { get; set; } = String.Empty;

		/// <inheritdoc/>
		public JobReportPlacement Placement { get; set; }

		/// <inheritdoc/>
		public ArtifactId? ArtifactId { get; set; }

		/// <inheritdoc/>
		public string? Content { get; set; }

		/// <inheritdoc/>
		public string? Message { get; set; }

		/// <inheritdoc/>
		public string? Severity { get; set; }

		/// <inheritdoc/>
		public string? Color { get; set; }
	}
}
