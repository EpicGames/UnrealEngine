// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Searchable reference to a jobstep
	/// </summary>
	public interface IJobStepRef
	{
		/// <summary>
		/// Globally unique identifier for the jobstep being referenced
		/// </summary>
		public JobStepRefId Id { get; }

		/// <summary>
		/// The job the jobstep belongs to.
		/// </summary>
		/// <remarks>This is optional due to backwards compatibility for previously serialized items.</remarks>
		public JobId? JobId { get; }

		/// <summary>
		/// The batch the jobstep belongs to.
		/// </summary>
		/// <remarks>This is optional due to backwards compatibility for previously serialized items.</remarks>
		public JobStepBatchId? BatchId { get; }

		/// <summary>
		/// The step the job step represents.
		/// </summary>
		/// <remarks>This is optional due to backwards compatibility for previously serialized items.</remarks>
		public JobStepId? StepId { get; }

		/// <summary>
		/// Name of the job
		/// </summary>
		public string JobName { get; }

		/// <summary>
		/// Name of the node
		/// </summary>
		public string NodeName { get; }

		/// <summary>
		/// Unique id of the stream containing the job
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// Template for the job being executed
		/// </summary>
		public TemplateId TemplateId { get; }

		/// <summary>
		/// The change number being built
		/// </summary>
		public CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// Log for this step
		/// </summary>
		public LogId? LogId { get; }

		/// <summary>
		/// The agent type
		/// </summary>
		public PoolId? PoolId { get; }

		/// <summary>
		/// The agent id
		/// </summary>
		public AgentId? AgentId { get; }

		/// <summary>
		/// State of the step.
		/// </summary>
		public JobStepState? State { get; }

		/// <summary>
		/// Outcome of the step, once complete.
		/// </summary>
		public JobStepOutcome? Outcome { get; }

		/// <summary>
		/// Metadata for the step
		/// </summary>
		public IReadOnlyList<string>? Metadata { get; }

		/// <summary>
		/// Issues ids affecting this job step
		/// </summary>
		public IReadOnlyList<int>? IssueIds { get; }

		/// <summary>
		/// Whether this step should update issues
		/// </summary>
		public bool UpdateIssues { get; }

		/// <summary>
		/// The last change that succeeded. Note that this is only set when the ref is updated; it is not necessarily consistent with steps run later.
		/// </summary>
		public CommitIdWithOrder? LastSuccess { get; }

		/// <summary>
		/// The last change that succeeded, or completed a warning. See <see cref="LastSuccess"/>.
		/// </summary>
		public CommitIdWithOrder? LastWarning { get; }

		/// <summary>
		/// Time taken for the batch containing this batch to start after it became ready
		/// </summary>
		public float BatchWaitTime { get; }

		/// <summary>
		/// Time taken for this batch to initialize
		/// </summary>
		public float BatchInitTime { get; }

		/// <summary>
		/// Time at which the job started
		/// </summary>
		public DateTime JobStartTimeUtc { get; }

		/// <summary>
		/// Time at which the step started.
		/// </summary>
		public DateTime StartTimeUtc { get; }

		/// <summary>
		/// Time at which the step finished.
		/// </summary>
		public DateTime? FinishTimeUtc { get; }
	}
}
