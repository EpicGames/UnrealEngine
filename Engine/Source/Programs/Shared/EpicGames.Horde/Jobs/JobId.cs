// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Identifier for a job
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<JobId, JobIdConverter>))]
	[BinaryIdConverter(typeof(JobIdConverter))]
	public readonly record struct JobId(BinaryId Id)
	{
		/// <summary>
		/// Constant value for an empty job id
		/// </summary>
		public static JobId Empty { get; } = default;

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static JobId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class JobIdConverter : BinaryIdConverter<JobId>
	{
		/// <inheritdoc/>
		public override JobId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(JobId value) => value.Id;
	}
}
