// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Text.Json.Serialization;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;

#pragma warning disable CA1056 // Change the type of property 'JobContainerOptions.ImageUrl' from 'string' to 'System.Uri'
#pragma warning disable CA2227 // Change 'Outcomes' to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// State of the job
	/// </summary>
	public enum JobState
	{
		/// <summary>
		/// Waiting for resources
		/// </summary>
		Waiting,

		/// <summary>
		/// Currently running one or more steps
		/// </summary>
		Running,

		/// <summary>
		/// All steps have completed
		/// </summary>
		Complete,
	}

	/// <summary>
	/// Read-only interface for job options
	/// </summary>
	public interface IReadOnlyJobOptions
	{
		/// <summary>
		/// Name of the executor to use
		/// </summary>
		string? Executor { get; }

		/// <summary>
		/// Whether to execute using Wine emulation on Linux
		/// </summary>
		bool? UseWine { get; }

		/// <summary>
		/// Executes the job lease in a separate process
		/// </summary>
		bool? RunInSeparateProcess { get; }

		/// <summary>
		/// What workspace materializer to use in WorkspaceExecutor. Will override any value from workspace config.
		/// </summary>
		string? WorkspaceMaterializer { get; }

		/// <summary>
		/// Options for executing a job inside a container
		/// </summary>
		IReadOnlyJobContainerOptions Container { get; }

		/// <summary>
		/// Number of days after which to expire jobs
		/// </summary>
		int? ExpireAfterDays { get; }

		/// <summary>
		/// Name of the driver to use
		/// </summary>
		string? Driver { get; }
	}

	/// <summary>
	/// Options for executing a job
	/// </summary>
	public class JobOptions : IReadOnlyJobOptions
	{
		/// <inheritdoc/>
		public string? Executor { get; set; }

		/// <inheritdoc/>
		public bool? UseWine { get; set; }

		/// <inheritdoc/>
		public bool? RunInSeparateProcess { get; set; }

		/// <inheritdoc/>
		public string? WorkspaceMaterializer { get; set; }

		/// <inheritdoc/>
		public JobContainerOptions Container { get; set; } = new JobContainerOptions();

		/// <inheritdoc/>
		public int? ExpireAfterDays { get; set; }

		/// <inheritdoc/>
		public string? Driver { get; set; }

		IReadOnlyJobContainerOptions IReadOnlyJobOptions.Container => Container;

		/// <summary>
		/// Merge defaults from another options object
		/// </summary>
		public void MergeDefaults(JobOptions other)
		{
			Executor ??= other.Executor;
			UseWine ??= other.UseWine;
			RunInSeparateProcess ??= other.RunInSeparateProcess;
			WorkspaceMaterializer ??= other.WorkspaceMaterializer;
			Container.MergeDefaults(other.Container);
			ExpireAfterDays ??= other.ExpireAfterDays;
			Driver ??= other.Driver;
		}
	}

	/// <summary>
	/// Options for a job container
	/// </summary>
	public interface IReadOnlyJobContainerOptions
	{
		/// <summary>
		/// Whether to execute job inside a container
		/// </summary>
		public bool? Enabled { get; }

		/// <summary>
		/// Image URL to container, such as "quay.io/podman/hello"
		/// </summary>
		public string? ImageUrl { get; }

		/// <summary>
		/// Container engine executable (docker or with full path like /usr/bin/podman)
		/// </summary>
		public string? ContainerEngineExecutable { get; }

		/// <summary>
		/// Additional arguments to pass to container engine
		/// </summary>
		public string? ExtraArguments { get; }
	}

	/// <summary>
	/// Options for executing a job inside a container
	/// </summary>
	public class JobContainerOptions : IReadOnlyJobContainerOptions
	{
		/// <summary>
		/// Whether to execute job inside a container
		/// </summary>
		public bool? Enabled { get; set; }

		/// <summary>
		/// Image URL to container, such as "quay.io/podman/hello"
		/// </summary>
		public string? ImageUrl { get; set; }

		/// <summary>
		/// Container engine executable (docker or with full path like /usr/bin/podman)
		/// </summary>
		public string? ContainerEngineExecutable { get; set; }

		/// <summary>
		/// Additional arguments to pass to container engine
		/// </summary>
		public string? ExtraArguments { get; set; }

		/// <summary>
		/// Merge defaults from another options object
		/// </summary>
		public void MergeDefaults(JobContainerOptions other)
		{
			Enabled ??= other.Enabled;
			ImageUrl ??= other.ImageUrl;
			ContainerEngineExecutable ??= other.ContainerEngineExecutable;
			ExtraArguments ??= other.ExtraArguments;
		}
	}

	/// <summary>
	/// Query selecting the base changelist to use
	/// </summary>
	public class ChangeQueryMessage : IChangeQuery
	{
		/// <inheritdoc/>
		public string? Name { get; set; }

		/// <inheritdoc/>
		public Condition? Condition { get; set; }

		/// <inheritdoc/>
		public TemplateId? TemplateId { get; set; }

		/// <inheritdoc/>
		public string? Target { get; set; }

		/// <inheritdoc cref="IChangeQuery.Outcomes"/>
		public List<JobStepOutcome>? Outcomes { get; set; }
		IReadOnlyList<JobStepOutcome>? IChangeQuery.Outcomes => Outcomes;

		/// <inheritdoc/>
		public CommitTag? CommitTag { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ChangeQueryMessage()
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public ChangeQueryMessage(IChangeQuery other)
		{
			Name = other.Name;
			Condition = other.Condition;
			TemplateId = other.TemplateId;
			Target = other.Target;
			Outcomes = other.Outcomes?.ToList();
			CommitTag = other.CommitTag;
		}
	}

	/// <summary>
	/// Parameters required to create a job
	/// </summary>
	public class CreateJobRequest
	{
		/// <summary>
		/// The stream that this job belongs to
		/// </summary>
		[Required]
		public StreamId StreamId { get; set; }

		/// <summary>
		/// The template for this job
		/// </summary>
		[Required]
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// Name of the job
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// The changelist number to build. Can be null for latest.
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int? Change
		{
			get => _change ?? _commitId?.GetPerforceChangeOrMinusOne();
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// The changelist number to build. Can be null for latest.
		/// </summary>
		public CommitId? CommitId
		{
			get => _commitId ?? CommitId.FromPerforceChange(_change);
			set => _commitId = value;
		}
		CommitId? _commitId;

		/// <summary>
		/// Parameters to use when selecting the change to execute at.
		/// </summary>
		[Obsolete("Use ChangeQueries instead")]
		public ChangeQueryMessage? ChangeQuery
		{
			get => (ChangeQueries != null && ChangeQueries.Count > 0) ? ChangeQueries[0] : null;
			set => ChangeQueries = (value == null) ? null : [value];
		}

		/// <summary>
		/// List of change queries to evaluate
		/// </summary>
		public List<ChangeQueryMessage>? ChangeQueries { get; set; }

		/// <summary>
		/// The preflight changelist number
		/// </summary>
		[Obsolete("Use PreflightCommitId instead")]
		public int? PreflightChange
		{
			get => _preflightChange ?? _preflightCommitId?.GetPerforceChangeOrMinusOne();
			set => _preflightChange = value;
		}
		int? _preflightChange;

		/// <summary>
		/// The preflight commit
		/// </summary>
		public CommitId? PreflightCommitId
		{
			get => _preflightCommitId ?? CommitId.FromPerforceChange(_preflightChange);
			set => _preflightCommitId = value;
		}
		CommitId? _preflightCommitId;

		/// <summary>
		/// Job options
		/// </summary>
		public JobOptions? JobOptions { get; set; }

		/// <summary>
		/// Priority for the job
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// Whether to automatically submit the preflighted change on completion
		/// </summary>
		public bool? AutoSubmit { get; set; }

		/// <summary>
		/// Whether to update issues based on the outcome of this job
		/// </summary>
		public bool? UpdateIssues { get; set; }

		/// <summary>
		/// Values for the template parameters
		/// </summary>
		public Dictionary<ParameterId, string>? Parameters { get; set; }

		/// <summary>
		/// Arguments for the job
		/// </summary>
		public List<string>? Arguments { get; set; }

		/// <summary>
		/// Additional arguments for the job
		/// </summary>
		public List<string>? AdditionalArguments { get; set; }

		/// <summary>
		/// Targets for the job. Will override any parameters specified in the Arguments or Parameters section if specified.
		/// </summary>
		public List<string>? Targets { get; set; }

		/// <summary>
		/// The parent job id if any
		/// </summary>
		public string? ParentJobId { get; set; }

		/// <summary>
		/// The parent step id if any
		/// </summary>
		public string? ParentJobStepId { get; set; }

		/// <summary>
		/// Run the job as the scheduler for debugging purposes, requires debugging ACL permission
		/// </summary>
		public bool? RunAsScheduler { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		public CreateJobRequest(StreamId streamId, TemplateId templateId)
		{
			StreamId = streamId;
			TemplateId = templateId;
		}
	}

	/// <summary>
	/// Response from creating a new job
	/// </summary>
	public class CreateJobResponse
	{
		/// <summary>
		/// Unique id for the new job
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id for the new job</param>
		public CreateJobResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Updates an existing job
	/// </summary>
	public class UpdateJobRequest
	{
		/// <summary>
		/// New name for the job
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// New priority for the job
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// Set whether the job should be automatically submitted or not
		/// </summary>
		public bool? AutoSubmit { get; set; }

		/// <summary>
		/// Mark this job as aborted
		/// </summary>
		public bool? Aborted { get; set; }

		/// <summary>
		/// Optional reason the job was canceled
		/// </summary>
		public string? CancellationReason { get; set; }

		/// <summary>
		/// New list of arguments for the job. Only -Target= arguments can be modified after the job has started.
		/// </summary>
		public List<string>? Arguments { get; set; }
	}

	/// <summary>
	/// Job metadata put request
	/// </summary>
	public class PutJobMetadataRequest
	{
		/// <summary>
		/// Meta data to append to the job
		/// </summary>
		public List<string>? JobMetaData { get; set; }

		/// <summary>
		/// Step meta data to append, JobStepId => list of meta data strings
		/// </summary>
		public Dictionary<string, List<string>>? StepMetaData { get; set; }
	}

	/// <summary>
	/// Placement for a job report
	/// </summary>
	public enum JobReportPlacement
	{
		/// <summary>
		/// On a panel of its own
		/// </summary>
		Panel = 0,

		/// <summary>
		/// In the summary panel
		/// </summary>
		Summary = 1
	}

	/// <summary>
	/// Information about a report associated with a job
	/// </summary>
	public class GetJobReportResponse
	{
		/// <summary>
		/// Name of the report
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Report placement
		/// </summary>
		public JobReportPlacement Placement { get; set; }

		/// <summary>
		/// The artifact id
		/// </summary>
		public string? ArtifactId { get; set; }

		/// <summary>
		/// Content for the report
		/// </summary>
		public string? Content { get; set; }

		/// <summary>
		/// The job/step message
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// The job/step message severity
		/// </summary>
		public string? Severity { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetJobReportResponse(string name, JobReportPlacement placement)
		{
			Name = name;
			Placement = placement;
		}
	}

	/// <summary>
	/// Information about a job
	/// </summary>
	public class GetJobResponse
	{
		/// <summary>
		/// Unique Id for the job
		/// </summary>
		public JobId Id { get; set; }

		/// <summary>
		/// Name of the job
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Unique id of the stream containing this job
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// The changelist number to build
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int Change
		{
			get => _change ?? _commitId?.GetPerforceChangeOrMinusOne() ?? 0;
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// The commit to build
		/// </summary>
		public CommitIdWithOrder CommitId
		{
			get => _commitId ?? CommitIdWithOrder.FromPerforceChange(_change) ?? CommitIdWithOrder.Empty;
			set => _commitId = value;
		}
		CommitIdWithOrder? _commitId;

		/// <summary>
		/// The code changelist
		/// </summary>
		[Obsolete("Use CodeCommitId instead")]
		public int? CodeChange
		{
			get => _codeChange ?? _codeCommitId?.GetPerforceChangeOrMinusOne();
			set => _codeChange = value;
		}
		int? _codeChange;

		/// <summary>
		/// The code commit to build
		/// </summary>
		public CommitIdWithOrder? CodeCommitId
		{
			get => _codeCommitId ?? CommitIdWithOrder.FromPerforceChange(_codeChange);
			set => _codeCommitId = value;
		}
		CommitIdWithOrder? _codeCommitId;

		/// <summary>
		/// The preflight changelist number
		/// </summary>
		[Obsolete("Use PreflightCommitId instead")]
		public int? PreflightChange
		{
			get => _preflightChange ?? _preflightCommitId?.GetPerforceChangeOrMinusOne();
			set => _preflightChange = value;
		}
		int? _preflightChange;

		/// <summary>
		/// The preflight commit
		/// </summary>
		public CommitId? PreflightCommitId
		{
			get => _preflightCommitId ?? Commits.CommitId.FromPerforceChange(_preflightChange);
			set => _preflightCommitId = value;
		}
		CommitId? _preflightCommitId;

		/// <summary>
		/// Description of the preflight
		/// </summary>
		public string? PreflightDescription { get; set; }

		/// <summary>
		/// The template type
		/// </summary>
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// Hash of the actual template data
		/// </summary>
		public string? TemplateHash { get; set; }

		/// <summary>
		/// Hash of the graph for this job
		/// </summary>
		public string? GraphHash { get; set; }

		/// <summary>
		/// The user that started this job [DEPRECATED]
		/// </summary>
		public string? StartedByUserId { get; set; }

		/// <summary>
		/// The user that started this job [DEPRECATED]
		/// </summary>
		public string? StartedByUser { get; set; }

		/// <summary>
		/// The user that started this job
		/// </summary>
		public GetThinUserInfoResponse? StartedByUserInfo { get; set; }

		/// <summary>
		/// Bisection task id that started this job
		/// </summary>
		public BisectTaskId? StartedByBisectTaskId { get; set; }

		/// <summary>
		/// The user that aborted this job [DEPRECATED]
		/// </summary>
		public string? AbortedByUser { get; set; }

		/// <summary>
		/// The user that aborted this job
		/// </summary>
		public GetThinUserInfoResponse? AbortedByUserInfo { get; set; }

		/// <summary>
		/// Optional reason the job was canceled
		/// </summary>
		public string? CancellationReason { get; set; }

		/// <summary>
		/// Priority of the job
		/// </summary>
		public Priority Priority { get; set; }

		/// <summary>
		/// Whether the change will automatically be submitted or not
		/// </summary>
		public bool AutoSubmit { get; set; }

		/// <summary>
		/// The submitted changelist number
		/// </summary>
		public int? AutoSubmitChange { get; set; }

		/// <summary>
		/// Message produced by trying to auto-submit the change
		/// </summary>
		public string? AutoSubmitMessage { get; set; }

		/// <summary>
		/// Time that the job was created
		/// </summary>
		public DateTimeOffset CreateTime { get; set; }

		/// <summary>
		/// The global job state
		/// </summary>
		public JobState State { get; set; }

		/// <summary>
		/// Array of jobstep batches
		/// </summary>
		public List<GetJobBatchResponse>? Batches { get; set; }

		/// <summary>
		/// List of labels
		/// </summary>
		public List<GetLabelStateResponse>? Labels { get; set; }

		/// <summary>
		/// The default label, containing the state of all steps that are otherwise not matched.
		/// </summary>
		public GetDefaultLabelStateResponse? DefaultLabel { get; set; }

		/// <summary>
		/// List of reports
		/// </summary>
		public List<GetJobReportResponse>? Reports { get; set; }

		/// <summary>
		/// Artifacts produced by this job
		/// </summary>
		public List<GetJobArtifactResponse>? Artifacts { get; set; }

		/// <summary>
		/// Parameters for the job
		/// </summary>
		public Dictionary<ParameterId, string> Parameters { get; set; } = [];

		/// <summary>
		/// Command line arguments for the job
		/// </summary>
		public List<string> Arguments { get; set; } = [];

		/// <summary>
		/// Additional command line arguments for the job for when using the parameters block
		/// </summary>
		public List<string> AdditionalArguments { get; set; } = [];

		/// <summary>
		/// Custom list of targets for the job
		/// </summary>
		public List<string>? Targets { get; set; }

		/// <summary>
		/// List of metadata for this job
		/// </summary>
		public List<string>? Metadata { get; set; }

		/// <summary>
		/// The last update time for this job
		/// </summary>
		public DateTimeOffset UpdateTime { get; set; }

		/// <summary>
		/// Whether to use the V2 artifacts endpoint
		/// </summary>
		public bool UseArtifactsV2 { get; set; }

		/// <summary>
		/// Whether issues are being updated by this job
		/// </summary>
		public bool UpdateIssues { get; set; }

		/// <summary>
		/// Whether the current user is allowed to update this job
		/// </summary>
		public bool CanUpdate { get; set; } = true;

		/// <summary>
		/// The parent job id, if any
		/// </summary>
		public string? ParentJobId { get; set; }

		/// <summary>
		/// The parent job step id, if any
		/// </summary>
		public string? ParentJobStepId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetJobResponse(JobId jobId, StreamId streamId, TemplateId templateId, string name)
		{
			Id = jobId;
			StreamId = streamId;
			TemplateId = templateId;
			Name = name;
		}

		/// <summary>
		/// Default constructor needed for JsonSerializer
		/// </summary>
		[JsonConstructor]
		public GetJobResponse()
		{
			Name = "";
		}
	}

	/// <summary>
	/// Response describing an artifact produced during a job
	/// </summary>
	/// <param name="Id">Identifier for this artifact, if it has been produced</param>
	/// <param name="Name">Name of the artifact</param>
	/// <param name="Type">Artifact type</param>
	/// <param name="Description">Description to display for the artifact on the dashboard</param>
	/// <param name="Keys">Keys for the artifact</param>
	/// <param name="Metadata">Metadata for the artifact</param>
	/// <param name="StepId">Step producing the artifact</param>
	public record class GetJobArtifactResponse
	(
		ArtifactId? Id,
		ArtifactName Name,
		ArtifactType Type,
		string? Description,
		List<string> Keys,
		List<string> Metadata,
		JobStepId StepId
	);

	/// <summary>
	/// The timing info for a job
	/// </summary>
	public class GetJobTimingResponse
	{
		/// <summary>
		/// The job response
		/// </summary>
		public GetJobResponse? JobResponse { get; set; }

		/// <summary>
		/// Timing info for each step
		/// </summary>
		public Dictionary<string, GetStepTimingInfoResponse> Steps { get; set; }

		/// <summary>
		/// Timing information for each label
		/// </summary>
		public List<GetLabelTimingInfoResponse> Labels { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>		
		/// <param name="jobResponse">The job response</param>
		/// <param name="steps">Timing info for each steps</param>
		/// <param name="labels">Timing info for each label</param>
		public GetJobTimingResponse(GetJobResponse? jobResponse, Dictionary<string, GetStepTimingInfoResponse> steps, List<GetLabelTimingInfoResponse> labels)
		{
			JobResponse = jobResponse;
			Steps = steps;
			Labels = labels;
		}
	}

	/// <summary>
	/// The timing info for 
	/// </summary>
	public class FindJobTimingsResponse
	{
		/// <summary>
		/// Timing info for each job
		/// </summary>
		public Dictionary<string, GetJobTimingResponse> Timings { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="timings">Timing info for each job</param>
		public FindJobTimingsResponse(Dictionary<string, GetJobTimingResponse> timings)
		{
			Timings = timings;
		}
	}

	/// <summary>
	/// Request used to update a jobstep
	/// </summary>
	public class UpdateStepRequest
	{
		/// <summary>
		/// The new jobstep state
		/// </summary>
		public JobStepState State { get; set; } = JobStepState.Unspecified;

		/// <summary>
		/// Outcome from the jobstep
		/// </summary>
		public JobStepOutcome Outcome { get; set; } = JobStepOutcome.Unspecified;

		/// <summary>
		/// If the step has been requested to abort
		/// </summary>
		public bool? AbortRequested { get; set; }

		/// <summary>
		/// Optional reason the job step was canceled
		/// </summary>
		public string? CancellationReason { get; set; }

		/// <summary>
		/// Specifies the log file id for this step
		/// </summary>
		public string? LogId { get; set; }

		/// <summary>
		/// Whether the step should be re-run
		/// </summary>
		public bool? Retry { get; set; }

		/// <summary>
		/// New priority for this step
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// Properties to set. Any entries with a null value will be removed.
		/// </summary>
		public Dictionary<string, string?>? Properties { get; set; }
	}

	/// <summary>
	/// Response object when updating a jobstep
	/// </summary>
	public class UpdateStepResponse
	{
		/// <summary>
		/// If a new step is created (due to specifying the retry flag), specifies the batch id
		/// </summary>
		public string? BatchId { get; set; }

		/// <summary>
		/// If a step is retried, includes the new step id
		/// </summary>
		public string? StepId { get; set; }
	}

	/// <summary>
	/// Reference to the output of a step within the job
	/// </summary>
	/// <param name="StepId">Step producing the output</param>
	/// <param name="OutputIdx">Index of the output from this step</param>
	public record struct JobStepOutputRef(JobStepId StepId, int OutputIdx);

	/// <summary>
	/// Returns information about a jobstep
	/// </summary>
	public class GetJobStepResponse
	{
		/// <summary>
		/// The unique id of the step
		/// </summary>
		public JobStepId Id { get; set; }

		/// <summary>
		/// Index of the node which this jobstep is to execute
		/// </summary>
		public int NodeIdx { get; set; }

		/// <summary>
		/// The name of this node 
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Whether this node can be run multiple times
		/// </summary>
		public bool AllowRetry { get; set; }

		/// <summary>
		/// This node can start running early, before dependencies of other nodes in the same group are complete
		/// </summary>
		public bool RunEarly { get; set; }

		/// <summary>
		/// Whether to include warnings in the output (defaults to true)
		/// </summary>
		public bool Warnings { get; set; }

		/// <summary>
		/// References to inputs for this node
		/// </summary>
		public List<JobStepOutputRef>? Inputs { get; set; }

		/// <summary>
		/// References to optional inputs for this node
		/// </summary>
		public List<JobStepOutputRef>? OptionalInputs { get; set; }

		/// <summary>
		/// List of output names
		/// </summary>
		public List<string>? OutputNames { get; set; }

		/// <summary>
		/// Indices of nodes which must have succeeded for this node to run
		/// </summary>
		public List<JobStepId>? InputDependencies { get; set; }

		/// <summary>
		/// Indices of nodes which may have succeeded for this node to run
		/// </summary>
		public List<JobStepId>? OptionalInputDependencies { get; set; }

		/// <summary>
		/// Indices of nodes which must have completed for this node to run
		/// </summary>
		public List<JobStepId>? OrderDependencies { get; set; }

		/// <summary>
		/// List of credentials required for this node. Each entry maps an environment variable name to a credential in the form "CredentialName.PropertyName".
		/// </summary>
		public IReadOnlyDictionary<string, string>? Credentials { get; set; }

		/// <summary>
		/// Annotations for this node
		/// </summary>
		public IReadOnlyDictionary<string, string>? Annotations { get; set; }

		/// <summary>
		/// List of metadata for this step
		/// </summary>
		public List<string>? Metadata { get; set; }

		/// <summary>
		/// Current state of the job step. This is updated automatically when runs complete.
		/// </summary>
		public JobStepState State { get; set; }

		/// <summary>
		/// Current outcome of the jobstep
		/// </summary>
		public JobStepOutcome Outcome { get; set; }

		/// <summary>
		/// Error describing additional context for why a step failed to complete
		/// </summary>
		public JobStepError Error { get; set; }

		/// <summary>
		/// If the step has been requested to abort
		/// </summary>
		public bool AbortRequested { get; set; }

		/// <summary>
		/// Name of the user that requested the abort of this step [DEPRECATED]
		/// </summary>
		public string? AbortByUser { get; set; }

		/// <summary>
		/// The user that requested this step be run again 
		/// </summary>
		public GetThinUserInfoResponse? AbortedByUserInfo { get; set; }

		/// <summary>
		/// Optional reason the job step was canceled
		/// </summary>
		public string? CancellationReason { get; set; }

		/// <summary>
		/// Name of the user that requested this step be run again [DEPRECATED]
		/// </summary>
		public string? RetryByUser { get; set; }

		/// <summary>
		/// The user that requested this step be run again 
		/// </summary>
		public GetThinUserInfoResponse? RetriedByUserInfo { get; set; }

		/// <summary>
		/// The log id for this step
		/// </summary>
		public string? LogId { get; set; }

		/// <summary>
		/// Time at which the batch was ready (UTC).
		/// </summary>
		public DateTimeOffset? ReadyTime { get; set; }

		/// <summary>
		/// Time at which the batch started (UTC).
		/// </summary>
		public DateTimeOffset? StartTime { get; set; }

		/// <summary>
		/// Time at which the batch finished (UTC)
		/// </summary>
		public DateTimeOffset? FinishTime { get; set; }

		/// <summary>
		/// List of reports
		/// </summary>
		public List<GetJobReportResponse>? Reports { get; set; }

		/// <summary>
		/// List of spawned job ids
		/// </summary>
		public List<string>? SpawnedJobs { get; set; }

		/// <summary>
		/// User-defined properties for this jobstep.
		/// </summary>
		public Dictionary<string, string>? Properties { get; set; }
	}

	/// <summary>
	/// The state of a particular run
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum JobStepBatchState
	{
		/// <summary>
		/// Waiting for dependencies of at least one jobstep to complete
		/// </summary>
		Waiting = 0,

		/// <summary>
		/// Ready to execute
		/// </summary>
		Ready = 1,

		/// <summary>
		/// Preparing to execute work
		/// </summary>
		Starting = 2,

		/// <summary>
		/// Executing work
		/// </summary>
		Running = 3,

		/// <summary>
		/// Preparing to stop
		/// </summary>
		Stopping = 4,

		/// <summary>
		/// All steps have finished executing
		/// </summary>
		Complete = 5
	}

#pragma warning disable CA1027
	/// <summary>
	/// Error code for a batch not being executed
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum JobStepBatchError
	{
		/// <summary>
		/// No error
		/// </summary>
		None = 0,

		/// <summary>
		/// The stream for this job is unknown
		/// </summary>
		UnknownStream = 1,

		/// <summary>
		/// The given agent type for this batch was not valid for this stream
		/// </summary>
		UnknownAgentType = 2,

		/// <summary>
		/// The pool id referenced by the agent type was not found
		/// </summary>
		UnknownPool = 3,

		/// <summary>
		/// There are no agents in the given pool currently online
		/// </summary>
		NoAgentsInPool = 4,

		/// <summary>
		/// There are no agents in this pool that are onlinbe
		/// </summary>
		NoAgentsOnline = 5,

		/// <summary>
		/// Unknown workspace referenced by the agent type
		/// </summary>
		UnknownWorkspace = 6,

		/// <summary>
		/// Cancelled
		/// </summary>
		Cancelled = 7,

		/// <summary>
		/// Lost connection with the agent machine
		/// </summary>
		LostConnection = 8,

		/// <summary>
		/// Lease terminated prematurely but can be retried.
		/// </summary>
		Incomplete = 9,

		/// <summary>
		/// An error ocurred while executing the lease. Cannot be retried.
		/// </summary>
		ExecutionError = 10,

		/// <summary>
		/// The change that the job is running against is invalid
		/// </summary>
		UnknownShelf = 11,

		/// <summary>
		/// Step was no longer needed during a job update
		/// </summary>
		NoLongerNeeded = 12,

		/// <summary>
		/// Syncing the branch failed
		/// </summary>
		SyncingFailed = 13,

		/// <summary>
		/// Legacy alias for <see cref="SyncingFailed"/>
		/// </summary>
		[Obsolete("Use SyncingFailed instead")]
		AgentSetupFailed = SyncingFailed,
	}
#pragma warning restore CA1027

	/// <summary>
	/// Request to update a jobstep batch
	/// </summary>
	public class UpdateBatchRequest
	{
		/// <summary>
		/// The new log file id
		/// </summary>
		public string? LogId { get; set; }

		/// <summary>
		/// The state of this batch
		/// </summary>
		public JobStepBatchState? State { get; set; }
	}

	/// <summary>
	/// Information about a jobstep batch
	/// </summary>
	public class GetJobBatchResponse
	{
		/// <summary>
		/// Unique id for this batch
		/// </summary>
		public JobStepBatchId Id { get; set; }

		/// <summary>
		/// Index of the group being executed
		/// </summary>
		public int GroupIdx { get; set; }

		/// <summary>
		/// The agent type
		/// </summary>
		public string AgentType { get; set; } = String.Empty;

		/// <summary>
		/// The agent assigned to execute this group
		/// </summary>
		public string? AgentId { get; set; }

		/// <summary>
		/// Rate for using this agent (per hour)
		/// </summary>
		public double? AgentRate { get; set; }

		/// <summary>
		/// The agent session holding this lease
		/// </summary>
		public string? SessionId { get; set; }

		/// <summary>
		/// The lease that's executing this group
		/// </summary>
		public string? LeaseId { get; set; }

		/// <summary>
		/// The unique log file id
		/// </summary>
		public string? LogId { get; set; }

		/// <summary>
		/// The state of this batch
		/// </summary>
		public JobStepBatchState State { get; set; }

		/// <summary>
		/// Error code for this batch
		/// </summary>
		public JobStepBatchError Error { get; set; }

		/// <summary>
		/// The priority of this batch
		/// </summary>
		public int WeightedPriority { get; set; }

		/// <summary>
		/// Time at which the group started (UTC).
		/// </summary>
		public DateTimeOffset? StartTime { get; set; }

		/// <summary>
		/// Time at which the group finished (UTC)
		/// </summary>
		public DateTimeOffset? FinishTime { get; set; }

		/// <summary>
		/// Time at which the group became ready (UTC).
		/// </summary>
		public DateTimeOffset? ReadyTime { get; set; }

		/// <summary>
		/// Human-readable names of lease outcome reasons (only populated when batch has an error)
		/// </summary>
		public List<string>? LeaseOutcomeReasons { get; set; }

		/// <summary>
		/// Steps within this run
		/// </summary>
		public List<GetJobStepResponse> Steps { get; set; } = [];
	}

	/// <summary>
	/// State of an aggregate
	/// </summary>
	public enum LabelState
	{
		/// <summary>
		/// Aggregate is not currently being built (no required nodes are present)
		/// </summary>
		Unspecified,

		/// <summary>
		/// Steps are still running
		/// </summary>
		Running,

		/// <summary>
		/// All steps are complete
		/// </summary>
		Complete
	}

	/// <summary>
	/// Outcome of an aggregate
	/// </summary>
	public enum LabelOutcome
	{
		/// <summary>
		/// Aggregate is not currently being built
		/// </summary>
		Unspecified,

		/// <summary>
		/// A step dependency failed
		/// </summary>
		Failure,

		/// <summary>
		/// A dependency finished with warnings
		/// </summary>
		Warnings,

		/// <summary>
		/// Successful
		/// </summary>
		Success,
	}

	/// <summary>
	/// State of a label within a job
	/// </summary>
	public class GetLabelStateResponse
	{
		/// <summary>
		/// Name to show for this label on the dashboard
		/// </summary>
		public string? DashboardName { get; set; }

		/// <summary>
		/// Category to show this label in on the dashboard
		/// </summary>
		public string? DashboardCategory { get; set; }

		/// <summary>
		/// Name to show for this label in UGS
		/// </summary>
		public string? UgsName { get; set; }

		/// <summary>
		/// Project to display this label for in UGS
		/// </summary>
		public string? UgsProject { get; set; }

		/// <summary>
		/// State of the label
		/// </summary>
		public LabelState? State { get; set; }

		/// <summary>
		/// Outcome of the label
		/// </summary>
		public LabelOutcome? Outcome { get; set; }

		/// <summary>
		/// Steps to include in the status of this label
		/// </summary>
		public List<JobStepId> Steps { get; set; } = [];
	}

	/// <summary>
	/// Information about the default label (ie. with inlined list of nodes)
	/// </summary>
	public class GetDefaultLabelStateResponse : GetLabelStateResponse
	{
		/// <summary>
		/// List of nodes covered by default label
		/// </summary>
		public List<string> Nodes { get; set; } = [];
	}

	/// <summary>
	/// Information about the timing info for a particular target
	/// </summary>
	public class GetTimingInfoResponse
	{
		/// <summary>
		/// Wait time on the critical path
		/// </summary>
		public float? TotalWaitTime { get; set; }

		/// <summary>
		/// Sync time on the critical path
		/// </summary>
		public float? TotalInitTime { get; set; }

		/// <summary>
		/// Duration to this point
		/// </summary>
		public float? TotalTimeToComplete { get; set; }

		/// <summary>
		/// Average wait time by the time the job reaches this point
		/// </summary>
		public float? AverageTotalWaitTime { get; set; }

		/// <summary>
		/// Average sync time to this point
		/// </summary>
		public float? AverageTotalInitTime { get; set; }

		/// <summary>
		/// Average duration to this point
		/// </summary>
		public float? AverageTotalTimeToComplete { get; set; }
	}

	/// <summary>
	/// Information about the timing info for a particular target
	/// </summary>
	public class GetStepTimingInfoResponse : GetTimingInfoResponse
	{
		/// <summary>
		/// Name of this node
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Average wait time for this step
		/// </summary>
		public float? AverageStepWaitTime { get; set; }

		/// <summary>
		/// Average init time for this step
		/// </summary>
		public float? AverageStepInitTime { get; set; }

		/// <summary>
		/// Average duration for this step
		/// </summary>
		public float? AverageStepDuration { get; set; }
	}

	/// <summary>
	/// Information about the timing info for a label
	/// </summary>
	public class GetLabelTimingInfoResponse : GetTimingInfoResponse
	{
		/// <summary>
		/// Name of the label
		/// </summary>
		[Obsolete("Use DashboardName instead")]
		public string? Name => DashboardName;

		/// <summary>
		/// Category for the label
		/// </summary>
		[Obsolete("Use DashboardCategory instead")]
		public string? Category => DashboardCategory;

		/// <summary>
		/// Name of the label
		/// </summary>
		public string? DashboardName { get; set; }

		/// <summary>
		/// Category for the label
		/// </summary>
		public string? DashboardCategory { get; set; }

		/// <summary>
		/// Name of the label
		/// </summary>
		public string? UgsName { get; set; }

		/// <summary>
		/// Category for the label
		/// </summary>
		public string? UgsProject { get; set; }
	}

	/// <summary>
	/// Describes the history of a step
	/// </summary>
	public class GetJobStepRefResponse
	{
		/// <summary>
		/// The job id
		/// </summary>
		public JobId JobId { get; set; }

		/// <summary>
		/// The step name.
		/// </summary>
		public string? StepName { get; set; }

		/// <summary>
		/// The batch containing the step
		/// </summary>
		public JobStepBatchId BatchId { get; set; }

		/// <summary>
		/// The step identifier
		/// </summary>
		public JobStepId StepId { get; set; }

		/// <summary>
		/// The change number being built
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int Change
		{
			get => _change ?? _commitId?.GetPerforceChangeOrMinusOne() ?? 0;
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// The commit being built
		/// </summary>
		public CommitIdWithOrder CommitId
		{
			get => _commitId ?? CommitIdWithOrder.FromPerforceChange(_change) ?? CommitIdWithOrder.Empty;
			set => _commitId = value;
		}
		CommitIdWithOrder? _commitId;

		/// <summary>
		/// The step log id
		/// </summary>
		public string? LogId { get; set; }

		/// <summary>
		/// The pool id
		/// </summary>
		public string? PoolId { get; set; }

		/// <summary>
		/// The agent id
		/// </summary>
		public string? AgentId { get; set; }

		/// <summary>
		/// The step state.
		/// </summary>
		public JobStepState? State { get; set; }

		/// <summary>
		/// Outcome of the step, once complete.
		/// </summary>
		public JobStepOutcome? Outcome { get; set; }

		/// <summary>
		/// The issues which affected this step
		/// </summary>
		public List<int>? IssueIds { get; set; }

		/// <summary>
		/// The step meta data for this step
		/// </summary>
		public List<string>? Metadata { get; set; }

		/// <summary>
		/// Time at which the job started
		/// </summary>
		public DateTimeOffset JobStartTime { get; set; }

		/// <summary>
		/// Time at which the step started.
		/// </summary>
		public DateTimeOffset StartTime { get; set; }

		/// <summary>
		/// Time at which the step finished.
		/// </summary>
		public DateTimeOffset? FinishTime { get; set; }
	}
}
