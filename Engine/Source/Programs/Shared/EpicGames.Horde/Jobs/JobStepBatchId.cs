// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Identifier for a job step batch
	/// </summary>
	/// <param name="SubResourceId">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(SubResourceIdTypeConverter<JobStepBatchId, JobStepBatchIdConverter>))]
	[SubResourceIdConverter(typeof(JobStepBatchIdConverter))]
	public readonly record struct JobStepBatchId(SubResourceId SubResourceId)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public JobStepBatchId(ushort value) : this(new SubResourceId(value))
		{ }

		/// <summary>
		/// Creates a new <see cref="JobId"/>
		/// </summary>
		public static JobStepBatchId GenerateNewId() => new(SubResourceId.GenerateNewId());

		/// <inheritdoc cref="SubResourceId.Parse(System.String)"/>
		public static JobStepBatchId Parse(string text) => new(SubResourceId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => SubResourceId.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="SubResourceId"/> instances.
	/// </summary>
	class JobStepBatchIdConverter : SubResourceIdConverter<JobStepBatchId>
	{
		/// <inheritdoc/>
		public override JobStepBatchId FromSubResourceId(SubResourceId subResourceId) => new(subResourceId);

		/// <inheritdoc/>
		public override SubResourceId ToSubResourceId(JobStepBatchId value) => value.SubResourceId;
	}
}
