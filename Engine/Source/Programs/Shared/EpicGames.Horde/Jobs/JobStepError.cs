// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Systemic error codes for a job step failing
	/// </summary>
	public enum JobStepError
	{
		/// <summary>
		/// No systemic error
		/// </summary>
		None,

		/// <summary>
		/// Step did not complete in the required amount of time
		/// </summary>
		TimedOut,

		/// <summary>
		/// Step is in is paused state so was skipped
		/// </summary>
		Paused,

		/// <summary>
		/// Step did not complete because the batch exited
		/// </summary>
		Incomplete
	}
}
