// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Identifier for a jobstep
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(SubResourceIdTypeConverter<JobStepId, JobStepIdConverter>))]
	[SubResourceIdConverter(typeof(JobStepIdConverter))]
	public readonly record struct JobStepId(SubResourceId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public JobStepId(ushort value) : this(new SubResourceId(value))
		{ }

		/// <summary>
		/// Creates a new <see cref="JobId"/>
		/// </summary>
		public static JobStepId GenerateNewId() => new(SubResourceId.GenerateNewId());

		/// <inheritdoc cref="SubResourceId.Parse(System.String)"/>
		public static JobStepId Parse(string text) => new(SubResourceId.Parse(text));

		/// <inheritdoc cref="SubResourceId.TryParse(System.String, out SubResourceId)"/>
		public static bool TryParse(string text, out JobStepId stepId)
		{
			if (SubResourceId.TryParse(text, out SubResourceId subResourceId))
			{
				stepId = new JobStepId(subResourceId);
				return true;
			}
			else
			{
				stepId = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="SubResourceId"/> instances.
	/// </summary>
	class JobStepIdConverter : SubResourceIdConverter<JobStepId>
	{
		/// <inheritdoc/>
		public override JobStepId FromSubResourceId(SubResourceId id) => new(id);

		/// <inheritdoc/>
		public override SubResourceId ToSubResourceId(JobStepId value) => value.Id;
	}
}
