// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Outcome for a jobstep
	/// </summary>
	public enum JobStepOutcome
	{
		/// <summary>
		/// Outcome is not known
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Step failed
		/// </summary>
		Failure = 1,

		/// <summary>
		/// Step completed with warnings
		/// </summary>
		Warnings = 2,

		/// <summary>
		/// Step succeeded
		/// </summary>
		Success = 3
	}
}
