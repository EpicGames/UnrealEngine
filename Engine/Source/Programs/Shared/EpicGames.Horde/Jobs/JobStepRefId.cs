// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Unique id struct for JobStepRef objects. Includes a job id, batch id, and step id to uniquely identify the step.
	/// </summary>
	public struct JobStepRefId : IEquatable<JobStepRefId>, IComparable<JobStepRefId>
	{
		/// <summary>
		/// The job id
		/// </summary>
		public JobId JobId { get; set; }

		/// <summary>
		/// The batch id within the job
		/// </summary>
		public JobStepBatchId BatchId { get; set; }

		/// <summary>
		/// The step id
		/// </summary>
		public JobStepId StepId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="jobId">The job id</param>
		/// <param name="batchId">The batch id within the job</param>
		/// <param name="stepId">The step id</param>
		public JobStepRefId(JobId jobId, JobStepBatchId batchId, JobStepId stepId)
		{
			JobId = jobId;
			BatchId = batchId;
			StepId = stepId;
		}

		/// <summary>
		/// Parse a job step id from a string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>The parsed id</returns>
		public static JobStepRefId Parse(string text)
		{
			string[] components = text.Split(':');
			return new JobStepRefId(JobId.Parse(components[0]), JobStepBatchId.Parse(components[1]), JobStepId.Parse(components[2]));
		}

		/// <summary>
		/// Formats this id as a string
		/// </summary>
		/// <returns>Formatted id</returns>
		public override string ToString()
		{
			return $"{JobId}:{BatchId}:{StepId}";
		}

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(JobId, BatchId, StepId);

		/// <inheritdoc/>
		public override bool Equals([NotNullWhen(true)] object? obj) => obj is JobStepRefId other && Equals(other);

		/// <inheritdoc/>
		public bool Equals(JobStepRefId other)
		{
			return JobId.Equals(other.JobId) && BatchId.Equals(other.BatchId) && StepId.Equals(other.StepId);
		}

		/// <inheritdoc/>
		public int CompareTo(JobStepRefId other)
		{
			int result = JobId.Id.CompareTo(other.JobId.Id);
			if (result != 0)
			{
				return result;
			}

			result = BatchId.SubResourceId.Value.CompareTo(other.BatchId.SubResourceId.Value);
			if (result != 0)
			{
				return result;
			}

			return StepId.Id.Value.CompareTo(other.StepId.Id.Value);
		}

		/// <inheritdoc/>
		public static bool operator ==(JobStepRefId lhs, JobStepRefId rhs) => lhs.Equals(rhs);

		/// <inheritdoc/>
		public static bool operator !=(JobStepRefId lhs, JobStepRefId rhs) => !lhs.Equals(rhs);

		/// <inheritdoc/>
		public static bool operator <(JobStepRefId lhs, JobStepRefId rhs) => lhs.CompareTo(rhs) < 0;

		/// <inheritdoc/>
		public static bool operator >(JobStepRefId lhs, JobStepRefId rhs) => lhs.CompareTo(rhs) > 0;

		/// <inheritdoc/>
		public static bool operator <=(JobStepRefId lhs, JobStepRefId rhs) => lhs.CompareTo(rhs) <= 0;

		/// <inheritdoc/>
		public static bool operator >=(JobStepRefId lhs, JobStepRefId rhs) => lhs.CompareTo(rhs) >= 0;
	}
}
