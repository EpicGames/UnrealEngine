// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// State of a job step
	/// </summary>
	public enum JobStepState
	{
		/// <summary>
		/// Unspecified
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Waiting for dependencies of this step to complete (or paused)
		/// </summary>
		Waiting = 1,

		/// <summary>
		/// Ready to run, but has not been scheduled yet
		/// </summary>
		Ready = 2,

		/// <summary>
		/// Dependencies of this step failed, so it cannot be executed
		/// </summary>
		Skipped = 3,

		/// <summary>
		/// There is an active instance of this step running
		/// </summary>
		Running = 4,

		/// <summary>
		/// This step has been run
		/// </summary>
		Completed = 5,

		/// <summary>
		/// This step started to execute, but was aborted
		/// </summary>
		Aborted = 6
	}
}
