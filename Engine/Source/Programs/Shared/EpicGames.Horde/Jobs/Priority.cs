// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Priority of a job or step
	/// </summary>
	public enum Priority
	{
		/// <summary>
		/// Not specified
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Lowest priority
		/// </summary>
		Lowest = 1,

		/// <summary>
		/// Below normal priority
		/// </summary>
		BelowNormal = 2,

		/// <summary>
		/// Normal priority
		/// </summary>
		Normal = 3,

		/// <summary>
		/// Above normal priority
		/// </summary>
		AboveNormal = 4,

		/// <summary>
		/// High priority
		/// </summary>
		High = 5,

		/// <summary>
		/// Highest priority
		/// </summary>
		Highest = 6
	}
}
