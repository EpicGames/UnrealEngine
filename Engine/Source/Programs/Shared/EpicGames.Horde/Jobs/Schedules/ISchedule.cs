// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs.Templates;

namespace EpicGames.Horde.Jobs.Schedules
{
	/// <summary>
	/// Schedule for a template
	/// </summary>
	public interface ISchedule
	{
		/// <summary>
		/// Whether the schedule should be enabled
		/// </summary>
		bool Enabled { get; }

		/// <summary>
		/// Maximum number of builds that can be active at once
		/// </summary>
		int MaxActive { get; }

		/// <summary>
		/// Maximum number of changes the schedule can fall behind head revision. If greater than zero, builds will be triggered for every submitted changelist until the backlog is this size.
		/// </summary>
		int MaxChanges { get; }

		/// <summary>
		/// Whether the build requires a change to be submitted
		/// </summary>
		bool RequireSubmittedChange { get; }

		/// <summary>
		/// Gate allowing the schedule to trigger
		/// </summary>
		IScheduleGate? Gate { get; }

		/// <summary>
		/// Commit tags for this schedule
		/// </summary>
		IReadOnlyList<CommitTag> Commits { get; }

		/// <summary>
		/// Roles to impersonate for this schedule
		/// </summary>
		IReadOnlyList<IScheduleClaim>? Claims { get; }

		/// <summary>
		/// Last changelist number that this was triggered for
		/// </summary>
		CommitIdWithOrder? LastTriggerCommitId { get; }

		/// <summary>
		/// Gets the last trigger time, in UTC
		/// </summary>
		DateTime LastTriggerTimeUtc { get; }

		/// <summary>
		/// List of jobs that are currently active
		/// </summary>
		IReadOnlyList<JobId> ActiveJobs { get; }

		/// <summary>
		/// Patterns for starting this scheduled job
		/// </summary>
		IReadOnlyList<ISchedulePattern> Patterns { get; }

		/// <summary>
		/// Files that should cause the job to trigger
		/// </summary>
		IReadOnlyList<string>? Files { get; }

		/// <summary>
		/// Parameters for the template
		/// </summary>
		IReadOnlyDictionary<string, string> TemplateParameters { get; }
	}

	/// <summary>
	/// Claim granted to a schedule
	/// </summary>
	public interface IScheduleClaim
	{
		/// <summary>
		/// The claim type
		/// </summary>
		string Type { get; }

		/// <summary>
		/// The claim value
		/// </summary>
		string Value { get; }
	}

	/// <summary>
	/// Required gate for starting a schedule
	/// </summary>
	public interface IScheduleGate
	{
		/// <summary>
		/// The template containing the dependency
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// Target to wait for
		/// </summary>
		string Target { get; }
	}

	/// <summary>
	/// Pattern for executing a schedule
	/// </summary>
	public interface ISchedulePattern
	{
		/// <summary>
		/// Days of the week to run this schedule on. If null, the schedule will run every day.
		/// </summary>
		IReadOnlyList<DayOfWeek>? DaysOfWeek { get; }

		/// <summary>
		/// Time during the day for the first schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		ScheduleTimeOfDay MinTime { get; }

		/// <summary>
		/// Time during the day for the last schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		ScheduleTimeOfDay? MaxTime { get; }

		/// <summary>
		/// Interval between each schedule triggering
		/// </summary>
		ScheduleInterval? Interval { get; }
	}

	/// <summary>
	/// Extension methods for schedules
	/// </summary>
	public static class ScheduleExtensions
	{
		/// <summary>
		/// Gets the next trigger time for a schedule
		/// </summary>
		/// <param name="schedule"></param>
		/// <param name="timeZone"></param>
		/// <returns></returns>
		public static DateTime? GetNextTriggerTimeUtc(this ISchedule schedule, TimeZoneInfo timeZone)
		{
			return schedule.GetNextTriggerTimeUtc(schedule.LastTriggerTimeUtc, timeZone);
		}

		/// <summary>
		/// Get the next time that the schedule will trigger
		/// </summary>
		/// <param name="schedule">Schedule to query</param>
		/// <param name="lastTimeUtc">Last time at which the schedule triggered</param>
		/// <param name="timeZone">Timezone to evaluate the trigger</param>
		/// <returns>Next time at which the schedule will trigger</returns>
		public static DateTime? GetNextTriggerTimeUtc(this ISchedule schedule, DateTime lastTimeUtc, TimeZoneInfo timeZone)
		{
			DateTime? nextTriggerTimeUtc = null;
			foreach (ISchedulePattern pattern in schedule.Patterns)
			{
				DateTime patternTriggerTime = pattern.GetNextTriggerTimeUtc(lastTimeUtc, timeZone);
				if (nextTriggerTimeUtc == null || patternTriggerTime < nextTriggerTimeUtc)
				{
					nextTriggerTimeUtc = patternTriggerTime;
				}
			}
			return nextTriggerTimeUtc;
		}

		/// <summary>
		/// Calculates the trigger index based on the given time in minutes
		/// </summary>
		/// <param name="pattern">Pattern to query</param>
		/// <param name="lastTimeUtc">Time for the last trigger</param>
		/// <param name="timeZone">The timezone for running the schedule</param>
		/// <returns>Index of the trigger</returns>
		public static DateTime GetNextTriggerTimeUtc(this ISchedulePattern pattern, DateTime lastTimeUtc, TimeZoneInfo timeZone)
		{
			// Convert last time into the correct timezone for running the scheule
			DateTimeOffset lastTime = TimeZoneInfo.ConvertTime((DateTimeOffset)lastTimeUtc, timeZone);

			// Get the base time (ie. the start of this day) for anchoring the schedule
			DateTimeOffset baseTime = new(lastTime.Year, lastTime.Month, lastTime.Day, 0, 0, 0, lastTime.Offset);
			for (; ; )
			{
				if (pattern.DaysOfWeek == null || pattern.DaysOfWeek.Contains(baseTime.DayOfWeek))
				{
					// Get the last time in minutes from the start of this day
					int lastTimeMinutes = (int)(lastTime - baseTime).TotalMinutes;

					// Get the time of the first trigger of this day. If the last time is less than this, this is the next trigger.
					if (lastTimeMinutes < pattern.MinTime.Minutes)
					{
						return baseTime.AddMinutes(pattern.MinTime.Minutes).UtcDateTime;
					}

					// Otherwise, get the time for the last trigger in the day.
					if (pattern.Interval != null && pattern.Interval.Minutes > 0)
					{
						int actualMaxTime = pattern.MaxTime?.Minutes ?? ((24 * 60) - 1);
						if (lastTimeMinutes < actualMaxTime)
						{
							int lastIndex = (lastTimeMinutes - pattern.MinTime.Minutes) / pattern.Interval.Minutes;
							int nextIndex = lastIndex + 1;

							int nextTimeMinutes = pattern.MinTime.Minutes + (nextIndex * pattern.Interval.Minutes);
							if (nextTimeMinutes <= actualMaxTime)
							{
								return baseTime.AddMinutes(nextTimeMinutes).UtcDateTime;
							}
						}
					}
				}
				baseTime = baseTime.AddDays(1.0);
			}
		}
	}
}
