// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Utilities;

namespace EpicGames.Horde.Jobs.Schedules
{
	/// <summary>
	/// Time of day value for a schedule
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(ScheduleIntervalJsonConverter))]
	public record class ScheduleInterval(int Minutes)
	{
		/// <summary>
		/// Parse a string as a time of day
		/// </summary>
		[return: NotNullIfNotNull(nameof(text))]
		public static ScheduleInterval? Parse(string? text)
			=> (text != null) ? new ScheduleInterval((int)IntervalJsonConverter.Parse(text).TotalMinutes) : null;
	}

	class ScheduleIntervalJsonConverter : JsonConverter<ScheduleInterval>
	{
		/// <inheritdoc/>
		public override ScheduleInterval? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType == JsonTokenType.Number)
			{
				return new ScheduleInterval(reader.GetInt32());
			}
			else
			{
				return ScheduleInterval.Parse(reader.GetString());
			}
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ScheduleInterval value, JsonSerializerOptions options)
		{
			writer.WriteNumberValue(value.Minutes);
		}
	}
}
