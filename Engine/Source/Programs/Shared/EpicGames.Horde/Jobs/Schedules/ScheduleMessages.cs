// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs.Templates;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.Schedules
{
	/// <summary>
	/// Response describing a schedule
	/// </summary>
	public class GetScheduleResponse
	{
		/// <summary>
		/// Whether the schedule is currently enabled
		/// </summary>
		public bool Enabled { get; set; }

		/// <summary>
		/// Maximum number of scheduled jobs at once
		/// </summary>
		public int MaxActive { get; set; }

		/// <summary>
		/// Maximum number of changes the schedule can fall behind head revision. If greater than zero, builds will be triggered for every submitted changelist until the backlog is this size.
		/// </summary>
		public int MaxChanges { get; set; }

		/// <summary>
		/// Whether the build requires a change to be submitted
		/// </summary>
		public bool RequireSubmittedChange { get; set; }

		/// <summary>
		/// Gate for this schedule to trigger
		/// </summary>
		public GetScheduleGateResponse? Gate { get; set; }

		/// <summary>
		/// Which commits to run this job for
		/// </summary>
		public List<CommitTag>? Commits { get; set; }

		/// <summary>
		/// Parameters for the template
		/// </summary>
		public Dictionary<string, string> TemplateParameters { get; set; }

		/// <summary>
		/// New patterns for the schedule
		/// </summary>
		public List<GetSchedulePatternResponse> Patterns { get; set; }

		/// <summary>
		/// Last changelist number that this was triggered for
		/// </summary>
		[Obsolete("Use LastTriggerCommitId instead")]
		public int LastTriggerChange
		{
			get => _lastTriggerChange ?? _lastTriggerCommitId?.TryGetPerforceChange() ?? 0;
			set => _lastTriggerChange = value;
		}
		int? _lastTriggerChange;

		/// <summary>
		/// Last changelist number that this was triggered for
		/// </summary>
		public CommitIdWithOrder? LastTriggerCommitId
		{
			get => _lastTriggerCommitId ?? CommitIdWithOrder.FromPerforceChange(_lastTriggerChange);
			set => _lastTriggerCommitId = value;
		}
		CommitIdWithOrder? _lastTriggerCommitId;

		/// <summary>
		/// Last time that the schedule was triggered
		/// </summary>
		public DateTimeOffset LastTriggerTime { get; set; }

		/// <summary>
		/// Next trigger times for schedule
		/// </summary>
		public List<DateTime> NextTriggerTimesUTC { get; set; }

		/// <summary>
		/// List of active jobs
		/// </summary>
		public List<JobId> ActiveJobs { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetScheduleResponse()
		{
			Patterns = [];
			TemplateParameters = [];
			NextTriggerTimesUTC = [];
			ActiveJobs = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="schedule">Schedule to construct from</param>
		/// <param name="schedulerTimeZone">The scheduler time zone</param>
		public GetScheduleResponse(ISchedule schedule, TimeZoneInfo schedulerTimeZone)
		{
			Enabled = schedule.Enabled;
			MaxActive = schedule.MaxActive;
			MaxChanges = schedule.MaxChanges;
			RequireSubmittedChange = schedule.RequireSubmittedChange;
			Gate = (schedule.Gate == null) ? null : new GetScheduleGateResponse(schedule.Gate);
			Commits = schedule.Commits?.ToList();
			TemplateParameters = schedule.TemplateParameters.ToDictionary();
			Patterns = schedule.Patterns.ConvertAll(x => new GetSchedulePatternResponse(x));
			LastTriggerCommitId = schedule.LastTriggerCommitId;
			LastTriggerTime = schedule.LastTriggerTimeUtc;
			ActiveJobs = [.. schedule.ActiveJobs];

			DateTime curTime = schedule.LastTriggerTimeUtc;
			NextTriggerTimesUTC = [];
			for (int i = 0; i < 16; i++)
			{
				DateTime? nextTime = schedule.GetNextTriggerTimeUtc(curTime, schedulerTimeZone);
				if (nextTime == null)
				{
					break;
				}

				curTime = nextTime.Value;
				NextTriggerTimesUTC.Add(curTime);
			}
		}
	}

	/// <summary>
	/// Gate allowing a schedule to trigger.
	/// </summary>
	public class GetScheduleGateResponse
	{
		/// <summary>
		/// The template containing the dependency
		/// </summary>
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// Target to wait for
		/// </summary>
		public string Target { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetScheduleGateResponse()
		{
			Target = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetScheduleGateResponse(IScheduleGate gate)
		{
			TemplateId = gate.TemplateId;
			Target = gate.Target;
		}
	}

	/// <summary>
	/// Parameters to create a new schedule
	/// </summary>
	public class GetSchedulePatternResponse
	{
		/// <summary>
		/// Days of the week to run this schedule on. If null, the schedule will run every day.
		/// </summary>
		public List<DayOfWeek>? DaysOfWeek { get; set; }

		/// <summary>
		/// Time during the day for the first schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		public ScheduleTimeOfDay MinTime { get; set; } = new ScheduleTimeOfDay(0);

		/// <summary>
		/// Time during the day for the last schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		public ScheduleTimeOfDay? MaxTime { get; set; }

		/// <summary>
		/// Interval between each schedule triggering
		/// </summary>
		public ScheduleInterval? Interval { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetSchedulePatternResponse()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetSchedulePatternResponse(ISchedulePattern pattern)
		{
			DaysOfWeek = pattern.DaysOfWeek?.ToList();
			MinTime = pattern.MinTime;
			MaxTime = pattern.MaxTime;
			Interval = pattern.Interval;
		}
	}

	/// <summary>
	/// Response describing when a schedule is expected to trigger
	/// </summary>
	public class GetScheduleForecastResponse
	{
		/// <summary>
		/// Next trigger times
		/// </summary>
		public List<DateTime> Times { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="times">List of trigger times</param>
		public GetScheduleForecastResponse(List<DateTime> times)
		{
			Times = times;
		}
	}
}
