// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Utilities;

namespace EpicGames.Horde.Jobs.Schedules
{
	/// <summary>
	/// Time of day value for a schedule
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(ScheduleTimeOfDayJsonConverter))]
	public record class ScheduleTimeOfDay(int Minutes)
	{
		/// <summary>
		/// Parse a string as a time of day
		/// </summary>
		[return: NotNullIfNotNull(nameof(text))]
		public static ScheduleTimeOfDay? Parse(string? text)
			=> (text != null) ? new ScheduleTimeOfDay((int)TimeOfDayJsonConverter.Parse(text).TotalMinutes) : null;
	}

	class ScheduleTimeOfDayJsonConverter : JsonConverter<ScheduleTimeOfDay>
	{
		/// <inheritdoc/>
		public override ScheduleTimeOfDay? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType == JsonTokenType.Number)
			{
				return new ScheduleTimeOfDay(reader.GetInt32());
			}
			else
			{
				return ScheduleTimeOfDay.Parse(reader.GetString());
			}
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ScheduleTimeOfDay value, JsonSerializerOptions options)
		{
			writer.WriteNumberValue(value.Minutes);
		}
	}
}
