// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics;
using System.Globalization;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Identifier for subresources. Assigning unique ids to subresources prevents against race conditions using indices when subresources are added and removed.
	/// 
	/// Subresource identifiers are stored as 16-bit integers formatted as a 4-digit hex code, in order to keep URLs short. Calling Next() will generate a new
	/// identifier with more entropy than just incrementing the value but an identical period before repeating, in order to make URL fragments more distinctive.
	/// </summary>
	[TypeConverter(typeof(SubResourceIdTypeConverter))]
	public readonly struct SubResourceId : IEquatable<SubResourceId>
	{
		/// <summary>
		/// The unique identifier value
		/// </summary>
		public ushort Value { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="value">New identifier for this subresource</param>
		public SubResourceId(ushort value)
		{
			Value = value;
		}

		/// <summary>
		/// Creates a new random subresource id. We use random numbers for this to increase distinctiveness.
		/// </summary>
		/// <returns>New subresource id</returns>
		public static SubResourceId GenerateNewId()
		{
			return new SubResourceId((ushort)Stopwatch.GetTimestamp());
		}

		/// <summary>
		/// Updates the current value, and returns a copy of the previous value.
		/// </summary>
		/// <returns>New subresource identifier</returns>
		public SubResourceId Next()
		{
			// 771 is a factor of 65535, so we won't repeat when wrapping round 64k.
			return new SubResourceId((ushort)((int)Value + 771));
		}

		/// <summary>
		/// Parse a subresource id from a string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>New subresource id</returns>
		public static SubResourceId Parse(string text)
		{
			return new SubResourceId(UInt16.Parse(text, NumberStyles.HexNumber, CultureInfo.InvariantCulture));
		}

		/// <summary>
		/// Attempt to parse a subresource id from a string
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <param name="subResourceId">Receives the parsed subresource id on success</param>
		/// <returns>True if the id was parsed correctly</returns>
		public static bool TryParse(string text, out SubResourceId subResourceId)
		{
			ushort result;
			if (UInt16.TryParse(text, NumberStyles.HexNumber, CultureInfo.InvariantCulture, out result))
			{
				subResourceId = new SubResourceId(result);
				return true;
			}
			else
			{
				subResourceId = default;
				return false;
			}
		}

		/// <summary>
		/// Converts this identifier to a string
		/// </summary>
		/// <returns>String representation of this id</returns>
		public override string ToString()
		{
			return Value.ToString("x4", CultureInfo.InvariantCulture);
		}

		/// <inheritdoc/>
		public override bool Equals(object? other)
		{
			return other is SubResourceId otherId && Equals(otherId);
		}

		/// <inheritdoc/>
		public bool Equals(SubResourceId other)
		{
			return Value == other.Value;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return (int)Value;
		}

		/// <summary>
		/// Equality operator for identifiers
		/// </summary>
		/// <param name="left">First identifier to compare</param>
		/// <param name="right">Second identifier to compare</param>
		/// <returns>True if the identifiers are equal</returns>
		public static bool operator ==(SubResourceId left, SubResourceId right)
		{
			return left.Value == right.Value;
		}

		/// <summary>
		/// Inequality operator for identifiers
		/// </summary>
		/// <param name="left">First identifier to compare</param>
		/// <param name="right">Second identifier to compare</param>
		/// <returns>True if the identifiers are equal</returns>
		public static bool operator !=(SubResourceId left, SubResourceId right)
		{
			return left.Value != right.Value;
		}
	}

	/// <summary>
	/// Extension methods for manipulating subresource ids
	/// </summary>
	public static class SubResourceIdExtensions
	{
		/// <summary>
		/// Parse a string as a subresource identifier
		/// </summary>
		/// <param name="text">Text to parse</param>
		/// <returns>The new subresource identifier</returns>
		public static SubResourceId ToSubResourceId(this string text)
		{
			return new SubResourceId(UInt16.Parse(text, NumberStyles.HexNumber, CultureInfo.InvariantCulture));
		}
	}

	/// <summary>
	/// Type converter from strings to SubResourceId objects
	/// </summary>
	sealed class SubResourceIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return SubResourceId.Parse((string)value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			return destinationType == typeof(string);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			else
			{
				return null;
			}
		}
	}
}
