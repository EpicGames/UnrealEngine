// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Horde.Jobs
{
	/// <summary>
	/// Base class for converting to and from types containing a <see cref="SubResourceId"/>. Useful pattern for reducing boilerplate with strongly typed records.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public abstract class SubResourceIdConverter<T> where T : struct
	{
		/// <summary>
		/// Converts a type to a <see cref="SubResourceId"/>
		/// </summary>
		public abstract SubResourceId ToSubResourceId(T value);

		/// <summary>
		/// Constructs a type from a <see cref="SubResourceId"/>
		/// </summary>
		public abstract T FromSubResourceId(SubResourceId id);
	}

	/// <summary>
	/// Attribute declaring a <see cref="SubResourceIdConverter{T}"/> for a particular type
	/// </summary>
	[AttributeUsage(AttributeTargets.Struct)]
	public sealed class SubResourceIdConverterAttribute : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SubResourceIdConverterAttribute(Type converterType) => ConverterType = converterType;
	}

	/// <summary>
	/// Class which serializes types with a <see cref="SubResourceIdConverter{T}"/> to Json
	/// </summary>
	public sealed class SubResourceIdTypeConverter<TValue, TConverter> : TypeConverter where TValue : struct where TConverter : SubResourceIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new();

		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || sourceType == typeof(SubResourceId);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return _converter.FromSubResourceId(SubResourceId.Parse(str));
			}
			if (value is SubResourceId stringId)
			{
				return _converter.FromSubResourceId(stringId);
			}
			return null;
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string) || destinationType == typeof(SubResourceId);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return _converter.ToSubResourceId((TValue)value!).ToString();
			}
			if (destinationType == typeof(SubResourceId))
			{
				return _converter.ToSubResourceId((TValue)value!);
			}
			return null;
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="SubResourceIdConverter{T}"/> to Json
	/// </summary>
	public sealed class SubResourceIdJsonConverter<TValue, TConverter> : JsonConverter<TValue> where TValue : struct where TConverter : SubResourceIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new();

		/// <inheritdoc/>
		public override TValue Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => _converter.FromSubResourceId(SubResourceId.Parse(reader.GetString() ?? String.Empty));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TValue value, JsonSerializerOptions options) => writer.WriteStringValue(_converter.ToSubResourceId(value).ToString());
	}

	/// <summary>
	/// Creates constructors for types with a <see cref="SubResourceIdConverter{T}"/> to Json
	/// </summary>
	public sealed class SubResourceIdJsonConverterFactory : JsonConverterFactory
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type typeToConvert) => typeToConvert.GetCustomAttribute<SubResourceIdConverterAttribute>() != null;

		/// <inheritdoc/>
		public override JsonConverter? CreateConverter(Type type, JsonSerializerOptions options)
		{
			SubResourceIdConverterAttribute? attribute = type.GetCustomAttribute<SubResourceIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (JsonConverter?)Activator.CreateInstance(typeof(SubResourceIdJsonConverter<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}
}
