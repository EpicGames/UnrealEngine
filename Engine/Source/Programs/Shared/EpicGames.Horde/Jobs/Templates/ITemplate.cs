// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;

#pragma warning disable CA1716 // Rename virtual/interface member x so that it no longer conflicts with the reserved language keyword 'Default'. Using a reserved keyword as the name of a virtual/interface member makes it harder for consumers in other languages to override/implement the member. 
#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.Templates
{
	/// <summary>
	/// Document describing a job template. These objects are considered immutable once created and uniquely referenced by hash, in order to de-duplicate across all job runs.
	/// </summary>
	public interface ITemplate
	{
		/// <summary>
		/// Hash of this template
		/// </summary>
		ContentHash Hash { get; }

		/// <summary>
		/// Name of the template.
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Description for the template
		/// </summary>
		string? Description { get; }

		/// <summary>
		/// Priority of this job
		/// </summary>
		Priority? Priority { get; }

		/// <summary>
		/// Whether to allow preflights for this job type
		/// </summary>
		bool AllowPreflights { get; }

		/// <summary>
		/// Whether to always issues for jobs using this template
		/// </summary>
		bool UpdateIssues { get; }

		/// <summary>
		/// Whether to promote issues by default for jobs using this template
		/// </summary>
		bool PromoteIssuesByDefault { get; }

		/// <summary>
		/// Agent type to use for parsing the job state
		/// </summary>
		string? InitialAgentType { get; }

		/// <summary>
		/// Path to a file within the stream to submit to generate a new changelist for jobs
		/// </summary>
		string? SubmitNewChange { get; }

		/// <summary>
		/// Description for new changelists
		/// </summary>
		string? SubmitDescription { get; }

		/// <summary>
		/// Optional predefined user-defined properties for this job
		/// </summary>
		IReadOnlyList<string> Arguments { get; }

		/// <summary>
		/// Parameters for this template
		/// </summary>
		IReadOnlyList<ITemplateParameter> Parameters { get; }
	}

	/// <summary>
	/// Base class for parameters used to configure templates via the new build dialog
	/// </summary>
	public interface ITemplateParameter
	{
		/// <summary>
		/// Gets the arguments for a job given a set of parameters
		/// </summary>
		/// <param name="parameters">Map of parameter id to value</param>
		/// <param name="scheduledBuild">Whether this is a scheduled build</param>
		/// <param name="arguments">Receives command line arguments for the job</param>
		void GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments);

		/// <summary>
		/// Gets the default arguments for this parameter and its children
		/// </summary>
		/// <param name="parameters">List of default parameters</param>
		/// <param name="scheduledBuild">Whether the arguments are being queried for a scheduled build</param>
		void GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild);
	}

	/// <summary>
	/// Allows the user to toggle an option on or off
	/// </summary>
	public interface ITemplateBoolParameter : ITemplateParameter
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		ParameterId Id { get; }

		/// <summary>
		/// Label to display next to this parameter.
		/// </summary>
		string Label { get; }

		/// <summary>
		/// Argument to add if this parameter is enabled
		/// </summary>
		string? ArgumentIfEnabled { get; }

		/// <summary>
		/// Arguments to add if this parameter is enabled
		/// </summary>
		IReadOnlyList<string>? ArgumentsIfEnabled { get; }

		/// <summary>
		/// Argument to add if this parameter is disabled
		/// </summary>
		string? ArgumentIfDisabled { get; }

		/// <summary>
		/// Arguments to add if this parameter is disabled
		/// </summary>
		IReadOnlyList<string>? ArgumentsIfDisabled { get; }

		/// <summary>
		/// Whether this option should be enabled by default
		/// </summary>
		bool Default { get; }

		/// <summary>
		/// Whether this option should be enabled by default
		/// </summary>
		bool? ScheduleOverride { get; }

		/// <summary>
		/// Tool tip text to display
		/// </summary>
		string? ToolTip { get; }
	}

	/// <summary>
	/// Free-form text entry parameter
	/// </summary>
	public interface ITemplateTextParameter : ITemplateParameter
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		ParameterId Id { get; }

		/// <summary>
		/// Label to display next to this parameter. Should default to the parameter name.
		/// </summary>
		string Label { get; }

		/// <summary>
		/// Argument to add (will have the value of this field appended)
		/// </summary>
		string Argument { get; }

		/// <summary>
		/// Default value for this argument
		/// </summary>
		string Default { get; }

		/// <summary>
		/// Override for this argument in scheduled builds. 
		/// </summary>
		string? ScheduleOverride { get; }

		/// <summary>
		/// Hint text to display when the field is empty
		/// </summary>
		string? Hint { get; }

		/// <summary>
		/// Regex used to validate values entered into this text field.
		/// </summary>
		string? Validation { get; }

		/// <summary>
		/// Message displayed to explain valid values if validation fails.
		/// </summary>
		string? ValidationError { get; }

		/// <summary>
		/// Tool tip text to display
		/// </summary>
		string? ToolTip { get; }
	}

	/// <summary>
	/// Style of list parameter
	/// </summary>
	public enum TemplateListParameterStyle
	{
		/// <summary>
		/// Regular drop-down list. One item is always selected.
		/// </summary>
		List,

		/// <summary>
		/// Drop-down list with checkboxes
		/// </summary>
		MultiList,

		/// <summary>
		/// Tag picker from list of options
		/// </summary>
		TagPicker,
	}

	/// <summary>
	/// Allows the user to select a value from a constrained list of choices
	/// </summary>
	public interface ITemplateListParameter : ITemplateParameter
	{
		/// <summary>
		/// Label to display next to this parameter.
		/// </summary>
		string Label { get; }

		/// <summary>
		/// Style of picker parameter to use
		/// </summary>
		TemplateListParameterStyle Style { get; }

		/// <summary>
		/// List of values to display in the list
		/// </summary>
		IReadOnlyList<ITemplateListParameterItem> Items { get; }

		/// <summary>
		/// Tool tip text to display
		/// </summary>
		string? ToolTip { get; }
	}

	/// <summary>
	/// Possible option for a list parameter
	/// </summary>
	public interface ITemplateListParameterItem
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		ParameterId Id { get; }

		/// <summary>
		/// Group to display this entry in
		/// </summary>
		string? Group { get; }

		/// <summary>
		/// Text to display for this option.
		/// </summary>
		string Text { get; }

		/// <summary>
		/// Argument to add if this parameter is enabled.
		/// </summary>
		string? ArgumentIfEnabled { get; }

		/// <summary>
		/// Arguments to add if this parameter is enabled.
		/// </summary>
		IReadOnlyList<string>? ArgumentsIfEnabled { get; }

		/// <summary>
		/// Argument to add if this parameter is disabled.
		/// </summary>
		string? ArgumentIfDisabled { get; }

		/// <summary>
		/// Arguments to add if this parameter is disabled.
		/// </summary>
		IReadOnlyList<string>? ArgumentsIfDisabled { get; }

		/// <summary>
		/// Whether this item is selected by default
		/// </summary>
		bool Default { get; }

		/// <summary>
		/// Whether this item is selected by default
		/// </summary>
		bool? ScheduleOverride { get; }
	}

	/// <summary>
	/// Extension methods for templates
	/// </summary>
	public static class TemplateExtensions
	{
		/// <summary>
		/// Gets the full argument list for a template
		/// </summary>
		public static void GetArgumentsForParameters(this ITemplate template, IReadOnlyDictionary<ParameterId, string> parameters, List<string> arguments)
		{
			arguments.AddRange(template.Arguments);

			foreach (ITemplateParameter parameter in template.Parameters)
			{
				parameter.GetArguments(parameters, false, arguments);
			}
		}

		/// <summary>
		/// Gets the arguments for default options in this template. Does not include the standard template arguments.
		/// </summary>
		/// <returns>List of default arguments</returns>
		public static void GetDefaultParameters(this ITemplate template, Dictionary<ParameterId, string> parameters, bool scheduledBuild)
		{
			foreach (ITemplateParameter parameter in template.Parameters)
			{
				parameter.GetDefaultParameters(parameters, scheduledBuild);
			}
		}
	}

	/// <summary>
	/// Query selecting the base changelist to use
	/// </summary>
	public interface IChangeQuery
	{
		/// <summary>
		/// Name of this query, for display on the dashboard.
		/// </summary>
		string? Name { get; }

		/// <summary>
		/// Condition to evaluate before deciding to use this query. May query tags in a preflight.
		/// </summary>
		Condition? Condition { get; }

		/// <summary>
		/// The template id to query
		/// </summary>
		TemplateId? TemplateId { get; }

		/// <summary>
		/// The target to query
		/// </summary>
		string? Target { get; }

		/// <summary>
		/// Whether to match a job that produced warnings
		/// </summary>
		IReadOnlyList<JobStepOutcome>? Outcomes { get; }

		/// <summary>
		/// Finds the last commit with this tag
		/// </summary>
		CommitTag? CommitTag { get; }
	}
}
