// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Jobs.Templates
{
	/// <summary>
	/// Interface for a collection of template documents
	/// </summary>
	public interface ITemplateCollection
	{
		/// <summary>
		/// Gets a template by ID
		/// </summary>
		/// <param name="templateId">Unique id of the template</param>
		/// <returns>The template document</returns>
		Task<ITemplate?> GetAsync(ContentHash templateId);
	}
}
