// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace EpicGames.Horde.Jobs.Templates
{
	/// <summary>
	/// Identifier for a template parameter
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[StringIdConverter(typeof(ParameterIdConverter))]
	public record struct ParameterId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ParameterId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.Sanitize"/>
		public static ParameterId Sanitize(string name)
			=> new(StringId.Sanitize(name));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class ParameterIdConverter : StringIdConverter<ParameterId>
	{
		/// <inheritdoc/>
		public override ParameterId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(ParameterId value) => value.Id;
	}
}
