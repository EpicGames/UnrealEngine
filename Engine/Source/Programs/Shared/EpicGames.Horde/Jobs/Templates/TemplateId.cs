// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Jobs.Templates
{
	/// <summary>
	/// Identifier for a job template
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<TemplateId, TemplateIdConverter>))]
	[StringIdConverter(typeof(TemplateIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<TemplateId, TemplateIdConverter>))]
	public readonly record struct TemplateId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public TemplateId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class TemplateIdConverter : StringIdConverter<TemplateId>
	{
		/// <inheritdoc/>
		public override TemplateId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(TemplateId value) => value.Id;
	}
}
