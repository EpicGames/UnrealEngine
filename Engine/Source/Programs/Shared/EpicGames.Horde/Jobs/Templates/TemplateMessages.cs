// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.Templates
{
	/// <summary>
	/// Response describing a template
	/// </summary>
	public class GetTemplateResponseBase
	{
		/// <summary>
		/// Name of the template
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Description for the template
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Default priority for this job
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// Whether to allow preflights of this template
		/// </summary>
		public bool AllowPreflights { get; set; }

		/// <summary>
		/// Whether to always update issues on jobs using this template
		/// </summary>
		public bool UpdateIssues { get; set; }

		/// <summary>
		/// The initial agent type to parse the BuildGraph script on
		/// </summary>
		public string? InitialAgentType { get; set; }

		/// <summary>
		/// Path to a file within the stream to submit to generate a new changelist for jobs
		/// </summary>
		public string? SubmitNewChange { get; }

		/// <summary>
		/// Parameters for the job.
		/// </summary>
		public List<string> Arguments { get; set; }

		/// <summary>
		/// List of parameters for this template
		/// </summary>
		public List<GetTemplateParameterResponse> Parameters { get; set; }

		/// <summary>
		/// Parameterless constructor for serialization
		/// </summary>
		protected GetTemplateResponseBase()
		{
			Name = null!;
			AllowPreflights = true;
			Arguments = [];
			Parameters = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="template">The template to construct from</param>
		public GetTemplateResponseBase(ITemplate template)
		{
			Name = template.Name;
			Description = template.Description;
			Priority = template.Priority;
			AllowPreflights = template.AllowPreflights;
			UpdateIssues = template.UpdateIssues;
			InitialAgentType = template.InitialAgentType;
			SubmitNewChange = template.SubmitNewChange;
			Arguments = [.. template.Arguments];
			Parameters = template.Parameters.ConvertAll(x => CreateParameterResponse(x));
		}

		static GetTemplateParameterResponse CreateParameterResponse(ITemplateParameter parameter)
		{
			return parameter switch
			{
				ITemplateBoolParameter boolParameter
					=> new GetTemplateBoolParameterResponse(boolParameter),

				ITemplateTextParameter textParameter
					=> new GetTemplateTextParameterResponse(textParameter),

				ITemplateListParameter listParameter
					=> new GetTemplateListParameterResponse(listParameter),

				_ => throw new NotImplementedException()
			};
		}
	}

	/// <summary>
	/// Response describing a template
	/// </summary>
	public class GetTemplateResponse : GetTemplateResponseBase
	{
		/// <summary>
		/// Unique id of the template
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Parameterless constructor for serialization
		/// </summary>
		protected GetTemplateResponse()
			: base()
		{
			Id = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="template">The template to construct from</param>
		public GetTemplateResponse(ITemplate template)
			: base(template)
		{
			Id = template.Hash.ToString();
		}
	}

	/// <summary>
	/// Base class for template parameters
	/// </summary>
	[JsonKnownTypes(typeof(GetTemplateBoolParameterResponse), typeof(GetTemplateTextParameterResponse), typeof(GetTemplateListParameterResponse))]
	public abstract class GetTemplateParameterResponse : ITemplateParameter
	{
		void ITemplateParameter.GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments)
			=> throw new NotSupportedException();

		void ITemplateParameter.GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild)
			=> throw new NotSupportedException();
	}

	/// <summary>
	/// Allows the user to toggle an option on or off
	/// </summary>
	[JsonDiscriminator("Bool")]
	public class GetTemplateBoolParameterResponse : GetTemplateParameterResponse, ITemplateBoolParameter
	{
		/// <inheritdoc/>
		public ParameterId Id { get; set; }

		/// <inheritdoc/>
		public string Label { get; set; }

		/// <inheritdoc/>
		public string? ArgumentIfEnabled { get; set; }

		/// <inheritdoc/>
		public List<string>? ArgumentsIfEnabled { get; set; }

		/// <inheritdoc/>
		public string? ArgumentIfDisabled { get; set; }

		/// <inheritdoc/>
		public List<string>? ArgumentsIfDisabled { get; set; }

		/// <inheritdoc/>
		public bool Default { get; set; }

		/// <inheritdoc/>
		public bool? ScheduleOverride { get; set; }

		/// <inheritdoc/>
		public string? ToolTip { get; set; }

		IReadOnlyList<string>? ITemplateBoolParameter.ArgumentsIfEnabled => ArgumentsIfEnabled;
		IReadOnlyList<string>? ITemplateBoolParameter.ArgumentsIfDisabled => ArgumentsIfDisabled;

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetTemplateBoolParameterResponse()
		{
			Label = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetTemplateBoolParameterResponse(ITemplateBoolParameter parameter)
		{
			Id = parameter.Id;
			Label = parameter.Label;
			ArgumentIfEnabled = parameter.ArgumentIfEnabled;
			ArgumentsIfEnabled = parameter.ArgumentsIfEnabled?.ToList();
			ArgumentIfDisabled = parameter.ArgumentIfDisabled;
			ArgumentsIfDisabled = parameter.ArgumentsIfDisabled?.ToList();
			Default = parameter.Default;
			ScheduleOverride = parameter.ScheduleOverride;
			ToolTip = parameter.ToolTip;
		}
	}

	/// <summary>
	/// Free-form text entry parameter
	/// </summary>
	[JsonDiscriminator("Text")]
	public class GetTemplateTextParameterResponse : GetTemplateParameterResponse, ITemplateTextParameter
	{
		/// <inheritdoc/>
		public ParameterId Id { get; set; }

		/// <inheritdoc/>
		public string Label { get; set; }

		/// <inheritdoc/>
		public string Argument { get; set; }

		/// <inheritdoc/>
		public string Default { get; set; }

		/// <inheritdoc/>
		public string? ScheduleOverride { get; set; }

		/// <inheritdoc/>
		public string? Hint { get; set; }

		/// <inheritdoc/>
		public string? Validation { get; set; }

		/// <inheritdoc/>
		public string? ValidationError { get; set; }

		/// <inheritdoc/>
		public string? ToolTip { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetTemplateTextParameterResponse()
		{
			Label = String.Empty;
			Argument = String.Empty;
			Default = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetTemplateTextParameterResponse(ITemplateTextParameter parameter)
		{
			Id = parameter.Id;
			Label = parameter.Label;
			Argument = parameter.Argument;
			Default = parameter.Default;
			ScheduleOverride = parameter.ScheduleOverride;
			Hint = parameter.Hint;
			Validation = parameter.Validation;
			ValidationError = parameter.ValidationError;
			ToolTip = parameter.ToolTip;
		}
	}

	/// <summary>
	/// Allows the user to select a value from a constrained list of choices
	/// </summary>
	[JsonDiscriminator("List")]
	public class GetTemplateListParameterResponse : GetTemplateParameterResponse, ITemplateListParameter
	{
		/// <inheritdoc/>
		public string Label { get; }

		/// <inheritdoc/>
		public TemplateListParameterStyle Style { get; }

		/// <inheritdoc cref="ITemplateListParameter.Items"/>
		public List<GetTemplateListParameterItemResponse> Items { get; set; }

		/// <inheritdoc/>
		public string? ToolTip { get; }

		IReadOnlyList<ITemplateListParameterItem> ITemplateListParameter.Items => Items;

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetTemplateListParameterResponse()
		{
			Label = String.Empty;
			Items = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetTemplateListParameterResponse(ITemplateListParameter parameter)
		{
			Label = parameter.Label;
			Style = parameter.Style;
			Items = parameter.Items.ConvertAll(x => new GetTemplateListParameterItemResponse(x));
			ToolTip = parameter.ToolTip;
		}
	}

	/// <summary>
	/// Possible option for a list parameter
	/// </summary>
	public class GetTemplateListParameterItemResponse : ITemplateListParameterItem
	{
		/// <inheritdoc/>
		public ParameterId Id { get; set; }

		/// <inheritdoc/>
		public string? Group { get; set; }

		/// <inheritdoc/>
		public string Text { get; set; }

		/// <inheritdoc/>
		public string? ArgumentIfEnabled { get; set; }

		/// <inheritdoc/>
		public List<string>? ArgumentsIfEnabled { get; set; }

		/// <inheritdoc/>
		public string? ArgumentIfDisabled { get; set; }

		/// <inheritdoc/>
		public List<string>? ArgumentsIfDisabled { get; set; }

		/// <inheritdoc/>
		public bool Default { get; set; }

		/// <inheritdoc/>
		public bool? ScheduleOverride { get; set; }

		IReadOnlyList<string>? ITemplateListParameterItem.ArgumentsIfEnabled => ArgumentsIfEnabled;
		IReadOnlyList<string>? ITemplateListParameterItem.ArgumentsIfDisabled => ArgumentsIfDisabled;

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetTemplateListParameterItemResponse()
		{
			Text = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetTemplateListParameterItemResponse(ITemplateListParameterItem item)
		{
			Id = item.Id;
			Group = item.Group;
			Text = item.Text;
			ArgumentIfEnabled = item.ArgumentIfEnabled;
			ArgumentsIfEnabled = item.ArgumentsIfEnabled?.ToList();
			ArgumentIfDisabled = item.ArgumentIfDisabled;
			ArgumentsIfDisabled = item.ArgumentsIfDisabled?.ToList();
			Default = item.Default;
			ScheduleOverride = item.ScheduleOverride;
		}
	}
}
