// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using EpicGames.Horde.Commits;

#pragma warning disable CA2227 // Change 'x' to be read-only by removing the property setter

namespace EpicGames.Horde.Jobs.TestData
{
	/// <summary>
	/// Test outcome
	/// </summary>
	public enum TestOutcome
	{
		/// <summary>
		/// The test was successful
		/// </summary>
		Success,
		/// <summary>
		/// The test failed
		/// </summary>
		Failure,
		/// <summary>
		/// The test was skipped
		/// </summary>
		Skipped,
		/// <summary>
		/// The test had an unspecified result
		/// </summary>
		Unspecified
	}

	/// <summary>
	/// Test phase outcome
	/// </summary>
	public enum TestPhaseOutcome
	{
		/// <summary>
		/// The phase has failed
		/// </summary>
		Failed,
		/// <summary>
		/// The phase was successful
		/// </summary>
		Success,
		/// <summary>
		/// The phase was not run
		/// </summary>
		NotRun,
		/// <summary>
		/// The phase was interrupted in its execution
		/// </summary>
		Interrupted,
		/// <summary>
		/// The phase was skipped
		/// </summary>
		Skipped,
		/// <summary>
		/// The phase outcome is unknown
		/// </summary>
		Unknown
	}

	/// <summary>
	/// Response object describing test data to store
	/// </summary>
	public class CreateTestDataRequest
	{
		/// <summary>
		/// The job which produced the data
		/// </summary>
		[Required]
		public JobId JobId { get; set; }

		/// <summary>
		/// The step that ran
		/// </summary>
		[Required]
		public JobStepId StepId { get; set; }

		/// <summary>
		/// Key used to identify the particular data
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The data stored for this test
		/// </summary>
		[Required]
		public Dictionary<string, object> Data { get; set; } = [];
	}

	/// <summary>
	/// Response object describing the created document
	/// </summary>
	public class CreateTestDataResponse
	{
		/// <summary>
		/// The id for the new document
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Id of the new document</param>
		public CreateTestDataResponse(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Response object containing the testdata id and the associated test
	/// </summary>
	public class GetJobStepTestDataIDResponse
	{
		/// <summary>
		/// The unique testdata id
		/// </summary>
		public string TestDataId { get; set; } = String.Empty;

		/// <summary>
		/// The test key related to the testdata
		/// </summary>
		public string TestKey { get; set; } = String.Empty;

		/// <summary>
		/// The test name related to the testdata
		/// </summary>
		public string TestName { get; set; } = String.Empty;

		/// <summary>
		/// The test name reference id related to the testdata
		/// </summary>
		public string TestNameRef { get; set; } = String.Empty;
	}

	/// <summary>
	/// Response object describing test results
	/// </summary>
	public class GetTestDataResponse
	{
		/// <summary>
		/// Unique id of the test data
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// Stream that generated the test data
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// The template reference id
		/// </summary>
		public string TemplateRefId { get; set; } = String.Empty;

		/// <summary>
		/// The job which produced the data
		/// </summary>
		public string JobId { get; set; } = String.Empty;

		/// <summary>
		/// The step that ran
		/// </summary>
		public string StepId { get; set; } = String.Empty;

		/// <summary>
		/// The changelist number that contained the data
		/// </summary>
		[Obsolete("Use CommitId instead")]
		public int Change
		{
			get => _change ?? _commitId?.GetPerforceChangeOrMinusOne() ?? 0;
			set => _change = value;
		}
		int? _change;

		/// <summary>
		/// The changelist number that contained the data
		/// </summary>
		public CommitIdWithOrder CommitId
		{
			get => _commitId ?? CommitIdWithOrder.FromPerforceChange(_change) ?? CommitIdWithOrder.Empty;
			set => _commitId = value;
		}
		CommitIdWithOrder? _commitId;

		/// <summary>
		/// The preflight commit id if any
		/// </summary>
		public CommitId? PreflightCommitId { get; set; }

		/// <summary>
		/// Key used to identify the particular data
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The data stored for this test
		/// </summary>
		public Dictionary<string, object> Data { get; set; } = [];
	}

	/// <summary>
	/// A test environment running in a stream
	/// </summary>
	public class GetTestMetaResponse
	{
		/// <summary>
		/// Meta unique id for environment 
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The platforms in the environment
		/// </summary>
		public List<string> Platforms { get; set; } = [];

		/// <summary>
		/// The build configurations being tested
		/// </summary>
		public List<string> Configurations { get; set; } = [];

		/// <summary>
		/// The build targets being tested
		/// </summary>
		public List<string> BuildTargets { get; set; } = [];

		/// <summary>
		/// The test project name
		/// </summary>	
		public string ProjectName { get; set; } = String.Empty;

		/// <summary>
		/// The rendering hardware interface being used with the test
		/// </summary>
		public string RHI { get; set; } = String.Empty;

		/// <summary>
		/// The variation of the test meta data, for example address sanitizing
		/// </summary>
		public string Variation { get; set; } = String.Empty;
	}

	/// <summary>
	/// A test that runs in a stream
	/// </summary>
	public class GetTestResponse
	{
		/// <summary>
		/// The id of the test
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test 
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test 
		/// </summary>
		public string? DisplayName { get; set; }

		/// <summary>
		/// The name of the test suite
		/// </summary>
		public string? SuiteName { get; set; }

		/// <summary>
		/// The meta data the test runs on
		/// </summary>
		public List<string> Metadata { get; set; } = [];
	}

	/// <summary>
	/// Get tests request
	/// </summary>
	public class GetTestsRequest
	{
		/// <summary>
		/// Test ids to get
		/// </summary>
		public List<string> TestIds { get; set; } = [];
	}

	/// <summary>
	/// Get Tests for streams request
	/// </summary>
	public class GetStreamTestsRequest
	{
		/// <summary>
		/// Stream ids to get
		/// </summary>
		public List<string> StreamIds { get; set; } = [];
	}

	/// <summary>
	/// Get test metadata request
	/// </summary>
	public class GetMetadataRequest
	{
		/// <summary>
		/// Test metadata ids to get
		/// </summary>
		public List<string>? MetadataIds { get; set; }

		/// <summary>
		/// Matching key/value pair Test metadata to get
		/// </summary>
		public Dictionary<string, string>? Entries { get; set; }
	}

	/// <summary>
	/// Get test tag request
	/// </summary>
	public class GetTestTagRequest
	{
		/// <summary>
		/// Test tag ids to get
		/// </summary>
		public List<string>? TagIds { get; set; }

		/// <summary>
		/// Test tag name to get
		/// </summary>
		public List<string>? TagNames { get; set; }
	}

	/// <summary>
	/// Get test recipe request
	/// </summary>
	public class GetTestRecipesRequest
	{
		/// <summary>
		/// Test recipe ids to get
		/// </summary>
		public List<string>? RecipeIds { get; set; }

		/// <summary>
		/// Test ids to get
		/// </summary>
		public List<string>? TestIds { get; set; }

		/// <summary>
		/// Stream ids to get
		/// </summary>
		public List<string>? StreamIds { get; set; }
	}

	/// <summary>
	/// Get test phases from test keys and ids request
	/// </summary>
	public class GetTestPhasesRequest
	{
		/// <summary>
		/// Test keys to get
		/// </summary>
		public List<string>? TestKeys { get; set; }

		/// <summary>
		/// Test ids to get
		/// </summary>
		public List<string>? TestIds { get; set; }
	}

	/// <summary>
	/// Get test sessions from streams, tests and optionally meta ids
	/// </summary>
	public class GetTestSessionsRequest
	{
		/// <summary>
		/// Stream ids to get
		/// </summary>
		public List<string> StreamIds { get; set; } = [];

		/// <summary>
		/// Test ids to get
		/// </summary>
		public List<string>? TestIds { get; set; }

		/// <summary>
		/// Metadata ids to get
		/// </summary>
		public List<string>? MetaIds { get; set; }
	}

	/// <summary>
	/// Get test health report from streams and tests
	/// </summary>
	public class GetTestHealthReportRequest
	{
		/// <summary>
		/// Stream ids to get
		/// </summary>
		public List<string> StreamIds { get; set; } = [];

		/// <summary>
		/// Test ids to get
		/// </summary>
		public List<string>? TestIds { get; set; }
	}

	/// <summary>
	/// Get phase sessions from stream and phase ids
	/// </summary>
	public class GetPhaseSessionsRequest
	{
		/// <summary>
		/// Stream ids to get
		/// </summary>
		public List<string> StreamIds { get; set; } = [];

		/// <summary>
		/// Phase ids to get
		/// </summary>
		public List<string> PhaseIds { get; set; } = [];
	}

	/// <summary>
	/// Add an audit entry for a test
	/// </summary>
	public class AddTestAuditRequest
	{
		/// <summary>
		/// The test name reference id
		/// </summary>
		public string TestId { get; set; } = String.Empty;

		/// <summary>
		/// Whether or not the test schedules should be enabled
		/// </summary>
		public bool? EnableSchedule { get; set; } = true;

		/// <summary>
		/// The date of the last audit
		/// </summary>
		public DateTimeOffset? LastAuditDate { get; set; }

		/// <summary>
		/// Whether or not the test is under audit
		/// </summary>
		public bool? IsUnderAudit { get; set; }

		/// <summary>
		/// The owner id of the related test
		/// </summary>
		public string? Owner { get; set; }

		/// <summary>
		/// The customer ids of the related test
		/// </summary>
		public List<string>? Customers { get; set; }

		/// <summary>
		/// key/value pairs of stream id mapping to workflow id
		/// </summary>
		public Dictionary<string, string>? Notifications { get; set; }

		/// <summary>
		/// The team for which the test was intended for
		/// </summary>
		public string? Team { get; set; }

		/// <summary>
		/// The test harness
		/// </summary>
		public string? Harness { get; set; }

		/// <summary>
		/// The intent of the test
		/// </summary>
		public string? Intent { get; set; }

		/// <summary>
		/// The test profile
		/// </summary>
		public string? Profile { get; set; }

		/// <summary>
		/// Additional notes attached to the test audit
		/// </summary>
		public string? Notes { get; set; }
	}

	/// <summary>
	/// Get test audit entries for test ids
	/// </summary>
	public class GetTestAuditRequest
	{
		/// <summary>
		/// The test ids
		/// </summary>
		public List<string> TestIds { get; set; } = [];
	}

	/// <summary>
	/// A test audit entry
	/// </summary>
	public class GetTestAuditResponse
	{
		/// <summary>
		/// The test id
		/// </summary>
		public string TestId { get; set; } = String.Empty;

		/// <summary>
		/// The user id that send the test audit
		/// </summary>
		public string UserId { get; set; } = String.Empty;

		/// <summary>
		/// The date at which this audit entry was done
		/// </summary>
		public DateTime UserInputDate { get; set; }

		/// <summary>
		/// Whether or not the test schedules should be enabled
		/// </summary>
		public bool? EnableSchedule { get; set; } = true;

		/// <summary>
		/// The date of the last audit
		/// </summary>
		public DateTime? LastAuditDate { get; set; }

		/// <summary>
		/// Whether or not the test is under audit
		/// </summary>
		public bool? IsUnderAudit { get; set; }

		/// <summary>
		/// The owner id of the related test
		/// </summary>
		public string? Owner { get; set; }

		/// <summary>
		/// The customer ids of the related test
		/// </summary>
		public List<string>? Customers { get; set; }

		/// <summary>
		/// key/value pairs of stream id mapping to workflow id
		/// </summary>
		public Dictionary<string, string>? Notifications { get; set; }

		/// <summary>
		/// The team for which the test was intended for
		/// </summary>
		public string? Team { get; set; }

		/// <summary>
		/// The test harness
		/// </summary>
		public string? Harness { get; set; }

		/// <summary>
		/// The intent of the test
		/// </summary>
		public string? Intent { get; set; }

		/// <summary>
		/// The test profile
		/// </summary>
		public string? Profile { get; set; }

		/// <summary>
		/// Additional notes attached to the test audit
		/// </summary>
		public string? Notes { get; set; }
	}

	/// <summary>
	/// A test health report
	/// </summary>
	public class GetTestHealthReportResponse
	{
		/// <summary>
		/// The report id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The test id associated to the report
		/// </summary>
		public string TestId { get; set; } = String.Empty;

		/// <summary>
		/// The stream id associated to the report
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// The date of the last update
		/// </summary>
		public DateTime LastUpdateDate { get; set; }

		/// <summary>
		/// Whether or not the test healthy
		/// </summary>
		public bool IsHealthy { get; set; }

		/// <summary>
		/// The state associated with the test health
		/// </summary>
		public string State { get; set; } = String.Empty;

		/// <summary>
		/// The success rate of the test health
		/// </summary>
		public int SuccessRate { get; set; }

		/// <summary>
		/// The failure rate of the test health
		/// </summary>
		public int FailureRate { get; set; }

		/// <summary>
		/// The catastrophic rate of the test failure
		/// </summary>
		public int CatastrophicFailureRate { get; set; }

		/// <summary>
		/// The redundant error rate of the test failure
		/// </summary>
		public int RedundantErrorRate { get; set; }

		/// <summary>
		/// The last date of the notification sent
		/// </summary>
		public DateTime? NotificationLastDate { get; set; }
	}

	/// <summary>
	/// A test suite that runs in a stream, contain subtests
	/// </summary>
	public class GetTestSuiteResponse
	{
		/// <summary>
		/// The id of the suite
		/// </summary>	
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test suite
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The meta data the test suite runs on
		/// </summary>
		public List<string> Metadata { get; set; } = [];
	}

	/// <summary>
	/// Response object describing test results
	/// </summary>
	public class GetTestStreamResponse
	{
		/// <summary>
		/// The stream id
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// Individual tests which run in the stream
		/// </summary>
		public List<GetTestResponse> Tests { get; set; } = [];

		/// <summary>
		/// Test suites that run in the stream
		/// </summary>
		public List<GetTestSuiteResponse> TestSuites { get; set; } = [];

		/// <summary>
		/// Test suites that run in the stream
		/// </summary>
		public List<GetTestMetaResponse> TestMetadata { get; set; } = [];
	}

	/// <summary>
	/// Suite test data
	/// </summary>
	public class GetSuiteTestDataResponse
	{
		/// <summary>
		/// The test id
		/// </summary>
		public string TestId { get; set; } = String.Empty;

		/// <summary>
		/// The outcome of the suite test
		/// </summary>
		public TestOutcome Outcome { get; set; }

		/// <summary>
		/// How long the suite test ran
		/// </summary>
		public TimeSpan Duration { get; set; }

		/// <summary>
		/// Test UID for looking up in test details
		/// </summary>
		public string UID { get; set; } = String.Empty;

		/// <summary>
		/// The number of test warnings generated
		/// </summary>
		public int? WarningCount { get; set; }

		/// <summary>
		/// The number of test errors generated
		/// </summary>
		public int? ErrorCount { get; set; }
	}

	/// <summary>
	/// Test details
	/// </summary>
	public class GetTestDataDetailsResponse
	{
		/// <summary>
		/// The corresponding test ref
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The test documents for this ref
		/// </summary>
		public List<string> TestDataIds { get; set; } = [];

		/// <summary>
		/// Suite test data
		/// </summary>		
		public List<GetSuiteTestDataResponse>? SuiteTests { get; set; }
	}

	/// <summary>
	/// Data ref 
	/// </summary>
	public class GetTestDataRefResponse
	{
		/// <summary>
		/// The test ref id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The associated stream
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// The associated job id
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// The associated step id
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// How long the test ran
		/// </summary>
		public TimeSpan Duration { get; set; }

		/// <summary>
		/// The build changelist upon which the test ran, may not correspond to the job changelist
		/// </summary>
		[Obsolete("Use BuildCommitId instead")]
		public int BuildChangeList
		{
			get => _buildChangeList ?? _buildCommitId?.GetPerforceChangeOrMinusOne() ?? 0;
			set => _buildChangeList = value;
		}
		int? _buildChangeList;

#pragma warning disable CS0618 // Type or member is obsolete
		/// <summary>
		/// The build changelist upon which the test ran, may not correspond to the job changelist
		/// </summary>
		public CommitId BuildCommitId
		{
			get => _buildCommitId ?? CommitId.FromPerforceChange(_buildChangeList) ?? CommitId.Empty;
			set => _buildCommitId = value;
		}
#pragma warning restore CS0618 // Type or member is obsolete

		CommitId? _buildCommitId;

		/// <summary>
		/// The platform the test ran on 
		/// </summary>
		public string MetaId { get; set; } = String.Empty;

		/// <summary>
		/// The test id in stream
		/// </summary>
		public string? TestId { get; set; }

		/// <summary>
		/// The outcome of the test
		/// </summary>
		public TestOutcome? Outcome { get; set; }

		/// <summary>
		/// The if of the stream test suite
		/// </summary>
		public string? SuiteId { get; set; }

		/// <summary>
		/// Suite tests skipped
		/// </summary>
		public int? SuiteSkipCount { get; set; }

		/// <summary>
		/// Suite test warnings
		/// </summary>
		public int? SuiteWarningCount { get; set; }

		/// <summary>
		/// Suite test errors
		/// </summary>
		public int? SuiteErrorCount { get; set; }

		/// <summary>
		/// Suite test successes
		/// </summary>
		public int? SuiteSuccessCount { get; set; }
	}

	/// <summary>
	/// A test name/key ref
	/// </summary>
	public class GetTestNameResponse
	{
		/// <summary>
		/// The id of the test
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test 
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The key of the test
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The team associated with this test
		/// </summary>
		public string? Team { get; set; }

		/// <summary>
		/// The intent associated with this test
		/// </summary>
		public string? Intent { get; set; }

		/// <summary>
		/// The harness associated with this test
		/// </summary>
		public string? Harness { get; set; }

		/// <summary>
		/// The profile associated with this test
		/// </summary>
		public string? Profile { get; set; }
	}

	/// <summary>
	/// A set of environment conditions in which a test was run
	/// </summary>
	public class GetTestMetadataResponse
	{
		/// <summary>
		/// Metadata unique id for environment 
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The set of key/value pairs this environment represent
		/// </summary>
		public Dictionary<string, string> Entries { get; set; } = [];
	}

	/// <summary>
	/// A test tag ref
	/// </summary>
	public class GetTestTagResponse
	{
		/// <summary>
		/// Tag unique id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The name of the tag
		/// </summary>
		public string Name { get; set; } = String.Empty;
	}

	/// <summary>
	/// A test recipe ref
	/// </summary>
	public class GetTestRecipeResponse
	{
		/// <summary>
		/// Recipe unique id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The test ref id associated with this recipe
		/// </summary>
		public string NameRef { get; set; } = String.Empty;

		/// <summary>
		/// The stream id associated with this recipe
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// The job template id associated with this recipe
		/// </summary>
		public string JobTemplateId { get; set; } = String.Empty;

		/// <summary>
		/// The job step name associated with this recipe
		/// </summary>
		public string JobStep { get; set; } = String.Empty;

		/// <summary>
		/// The test node class associated with this recipe
		/// </summary>
		public string NodeClass { get; set; } = String.Empty;

		/// <summary>
		/// The test version associated with this recipe
		/// </summary>
		public string Version { get; set; } = String.Empty;

		/// <summary>
		/// The test timeout in minutes associated with this recipe
		/// </summary>
		public float TimeoutMinutes { get; set; } = 0;

		/// <summary>
		/// The workflow id associated with this recipe
		/// </summary>
		public string? WorkflowId { get; set; }

		/// <summary>
		/// The test parameters associated with this recipe
		/// </summary>
		public Dictionary<string, object>? Params { get; set; }
	}

	/// <summary>
	/// A test phase from a test
	/// </summary>
	public class GetPhaseResponse
	{
		/// <summary>
		/// The id of the test phase
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test phase
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The key of the test phase
		/// </summary>
		public string Key { get; set; } = String.Empty;
	}

	/// <summary>
	/// All test phases from a test
	/// </summary>
	public class GetTestPhaseResponse
	{
		/// <summary>
		/// The id of the test phase
		/// </summary>
		public string TestId { get; set; } = String.Empty;

		/// <summary>
		/// The name of the test phase
		/// </summary>
		public string TestName { get; set; } = String.Empty;

		/// <summary>
		/// The key of the test phase
		/// </summary>
		public string TestKey { get; set; } = String.Empty;

		/// <summary>
		/// The phases associated with the test
		/// </summary>
		public List<GetPhaseResponse> Phases { get; set; } = [];
	}

	/// <summary>
	/// Response object describing test results
	/// </summary>
	public class GetTestSessionStreamResponse
	{
		/// <summary>
		/// The stream id
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// Individual tests which run in the stream
		/// </summary>
		public List<GetTestNameResponse> Tests { get; set; } = [];

		/// <summary>
		/// Known test metadata
		/// </summary>
		public List<GetTestMetadataResponse> TestMetadata { get; set; } = [];

		/// <summary>
		/// Known test tags
		/// </summary>
		public List<GetTestTagResponse> TestTags { get; set; } = [];
	}

	/// <summary>
	/// Response object for a test session
	/// </summary>
	public class GetTestSessionResponse
	{
		/// <summary>
		/// The test session id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The stream id associated with the test session
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// Test metadata id associated with the test session
		/// </summary>
		public string MetadataId { get; set; } = String.Empty;

		/// <summary>
		/// The test data id associated with the test session
		/// </summary>
		public string TestDataId { get; set; } = String.Empty;

		/// <summary>
		/// The list of tag ids associated with the test session
		/// </summary>
		public string[]? TagIds { get; set; }

		/// <summary>
		/// The commit id associated with the test session
		/// </summary>
		public string CommitId { get; set; } = String.Empty;

		/// <summary>
		/// The commit order associated with the test session
		/// </summary>
		public int CommitOrder { get; set; } = -1;

		/// <summary>
		/// The duration of the test session
		/// </summary>
		public double Duration { get; set; }

		/// <summary>
		/// The date time at which the test session started
		/// </summary>
		public DateTime StartDateTime { get; set; }

		/// <summary>
		/// The test name reference associated with the test session
		/// </summary>
		public string NameRef { get; set; } = String.Empty;

		/// <summary>
		/// The recipe id associated with the test session
		/// </summary>
		public string? RecipeId { get; set; }

		/// <summary>
		/// The outcome of the test session
		/// </summary>
		public TestOutcome Outcome { get; set; }

		/// <summary>
		/// The total count of phases
		/// </summary>
		public int PhasesTotalCount { get; set; }

		/// <summary>
		/// The count of succeeded phases
		/// </summary>
		public int PhasesSucceededCount { get; set; }

		/// <summary>
		/// The count of undefined phases
		/// </summary>
		public int PhasesUndefinedCount { get; set; }

		/// <summary>
		/// The count of failed phases
		/// </summary>
		public int PhasesFailedCount { get; set; }

		/// <summary>
		/// The job id associated with the test session
		/// </summary>
		public string JobId { get; set; } = String.Empty;

		/// <summary>
		/// The step job id associated with the test session
		/// </summary>
		public string StepId { get; set; } = String.Empty;

		/// <summary>
		/// The error fingerprints associated with the test session
		/// </summary>
		public List<GetErrorFingerprintResponse>? ErrorFingerprints { get; set; }
	}

	/// <summary>
	/// Response object for error fingerprints
	/// </summary>
	public class GetErrorFingerprintResponse
	{
		/// <summary>
		/// The error fingerprint key
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The phase keys associated with the error fingerprint
		/// </summary>
		public List<string> Phases { get; set; } = [];
	}

	/// <summary>
	/// Response object for a test phase session
	/// </summary>
	public class GetTestPhaseSessionResponse
	{
		/// <summary>
		/// The phase session id
		/// </summary>
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// The stream id associated with the phase session
		/// </summary>
		public string StreamId { get; set; } = String.Empty;

		/// <summary>
		/// Test metadata id associated with the phase session
		/// </summary>
		public string MetadataId { get; set; } = String.Empty;

		/// <summary>
		/// The test session id associated with the phase session
		/// </summary>
		public string SessionId { get; set; } = String.Empty;

		/// <summary>
		/// The list of tag ids associated with the phase session
		/// </summary>
		public string[]? TagIds { get; set; }

		/// <summary>
		/// The commit id associated with the phase session
		/// </summary>
		public string CommitId { get; set; } = String.Empty;

		/// <summary>
		/// The commit order associated with the phase session
		/// </summary>
		public int CommitOrder { get; set; } = -1;

		/// <summary>
		/// The duration of the phase session
		/// </summary>
		public double Duration { get; set; }

		/// <summary>
		/// The date time at which the phase session started
		/// </summary>
		public DateTime StartDateTime { get; set; }

		/// <summary>
		/// The test phase reference associated with the phase session
		/// </summary>
		public string PhaseRef { get; set; } = String.Empty;

		/// <summary>
		/// The outcome of the phase session
		/// </summary>
		public TestPhaseOutcome Outcome { get; set; }

		/// <summary>
		/// The job id associated with the phase session
		/// </summary>
		public string JobId { get; set; } = String.Empty;

		/// <summary>
		/// The step job id associated with the phase session
		/// </summary>
		public string StepId { get; set; } = String.Empty;

		/// <summary>
		/// The path to the event stream
		/// </summary>
		public string? EventStreamPath { get; set; }

		/// <summary>
		/// Whether the test phase encountered at least one warning
		/// </summary>
		public bool? HasWarning { get; set; }

		/// <summary>
		/// The error fingerprint from the event stream
		/// </summary>
		public string? ErrorFingerprint { get; set; }
	}
}
