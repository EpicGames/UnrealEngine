// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Jobs.Timing
{
	/// <summary>
	/// Timing information for a particular job
	/// </summary>
	public interface IJobTiming
	{
		/// <summary>
		/// Gets timing information for a particular step
		/// </summary>
		/// <param name="name">Name of the node being executed</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		/// <param name="timing">Receives the timing information for the given step</param>
		/// <returns>True if the timing was found</returns>
		public bool TryGetStepTiming(string name, ILogger logger, [NotNullWhen(true)] out IJobStepTiming? timing);
	}

	/// <summary>
	/// Information about the timing for an individual step
	/// </summary>
	public interface IJobStepTiming
	{
		/// <summary>
		/// Wait time before executing the group containing this node
		/// </summary>
		public float? AverageWaitTime { get; }

		/// <summary>
		/// Time taken for the group containing this node to initialize
		/// </summary>
		public float? AverageInitTime { get; }

		/// <summary>
		/// Time spent executing this node
		/// </summary>
		public float? AverageDuration { get; }
	}
}
