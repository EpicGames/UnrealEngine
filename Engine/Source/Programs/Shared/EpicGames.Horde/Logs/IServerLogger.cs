// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Interface for a log device
	/// </summary>
	public interface IServerLogger : ILogger, IAsyncDisposable
	{
		/// <summary>
		/// Flushes the logger with the server and stops the background work
		/// </summary>
		Task StopAsync();
	}

	/// <summary>
	/// Extension methods for <see cref="IServerLogger"/>
	/// </summary>
	public static class ServerLoggerExtensions
	{
		sealed class ServerLoggerWithLeaseLogger : IServerLogger
		{
			readonly IServerLogger _serverLogger;
			readonly ILogger _localLogger;

			public ServerLoggerWithLeaseLogger(IServerLogger serverLogger, ILogger localLogger)
			{
				_serverLogger = serverLogger;
				_localLogger = localLogger;
			}

			/// <inheritdoc/>
			public IDisposable? BeginScope<TState>(TState state) where TState : notnull
				=> _localLogger.BeginScope(state);

			/// <inheritdoc/>
			public ValueTask DisposeAsync()
				=> _serverLogger.DisposeAsync();

			/// <inheritdoc/>
			public bool IsEnabled(LogLevel logLevel)
				=> _serverLogger.IsEnabled(logLevel) || _localLogger.IsEnabled(logLevel);

			/// <inheritdoc/>
			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
			{
				_serverLogger.Log(logLevel, eventId, state, exception, formatter);
				_localLogger.Log(logLevel, eventId, state, exception, formatter);
			}

			/// <inheritdoc/>
			public Task StopAsync()
				=> _serverLogger.StopAsync();
		}

		/// <summary>
		/// Creates a logger which uploads data to the server
		/// </summary>
		/// <param name="serverLogger">Logger for the server</param>
		/// <param name="localLogger">Local log output device</param>
		/// <returns>New logger instance</returns>
		public static IServerLogger WithLocalLogger(this IServerLogger serverLogger, ILogger localLogger)
		{
			return new ServerLoggerWithLeaseLogger(serverLogger, localLogger);
		}
	}
}
