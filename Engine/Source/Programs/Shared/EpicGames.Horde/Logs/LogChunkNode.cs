// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Read-only buffer for log text, with indexed line offsets.
	/// </summary>
	[BlobConverter(typeof(LogChunkConverter))]
	public class LogChunkNode
	{
		/// <summary>
		/// Type of this blob when serialized
		/// </summary>
		public static BlobType BlobType { get; } = new BlobType("{7020B6CA-4174-0F72-AA06-AAB630EFA360}", 1);

		/// <summary>
		/// Provides access to the lines for this chunk through a list interface
		/// </summary>
		class LinesCollection : IReadOnlyList<Utf8String>
		{
			readonly LogChunkNode _owner;

			public LinesCollection(LogChunkNode owner) => _owner = owner;

			/// <inheritdoc/>
			public Utf8String this[int index] => _owner.GetLine(index);

			/// <inheritdoc/>
			public int Count => _owner.LineCount;

			/// <inheritdoc/>
			public IEnumerator<Utf8String> GetEnumerator()
			{
				for (int idx = 0; idx < Count; idx++)
				{
					yield return _owner.GetLine(idx);
				}
			}

			/// <inheritdoc/>
			IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
		}

		/// <summary>
		/// Empty log chunk
		/// </summary>
		public static LogChunkNode Empty { get; } = new LogChunkNode(Array.Empty<byte>(), new int[1]);

		/// <summary>
		/// The raw text data. Contains a complete set of lines followed by newline characters.
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Span for the raw text data.
		/// </summary>
		public ReadOnlySpan<byte> Span => Data.Span;

		/// <summary>
		/// Accessor for the lines in this chunk
		/// </summary>
		public IReadOnlyList<Utf8String> Lines { get; }

		/// <summary>
		/// Offsets of lines within the data object, including a sentinel for the end of the data (LineCount + 1 entries).
		/// </summary>
		public IReadOnlyList<int> LineOffsets { get; }

		/// <summary>
		/// Length of this chunk
		/// </summary>
		public int Length => Data.Length;

		/// <summary>
		/// Number of lines in the block (excluding the sentinel).
		/// </summary>
		public int LineCount => LineOffsets.Count - 1;

		/// <summary>
		/// Default constructor
		/// </summary>
		public LogChunkNode()
			: this(Empty.Data, Empty.LineOffsets)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">Data to construct from</param>
		public LogChunkNode(ReadOnlyMemory<byte> data)
			: this(data, FindLineOffsets(data.Span))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		internal LogChunkNode(ReadOnlyMemory<byte> data, IReadOnlyList<int> lineOffsets)
		{
			Data = data;
			Lines = new LinesCollection(this);
			LineOffsets = lineOffsets;
		}

		/// <summary>
		/// Accessor for an individual line
		/// </summary>
		/// <param name="idx">Index of the line to retrieve</param>
		/// <returns>Line at the given index</returns>
		public Utf8String GetLine(int idx) => new(Data.Slice(LineOffsets[idx], LineOffsets[idx + 1] - LineOffsets[idx] - 1));

		/// <summary>
		/// Accessor for an individual line, including the trailing newline character
		/// </summary>
		/// <param name="idx">Index of the line to retrieve</param>
		/// <returns>Line at the given index</returns>
		public Utf8String GetLineWithNewline(int idx) => new(Data.Slice(LineOffsets[idx], LineOffsets[idx + 1] - LineOffsets[idx]));

		/// <summary>
		/// Find the line index for a particular offset
		/// </summary>
		/// <param name="offset">Offset within the text</param>
		/// <returns>The line index</returns>
		public int GetLineIndexForOffset(int offset)
		{
			int lineIdx = LineOffsets.BinarySearch(offset);
			if (lineIdx < 0)
			{
				lineIdx = ~lineIdx - 1;
			}
			return lineIdx;
		}

		/// <summary>
		/// Creates a new list of line offsets for the given text
		/// </summary>
		/// <param name="data"></param>
		/// <returns></returns>
		public static List<int> FindLineOffsets(ReadOnlySpan<byte> data)
		{
			List<int> lineOffsets = [0];
			UpdateLineOffsets(data, 0, lineOffsets);
			return lineOffsets;
		}

		/// <summary>
		/// Updates the length of this chunk, computing all the newline offsets
		/// </summary>
		/// <param name="data">Text to search for line endings</param>
		/// <param name="start">Start offset within the text buffer</param>
		/// <param name="lineOffsets">Offsets of each line within the text</param>
		public static void UpdateLineOffsets(ReadOnlySpan<byte> data, int start, List<int> lineOffsets)
		{
			for (int idx = start; idx < data.Length; idx++)
			{
				if (data[idx] == '\n')
				{
					lineOffsets.Add(idx + 1);
				}
			}
		}
	}

	/// <summary>
	/// Converter from log chunks to blobs
	/// </summary>
	class LogChunkConverter : BlobConverter<LogChunkNode>
	{
		/// <inheritdoc/>
		public override LogChunkNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			byte[] data = reader.ReadVariableLengthBytes().ToArray();
			return new LogChunkNode(data);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, LogChunkNode value, BlobSerializerOptions options)
		{
			writer.WriteVariableLengthBytes(value.Data.Span);
			return LogChunkNode.BlobType;
		}
	}

	/// <summary>
	/// Reference to a chunk of text, with information about its placement in the larger log file
	/// </summary>
	public class LogChunkRef
	{
		/// <summary>
		/// First line within the file
		/// </summary>
		public int LineIndex { get; }

		/// <summary>
		/// Number of lines in this block
		/// </summary>
		public int LineCount { get; }

		/// <summary>
		/// Offset within the entire log file
		/// </summary>
		public long Offset { get; }

		/// <summary>
		/// Length of this chunk
		/// </summary>
		public int Length { get; }

		/// <summary>
		/// Handle to the target chunk
		/// </summary>
		public IHashedBlobRef<LogChunkNode> Target { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="lineIndex">Index of the first line within this block</param>
		/// <param name="lineCount">Number of lines in the chunk</param>
		/// <param name="offset">Offset within the log file</param>
		/// <param name="length">Length of the chunk</param>
		/// <param name="target">Referenced log text</param>
		public LogChunkRef(int lineIndex, int lineCount, long offset, int length, IHashedBlobRef<LogChunkNode> target)
		{
			LineIndex = lineIndex;
			LineCount = lineCount;
			Offset = offset;
			Length = length;
			Target = target;
		}

		/// <summary>
		/// Deserializing constructor
		/// </summary>
		/// <param name="reader"></param>
		public LogChunkRef(IBlobReader reader)
		{
			Target = reader.ReadBlobRef<LogChunkNode>();
			LineIndex = (int)reader.ReadUnsignedVarInt();
			LineCount = (int)reader.ReadUnsignedVarInt();
			Offset = (long)reader.ReadUnsignedVarInt();
			Length = (int)reader.ReadUnsignedVarInt();
		}

		/// <inheritdoc/>
		public void Serialize(IBlobWriter writer)
		{
			writer.WriteBlobRef(Target);
			writer.WriteUnsignedVarInt(LineIndex);
			writer.WriteUnsignedVarInt(LineCount);
			writer.WriteUnsignedVarInt((ulong)Offset);
			writer.WriteUnsignedVarInt(Length);
		}
	}

	/// <summary>
	/// Builder for <see cref="LogChunkNode"/> objects.
	/// </summary>
	public class LogChunkBuilder
	{
		/// <summary>
		/// Accessor for Data
		/// </summary>
		byte[] _data;

		/// <summary>
		/// Current used length of the buffer
		/// </summary>
		int _length;

		/// <summary>
		/// Offsets of the start of each line within the data
		/// </summary>
		readonly List<int> _lineOffsets = [0];

		/// <summary>
		/// Current length of the buffer
		/// </summary>
		public int Length => _length;

		/// <summary>
		/// Number of lines in this buffer
		/// </summary>
		public int LineCount => _lineOffsets.Count - 1;

		/// <summary>
		/// Capacity of the buffer
		/// </summary>
		public int Capacity => _data.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public LogChunkBuilder(int maxLength = 64 * 1024)
		{
			_data = new byte[maxLength];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">Data to initialize this chunk with. Ownership of this array is transferred to the chunk, and its length determines the chunk size.</param>
		/// <param name="length">Number of valid bytes within the initial data array</param>
		public LogChunkBuilder(byte[] data, int length)
			: this(data, length, LogChunkNode.FindLineOffsets(data.AsSpan(0, length)))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data"></param>
		/// <param name="length"></param>
		/// <param name="lineOffsets"></param>
		private LogChunkBuilder(byte[] data, int length, List<int> lineOffsets)
		{
			_data = data;
			_length = length;
			_lineOffsets = lineOffsets;
		}

		/// <summary>
		/// Clear the contents of the buffer
		/// </summary>
		public void Clear()
		{
			_length = 0;
			_lineOffsets.RemoveRange(1, _lineOffsets.Count - 1);
		}

		/// <summary>
		/// Gets a line at the given index
		/// </summary>
		/// <param name="index">Index of the line</param>
		/// <returns>Text for the line</returns>
		public Utf8String GetLine(int index) => new(_data.AsMemory(_lineOffsets[index], _lineOffsets[index + 1] - _lineOffsets[index]));

		/// <summary>
		/// Create a new chunk data object with the given data appended. The internal buffers are reused, with the assumption that
		/// there is no contention over writing to the same location in the chunk.
		/// </summary>
		/// <param name="textData">The data to append</param>
		/// <returns>New chunk data object</returns>
		public void Append(ReadOnlySpan<byte> textData)
		{
			CreateOutputSpace(textData.Length);
			textData.CopyTo(_data.AsSpan(_length, textData.Length));

			int prevLength = _length;
			_length += textData.Length;

			LogChunkNode.UpdateLineOffsets(_data.AsSpan(0, _length), prevLength, _lineOffsets);
		}

		/// <summary>
		/// Appends JSON text from another buffer as plain text in this one
		/// </summary>
		public void AppendJsonAsPlainText(ReadOnlySpan<byte> inputLine, ILogger logger)
		{
			CreateOutputSpace(inputLine.Length);
			try
			{
				_length = ConvertToPlainText(inputLine, _data, _length);
			}
			catch (Exception ex)
			{
				inputLine.CopyTo(_data.AsSpan(_length));
				_length += inputLine.Length;
				logger.LogWarning(ex, "Exception while attempting to parse log text as JSON. Line: \"{Line}\"", Encoding.UTF8.GetString(inputLine).Trim());
			}
			_lineOffsets.Add(_length);
		}

		/// <summary>
		/// Appends JSON text from another buffer as plain text in this one
		/// </summary>
		public void AppendJsonAsPlainText(LogChunkNode srcText, int srcLineIndex, int srcLineCount, ILogger logger)
		{
			for (int idx = 0; idx < srcLineCount; idx++)
			{
				int lineOffset = srcText.LineOffsets[srcLineIndex + idx];
				int nextLineOffset = srcText.LineOffsets[srcLineIndex + idx + 1];
				ReadOnlySpan<byte> inputLine = srcText.Data.Slice(lineOffset, nextLineOffset - lineOffset).Span;
				AppendJsonAsPlainText(inputLine, logger);
			}
		}

		/// <summary>
		/// Ensure there is a certain amount of space in the output buffer
		/// </summary>
		/// <param name="appendLength">Required space</param>
		void CreateOutputSpace(int appendLength)
		{
			int requiredLength = _length + appendLength;
			if (_data.Length < requiredLength)
			{
				Array.Resize(ref _data, requiredLength);
			}
		}

		/// <summary>
		/// Determines if the given line is empty
		/// </summary>
		/// <param name="input">The input data</param>
		/// <returns>True if the given text is empty</returns>
		static bool IsEmptyOrWhitespace(ReadOnlySpan<byte> input)
		{
			for (int idx = 0; idx < input.Length; idx++)
			{
				byte v = input[idx];
				if (v != (byte)'\n' && v != '\r' && v != ' ')
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Converts a JSON log line to plain text
		/// </summary>
		/// <param name="input">The JSON data</param>
		/// <param name="output">Output buffer for the converted line</param>
		/// <param name="outputOffset">Offset within the buffer to write the converted data</param>
		/// <returns></returns>
		static int ConvertToPlainText(ReadOnlySpan<byte> input, byte[] output, int outputOffset)
		{
			if (IsEmptyOrWhitespace(input))
			{
				output[outputOffset] = (byte)'\n';
				return outputOffset + 1;
			}

			Utf8JsonReader reader = new(input);
			if (reader.Read() && reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					if (!reader.ValueTextEquals("message"))
					{
						reader.Skip();
						continue;
					}
					if (!reader.Read() || reader.TokenType != JsonTokenType.String)
					{
						reader.Skip();
						continue;
					}

					int unescapedLength = UnescapeUtf8(reader.ValueSpan, output.AsSpan(outputOffset));
					outputOffset += unescapedLength;

					output[outputOffset] = (byte)'\n';
					outputOffset++;

					break;
				}
			}
			return outputOffset;
		}

		/// <summary>
		/// Unescape a json utf8 string
		/// </summary>
		/// <param name="source">Source span of bytes</param>
		/// <param name="target">Target span of bytes</param>
		/// <returns>Length of the converted data</returns>
		static int UnescapeUtf8(ReadOnlySpan<byte> source, Span<byte> target)
		{
			int length = 0;
			for (; ; )
			{
				// Copy up to the next backslash
				int backslash = source.IndexOf((byte)'\\');
				if (backslash == -1)
				{
					source.CopyTo(target);
					length += source.Length;
					break;
				}
				else if (backslash > 0)
				{
					source.Slice(0, backslash).CopyTo(target);
					source = source.Slice(backslash);
					target = target.Slice(backslash);
					length += backslash;
				}

				// Check what the escape code is
				if (source[1] == 'u')
				{
					char[] chars = [(char)((StringUtils.ParseHexByte(source, 2) << 8) | StringUtils.ParseHexByte(source, 4))];
					int encodedLength = Encoding.UTF8.GetBytes(chars.AsSpan(), target);
					source = source.Slice(6);
					target = target.Slice(encodedLength);
					length += encodedLength;
				}
				else
				{
					target[0] = source[1] switch
					{
						(byte)'\"' => (byte)'\"',
						(byte)'\\' => (byte)'\\',
						(byte)'b' => (byte)'\b',
						(byte)'f' => (byte)'\f',
						(byte)'n' => (byte)'\n',
						(byte)'r' => (byte)'\r',
						(byte)'t' => (byte)'\t',
						_ => source[1]
					};
					source = source.Slice(2);
					target = target.Slice(1);
					length++;
				}
			}
			return length;
		}

		/// <summary>
		/// Shrinks the data allocated by this chunk to the minimum required
		/// </summary>
		public void Shrink()
		{
			if (_data.Length > _length)
			{
				Array.Resize(ref _data, _length);
			}
		}

		/// <summary>
		/// Create an array of lines from the text
		/// </summary>
		/// <returns>Array of lines</returns>
		public Utf8String[] ToArray()
		{
			Utf8String[] lines = new Utf8String[LineCount];
			for (int idx = 0; idx < LineCount; idx++)
			{
				lines[idx] = new Utf8String(_data, _lineOffsets[idx], _lineOffsets[idx + 1] - 1).Clone();
			}
			return lines;
		}

		/// <summary>
		/// Create a <see cref="LogChunkNode"/> object from the current state
		/// </summary>
		/// <returns></returns>
		public LogChunkNode ToLogChunk() => new(_data.AsMemory(0, _length).ToArray(), [.. _lineOffsets]);
	}

	/// <summary>
	/// Builds a sequence of log chunks
	/// </summary>
	class LogChunkSequenceBuilder
	{
		readonly List<LogChunkNode> _chunks = [];
		readonly LogChunkBuilder _nextChunkBuilder;
		int _flushedLength;
		int _flushedLineCount;

		/// <summary>
		/// Desired size for each chunk
		/// </summary>
		public int ChunkSize { get; }

		/// <summary>
		/// The complete chunks. Note that this does not include data which has not yet been flushed.
		/// </summary>
		public IReadOnlyList<LogChunkNode> Chunks => _chunks;

		/// <summary>
		/// Total length of the sequence
		/// </summary>
		public int Length => _flushedLength + _nextChunkBuilder.Length;

		/// <summary>
		/// Number of lines in this builder
		/// </summary>
		public int LineCount => _flushedLineCount + _nextChunkBuilder.LineCount;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="chunkSize">Desired size for each chunk. Each chunk will be limited to this size.</param>
		public LogChunkSequenceBuilder(int chunkSize)
		{
			ChunkSize = chunkSize;
			_nextChunkBuilder = new LogChunkBuilder(chunkSize);
		}

		/// <summary>
		/// Clear the current contents of the buffer
		/// </summary>
		public void Clear()
		{
			_chunks.Clear();
			_nextChunkBuilder.Clear();
			_flushedLength = 0;
			_flushedLineCount = 0;
		}

		/// <summary>
		/// Remove a number of chunks from the start of the builder
		/// </summary>
		/// <param name="count">Number of chunks to remove</param>
		public void Remove(int count)
		{
			for (int idx = 0; idx < count; idx++)
			{
				LogChunkNode chunk = _chunks[idx];
				_flushedLength -= chunk.Length;
				_flushedLineCount -= chunk.LineCount;
			}
			_chunks.RemoveRange(0, count);
		}

		/// <inheritdoc cref="LogChunkBuilder.Append(ReadOnlySpan{Byte})"/>
		public void Append(ReadOnlySpan<byte> textData)
		{
			if (textData.Length == 0)
			{
				return;
			}
			if (textData[^1] != (byte)'\n')
			{
				throw new ArgumentException("Text data to append must end with a newline", nameof(textData));
			}

			while (textData.Length > 0)
			{
				ReadOnlySpan<byte> lineData = textData.Slice(0, textData.IndexOf((byte)'\n') + 1);

				CreateOutputSpace(lineData.Length);
				_nextChunkBuilder.Append(lineData);

				textData = textData.Slice((int)lineData.Length);
			}
		}

		/// <inheritdoc cref="LogChunkBuilder.AppendJsonAsPlainText(ReadOnlySpan{Byte}, ILogger)"/>
		public void AppendJsonAsPlainText(ReadOnlySpan<byte> textData, ILogger logger)
		{
			if (textData.Length == 0)
			{
				return;
			}
			if (textData[^1] != (byte)'\n')
			{
				throw new ArgumentException("Text data to append must end with a newline", nameof(textData));
			}

			while (textData.Length > 0)
			{
				ReadOnlySpan<byte> lineData = textData.Slice(0, textData.IndexOf((byte)'\n') + 1);

				CreateOutputSpace(lineData.Length);
				_nextChunkBuilder.AppendJsonAsPlainText(lineData, logger);

				textData = textData.Slice((int)lineData.Length);
			}
		}

		/// <summary>
		/// Flushes the current contents of the builder
		/// </summary>
		public void Flush()
		{
			if (_nextChunkBuilder.Length > 0)
			{
				LogChunkNode nextChunk = _nextChunkBuilder.ToLogChunk();
				_chunks.Add(nextChunk);
				_nextChunkBuilder.Clear();
				_flushedLength += nextChunk.Length;
				_flushedLineCount += nextChunk.LineCount;
			}
		}

		/// <summary>
		/// Enumerate lines starting at the given index
		/// </summary>
		/// <param name="startIdx">Index to start from</param>
		/// <returns>Sequence of lines</returns>
		public IEnumerable<Utf8String> EnumerateLines(int startIdx = 0)
		{
			int lineIdx = startIdx;
			foreach (LogChunkNode chunk in _chunks)
			{
				for (; lineIdx < chunk.LineCount; lineIdx++)
				{
					yield return chunk.GetLineWithNewline(lineIdx);
				}
				lineIdx -= chunk.LineCount;
			}
			for (; lineIdx < _nextChunkBuilder.LineCount; lineIdx++)
			{
				yield return _nextChunkBuilder.GetLine(lineIdx);
			}
		}

		/// <summary>
		/// Flushes the current chunk if necessary to provide the requested space
		/// </summary>
		/// <param name="requiredSpace">Space required in <see cref="_nextChunkBuilder"/></param>
		void CreateOutputSpace(int requiredSpace)
		{
			if (_nextChunkBuilder.Length + requiredSpace > ChunkSize)
			{
				Flush();
			}
		}
	}

	/// <summary>
	/// Extension methods for ILogText
	/// </summary>
	public static class LogChunkExtensions
	{
		/// <summary>
		/// Gets the chunk index containing the given offset.
		/// </summary>
		/// <param name="chunks">The chunks to search</param>
		/// <param name="offset">The offset to search for</param>
		/// <returns>The chunk index containing the given offset</returns>
		public static int GetChunkForOffset(this IReadOnlyList<LogChunkRef> chunks, long offset)
		{
			int chunkIndex = chunks.BinarySearch(x => x.Offset, offset);
			if (chunkIndex < 0)
			{
				chunkIndex = ~chunkIndex - 1;
			}
			return chunkIndex;
		}

		/// <summary>
		/// Gets the starting chunk index for the given line
		/// </summary>
		/// <param name="chunks">The chunks to search</param>
		/// <param name="lineIndex">Index of the line to query</param>
		/// <returns>Index of the chunk to fetch</returns>
		public static int GetChunkForLine(this IReadOnlyList<LogChunkRef> chunks, int lineIndex)
		{
			int chunkIndex = chunks.BinarySearch(x => x.LineIndex, lineIndex);
			if (chunkIndex < 0)
			{
				chunkIndex = ~chunkIndex - 1;
			}
			return chunkIndex;
		}

		/// <summary>
		/// Converts a log text instance to plain text
		/// </summary>
		/// <param name="logText">The text to convert</param>
		/// <param name="logger">Logger for conversion warnings</param>
		/// <returns>The plain text instance</returns>
		public static LogChunkNode ConvertJsonToPlainText(this LogChunkNode logText, ILogger logger)
		{
			LogChunkBuilder other = new();
			other.AppendJsonAsPlainText(logText, 0, logText.LineCount, logger);
			return other.ToLogChunk();
		}
	}
}
