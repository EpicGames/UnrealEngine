// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json;

#pragma warning disable CA2227

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Severity of a log event
	/// </summary>
	public enum LogEventSeverity
	{
		/// <summary>
		/// Severity is not specified
		/// </summary>
		Unspecified = 0,

		/// <summary>
		/// Information severity
		/// </summary>
		Information = 1,

		/// <summary>
		/// Warning severity
		/// </summary>
		Warning = 2,

		/// <summary>
		/// Error severity
		/// </summary>
		Error = 3,
	}

	/// <summary>
	/// Request to get events for multiple logs at once
	/// </summary>
	public class GetBatchLogEventsRequest
	{
		/// <summary>
		/// Log ids to fetch events for
		/// </summary>
		public List<string> LogIds { get; set; } = [];

		/// <summary>
		/// Maximum number of events to return per log. If null, returns all events.
		/// </summary>
		public int? Count { get; set; }

		/// <summary>
		/// Severity filter. If specified, only events matching this severity are returned.
		/// </summary>
		public LogEventSeverity? Severity { get; set; }
	}

	/// <summary>
	/// Events for a single log within a batch response
	/// </summary>
	public class BatchLogEventsEntry
	{
		/// <summary>
		/// Total number of matching events in this log (before count limit)
		/// </summary>
		public int Total { get; set; }

		/// <summary>
		/// The returned events (may be truncated by count)
		/// </summary>
		public List<GetLogEventResponse> Events { get; set; } = [];
	}

	/// <summary>
	/// Response containing events grouped by log id
	/// </summary>
	public class GetBatchLogEventsResponse
	{
		/// <summary>
		/// Events grouped by log id
		/// </summary>
		public Dictionary<string, BatchLogEventsEntry> LogEvents { get; set; } = [];
	}

	/// <summary>
	/// Information about an uploaded event
	/// </summary>
	public class GetLogEventResponse
	{
		/// <summary>
		/// Unique id of the log containing this event
		/// </summary>
		public LogId LogId { get; set; }

		/// <summary>
		/// Severity of this event
		/// </summary>
		public LogEventSeverity Severity { get; set; }

		/// <summary>
		/// Index of the first line for this event
		/// </summary>
		public int LineIndex { get; set; }

		/// <summary>
		/// Number of lines in the event
		/// </summary>
		public int LineCount { get; set; }

		/// <summary>
		/// The issue id associated with this event
		/// </summary>
		public int? IssueId { get; set; }

		/// <summary>
		/// The structured message data for this event
		/// </summary>
		public List<JsonElement> Lines { get; set; } = [];
	}
}
