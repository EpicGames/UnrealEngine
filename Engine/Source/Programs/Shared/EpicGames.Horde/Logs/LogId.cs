// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Identifier for a log
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<LogId, LogIdConverter>))]
	[BinaryIdConverter(typeof(LogIdConverter))]
	public readonly record struct LogId(BinaryId Id)
	{
		/// <summary>
		/// Constant value for empty user id
		/// </summary>
		public static LogId Empty { get; } = new LogId(default);

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static LogId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc cref="BinaryId.TryParse(System.String, out BinaryId)"/>
		public static bool TryParse(string text, out LogId id)
		{
			if (BinaryId.TryParse(text, out BinaryId objectId))
			{
				id = new LogId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class LogIdConverter : BinaryIdConverter<LogId>
	{
		/// <inheritdoc/>
		public override LogId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(LogId value) => value.Id;
	}
}
