// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using OpenTracing;
using OpenTracing.Util;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Index for a log file.
	/// 
	/// The index consists of a sequence of compressed, plain text blocks (see <see cref="LogChunkRef"/>, and a 
	/// set of (ngram, block index) pairs encoded as 64-bit integers (see <see cref="Logs.NgramSet"/>). 
	/// 
	/// Each ngram is a 1-4 byte sequence of utf8 bytes, padded out to 32-bits (see <see cref="Ngram"/>).
	/// 
	/// When performing a text search, the search term is split into a set of ngrams, and the set queried for blocks containing
	/// them all. Matching blocks are decompressed and scanned for matches using a simplified Knuth-Morris-Pratt search.
	/// 
	/// Since alignment of ngrams may not match alignment of ngrams in the search term, we offset the search term by
	/// 1-4 bytes and include the union of blocks matching at any offset.
	/// </summary>
	[BlobConverter(typeof(LogIndexNodeConverter))]
	public class LogIndexNode
	{
		/// <summary>
		/// Index for tokens into the block list
		/// </summary>
		public NgramSet NgramSet { get; }

		/// <summary>
		/// Number of bits in the index devoted to the block index
		/// </summary>
		public int NumChunkBits { get; }

		/// <summary>
		/// List of text blocks
		/// </summary>
		readonly LogChunkRef[] _plainTextChunkRefs;

		/// <summary>
		/// Empty index data
		/// </summary>
		public static LogIndexNode Empty { get; } = new LogIndexNode(NgramSet.Empty, 0, []);

		/// <summary>
		/// Public accessor for the plain text chunks
		/// </summary>
		public IReadOnlyList<LogChunkRef> PlainTextChunkRefs => _plainTextChunkRefs;

		/// <summary>
		/// Number of lines covered by the index
		/// </summary>
		public int LineCount => (_plainTextChunkRefs.Length > 0) ? (_plainTextChunkRefs[^1].LineIndex + _plainTextChunkRefs[^1].LineCount) : 0;

		/// <summary>
		/// Length of the text data
		/// </summary>
		public long Length => (_plainTextChunkRefs.Length > 0) ? (_plainTextChunkRefs[^1].Offset + _plainTextChunkRefs[^1].Length) : 0;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ngramSet">Index into the text chunks</param>
		/// <param name="numChunkBits">Number of bits devoted to the chunks index</param>
		/// <param name="plainTextChunkRefs">Plain text chunks for this log file</param>
		public LogIndexNode(NgramSet ngramSet, int numChunkBits, LogChunkRef[] plainTextChunkRefs)
		{
			NgramSet = ngramSet;
			NumChunkBits = numChunkBits;
			_plainTextChunkRefs = plainTextChunkRefs;
		}

		/// <summary>
		/// Appends a set of text blocks to this index
		/// </summary>
		/// <param name="writer">Writer for output nodes</param>
		/// <param name="appendPlainTextChunks">Text blocks to append</param>
		/// <param name="cancellationToken"></param>
		/// <returns>New log index with the given blocks appended</returns>
		public async ValueTask<LogIndexNode> AppendAsync(IBlobWriter writer, IReadOnlyList<LogChunkNode> appendPlainTextChunks, CancellationToken cancellationToken)
		{
			using IScope scope = GlobalTracer.Instance.BuildSpan("LogIndex.Append").StartActive();

			// Create the new blocks array
			LogChunkRef[] newChunks = new LogChunkRef[_plainTextChunkRefs.Length + appendPlainTextChunks.Count];
			for (int idx = 0; idx < _plainTextChunkRefs.Length; idx++)
			{
				newChunks[idx] = _plainTextChunkRefs[idx];
			}

			int lineIndex = LineCount;
			long offset = Length;
			for (int idx = 0; idx < appendPlainTextChunks.Count; idx++)
			{
				LogChunkNode newChunk = appendPlainTextChunks[idx];
				IHashedBlobRef<LogChunkNode> newChunkRef = await writer.WriteBlobAsync(newChunk, cancellationToken);
				newChunks[_plainTextChunkRefs.Length + idx] = new LogChunkRef(lineIndex, newChunk.LineCount, offset, newChunk.Length, newChunkRef);
				lineIndex += newChunk.LineCount;
				offset += newChunk.Length;
			}

			// Figure out how many bits to devote to the block size
			int newNumBlockBits = 0;
			while (newChunks.Length > (1 << newNumBlockBits))
			{
				newNumBlockBits += 4;
			}

			// Add the existing index data into a new ngram set
			NgramSetBuilder newNgramSetBuilder = new();

			ulong blockMask = ((1UL << NumChunkBits) - 1);
			foreach (ulong value in NgramSet)
			{
				ulong token = value >> NumChunkBits;
				ulong blockIdx = value & blockMask;

				ulong newToken = (token << newNumBlockBits) | (blockIdx);
				newNgramSetBuilder.Add(newToken);
			}

			// Add the new blocks to the index
			for (int idx = 0; idx < appendPlainTextChunks.Count; idx++)
			{
				LogChunkNode appendTextBlock = appendPlainTextChunks[idx];
				int blockIdx = _plainTextChunkRefs.Length + idx;
				Ngram.Decompose(appendTextBlock.Span, token => newNgramSetBuilder.Add((token << newNumBlockBits) | (uint)blockIdx));
			}

			// Create the new index
			NgramSet newNgramSet = newNgramSetBuilder.ToNgramSet();
			return new LogIndexNode(newNgramSet, newNumBlockBits, newChunks);
		}

		/// <summary>
		/// Search for the given text in the index
		/// </summary>
		/// <param name="firstLineIndex">First line index to search from</param>
		/// <param name="text">Text to search for</param>
		/// <param name="stats">Receives stats for the search</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of line numbers for the text</returns>
		public async IAsyncEnumerable<int> SearchAsync(int firstLineIndex, SearchTerm text, SearchStats stats, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			int lastBlockCount = 0;
			foreach (int blockIdx in EnumeratePossibleChunks(text.Bytes, firstLineIndex))
			{
				LogChunkRef indexChunk = _plainTextChunkRefs[blockIdx];

				stats.NumScannedBlocks++;
				//				stats.NumDecompressedBlocks += (indexChunk.Target == null)? 1 : 0;

				stats.NumSkippedBlocks += blockIdx - lastBlockCount;
				lastBlockCount = blockIdx + 1;

				// Decompress the text
				LogChunkNode chunk = await indexChunk.Target.ReadBlobAsync(cancellationToken);

				// Find the initial offset within this block
				int offset = 0;
				if (firstLineIndex > indexChunk.LineIndex)
				{
					int lineIndexWithinBlock = firstLineIndex - indexChunk.LineIndex;
					offset = chunk.LineOffsets[lineIndexWithinBlock];
				}

				// Search within this block
				for (; ; )
				{
					// Find the next offset
					int nextOffset = chunk.Data.Span.FindNextOcurrence(offset, text);
					if (nextOffset == -1)
					{
						stats.NumScannedBytes += chunk.Data.Length - offset;
						break;
					}

					// Update the stats
					stats.NumScannedBytes += nextOffset - offset;
					offset = nextOffset;

					// Check it's not another match within the same line
					int lineIndexWithinBlock = chunk.GetLineIndexForOffset(offset);
					yield return indexChunk.LineIndex + lineIndexWithinBlock;

					// Move to the next line
					offset = chunk.LineOffsets[lineIndexWithinBlock + 1];
				}

				// If the last scanned bytes is zero, we didn't have any matches from this chunk
				if (offset == 0 && NgramSet != null)
				{
					stats.NumFalsePositiveBlocks++;
				}
			}
			stats.NumSkippedBlocks += _plainTextChunkRefs.Length - lastBlockCount;
		}

		/// <summary>
		/// Search for the given text in the index
		/// </summary>
		/// <param name="text">Text to search for</param>
		/// <param name="lineIndex">The first </param>
		/// <returns>List of line numbers for the text</returns>
		IEnumerable<int> EnumeratePossibleChunks(ReadOnlyMemory<byte> text, int lineIndex)
		{
			// Find the starting chunk index
			int chunkIdx = _plainTextChunkRefs.BinarySearch(x => x.LineIndex, lineIndex);
			if (chunkIdx < 0)
			{
				chunkIdx = Math.Max(~chunkIdx - 1, 0);
			}

			// Make sure we're not starting out of range
			if (chunkIdx < _plainTextChunkRefs.Length)
			{
				// In order to be considered a possible chunk for a positive match, the chunk must contain an exact list of tokens parsed from the
				// search text, along with a set of partial matches (for substrings that are not guaranteed to start on token boundaries). The former
				// is relatively cheap to compute, and done by stepping through all enumerators for each token in a wave, only returning chunks
				// which all contain the token. At the very least, this must contain chunks in the index from chunkIdx onwards.
				List<IEnumerator<int>> enumerators = [Enumerable.Range(chunkIdx, _plainTextChunkRefs.Length - chunkIdx).GetEnumerator()];

				// The second aspect of the search requires a more expensive search through the trie. This is done by running a set of arbitrary 
				// delegates to filter the matches returned from the enumerators.
				List<Predicate<int>> predicates = [];

				// If the index has a trie, tokenize the input and generate a list of enumerators and predicates.
				if (NgramSet != null)
				{
					// Find a list of filters for matching chunks
					HashSet<ulong> tokens = [];
					for (int tokenPos = 0; tokenPos < text.Length;)
					{
						ReadOnlySpan<byte> token = Ngram.GetText(text.Span, tokenPos);
						if (tokenPos == 0)
						{
							GetUnalignedTokenPredicate(token, token.Length == text.Length, predicates);
						}
						else
						{
							GetAlignedTokenPredicate(token, tokenPos + token.Length == text.Length, tokens, predicates);
						}
						tokenPos += token.Length;
					}

					// Create an enumerator for each token
					foreach (ulong token in tokens)
					{
						ulong minValue = (token << NumChunkBits) | (ulong)(long)chunkIdx;
						ulong maxValue = minValue + (1UL << NumChunkBits) - 1;

						IEnumerator<int> enumerator = NgramSet.EnumerateRange(minValue, maxValue).Select(x => (int)(x - minValue)).GetEnumerator();
						if (!enumerator.MoveNext())
						{
							yield break;
						}

						enumerators.Add(enumerator);
					}
				}

				// Enumerate the matches
				for (; ; )
				{
					// Advance all the enumerators that are behind the current chunk index. If they are all equal, we have a match.
					bool match = true;
					foreach (IEnumerator<int> enumerator in enumerators)
					{
						while (enumerator.Current < chunkIdx)
						{
							if (!enumerator.MoveNext())
							{
								yield break;
							}
						}

						if (enumerator.Current > chunkIdx)
						{
							chunkIdx = enumerator.Current;
							match = false;
						}
					}

					// Return the match and move to the next chunk
					if (match)
					{
						if (predicates.All(predicate => predicate(chunkIdx)))
						{
							yield return chunkIdx;
						}
						chunkIdx++;
					}
				}
			}
		}

		/// <summary>
		/// Gets predicates for matching a token that starts 
		/// </summary>
		/// <param name="text">The token text</param>
		/// <param name="allowPartialMatch">Whether to allow a partial match of the token</param>
		/// <param name="tokens">Set of aligned tokens that are required</param>
		/// <param name="predicates">List of predicates for the search</param>
		void GetAlignedTokenPredicate(ReadOnlySpan<byte> text, bool allowPartialMatch, HashSet<ulong> tokens, List<Predicate<int>> predicates)
		{
			for (int offset = 0; offset < text.Length; offset += Ngram.MaxBytes)
			{
				ulong token = Ngram.GetWindowedValue(text, offset);
				if (offset + Ngram.MaxBytes > text.Length && allowPartialMatch)
				{
					ulong tokenMask = Ngram.GetWindowedMask(text, offset, true);
					predicates.Add(chunkIdx => ChunkContainsToken(chunkIdx, token, tokenMask));
					break;
				}
				tokens.Add(token);
			}
		}

		/// <summary>
		/// Generates a predicate for matching a token which may or may not start on a regular token boundary
		/// </summary>
		/// <param name="text">The token text</param>
		/// <param name="allowPartialMatch">Whether to allow a partial match of the token</param>
		/// <param name="predicates">List of predicates for the search</param>
		void GetUnalignedTokenPredicate(ReadOnlySpan<byte> text, bool allowPartialMatch, List<Predicate<int>> predicates)
		{
			byte[] textCopy = text.ToArray();

			Lazy<HashSet<int>> lazyChunks = new(() =>
			{
				HashSet<int> union = [];
				for (int shift = 0; shift < Ngram.MaxBytes; shift++)
				{
					HashSet<int> chunks = [.. ChunksContainingNgram(textCopy.AsSpan(), -shift, allowPartialMatch)];
					for (int offset = -shift + Ngram.MaxBytes; offset < textCopy.Length && chunks.Count > 0; offset += Ngram.MaxBytes)
					{
						chunks.IntersectWith(ChunksContainingNgram(textCopy.AsSpan(), offset, allowPartialMatch));
					}
					if (chunks.Count > 0)
					{
						union.UnionWith(chunks);
					}
				}
				return union;
			});

			predicates.Add(chunkIdx => lazyChunks.Value.Contains(chunkIdx));
		}

		/// <summary>
		/// Tests whether a chunk contains a particular token
		/// </summary>
		/// <param name="chunkIdx">Index of the chunk to search</param>
		/// <param name="token">The token to test</param>
		/// <param name="tokenMask">Mask of which bits in the token are valid</param>
		/// <returns>True if the given block contains a token</returns>
		bool ChunkContainsToken(int chunkIdx, ulong token, ulong tokenMask)
		{
			token = (token << NumChunkBits) | (uint)chunkIdx;
			tokenMask = (tokenMask << NumChunkBits) | ((1UL << NumChunkBits) - 1);
			return NgramSet!.EnumerateValues((value, valueMask) => (value & tokenMask) == (token & valueMask)).Any();
		}

		/// <summary>
		/// Tests whether a chunk contains a particular ngram
		/// </summary>
		/// <param name="text">The token to test</param>
		/// <param name="offset">Offset of the window into the token to test</param>
		/// <param name="allowPartialMatch">Whether to allow a partial match of the token</param>
		/// <returns>True if the given block contains a token</returns>
		IEnumerable<int> ChunksContainingNgram(ReadOnlySpan<byte> text, int offset, bool allowPartialMatch)
		{
			ulong token = Ngram.GetWindowedValue(text, offset) << NumChunkBits;
			ulong tokenMask = Ngram.GetWindowedMask(text, offset, allowPartialMatch) << NumChunkBits;
			ulong blockMask = (1UL << NumChunkBits) - 1;
			return NgramSet!.EnumerateValues((value, valueMask) => (value & tokenMask) == (token & valueMask)).Select(x => (int)(x & blockMask)).Distinct();
		}
	}

	class LogIndexNodeConverter : BlobConverter<LogIndexNode>
	{
		/// <summary>
		/// Type for serializing index nodes
		/// </summary>
		public static BlobType BlobType { get; } = new BlobType("{BAE1A00E-474E-FD63-8180-04A8F93401E2}", 1);

		/// <summary>
		/// Version number for serialized data
		/// </summary>
		const int CurrentVersion = 1;

		/// <inheritdoc/>
		public override LogIndexNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			int version = (int)reader.ReadUnsignedVarInt();
			if (version != CurrentVersion)
			{
				throw new InvalidDataException("Incorrect version number for log data");
			}

			NgramSet ngramSet = reader.ReadNgramSet();
			int numChunkBits = (int)reader.ReadUnsignedVarInt();
			LogChunkRef[] plainTextChunkRefs = reader.ReadVariableLengthArray(() => new LogChunkRef(reader));

			return new LogIndexNode(ngramSet, numChunkBits, plainTextChunkRefs);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, LogIndexNode value, BlobSerializerOptions options)
		{
			writer.WriteUnsignedVarInt(CurrentVersion);
			writer.WriteNgramSet(value.NgramSet);
			writer.WriteUnsignedVarInt(value.NumChunkBits);
			writer.WriteVariableLengthArray(value.PlainTextChunkRefs, x => x.Serialize(writer));

			return BlobType;
		}
	}
}
