// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Jobs;

#pragma warning disable CA2227 // Change 'Lines' to be read-only by removing the property setter

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// The type of data stored in this log file
	/// </summary>
	public enum LogType
	{
		/// <summary>
		/// Plain text data
		/// </summary>
		Text,

		/// <summary>
		/// Structured json objects, output as one object per line (without trailing commas)
		/// </summary>
		Json
	}

	/// <summary>
	/// Creates a new log file
	/// </summary>
	public class CreateLogRequest
	{
		/// <summary>
		/// Type of the log file
		/// </summary>
		public LogType Type { get; set; } = LogType.Json;
	}

	/// <summary>
	/// Response from creating a log file
	/// </summary>
	public class CreateLogResponse
	{
		/// <summary>
		/// Identifier for the created log file
		/// </summary>
		public string Id { get; set; } = String.Empty;
	}

	/// <summary>
	/// Response describing a log file
	/// </summary>
	public class GetLogResponse
	{
		/// <summary>
		/// Unique id of the log file
		/// </summary>
		public LogId Id { get; set; }

		/// <summary>
		/// Unique id of the job for this log file
		/// </summary>
		public JobId JobId { get; set; }

		/// <summary>
		/// The lease allowed to write to this log
		/// </summary>
		public LeaseId? LeaseId { get; set; }

		/// <summary>
		/// The session allowed to write to this log
		/// </summary>
		public SessionId? SessionId { get; set; }

		/// <summary>
		/// Type of events stored in this log
		/// </summary>
		public LogType Type { get; set; }

		/// <summary>
		/// Number of lines in the file
		/// </summary>
		public int LineCount { get; set; }
	}

	/// <summary>
	/// Response describing a log file
	/// </summary>
	public class SearchLogResponse
	{
		/// <summary>
		/// List of line numbers containing the search text
		/// </summary>
		public List<int> Lines { get; set; } = [];

		/// <summary>
		/// Stats for the search
		/// </summary>
		public SearchStats? Stats { get; set; }
	}

	/// <summary>
	/// Response when querying for specific lines from a log file
	/// </summary>
	public class LogLinesResponse
	{
		/// <summary>
		/// start index of the lines returned
		/// </summary>
		public int Index { get; set; }

		/// <summary>
		/// Number of lines returned
		/// </summary>
		public int Count { get; set; }

		/// <summary>
		/// Last index of the returned messages
		/// </summary>
		public int MaxLineIndex { get; set; }

		/// <summary>
		/// Type of response, Json or Text
		/// </summary>
		public LogType Format { get; set; }

		/// <summary>
		/// List of lines received
		/// </summary>
		public List<LogLineResponse> Lines { get; set; } = [];
	}

	/// <summary>
	/// Response object for individual lines
	/// </summary>
	public class LogLineResponse
	{
		/// <summary>
		/// Timestamp for log line
		/// </summary>
		public DateTime Time { get; set; }

		/// <summary>
		/// Level of Message (Information, Warning, Error)
		/// </summary>
		public string? Level { get; set; }

		/// <summary>
		/// Message itself
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// Format string for the message
		/// </summary>
		public string? Format { get; set; }

		/// <summary>
		/// User-defined properties for this jobstep.
		/// </summary>
		public Dictionary<string, string>? Properties { get; set; }
	}
}

