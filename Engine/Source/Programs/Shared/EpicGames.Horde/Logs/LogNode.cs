// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Format for the log file
	/// </summary>
	public enum LogFormat
	{
		/// <summary>
		/// Text data
		/// </summary>
		Text = 0,

		/// <summary>
		/// Json data
		/// </summary>
		Json = 1,
	}

	/// <summary>
	/// Represents an entire log
	/// </summary>
	[BlobConverter(typeof(LogNodeConverter))]
	public class LogNode
	{
		/// <summary>
		/// Format for this log file
		/// </summary>
		public LogFormat Format { get; }

		/// <summary>
		/// Total number of lines
		/// </summary>
		public int LineCount { get; }

		/// <summary>
		/// Length of the log file
		/// </summary>
		public long Length { get; }

		/// <summary>
		/// Text blocks for this chunk
		/// </summary>
		public IReadOnlyList<LogChunkRef> TextChunkRefs { get; }

		/// <summary>
		/// Index for this log
		/// </summary>
		public IHashedBlobRef<LogIndexNode> IndexRef { get; }

		/// <summary>
		/// Whether this log is complete
		/// </summary>
		public bool Complete { get; }

		/// <summary>
		/// Deserializing constructor
		/// </summary>
		public LogNode(LogFormat format, int lineCount, long length, IReadOnlyList<LogChunkRef> textChunkRefs, IHashedBlobRef<LogIndexNode> indexRef, bool complete)
		{
			Format = format;
			LineCount = lineCount;
			Length = length;
			TextChunkRefs = [.. textChunkRefs];
			IndexRef = indexRef;
			Complete = complete;
		}
	}

	/// <summary>
	/// Serializer for <see cref="LogNode"/> types
	/// </summary>
	class LogNodeConverter : BlobConverter<LogNode>
	{
		/// <summary>
		/// Type of blob when serialized to storage
		/// </summary>
		public static BlobType BlobType { get; } = new BlobType("{274DF8F7-4B4F-9E87-8C31-D58A33AD25DB}", 1);

		/// <inheritdoc/>
		public override LogNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			LogFormat format = (LogFormat)reader.ReadUInt8();
			int lineCount = (int)reader.ReadUnsignedVarInt();
			long length = (long)reader.ReadUnsignedVarInt();
			IHashedBlobRef<LogIndexNode> indexRef = reader.ReadBlobRef<LogIndexNode>();
			List<LogChunkRef> textChunkRefs = reader.ReadList(() => new LogChunkRef(reader));
			bool complete = reader.ReadBoolean();

			return new LogNode(format, lineCount, length, textChunkRefs, indexRef, complete);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, LogNode value, BlobSerializerOptions options)
		{
			writer.WriteUInt8((byte)value.Format);
			writer.WriteUnsignedVarInt(value.LineCount);
			writer.WriteUnsignedVarInt((ulong)value.Length);
			writer.WriteBlobRef(value.IndexRef);
			writer.WriteList(value.TextChunkRefs, x => x.Serialize(writer));
			writer.WriteBoolean(value.Complete);

			return BlobType;
		}
	}

	/// <summary>
	/// Assists building log files through trees of <see cref="LogNode"/>, <see cref="LogIndexNode"/> and <see cref="LogChunkNode"/> nodes. This
	/// class is designed to be thread safe, and presents a consistent view to readers and writers.
	/// </summary>
	public class LogBuilder
	{
		/// <summary>
		/// Default maximum size for a log text block
		/// </summary>
		public const int DefaultTextBlockLength = 256 * 1024;

		/// <summary>
		/// Default maximum size for an index text block
		/// </summary>
		public const int DefaultIndexBlockLength = 64 * 1024;

		/// <summary>
		/// Number of lines written to the log
		/// </summary>
		public int LineCount => _lineCount;

		int _lineCount;
		readonly LogFormat _format;

		// Data for the log file which has been flushed to disk so far
		LogNode? _root;
		LogIndexNode _index = LogIndexNode.Empty;

		// Json data read but not flushed
		readonly LogChunkSequenceBuilder _textBuilder;
		readonly LogChunkSequenceBuilder _indexTextBuilder;

		// Lock object for access to the fields above
		readonly Lock _lockObject = new();

		// Inner log device; used for messaging encoding errors.
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">Format for data in the log file</param>
		/// <param name="logger"></param>
		public LogBuilder(LogFormat format, ILogger logger)
			: this(format, DefaultTextBlockLength, DefaultIndexBlockLength, logger)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">Format of data in the log file</param>
		/// <param name="maxTextBlockLength">maximum size for a regular text block</param>
		/// <param name="maxIndexBlockLength">Maximum size for an index text block</param>
		/// <param name="logger">Logger for conversion errors</param>
		public LogBuilder(LogFormat format, int maxTextBlockLength, int maxIndexBlockLength, ILogger logger)
		{
			_format = format;
			_textBuilder = new LogChunkSequenceBuilder(maxTextBlockLength);
			_indexTextBuilder = new LogChunkSequenceBuilder(maxIndexBlockLength);
			_logger = logger;
		}

		/// <summary>
		/// Read data from the unflushed log tail
		/// </summary>
		/// <param name="firstLineIdx">The first line to read, from the end of the flushed data</param>
		/// <param name="maxLength"></param>
		/// <returns></returns>
		public (int LineIdx, ReadOnlyMemory<byte> Data) ReadTailData(int firstLineIdx, int maxLength)
		{
			lock (_lockObject)
			{
				// Clamp the first line index to the first available
				int flushedLineCount = _root?.LineCount ?? 0;
				firstLineIdx = Math.Max(firstLineIdx, flushedLineCount);

				// Measure the size of buffer required for the tail data
				int length = 0;
				int lineCount = 0;
				foreach (Utf8String line in _textBuilder.EnumerateLines(firstLineIdx))
				{
					int nextLength = length + line.Length;
					if (length > 0 && nextLength > maxLength)
					{
						break;
					}
					length = nextLength;
					lineCount++;
				}

				// Allocate the buffer
				byte[] buffer = new byte[length];

				// Copy lines into the buffer
				Span<byte> output = buffer.AsSpan();
				foreach (Utf8String line in _textBuilder.EnumerateLines(firstLineIdx).Take(lineCount))
				{
					line.Span.CopyTo(output);
					output = output.Slice(line.Length);
				}
				Debug.Assert(output.Length == 0);

				return (firstLineIdx, buffer);
			}
		}

		/// <summary>
		/// Append JSON data to the end of the log
		/// </summary>
		/// <param name="data">Log data to append</param>
		public void WriteData(ReadOnlyMemory<byte> data)
		{
			lock (_lockObject)
			{
				ReadOnlyMemory<byte> remaining = data;
				for (; ; )
				{
					int newlineIdx = remaining.Span.IndexOf((byte)'\n');
					if (newlineIdx == -1)
					{
						break;
					}

					ReadOnlyMemory<byte> line = remaining.Slice(0, newlineIdx + 1);
					_textBuilder.Append(line.Span);
					_lineCount++;

					if (_format == LogFormat.Json)
					{
						_indexTextBuilder.AppendJsonAsPlainText(line.Span, _logger);
					}
					else
					{
						_indexTextBuilder.Append(line.Span);
					}

					remaining = remaining.Slice(newlineIdx + 1);
				}
			}
		}

		/// <summary>
		/// Flushes the written data to the log
		/// </summary>
		/// <param name="writer">Writer for the output nodes</param>
		/// <param name="complete">Whether the log is complete</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<IHashedBlobRef<LogNode>> FlushAsync(IBlobWriter writer, bool complete, CancellationToken cancellationToken)
		{
			// Capture the new data that needs to be written
			IReadOnlyList<LogChunkNode> writeTextChunks;
			IReadOnlyList<LogChunkNode> writeIndexTextChunks;

			lock (_lockObject)
			{
				_textBuilder.Flush();
				writeTextChunks = [.. _textBuilder.Chunks];

				_indexTextBuilder.Flush();
				writeIndexTextChunks = [.. _indexTextBuilder.Chunks];
			}

			// Flush any complete chunks to storage
			LogIndexNode newIndex = await _index.AppendAsync(writer, writeIndexTextChunks, cancellationToken);
			IHashedBlobRef<LogIndexNode> newIndexRef = await writer.WriteBlobAsync(newIndex, cancellationToken);

			List<LogChunkRef> newJsonChunkRefs = [.. _root?.TextChunkRefs ?? []];
			int lineCount = _root?.LineCount ?? 0;
			long length = _root?.Length ?? 0;
			foreach (LogChunkNode writeTextChunk in writeTextChunks)
			{
				IHashedBlobRef<LogChunkNode> writeTextChunkRef = await writer.WriteBlobAsync(writeTextChunk, cancellationToken);
				newJsonChunkRefs.Add(new LogChunkRef(lineCount, writeTextChunk.LineCount, length, writeTextChunk.Length, writeTextChunkRef));
				lineCount += writeTextChunk.LineCount;
				length += writeTextChunk.Length;
			}

			LogNode newRoot = new(_format, lineCount, length, newJsonChunkRefs, newIndexRef, complete);
			IHashedBlobRef<LogNode> newRootRef = await writer.WriteBlobAsync(newRoot, cancellationToken);

			await writer.FlushAsync(cancellationToken);

			// Update the new state
			lock (_lockObject)
			{
				_root = newRoot;
				_index = newIndex;
				_textBuilder.Remove(writeTextChunks.Count);
				_indexTextBuilder.Remove(writeIndexTextChunks.Count);
			}

			return newRootRef;
		}
	}

	/// <summary>
	/// Extension methods
	/// </summary>
	public static class LogNodeExtensions
	{
		/// <summary>
		/// Reads lines from a line
		/// </summary>
		/// <param name="logNode">Log to read from</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Sequence of line buffers</returns>
		public static async IAsyncEnumerable<ReadOnlyMemory<byte>> ReadLogAsync(this LogNode logNode, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			foreach (LogChunkRef textChunkRef in logNode.TextChunkRefs)
			{
				LogChunkNode textChunk = await textChunkRef.Target.ReadBlobAsync(cancellationToken);
				yield return textChunk.Data;
			}
		}

		/// <summary>
		/// Reads lines from a line
		/// </summary>
		/// <param name="logNode">Log to read from</param>
		/// <param name="index">Zero-based index of the first line to read from</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Sequence of line buffers</returns>
		public static async IAsyncEnumerable<ReadOnlyMemory<byte>> ReadLogLinesAsync(this LogNode logNode, int index, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			foreach (LogChunkRef textChunkRef in logNode.TextChunkRefs)
			{
				int lineIdx = Math.Max(index - textChunkRef.LineIndex, 0);
				if (lineIdx < textChunkRef.LineCount)
				{
					LogChunkNode textChunk = await textChunkRef.Target.ReadBlobAsync(cancellationToken);

					int offset = textChunk.LineOffsets[lineIdx];
					for (; lineIdx < textChunk.LineCount; lineIdx++)
					{
						int nextOffset = textChunk.LineOffsets[lineIdx + 1];
						ReadOnlyMemory<byte> line = textChunk.Data.Slice(offset, nextOffset - offset);
						yield return line;
						offset = nextOffset;
					}
				}
			}
		}
	}
}
