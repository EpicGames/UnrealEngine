// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Helper methods for logger functionality
	/// </summary>
	public static class LoggerUtils
	{
		/// <summary>
		/// Helper method to fan a single message out to a list loggers
		/// </summary>
		/// <param name="loggerAndLevels">Array of tuples containing the logger and log level for the message.</param>
		/// <param name="message">Message template to be logged.</param>
		/// <param name="args">Structured logging arguments to inject in the message template.</param>
		public static void MultiLog((ILogger? logger, LogLevel logLevel)[] loggerAndLevels, string message, params object?[] args)
		{
			foreach ((ILogger? logger, LogLevel level) in loggerAndLevels)
			{
#pragma warning disable CA2254 // Template should be a static 
				logger?.Log(level, message, args);
#pragma warning restore CA2254 // Template should be a static 
			}
		}

		/// <summary>
		/// Helper method to fan a single message out to a list loggers
		/// </summary>
		/// <param name="loggerAndLevels">Array of tuples containing the logger and log level for the message.</param>
		/// <param name="exception">Exception object to include in logging.</param>
		/// <param name="message">Message template to be logged.</param>
		/// <param name="args">Structured logging arguments to inject in the message template.</param>
		public static void MultiLog((ILogger? logger, LogLevel logLevel)[] loggerAndLevels, Exception exception, string message, params object?[] args)
		{
			foreach ((ILogger? logger, LogLevel level) in loggerAndLevels)
			{
#pragma warning disable CA2254 // Template should be a static 
				logger?.Log(level, exception, message, args);
#pragma warning restore CA2254 // Template should be a static 
			}
		}
	}
}
