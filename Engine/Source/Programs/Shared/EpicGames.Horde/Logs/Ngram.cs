// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Functionality for decomposing log text into ngrams.
	/// </summary>
	public static class Ngram
	{
		/// <summary>
		/// Maximum number of bytes in each ngram
		/// </summary>
		public const int MaxBytes = 4;

		/// <summary>
		/// Number of bits in each ngram
		/// </summary>
		public const int MaxBits = MaxBytes * 8;

		/// <summary>
		/// Lookup from input byte to token type
		/// </summary>
		static readonly byte[] s_tokenTypes = GetTokenTypes();

		/// <summary>
		/// Lookup from input byte to token char
		/// </summary>
		static readonly byte[] s_tokenChars = GetTokenChars();

		/// <summary>
		/// Gets a single token
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <returns>The token value</returns>
		public static ulong Read(ReadOnlySpan<byte> text)
		{
			ulong token = 0;
			Decompose(text, x => token = x);
			return token;
		}

		/// <summary>
		/// Decompose a span of text into tokens
		/// </summary>
		/// <param name="text">Text to scan</param>
		/// <param name="addNgram">Receives a set of tokens</param>
		public static void Decompose(ReadOnlySpan<byte> text, Action<ulong> addNgram)
		{
			if (text.Length > 0)
			{
				int type = s_tokenTypes[text[0]];
				int numTokenBits = 8;
				ulong token = s_tokenChars[text[0]];

				for (int textIdx = 1; textIdx < text.Length; textIdx++)
				{
					byte nextChar = s_tokenChars[text[textIdx]];
					int nextType = s_tokenTypes[nextChar];
					if (type != nextType || numTokenBits + 8 > MaxBits)
					{
						addNgram(token << (MaxBits - numTokenBits));
						token = 0;
						numTokenBits = 0;
						type = nextType;
					}
					token = (token << 8) | nextChar;
					numTokenBits += 8;
				}

				addNgram(token << (MaxBits - numTokenBits));
			}
		}

		/// <summary>
		/// Decompose a string to a set of ngrams
		/// </summary>
		/// <param name="text">Text to scan</param>
		/// <returns></returns>
		public static NgramSet Decompose(ReadOnlySpan<byte> text)
		{
			NgramSetBuilder builder = new();
			Decompose(text, builder.Add);
			return builder.ToNgramSet();
		}

		/// <summary>
		/// Gets the length of the first token in the given span
		/// </summary>
		/// <param name="text">The text to search</param>
		/// <param name="pos">Start position for the search</param>
		/// <returns>Length of the first token</returns>
		public static ReadOnlySpan<byte> GetText(ReadOnlySpan<byte> text, int pos)
		{
			int type = s_tokenTypes[text[pos]];
			for (int end = pos + 1; ; end++)
			{
				if (end == text.Length || s_tokenTypes[text[end]] != type)
				{
					return text.Slice(pos, end - pos);
				}
			}
		}

		/// <summary>
		/// Gets the length of the first token in the given span
		/// </summary>
		/// <param name="text">The text to search</param>
		/// <param name="offset">Offset of the window to read from the token</param>
		/// <returns>Length of the first token</returns>
		public static ulong GetWindowedValue(ReadOnlySpan<byte> text, int offset)
		{
			ulong token = 0;
			for (int idx = 0; idx < MaxBytes; idx++)
			{
				token <<= 8;
				if (offset >= 0 && offset < text.Length)
				{
					token |= s_tokenChars[text[offset]];
				}
				offset++;
			}
			return token;
		}

		/// <summary>
		/// Gets the length of the first token in the given span
		/// </summary>
		/// <param name="text">The text to search</param>
		/// <param name="offset">Offset of the window to read from the token</param>
		/// <param name="allowPartialMatch">Whether to allow only matching the start of the string</param>
		/// <returns>Length of the first token</returns>
		public static ulong GetWindowedMask(ReadOnlySpan<byte> text, int offset, bool allowPartialMatch)
		{
			ulong token = 0;
			for (int idx = 0; idx < MaxBytes; idx++)
			{
				token <<= 8;
				if (offset >= 0 && (offset < text.Length || !allowPartialMatch))
				{
					token |= 0xff;
				}
				offset++;
			}
			return token;
		}

		/// <summary>
		/// Build the lookup table for token types
		/// </summary>
		/// <returns>Array whose elements map from an input byte to token type</returns>
		static byte[] GetTokenTypes()
		{
			byte[] charTypes = new byte[256];
			for (int idx = 'a'; idx <= 'z'; idx++)
			{
				charTypes[idx] = 1;
			}
			for (int idx = 'A'; idx <= 'Z'; idx++)
			{
				charTypes[idx] = 1;
			}
			for (int idx = '0'; idx <= '9'; idx++)
			{
				charTypes[idx] = 2;
			}
			charTypes[' '] = 3;
			charTypes['\t'] = 3;
			charTypes['\n'] = 4;
			return charTypes;
		}

		/// <summary>
		/// Build the lookup table for token types
		/// </summary>
		/// <returns>Array whose elements map from an input byte to token type</returns>
		static byte[] GetTokenChars()
		{
			byte[] chars = new byte[256];
			for (int idx = 0; idx < 256; idx++)
			{
				chars[idx] = (byte)idx;
			}
			for (int idx = 'A'; idx <= 'Z'; idx++)
			{
				chars[idx] = (byte)('a' + idx - 'A');
			}
			return chars;
		}
	}
}
