// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Structure used for building compact <see cref="NgramSet"/> instances
	/// </summary>
	public class NgramSetBuilder
	{
		/// <summary>
		/// Node within the trie
		/// </summary>
		class Node
		{
			public Node?[]? _children;
		}

		/// <summary>
		/// The root node
		/// </summary>
		readonly Node _root;

		/// <summary>
		/// Default constructor
		/// </summary>
		public NgramSetBuilder()
		{
			_root = new Node();
		}

		/// <summary>
		/// Adds a value to the trie
		/// </summary>
		/// <param name="value">Value to add</param>
		public void Add(ulong value)
		{
			// Loop through the tree until we've added the item
			Node leaf = _root;
			for (int shift = (sizeof(ulong) * 8) - 4; shift >= 0; shift -= 4)
			{
				int index = (int)(value >> shift) & 15;
				leaf._children ??= new Node[16];
				leaf._children[index] ??= new Node();
				leaf = leaf._children[index]!;
			}
		}

		/// <summary>
		/// Searches for the given item in the trie
		/// </summary>
		/// <param name="value">Value to add</param>
		public bool Contains(ulong value)
		{
			// Loop through the tree until we've added the item
			Node leaf = _root;
			for (int shift = (sizeof(ulong) * 8) - 4; shift >= 0; shift -= 4)
			{
				int index = (int)(value >> shift) & 15;
				if (leaf._children == null)
				{
					return false;
				}
				if (leaf._children[index] == null)
				{
					return false;
				}
				leaf = leaf._children[index]!;
			}
			return true;
		}

		/// <summary>
		/// Creates a <see cref="NgramSet"/> from this data
		/// </summary>
		/// <returns></returns>
		public NgramSet ToNgramSet()
		{
			List<ushort> values = [];

			List<Node> nodes = [_root];

			for (int bits = 0; bits < (sizeof(ulong) * 8); bits += 4)
			{
				List<Node> nextNodes = [];
				foreach (Node node in nodes)
				{
					ushort value = 0;
					if (node._children != null)
					{
						for (int idx = 0; idx < node._children.Length; idx++)
						{
							if (node._children[idx] != null)
							{
								value |= (ushort)(1 << idx);
								nextNodes.Add(node._children[idx]!);
							}
						}
					}
					values.Add(value);
				}
				nodes = nextNodes;
			}

			return new NgramSet([.. values]);
		}
	}
}
