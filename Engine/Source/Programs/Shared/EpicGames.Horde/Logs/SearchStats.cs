// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Stats for a search
	/// </summary>
	public class SearchStats
	{
		/// <summary>
		/// Number of blocks that were scanned
		/// </summary>
		public int NumScannedBlocks { get; set; }

		/// <summary>
		/// Number of bytes that had to be scanned for results
		/// </summary>
		public int NumScannedBytes { get; set; }

		/// <summary>
		/// Number of blocks that were skipped
		/// </summary>
		public int NumSkippedBlocks { get; set; }

		/// <summary>
		/// Number of blocks that had to be decompressed
		/// </summary>
		public int NumDecompressedBlocks { get; set; }

		/// <summary>
		/// Number of blocks that were searched but did not contain the search term
		/// </summary>
		public int NumFalsePositiveBlocks { get; set; }
	}
}
