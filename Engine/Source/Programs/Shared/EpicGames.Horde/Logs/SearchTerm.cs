// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Stores cached information about a utf8 search term 
	/// </summary>
	public class SearchTerm
	{
		/// <summary>
		/// The search text
		/// </summary>
		public string Text { get; }

		/// <summary>
		/// The utf-8 bytes to search for
		/// </summary>
		public ReadOnlyMemory<byte> Bytes { get; }

		/// <summary>
		/// Normalized (lowercase) utf-8 bytes to search for
		/// </summary>
		readonly byte[] _searchBytes;

		/// <summary>
		/// Skip table for comparisons
		/// </summary>
		readonly byte[] _skipTable;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">The text to search for</param>
		public SearchTerm(string text)
		{
			byte[] bytes = Encoding.UTF8.GetBytes(text);
			Bytes = bytes;
			Text = text;

			// Find the byte sequence to search for, in lowercase
			_searchBytes = new byte[bytes.Length];
			for (int idx = 0; idx < _searchBytes.Length; idx++)
			{
				if (bytes[idx] >= 'A' && bytes[idx] <= 'Z')
				{
					_searchBytes[idx] = (byte)('a' + (bytes[idx] - 'A'));
				}
				else
				{
					_searchBytes[idx] = bytes[idx];
				}
			}

			// Build a table indicating how many characters to skip before attempting the next comparison
			_skipTable = new byte[256];
			for (int idx = 0; idx < 256; idx++)
			{
				_skipTable[idx] = (byte)_searchBytes.Length;
			}
			for (int idx = 0; idx < _searchBytes.Length - 1; idx++)
			{
				byte character = _searchBytes[idx];

				byte skipBytes = (byte)(_searchBytes.Length - 1 - idx);
				_skipTable[character] = skipBytes;

				if (character >= 'a' && character <= 'z')
				{
					_skipTable['A' + (character - 'a')] = skipBytes;
				}
			}
		}

		/// <summary>
		/// Find all ocurrences of the text in the given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>Sequence of offsets within the buffer</returns>
		public static IEnumerable<int> FindOcurrences(ReadOnlyMemory<byte> buffer, SearchTerm text)
		{
			for (int offset = 0; ; offset++)
			{
				offset = FindNextOcurrence(buffer.Span, offset, text);
				if (offset == -1)
				{
					break;
				}
				yield return offset;
			}
		}

		/// <summary>
		/// Perform a case insensitive search for the next occurerence of the search term in a given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="offset">Starting offset for the search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>Offset of the next occurence, or -1</returns>
		public static int FindNextOcurrence(ReadOnlySpan<byte> buffer, int offset, SearchTerm text)
		{
			while (offset + text._searchBytes.Length <= buffer.Length)
			{
				if (Matches(buffer, offset, text))
				{
					return offset;
				}
				else
				{
					offset += text._skipTable[buffer[offset + text._searchBytes.Length - 1]];
				}
			}
			return -1;
		}

		/// <summary>
		/// Compare the search term against the given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="offset">Starting offset for the search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>True if the text matches, false otherwise</returns>
		public static bool Matches(ReadOnlySpan<byte> buffer, int offset, SearchTerm text)
		{
			for (int idx = text._searchBytes.Length - 1; idx >= 0; idx--)
			{
				byte character = buffer[offset + idx];
				if (character >= 'A' && character <= 'Z')
				{
					character = (byte)('a' + (character - 'A'));
				}
				if (character != text._searchBytes[idx])
				{
					return false;
				}
			}
			return true;
		}
	}

	/// <summary>
	/// Stores cached information about a utf8 search term 
	/// </summary>
	public static class SearchTextExtensions
	{
		/// <summary>
		/// Find all ocurrences of the text in the given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>Sequence of offsets within the buffer</returns>
		public static IEnumerable<int> FindOcurrences(this ReadOnlyMemory<byte> buffer, SearchTerm text)
		{
			return SearchTerm.FindOcurrences(buffer, text);
		}

		/// <summary>
		/// Perform a case sensitive search for the next occurerence of the search term in a given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="offset">Starting offset for the search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>Offset of the next occurence, or -1</returns>
		public static int FindNextOcurrence(this ReadOnlySpan<byte> buffer, int offset, SearchTerm text)
		{
			return SearchTerm.FindNextOcurrence(buffer, offset, text);
		}

		/// <summary>
		/// Compare the search term against the given buffer
		/// </summary>
		/// <param name="buffer">The buffer to search</param>
		/// <param name="offset">Starting offset for the search</param>
		/// <param name="text">The text to search for</param>
		/// <returns>True if the text matches, false otherwise</returns>
		public static bool Matches(this ReadOnlySpan<byte> buffer, int offset, SearchTerm text)
		{
			return SearchTerm.Matches(buffer, offset, text);
		}
	}
}
