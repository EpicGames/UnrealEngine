// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Utility class to split log events into separate lines and buffer them for writing to the server
	/// </summary>
	public class ServerLogPacketBuilder
	{
		[DebuggerDisplay("{Format}")]
		class FormattedLine
		{
			public string Format { get; set; }
			public Dictionary<string, string> Properties { get; } = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

			public FormattedLine(string format)
			{
				Format = format;
			}
		}

		static readonly string s_messagePropertyName = LogEventPropertyName.Message.ToString();
		static readonly string s_formatPropertyName = LogEventPropertyName.Format.ToString();
		static readonly string s_linePropertyName = LogEventPropertyName.Line.ToString();
		static readonly string s_lineCountPropertyName = LogEventPropertyName.LineCount.ToString();

		static readonly Utf8String s_escapedNewline = new("\\n");

		readonly List<JsonLogEvent> _logEvents = [];
		readonly ArrayBufferWriter<byte> _lineWriter;
		readonly ArrayBufferWriter<byte> _packetWriter;
		int _packetLength;

		/// <summary>
		/// Current packet length
		/// </summary>
		public int PacketLength => _packetLength;

		/// <summary>
		/// Maximum length of an individual line
		/// </summary>
		public int MaxLineLength { get; }

		/// <summary>
		/// Maximum size of a packet
		/// </summary>
		public int MaxPacketLength { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="maxLineLength">Maximum length for an individual line</param>
		/// <param name="maxPacketLength">Maximum length for a packet</param>
		public ServerLogPacketBuilder(int maxLineLength = 64 * 1024, int maxPacketLength = 256 * 1024)
		{
			MaxLineLength = maxLineLength;
			_lineWriter = new ArrayBufferWriter<byte>(maxLineLength);

			MaxPacketLength = maxPacketLength;
			_packetWriter = new ArrayBufferWriter<byte>(maxPacketLength);
		}

		/// <summary>
		/// Creates a packet from the current data
		/// </summary>
		/// <returns>Packet data and number of lines written</returns>
		public (ReadOnlyMemory<byte>, int) CreatePacket()
		{
			_packetWriter.Clear();

			int eventCount = 0;
			for (; eventCount < _logEvents.Count && (eventCount == 0 || _packetWriter.WrittenCount + _logEvents[eventCount].Data.Length + 1 < MaxPacketLength); eventCount++)
			{
				JsonLogEvent jsonLogEvent = _logEvents[eventCount];

				Span<byte> span = _packetWriter.GetSpan(jsonLogEvent.Data.Length + 1);
				jsonLogEvent.Data.Span.CopyTo(span);
				span[jsonLogEvent.Data.Length] = (byte)'\n';
				_packetWriter.Advance(jsonLogEvent.Data.Length + 1);

				_packetLength -= jsonLogEvent.Data.Length + 1;
			}
			_logEvents.RemoveRange(0, eventCount);

			return (_packetWriter.WrittenMemory, eventCount);
		}

		/// <summary>
		/// Writes an event
		/// </summary>
		/// <param name="jsonLogEvent">Event to write</param>
		public int SanitizeAndWriteEvent(JsonLogEvent jsonLogEvent)
		{
			try
			{
				return SanitizeAndWriteEventInternal(jsonLogEvent);
			}
			catch (Exception ex)
			{
				StringBuilder escapedLineBuilder = new();

				ReadOnlySpan<byte> span = jsonLogEvent.Data.Span;
				for (int idx = 0; idx < span.Length; idx++)
				{
					if (span[idx] >= 32 && span[idx] <= 127)
					{
						escapedLineBuilder.Append((char)span[idx]);
					}
					else
					{
						escapedLineBuilder.Append($"\\x{span[idx]:x2}");
					}
				}

				string escapedLine = escapedLineBuilder.ToString();
				KeyValuePair<string, object?>[] properties = [new KeyValuePair<string, object?>("Text", escapedLine)];
				LogEvent newLogEvent = new(DateTime.UtcNow, LogLevel.Error, default, $"Invalid json log event: {escapedLineBuilder}", "Invalid json log event: {Text}", properties, LogException.FromException(ex));
				JsonLogEvent newJsonLogEvent = new(newLogEvent);

				return SanitizeAndWriteEventInternal(newJsonLogEvent);
			}
		}

		int SanitizeAndWriteEventInternal(JsonLogEvent jsonLogEvent)
		{
			ReadOnlySpan<byte> span = jsonLogEvent.Data.Span;
			if (jsonLogEvent.LineCount == 1 && span.IndexOf(s_escapedNewline) != -1)
			{
				JsonObject obj = (JsonObject)JsonNode.Parse(span)!;

				JsonValue? formatValue = obj["format"] as JsonValue;
				if (formatValue != null && formatValue.TryGetValue(out string? format))
				{
					return WriteEventWithFormat(jsonLogEvent, obj, format);
				}

				JsonValue? messageValue = obj["message"] as JsonValue;
				if (messageValue != null && messageValue.TryGetValue(out string? message))
				{
					return WriteEventWithMessage(jsonLogEvent, obj, message);
				}
			}

			WriteEventInternal(jsonLogEvent);
			return 1;
		}

		/// <summary>
		/// Writes an event with a format string, splitting it into multiple lines if necessary
		/// </summary>
		int WriteEventWithFormat(JsonLogEvent jsonLogEvent, JsonObject obj, string format)
		{
			// There is an object containing "common" properties
			IEnumerable<KeyValuePair<string, object?>> propertyValueList = [];

			// Split the format string into lines
			string[] formatLines = format.Split('\n');
			List<FormattedLine> lines = [];

			// Split all the multi-line properties into separate properties
			JsonObject? properties = obj["properties"] as JsonObject;
			if (properties == null)
			{
				lines.AddRange(formatLines.Select(x => new FormattedLine(x)));
			}
			else
			{
				// Get all the current property values
				Dictionary<string, string> propertyValues = new(StringComparer.OrdinalIgnoreCase);
				foreach ((string name, JsonNode? node) in properties)
				{
					string value = String.Empty;
					if (node != null)
					{
						if (node is JsonObject valueObject)
						{
							value = valueObject["$text"]?.ToString() ?? String.Empty;
						}
						else
						{
							value = node.ToString();
						}
					}
					propertyValues[name] = value;
				}

				// Keep splitting properties in the last line
				foreach (string formatLine in formatLines)
				{
					lines.Add(new FormattedLine(formatLine));
					for (; ; )
					{
						FormattedLine line = lines[^1];
						if (!TryGetNextMultiLineProperty(line.Format, propertyValues, out string? prefix, out string? suffix, out string? propertyName, out string[]? propertyLines))
						{
							break;
						}

						properties!.Remove(propertyName);

						StringBuilder builder = new();
						builder.Append(prefix);

						for (int lineNum = 0; ; lineNum++)
						{
							string newPropertyName = $"{propertyName}${lineNum}";
							builder.Append($"{{{newPropertyName}}}");

							line.Properties.Add(newPropertyName, propertyLines[lineNum]);

							if (lineNum + 1 >= propertyLines.Length)
							{
								break;
							}

							line.Format = builder.ToString();
							builder.Clear();

							line = new FormattedLine(String.Empty);
							lines.Add(line);
						}

						builder.Append(suffix);
						line.Format = builder.ToString();
					}
				}

				// Get the enumerable property list for formatting
				propertyValueList = propertyValues.Select(x => new KeyValuePair<string, object?>(x.Key, x.Value));
			}

			// Finally split the format string into multiple lines
			for (int idx = 0; idx < lines.Count; idx++)
			{
				FormattedLine line = lines[idx];
				foreach ((string name, string value) in line.Properties)
				{
					properties![name] = value;
				}

				string message = MessageTemplate.Render(lines[idx].Format, propertyValueList.Concat(line.Properties.Select(x => new KeyValuePair<string, object?>(x.Key, x.Value))));
				WriteSingleEvent(jsonLogEvent, obj, message, lines[idx].Format, idx, lines.Count);

				foreach ((string name, _) in line.Properties)
				{
					properties!.Remove(name);
				}
			}
			return lines.Count;
		}

		static bool TryGetNextMultiLineProperty(string format, Dictionary<string, string> properties, [NotNullWhen(true)] out string? prefix, [NotNullWhen(true)] out string? suffix, [NotNullWhen(true)] out string? propertyName, [NotNullWhen(true)] out string[]? propertyLines)
		{
			int nameStart = -1;
			for (int idx = 0; idx < format.Length; idx++)
			{
				if (format[idx] == '{')
				{
					nameStart = idx + 1;
				}
				else if (format[idx] == '}' && nameStart != -1)
				{
					string name = format.Substring(nameStart, idx - nameStart);
					if (properties.TryGetValue(name, out string? text))
					{
						if (text.Contains('\n', StringComparison.Ordinal))
						{
							prefix = format.Substring(0, nameStart - 1);
							suffix = format.Substring(idx + 1);
							propertyName = name;
							propertyLines = text.Split('\n');
							return true;
						}
					}
				}
			}

			prefix = null;
			suffix = null;
			propertyName = null;
			propertyLines = null;
			return false;
		}

		int WriteEventWithMessage(JsonLogEvent jsonLogEvent, JsonObject obj, string message)
		{
			string[] lines = message.Split('\n');
			for (int idx = 0; idx < lines.Length; idx++)
			{
				WriteSingleEvent(jsonLogEvent, obj, lines[idx], null, idx, lines.Length);
			}
			return lines.Length;
		}

		void WriteSingleEvent(JsonLogEvent jsonLogEvent, JsonObject obj, string message, string? format, int line, int lineCount)
		{
			obj[s_messagePropertyName] = message;
			if (format != null)
			{
				obj[s_formatPropertyName] = format;
			}
			if (lineCount > 1)
			{
				obj[s_linePropertyName] = line;
				obj[s_lineCountPropertyName] = lineCount;
			}

			_lineWriter.Clear();
			using (Utf8JsonWriter jsonWriter = new(_lineWriter))
			{
				obj.WriteTo(jsonWriter);
			}

			JsonLogEvent nextEvent = new(jsonLogEvent.Level, jsonLogEvent.EventId, line, lineCount, _lineWriter.WrittenMemory.ToArray());
			WriteEventInternal(nextEvent);
		}

		void WriteEventInternal(JsonLogEvent jsonLogEvent)
		{
			if (jsonLogEvent.Data.Length > MaxLineLength)
			{
				LogEvent logEvent = LogEvent.Read(jsonLogEvent.Data.Span);

				int maxMessageLength = Math.Max(10, MaxLineLength - 50);

				string message = logEvent.Message;
				if (message.Length > maxMessageLength)
				{
					message = message.Substring(0, maxMessageLength);
				}

				logEvent = new LogEvent(logEvent.Time, logEvent.Frame, logEvent.Level, logEvent.Id, logEvent.LineIndex, logEvent.LineCount, $"{message} [...]", null, null, logEvent.Exception);
				jsonLogEvent = new JsonLogEvent(logEvent);
			}

			_logEvents.Add(jsonLogEvent);
			_packetLength += jsonLogEvent.Data.Length + 1;
		}
	}
}
