// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Google.Protobuf;
using Grpc.Core;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Logs
{
	/// <summary>
	/// Class to handle uploading log data to the server in the background
	/// </summary>
	sealed class ServerLogger : IServerLogger
	{
		const int FlushLength = 1024 * 1024;

		readonly IHordeClient _hordeClient;
		readonly LogId _logId;
		readonly LogLevel _minimumLevel;
		readonly ILogger _internalLogger;
		readonly LogBuilder _builder;
		readonly IStorageNamespace _store;
		readonly IBlobWriter _writer;

		int _bufferLength;
		readonly Stopwatch _timeSinceFlush = Stopwatch.StartNew();
		readonly TimeSpan _flushInterval;
		string? _closedByStackTrace;

		const int MaxErrors = 200;
		const int MaxWarnings = 200;
		int _numErrors;
		int _numWarnings;

		// Tailing task
		readonly Task _tailTask;
		AsyncEvent _tailTaskStop;
		readonly AsyncEvent _newTailDataEvent = new();

		readonly Channel<JsonLogEvent> _dataChannel;
		Task? _dataWriter;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="hordeClient">Horde instance to write to</param>
		/// <param name="logId">The log id to write to</param>
		/// <param name="minimumLevel">Minimum level for output</param>
		/// <param name="internalLogger">Logger for systemic messages</param>
		/// <param name="flushInterval">Interval for time-based persistent flush (default 30s)</param>
		public ServerLogger(IHordeClient hordeClient, LogId logId, LogLevel minimumLevel, ILogger internalLogger, TimeSpan? flushInterval = null)
		{
			_hordeClient = hordeClient;
			_logId = logId;
			_minimumLevel = minimumLevel;
			_internalLogger = internalLogger;
			_flushInterval = flushInterval ?? TimeSpan.FromSeconds(30);
			_builder = new LogBuilder(LogFormat.Json, internalLogger);
			_store = _hordeClient.GetStorageNamespace(logId);
			_writer = _store.CreateBlobWriter($"{logId}");

			_tailTaskStop = new AsyncEvent();
			_tailTask = Task.Run(() => TickTailAsync());

			_dataChannel = Channel.CreateUnbounded<JsonLogEvent>();
			_dataWriter = Task.Run(() => RunDataWriterAsync());
		}

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			JsonLogEvent jsonLogEvent = JsonLogEvent.FromLoggerState(logLevel, eventId, state, exception, formatter);
			WriteFormattedEvent(jsonLogEvent);
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => logLevel >= _minimumLevel;

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
			=> null!;

		private void WriteFormattedEvent(JsonLogEvent jsonLogEvent)
		{
			if (!_dataChannel.Writer.TryWrite(jsonLogEvent))
			{
				_internalLogger.LogWarning(KnownLogEvents.Systemic_Horde, "Channel for sending logs to Horde server is closed. Unable to write {LogEvent}. Closed by {Stacktrace}", jsonLogEvent, _closedByStackTrace);
			}
		}

		/// <summary>
		/// Stops the log writer's background task
		/// </summary>
		/// <returns>Async task</returns>
		public async Task StopAsync()
		{
			if (_dataWriter != null)
			{
				_closedByStackTrace = Environment.StackTrace;
				_dataChannel.Writer.TryComplete();
				await _dataWriter;
				_dataWriter = null;
			}
		}

		/// <summary>
		/// Dispose of this object. Call StopAsync() to stop asynchronously.
		/// </summary>
		public async ValueTask DisposeAsync()
		{
			await StopAsync();

			_internalLogger.LogInformation("Disposing json log task");

			if (_tailTaskStop != null)
			{
				_tailTaskStop.Latch();
				_newTailDataEvent.Latch();

				try
				{
					await _tailTask;
				}
				catch (Exception ex)
				{
					_internalLogger.LogError(ex, "Exception while waiting for tail task to complete (log {LogId})", _logId);
				}
				_tailTaskStop = null!;
			}

			try
			{
				await _writer.DisposeAsync();
			}
			catch (Exception ex)
			{
				_internalLogger.LogError(ex, "Exception while disposing blob writer (log {LogId})", _logId);
			}
		}

		/// <summary>
		/// Upload the log data to the server in the background
		/// </summary>
		/// <returns>Async task</returns>
		async Task RunDataWriterAsync()
		{
			long packetOffset = 0;
			int packetLineIndex = 0;
			int nextLineIndex = 0;

			ServerLogPacketBuilder writer = new();
			List<RpcCreateLogEventRequest> events = [];

			for (; ; )
			{
				events.Clear();

				// Read events from the channel until the packet is full or a 2-second timeout elapses
				nextLineIndex = await ReadEventsIntoBatchAsync(writer, events, nextLineIndex);

				// Upload packet to the server
				if (writer.PacketLength > 0)
				{
					(ReadOnlyMemory<byte> packet, int packetLineCount) = writer.CreatePacket();
					try
					{
						await WriteOutputAsync(packet, false, CancellationToken.None);
						packetOffset += packet.Length;
						packetLineIndex += packetLineCount;
					}
					catch (Exception ex)
					{
						_internalLogger.LogWarning(ex, "Unable to write data to server (log {LogId}, offset {Offset}, length {Length}, lines {StartLine}-{EndLine})", _logId, packetOffset, packet.Length, packetLineIndex, packetLineIndex + packetLineCount);
					}
				}

				// Upload log events to the server
				if (events.Count > 0)
				{
					try
					{
						await WriteEventsAsync(events, CancellationToken.None);
					}
					catch (Exception ex)
					{
						_internalLogger.LogWarning(ex, "Unable to create events");
					}
				}

				// When no packet was produced, wait for more data (flushing periodically if needed)
				if (writer.PacketLength <= 0)
				{
					if (!await WaitForDataWithPeriodicFlushAsync(packetOffset))
					{
						break;
					}
				}
			}
		}

		/// <summary>
		/// Reads events from the data channel into a packet builder, with a 2-second timeout.
		/// Returns the updated next line index.
		/// </summary>
		async Task<int> ReadEventsIntoBatchAsync(ServerLogPacketBuilder writer, List<RpcCreateLogEventRequest> events, int nextLineIndex)
		{
			using CancellationTokenSource timeoutCts = new(TimeSpan.FromSeconds(2.0));
			while (writer.PacketLength < writer.MaxPacketLength)
			{
				if (_dataChannel.Reader.TryRead(out JsonLogEvent jsonLogEvent))
				{
					int lineCount = writer.SanitizeAndWriteEvent(jsonLogEvent);
					ClassifyAndAddEvent(jsonLogEvent, nextLineIndex, lineCount, events);
					nextLineIndex += lineCount;
				}
				else
				{
					// Wait asynchronously only when TryRead fails; avoids unnecessary allocations during bursts
					bool hasMore;
					try
					{
						hasMore = await _dataChannel.Reader.WaitToReadAsync(timeoutCts.Token);
					}
					catch (OperationCanceledException)
					{
						break;
					}
					if (!hasMore)
					{
						break;
					}
				}
			}
			return nextLineIndex;
		}

		/// <summary>
		/// Classifies a log event and adds it to the event list if it is a warning or error
		/// (up to the configured maximum for each severity).
		/// </summary>
		void ClassifyAndAddEvent(JsonLogEvent jsonLogEvent, int lineIndex, int lineCount, List<RpcCreateLogEventRequest> events)
		{
			if (jsonLogEvent.LineIndex != 0)
			{
				return;
			}

			int effectiveLineCount = Math.Max(lineCount, jsonLogEvent.LineCount);
			if (jsonLogEvent.Level == LogLevel.Warning && ++_numWarnings <= MaxWarnings)
			{
				AddEvent(jsonLogEvent.Data.Span, lineIndex, effectiveLineCount, LogEventSeverity.Warning, events);
			}
			else if (jsonLogEvent.Level is LogLevel.Error or LogLevel.Critical && ++_numErrors <= MaxErrors)
			{
				AddEvent(jsonLogEvent.Data.Span, lineIndex, effectiveLineCount, LogEventSeverity.Error, events);
			}
		}

		/// <summary>
		/// Waits for new data on the channel, periodically flushing buffered output while idle.
		/// Returns true if more data is available, or false if the channel has been closed.
		/// </summary>
		async Task<bool> WaitForDataWithPeriodicFlushAsync(long packetOffset)
		{
			Task<bool> waitForData = _dataChannel.Reader.WaitToReadAsync().AsTask();

			// If there's unflushed data, use a bounded wait loop so we can flush periodically.
			// This reuses the same waitForData task to avoid violating the ChannelReader contract
			// (only one WaitToReadAsync may be outstanding at a time).
			while (_bufferLength > 0)
			{
				double flushDelay = Math.Max(0, _flushInterval.TotalSeconds - _timeSinceFlush.Elapsed.TotalSeconds);
				if (flushDelay > 0)
				{
					Task delayTask = Task.Delay(TimeSpan.FromSeconds(flushDelay));
					await Task.WhenAny(waitForData, delayTask);

					if (waitForData.IsCompleted)
					{
						break;
					}
				}

				// Flush timer expired with no new data; persist what we have
				try
				{
					await WriteOutputAsync(ReadOnlyMemory<byte>.Empty, false, CancellationToken.None);
				}
				catch (Exception ex)
				{
					_internalLogger.LogWarning(ex, "Unable to flush data to server (log {LogId}, offset {Offset})", _logId, packetOffset);
				}
				// On success: _bufferLength is reset to 0 inside WriteOutputAsync, so the while
				// condition becomes false and we fall through to await waitForData below.
				// On failure: _bufferLength stays >0, _timeSinceFlush.Restart() fires in the
				// finally block, so the next iteration waits a full 30 seconds.
			}

			if (!await waitForData)
			{
				// Channel closed by StopAsync; do a final flush
				try
				{
					await WriteOutputAsync(ReadOnlyMemory<byte>.Empty, true, CancellationToken.None);
				}
				catch (Exception ex)
				{
					_internalLogger.LogWarning(ex, "Unable to flush data to server (log {LogId}, offset {Offset})", _logId, packetOffset);
				}
				return false;
			}
			return true;
		}

		void AddEvent(ReadOnlySpan<byte> span, int lineIndex, int lineCount, LogEventSeverity severity, List<RpcCreateLogEventRequest> events)
		{
			try
			{
				events.Add(new RpcCreateLogEventRequest { Severity = (int)severity, LogId = _logId.ToString(), LineIndex = lineIndex, LineCount = lineCount });
			}
			catch (Exception ex)
			{
				_internalLogger.LogError(ex, "Exception while trying to parse line count from data ({Message})", Encoding.UTF8.GetString(span));
			}
		}

		async Task TickTailAsync()
		{
			for (; ; )
			{
				try
				{
					await TickTailInternalAsync();
					break;
				}
				catch (OperationCanceledException ex)
				{
					_internalLogger.LogInformation(ex, "Cancelled log tailing task");
					break;
				}
				catch (Exception ex)
				{
					_internalLogger.LogError(ex, "Exception on log tailing task ({LogId}): {Message}", _logId, ex.Message);
					await Task.Delay(TimeSpan.FromSeconds(10.0));
				}
			}
		}

		async Task TickTailInternalAsync()
		{
			int tailNext = -1;
			Task tickTask = Task.CompletedTask;
			while (!_tailTaskStop.IsSet())
			{
				Task newTailDataTask = _newTailDataEvent.Task;
				int initialTailNext = tailNext;

				// Get the data to send to the server
				ReadOnlyMemory<byte> tailData = ReadOnlyMemory<byte>.Empty;
				if (tailNext != -1)
				{
					(tailNext, tailData) = _builder.ReadTailData(tailNext, 16 * 1024);
				}

				// If we don't have any updates for the server, wait until we do. We need to ensure
				// we keep pumping the RPC with the server in case the requested tail next value changes,
				// and to make sure that we don't expire the existing tail data.
				if (tailNext != -1 && tailData.IsEmpty && tailNext == initialTailNext && !tickTask.IsCompleted)
				{
					_internalLogger.LogInformation("No tail data available for log {LogId} after line {TailNext}; waiting for more...", _logId, tailNext);
					await Task.WhenAny(newTailDataTask, tickTask);
					continue;
				}

				string start = "";
				if (tailData.Length > 0)
				{
					start = Encoding.UTF8.GetString(tailData.Slice(0, Math.Min(tailData.Length, 256)).Span);
				}

				// Update the next tailing position
				int numLines = CountLines(tailData.Span);
				_internalLogger.LogInformation("Setting log {LogId} tail = {TailNext}, data = {TailDataSize} bytes, {NumLines} lines ('{Start}')", _logId, tailNext, tailData.Length, numLines, start);

				int newTailNext = await UpdateLogTailAsync(tailNext, tailData, CancellationToken.None);
				_internalLogger.LogInformation("Log {LogId} tail next = {TailNext}", _logId, newTailNext);

				if (newTailNext != tailNext)
				{
					tailNext = newTailNext;
					_internalLogger.LogInformation("Modified tail position for log {LogId} to {TailNext}", _logId, tailNext);
				}

				tickTask = Task.Delay(TimeSpan.FromSeconds(10.0));
			}
			_internalLogger.LogInformation("Finishing log tail task");
		}

		static int CountLines(ReadOnlySpan<byte> data)
		{
			int lines = 0;
			for (int idx = 0; idx < data.Length; idx++)
			{
				if (data[idx] == '\n')
				{
					lines++;
				}
			}
			return lines;
		}

		private async Task WriteEventsAsync(List<RpcCreateLogEventRequest> events, CancellationToken cancellationToken)
		{
			LogRpc.LogRpcClient logRpc = await _hordeClient.CreateGrpcClientAsync<LogRpc.LogRpcClient>(cancellationToken);

			RpcCreateLogEventsRequest request = new();
			request.Events.AddRange(events);

			await logRpc.CreateLogEventsAsync(request, cancellationToken: cancellationToken);
		}

		private async Task WriteOutputAsync(ReadOnlyMemory<byte> data, bool flush, CancellationToken cancellationToken)
		{
			_builder.WriteData(data);
			_bufferLength += data.Length;

			if (flush || _bufferLength > FlushLength || (_bufferLength > 0 && _timeSinceFlush.Elapsed > _flushInterval))
			{
				try
				{
					IHashedBlobRef<LogNode> target = await _builder.FlushAsync(_writer, flush, cancellationToken);
					await UpdateLogAsync(target, _builder.LineCount, flush, cancellationToken);
					_bufferLength = 0;
				}
				finally
				{
					_timeSinceFlush.Restart();
				}
			}

			_newTailDataEvent.Set();
		}

		#region RPC calls

		async Task UpdateLogAsync(IHashedBlobRef target, int lineCount, bool complete, CancellationToken cancellationToken)
		{
			_internalLogger.LogInformation("Updating log {LogId} to line {LineCount}, target {Locator}", _logId, lineCount, target.GetLocator());

			RpcUpdateLogRequest request = new();
			request.LogId = _logId.ToString();
			request.LineCount = lineCount;
			request.TargetHash = target.Hash.ToString();
			request.TargetLocator = target.GetLocator().ToString();
			request.Complete = complete;

			LogRpc.LogRpcClient clientRef = await _hordeClient.CreateGrpcClientAsync<LogRpc.LogRpcClient>(cancellationToken);
			await clientRef.UpdateLogAsync(request, cancellationToken: cancellationToken);
		}

		async Task<int> UpdateLogTailAsync(int tailNext, ReadOnlyMemory<byte> tailData, CancellationToken cancellationToken)
		{
			DateTime deadline = DateTime.UtcNow.AddMinutes(2.0);
			try
			{
				LogRpc.LogRpcClient clientRef = await _hordeClient.CreateGrpcClientAsync<LogRpc.LogRpcClient>(cancellationToken);
				using AsyncDuplexStreamingCall<RpcUpdateLogTailRequest, RpcUpdateLogTailResponse> call = clientRef.UpdateLogTail(deadline: deadline, cancellationToken: cancellationToken);

				// Write the request to the server
				RpcUpdateLogTailRequest request = new();
				request.LogId = _logId.ToString();
				request.TailNext = tailNext;
				request.TailData = UnsafeByteOperations.UnsafeWrap(tailData);
				await call.RequestStream.WriteAsync(request, cancellationToken);
				_internalLogger.LogInformation("Writing log data: {LogId}, {TailNext}, {TailData} bytes", _logId, tailNext, tailData.Length);

				// Wait until the server responds or we need to trigger a new update
				Task<bool> moveNextAsync = call.ResponseStream.MoveNext();

				Task task = await Task.WhenAny(moveNextAsync, _tailTaskStop.Task, Task.Delay(TimeSpan.FromMinutes(1.0), CancellationToken.None));
				if (task == _tailTaskStop.Task)
				{
					_internalLogger.LogInformation("Cancelling long poll from client side (complete)");
				}

				// Close the request stream to indicate that we're finished
				await call.RequestStream.CompleteAsync();

				// Drain any remaining responses from the server
				RpcUpdateLogTailResponse? response = null;
				while (await moveNextAsync)
				{
					response = call.ResponseStream.Current;
					moveNextAsync = call.ResponseStream.MoveNext();
				}
				return response?.TailNext ?? -1;
			}
			catch (RpcException ex) when (ex.StatusCode == StatusCode.DeadlineExceeded)
			{
				_internalLogger.LogInformation(ex, "Log tail deadline exceeded, ignoring.");
				return -1;
			}
			catch (Exception ex)
			{
				_internalLogger.LogWarning(ex, "Exception while updating log: {Message}", ex.Message);
				throw;
			}
		}

		#endregion
	}
}
