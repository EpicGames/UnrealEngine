// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Notifications
{
	/// <summary>
	/// Unique id for a notification trigger id
	/// </summary>
	/// <param name="Id">Identifier for the notification trigger</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<NotificationTriggerId, NotificationTriggerIdConverter>))]
	[BinaryIdConverter(typeof(NotificationTriggerIdConverter))]
	public readonly record struct NotificationTriggerId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static NotificationTriggerId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override readonly string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter class to and from ObjectId values
	/// </summary>
	class NotificationTriggerIdConverter : BinaryIdConverter<NotificationTriggerId>
	{
		/// <inheritdoc/>
		public override NotificationTriggerId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(NotificationTriggerId value) => value.Id;
	}
}
