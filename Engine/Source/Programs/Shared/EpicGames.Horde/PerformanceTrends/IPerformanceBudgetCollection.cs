// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.PerformanceTrends
{
	/// <summary>
	/// Collection interface for managing performance budgets.
	/// </summary>
	public interface IPerformanceBudgetCollection
	{
		/// <summary>
		/// Creates a new performance budget.
		/// </summary>
		/// <param name="owner">The owner of the new budget.</param>
		/// <param name="request">The add request containing budget details.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The new budget, if creation was successful.</returns>
		Task<IPerformanceBudget?> AddPerformanceBudgetAsync(UserId? owner, PerformanceBudgetAddRequest request, CancellationToken cancellationToken);

		/// <summary>
		/// Updates an existing performance budget.
		/// </summary>
		/// <param name="budgetId">The id of the budget to update.</param>
		/// <param name="request">The update request containing fields to modify.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The updated budget, if applicable; null if no updates were enacted.</returns>
		/// <remarks>The provided owner must be the budget owner to perform the update.</remarks>
		Task<IPerformanceBudget?> UpdatePerformanceBudgetAsync(PerformanceBudgetId budgetId, PerformanceBudgetUpdateRequest request, CancellationToken cancellationToken);

		/// <summary>
		/// Gets a single performance budget by ID.
		/// </summary>
		/// <param name="budgetId">The id of the budget to retrieve.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The budget, if found.</returns>
		Task<IPerformanceBudget?> GetPerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken);

		/// <summary>
		/// Gets performance budgets matching the specified criteria.
		/// </summary>
		/// <param name="computedStream">The computed stream to filter budgets by (e.g., stream-main (Horde semantics) or "++Stream+Main" (branch semantics)).</param>
		/// <param name="testProject">Optional test project name filter.</param>
		/// <param name="platform">Optional platform filter. Returns budgets that include this platform or have no platform restriction.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>A list of matching budgets.</returns>
		Task<IEnumerable<IPerformanceBudget>> GetPerformanceBudgetsAsync(string computedStream, string? testProject = null, string? platform = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a performance budget.
		/// </summary>
		/// <param name="budgetId">The id of the budget to delete.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>True if the budget was deleted successfully, false otherwise.</returns>
		/// <remarks>Only the owner of an owned budget can delete it. Unowned budgets can be deleted by any user.</remarks>
		Task<bool> DeletePerformanceBudgetAsync(PerformanceBudgetId budgetId, CancellationToken cancellationToken);
	}
}
