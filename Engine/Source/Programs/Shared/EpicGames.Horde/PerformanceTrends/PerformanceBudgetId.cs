// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.PerformanceTrends
{
	/// <summary>
	/// Identifier for a performance budget.
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<PerformanceBudgetId, PerformanceBudgetIdConverter>))]
	[BinaryIdConverter(typeof(PerformanceBudgetIdConverter))]
	public readonly record struct PerformanceBudgetId(BinaryId Id)
	{
		/// <summary>
		/// Constant value for an empty performance budget id.
		/// </summary>
		public static PerformanceBudgetId Empty { get; } = default;

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static PerformanceBudgetId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class PerformanceBudgetIdConverter : BinaryIdConverter<PerformanceBudgetId>
	{
		/// <inheritdoc/>
		public override PerformanceBudgetId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(PerformanceBudgetId value) => value.Id;
	}
}
