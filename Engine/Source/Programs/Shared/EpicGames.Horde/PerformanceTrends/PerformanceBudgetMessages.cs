// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.PerformanceTrends
{
	/// <summary>
	/// Static class that describes performance budget restrictions.
	/// </summary>
	public static class PerformanceBudgetRestrictions
	{
		/// <summary>
		/// The maximum budget name length.
		/// </summary>
		public const int MaxNameLength = 200;

		/// <summary>
		/// The maximum budget description length.
		/// </summary>
		public const int MaxDescriptionLength = 1000;

		/// <summary>
		/// The maximum number of thresholds per budget.
		/// </summary>
		public const int MaxThresholdsCount = 100;
	}

	#region -- DTO for Budget API --

	/// <summary>
	/// DTO for a metric threshold within a budget.
	/// </summary>
	public record MetricThresholdRequest
	{
		/// <summary>
		/// The test type this threshold applies to (e.g., "Perf test", "GPU Perf").
		/// </summary>
		public string TestType { get; set; } = default!;

		/// <summary>
		/// The name of the metric this threshold is for (e.g., "gpuTimeAvg", "frametimeAvg").
		/// </summary>
		public string MetricName { get; set; } = default!;

		/// <summary>
		/// The threshold value for this metric.
		/// </summary>
		public double ThresholdValue { get; set; }

		/// <summary>
		/// Indicates whether a larger value is worse for this metric.
		/// True for metrics like frame time (higher = worse), false for metrics like FPS (lower = worse).
		/// Defaults to true.
		/// </summary>
		public bool LargerIsWorse { get; set; } = true;

		/// <summary>
		/// Returns whether this threshold request is valid.
		/// </summary>
		public bool IsValid() =>
			!String.IsNullOrEmpty(TestType) &&
			!String.IsNullOrEmpty(MetricName) &&
			!Double.IsNaN(ThresholdValue) && !Double.IsInfinity(ThresholdValue);
	}

	/// <summary>
	/// DTO used for requests to add a performance budget.
	/// </summary>
	public record PerformanceBudgetAddRequest
	{
		/// <summary>
		/// The name of this budget group.
		/// </summary>
		public string Name { get; set; } = default!;

		/// <summary>
		/// Optional description of this budget group.
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// The computed stream this budget belongs to (e.g., stream-main or "++Stream+Main").
		/// </summary>
		public string ComputedStream { get; set; } = default!;

		/// <summary>
		/// The test project name this budget applies to.
		/// This corresponds to the "testName" field in the performance trends data.
		/// </summary>
		public string TestProject { get; set; } = default!;

		/// <summary>
		/// The platforms this budget applies to.
		/// Null or empty means applicable to all platforms.
		/// </summary>
#pragma warning disable CA2227 // Collection properties should be read only
		public List<string>? Platforms { get; set; }
#pragma warning restore CA2227 // Collection properties should be read only

		/// <summary>
		/// The metric thresholds for this budget group.
		/// </summary>
#pragma warning disable CA2227 // Collection properties should be read only
		public List<MetricThresholdRequest> Thresholds { get; set; } = [];
#pragma warning restore CA2227 // Collection properties should be read only

		/// <summary>
		/// Returns whether the name is valid.
		/// </summary>
		public bool IsValidName() => !String.IsNullOrEmpty(Name) && Name.Length <= PerformanceBudgetRestrictions.MaxNameLength;

		/// <summary>
		/// Returns whether the description is valid.
		/// </summary>
		public bool IsValidDescription() => Description == null || Description.Length <= PerformanceBudgetRestrictions.MaxDescriptionLength;

		/// <summary>
		/// Returns whether the thresholds are valid.
		/// </summary>
		public bool IsValidThresholds() => Thresholds != null && Thresholds.Count > 0 && Thresholds.Count <= PerformanceBudgetRestrictions.MaxThresholdsCount && Thresholds.All(t => t.IsValid());

		/// <summary>
		/// Returns whether the add request is valid.
		/// </summary>
		public bool IsValidAddRequest() => IsValidName() && IsValidDescription() && IsValidThresholds();
	}

	/// <summary>
	/// DTO used for requests to update a performance budget.
	/// </summary>
	public record PerformanceBudgetUpdateRequest
	{
		/// <summary>
		/// The name of this budget group (optional for update).
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Optional description of this budget group (optional for update).
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// The platforms this budget applies to (optional for update).
		/// Empty list clears platforms (applies to all). Null means no change.
		/// </summary>
#pragma warning disable CA2227 // Collection properties should be read only
		public List<string>? Platforms { get; set; }
#pragma warning restore CA2227 // Collection properties should be read only

		/// <summary>
		/// The metric thresholds for this budget group (optional for update).
		/// Replaces all existing thresholds when provided.
		/// </summary>
#pragma warning disable CA2227 // Collection properties should be read only
		public List<MetricThresholdRequest>? Thresholds { get; set; }
#pragma warning restore CA2227 // Collection properties should be read only

		/// <summary>
		/// Returns whether the name is valid.
		/// </summary>
		public bool IsValidName() => Name == null || Name.Length <= PerformanceBudgetRestrictions.MaxNameLength;

		/// <summary>
		/// Returns whether the description is valid.
		/// </summary>
		public bool IsValidDescription() => Description == null || Description.Length <= PerformanceBudgetRestrictions.MaxDescriptionLength;

		/// <summary>
		/// Returns whether the thresholds are valid.
		/// </summary>
		public bool IsValidThresholds() => Thresholds == null || (Thresholds.Count <= PerformanceBudgetRestrictions.MaxThresholdsCount && Thresholds.All(t => t.IsValid()));

		/// <summary>
		/// Returns whether the update request is valid.
		/// </summary>
		public bool IsValidUpdateRequest() => IsValidName() && IsValidDescription() && IsValidThresholds();

		/// <summary>
		/// Returns whether the update request has any updates.
		/// </summary>
		public bool HasUpdates() => Name != null || Description != null || Platforms != null || Thresholds != null;
	}

	/// <summary>
	/// Response object for a metric threshold.
	/// </summary>
	public class MetricThresholdResponse
	{
		/// <summary>
		/// The test type this threshold applies to.
		/// </summary>
		public string TestType { get; init; } = default!;

		/// <summary>
		/// The name of the metric this threshold is for.
		/// </summary>
		public string MetricName { get; init; } = default!;

		/// <summary>
		/// The threshold value for this metric.
		/// </summary>
		public double ThresholdValue { get; init; }

		/// <summary>
		/// Indicates whether a larger value is worse for this metric.
		/// </summary>
		public bool LargerIsWorse { get; init; }

		/// <summary>
		/// Creates a metric threshold response from a provided threshold.
		/// </summary>
		/// <param name="threshold">The source threshold.</param>
		public MetricThresholdResponse(IMetricThreshold threshold)
		{
			TestType = threshold.TestType;
			MetricName = threshold.MetricName;
			ThresholdValue = threshold.ThresholdValue;
			LargerIsWorse = threshold.LargerIsWorse;
		}
	}

	/// <summary>
	/// Response object for performance budgets.
	/// </summary>
	public class PerformanceBudgetResponse
	{
		/// <summary>
		/// The unique identifier for this budget.
		/// </summary>
		public PerformanceBudgetId Id { get; init; } = default;

		/// <summary>
		/// The name of this budget group.
		/// </summary>
		public string Name { get; init; } = default!;

		/// <summary>
		/// Optional description of this budget group.
		/// </summary>
		public string? Description { get; init; }

		/// <summary>
		/// The owner of the budget.
		/// </summary>
		public GetThinUserInfoResponse? Owner { get; init; } = default;

		/// <summary>
		/// The computed stream this budget belongs to (e.g., stream-main or "++Stream+Main").
		/// </summary>
		public string ComputedStream { get; init; } = default!;

		/// <summary>
		/// The test project name this budget applies to.
		/// </summary>
		public string TestProject { get; init; } = default!;

		/// <summary>
		/// The platforms this budget applies to.
		/// Null or empty means applicable to all platforms.
		/// </summary>
		public List<string>? Platforms { get; init; }

		/// <summary>
		/// The metric thresholds defined in this budget group.
		/// </summary>
		public List<MetricThresholdResponse> Thresholds { get; init; } = [];

		/// <summary>
		/// The last updated time of the budget.
		/// </summary>
		public DateTime UpdateTimeUtc { get; init; }

		/// <summary>
		/// Creates a performance budget response from a provided budget.
		/// </summary>
		/// <param name="budget">The source budget.</param>
		public PerformanceBudgetResponse(IPerformanceBudget budget)
		{
			Id = budget.Id;
			Owner = null;
			Name = budget.Name;
			Description = budget.Description;
			ComputedStream = budget.ComputedStream;
			TestProject = budget.TestProject;
			Platforms = budget.Platforms?.ToList();
			Thresholds = [.. budget.Thresholds.Select(t => new MetricThresholdResponse(t))];
			UpdateTimeUtc = budget.UpdateTimeUtc;
		}

		/// <summary>
		/// Creates a performance budget response from a provided budget and owner.
		/// </summary>
		/// <param name="budget">The source budget.</param>
		/// <param name="owner">The owner of the budget.</param>
		public PerformanceBudgetResponse(IPerformanceBudget budget, GetThinUserInfoResponse owner)
		{
			Id = budget.Id;
			Owner = owner;
			Name = budget.Name;
			Description = budget.Description;
			ComputedStream = budget.ComputedStream;
			TestProject = budget.TestProject;
			Platforms = budget.Platforms?.ToList();
			Thresholds = [.. budget.Thresholds.Select(t => new MetricThresholdResponse(t))];
			UpdateTimeUtc = budget.UpdateTimeUtc;
		}
	}

	#endregion -- DTO for Budget API --
}
