// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Projects
{
	class HttpProjectCollection : IProjectCollection
	{
		[DebuggerDisplay("{Id}")]
		class Project : IProject
		{
			readonly GetProjectResponse _response;
			readonly IReadOnlyList<ProjectStream> _projectStreams;

			public ProjectId Id => _response.Id;
			public string Name => _response.Name;
			public int Order => _response.Order;
			public IReadOnlyList<IProjectStream> Streams => _projectStreams;

			public Project(GetProjectResponse response)
			{
				_response = response;
				_projectStreams = (IReadOnlyList<ProjectStream>?)response.Streams?.ConvertAll(x => new ProjectStream(x)) ?? [];
			}
		}

		class ProjectStream : IProjectStream
		{
			readonly GetProjectStreamResponse _response;

			public StreamId Id => new(_response.Id);
			public string Name => _response.Name;

			public ProjectStream(GetProjectStreamResponse response)
				=> _response = response;
		}

		readonly IHordeClient _hordeClient;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpProjectCollection(IHordeClient hordeClient)
			=> _hordeClient = hordeClient;

		/// <inheritdoc/>
		public async Task<IProject> GetAsync(ProjectId projectId, CancellationToken cancellationToken)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetProjectResponse? response = await hordeHttpClient.GetProjectAsync(projectId, cancellationToken);
			return new Project(response);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IProject>> GetAllAsync(CancellationToken cancellationToken)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			List<GetProjectResponse> responses = await hordeHttpClient.GetProjectsAsync(cancellationToken: cancellationToken);
			return responses.ConvertAll(x => new Project(x));
		}
	}
}
