// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Projects
{
	/// <summary>
	/// Handle to a Horde project
	/// </summary>
	public interface IProject
	{
		/// <summary>
		/// Unique id of the project
		/// </summary>
		ProjectId Id { get; }

		/// <summary>
		/// Name of the project
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Order to display this project on the dashboard
		/// </summary>
		int Order { get; }

		/// <summary>
		/// List of streams that are in this project
		/// </summary>
		IReadOnlyList<IProjectStream> Streams { get; }
	}

	/// <summary>
	/// Describes a stream within a project
	/// </summary>
	public interface IProjectStream
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId Id { get; }

		/// <summary>
		/// The stream name
		/// </summary>
		string Name { get; }
	}
}
