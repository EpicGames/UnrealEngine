// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Projects
{
	/// <summary>
	/// Collection of projects
	/// </summary>
	public interface IProjectCollection
	{
		/// <summary>
		/// Retrieve information about a specific project
		/// </summary>
		/// <param name="projectId">Id of the project to get information about</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested project</returns>
		Task<IProject> GetAsync(ProjectId projectId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Query all the projects
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the projects</returns>
		Task<IReadOnlyList<IProject>> GetAllAsync(CancellationToken cancellationToken = default);
	}
}
