// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Projects
{
	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<ProjectId, ProjectIdConverter>))]
	[StringIdConverter(typeof(ProjectIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<ProjectId, ProjectIdConverter>))]
	public readonly record struct ProjectId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ProjectId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	public class ProjectIdConverter : StringIdConverter<ProjectId>
	{
		/// <inheritdoc/>
		public override ProjectId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(ProjectId value) => value.Id;
	}
}
