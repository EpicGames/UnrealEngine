// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics;

#pragma warning disable CA2227 // Collection properties should be read only

namespace EpicGames.Horde.Projects
{
	/// <summary>
	/// Response describing a project
	/// </summary>
	[DebuggerDisplay("{Id,nq}")]
	public class GetProjectResponse
	{
		/// <summary>
		/// Unique id of the project
		/// </summary>
		public ProjectId Id { get; set; }

		/// <summary>
		/// Name of the project
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Order to display this project on the dashboard
		/// </summary>
		public int Order { get; set; }

		/// <summary>
		/// List of streams that are in this project
		/// </summary>
		public List<GetProjectStreamResponse>? Streams { get; set; }

		/// <summary>
		/// List of stream categories to display
		/// </summary>
		public List<GetProjectCategoryResponse>? Categories { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id of the project</param>
		/// <param name="name">Name of the project</param>
		/// <param name="order">Order to show this project on the dashboard</param>
		public GetProjectResponse(ProjectId id, string name, int order)
		{
			Id = id;
			Name = name;
			Order = order;
		}
	}

	/// <summary>
	/// Information about a stream within a project
	/// </summary>
	[DebuggerDisplay("{Id,nq} ({Name})")]
	public class GetProjectStreamResponse
	{
		/// <summary>
		/// The stream id
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// The stream name
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetProjectStreamResponse(string id, string name)
		{
			Id = id;
			Name = name;
		}
	}

	/// <summary>
	/// Information about a category to display for a stream
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class GetProjectCategoryResponse
	{
		/// <summary>
		/// Heading for this column
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Index of the row to display this category on
		/// </summary>
		public int Row { get; set; }

		/// <summary>
		/// Whether to show this category on the nav menu
		/// </summary>
		public bool ShowOnNavMenu { get; set; }

		/// <summary>
		/// Patterns for stream names to include
		/// </summary>
		public List<string> IncludePatterns { get; init; } = [];

		/// <summary>
		/// Patterns for stream names to exclude
		/// </summary>
		public List<string> ExcludePatterns { get; init; } = [];

		/// <summary>
		/// Streams to include in this category
		/// </summary>
		public List<string> Streams { get; init; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public GetProjectCategoryResponse(string name, int row)
		{
			Name = name;
			Row = row;
		}
	}
}
