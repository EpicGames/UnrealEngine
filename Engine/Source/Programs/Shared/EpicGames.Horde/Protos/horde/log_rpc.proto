// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/empty.proto";
import "horde/log_rpc_messages.proto";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

service LogRpc {

	// Update the current log state
	rpc UpdateLog(RpcUpdateLogRequest) returns (RpcUpdateLogResponse);

	// Long poll for requests to return unflushed log tail data
	rpc UpdateLogTail(stream RpcUpdateLogTailRequest) returns (stream RpcUpdateLogTailResponse);

	// Creates events for a log file, highlighting particular lines of interest
	rpc CreateLogEvents(RpcCreateLogEventsRequest) returns (google.protobuf.Empty);
}
