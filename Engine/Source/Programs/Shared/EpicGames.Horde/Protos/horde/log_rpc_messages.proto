// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

message RpcUpdateLogRequest {

	// The unique log id
	string LogId = 1;
	
	// Hash of the latest flushed node
	string TargetHash = 5;

	// Locator for the latest flushed node
	string TargetLocator = 2;

	// Number of lines that have been flushed
	int32 LineCount = 3;

	// Whether the log is complete
	bool Complete = 4;
}

message RpcUpdateLogResponse {
}

message RpcUpdateLogTailRequest {

	// The unique log id
	string LogId = 1;

	// Starting line index of the new tail data
	int32 TailNext = 2;

	// New tail data to append (from LineCount backwards)
	bytes TailData = 3;
}

message RpcUpdateLogTailResponse {

	// Index of the next requested tail line, or -1 if tailing is not desired.
	int32 TailNext = 1;
}

message RpcCreateLogEventsRequest {

	// List of events to send
	repeated RpcCreateLogEventRequest Events = 1;
}

message RpcCreateLogEventRequest {

	// Severity of this event
	int32 Severity = 2;

	// Unique id of the log containing this event
	string LogId = 4;

	// Index of the first line relating to this event
	int32 LineIndex = 5;

	// Number of lines in this event
	int32 LineCount = 6;
}

