// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Streams;
using EpicGames.Serialization;

namespace EpicGames.Horde.Replicators
{
	/// <summary>
	/// Unique identifier for a replicator across all streams
	/// </summary>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(ReplicatorIdTypeConverter))]
	[JsonConverter(typeof(ReplicatorIdJsonConverter))]
	public readonly record struct ReplicatorId(StreamId StreamId, StreamReplicatorId StreamReplicatorId)
	{
		/// <summary>
		/// Parse a replicator id
		/// </summary>
		public static ReplicatorId Parse(string text)
		{
			ReplicatorId replicatorId;
			if (!TryParse(text, out replicatorId))
			{
				throw new FormatException($"Unable to parse '{text}' as a replicator id");
			}
			return replicatorId;
		}

		/// <summary>
		/// Parse a replicator id
		/// </summary>
		public static bool TryParse(string text, out ReplicatorId replicatorId)
		{
			int colonIdx = text.IndexOf(':', StringComparison.Ordinal);
			if (colonIdx != -1)
			{
				replicatorId = new ReplicatorId(new StreamId(new StringId(text.Substring(0, colonIdx))), new StreamReplicatorId(new StringId(text.Substring(colonIdx + 1))));
				return true;
			}
			else
			{
				replicatorId = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => $"{StreamId}:{StreamReplicatorId}";
	}

	sealed class ReplicatorIdJsonConverter : JsonConverter<ReplicatorId>
	{
		/// <inheritdoc/>
		public override ReplicatorId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			=> ReplicatorId.Parse(reader.GetString() ?? String.Empty);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ReplicatorId value, JsonSerializerOptions options)
			=> writer.WriteStringValue(value.ToString());
	}

	sealed class ReplicatorIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str && ReplicatorId.TryParse(str, out ReplicatorId replicatorId))
			{
				return replicatorId;
			}
			return null;
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			return null;
		}
	}

	/// <summary>
	/// Identifier for a replicator
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<StreamReplicatorId, StreamReplicatorIdConverter>))]
	[StringIdConverter(typeof(StreamReplicatorIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<StreamReplicatorId, StreamReplicatorIdConverter>))]
	public readonly record struct StreamReplicatorId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public StreamReplicatorId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	public class StreamReplicatorIdConverter : StringIdConverter<StreamReplicatorId>
	{
		/// <inheritdoc/>
		public override StreamReplicatorId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(StreamReplicatorId value) => value.Id;
	}
}
