// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Streams;

namespace EpicGames.Horde.Replicators
{
	/// <summary>
	/// Information about a replicator
	/// </summary>
	public class GetReplicatorResponse
	{
		/// <summary>
		/// Identifier for this replicator
		/// </summary>
		public ReplicatorId Id { get; set; }

		/// <summary>
		/// Identifier for the stream
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// Identifier for this replicator within the stream
		/// </summary>
		public StreamReplicatorId StreamReplicatorId { get; set; }

		/// <summary>
		/// Status description
		/// </summary>
		public string Status { get; set; } = String.Empty;

		/// <summary>
		/// Whether to pause replication
		/// </summary>
		public bool? Pause { get; set; }

		/// <summary>
		/// Whether to perform a clean snapshot
		/// </summary>
		public bool? Clean { get; set; }

		/// <summary>
		/// Resets the replication
		/// </summary>
		public bool? Reset { get; set; }

		/// <summary>
		/// Pauses replication after the current change
		/// </summary>
		public bool? SingleStep { get; set; }

		/// <summary>
		/// The last change that was replicated
		/// </summary>
		public int? LastChange { get; set; }

		/// <summary>
		/// Time at which the last change was replicated
		/// </summary>
		public DateTime? LastChangeFinishTime { get; set; }

		/// <summary>
		/// The current change being replicated
		/// </summary>
		public int? CurrentChange { get; set; }

		/// <summary>
		/// Time at which the current change was replicated
		/// </summary>
		public DateTime? CurrentChangeStartTime { get; set; }

		/// <summary>
		/// Size of data currently being replicated
		/// </summary>
		public long? CurrentSize { get; set; }

		/// <summary>
		/// Amount of data copied for the current change
		/// </summary>
		public long? CurrentCopiedSize { get; set; }

		/// <summary>
		/// Last error with replication, if there is one.
		/// </summary>
		public string? CurrentError { get; set; }
	}

	/// <summary>
	/// Information about a replicator
	/// </summary>
	public class UpdateReplicatorRequest
	{
		/// <summary>
		/// Whether to pause replication immediately
		/// </summary>
		public bool? Pause { get; set; }

		/// <summary>
		/// Whether to perform a clean snapshot for the next replicated change
		/// </summary>
		public bool? Clean { get; set; }

		/// <summary>
		/// Discards all replicated changes and starts replication from scratch
		/// </summary>
		public bool? Reset { get; set; }

		/// <summary>
		/// Pauses replication after one change has been replicated
		/// </summary>
		public bool? SingleStep { get; set; }

		/// <summary>
		/// Change that should be replicated. Setting this to a value ahead of the last replicated change will cause changes inbetween to be skipped.
		/// </summary>
		public int? NextChange { get; set; }
	}
}
