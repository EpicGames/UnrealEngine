// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Secrets
{
	class HttpSecretCollection : ISecretCollection
	{
		class Secret : ISecret
		{
			readonly GetSecretResponse _response;

			public SecretId Id => _response.Id;
			public IReadOnlyDictionary<string, string> Data => _response.Data;

			public Secret(GetSecretResponse response)
				=> _response = response;
		}

		class SecretProperty : ISecretProperty
		{
			private readonly GetSecretPropertyResponse _response;

			public SecretId Id => _response.Id;
			public string Name => _response.Name;
			public string Value => _response.Value;

			public SecretProperty(GetSecretPropertyResponse response)
				=> _response = response;
		}

		readonly IHordeClient _hordeClient;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpSecretCollection(IHordeClient hordeClient)
			=> _hordeClient = hordeClient;

		/// <inheritdoc/>
		public async Task<ISecret?> GetAsync(SecretId secretId, CancellationToken cancellationToken)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetSecretResponse response = await hordeHttpClient.GetSecretAsync(secretId, cancellationToken);
			return new Secret(response);
		}

		/// <inheritdoc/>
		public Task<ISecret?> GetAsync(SecretId secretId, object config, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public async Task<ISecretProperty?> ResolveAsync(string value, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetSecretPropertyResponse response = await hordeHttpClient.ResolveSecretAsync(value, cancellationToken);
			return new SecretProperty(response);
		}
	}
}
