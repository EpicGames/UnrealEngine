// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Horde.Secrets
{
	/// <summary>
	/// Information about a secret
	/// </summary>
	public interface ISecret
	{
		/// <summary>
		/// Identifier for the secret
		/// </summary>
		SecretId Id { get; }

		/// <summary>
		/// The secret values
		/// </summary>
		IReadOnlyDictionary<string, string> Data { get; }
	}

	/// <summary>
	/// Information about a single property of a secret
	/// </summary>
	public interface ISecretProperty
	{
		/// <summary>
		/// Identifier for the secret
		/// </summary>
		SecretId Id { get; }

		/// <summary>
		/// The name of the property from the secret
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The value of the property from the secret
		/// </summary>
		string Value { get; }
	}
}
