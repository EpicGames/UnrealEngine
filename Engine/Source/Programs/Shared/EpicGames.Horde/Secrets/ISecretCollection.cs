// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Secrets
{
	/// <summary>
	/// Collection of secrets
	/// </summary>
	public interface ISecretCollection
	{
		/// <summary>
		/// Resolve a secret to concrete values
		/// </summary>
		/// <param name="secretId">Identifier for the secret</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<ISecret?> GetAsync(SecretId secretId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Resolve a secret to concrete values
		/// </summary>
		/// <param name="secretId">Identifier for the secret</param>
		/// <param name="config">A SecretsConfig object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <remarks>The method should only be used when the config cannot be retrieved from global config object, for example when it as not initialized during a config load</remarks>
		Task<ISecret?> GetAsync(SecretId secretId, object config, CancellationToken cancellationToken = default);

		/// <summary>
		/// Converts the string representation of a secret and property to a concrete value
		/// The format of the string is 'horde:secret:secret-id.property-name'
		/// </summary>
		/// <param name="value">A string that contains a secret ID and property name</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<ISecretProperty?> ResolveAsync(string value, CancellationToken cancellationToken = default);
	}
}
