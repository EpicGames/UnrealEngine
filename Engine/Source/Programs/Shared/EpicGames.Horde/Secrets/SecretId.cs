// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Secrets
{
	/// <summary>
	/// Identifier for a secret
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<SecretId, SecretIdConverter>))]
	[StringIdConverter(typeof(SecretIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<SecretId, SecretIdConverter>))]
	public readonly record struct SecretId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public SecretId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class SecretIdConverter : StringIdConverter<SecretId>
	{
		/// <inheritdoc/>
		public override SecretId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(SecretId value) => value.Id;
	}
}
