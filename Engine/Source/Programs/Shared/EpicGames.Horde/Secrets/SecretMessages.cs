// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics;

#pragma warning disable CA2227 // Collection properties should be read only

namespace EpicGames.Horde.Secrets
{
	/// <summary>
	/// Response listing all the secrets available to the current user
	/// </summary>
	public class GetSecretsResponse
	{
		/// <summary>
		/// List of secret ids
		/// </summary>
		public List<SecretId> Ids { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetSecretsResponse(List<SecretId> ids) => Ids = ids;
	}

	/// <summary>
	/// Gets data for a particular secret
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class GetSecretResponse
	{
		/// <summary>
		/// Id of the secret
		/// </summary>
		public SecretId Id { get; }

		/// <summary>
		/// Key value pairs for the secret
		/// </summary>
		public Dictionary<string, string> Data { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetSecretResponse(SecretId id, Dictionary<string, string> data)
		{
			Id = id;
			Data = data;
		}
	}

	/// <summary>
	/// Gets data for a particular secret and property
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class GetSecretPropertyResponse
	{
		/// <summary>
		/// Id of the secret
		/// </summary>
		public SecretId Id { get; }

		/// <summary>
		/// Name of the property in the secret
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Value of the property in the secret
		/// </summary>
		public string Value { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetSecretPropertyResponse(SecretId id, string name, string value)
		{
			Id = id;
			Name = name;
			Value = value;
		}
	}
}
