// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel.DataAnnotations;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.Server.Notices
{
	/// <summary>
	/// Parameters required to create a notice
	/// </summary>
	public class CreateNoticeRequest
	{
		/// <summary>
		/// Start time to display this message
		/// </summary>
		public DateTime? StartTime { get; set; }

		/// <summary>
		/// Finish time to display this message
		/// </summary>
		public DateTime? FinishTime { get; set; }

		/// <summary>
		/// Message to display
		/// </summary>
		[Required]
		public string Message { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CreateNoticeRequest(string message)
		{
			Message = message;
		}
	}

	/// <summary>
	/// Parameters required to update a notice
	/// </summary>
	public class UpdateNoticeRequest
	{
		/// <summary>
		/// The id of the notice to update
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Start time to display this message
		/// </summary>
		public DateTime? StartTime { get; set; }

		/// <summary>
		/// Finish time to display this message
		/// </summary>
		public DateTime? FinishTime { get; set; }

		/// <summary>
		/// Message to display
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public UpdateNoticeRequest(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Notice information
	/// </summary>
	public class GetNoticeResponse
	{
		/// <summary>
		/// The id of the notice to update
		/// </summary>
		public string? Id { get; set; }

		/// <summary>
		/// Start time to display this message
		/// </summary>
		public DateTime? StartTime { get; set; }

		/// <summary>
		/// Finish time to display this message
		/// </summary>
		public DateTime? FinishTime { get; set; }

		/// <summary>
		/// Whether this notice is for scheduled downtime
		/// </summary>
		public bool ScheduledDowntime { get; set; } = false;

		/// <summary>
		/// Whether the notice is currently active
		/// </summary>
		public bool Active { get; set; }

		/// <summary>
		/// Message to display
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// User id who created the notice, otherwise null if a system message
		/// </summary>
		public GetThinUserInfoResponse? CreatedByUser { get; set; }
	}
}