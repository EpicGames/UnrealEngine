// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json.Serialization;

#pragma warning disable CA1056 // Change string to URI

namespace EpicGames.Horde.Server
{
	/// <summary>
	/// Server Info
	/// </summary>
	public class GetServerInfoResponse
	{
		/// <summary>
		/// Current API version number of the server
		/// </summary>
		[JsonConverter(typeof(HordeApiVersionConverter))]
		public HordeApiVersion ApiVersion { get; set; }

		/// <summary>
		/// Server version info
		/// </summary>
		public string ServerVersion { get; set; } = String.Empty;

		/// <summary>
		/// The current agent version string
		/// </summary>
		public string? AgentVersion { get; set; }

		/// <summary>
		/// List of plugins
		/// </summary>
		public ServerPluginInfoResponse[] Plugins { get; set; } = [];

		/// <summary>
		/// The environment that the server is running in (e.g. "Development", "Production")
		/// </summary>
		public string? Environment { get; set; }
	}

	/// <summary>
	/// Gets connection information to the server
	/// </summary>
	public class GetConnectionResponse
	{
		/// <summary>
		/// Public IP address of the remote machine
		/// </summary>
		public string? Ip { get; set; }

		/// <summary>
		/// Public port of the connecting machine
		/// </summary>
		public int Port { get; set; }
	}

	/// <summary>
	/// Gets ports configured for this server
	/// </summary>
	public class GetPortsResponse
	{
		/// <summary>
		/// Port for HTTP communication
		/// </summary>
		public int? Http { get; set; }

		/// <summary>
		/// Port number for HTTPS communication
		/// </summary>
		public int? Https { get; set; }

		/// <summary>
		/// Port number for unencrpyted HTTPS communication
		/// </summary>
		public int? UnencryptedHttp2 { get; set; }
	}

	/// <summary>
	/// Authentication method used for logging users in
	/// </summary>
	public enum AuthMethod
	{
		/// <summary>
		/// No authentication enabled. *Only* for demo and testing purposes.
		/// </summary>
		Anonymous,

		/// <summary>
		/// OpenID Connect authentication, tailored for Okta
		/// </summary>
		Okta,

		/// <summary>
		/// Generic OpenID Connect authentication, recommended for most
		/// </summary>
		OpenIdConnect,

		/// <summary>
		/// Authenticate using username and password credentials stored in Horde
		/// OpenID Connect (OIDC) is first and foremost recommended.
		/// But if you have a small installation (less than ~10 users) or lacking an OIDC provider, this is an option.
		/// </summary>
		Horde,
	}

	/// <summary>
	/// Describes the auth config for this server
	/// </summary>
	public class GetAuthConfigResponse
	{
		/// <summary>
		/// Issuer for tokens from the auth provider
		/// </summary>
		public AuthMethod Method { get; set; }

		/// <summary>
		/// Optional profile name used by OidcToken
		/// </summary>
		public string? ProfileName { get; set; }

		/// <summary>
		/// Issuer for tokens from the auth provider
		/// </summary>
		public string? ServerUrl { get; set; }

		/// <summary>
		/// Client id for the OIDC authority
		/// </summary>
		public string? ClientId { get; set; }

		/// <summary>
		/// Optional redirect url provided to OIDC login for external tools (typically to a local server)
		/// </summary>
		public string[]? LocalRedirectUrls { get; set; }

		/// <summary>
		/// Optional OIDC scopes for authorizing API requests from external tools
		/// </summary>
		public string[]? Scopes { get; set; }
	}

	/// <summary>
	/// Request to validate server configuration with the given files replacing their checked-in counterparts.
	/// </summary>
	public class PreflightConfigRequest
	{
		/// <summary>
		/// Perforce cluster to retrieve from
		/// </summary>
		public string? Cluster { get; set; }

		/// <summary>
		/// Change to test
		/// </summary>
		public int ShelvedChange { get; set; }
	}

	/// <summary>
	/// Response from validating config files
	/// </summary>
	public class PreflightConfigResponse
	{
		/// <summary>
		/// Whether the files were validated successfully
		/// </summary>
		public bool Result { get; set; }

		/// <summary>
		/// Output message from validation
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// Detailed response
		/// </summary>
		public string? Detail { get; set; }

		/// <summary>
		/// Warnings generated during config processing (e.g. diamond include dependencies)
		/// </summary>
		public string[] Warnings { get; set; } = [];
	}

	/// <summary>
	/// Status for a subsystem within Horde
	/// </summary>
	public class ServerStatusSubsystem
	{

		/// <summary>
		/// Name of the subsystem
		/// </summary>
		public string Name { get; init; } = "";

		/// <summary>
		/// List of updates
		/// </summary>
		public ServerStatusUpdate[] Updates { get; set; } = [];
	}

	/// <summary>
	/// Type of status result for a single update
	/// </summary>
	public enum ServerStatusResult
	{
		/// <summary>
		/// Indicates that the health check determined that the subsystem was unhealthy
		/// </summary>
		Unhealthy,

		/// <summary>
		/// Indicates that the health check determined that the component was in a subsystem state
		/// </summary>
		Degraded,

		/// <summary>
		/// Indicates that the health check determined that the subsystem was healthy
		/// </summary>
		Healthy,
	}

	/// <summary>
	/// A single status update
	/// </summary>
	public class ServerStatusUpdate
	{
		/// <summary>
		/// Result of status update
		/// </summary>
		public ServerStatusResult Result { get; set; }

		/// <summary>
		/// Optional message describing the result
		/// </summary>
		public string? Message { get; set; }

		/// <summary>
		/// Time this update was created
		/// </summary>
		public DateTimeOffset UpdatedAt { get; set; }
	}

	/// <summary>
	/// Response from server status controller
	/// </summary>
	public class ServerStatusResponse
	{
		/// <summary>
		/// List of subsystem statuses
		/// </summary>
		public ServerStatusSubsystem[] Statuses { get; set; } = [];
	}

	/// <summary>
	/// Information about a server plugin
	/// </summary>
	/// <param name="Name">Name of the plugin</param>
	/// <param name="Description">Optional description of the plugin</param>
	/// <param name="Loaded">Whether the plugin is loaded</param>
	/// <param name="Version">The version of the plugin assembly</param>
	/// <param name="UserAcls">List of ACL actions the current user has for this plugin</param>
	public record ServerPluginInfoResponse(string Name, string? Description, bool Loaded, string? Version = null, string[]? UserAcls = null);

}

