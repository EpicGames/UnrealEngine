// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.ServiceAccounts
{
	/// <summary>
	/// Identifier for a user account
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<ServiceAccountId, ServiceAccountIdConverter>))]
	[BinaryIdConverter(typeof(ServiceAccountIdConverter))]
	public readonly record struct ServiceAccountId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static ServiceAccountId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class ServiceAccountIdConverter : BinaryIdConverter<ServiceAccountId>
	{
		/// <inheritdoc/>
		public override ServiceAccountId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(ServiceAccountId value) => value.Id;
	}
}
