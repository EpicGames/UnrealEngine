// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Horde.Accounts;

#pragma warning disable CA1054 // URI-like parameters should not be strings

namespace EpicGames.Horde.ServiceAccounts
{
	/// <summary>
	/// Creates a new user account
	/// </summary>
	/// <param name="Name">Name of the account</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record CreateServiceAccountRequest(string Name, string Description, List<AccountClaimMessage> Claims, bool? Enabled = true);

	/// <summary>
	/// Response from the request to create a new user account
	/// </summary>
	/// <param name="Id">The created account id</param>
	/// <param name="SecretToken">Secret used to auth with this account</param>
	public record CreateServiceAccountResponse(ServiceAccountId Id, string SecretToken);

	/// <summary>
	/// Update request for a user account
	/// </summary>
	/// <param name="Name">Name of the account</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="ResetToken">Request that the token get reset</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record UpdateServiceAccountRequest(string? Name = null, string? Description = null, List<AccountClaimMessage>? Claims = null, bool? ResetToken = null, bool? Enabled = null);

	/// <summary>
	/// Response from updating a user account
	/// </summary>
	public record UpdateServiceAccountResponse(string? NewSecretToken = null);

	/// <summary>
	/// Creates a new user account
	/// </summary>
	/// <param name="Id">Id of the account</param>
	/// <param name="Claims">Claims for the user</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Enabled">Whether the account is enabled</param>
	public record GetServiceAccountResponse(ServiceAccountId Id, List<AccountClaimMessage> Claims, string Description, bool Enabled);
}
