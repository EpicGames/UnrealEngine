// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.ObjectStores;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Backends
{
	/// <summary>
	/// Storage backend that utilizes the local filesystem
	/// </summary>
	public sealed class FileStorageBackend : IStorageBackend
	{
		readonly DirectoryReference _rootDir;
		readonly FileObjectStore _objectStore;
		readonly ILogger _logger;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="objectStore">The backing object store</param>
		/// <param name="logger">Logger interface</param>
		public FileStorageBackend(FileObjectStore objectStore, ILogger logger)
		{
			_rootDir = objectStore.BaseDir;
			_objectStore = objectStore;
			_logger = logger;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FileStorageBackend(DirectoryReference rootDir, MemoryMappedFileCache memoryMappedFileCache, ILogger logger)
			: this(new FileObjectStore(rootDir, memoryMappedFileCache), logger)
		{
		}

		#region Blobs

		/// <summary>
		/// Gets the path for storing a file on disk
		/// </summary>
		static ObjectKey GetBlobFile(BlobLocator locator) => new($"{locator}.blob");

		/// <inheritdoc/>
		public Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken)
			=> _objectStore.OpenAsync(GetBlobFile(locator), offset, length, cancellationToken);

		/// <summary>
		/// Maps a file into memory for reading, and returns a handle to it
		/// </summary>
		/// <param name="locator">Path to the file</param>
		/// <param name="offset">Offset of the data to retrieve</param>
		/// <param name="length">Length of the data</param>
		/// <returns>Handle to the data. Must be disposed by the caller.</returns>
		public IReadOnlyMemoryOwner<byte> Read(BlobLocator locator, int offset, int? length)
			=> _objectStore.Read(GetBlobFile(locator), offset, length);

		/// <inheritdoc/>
		public Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
			=> _objectStore.ReadAsync(GetBlobFile(locator), offset, length, cancellationToken);

		/// <inheritdoc/>
		public Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, CancellationToken cancellationToken = default)
			=> _objectStore.WriteAsync(GetBlobFile(locator), stream, cancellationToken);

		/// <inheritdoc/>
		public async Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? prefix = null, CancellationToken cancellationToken = default)
		{
			BlobLocator locator = StorageHelpers.CreateUniqueLocator(prefix);
			await WriteBlobAsync(locator, stream, imports, cancellationToken);
			return locator;
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator path, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator path, IReadOnlyCollection<BlobLocator>? imports = null, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator>? imports = null, string? prefix = null, CancellationToken cancellationToken = default)
			=> default;

		#endregion

		/// <summary>
		/// Reads a ref from a file on disk
		/// </summary>
		public static async ValueTask<BlobLocator> ReadRefAsync(FileReference file)
		{
			string text = await FileReference.ReadAllTextAsync(file);
			return new BlobLocator(text);
		}

		FileReference GetRefFile(RefName name) => FileReference.Combine(_rootDir, name.ToString() + ".ref");

		/// <summary>
		/// Enumerate all the refs in a file store
		/// </summary>
		/// <param name="rootDir">Root directory to search</param>
		public static IEnumerable<RefName> EnumerateRefs(DirectoryReference rootDir)
		{
			foreach (FileReference refFile in DirectoryReference.EnumerateFiles(rootDir, "*.ref", SearchOption.AllDirectories))
			{
				string name = Path.GetFileNameWithoutExtension(refFile.MakeRelativeTo(rootDir)).Replace(Path.DirectorySeparatorChar, '/');
				yield return new RefName(name);
			}
		}

		#region Aliases

		/// <inheritdoc/>
		public Task AddAliasAsync(string name, BlobLocator locator, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("File storage backend does not currently support aliases.");

		/// <inheritdoc/>
		public Task RemoveAliasAsync(string name, BlobLocator locator, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("File storage backend does not currently support aliases.");

		/// <inheritdoc/>
		public Task<BlobAliasLocator[]> FindAliasesAsync(string alias, int? maxResults = null, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("File storage backend does not currently support aliases.");

		#endregion

		#region Refs

		/// <inheritdoc/>
		public void DeleteRef(RefName name)
		{
			FileReference.Delete(GetRefFile(name));
		}

		/// <inheritdoc/>
		public async Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			FileReference file = GetRefFile(name);
			if (!FileReference.Exists(file))
			{
				return null;
			}

			_logger.LogInformation("Reading {File}", file);
			string[] lines = await FileReference.ReadAllLinesAsync(file, cancellationToken);

			IoHash hash = IoHash.Parse(lines[0].Trim());
			BlobLocator locator = new(lines[1].Trim());

			return new HashedBlobRefValue(hash, locator);
		}

		/// <inheritdoc/>
		public async Task WriteRefAsync(RefName name, HashedBlobRefValue value, CancellationToken cancellationToken = default)
		{
			FileReference file = GetRefFile(name);
			DirectoryReference.CreateDirectory(file.Directory);
			_logger.LogInformation("Writing {File}", file);

			for (int attempt = 0; ; attempt++)
			{
				try
				{
					await FileReference.WriteAllTextAsync(file, $"{value.Hash}\n{value.Locator}\n", cancellationToken);
					break;
				}
				catch (IOException ex) when (attempt < 3)
				{
					_logger.LogDebug(ex, "Unable to write to {File}; retrying...", file);
					await Task.Delay(100 * attempt, cancellationToken);
				}
			}
		}

		#endregion

		/// <inheritdoc/>
		public async Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
		{
			foreach (AddAliasRequest addAlias in request.AddAliases)
			{
				await AddAliasAsync(addAlias.Name, addAlias.Target, addAlias.Rank, addAlias.Data, cancellationToken);
			}
			foreach (RemoveAliasRequest removeAlias in request.RemoveAliases)
			{
				await RemoveAliasAsync(removeAlias.Name, removeAlias.Target, cancellationToken);
			}
			foreach (AddRefRequest addRef in request.AddRefs)
			{
				await WriteRefAsync(addRef.RefName, new HashedBlobRefValue(addRef.Hash, addRef.Target), cancellationToken);
			}
			foreach (RemoveRefRequest removeRef in request.RemoveRefs)
			{
				DeleteRef(removeRef.RefName);
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
			=> _objectStore.GetStats(stats);
	}
}