// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Http.Json;
using System.Threading;
using System.Threading.Tasks;
using System.Web;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Polly;

namespace EpicGames.Horde.Storage.Backends
{
	/// <summary>
	/// Storage backend which communicates with the Horde server over HTTP for content
	/// </summary>
	public sealed class HttpStorageBackend : IStorageBackend
	{
		class WriteBlobResponse
		{
			public string Blob { get; set; } = String.Empty;
			public Uri? UploadUrl { get; set; }
			public bool? SupportsRedirects { get; set; }
		}

		/// <summary>
		/// Wraps a stream and owns the <see cref="HttpResponseMessage"/>, disposing it when the stream is disposed.
		/// This ensures the underlying TCP connection is returned to the pool promptly.
		/// </summary>
		sealed class ResponseOwningStream(HttpResponseMessage response, Stream inner) : Stream
		{
			public override bool CanRead => inner.CanRead;
			public override bool CanSeek => inner.CanSeek;
			public override bool CanWrite => inner.CanWrite;
			public override long Length => inner.Length;
			public override long Position { get => inner.Position; set => inner.Position = value; }

			public override int Read(byte[] buffer, int offset, int count) => inner.Read(buffer, offset, count);
			public override Task<int> ReadAsync(byte[] buffer, int offset, int count, CancellationToken cancellationToken) => inner.ReadAsync(buffer, offset, count, cancellationToken);
			public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => inner.ReadAsync(buffer, cancellationToken);
			public override long Seek(long offset, SeekOrigin origin) => inner.Seek(offset, origin);
			public override void SetLength(long value) => inner.SetLength(value);
			public override void Write(byte[] buffer, int offset, int count) => inner.Write(buffer, offset, count);
			public override void Flush() => inner.Flush();
			public override Task FlushAsync(CancellationToken cancellationToken) => inner.FlushAsync(cancellationToken);
			public override Task CopyToAsync(Stream destination, int bufferSize, CancellationToken cancellationToken) => inner.CopyToAsync(destination, bufferSize, cancellationToken);

			protected override void Dispose(bool disposing)
			{
				if (disposing)
				{
					try
					{
						inner.Dispose();
					}
					finally
					{
						response.Dispose();
					}
				}
				base.Dispose(disposing);
			}

			public override async ValueTask DisposeAsync()
			{
				try
				{
					await inner.DisposeAsync();
				}
				finally
				{
					response.Dispose();
				}
				await base.DisposeAsync();
			}
		}

		readonly string _basePath;
		readonly Func<HttpClient> _createClient;
		readonly Func<HttpClient> _createUploadRedirectClient;
		readonly ILogger _logger;
		bool _supportsUploadRedirects = true;

		long _numBytes;
		int _numActive;
		TimeSpan _sequentialReadTime;
		readonly Stopwatch _readTime = new();

		/// <inheritdoc/>
		public bool SupportsRedirects => _supportsUploadRedirects;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpStorageBackend(string basePath, Func<HttpClient> createClient, Func<HttpClient> createUploadRedirectClient, ILogger logger)
		{
			_basePath = basePath.TrimEnd('/');
			_createClient = createClient;
			_createUploadRedirectClient = createUploadRedirectClient;
			_logger = logger;
		}

		#region Blobs

		/// <inheritdoc/>
		public async Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			Stopwatch blobTimer = Stopwatch.StartNew();
			try
			{
				lock (_readTime)
				{
					if (++_numActive == 1)
					{
						_readTime.Start();
					}
				}

				if (offset == 0 && length == null)
				{
					_logger.LogDebug("Reading {Locator}", locator);
				}
				else if (length == null)
				{
					_logger.LogDebug("Reading {Locator} ({Offset}..)", locator, offset);
				}
				else
				{
					_logger.LogDebug("Reading {Locator} ({Offset}+{Length})", locator, offset, length);
				}

				if (length.HasValue && length.Value == 0)
				{
					return new MemoryStream([]);
				}

				HttpClient httpClient = _createClient();
				try
				{
					IAsyncPolicy<HttpResponseMessage> retryPolicy = HordeHttpMessageHandler.CreateDefaultTransientErrorPolicy();

					HttpResponseMessage response = await retryPolicy.ExecuteAsync(async () =>
					{
						using (HttpRequestMessage request = new(HttpMethod.Get, $"{_basePath}/blobs/{locator}"))
						{
							if (offset != 0 || length != null)
							{
								request.Headers.Range = new RangeHeaderValue(offset, (length == null) ? null : (offset + (length - 1)));
							}

							HttpResponseMessage requestResponse = await httpClient.SendAsync(request, cancellationToken);

							return requestResponse;
						}
					});

					await EnsureSuccessAsync(response, cancellationToken);

					Stream stream = await response.Content.ReadAsStreamAsync(cancellationToken);

					try
					{
						Interlocked.Add(ref _numBytes, response.Content.Headers.ContentLength ?? stream.Length);
					}
					catch
					{
					}

					// Wrap the stream so the HttpResponseMessage (and its underlying connection)
					// is disposed when the caller disposes the stream, rather than waiting for GC.
					return new ResponseOwningStream(response, stream);
				}
				catch
				{
					httpClient.Dispose();
					throw;
				}
			}
			finally
			{
				lock (_readTime)
				{
					_sequentialReadTime += blobTimer.Elapsed;
					if (--_numActive == 0)
					{
						_readTime.Stop();
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			using (Stream stream = await OpenBlobAsync(locator, offset, length, cancellationToken))
			{
				byte[] data = await stream.ReadAllBytesAsync(cancellationToken);
				return ReadOnlyMemoryOwner.Create(data);
			}
		}

		/// <inheritdoc/>
		public Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, CancellationToken cancellationToken = default)
			=> WriteBlobAsync(locator, stream, imports, null, cancellationToken);

		/// <inheritdoc/>
		public Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? prefix = null, CancellationToken cancellationToken = default)
			=> WriteBlobAsync(null, stream, imports, prefix, cancellationToken);

		/// <inheritdoc/>
		public async Task<BlobLocator> WriteBlobAsync(BlobLocator? locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? prefix = null, CancellationToken cancellationToken = default)
		{
			if (_supportsUploadRedirects)
			{
				WriteBlobResponse redirectResponse = await SendWriteRequestAsync(locator, null, imports, prefix, cancellationToken);
				if (redirectResponse.UploadUrl != null)
				{
					long streamLength = stream.CanSeek ? stream.Length : -1;

					using HttpClient uploadRedirectClient = _createUploadRedirectClient();
					// HttpClient.Timeout is a wall-clock deadline for the entire operation including retries.
					// The default of 100s is too low for the outer retry policy (6 retries, exponential backoff up to 64s).
					// 240s allows ~5 retry attempts before the deadline fires.
					uploadRedirectClient.Timeout = TimeSpan.FromSeconds(240);
					IAsyncPolicy<HttpResponseMessage> retryPolicy = HordeHttpMessageHandler.CreateDefaultTransientErrorPolicy();

					int attemptNum = 0;
					using HttpResponseMessage uploadResponse = await retryPolicy.ExecuteAsync(() =>
					{
						// Reset stream position before each attempt so retries send the full data
						if (stream.CanSeek)
						{
							stream.Position = 0;
						}
						else if (attemptNum > 0)
						{
							_logger.LogWarning("Retrying upload to {Url} but stream is not seekable; retry will send from current position", redirectResponse.UploadUrl);
						}
						attemptNum++;
						StreamContent streamContent = new(stream);
						return uploadRedirectClient.PutAsync(redirectResponse.UploadUrl, streamContent, cancellationToken);
					});
					if (!uploadResponse.IsSuccessStatusCode)
					{
						string body = await uploadResponse.Content.ReadAsStringAsync(cancellationToken);
						string headersText = String.Join(", ", uploadResponse.Headers.Select(h => $"{h.Key}={String.Join(",", h.Value)}"));
						throw new StorageException($"Unable to upload data ({streamLength} bytes) to redirected URL {redirectResponse.UploadUrl}. Status:{uploadResponse.StatusCode} Reason:{uploadResponse.ReasonPhrase} Response:{body} Headers:{headersText}");
					}

					_logger.LogDebug("Written {Locator} ({Size} bytes, using redirect)", redirectResponse.Blob, streamLength);
					return new BlobLocator(redirectResponse.Blob);
				}
			}

			using StreamContent directStreamContent = new(stream);
			WriteBlobResponse response = await SendWriteRequestAsync(locator, directStreamContent, imports, prefix, cancellationToken);
			_supportsUploadRedirects = response.SupportsRedirects ?? false;
			_logger.LogDebug("Written {Locator} (direct)", response.Blob);
			return new BlobLocator(response.Blob);
		}

		async Task<WriteBlobResponse> SendWriteRequestAsync(BlobLocator? locator, StreamContent? streamContent, IReadOnlyCollection<BlobLocator>? imports = null, string? prefix = null, CancellationToken cancellationToken = default)
		{
			using (HttpClient httpClient = _createClient())
			{
				IAsyncPolicy<HttpResponseMessage> retryPolicy = HordeHttpMessageHandler.CreateDefaultTransientErrorPolicy();
				using HttpResponseMessage response = await retryPolicy.ExecuteAsync(async () =>
				{
					using (HttpRequestMessage request = (locator != null)
					? new HttpRequestMessage(HttpMethod.Put, $"{_basePath}/blobs/{locator.Value}")
					: new HttpRequestMessage(HttpMethod.Post, $"{_basePath}/blobs"))
					{
						using MultipartFormDataContent form = [];
						if (streamContent != null)
						{
							form.Add(streamContent, "file", "filename");
						}
#pragma warning disable CA2000 // Disposed by form
						if (imports != null)
						{
							foreach (BlobLocator import in imports)
							{
								form.Add(new StringContent(import.ToString()), "import");
							}
							if (imports.Count == 0)
							{
								form.Add(new StringContent("true"), "leaf");
							}
						}
						form.Add(new StringContent(prefix ?? String.Empty), "prefix");
#pragma warning restore CA2000

						request.Content = form;

						HttpResponseMessage requestResponse = await httpClient.SendAsync(request, cancellationToken);

						return requestResponse;
					}
				});

				if (!response.IsSuccessStatusCode)
				{
					string responseText = await response.Content.ReadAsStringAsync(cancellationToken);

					throw new StorageException($"{response.RequestMessage?.Method} to {response.RequestMessage?.RequestUri} failed ({response.StatusCode}). Response: {responseText}");
				}

				WriteBlobResponse? data = await response.Content.ReadFromJsonAsync<WriteBlobResponse>(cancellationToken: cancellationToken);
				return data!;
			}
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
		{
			// We don't currently have any need for getting a read redirect explicitly, though ReadAsync() calls may redirect us automatically.
			return default;
		}

		/// <inheritdoc/>
		public async ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
		{
			if (_supportsUploadRedirects)
			{
				WriteBlobResponse redirectResponse = await SendWriteRequestAsync(locator, null, imports, null, cancellationToken);
				if (redirectResponse.UploadUrl != null)
				{
					return redirectResponse.UploadUrl;
				}
			}
			return null;
		}

		/// <inheritdoc/>
		public async ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator>? imports = null, string? prefix = null, CancellationToken cancellationToken = default)
		{
			if (_supportsUploadRedirects)
			{
				WriteBlobResponse redirectResponse = await SendWriteRequestAsync(null, null, imports, prefix, cancellationToken);
				if (redirectResponse.UploadUrl != null)
				{
					return (new BlobLocator(redirectResponse.Blob), redirectResponse.UploadUrl);
				}
			}
			return null;
		}

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public async Task<BlobAliasLocator[]> FindAliasesAsync(string alias, int? maxResults = null, CancellationToken cancellationToken = default)
		{
			_logger.LogDebug("Finding nodes with alias {Alias}", alias);
			using (HttpClient httpClient = _createClient())
			{
				string queryPath = $"{_basePath}/nodes?alias={HttpUtility.UrlEncode(alias.ToString())}";
				if (maxResults != null)
				{
					queryPath += $"&maxResults={maxResults.Value}";
				}

				using (HttpRequestMessage request = new(HttpMethod.Get, queryPath))
				{
					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						await EnsureSuccessAsync(response, cancellationToken);

						FindNodesResponse? message = await response.Content.ReadFromJsonAsync<FindNodesResponse>(cancellationToken: cancellationToken);

						BlobAliasLocator[] aliases = new BlobAliasLocator[message!.Nodes.Count];
						for (int idx = 0; idx < message.Nodes.Count; idx++)
						{
							FindNodeResponse node = message.Nodes[idx];
							aliases[idx] = new BlobAliasLocator(node.Blob, node.Rank, node.Data);
						}

						return aliases;
					}
				}
			}
		}

		#endregion

		#region Refs

		/// <inheritdoc/>
		public async Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			using (HttpClient httpClient = _createClient())
			{
				using (HttpRequestMessage request = new(HttpMethod.Get, $"{_basePath}/refs/{name}"))
				{
					if (cacheTime.IsSet())
					{
						request.Headers.CacheControl = new CacheControlHeaderValue { MaxAge = cacheTime.MaxAge };
					}

					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						if (response.StatusCode == HttpStatusCode.NotFound)
						{
							_logger.LogDebug("Read ref {RefName} -> None", name);
							return null;
						}
						else if (!response.IsSuccessStatusCode)
						{
							_logger.LogError("Unable to read ref {RefName} (status: {StatusCode}, body: {Body})", name, response.StatusCode, await response.Content.ReadAsStringAsync(cancellationToken));
							throw new StorageException($"Unable to read ref '{name}'");
						}
						else
						{
							await EnsureSuccessAsync(response, cancellationToken);

							ReadRefResponse? data = await response.Content.ReadFromJsonAsync<ReadRefResponse>(cancellationToken: cancellationToken);
							_logger.LogDebug("Read ref {RefName} -> {Hash} / {Locator}", name, data!.Hash, data!.Target);

							return new HashedBlobRefValue(data.Hash, data.Target);
						}
					}
				}
			}
		}

		#endregion

		/// <inheritdoc/>
		public async Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
		{
			foreach (AddRefRequest addRef in request.AddRefs)
			{
				_logger.LogDebug("Writing ref {RefName} -> {Hash} / {Locator}", addRef.RefName, addRef.Hash, addRef.Target);
			}
			foreach (RemoveRefRequest removeRef in request.RemoveRefs)
			{
				_logger.LogDebug("Deleting ref {RefName}", removeRef.RefName);
			}

			using (HttpClient httpClient = _createClient())
			{
				IAsyncPolicy<HttpResponseMessage> retryPolicy = HordeHttpMessageHandler.CreateDefaultTransientErrorPolicy();
				using HttpResponseMessage response = await retryPolicy.ExecuteAsync(() => httpClient.PostAsJsonAsync($"{_basePath}", request, cancellationToken: cancellationToken));
				await EnsureSuccessAsync(response, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
			stats.Add("backend.http.wall_time_secs", (long)_readTime.Elapsed.TotalSeconds);
			stats.Add("backend.http.num_bytes", _numBytes);
			if (_readTime.Elapsed > TimeSpan.Zero)
			{
				stats.Add("backend.http.speed_mb_sec", (long)(_numBytes / (1024.0 * 1024.0 * _readTime.Elapsed.TotalSeconds)));
				stats.Add("backend.http.concurrency_ratio", (long)(_sequentialReadTime.TotalSeconds * 100.0 / _readTime.Elapsed.TotalSeconds));
			}
		}

		async Task EnsureSuccessAsync(HttpResponseMessage message, CancellationToken cancellationToken)
		{
			if (!message.IsSuccessStatusCode)
			{
				string response = await message.Content.ReadAsStringAsync(cancellationToken);
				_logger.LogError("Http response {StatusCode} (Content: {Response})", message.StatusCode, response);
				throw new StorageException($"Http response {message.StatusCode} (Content: {response})");
			}
		}
	}

	/// <summary>
	/// Factory for constructing HttpStorageBackend instances
	/// </summary>
	public sealed class HttpStorageBackendFactory
	{
		readonly IHttpClientFactory _httpClientFactory;
		readonly StorageBackendCache _backendCache;
		readonly ILogger<HttpStorageBackend> _backendLogger;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpStorageBackendFactory(IHttpClientFactory httpClientFactory, StorageBackendCache backendCache, ILogger<HttpStorageBackend> backendLogger)
		{
			_httpClientFactory = httpClientFactory;
			_backendCache = backendCache;
			_backendLogger = backendLogger;
		}

		/// <summary>
		/// Creates a new HTTP storage backend
		/// </summary>
		/// <param name="basePath">Base path for all requests</param>
		/// <param name="accessToken">Custom access token to use for requests</param>
		/// <param name="withBackendCache"></param>
		public IStorageBackend CreateBackend(string basePath, string? accessToken = null, bool withBackendCache = true)
		{
			HttpClient CreateClient()
			{
				HttpClient httpClient = _httpClientFactory.CreateClient(HordeHttpClient.StorageHttpClientName);
				if (accessToken != null)
				{
					httpClient.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Bearer", accessToken);
				}
				return httpClient;
			}

			HttpClient CreateUploadRedirectClient() => _httpClientFactory.CreateClient(HordeHttpClient.UploadRedirectHttpClientName);

			IStorageBackend backend = new HttpStorageBackend(basePath, CreateClient, CreateUploadRedirectClient, _backendLogger);
			if (_backendCache != null && withBackendCache)
			{
				backend = _backendCache.CreateWrapper(basePath, backend);
			}

			return backend;
		}

		/// <summary>
		/// Creates a new HTTP storage backend
		/// </summary>
		/// <param name="namespaceId">Namespace to create a client for</param>
		/// <param name="accessToken">Custom access token to use for requests</param>
		/// <param name="withBackendCache">Whether to enable the backend cache, which caches full bundles to disk</param>
		public IStorageBackend CreateBackend(NamespaceId namespaceId, string? accessToken = null, bool withBackendCache = true) => CreateBackend($"api/v1/storage/{namespaceId}", accessToken, withBackendCache);
	}
}
