// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Http.Json;
using System.Net.Mime;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Backends
{
	/// <summary>
	/// Storage backend which communicates with Jupiter over HTTP
	/// </summary>
	public sealed class JupiterStorageBackend : IStorageBackend
	{
		const string CompactBinaryMimeType = "application/x-ue-cb";

		static class HeaderNames
		{
			public const string JupiterIoHash = "X-Jupiter-IoHash";
		}

		// Names of fields in ref objects
		static class RefFieldNames
		{
			public const string Target = "tgt";
		}

		// Names of fields when blobs are encoded as cb objects
		static class ObjectFieldNames
		{
			public const string Data = "data";
			public const string BinaryAttachments = "bin";
			public const string ObjectAttachments = "obj";
		}

		// Extensions for locators to raw data
		static Utf8String BinaryExtension { get; } = new Utf8String(".bin");

		// Extensions for locators that point to cb objects
		static Utf8String ObjectExtension { get; } = new Utf8String(".obj");

		record class WriteBlobResponse(string Identifier);

		readonly NamespaceId _namespaceId;
		readonly string _bucketId;
		readonly Func<HttpClient> _createClient;
		readonly ILogger _logger;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		/// <summary>
		/// Constructor
		/// </summary>
		public JupiterStorageBackend(NamespaceId namespaceId, string bucketId, Func<HttpClient> createClient, ILogger logger)
		{
			_namespaceId = namespaceId;
			_bucketId = bucketId;
			_createClient = createClient;
			_logger = logger;
		}

		static bool TryGetBinaryAttachment(Utf8String path, out CbBinaryAttachment attachment)
		{
			IoHash hash;
			if (path.EndsWith(BinaryExtension) && IoHash.TryParse(path.Substring(0, path.Length - BinaryExtension.Length), out hash))
			{
				attachment = new CbBinaryAttachment(hash);
				return true;
			}
			else
			{
				attachment = default;
				return false;
			}
		}

		static bool TryGetObjectAttachment(Utf8String path, out CbObjectAttachment attachment)
		{
			IoHash hash;
			if (path.EndsWith(ObjectExtension) && IoHash.TryParse(path.Substring(0, path.Length - ObjectExtension.Length), out hash))
			{
				attachment = new CbObjectAttachment(hash);
				return true;
			}
			else
			{
				attachment = default;
				return false;
			}
		}

		#region Blobs

		/// <inheritdoc/>
		public async Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = await ReadBlobDataAsync(locator, offset, length, cancellationToken);
			return new ReadOnlyMemoryStream(data);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = await ReadBlobDataAsync(locator, offset, length, cancellationToken);
			return ReadOnlyMemoryOwner.Create(data);
		}

		async Task<ReadOnlyMemory<byte>> ReadBlobDataAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			if (offset == 0 && length == null)
			{
				_logger.LogDebug("Reading {Locator}", locator);
			}
			else if (length == null)
			{
				_logger.LogDebug("Reading {Locator} ({Offset}..)", locator, offset);
			}
			else
			{
				_logger.LogDebug("Reading {Locator} ({Offset}+{Length})", locator, offset, length);
			}

			ReadOnlyMemory<byte> data = ReadOnlyMemory<byte>.Empty;
			if (!length.HasValue || length.Value > 0)
			{
				if (TryGetBinaryAttachment(locator.Path, out CbBinaryAttachment binaryAttachment))
				{
					byte[] encodedData = await ReadEncodedBlobDataAsync(binaryAttachment.Hash, cancellationToken);
					data = encodedData;
				}
				else if (TryGetObjectAttachment(locator.Path, out CbObjectAttachment objectAttachment))
				{
					byte[] encodedData = await ReadEncodedBlobDataAsync(objectAttachment.Hash, cancellationToken);
					data = UnwrapDataFromCbObject(encodedData);
				}
				else
				{
					throw new InvalidOperationException($"Locator is not a valid form for Jupiter ({locator})");
				}
			}

			if (offset > 0)
			{
				data = data.Slice(offset);
			}
			if (length.HasValue)
			{
				data = data.Slice(0, length.Value);
			}

			return data;
		}

		// Extract the data field from an encoded cb object
		static ReadOnlyMemory<byte> UnwrapDataFromCbObject(ReadOnlyMemory<byte> encodedData)
		{
			CbObject obj = new(encodedData);
			return obj[ObjectFieldNames.Data].AsBinary();
		}

		// Reads a blob and returns the raw encoded CB object
		async Task<byte[]> ReadEncodedBlobDataAsync(IoHash hash, CancellationToken cancellationToken = default)
		{
			using (HttpClient httpClient = _createClient())
			{
				using (HttpRequestMessage request = new(HttpMethod.Get, $"api/v1/blobs/{_namespaceId}/{hash}"))
				{
					HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken);
					await EnsureSuccessAsync(response, cancellationToken);

					byte[] data = await response.Content.ReadAsByteArrayAsync(cancellationToken);
					return data;
				}
			}
		}

		/// <inheritdoc/>
		public async Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, CancellationToken cancellationToken = default)
		{
			BlobLocator result = await WriteBlobAsync(stream, imports, null, cancellationToken);
			if (locator != result)
			{
				throw new InvalidOperationException("Incorrect locator passed to Jupiter backend for upload.");
			}
		}

		static byte[] CreateCbObject(byte[] payload, IReadOnlyCollection<BlobLocator> imports)
		{
			CbWriter writer = new();
			writer.BeginObject();
			writer.WriteBinaryArray(ObjectFieldNames.Data, payload);

			List<CbBinaryAttachment> binaryAttachments = [];
			List<CbObjectAttachment> objectAttachments = [];
			foreach (BlobLocator import in imports)
			{
				if (TryGetBinaryAttachment(import.Path, out CbBinaryAttachment binaryAttachment))
				{
					binaryAttachments.Add(binaryAttachment);
				}
				else if (TryGetObjectAttachment(import.Path, out CbObjectAttachment objectAttachment))
				{
					objectAttachments.Add(objectAttachment);
				}
				else
				{
					throw new InvalidOperationException("Cannot reference objects that are not in Jupiter store.");
				}
			}

			if (binaryAttachments.Count > 0)
			{
				writer.BeginArray(ObjectFieldNames.BinaryAttachments);
				foreach (CbBinaryAttachment binaryAttachment in binaryAttachments)
				{
					writer.WriteBinaryAttachmentValue(binaryAttachment);
				}
				writer.EndArray();
			}

			if (objectAttachments.Count > 0)
			{
				writer.BeginArray(ObjectFieldNames.ObjectAttachments);
				foreach (CbObjectAttachment objectAttachment in objectAttachments)
				{
					writer.WriteObjectAttachmentValue(objectAttachment);
				}
				writer.EndArray();
			}

			writer.EndObject();

			return writer.ToByteArray();
		}

		/// <inheritdoc/>
		public async Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? prefix = null, CancellationToken cancellationToken = default)
		{
			// Read all the input data
			byte[] data = await stream.ReadAllBytesAsync(cancellationToken);

			// Frame it within a CB object if there are any imports
			Utf8String extension = BinaryExtension;
			if (imports != null && imports.Count > 0)
			{
				data = CreateCbObject(data, imports);
				extension = ObjectExtension;
			}

			// Send the message
			using (HttpClient httpClient = _createClient())
			{
				using ByteArrayContent content = new(data);
				content.Headers.ContentType = new MediaTypeHeaderValue(MediaTypeNames.Application.Octet);

				using (HttpRequestMessage request = new(HttpMethod.Post, $"api/v1/blobs/{_namespaceId}"))
				{
					request.Content = content;

					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						if (!response.IsSuccessStatusCode)
						{
							string responseText = await response.Content.ReadAsStringAsync(cancellationToken);
							throw new StorageException($"Upload to {request.RequestUri} failed ({response.StatusCode}). Response: {responseText}");
						}

						WriteBlobResponse responseBody = await response.Content.ReadFromJsonAsync<WriteBlobResponse>(cancellationToken: cancellationToken)
							?? throw new InvalidDataException("Expected non-null response");

						_logger.LogDebug("Written blob {BlobId}", responseBody.Identifier);
						return new BlobLocator(responseBody.Identifier + extension);
					}
				}
			}
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator>? imports = null, string? prefix = null, CancellationToken cancellationToken = default)
			=> default;

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public Task AddAliasAsync(string name, BlobLocator target, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("Aliases are not currently supported for the Jupiter backend.");

		/// <inheritdoc/>
		public Task RemoveAliasAsync(string name, BlobLocator target, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("Aliases are not currently supported for the Jupiter backend.");

		/// <inheritdoc/>
		public Task<BlobAliasLocator[]> FindAliasesAsync(string alias, int? maxResults = null, CancellationToken cancellationToken = default)
			=> throw new NotSupportedException("Aliases are not currently supported for the Jupiter backend.");

		#endregion

		#region Refs

		/// <inheritdoc/>
		public async Task<bool> DeleteRefAsync(RefName name, CancellationToken cancellationToken)
		{
			_logger.LogDebug("Deleting ref {RefName}", name);
			using (HttpClient httpClient = _createClient())
			{
				IoHash refId = IoHash.Compute(name.Text);
				using (HttpRequestMessage request = new(HttpMethod.Delete, $"api/v1/refs/{_namespaceId}/{_bucketId}/{refId}"))
				{
					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						if (response.IsSuccessStatusCode)
						{
							return true;
						}
						if (response.StatusCode == HttpStatusCode.NotFound)
						{
							return false;
						}

						await EnsureSuccessAsync(response, cancellationToken);
						return false;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			using (HttpClient httpClient = _createClient())
			{
				IoHash refId = IoHash.Compute(name.Text.Span);

				using (HttpRequestMessage request = new(HttpMethod.Get, $"api/v1/refs/{_namespaceId}/{_bucketId}/{refId}"))
				{
					if (cacheTime.IsSet())
					{
						request.Headers.CacheControl = new CacheControlHeaderValue { MaxAge = cacheTime.MaxAge };
					}

					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						if (response.StatusCode == HttpStatusCode.NotFound)
						{
							_logger.LogDebug("Read ref {RefName} -> None", name);
							return null;
						}
						else if (!response.IsSuccessStatusCode)
						{
							_logger.LogError("Unable to read ref {RefName} (status: {StatusCode}, body: {Body})", name, response.StatusCode, await response.Content.ReadAsStringAsync(cancellationToken));
							throw new StorageException($"Unable to read ref '{name}'");
						}
						else
						{
							await EnsureSuccessAsync(response, cancellationToken);

							if (!String.Equals(response.Content.Headers.ContentType?.MediaType, CompactBinaryMimeType, StringComparison.Ordinal))
							{
								throw new NotSupportedException("Unknown response type. Expected compact binary.");
							}

							byte[] encodedData = await response.Content.ReadAsByteArrayAsync(cancellationToken);

							CbObject objectData = new(encodedData);
							CbField target = objectData[RefFieldNames.Target];

							IoHash hash;
							BlobLocator locator;

							if (target.IsBinaryAttachment())
							{
								hash = target.AsBinaryAttachment().Hash;
								locator = new BlobLocator(hash.ToUtf8String() + BinaryExtension);
							}
							else if (target.IsObjectAttachment())
							{
								hash = target.AsObjectAttachment().Hash;
								locator = new BlobLocator(hash.ToUtf8String() + ObjectExtension);
							}
							else
							{
								_logger.LogDebug("Ref {RefName} exists but does not contain a target field", name);
								return null;
							}

							_logger.LogDebug("Read ref {RefName} -> {Locator}", name, locator);
							return new HashedBlobRefValue(hash, locator);
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task WriteRefAsync(RefName name, HashedBlobRefValue value, CancellationToken cancellationToken = default)
		{
			_logger.LogDebug("Writing ref {RefName} -> {Locator}", name, value.Locator);
			using (HttpClient httpClient = _createClient())
			{
				IoHash refId = IoHash.Compute(name.Text.Span);
				using (HttpRequestMessage request = new(HttpMethod.Put, $"api/v1/refs/{_namespaceId}/{_bucketId}/{refId}"))
				{
					CbWriter writer = new();
					writer.BeginObject();

					if (TryGetBinaryAttachment(value.Locator.Path, out CbBinaryAttachment binaryAttachment))
					{
						writer.WriteBinaryAttachment(RefFieldNames.Target, binaryAttachment);
					}
					else if (TryGetObjectAttachment(value.Locator.Path, out CbObjectAttachment objectAttachment))
					{
						writer.WriteObjectAttachment(RefFieldNames.Target, objectAttachment);
					}
					else
					{
						throw new InvalidOperationException($"Cannot write ref to non-Jupiter blob ({value.Locator})");
					}

					writer.EndObject();

					byte[] data = writer.ToByteArray();
					IoHash hash = IoHash.Compute(data);
					request.Headers.Add(HeaderNames.JupiterIoHash, hash.ToString());

					request.Content = new ByteArrayContent(data);
					request.Content.Headers.ContentType = new MediaTypeHeaderValue(CompactBinaryMimeType);

					using (HttpResponseMessage response = await httpClient.SendAsync(request, cancellationToken))
					{
						await EnsureSuccessAsync(response, cancellationToken);
					}
				}
			}
		}

		#endregion

		/// <inheritdoc/>
		public async Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
		{
			foreach (AddAliasRequest addAlias in request.AddAliases)
			{
				await AddAliasAsync(addAlias.Name, addAlias.Target, addAlias.Rank, addAlias.Data, cancellationToken);
			}
			foreach (RemoveAliasRequest removeAlias in request.RemoveAliases)
			{
				await RemoveAliasAsync(removeAlias.Name, removeAlias.Target, cancellationToken);
			}
			foreach (AddRefRequest addRef in request.AddRefs)
			{
				await WriteRefAsync(addRef.RefName, new HashedBlobRefValue(addRef.Hash, addRef.Target), cancellationToken);
			}
			foreach (RemoveRefRequest removeRef in request.RemoveRefs)
			{
				await DeleteRefAsync(removeRef.RefName, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{ }

		static async Task EnsureSuccessAsync(HttpResponseMessage message, CancellationToken cancellationToken)
		{
			if (!message.IsSuccessStatusCode)
			{
				string response = await message.Content.ReadAsStringAsync(cancellationToken);
				throw new StorageException($"Http response {message.StatusCode} (Content: {response})");
			}
		}
	}
}
