// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Backends
{
	/// <summary>
	/// In-memory implementation of a storage backend
	/// </summary>
	public sealed class MemoryStorageBackend : IStorageBackend
	{
		record class AliasListNode(BlobLocator Locator, int Rank, ReadOnlyMemory<byte> Data, AliasListNode? Next);

		readonly ConcurrentDictionary<BlobLocator, ReadOnlyMemory<byte>> _blobs = new();
		readonly ConcurrentDictionary<RefName, HashedBlobRefValue> _refs = new();
		readonly ConcurrentDictionary<string, AliasListNode?> _aliases = new(StringComparer.Ordinal);

		/// <summary>
		/// All data stored by the client
		/// </summary>
		public IReadOnlyDictionary<BlobLocator, ReadOnlyMemory<byte>> Blobs => _blobs;

		/// <summary>
		/// Accessor for all refs stored by the client
		/// </summary>
		public IReadOnlyDictionary<RefName, HashedBlobRefValue> Refs => _refs;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		#region Blobs

		/// <inheritdoc/>
		public Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> memory = GetBlob(locator, offset, length);
			return Task.FromResult<Stream>(new ReadOnlyMemoryStream(memory));
		}

		/// <inheritdoc/>
		public Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> memory = GetBlob(locator, offset, length);
			return Task.FromResult<IReadOnlyMemoryOwner<byte>>(ReadOnlyMemoryOwner.Create<byte>(memory));
		}

		ReadOnlyMemory<byte> GetBlob(BlobLocator locator, int offset, int? length)
		{
			ReadOnlyMemory<byte> memory = _blobs[locator];
			if (offset != 0)
			{
				memory = memory.Slice(offset);
			}
			if (length != null)
			{
				memory = memory.Slice(0, Math.Min(length.Value, memory.Length));
			}
			return memory;
		}

		/// <inheritdoc/>
		public async Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator>? imports, CancellationToken cancellationToken = default)
		{
			byte[] data = await stream.ReadAllBytesAsync(cancellationToken);
			if (!_blobs.TryAdd(locator, data))
			{
				throw new InvalidOperationException($"Locator {locator} has already been written");
			}
		}

		/// <inheritdoc/>
		public async Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator>? imports, string? prefix = null, CancellationToken cancellationToken = default)
		{
			BlobLocator locator = StorageHelpers.CreateUniqueLocator(prefix);
			await WriteBlobAsync(locator, stream, imports, cancellationToken);
			return locator;
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
			=> default;

		/// <inheritdoc/>
		public ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default)
			=> default;

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public void AddAlias(string name, BlobLocator locator, int rank = 0, ReadOnlyMemory<byte> data = default)
		{
			_aliases.AddOrUpdate(name, _ => new AliasListNode(locator, rank, data, null), (_, entry) => new AliasListNode(locator, rank, data, entry));
		}

		/// <inheritdoc/>
		public void RemoveAlias(string name, BlobLocator locator)
		{
			for (; ; )
			{
				AliasListNode? entry;
				if (!_aliases.TryGetValue(name, out entry))
				{
					break;
				}

				AliasListNode? newEntry = RemoveAliasFromList(entry, locator);
				if (entry == newEntry)
				{
					break;
				}

				if (newEntry == null)
				{
					if (_aliases.TryRemove(new KeyValuePair<string, AliasListNode?>(name, entry)))
					{
						break;
					}
				}
				else
				{
					if (_aliases.TryUpdate(name, newEntry, entry))
					{
						break;
					}
				}
			}
		}

		static AliasListNode? RemoveAliasFromList(AliasListNode? entry, BlobLocator locator)
		{
			if (entry == null)
			{
				return null;
			}
			if (entry.Locator == locator)
			{
				return entry.Next;
			}

			AliasListNode? nextEntry = RemoveAliasFromList(entry.Next, locator);
			if (nextEntry != entry.Next)
			{
				entry = new AliasListNode(entry.Locator, entry.Rank, entry.Data, nextEntry);
			}
			return entry;
		}

		/// <inheritdoc/>
		public Task<BlobAliasLocator[]> FindAliasesAsync(string alias, int? maxResults = null, CancellationToken cancellationToken = default)
		{
			List<BlobAliasLocator> aliases = [];
			if (_aliases.TryGetValue(alias, out AliasListNode? entry))
			{
				for (; entry != null; entry = entry.Next)
				{
					aliases.Add(new BlobAliasLocator(entry.Locator, entry.Rank, entry.Data));
				}
			}
			return Task.FromResult(aliases.ToArray());
		}

		#endregion

		#region Refs

		/// <inheritdoc/>
		public Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			HashedBlobRefValue? value;
			if (_refs.TryGetValue(name, out value))
			{
				return Task.FromResult<HashedBlobRefValue?>(value);
			}
			else
			{
				return Task.FromResult<HashedBlobRefValue?>(null);
			}
		}

		#endregion

		/// <inheritdoc/>
		public Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
		{
			foreach (AddAliasRequest addAlias in request.AddAliases)
			{
				AddAlias(addAlias.Name, addAlias.Target, addAlias.Rank, addAlias.Data);
			}
			foreach (RemoveAliasRequest removeAlias in request.RemoveAliases)
			{
				RemoveAlias(removeAlias.Name, removeAlias.Target);
			}
			foreach (AddRefRequest addRef in request.AddRefs)
			{
				_refs[addRef.RefName] = new HashedBlobRefValue(addRef.Hash, addRef.Target);
			}
			foreach (RemoveRefRequest removeRef in request.RemoveRefs)
			{
				_refs.TryRemove(removeRef.RefName, out _);
			}
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
		}
	}
}
