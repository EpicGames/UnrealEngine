// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Bundles.V2;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Request for a blob to be read
	/// </summary>
	/// <param name="BlobRef">Reference to the blob data</param>
	public record class BlobReadRequest(IHashedBlobRef BlobRef);

	/// <summary>
	/// Response from a read request
	/// </summary>
	public record struct BlobReadResponse<TRequest>(TRequest Request, BlobData Data) : IDisposable
	{
		/// <inheritdoc/>
		public void Dispose()
			=> Data.Dispose();
	}

	/// <summary>
	/// Stats for a batch read
	/// </summary>
	public record class BatchReaderStats(int NumRequests, int NumReads, int NumBundles, int NumPackets, long BytesRead, long BytesDecoded);

	/// <summary>
	/// Implements an efficient pipeline for streaming blob data
	/// </summary>
	public abstract class BatchBlobReader<TRequest> : IDisposable
		where TRequest : BlobReadRequest
	{
		record class BundleRequest(BundleHandle Handle, int MinOffset, int MaxOffset, List<PacketRequest> Packets);

		record class BundleResponse(IReadOnlyMemoryOwner<byte> Data, int MinOffset, int MaxOffset, List<PacketRequest> Packets) : IDisposable
		{
			public void Dispose()
				=> Data.Dispose();
		}

		record class PacketRequest(FlushedPacketHandle Handle, List<ExportRequest> Exports)
		{
			public int MinOffset => Handle.PacketOffset;
			public int MaxOffset => Handle.PacketOffset + Handle.PacketLength;
		}

		record class ExportRequest(IoHash Hash, ExportHandle Handle, TRequest OriginalRequest);

		record class PacketResponse(IReadOnlyMemoryOwner<byte> Data, List<ExportRequest> Exports) : IDisposable
		{
			public void Dispose()
				=> Data.Dispose();
		}

		/// <summary>
		/// Number of items to read from the input queue before partitioning into batches
		/// </summary>
		public int MinQueueLength { get; set; } = 2000;

		/// <summary>
		/// Maximum gap between reads that should be coalesced and executed together
		/// </summary>
		public long CoalesceReadsBelowSize { get; set; } = 2 * 1024 * 1024;

		/// <summary>
		/// Whether to verify hashes of data read from storage
		/// </summary>
		public bool VerifyHashes { get; set; }

		readonly Channel<TRequest> _otherRequests;
		readonly Channel<BundleRequest> _bundleRequests;

		readonly Channel<BundleResponse> _bundleResponses;
		readonly Channel<PacketResponse> _packetResponses;

		int _numRequests;
		int _numReads;
		int _numBundles;
		int _numPackets;
		long _bytesRead;
		long _bytesDecoded;

		/// <summary>
		/// Constructor
		/// </summary>
		protected BatchBlobReader()
		{
			_otherRequests = Channel.CreateUnbounded<TRequest>();
			_bundleRequests = Channel.CreateUnbounded<BundleRequest>();
			_bundleResponses = Channel.CreateBounded<BundleResponse>(new BoundedChannelOptions(32));
			_packetResponses = Channel.CreateBounded<PacketResponse>(new BoundedChannelOptions(128));
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				while (_bundleResponses.Reader.TryRead(out BundleResponse? item))
				{
					item.Dispose();
				}

				while (_packetResponses.Reader.TryRead(out PacketResponse? item))
				{
					item.Dispose();
				}
			}
		}

		/// <summary>
		/// Gets stats for the reader
		/// </summary>
		public BatchReaderStats GetStats()
			=> new(_numRequests, _numReads, _numBundles, _numPackets, _bytesRead, _bytesDecoded);

		/// <summary>
		/// Adds a batch reader to the given pipeline
		/// </summary>
		public Task[] AddToPipeline(AsyncPipeline pipeline, int numReadTasks, int numDecodeTasks, ChannelReader<TRequest> requestReader)
		{
			_ = pipeline.AddTask(cancellationToken => CreateBundleRequestsAsync(requestReader, cancellationToken));
			Task otherTask = pipeline.AddTask(HandleOtherRequestsAsync);

			// Read bundles and mark the bundle response channel as complete once we're finished
			Task[] bundleTasks = pipeline.AddTasks(numReadTasks, ReadBundlesAsync);
			_ = Task.WhenAll(bundleTasks).ContinueWith(_ => _bundleResponses.Writer.TryComplete(), TaskScheduler.Default);

			// Read packets
			Task[] packetTasks = pipeline.AddTasks(numDecodeTasks, ReadPacketsAsync);

			// Read all the output blobs and mark the blob response channel as complete
			return [.. packetTasks, otherTask];
		}

		// Read a blob using the naive read pipeline
		async Task HandleOtherRequestsAsync(CancellationToken cancellationToken)
		{
			await foreach (TRequest request in _otherRequests.Reader.ReadAllAsync(cancellationToken))
			{
#pragma warning disable CA2000
				BlobData? blobData = null;
				try
				{
					blobData = await request.BlobRef.ReadBlobDataAsync(cancellationToken);
					VerifyHash(request.BlobRef, blobData.Data.Span, request.BlobRef.Hash);
					await HandleResponsesAsync([new BlobReadResponse<TRequest>(request, blobData)], cancellationToken);
				}
				catch
				{
					blobData?.Dispose();
				}
#pragma warning restore CA2000
			}
		}

		/// <summary>
		/// Handle responses from the read
		/// </summary>
		/// <param name="responses">Responses from the read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		protected abstract ValueTask HandleResponsesAsync(List<BlobReadResponse<TRequest>> responses, CancellationToken cancellationToken);

		#region Grouping requests

		record class PacketExport(FlushedPacketHandle PacketHandle, IoHash ExportHash, ExportHandle ExportHandle, TRequest OriginalRequest);

		async Task CreateBundleRequestsAsync(ChannelReader<TRequest> requestReader, CancellationToken cancellationToken)
		{
			int queueLength = 0;
			Queue<BundleHandle> bundleQueue = new();
			Dictionary<BundleHandle, List<PacketExport>> bundleHandleToExportBatch = [];

			for (; ; )
			{
				// Fill the queue up to the max length
				for (; ; )
				{
					TRequest? request;
					if (!requestReader.TryRead(out request))
					{
						if (queueLength >= MinQueueLength)
						{
							break;
						}
						if (!await requestReader.WaitToReadAsync(cancellationToken))
						{
							break;
						}
					}
					else
					{
						Interlocked.Increment(ref _numRequests);

						PacketExport? outputExport;
						if (!TryGetPacketExport(request, out outputExport))
						{
							await _otherRequests.Writer.WriteAsync(request, cancellationToken);
						}
						else
						{
							BundleHandle bundleHandle = outputExport.PacketHandle.Bundle;
							if (!bundleHandleToExportBatch.TryGetValue(bundleHandle, out List<PacketExport>? existingExportBatch))
							{
								existingExportBatch = [];
								bundleHandleToExportBatch.Add(bundleHandle, existingExportBatch);
								bundleQueue.Enqueue(bundleHandle);
							}

							existingExportBatch.Add(outputExport);
							queueLength++;
						}
					}
				}

				// Exit once we've processed everything and can't get any more items to read.
				if (queueLength == 0)
				{
					_bundleRequests.Writer.TryComplete();
					_otherRequests.Writer.TryComplete();
					break;
				}

				// Flush the first queue
				{
					BundleHandle bundleHandle = bundleQueue.Dequeue();
					List<PacketExport> exportBatch = bundleHandleToExportBatch[bundleHandle];
					queueLength -= exportBatch.Count;
					bundleHandleToExportBatch.Remove(bundleHandle);

					await WriteBundleRequestsAsync(bundleHandle, exportBatch, cancellationToken);
				}
			}
		}

		async Task WriteBundleRequestsAsync(BundleHandle bundleHandle, List<PacketExport> exports, CancellationToken cancellationToken)
		{
			Interlocked.Increment(ref _numBundles);

			// Group the reads by packet
			List<PacketRequest> packetRequests = [];
			foreach (IGrouping<int, PacketExport> group in exports.GroupBy(x => x.PacketHandle.PacketOffset))
			{
				List<ExportRequest> exportRequests = [.. group.Select(x => new ExportRequest(x.ExportHash, x.ExportHandle, x.OriginalRequest))];
				PacketRequest packetRequest = new(group.First().PacketHandle, exportRequests);
				packetRequests.Add(packetRequest);
			}
			packetRequests.SortBy(x => x.MinOffset);

			// Split the packet requests into contiguous bundle reads
			for (int maxIdx = 0; maxIdx < packetRequests.Count; maxIdx++)
			{
				int minIdx = maxIdx;
				while (maxIdx + 1 < packetRequests.Count && packetRequests[maxIdx + 1].MinOffset < packetRequests[maxIdx].MaxOffset + CoalesceReadsBelowSize)
				{
					maxIdx++;
				}

				int minOffset = packetRequests[minIdx].MinOffset;
				int maxOffset = packetRequests[maxIdx].MaxOffset;

				BundleRequest request = new(bundleHandle, minOffset, maxOffset, packetRequests.GetRange(minIdx, (maxIdx + 1) - minIdx));
				await _bundleRequests.Writer.WriteAsync(request, cancellationToken);
			}
		}

		static bool TryGetPacketExport(TRequest request, [NotNullWhen(true)] out PacketExport? export)
		{
			if (request.BlobRef.Innermost is ExportHandle exportHandle && exportHandle.Packet is FlushedPacketHandle packetHandle)
			{
				export = new PacketExport(packetHandle, request.BlobRef.Hash, exportHandle, request);
				return true;
			}
			else
			{
				export = null;
				return false;
			}
		}

		#endregion
		#region Bundle Reads

		async Task ReadBundlesAsync(CancellationToken cancellationToken)
		{
			await foreach (BundleRequest request in _bundleRequests.Reader.ReadAllAsync(cancellationToken))
			{
				IReadOnlyMemoryOwner<byte>? data = null;
				try
				{
					data = await request.Handle.ReadAsync(request.MinOffset, request.MaxOffset - request.MinOffset, cancellationToken);
					Interlocked.Increment(ref _numReads);
					Interlocked.Add(ref _bytesRead, data.Memory.Length);
#pragma warning disable CA2000
					BundleResponse response = new(data, request.MinOffset, request.MaxOffset, request.Packets);
					await _bundleResponses.Writer.WriteAsync(response, cancellationToken);
#pragma warning restore CA2000
				}
				catch
				{
					data?.Dispose();
					throw;
				}
			}
		}

		void VerifyHash(IBlobRef blobRef, ReadOnlySpan<byte> data, IoHash hash)
		{
			if (VerifyHashes)
			{
				IoHash actualHash = IoHash.Compute(data);
				if (actualHash != hash)
				{
					throw new InvalidDataException($"Invalid hash while reading {blobRef.GetLocator()}: Expected {hash}, got {actualHash}.");
				}
			}
		}

		#endregion
		#region Packet reads

		async Task ReadPacketsAsync(CancellationToken cancellationToken)
		{
			await foreach (BundleResponse bundleResponse in _bundleResponses.Reader.ReadAllAsync(cancellationToken))
			{
				using IDisposable lifetime = bundleResponse;

#pragma warning disable CA2000
				List<BlobReadResponse<TRequest>> responses = [];
				try
				{
					foreach (PacketRequest packetRequest in bundleResponse.Packets)
					{
						// Decode the data for this packet
						ReadOnlyMemory<byte> memory = bundleResponse.Data.Memory.Slice(packetRequest.Handle.PacketOffset - bundleResponse.MinOffset, packetRequest.Handle.PacketLength);

						using PacketReader packetReader = packetRequest.Handle.CreatePacketReader(memory);
						Interlocked.Increment(ref _numPackets);
						Interlocked.Add(ref _bytesDecoded, packetReader.Packet.Length);

						// Create responses for each blob
						for (int idx = 0; idx < packetRequest.Exports.Count; idx++)
						{
							ExportRequest exportRequest = packetRequest.Exports[idx];
							BlobData blobData = packetReader.ReadExport(exportRequest.Handle.ExportIdx);
							responses.Add(new BlobReadResponse<TRequest>(exportRequest.OriginalRequest, blobData));
							VerifyHash(exportRequest.Handle, blobData.Data.Span, exportRequest.Hash);
						}
					}
					await HandleResponsesAsync(responses, cancellationToken);
				}
				catch
				{
					foreach (BlobReadResponse<TRequest> response in responses)
					{
						response.Dispose();
					}
					throw;
				}
#pragma warning restore CA2000
			}
		}

		#endregion
	}
}
