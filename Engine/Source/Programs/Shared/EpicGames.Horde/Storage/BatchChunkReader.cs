// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Request for a chunk to be read before being written to disk
	/// </summary>
	/// <param name="File">File to read from</param>
	/// <param name="Offset">Offset within the output file</param>
	/// <param name="Handle">Handle to the blob data</param>
	record class ChunkReadRequest(OutputFile File, long Offset, IHashedBlobRef Handle) : BlobReadRequest(Handle);

	/// <summary>
	/// Implementation of <see cref="BatchBlobReader{TRequest}"/> designed for reading leaf chunks of data from storage
	/// </summary>
	class BatchChunkReader : BatchBlobReader<ChunkReadRequest>, IDisposable
	{
		public const int DefaultMaxBatchSize = 20;

		/// <summary>
		/// Maximum number of exports to write in a single request
		/// </summary>
		public int MaxBatchSize
		{
			get => _maxBatchSize;
			init
			{
				if (value > 0)
				{
					_maxBatchSize = value;
				}
			}
		}

		readonly int _maxBatchSize = DefaultMaxBatchSize;
		readonly ChannelWriter<OutputChunk[]> _writer;

		/// <summary>
		/// Constructor
		/// </summary>
		public BatchChunkReader(ChannelWriter<OutputChunk[]> writer)
		{
			_writer = writer;
		}

		/// <inheritdoc/>
		protected override async ValueTask HandleResponsesAsync(List<BlobReadResponse<ChunkReadRequest>> responses, CancellationToken cancellationToken)
		{
			for (int idx = 0; idx < responses.Count;)
			{
				int firstIdx = idx;

				List<OutputChunk> batch = [];
				for (; idx < responses.Count && batch.Count < MaxBatchSize && responses[idx].Request.File == responses[firstIdx].Request.File; idx++)
				{
					BlobReadResponse<ChunkReadRequest> response = responses[idx];
					OutputChunk chunk = new(response.Request.File, response.Request.Offset, response.Data.Data, response.Request.Handle, response.Data);
					batch.Add(chunk);
				}

				await _writer.WriteAsync([.. batch], cancellationToken);
			}
		}
	}
}
