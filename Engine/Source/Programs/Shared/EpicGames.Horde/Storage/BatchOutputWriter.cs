// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.MemoryMappedFiles;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Utilities;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Represents an output file that write requests may be issued against
	/// </summary>
	class OutputFile
	{
		/// <summary>
		/// The relative output path
		/// </summary>
		public string Path { get; }

		/// <summary>
		/// File metadata
		/// </summary>
		public FileInfo FileInfo { get; }

		/// <summary>
		/// File entry with metadata about the target file
		/// </summary>
		public FileEntry FileEntry { get; }

		bool _createdFile;
		int _remainingChunks;

		/// <summary>
		/// Constructor
		/// </summary>
		public OutputFile(string path, FileInfo fileInfo, FileEntry fileEntry)
		{
			Path = path;
			FileInfo = fileInfo;
			FileEntry = fileEntry;
		}

		public int IncrementRemaining() => Interlocked.Increment(ref _remainingChunks);
		public int DecrementRemaining() => Interlocked.Decrement(ref _remainingChunks);

		/// <summary>
		/// Opens the file for writing, setting its length on the first run
		/// </summary>
		public FileStream OpenStream()
		{
			lock (FileEntry)
			{
				if (!_createdFile)
				{
					if (FileInfo.Exists)
					{
						if (FileInfo.LinkTarget != null)
						{
							FileInfo.Delete();
						}
						else if (FileInfo.IsReadOnly)
						{
							FileInfo.IsReadOnly = false;
						}
					}
					else
					{
						FileInfo.Directory?.Create();
					}
				}

				FileStream? stream = null;
				try
				{
					try
					{
						stream = FileInfo.Open(FileMode.OpenOrCreate, FileAccess.ReadWrite, FileShare.ReadWrite);
					}
					catch (IOException ex)
					{
						string? lockInfo = FileUtils.GetFileLockInfo(FileInfo.FullName);
						if (lockInfo == null)
						{
							throw;
						}
						else
						{
							throw new WrappedFileOrDirectoryException(ex, $"{ex.Message}\n{lockInfo}");
						}
					}

					if (!_createdFile)
					{
						stream.SetLength(FileEntry.Length);
						_createdFile = true;
					}
					return stream;
				}
				catch
				{
					stream?.Dispose();
					throw;
				}
			}
		}

		/// <summary>
		/// Callback for a file having been fully written
		/// </summary>
		public virtual ValueTask CompleteAsync(CancellationToken cancellationToken)
			=> default;
	}

	/// <summary>
	/// Request to output a block of data A chunk which needs to be written to an output file
	/// </summary>
	record class OutputChunk(OutputFile File, long Offset, ReadOnlyMemory<byte> Data, object Source, IDisposable? DataOwner) : IDisposable
	{
		/// <inheritdoc/>
		public void Dispose()
			=> DataOwner?.Dispose();
	}

	/// <summary>
	/// Batches file write requests
	/// </summary>
	sealed class BatchOutputWriter : IDisposable
	{
		readonly Channel<OutputChunk[]> _channel;
		readonly ILogger _logger;

		int _writtenFiles;
		long _writtenBytes;

		/// <summary>
		/// Number of files that have been written
		/// </summary>
		public int WrittenFiles
			=> Interlocked.CompareExchange(ref _writtenFiles, 0, 0);

		/// <summary>
		/// Total number of bytes that have been written
		/// </summary>
		public long WrittenBytes
			=> Interlocked.CompareExchange(ref _writtenBytes, 0, 0);

		/// <summary>
		/// Number of writes to execute sequentially vs in parallel
		/// </summary>
		public int WriteBatchSize { get; set; } = 64;

		/// <summary>
		/// If false, disables output to disk. Useful for performance testing.
		/// </summary>
		public bool EnableOutput { get; set; } = true;

		/// <summary>
		/// If true, hashes output files after writing to verify their integrity
		/// </summary>
		public bool VerifyOutput { get; set; }

		/// <summary>
		/// If true, outputs verbose information to the log
		/// </summary>
		public bool VerboseOutput { get; set; }

		/// <summary>
		/// Sink for write requests
		/// </summary>
		public ChannelWriter<OutputChunk[]> RequestWriter
			=> _channel.Writer;

		/// <summary>
		/// Constructor
		/// </summary>
		public BatchOutputWriter(ILogger logger)
		{
			_channel = Channel.CreateBounded<OutputChunk[]>(new BoundedChannelOptions(128));
			_logger = logger;
		}

		/// <summary>
		/// Dispose remaining items in the queue
		/// </summary>
		public void Dispose()
		{
			while (_channel.Reader.TryRead(out OutputChunk[]? chunks))
			{
				foreach (OutputChunk chunk in chunks)
				{
					chunk.Dispose();
				}
			}
		}

		/// <summary>
		/// Adds tasks for the writer to an async pipeline
		/// </summary>
		/// <param name="pipeline">Pipeline instance</param>
		/// <param name="numWriteTasks">Number of parallel writes</param>
		public Task[] AddToPipeline(AsyncPipeline pipeline, int numWriteTasks)
		{
			return pipeline.AddTasks(numWriteTasks, ProcessAsync);
		}

		/// <summary>
		/// Processes requests from the given input channel
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task ProcessAsync(CancellationToken cancellationToken)
		{
			await foreach (OutputChunk[] chunks in _channel.Reader.ReadAllAsync(cancellationToken))
			{
				try
				{
					await WriteChunksAsync(chunks, cancellationToken);
				}
				catch (Exception ex) when (ex is not OperationCanceledException)
				{
					_logger.LogError(ex, "Error writing to disk: {Message}", ex.Message);
					throw;
				}
				finally
				{
					foreach (OutputChunk chunk in chunks)
					{
						chunk.Dispose();
					}
				}
			}
		}

		async Task WriteChunksAsync(OutputChunk[] chunks, CancellationToken cancellationToken)
		{
			for (int requestIdx = 0; requestIdx < chunks.Length;)
			{
				OutputFile outputFile = chunks[requestIdx].File;

				int maxChunkIdx = requestIdx + 1;
				while (maxChunkIdx < chunks.Length && chunks[maxChunkIdx].File == outputFile)
				{
					maxChunkIdx++;
				}

				try
				{
					await ExtractChunksToFileAsync(outputFile, new ArraySegment<OutputChunk>(chunks, requestIdx, maxChunkIdx - requestIdx), cancellationToken);
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception ex)
				{
					throw new StorageException($"Unable to extract {outputFile.FileInfo.FullName}: {ex.Message}", ex);
				}

				requestIdx = maxChunkIdx;
			}
		}

		async Task ExtractChunksToFileAsync(OutputFile outputFile, ArraySegment<OutputChunk> chunks, CancellationToken cancellationToken)
		{
			using PlatformFileLock? fileLock = await PlatformFileLock.CreateAsync(cancellationToken);

			// Open the file for the current chunk
			int remainingChunks = 0;
			await using (FileStream stream = outputFile.OpenStream())
			{
				if (outputFile.FileEntry.Length == 0 || !EnableOutput)
				{
					// If this file is empty, don't write anything and just move to the next chunk
					for (int idx = 0; idx < chunks.Count; idx++)
					{
						remainingChunks = outputFile.DecrementRemaining();
					}
				}
				else
				{
					// Process as many chunks as we can for this file
					using MemoryMappedFile memoryMappedFile = MemoryMappedFile.CreateFromFile(stream, null, outputFile.FileEntry.Length, MemoryMappedFileAccess.ReadWrite, HandleInheritability.None, false);
					using MemoryMappedView memoryMappedView = CreateMemoryMappedView(memoryMappedFile, outputFile);

					for (int chunkIdx = 0; chunkIdx < chunks.Count; chunkIdx++)
					{
						OutputChunk chunk = chunks[chunkIdx];
						cancellationToken.ThrowIfCancellationRequested();

						if (VerboseOutput)
						{
							_logger.LogInformation("Writing {File},{Offset} ({Source})", outputFile.Path, chunk.Offset, chunk.Source);
						}

						// Write this chunk
						ReadOnlyMemory<byte> memory = chunk.Data;
						memory.CopyTo(memoryMappedView!.GetMemory(chunk.Offset, memory.Length));

						// Update the stats
						remainingChunks = outputFile.DecrementRemaining();
						Interlocked.Add(ref _writtenBytes, memory.Length);
					}
				}
			}

			// Set correct permissions on the output file
			if (remainingChunks == 0)
			{
				if (VerifyOutput)
				{
					IoHash computedHash;
					using (Stream stream = outputFile.OpenStream())
					{
						computedHash = await IoHash.ComputeAsync(stream, cancellationToken);
					}
					if (computedHash != outputFile.FileEntry.Hash)
					{
						throw new InvalidDataException($"File {outputFile.Path} does not have correct hash after extract; expected {outputFile.FileEntry.Hash}, got {computedHash}.");
					}
					_logger.LogDebug("Verified hash of {Path}: {Hash}", outputFile.Path, computedHash);
				}

				outputFile.FileInfo.Refresh();
				FileEntry.SetPermissions(outputFile.FileInfo!, outputFile.FileEntry.Flags);

				if ((outputFile.FileEntry.Flags & FileEntryFlags.HasModTime) != 0)
				{
					outputFile.FileInfo.LastWriteTimeUtc = outputFile.FileEntry.ModTime;
				}

				Interlocked.Increment(ref _writtenFiles);

				await outputFile.CompleteAsync(cancellationToken);
			}
		}

		static MemoryMappedView CreateMemoryMappedView(MemoryMappedFile memoryMappedFile, OutputFile file)
		{
			try
			{
				MemoryMappedView view = new(memoryMappedFile, 0, file.FileEntry.Length);
				return view;
			}
			catch (Exception ex)
			{
				file.FileInfo.Refresh();
				throw new StorageException($"Unable to create mapped view of file (entry length: {file.FileEntry.Length}, file length: {file.FileInfo.Length}): {ex.Message}", ex);
			}
		}
	}
}
