// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Data for an alias in the storage system. An alias is a named weak reference to a node.
	/// </summary>
	/// <param name="Target">Handle to the target blob for the alias</param>
	/// <param name="Rank">Rank for the alias</param>
	/// <param name="Data">Data stored inline with the alias</param>
	public record class BlobAlias(IBlobRef Target, int Rank, ReadOnlyMemory<byte> Data);

	/// <summary>
	/// Data for an alias in the storage system. An alias is a named weak reference to a node.
	/// </summary>
	/// <param name="Target">Handle to the target blob for the alias</param>
	/// <param name="Rank">Rank for the alias</param>
	/// <param name="Data">Data stored inline with the alias</param>
	public record class BlobAliasLocator(BlobLocator Target, int Rank, ReadOnlyMemory<byte> Data);
}
