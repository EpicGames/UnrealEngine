// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Attribute used to denote the converter for an object type
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class BlobConverterAttribute : Attribute
	{
		/// <summary>
		/// Type of the converter
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobConverterAttribute(Type converterType) => ConverterType = converterType;
	}

	/// <summary>
	/// Base class for converter types that can serialize blobs from native C# types. Semantics mirror <see cref="System.Text.Json.Serialization.JsonConverter"/>.
	/// </summary>
	public abstract class BlobConverter
	{
		/// <summary>
		/// Determines if the converter can handle the given type
		/// </summary>
		/// <param name="typeToConvert">The type that needs to be converted</param>
		/// <returns>True if this converter can handle the given type</returns>
		public abstract bool CanConvert(Type typeToConvert);
	}

	/// <summary>
	/// Serializer for typed values to blob data. Mimics the <see cref="System.Text.Json.Serialization.JsonConverter{T}"/> interface for familiarity.
	/// </summary>
	public abstract class BlobConverter<T> : BlobConverter
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type type) => type == typeof(T);

		/// <summary>
		/// Reads a strongly typed value
		/// </summary>
		public abstract T Read(IBlobReader reader, BlobSerializerOptions options);

		/// <summary>
		/// Writes a strongly typed value
		/// </summary>
		public abstract BlobType Write(IBlobWriter writer, T value, BlobSerializerOptions options);
	}
}
