// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Data for an individual node. Must be disposed after use.
	/// </summary>
	public class BlobData : IDisposable
	{
		/// <summary>
		/// Type of the blob
		/// </summary>
		public BlobType Type { get; }

		/// <summary>
		/// Raw data for the blob. Lifetime of this data is tied to the lifetime of the <see cref="BlobData"/> object; consumers must not retain references to it.
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Handles to referenced blobs
		/// </summary>
		public IReadOnlyList<IBlobRef> Imports { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobData(BlobType type, ReadOnlyMemory<byte> data, IReadOnlyList<IBlobRef> imports)
		{
			Type = type;
			Data = data;
			Imports = imports;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		/// <param name="disposing">True if derived instances should dispose managed resources. False when called from a finalizer.</param>
		protected virtual void Dispose(bool disposing)
		{
		}
	}

	/// <summary>
	/// Implementation of <see cref="BlobData"/> for <see cref="IReadOnlyMemoryOwner{Byte}"/> instances.
	/// </summary>
	public class BlobDataWithOwner : BlobData
	{
		readonly IDisposable _owner;

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobDataWithOwner(BlobType type, ReadOnlyMemory<byte> data, IReadOnlyList<IBlobRef> imports, IDisposable owner)
			: base(type, data, imports)
		{
			_owner = owner;
		}

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
			base.Dispose(disposing);

			if (disposing)
			{
				_owner.Dispose();
			}
		}
	}
}
