// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Identifier for a blob within a particular namespace.
	/// </summary>
	[JsonSchemaString]
	[TypeConverter(typeof(BlobLocatorTypeConverter))]
	[JsonConverter(typeof(BlobLocatorJsonConverter))]
	public readonly struct BlobLocator : IEquatable<BlobLocator>
	{
		readonly Utf8String _path;

		/// <summary>
		/// Accessor for the internal path string
		/// </summary>
		public Utf8String Path => _path;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Path to the blob. The meaning of this string is implementation defined.</param>
		public BlobLocator(Utf8String path) => _path = path;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Path to the blob. The meaning of this string is implementation defined.</param>
		public BlobLocator(string path) => _path = new Utf8String(path);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="outer">The base locator to append to</param>
		/// <param name="fragment">Characters to append</param>
		public BlobLocator(BlobLocator outer, string fragment)
			: this(outer, Encoding.UTF8.GetBytes(fragment))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="outer">The base locator to append to</param>
		/// <param name="fragment">Characters to append</param>
		public BlobLocator(BlobLocator outer, ReadOnlySpan<byte> fragment)
		{
			byte[] buffer = new byte[outer._path.Length + 1 + fragment.Length];
			outer._path.Span.CopyTo(buffer);
			buffer[outer._path.Length] = (byte)'#';
			fragment.CopyTo(buffer.AsSpan(outer._path.Length + 1));
			_path = new Utf8String(buffer);
		}

		/// <summary>
		/// Whether the blob locator is valid
		/// </summary>
		public bool IsValid() => !_path.IsEmpty;

		/// <summary>
		/// The base blob locator
		/// </summary>
		public BlobLocator BaseLocator
		{
			get
			{
				int hashIdx = _path.IndexOf('#');
				return (hashIdx == -1) ? new BlobLocator(_path) : new BlobLocator(_path.Slice(0, hashIdx));
			}
		}

		/// <summary>
		/// Fragment within the base blob
		/// </summary>
		public Utf8String Fragment
		{
			get
			{
				int hashIdx = _path.IndexOf('#');
				return (hashIdx == -1) ? Utf8String.Empty : _path.Slice(hashIdx + 1);
			}
		}

		/// <summary>
		/// Determines if this locator can be unwrapped into an outer locator/fragment pair
		/// </summary>
		public bool CanUnwrap() => _path.IndexOf('#') != -1;

		/// <summary>
		/// Split this locator into a locator and fragment
		/// </summary>
		/// <param name="baseLocator">Receives the base blob locator</param>
		/// <param name="fragment">Receives the blob fragment</param>
		/// <returns>True if the locator was unwrapped, false otherwise</returns>
		public bool TryUnwrap(out BlobLocator baseLocator, out Utf8String fragment)
		{
			int hashIdx = _path.LastIndexOf('#');
			if (hashIdx == -1)
			{
				baseLocator = this;
				fragment = default;
				return false;
			}
			else
			{
				baseLocator = new BlobLocator(_path.Slice(0, hashIdx));
				fragment = _path.Slice(hashIdx + 1);
				return true;
			}
		}

		/// <summary>
		/// Checks whether this blob is within the given folder
		/// </summary>
		/// <param name="folderName">Name of the folder</param>
		/// <returns>True if the the blob id is within the given folder</returns>
		public bool WithinFolder(Utf8String folderName)
		{
			Utf8String path = Path;
			return path.Length > folderName.Length && path.StartsWith(folderName) && path[folderName.Length] == '/';
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is BlobLocator blobId && Equals(blobId);

		/// <inheritdoc/>
		public override int GetHashCode() => _path.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(BlobLocator other) => _path == other._path;

		/// <inheritdoc/>
		public override string ToString() => _path.ToString();

		/// <inheritdoc/>
		public static bool operator ==(BlobLocator left, BlobLocator right) => left.Path == right.Path;

		/// <inheritdoc/>
		public static bool operator !=(BlobLocator left, BlobLocator right) => !(left == right);
	}

	/// <summary>
	/// Type converter from strings to BlobLocator objects
	/// </summary>
	sealed class BlobLocatorTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return new BlobLocator((string)value);
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			return destinationType == typeof(string);
		}

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return value?.ToString();
			}
			else
			{
				return null;
			}
		}
	}

	/// <summary>
	/// Class which serializes BlobLocator objects to JSON
	/// </summary>
	public sealed class BlobLocatorJsonConverter : JsonConverter<BlobLocator>
	{
		/// <inheritdoc/>
		public override BlobLocator Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(new Utf8String(reader.GetUtf8String().ToArray()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, BlobLocator value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
	}
}
