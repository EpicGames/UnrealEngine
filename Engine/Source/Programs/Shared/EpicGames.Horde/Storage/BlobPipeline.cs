// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Bundles.V2;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Request to read a blob from storage
	/// </summary>
	public record class BlobRequest<TUserData>(IBlobRef Handle, TUserData UserData);

	/// <summary>
	/// Response from reading a blob from storage
	/// </summary>
	public sealed record class BlobResponse<TUserData>(BlobData BlobData, TUserData UserData) : IDisposable
	{
		/// <inheritdoc/>
		public void Dispose()
			=> BlobData.Dispose();
	}

	/// <summary>
	/// Batch of responses from the reader 
	/// </summary>
	public sealed record class BlobResponseBatch<TUserData>(BlobResponse<TUserData>[] Responses) : IDisposable
	{
		/// <inheritdoc/>
		public void Dispose()
		{
			foreach (BlobResponse<TUserData> response in Responses)
			{
				response.Dispose();
			}
		}
	}

	/// <summary>
	/// Options for <see cref="BlobPipeline{TUserData}"/>
	/// </summary>
	public class BlobPipelineOptions
	{
		/// <summary>
		/// Maximum number of responses to buffer before pausing.
		/// </summary>
		public int ResponseBufferSize { get; set; } = 200;

		/// <summary>
		/// Number of requests to enumerate before flushing the current batch
		/// </summary>
		public int FlushBatchLength { get; set; } = 200;

		/// <summary>
		/// Number of batches to fetch in parallel
		/// </summary>
		public int NumFetchTasks { get; set; } = 4;
	}

	/// <summary>
	/// Helper class to sort requested reads for optimal coherency within bundles
	/// </summary>
	/// <typeparam name="TUserData">Type of user data to include with requests</typeparam>
	public sealed class BlobPipeline<TUserData> : IAsyncDisposable
	{
		readonly BlobPipelineOptions _options;
		readonly CancellationTokenSource _cancellationTokenSource = new();
		readonly List<Task> _tasks = [];
		readonly Channel<BlobRequest<TUserData>> _requestChannel;
		readonly Channel<BlobRequest<TUserData>[]> _requestBatchChannel;
		readonly Channel<BlobResponseBatch<TUserData>> _responseBatchChannel;

		int _writerCount = 1;
		bool _writerFinished;

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobPipeline()
			: this(new BlobPipelineOptions())
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobPipeline(BlobPipelineOptions options)
		{
			_options = options;

			_requestChannel = Channel.CreateUnbounded<BlobRequest<TUserData>>();
			_requestBatchChannel = Channel.CreateUnbounded<BlobRequest<TUserData>[]>();
			_responseBatchChannel = Channel.CreateBounded<BlobResponseBatch<TUserData>>(new BoundedChannelOptions(options.ResponseBufferSize) { FullMode = BoundedChannelFullMode.Wait });

			_tasks.Add(Task.Run(() => BatchLoopAsync(_cancellationTokenSource.Token), _cancellationTokenSource.Token));
			_tasks.Add(Task.Run(() => FetchLoopAsync(_cancellationTokenSource.Token), _cancellationTokenSource.Token));
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_tasks.Count > 0)
			{
				await _cancellationTokenSource.CancelAsync();
				try
				{
					await Task.WhenAll(_tasks);
				}
				catch (OperationCanceledException)
				{
				}
				_tasks.Clear();
			}

			_cancellationTokenSource.Dispose();

			BlobResponseBatch<TUserData>? batch;
			while (_responseBatchChannel.Reader.TryRead(out batch))
			{
				batch.Dispose();
			}
		}

		/// <summary>
		/// Adds a new read request
		/// </summary>
		/// <param name="request">The request to add</param>
		public void Add(BlobRequest<TUserData> request)
		{
			if (!_requestChannel.Writer.TryWrite(request))
			{
				throw new InvalidOperationException("Cannot write request to channel");
			}
		}

		/// <summary>
		/// Adds a new request source
		/// </summary>
		/// <param name="factory">Method to construct the sequence of items</param>
		public void AddSource(Func<CancellationToken, IAsyncEnumerator<BlobRequest<TUserData>>> factory)
		{
			AdjustWriterCount(1);
			_tasks.Add(Task.Run(() => CopyToRequestChannelAsync(factory, _cancellationTokenSource.Token), _cancellationTokenSource.Token));
		}

		async Task CopyToRequestChannelAsync(Func<CancellationToken, IAsyncEnumerator<BlobRequest<TUserData>>> factory, CancellationToken cancellationToken)
		{
			await using IAsyncEnumerator<BlobRequest<TUserData>> source = factory(cancellationToken);
			while (await source.MoveNextAsync())
			{
				await _requestChannel.Writer.WriteAsync(source.Current, cancellationToken);
			}
			AdjustWriterCount(-1);
		}

		/// <summary>
		/// Indicate that we've finished adding new items to the reader
		/// </summary>
		public void FinishAdding()
		{
			if (!_writerFinished)
			{
				AdjustWriterCount(-1);
				_writerFinished = true;
			}
		}

		void AdjustWriterCount(int delta)
		{
			for (; ; )
			{
				int writerCount = Interlocked.CompareExchange(ref _writerCount, 0, 0);
				if (writerCount == 0)
				{
					throw new InvalidOperationException("Reading has already been marked complete");
				}
				if (Interlocked.CompareExchange(ref _writerCount, writerCount + delta, writerCount) == writerCount)
				{
					if (writerCount + delta == 0)
					{
						_requestChannel.Writer.TryComplete();
					}
					break;
				}
			}
		}

		record class BundleRequest(BundleHandle BundleHandle, int PacketOffset, int ExportIdx, BlobRequest<TUserData> OriginalRequest);
		record class BundleBatchRequest(BundleHandle BundleHandle, List<BundleRequest> Requests);

		async Task BatchLoopAsync(CancellationToken cancellationToken)
		{
			int queueLength = 0;
			Queue<BundleBatchRequest> bundleBatchQueue = new();
			Dictionary<BundleHandle, BundleBatchRequest> bundleHandleToBatch = [];

			for (; ; )
			{
				// Add requests to the queue
				for (; ; )
				{
					BlobRequest<TUserData>? request;
					if (_requestChannel.Reader.TryRead(out request))
					{
						if (TryAddBundleRequest(request, bundleBatchQueue, bundleHandleToBatch))
						{
							queueLength++;
						}
						else
						{
							await _requestBatchChannel.Writer.WriteAsync([request], cancellationToken);
						}
					}
					else
					{
						if (queueLength > _options.FlushBatchLength)
						{
							break;
						}
						if (!await _requestChannel.Reader.WaitToReadAsync(cancellationToken))
						{
							break;
						}
					}
				}

				// Exit once we've processed everything and can't get any more items to read.
				if (queueLength == 0)
				{
					_requestBatchChannel.Writer.TryComplete();
					break;
				}

				// Flush the first queue
				BundleBatchRequest exportBatch = bundleBatchQueue.Dequeue();
				queueLength -= exportBatch.Requests.Count;
				bundleHandleToBatch.Remove(exportBatch.BundleHandle);

				BlobRequest<TUserData>[] chunkBatch = [.. exportBatch.Requests.OrderBy(x => x.PacketOffset).ThenBy(x => x.ExportIdx).Select(x => x.OriginalRequest)];
				await _requestBatchChannel.Writer.WriteAsync(chunkBatch, cancellationToken);
			}
		}

		static bool TryAddBundleRequest(BlobRequest<TUserData> request, Queue<BundleBatchRequest> bundleBatchQueue, Dictionary<BundleHandle, BundleBatchRequest> bundleHandleToBatch)
		{
			ExportHandle? exportHandle = request.Handle.Innermost as ExportHandle;
			if (exportHandle == null)
			{
				return false;
			}

			FlushedPacketHandle? packetHandle = exportHandle.Packet as FlushedPacketHandle;
			if (packetHandle == null)
			{
				return false;
			}

			BundleBatchRequest? bundleBatchRequest;
			if (!bundleHandleToBatch.TryGetValue(packetHandle.Bundle, out bundleBatchRequest))
			{
				bundleBatchRequest = new BundleBatchRequest(packetHandle.Bundle, []);
				bundleBatchQueue.Enqueue(bundleBatchRequest);
				bundleHandleToBatch.Add(packetHandle.Bundle, bundleBatchRequest);
			}

			BundleRequest bundleRequest = new(packetHandle.Bundle, packetHandle.PacketOffset, exportHandle.ExportIdx, request);
			bundleBatchRequest.Requests.Add(bundleRequest);
			return true;
		}

		async Task FetchLoopAsync(CancellationToken cancellationToken)
		{
			try
			{
				List<Task> tasks = [];
				for (int idx = 0; idx < _options.NumFetchTasks; idx++)
				{
					tasks.Add(Task.Run(() => FetchWorkerAsync(cancellationToken), cancellationToken));
				}
				await Task.WhenAll(tasks);

				_responseBatchChannel.Writer.TryComplete();
			}
			catch (Exception ex)
			{
				_responseBatchChannel.Writer.TryComplete(ex);
			}
		}

		async Task FetchWorkerAsync(CancellationToken cancellationToken)
		{
			while (await _requestBatchChannel.Reader.WaitToReadAsync(cancellationToken))
			{
				BlobRequest<TUserData>[]? requestBatch;
				if (_requestBatchChannel.Reader.TryRead(out requestBatch))
				{
					BlobResponse<TUserData>[] responseBatch = new BlobResponse<TUserData>[requestBatch.Length];
					for (int idx = 0; idx < requestBatch.Length; idx++)
					{
						BlobRequest<TUserData> request = requestBatch[idx];
						BlobData blobData = await requestBatch[idx].Handle.ReadBlobDataAsync(cancellationToken);
						BlobResponse<TUserData> response = new(blobData, request.UserData);
						responseBatch[idx] = response;
					}
#pragma warning disable CA2000
					await _responseBatchChannel.Writer.WriteAsync(new BlobResponseBatch<TUserData>(responseBatch), cancellationToken);
#pragma warning restore CA2000
				}
			}
		}

		/// <summary>
		/// Reads all responses from the reader
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async IAsyncEnumerable<BlobResponse<TUserData>> ReadAllAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			while (await WaitToReadAsync(cancellationToken))
			{
#pragma warning disable CA2000
				BlobResponseBatch<TUserData>? batch;
				if (TryReadBatch(out batch))
				{
					int idx = 0;
					try
					{
						for (; idx < batch.Responses.Length; idx++)
						{
							yield return batch.Responses[idx];
						}
					}
					finally
					{
						for (; idx < batch.Responses.Length; idx++)
						{
							batch.Responses[idx].Dispose();
						}
					}
				}
#pragma warning restore CA2000
			}
		}

		/// <summary>
		/// Attempts to read a batch from the queue
		/// </summary>
		/// <param name="batch">Batch of responses</param>
		public bool TryReadBatch([NotNullWhen(true)] out BlobResponseBatch<TUserData>? batch)
			=> _responseBatchChannel.Reader.TryRead(out batch);

		/// <summary>
		/// Waits until there is data available to read
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public ValueTask<bool> WaitToReadAsync(CancellationToken cancellationToken)
			=> _responseBatchChannel.Reader.WaitToReadAsync(cancellationToken);
	}
}
