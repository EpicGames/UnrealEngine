// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Storage.Nodes;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Handles serialization of blobs using <see cref="BlobConverter"/> instances.
	/// </summary>
	public static class BlobSerializer
	{
		/// <summary>
		/// Deserialize an object
		/// </summary>
		/// <typeparam name="T">Return type for deserialization</typeparam>
		/// <param name="blobData">Data to deserialize from</param>
		/// <param name="options">Options to control serialization</param>
		public static T Deserialize<T>(BlobData blobData, BlobSerializerOptions? options = null)
		{
			options ??= BlobSerializerOptions.Default;
			BlobReader reader = new(blobData, options);
			return options.GetConverter<T>().Read(reader, options);
		}

		/// <summary>
		/// Serialize an object into a blob
		/// </summary>
		/// <typeparam name="T">Type of object to serialize</typeparam>
		/// <param name="writer">Writer for the blob data</param>
		/// <param name="value">Object to serialize</param>
		/// <param name="options">Options to control serialization</param>
		/// <returns>Type of the serialized blob</returns>
		public static BlobType Serialize<T>(IBlobWriter writer, T value, BlobSerializerOptions? options = null)
		{
			options ??= BlobSerializerOptions.Default;
			return options.GetConverter<T>().Write(writer, value, options);
		}
	}

	/// <summary>
	/// Options for serializing blobs
	/// </summary>
	public class BlobSerializerOptions
	{
		class FreezableList<T> : IList<T>
		{
			readonly BlobSerializerOptions _owner;
			readonly List<T> _list;

			public FreezableList(BlobSerializerOptions owner)
			{
				_owner = owner;
				_list = [];
			}

			public T this[int index]
			{
				get => _list[index];
				set
				{
					_owner.CheckMutable();
					_list[index] = value;
				}
			}

			public int Count => _list.Count;
			public bool IsReadOnly => _owner._readOnly;

			public void Add(T item)
			{
				_owner.CheckMutable();
				_list.Add(item);
			}

			public void Clear()
			{
				_owner.CheckMutable();
				_list.Clear();
			}

			public bool Contains(T item) => _list.Contains(item);
			public void CopyTo(T[] array, int arrayIndex) => _list.CopyTo(array, arrayIndex);
			public IEnumerator<T> GetEnumerator() => _list.GetEnumerator();
			public int IndexOf(T item) => _list.IndexOf(item);

			public void Insert(int index, T item)
			{
				_owner.CheckMutable();
				_list.Insert(index, item);
			}

			public bool Remove(T item)
			{
				_owner.CheckMutable();
				return _list.Remove(item);
			}

			public void RemoveAt(int index)
			{
				_owner.CheckMutable();
				_list.RemoveAt(index);
			}

			IEnumerator IEnumerable.GetEnumerator() => _list.GetEnumerator();
		}

		readonly ConcurrentDictionary<Type, BlobConverter> _cachedConverters = new();
		static readonly ConcurrentDictionary<Type, BlobConverter> s_cachedDefaultConverters = new();
		static readonly Func<Type, BlobConverter> s_createDefaultConverter = CreateDefaultConverter;

		bool _readOnly;
		readonly FreezableList<BlobConverter> _converters;

		/// <summary>
		/// Known converter types
		/// </summary>
		public IList<BlobConverter> Converters => _converters;

		/// <summary>
		/// Default options instance
		/// </summary>
		public static BlobSerializerOptions Default { get; } = CreateDefaultOptions();

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobSerializerOptions()
		{
			_converters = new FreezableList<BlobConverter>(this);
		}

		void CheckMutable()
		{
			if (_readOnly)
			{
				throw new NotSupportedException("Options instance is read-only");
			}
		}

		static BlobSerializerOptions CreateDefaultOptions()
		{
			BlobSerializerOptions options = new();
			options.MakeReadOnly();
			return options;
		}

		/// <summary>
		/// Create a read-only version of these options
		/// </summary>
		/// <returns></returns>
		public void MakeReadOnly() => _readOnly = true;

		/// <summary>
		/// Gets a converter for the given type
		/// </summary>
		public BlobConverter<T> GetConverter<T>()
		{
			return (BlobConverter<T>)_cachedConverters.GetOrAdd(typeof(T), CreateConverter);
		}

		BlobConverter CreateConverter(Type type)
		{
			foreach (BlobConverter converter in Converters)
			{
				if (converter.CanConvert(type))
				{
					return converter;
				}
			}
			return s_cachedDefaultConverters.GetOrAdd(type, s_createDefaultConverter);
		}

		static BlobConverter CreateDefaultConverter(Type type)
		{
			BlobConverterAttribute? attribute = type.GetCustomAttribute<BlobConverterAttribute>();
			if (attribute != null)
			{
				Type converterType = attribute.ConverterType;
				if (converterType.IsGenericTypeDefinition)
				{
					converterType = converterType.MakeGenericType(type.GetGenericArguments());
				}
				return (BlobConverter)Activator.CreateInstance(converterType)!;
			}
			throw new NotSupportedException($"No converter is available to handle type {type.Name}");
		}

		/// <summary>
		/// Creates options for serializing blobs compatible with a particular server API version
		/// </summary>
		/// <param name="version">The server API version</param>
		public static BlobSerializerOptions Create(HordeApiVersion version)
		{
			BlobSerializerOptions options = new();
			options.Converters.Add(new InteriorChunkedDataNodeConverter(version));
			options.Converters.Add(new DirectoryNodeConverter(version));
			return options;
		}
	}

	/// <summary>
	/// Extension methods for serializing blob types
	/// </summary>
	public static class BlobSerializerExtensions
	{
		/// <summary>
		/// Deserialize an object
		/// </summary>
		/// <typeparam name="T">Return type for the deserialized object</typeparam>
		/// <param name="handle">Handle to the blob to deserialize</param>
		/// <param name="options">Options to control serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask<T> ReadBlobAsync<T>(this IBlobRef handle, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default)
		{
			using BlobData data = await handle.ReadBlobDataAsync(cancellationToken);
			return BlobSerializer.Deserialize<T>(data, options);
		}

		/// <summary>
		/// Deserialize an object
		/// </summary>
		/// <typeparam name="T">Return type for the deserialized object</typeparam>
		/// <param name="handle">Handle to the blob to deserialize</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask<T> ReadBlobAsync<T>(this IBlobRef<T> handle, CancellationToken cancellationToken = default)
		{
			using BlobData data = await handle.ReadBlobDataAsync(cancellationToken);
			return BlobSerializer.Deserialize<T>(data, handle.SerializerOptions);
		}

		/// <summary>
		/// Deserialize an object
		/// </summary>
		/// <typeparam name="T">Return type for the deserialized object</typeparam>
		/// <param name="handle">Handle to the blob to deserialize</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask<T> ReadBlobAsync<T>(this IHashedBlobRef<T> handle, CancellationToken cancellationToken = default)
		{
			using BlobData data = await handle.ReadBlobDataAsync(cancellationToken);
			return BlobSerializer.Deserialize<T>(data, handle.SerializerOptions);
		}

		/// <summary>
		/// Serialize an object to storage
		/// </summary>
		/// <param name="writer">Writer for serialized data</param>
		/// <param name="value">The object to serialize</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the serialized blob</returns>
		public static async ValueTask<IHashedBlobRef<T>> WriteBlobAsync<T>(this IBlobWriter writer, T value, CancellationToken cancellationToken = default)
		{
			BlobType blobType = BlobSerializer.Serialize<T>(writer, value, writer.Options);
			return await writer.CompleteAsync<T>(blobType, cancellationToken);
		}

		/// <summary>
		/// Reads data for a ref from the store, along with the node's contents.
		/// </summary>
		/// <param name="store">Store instance to write to</param>
		/// <param name="name">The ref name</param>
		/// <param name="cacheTime">Minimum coherency for any cached value to be returned</param>
		/// <param name="options">Options to control serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Node for the given ref, or null if it does not exist</returns>
		public static async Task<IHashedBlobRef<TNode>?> TryReadRefAsync<TNode>(this IStorageNamespace store, RefName name, DateTime cacheTime = default, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default) where TNode : class
		{
			IHashedBlobRef? refTarget = await store.TryReadRefAsync(name, cacheTime, cancellationToken);
			if (refTarget == null)
			{
				return null;
			}
			return HashedBlobRef.Create<TNode>(refTarget.Hash, refTarget, options ?? BlobSerializerOptions.Default);
		}

		/// <summary>
		/// Reads a ref from the store, throwing an exception if it does not exist
		/// </summary>
		/// <param name="store">Store instance to write to</param>
		/// <param name="name">Id for the ref</param>
		/// <param name="cacheTime">Minimum coherency of any cached result</param>
		/// <param name="options">Options to control serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The blob instance</returns>
		public static async Task<IHashedBlobRef<TNode>> ReadRefAsync<TNode>(this IStorageNamespace store, RefName name, DateTime cacheTime = default, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default) where TNode : class
		{
			IHashedBlobRef<TNode>? refValue = await store.TryReadRefAsync<TNode>(name, cacheTime, options, cancellationToken);
			if (refValue == null)
			{
				throw new RefNameNotFoundException(name);
			}
			return refValue;
		}

		/// <summary>
		/// Reads data for a ref from the store, along with the node's contents.
		/// </summary>
		/// <param name="store">Store instance to write to</param>
		/// <param name="name">The ref name</param>
		/// <param name="cacheTime">Minimum coherency for any cached value to be returned</param>
		/// <param name="options">Options to control serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Node for the given ref, or null if it does not exist</returns>
		public static async Task<TNode?> TryReadRefTargetAsync<TNode>(this IStorageNamespace store, RefName name, DateTime cacheTime = default, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default) where TNode : class
		{
			IHashedBlobRef<TNode>? refTarget = await store.TryReadRefAsync<TNode>(name, cacheTime, options, cancellationToken);
			if (refTarget == null)
			{
				return null;
			}
			return await refTarget.ReadBlobAsync(cancellationToken);
		}

		/// <summary>
		/// Reads a ref from the store, throwing an exception if it does not exist
		/// </summary>
		/// <param name="store">Store instance to write to</param>
		/// <param name="name">Id for the ref</param>
		/// <param name="cacheTime">Minimum coherency of any cached result</param>
		/// <param name="options">Options to control serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The blob instance</returns>
		public static async Task<TNode> ReadRefTargetAsync<TNode>(this IStorageNamespace store, RefName name, DateTime cacheTime = default, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default) where TNode : class
		{
			IHashedBlobRef<TNode> blobRef = await ReadRefAsync<TNode>(store, name, cacheTime, options, cancellationToken);
			return await blobRef.ReadBlobAsync(cancellationToken);
		}
	}
}
