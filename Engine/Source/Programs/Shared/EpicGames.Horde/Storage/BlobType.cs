// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Identifies the type of a blob
	/// </summary>
	/// <param name="Guid">Nominal identifier for the type</param>
	/// <param name="Version">Version number for the serializer</param>
	public readonly record struct BlobType(Guid Guid, int Version)
	{
		/// <summary>
		/// Number of bytes in a serialized blob type instance
		/// </summary>
		public const int NumBytes = 20;

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobType(string guid, int version)
			: this(Guid.Parse(guid), version)
		{
		}

		/// <summary>
		/// Deserialize a type from a byte span
		/// </summary>
		public static BlobType Read(ReadOnlySpan<byte> span)
		{
			Guid guid = GuidUtils.ReadGuidUnrealOrder(span);
			int version = BinaryPrimitives.ReadInt32LittleEndian(span.Slice(16));

			return new BlobType(guid, version);
		}

		/// <summary>
		/// Serialize to a byte span
		/// </summary>
		public readonly void Write(Span<byte> data)
		{
			GuidUtils.WriteGuidUnrealOrder(data, Guid);
			BinaryPrimitives.WriteInt32LittleEndian(data.Slice(16), Version);
		}

		/// <inheritdoc/>
		public override readonly string ToString() => $"{Guid},{Version}";
	}
}
