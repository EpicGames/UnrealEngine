// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.IO;

namespace EpicGames.Horde.Storage.Bundles
{
	/// <summary>
	/// Bundle version number
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1027:Mark enums with FlagsAttribute")]
	public enum BundleVersion
	{
		/// <summary>
		/// Initial version number
		/// </summary>
		Initial = 0,

		/// <summary>
		/// Added the BundleExport.Alias property
		/// </summary>
		ExportAliases = 1,

		/// <summary>
		/// Back out change to include aliases. Will likely do this through an API rather than baked into the data. 
		/// </summary>
		RemoveAliases = 2,

		/// <summary>
		/// Use data structures which support in-place reading and writing.
		/// </summary>
		InPlace = 3,

		/// <summary>
		/// Add import hashes to imported nodes
		/// </summary>
		ImportHashes = 4,

		/// <summary>
		/// Structure bundles as a sequence of self-contained packets (uses V2 code)
		/// </summary>
		PacketSequence = 5,

		/// <summary>
		/// Last item in the enum. Used for <see cref="Latest"/>
		/// </summary>
		LatestPlusOne,

#pragma warning disable CA1069 // Enums values should not be duplicated
		/// <summary>
		/// The current version number
		/// </summary>
		Latest = (int)LatestPlusOne - 1,

		/// <summary>
		/// Last version using the V1 pipeline
		/// </summary>
		LatestV1 = ImportHashes,

		/// <summary>
		/// Last version using the V2 pipeline
		/// </summary>
		LatestV2 = Latest,
#pragma warning restore CA1069 // Enums values should not be duplicated
	}

	/// <summary>
	/// Signature for a bundle
	/// </summary>
	/// <param name="Version">Version number for the following file data</param>
	/// <param name="HeaderLength">Length of the initial header</param>
	public readonly record struct BundleSignature(BundleVersion Version, int HeaderLength)
	{
		/// <summary>
		/// Number of bytes in a signature when serialized
		/// </summary>
		public const int NumBytes = 8;

		/// <summary>
		/// Validates that the prelude bytes for a bundle header are correct
		/// </summary>
		/// <param name="span">The signature bytes</param>
		/// <returns>Length of the header data, including the prelude</returns>
		public static BundleSignature Read(ReadOnlySpan<byte> span)
		{
			if (span[0] == 'U' && span[1] == 'E' && span[2] == 'B' && span[3] == 'N')
			{
				return new BundleSignature(BundleVersion.Initial, BinaryPrimitives.ReadInt32BigEndian(span.Slice(4)));
			}
			else if (span[0] == 'U' && span[1] == 'B' && span[2] == 'N')
			{
				return new BundleSignature((BundleVersion)span[3], BinaryPrimitives.ReadInt32LittleEndian(span.Slice(4)));
			}
			else
			{
				throw new InvalidDataException("Invalid signature bytes for bundle. Corrupt data?");
			}
		}

		/// <summary>
		/// Writes a signature to the given memory
		/// </summary>
		public void Write(Span<byte> span)
		{
			if (Version == BundleVersion.Initial)
			{
				span[0] = (byte)'U';
				span[1] = (byte)'E';
				span[2] = (byte)'B';
				span[3] = (byte)'N';
				BinaryPrimitives.WriteInt32BigEndian(span[4..], HeaderLength);
			}
			else
			{
				span[0] = (byte)'U';
				span[1] = (byte)'B';
				span[2] = (byte)'N';
				span[3] = (byte)Version;
				BinaryPrimitives.WriteInt32LittleEndian(span[4..], HeaderLength);
			}
		}
	}
}
