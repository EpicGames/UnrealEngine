// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using BitFaster.Caching;
using BitFaster.Caching.Lru;
using EpicGames.Core;
using EpicGames.Horde.Storage.Bundles.V2;
using Microsoft.Extensions.Caching.Memory;

namespace EpicGames.Horde.Storage.Bundles
{
	/// <summary>
	/// Options for creating a storage cache
	/// </summary>
	public class BundleCacheOptions
	{
		/// <summary>
		/// Number of packet readers to keep in the cache
		/// </summary>
		public int PacketReaderCount { get; set; } = 200;

		/// <summary>
		/// Size of a bundle page
		/// </summary>
		public int BundlePageSize { get; set; } = 1024 * 1024;

		/// <summary>
		/// Number of bundle pages to keep in the cache.
		/// </summary>
		public int BundlePageCount { get; set; } = 500;

		/// <summary>
		/// Size of the header cache
		/// </summary>
		public long HeaderCacheSize { get; set; } = 64 * 1024 * 1024;

		/// <summary>
		/// Size of the packet cache
		/// </summary>
		public long PacketCacheSize { get; set; } = 192 * 1024 * 1024;
	}

	readonly record struct BundlePageCacheKey(BundleHandle Bundle, int Index)
	{
		public override string ToString()
			=> $"bundle-page:{Bundle}:{Index}";
	}

	/// <summary>
	/// Cache for reading bundle data.
	/// </summary>
	public sealed class BundleCache : IAsyncDisposable
	{
		readonly BundleCacheOptions _options;
		readonly CancellationTokenSource _cancellationSource = new();

		readonly IScopedAsyncCache<PacketReaderCacheKey, PacketReader> _packetReaderCache;
		readonly IScopedAsyncCache<BundlePageCacheKey, IReadOnlyMemoryOwner<byte>> _bundlePageCache;
		readonly MemoryCache? _headerCache;
		readonly MemoryCache? _packetCache;
#pragma warning disable CA2213
		readonly SemaphoreSlim _writeSemaphore;
#pragma warning restore CA2213

		internal IScopedAsyncCache<PacketReaderCacheKey, PacketReader> PacketReaderCache => _packetReaderCache;
		internal IScopedAsyncCache<BundlePageCacheKey, IReadOnlyMemoryOwner<byte>> BundlePageCache => _bundlePageCache;

		/// <summary>
		/// Instance of an empty cache
		/// </summary>
		public static BundleCache None { get; } = new BundleCache(new BundleCacheOptions { HeaderCacheSize = 0, PacketCacheSize = 0 });

		/// <summary>
		/// Accessor for the default allocator
		/// </summary>
		public IMemoryAllocator<byte> Allocator { get; }

		/// <summary>
		/// Size of a bundle page to keep in the cache
		/// </summary>
		public int BundlePageSize => _options.BundlePageSize;

		/// <summary>
		/// Size of the configured header cache
		/// </summary>
		public long HeaderCacheSize => _options.HeaderCacheSize;

		/// <summary>
		/// Size of the configured packet cache
		/// </summary>
		public long PacketCacheSize => _options.PacketCacheSize;

		/// <summary>
		/// Whether there is a packet cache present
		/// </summary>
		public bool HasPacketCache => _packetCache != null;

		/// <summary>
		/// Semaphore for writing new data
		/// </summary>
		public SemaphoreSlim WriteSemaphore => _writeSemaphore;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleCache()
			: this(new BundleCacheOptions())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="options">Options for the cache</param>
		public BundleCache(BundleCacheOptions options)
			: this(options, PoolAllocator.Shared)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="options">Options for the cache</param>
		/// <param name="innerAllocator">Inner allocator to use. Will be wrapped in an allocator that tracks allocations against the cache's budget.</param>
		public BundleCache(BundleCacheOptions options, IMemoryAllocator<byte> innerAllocator)
		{
			_options = options;
#if DEBUG
			innerAllocator = new TrackingMemoryAllocator(innerAllocator);
#endif
			Allocator = innerAllocator;

			_packetReaderCache = new ConcurrentLruBuilder<PacketReaderCacheKey, PacketReader>()
				.WithCapacity(_options.PacketReaderCount)
				.WithAtomicGetOrAdd()
#if TRACE
				.WithMetrics()
#endif
				.AsAsyncCache()
				.AsScopedCache()
				.Build();

			_bundlePageCache = new ConcurrentLruBuilder<BundlePageCacheKey, IReadOnlyMemoryOwner<byte>>()
				.WithCapacity(_options.BundlePageCount)
				.WithAtomicGetOrAdd()
#if TRACE
				.WithMetrics()
#endif
				.AsAsyncCache()
				.AsScopedCache()
				.Build();

			if (options.HeaderCacheSize > 0)
			{
				_headerCache = new MemoryCache(new MemoryCacheOptions { SizeLimit = options.HeaderCacheSize });
			}
			if (options.PacketCacheSize > 0)
			{
				_packetCache = new MemoryCache(new MemoryCacheOptions { SizeLimit = options.PacketCacheSize });
			}

			_writeSemaphore = new SemaphoreSlim(64);
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
		{
			WriteSemaphore.Dispose();

			_packetReaderCache.Clear();
			_bundlePageCache.Clear();

			_headerCache?.Dispose();
			_packetCache?.Dispose();

			_cancellationSource.Dispose();

			return default;
		}

		/// <summary>
		/// Gets stats for the cache
		/// </summary>
		public void GetStats(StorageStats stats)
		{
			Optional<ICacheMetrics> packetMetrics = PacketReaderCache.Metrics;
			if (packetMetrics.Value != null)
			{
				stats.Add("bundle.packet_cache.hits", packetMetrics.Value.Hits);
				stats.Add("bundle.packet_cache.misses", packetMetrics.Value.Misses);
			}

			Optional<ICacheMetrics> bundleMetrics = BundlePageCache.Metrics;
			if (bundleMetrics.Value != null)
			{
				stats.Add("bundle.bundle_cache.hits", bundleMetrics.Value.Hits);
				stats.Add("bundle.bundle_cache.misses", bundleMetrics.Value.Misses);
			}
		}

		#region V1

		static void AddCachedValue(IMemoryCache? cache, string cacheKey, object value, int size)
		{
			if (cache != null)
			{
				using (ICacheEntry entry = cache.CreateEntry(cacheKey))
				{
					entry.SetValue(value);
					entry.SetSize(size);
				}
			}
		}

		static bool TryGetCachedValue<T>(IMemoryCache? cache, string cacheKey, [MaybeNull, NotNullWhen(true)] out T value)
		{
			if (cache == null)
			{
				value = default;
				return false;
			}
			return cache.TryGetValue(cacheKey, out value);
		}

		static string GetBundleInfoCacheKey(BlobLocator locator) => $"bundle:{locator}";
		static string GetEncodedPacketCacheKey(BlobLocator locator, int packetIdx) => $"encoded-packet:{locator}#{packetIdx}";
		static string GetDecodedPacketCacheKey(BlobLocator locator, int packetIdx) => $"decoded-packet:{locator}#{packetIdx}";

		/// <summary>
		/// Adds a bundle info object to the cache
		/// </summary>
		public void AddCachedHeader(BlobLocator locator, Bundles.V1.BundleInfo bundleInfo)
		{
			AddCachedValue(_headerCache, GetBundleInfoCacheKey(locator), bundleInfo, bundleInfo.HeaderLength);
		}

		/// <summary>
		/// Try to read a bundle info object from the cache
		/// </summary>
		public bool TryGetCachedHeader(BlobLocator locator, [NotNullWhen(true)] out Bundles.V1.BundleInfo? bundleInfo)
		{
			return TryGetCachedValue(_headerCache, GetBundleInfoCacheKey(locator), out bundleInfo);
		}

		/// <summary>
		/// Adds an encoded bundle packet to the cache
		/// </summary>
		public void AddCachedEncodedPacket(BlobLocator locator, int packetIdx, ReadOnlyMemory<byte> data)
		{
			AddCachedValue(_packetCache, GetEncodedPacketCacheKey(locator, packetIdx), data, data.Length);
		}

		/// <summary>
		/// Try to read an encoded bundle packet from the cache
		/// </summary>
		public bool TryGetCachedEncodedPacket(BlobLocator locator, int packetIdx, out ReadOnlyMemory<byte> data)
		{
			return TryGetCachedValue(_packetCache, GetEncodedPacketCacheKey(locator, packetIdx), out data);
		}

		/// <summary>
		/// Adds a decoded bundle packet to the cache
		/// </summary>
		public void AddCachedDecodedPacket(BlobLocator locator, int packetIdx, ReadOnlyMemory<byte> data)
		{
			AddCachedValue(_packetCache, GetDecodedPacketCacheKey(locator, packetIdx), data, data.Length);
		}

		/// <summary>
		/// Try to read a decoded bundle packet from the cache
		/// </summary>
		public bool TryGetCachedDecodedPacket(BlobLocator locator, int packetIdx, out ReadOnlyMemory<byte> data)
		{
			return TryGetCachedValue(_packetCache, GetDecodedPacketCacheKey(locator, packetIdx), out data);
		}

		#endregion
	}
}
