// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.Compression;
using EpicGames.Compression;
using EpicGames.Core;
using K4os.Compression.LZ4;

namespace EpicGames.Horde.Storage.Bundles
{
	using ZstdCompressor = ZstdSharp.Compressor;
	using ZstdDecompressor = ZstdSharp.Decompressor;

	/// <summary>
	/// Indicates the compression format in the bundle
	/// </summary>
	public enum BundleCompressionFormat
	{
		/// <summary>
		/// Packets are uncompressed
		/// </summary>
		None = 0,

		/// <summary>
		/// LZ4 compression
		/// </summary>
		LZ4 = 1,

		/// <summary>
		/// Gzip compression
		/// </summary>
		Gzip = 2,

		/// <summary>
		/// Oodle compression (Selkie)
		/// </summary>
		Oodle = 3,

		/// <summary>
		/// Brotli compression
		/// </summary>
		Brotli = 4,

		/// <summary>
		/// ZStandard compression
		/// </summary>
		Zstd = 5,
	}

	/// <summary>
	/// Utility methods for bundles
	/// </summary>
	public static class BundleData
	{
		/// <summary>
		/// Compress a block of data into a newly allocated block of memory
		/// </summary>
		/// <param name="format">Format for the compressed data</param>
		/// <param name="input">The data to compress</param>
		/// <returns>The compressed data</returns>
		public static byte[] Compress(BundleCompressionFormat format, ReadOnlyMemory<byte> input)
		{
			ArrayMemoryWriter writer = new(0);
			Compress(format, input, writer);
			return writer.WrittenMemory.ToArray();
		}

		/// <summary>
		/// Compress a data packet
		/// </summary>
		/// <param name="format">Format for the compressed data</param>
		/// <param name="input">The data to compress</param>
		/// <param name="writer">Writer for output data</param>
		/// <returns>Length of the compressed data</returns>
		public static int Compress(BundleCompressionFormat format, ReadOnlyMemory<byte> input, IMemoryWriter writer)
		{
			switch (format)
			{
				case BundleCompressionFormat.None:
					{
						writer.WriteFixedLengthBytes(input.Span);
						return input.Length;
					}
				case BundleCompressionFormat.LZ4:
					{
						int maxSize = LZ4Codec.MaximumOutputSize(input.Length);

						Span<byte> buffer = writer.GetSpan(maxSize);
						int encodedLength = LZ4Codec.Encode(input.Span, buffer);

						writer.Advance(encodedLength);
						return encodedLength;
					}
				case BundleCompressionFormat.Gzip:
					{
						using MemoryStream outputStream = new(input.Length);

						using (GZipStream gzipStream = new(outputStream, CompressionLevel.Fastest, true))
						{
							gzipStream.Write(input.Span);
						}

						writer.WriteFixedLengthBytes(outputStream.ToArray());
						return (int)outputStream.Length;
					}
				case BundleCompressionFormat.Oodle:
					{
						int maxSize = Oodle.MaximumOutputSize(OodleCompressorType.Selkie, input.Length);

						Span<byte> outputSpan = writer.GetSpan(maxSize);
						int encodedLength = Oodle.Compress(OodleCompressorType.Selkie, input.Span, outputSpan, OodleCompressionLevel.Normal);
						writer.Advance(encodedLength);

						return encodedLength;
					}
				case BundleCompressionFormat.Brotli:
					{
						int maxSize = BrotliEncoder.GetMaxCompressedLength(input.Length);

						Span<byte> buffer = writer.GetSpan(maxSize);
						if (!BrotliEncoder.TryCompress(input.Span, buffer, out int encodedLength))
						{
							throw new InvalidOperationException("Unable to compress data using Brotli");
						}

						writer.Advance(encodedLength);
						return encodedLength;
					}
				case BundleCompressionFormat.Zstd:
					{
						int maxSize = ZstdCompressor.GetCompressBound(input.Length);

						using ZstdCompressor compressor = new();

						Span<byte> buffer = writer.GetSpan(maxSize);
						if (!compressor.TryWrap(input.Span, buffer, out int encodedLength))
						{
							throw new InvalidOperationException("Unable to compress data using Zstd");
						}

						writer.Advance(encodedLength);
						return encodedLength;
					}
				default:
					throw new InvalidDataException($"Invalid compression format '{(int)format}'");
			}
		}

		/// <summary>
		/// Decompress a packet of data
		/// </summary>
		/// <param name="format">Format of the compressed data</param>
		/// <param name="input">Compressed data</param>
		/// <param name="output">Buffer to receive the decompressed data</param>
		public static void Decompress(BundleCompressionFormat format, ReadOnlyMemory<byte> input, Memory<byte> output)
		{
			switch (format)
			{
				case BundleCompressionFormat.None:
					input.CopyTo(output);
					break;
				case BundleCompressionFormat.LZ4:
					{
						int length = LZ4Codec.Decode(input.Span, output.Span);
						if (length != output.Length)
						{
							throw new InvalidDataException($"Decoded data is shorter than expected (expected {output.Length} bytes, got {length} bytes)");
						}
						break;
					}
				case BundleCompressionFormat.Gzip:
					{
						using ReadOnlyMemoryStream inputStream = new(input);
						using GZipStream inflatedStream = new(inputStream, CompressionMode.Decompress, true);

						int length = 0;
						for (; ; )
						{
							int count = inflatedStream.Read(output.Span.Slice(length));
							if (count == 0)
							{
								break;
							}

							length += count;
						}

						if (length != output.Length)
						{
							throw new InvalidDataException($"Decoded data is shorter than expected (expected {output.Length} bytes, got {length} bytes)");
						}
						break;
					}
				case BundleCompressionFormat.Oodle:
#if WITH_OODLE
					Oodle.Decompress(input.Span, output.Span);
					break;
#else
					throw new NotSupportedException("Oodle is not compiled into this build.");
#endif
				case BundleCompressionFormat.Brotli:
					{
						int bytesWritten;
						if (!BrotliDecoder.TryDecompress(input.Span, output.Span, out bytesWritten) || bytesWritten != output.Length)
						{
							throw new InvalidOperationException("Unable to decompress data using Brotli");
						}
						break;
					}
				case BundleCompressionFormat.Zstd:
					{
						using ZstdDecompressor decompressor = new();
						if (!decompressor.TryUnwrap(input.Span, output.Span, out int bytesWritten) || bytesWritten != output.Length)
						{
							throw new InvalidOperationException("Unable to decompress data using Zstd");
						}
						break;
					}
				default:
					throw new InvalidDataException($"Invalid compression format '{(int)format}'");
			}
		}
	}
}
