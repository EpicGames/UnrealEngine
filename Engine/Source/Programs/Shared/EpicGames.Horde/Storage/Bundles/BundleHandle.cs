// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles
{
	/// <summary>
	/// Base class for packet handles
	/// </summary>
	public abstract class BundleHandle
	{
		/// <summary>
		/// Locator for this bundle
		/// </summary>
		public BlobLocator Locator { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		protected BundleHandle(BlobLocator locator)
			=> Locator = locator;

		/// <summary>
		/// Flush contents of the bundle to storage
		/// </summary>
		public abstract ValueTask FlushAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Open the bundle for reading
		/// </summary>
		public abstract Task<Stream> OpenAsync(int offset = 0, int? length = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get data for a packet of data from the bundle
		/// </summary>
		public abstract ValueTask<IReadOnlyMemoryOwner<byte>> ReadAsync(int offset, int? length, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to get a locator for this bundle
		/// </summary>
		public bool TryGetLocator(out BlobLocator locator)
		{
			locator = Locator;
			return true;
		}
	}

	/// <summary>
	/// Generic flushed bundle handle; can be either V1 or V2 format.
	/// </summary>
	class FlushedBundleHandle : BundleHandle
	{
		readonly BundleStorageNamespace _storageNamespace;

		/// <summary>
		/// Constructor
		/// </summary>
		public FlushedBundleHandle(BundleStorageNamespace storageNamespace, BlobLocator locator)
			: base(locator)
		{
			_storageNamespace = storageNamespace;
		}

		/// <inheritdoc/>
		public override ValueTask FlushAsync(CancellationToken cancellationToken = default) => default;

		/// <inheritdoc/>
		public override Task<Stream> OpenAsync(int offset = 0, int? length = null, CancellationToken cancellationToken = default)
			=> _storageNamespace.Backend.OpenBlobAsync(Locator, offset, length, cancellationToken);

		/// <inheritdoc/>
		public override async ValueTask<IReadOnlyMemoryOwner<byte>> ReadAsync(int offset, int? length, CancellationToken cancellationToken = default)
			=> await _storageNamespace.Backend.ReadBlobAsync(Locator, offset, length, cancellationToken);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is FlushedBundleHandle other && Locator == other.Locator;

		/// <inheritdoc/>
		public override int GetHashCode() => Locator.GetHashCode();

		/// <inheritdoc/>
		public override string ToString() => Locator.ToString();
	}
}
