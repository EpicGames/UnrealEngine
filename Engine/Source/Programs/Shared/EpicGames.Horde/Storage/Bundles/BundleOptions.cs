// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Horde.Storage.Bundles
{
	/// <summary>
	/// Options for configuring a bundle serializer
	/// </summary>
	public class BundleOptions
	{
		/// <summary>
		/// Default options value
		/// </summary>
		public static BundleOptions Default { get; } = new BundleOptions();

		/// <summary>
		/// Maximum version number of bundles to write
		/// </summary>
		public BundleVersion MaxVersion { get; set; } = BundleVersion.LatestV2;

		/// <summary>
		/// Maximum payload size fo a blob
		/// </summary>
		public int MaxBlobSize { get; set; } = 10 * 1024 * 1024;

		/// <summary>
		/// Compression format to use
		/// </summary>
		public BundleCompressionFormat CompressionFormat { get; set; } = BundleCompressionFormat.LZ4;

		/// <summary>
		/// Minimum size of a block to be compressed
		/// </summary>
		public int MinCompressionPacketSize { get; set; } = 16 * 1024;

		/// <summary>
		/// Maximum amount of data to store in memory. This includes any background writes as well as bundles being built.
		/// </summary>
		public long MaxWriteQueueLength { get; set; } = 256 * 1024 * 1024;
	}
}
