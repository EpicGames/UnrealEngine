// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Text;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles.V1;
using EpicGames.Horde.Storage.Bundles.V2;
using EpicGames.Horde.Storage.ObjectStores;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Bundles
{
	/// <summary>
	/// Base class for an implementation of <see cref="IStorageNamespace"/>, providing implementations for some common functionality using bundles.
	/// </summary>
	public sealed class BundleStorageNamespace : IStorageNamespace
	{
		class StorageWriterImpl : StorageWriter
		{
			readonly BundleStorageNamespace _outer;

			public StorageWriterImpl(BundleStorageNamespace outer, IStorageBackend storageBackend, CancellationToken cancellationToken)
				: base(storageBackend, cancellationToken)
			{
				_outer = outer;
			}

			protected override IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default)
				=> _outer.CreateBlobWriter(basePath, serializerOptions, cancellationToken);
		}

		readonly IStorageBackend _backend;
		readonly BundleCache _cache;
		readonly BundleOptions _options;
		readonly PacketReaderStats _packetReaderStats = new();
		readonly Bundles.V1.BundleReader _bundleReader;
		readonly ILogger _logger;

		internal Bundles.V1.BundleReader BundleReader => _bundleReader;

		/// <summary>
		/// Allocator which trims the cache to keep below a maximum size
		/// </summary>
		public IMemoryAllocator<byte> Allocator => _cache.Allocator;

		/// <summary>
		/// Accessor for the storage backend
		/// </summary>
		public IStorageBackend Backend => _backend;

		/// <summary>
		/// Cache for bundle data
		/// </summary>
		public BundleCache Cache => _cache;

		/// <summary>
		/// Stats for reading bundles
		/// </summary>
		internal PacketReaderStats PacketReaderStats => _packetReaderStats;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleStorageNamespace(IStorageBackend backend, BundleCache cache, BundleOptions? options, ILogger logger)
		{
			_backend = backend;
			_cache = cache;
			_options = options ?? BundleOptions.Default;
			_bundleReader = new Bundles.V1.BundleReader(this, cache, logger);
			_logger = logger;
		}

		/// <summary>
		/// Creates a bundle storage namespace around a memory backend
		/// </summary>
		public static BundleStorageNamespace CreateInMemory(ILogger logger)
			=> CreateInMemory(null, logger);

		/// <summary>
		/// Creates a bundle storage namespace around a memory backend
		/// </summary>
		public static BundleStorageNamespace CreateInMemory(BundleOptions? options, ILogger logger)
		{
			MemoryStorageBackend backend = new();
			return new BundleStorageNamespace(backend, BundleCache.None, options, logger);
		}

		/// <summary>
		/// Creates a bundle storage namespace around a directory on the filesystem
		/// </summary>
		public static BundleStorageNamespace CreateFromDirectory(DirectoryReference rootDir, BundleCache cache, MemoryMappedFileCache memoryMappedFileCache, ILogger logger)
			=> CreateFromDirectory(rootDir, cache, memoryMappedFileCache, null, logger);

		/// <summary>
		/// Creates a bundle storage namespace around a directory on the filesystem
		/// </summary>
		public static BundleStorageNamespace CreateFromDirectory(DirectoryReference rootDir, BundleCache cache, MemoryMappedFileCache memoryMappedFileCache, BundleOptions? options, ILogger logger)
		{
			FileStorageBackend backend = new(new FileObjectStore(rootDir, memoryMappedFileCache), logger);
			return new BundleStorageNamespace(backend, cache, options, logger);
		}

		/// <summary>
		/// Helper method for GC which allows enumerating all references to other bundles
		/// </summary>
		public async Task<IEnumerable<BlobLocator>> ReadBundleReferencesAsync(BlobLocator locator, CancellationToken cancellationToken)
		{
			using IReadOnlyMemoryOwner<byte> data = await Backend.ReadBlobAsync(locator, 0, 32 * 1024 * 1024, cancellationToken);
			return ReadBundleReferences(data.Memory);
		}

		/// <summary>
		/// Helper method for GC which allows enumerating all references to other bundles
		/// </summary>
		IEnumerable<BlobLocator> ReadBundleReferences(ReadOnlyMemory<byte> data)
		{
			BundleSignature signature = BundleSignature.Read(data.Span);
			if (signature.Version <= BundleVersion.LatestV1)
			{
				return ReadImportsFromDataV1(data);
			}
			else if (signature.Version <= BundleVersion.LatestV2)
			{
				return ReadImportsFromDataV2(data);
			}
			else
			{
				throw new InvalidOperationException($"Unsupported bundle version {(int)signature.Version}");
			}
		}

		static IEnumerable<BlobLocator> ReadImportsFromDataV1(ReadOnlyMemory<byte> data)
		{
			BundleHeader header = BundleHeader.Read(data);
			return header.Imports.Select(x => x.BaseLocator);
		}

		IEnumerable<BlobLocator> ReadImportsFromDataV2(ReadOnlyMemory<byte> data)
		{
			HashSet<BlobLocator> locators = [];
			while (data.Length > 0)
			{
				BundleSignature signature = BundleSignature.Read(data.Span);

				using IRefCountedHandle<Bundles.V2.Packet> packet = Packet.Decode(data, Cache.Allocator, nameof(ReadImportsFromDataV2));
				for (int idx = 0; idx < packet.Target.GetImportCount(); idx++)
				{
					PacketImport import = packet.Target.GetImport(idx);
					if (import.BaseIdx == -1)
					{
						locators.Add(new BlobLocator(import.Fragment.Clone()));
					}
				}

				data = data.Slice(signature.HeaderLength);
			}
			return locators;
		}

		/// <inheritdoc/>
		public IStorageWriter CreateWriter(CancellationToken cancellationToken)
			=> new StorageWriterImpl(this, _backend, cancellationToken);

		#region Blobs

		/// <inheritdoc/>
		public IBlobRef CreateBlobRef(BlobLocator locator)
		{
			if (!locator.TryUnwrap(out BlobLocator baseLocator, out Utf8String fragment))
			{
				throw new Exception($"{locator} is not valid");
			}

			FlushedBundleHandle bundleHandle = new(this, baseLocator);

			int exportIdx;
			if (Utf8Parser.TryParse(fragment.Span, out exportIdx, out int numBytesRead) && numBytesRead == fragment.Length)
			{
				return new Bundles.V1.FlushedNodeHandle(_bundleReader, baseLocator, bundleHandle, exportIdx);
			}

			int ampIdx = fragment.IndexOf('&');
			if (ampIdx == -1)
			{
				throw new Exception($"{locator} is not valid");
			}

			Bundles.V2.PacketHandle packetHandle = new Bundles.V2.FlushedPacketHandle(this, bundleHandle, fragment.Slice(0, ampIdx).Span, _cache);
			return new Bundles.V2.ExportHandle(packetHandle, fragment.Slice(ampIdx + 1));
		}

		/// <inheritdoc/>
		public IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default)
		{
			BundleVersion version = _options.MaxVersion;
			if (version == BundleVersion.LatestV1)
			{
				return new Bundles.V1.BundleWriter(this, _bundleReader, basePath, _options);
			}
			else if (version == BundleVersion.LatestV2)
			{
				return new Bundles.V2.BundleWriter(this, basePath, _cache, _options, serializerOptions, _logger, cancellationToken);
			}
			else
			{
				throw new InvalidOperationException($"Unsupported bundle version: {(int)version}");
			}
		}

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public async Task<BlobAlias[]> FindAliasesAsync(string name, int? maxLength = null, CancellationToken cancellationToken = default)
		{
			BlobAliasLocator[] aliases = await _backend.FindAliasesAsync(name, maxLength, cancellationToken);

			BlobAlias[] result = new BlobAlias[aliases.Length];
			for (int idx = 0; idx < aliases.Length; idx++)
			{
				BlobAliasLocator alias = aliases[idx];
				result[idx] = new BlobAlias(CreateBlobRef(alias.Target), alias.Rank, alias.Data);
			}
			return result;
		}

		#endregion

		#region Refs

		/// <inheritdoc/>
		public async Task<IHashedBlobRef?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			HashedBlobRefValue? target = await _backend.TryReadRefAsync(name, cacheTime, cancellationToken);
			if (target == null)
			{
				return null;
			}
			return this.CreateBlobRef(target.Hash, target.Locator);
		}

		#endregion

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
			_cache.GetStats(stats);
			_backend.GetStats(stats);
			_bundleReader.GetStats(stats);
			_packetReaderStats.GetStats(stats);
		}
	}
}
