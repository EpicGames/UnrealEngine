// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V1
{
	/// <summary>
	/// Identifier for the type of a section in the bundle header
	/// </summary>
	public enum BundleSectionType
	{
		/// <summary>
		/// List of custom types
		/// </summary>
		Types = 0,

		/// <summary>
		/// Imports of other bundles
		/// </summary>
		Imports = 1,

		/// <summary>
		/// List of export headers
		/// </summary>
		ExportHeaders = 2,

		/// <summary>
		/// References to exports in other bundles
		/// </summary>
		ExportReferences = 3,

		/// <summary>
		/// Packet headers
		/// </summary>
		Packets = 4,

		/// <summary>
		/// Merged export headers and references
		/// </summary>
		Exports = 5,
	}

	/// <summary>
	/// Header for the contents of a bundle.
	/// </summary>
	public class BundleHeader
	{
		/// <summary>
		/// Maximum size for a section
		/// </summary>
		public const int MaxSectionSize = 0x1000000;

		/// <summary>
		/// Maximum number of exports from a single bundle
		/// </summary>
		public const int MaxExports = 65535;

		/// <summary>
		/// Maximum number of export refs from a single bundle
		/// </summary>
		public const int MaxExportRefs = MaxSectionSize / BundleExportRef.NumBytes;

		/// <summary>
		/// Signature bytes
		/// </summary>
		public static ReadOnlyMemory<byte> Signature { get; } = Encoding.UTF8.GetBytes("UEBN");

		/// <summary>
		/// Types for exports within this bundle
		/// </summary>
		public BundleTypeCollection Types { get; }

		/// <summary>
		/// Bundles that we reference nodes in
		/// </summary>
		public BundleImportCollection Imports { get; }

		/// <summary>
		/// Nodes exported from this bundle
		/// </summary>
		public BundleExportCollection Exports { get; }

		/// <summary>
		/// List of data packets within this bundle
		/// </summary>
		public BundlePacketCollection Packets { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleHeader(BundleTypeCollection types, BundleImportCollection imports, BundleExportCollection exports, BundlePacketCollection packets)
		{
			Types = types;
			Imports = imports;
			Exports = exports;
			Packets = packets;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleHeader(BlobType[] types, BlobLocator[] imports, BundleExport[] exports, BundlePacket[] packets)
		{
			Types = new BundleTypeCollection(types);
			Imports = new BundleImportCollection(imports);
			Exports = new BundleExportCollection(exports);
			Packets = new BundlePacketCollection(packets);
		}

		/// <summary>
		/// Writes data for this bundle to a sequence builder
		/// </summary>
		public void AppendTo(ReadOnlySequenceBuilder<byte> builder)
		{
			// Create a writer for the output data
			ChunkedArrayMemoryWriter writer = new();

			// Create the prelude
			Span<byte> signatureData = writer.GetSpanAndAdvance(BundleSignature.NumBytes);

			// Write all the sections
			WriteSectionHeader(writer, BundleSectionType.Types, Types.Measure());
			Types.Write(writer);

			WriteSectionHeader(writer, BundleSectionType.Imports, Imports.Measure());
			Imports.Write(writer);

			WriteSectionHeader(writer, BundleSectionType.ExportHeaders, Exports.MeasureHeaders());
			Exports.WriteHeaders(writer);

			WriteSectionHeader(writer, BundleSectionType.ExportReferences, Exports.MeasureRefs());
			Exports.WriteRefs(writer);

			WriteSectionHeader(writer, BundleSectionType.Packets, Packets.Measure());
			Packets.Write(writer);

			// Fill in the length of the header, and append it to the builder
			BundleSignature signature = new(BundleVersion.LatestV1, writer.Length);
			signature.Write(signatureData);

			writer.AppendTo(builder);
		}

		/// <summary>
		/// Serialize the bundle to a byte array
		/// </summary>
		public byte[] ToByteArray()
		{
			ReadOnlySequenceBuilder<byte> builder = new();
			AppendTo(builder);
			return builder.Construct().ToArray();
		}

		const int SectionHeaderLength = 4;

		static void WriteSectionHeader(IMemoryWriter writer, BundleSectionType type, int length)
		{
			if (length >= 0x1000000)
			{
				throw new InvalidDataException($"Section header exceeds maximum allowed size (type: {type}, length: {length:n0}");
			}
			writer.WriteInt32((int)type | (length << 8));
		}

		record class ExportInfo(int TypeIdx, int Length, List<BundleExportRef> References);

#pragma warning disable VSTHRD002
		/// <summary>
		/// Reads a bundle header from memory
		/// </summary>
		/// <param name="memory">Memory to read from</param>
		/// <returns>New header object</returns>
		public static BundleHeader Read(ReadOnlyMemory<byte> memory)
		{
			using ReadOnlyMemoryStream stream = new(memory);
			return ReadAsync(stream).Result;
		}
#pragma warning restore VSTHRD002

		/// <summary>
		/// Reads a bundle header from a stream
		/// </summary>
		/// <param name="stream">Stream to deserialize from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New header object</returns>
		public static async Task<BundleHeader> ReadAsync(Stream stream, CancellationToken cancellationToken = default)
		{
			byte[] prelude = new byte[BundleSignature.NumBytes];
			await stream.ReadFixedLengthBytesAsync(prelude, cancellationToken);

			return await ReadAsync(BundleSignature.Read(prelude), stream, cancellationToken);
		}

		/// <summary>
		/// Reads a bundle header from a stream
		/// </summary>
		/// <param name="signature">File signature</param>
		/// <param name="stream">Stream to deserialize from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New header object</returns>
		public static async Task<BundleHeader> ReadAsync(BundleSignature signature, Stream stream, CancellationToken cancellationToken = default)
		{
			if (signature.Version > BundleVersion.LatestV1)
			{
				return new BundleHeader(new BundleTypeCollection(), new BundleImportCollection(), new BundleExportCollection(), new BundlePacketCollection());
			}
			else if (signature.Version >= BundleVersion.InPlace)
			{
				return await ReadLatestAsync(signature.Version, stream, signature.HeaderLength - BundleSignature.NumBytes, cancellationToken);
			}
			else
			{
				byte[] data = new byte[signature.HeaderLength - BundleSignature.NumBytes];
				await stream.ReadFixedLengthBytesAsync(data, cancellationToken);

				return ReadLegacy(data);
			}
		}

		/// <summary>
		/// Construct a header from the given data encoded in the latest format
		/// </summary>
		/// <param name="version">Version to serialize as</param>
		/// <param name="stream">Stream to read from</param>
		/// <param name="length">Length of the header</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		static async Task<BundleHeader> ReadLatestAsync(BundleVersion version, Stream stream, int length, CancellationToken cancellationToken)
		{
			byte[] exportHeaders = [];
			byte[] exportReferences = [];

			BundleTypeCollection types = new();
			BundleImportCollection imports = new();
			BundleExportCollection exports = new();
			BundlePacketCollection packets = new();

			byte[] sectionHeader = new byte[SectionHeaderLength];
			for (int offset = 0; offset < length;)
			{
				await stream.ReadFixedLengthBytesAsync(sectionHeader, cancellationToken);
				offset += SectionHeaderLength;

				BundleSectionType sectionType = (BundleSectionType)(sectionHeader[0] & 255);
				int sectionLength = (int)(BinaryPrimitives.ReadUInt32LittleEndian(sectionHeader) >> 8);

				switch (sectionType)
				{
					case BundleSectionType.Types:
						types = await BundleTypeCollection.ReadAsync(stream, sectionLength, cancellationToken);
						break;
					case BundleSectionType.Imports:
						imports = await BundleImportCollection.ReadAsync(stream, sectionLength, cancellationToken);
						break;
					case BundleSectionType.Exports:
						exports = await BundleExportCollection.ReadAsync(stream, sectionLength, cancellationToken);
						break;
					case BundleSectionType.ExportHeaders:
						exportHeaders = new byte[sectionLength];
						await stream.ReadFixedLengthBytesAsync(exportHeaders, cancellationToken);
						break;
					case BundleSectionType.ExportReferences:
						exportReferences = new byte[sectionLength];
						await stream.ReadFixedLengthBytesAsync(exportReferences, cancellationToken);
						break;
					case BundleSectionType.Packets:
						packets = await BundlePacketCollection.ReadAsync(stream, sectionLength, cancellationToken);
						break;
				}

				offset += sectionLength;
			}

			if (exportHeaders.Length > 0)
			{
				exports = BundleExportCollection.Read(exportHeaders, exportReferences, version);
			}

			return new BundleHeader(types, imports, exports, packets);
		}

		static BundleHeader ReadLegacy(ReadOnlyMemory<byte> memory)
		{
			MemoryReader reader = new(memory);

			BundleVersion version = (BundleVersion)reader.ReadUnsignedVarInt();
			if (version > BundleVersion.RemoveAliases)
			{
				throw new InvalidDataException($"Invalid bundle version {(int)version}. Max supported version in legacy format is {(int)BundleVersion.RemoveAliases}.");
			}

			BundleCompressionFormat compressionFormat = (BundleCompressionFormat)reader.ReadUnsignedVarInt();

			// Read the types
			int numTypes = (int)reader.ReadUnsignedVarInt();
			List<BlobType> types = new(numTypes);

			for (int typeIdx = 0; typeIdx < numTypes; typeIdx++)
			{
				Guid guid = reader.ReadGuidUnrealOrder();
				int serializerVersion = (int)reader.ReadUnsignedVarInt();

				types.Add(new BlobType(guid, serializerVersion));
			}

			// Read the imports
			int numImports = (int)reader.ReadUnsignedVarInt();
			List<BlobLocator> imports = new(numImports);
			List<BundleExportRef> allExportReferences = [];

			for (int importIdx = 0; importIdx < numImports; importIdx++)
			{
				BlobLocator locator = new(reader.ReadUtf8String());
				imports.Add(locator);

				int[] exportIndexes = reader.ReadVariableLengthArray(() => (int)reader.ReadUnsignedVarInt());
				for (int exportIdx = 0; exportIdx < exportIndexes.Length; exportIdx++)
				{
					BundleExportRef exportReference = new(imports.Count - 1, exportIndexes[exportIdx]);
					allExportReferences.Add(exportReference);
				}
			}

			// Read the exports
			int numExports = (int)reader.ReadUnsignedVarInt();
			List<ExportInfo> exportInfos = new(numExports);

			for (int exportIdx = 0; exportIdx < numExports; exportIdx++)
			{
				allExportReferences.Add(new BundleExportRef(-1, exportIdx));

				int typeIdx = (int)reader.ReadUnsignedVarInt();
				_ = reader.ReadIoHash();
				int length = (int)reader.ReadUnsignedVarInt();

				List<BundleExportRef> exportRefs = [];

				int numReferences = (int)reader.ReadUnsignedVarInt();
				if (numReferences > 0)
				{
					for (int idx = 0; idx < numReferences; idx++)
					{
						int referenceIdx = (int)reader.ReadUnsignedVarInt();
						exportRefs.Add(allExportReferences[referenceIdx]);
					}
				}

				if (version == BundleVersion.ExportAliases)
				{
					_ = reader.ReadUtf8String();
				}

				exportInfos.Add(new ExportInfo(typeIdx, length, exportRefs));
			}

			// Read the compression packets
			List<BundlePacket> packets;
			if (compressionFormat == BundleCompressionFormat.None)
			{
				packets = new List<BundlePacket>(exportInfos.Count);

				int encodedOffset = 0;
				foreach (ExportInfo export in exportInfos)
				{
					packets.Add(new BundlePacket(compressionFormat, encodedOffset, export.Length, export.Length));
					encodedOffset += export.Length;
				}
			}
			else
			{
				int numPackets = (int)reader.ReadUnsignedVarInt();
				packets = new List<BundlePacket>(numPackets);

				int encodedOffset = 0;
				for (int packetIdx = 0; packetIdx < numPackets; packetIdx++)
				{
					int encodedLength = (int)reader.ReadUnsignedVarInt();
					int decodedLength = (int)reader.ReadUnsignedVarInt();
					packets.Add(new BundlePacket(compressionFormat, encodedOffset, encodedLength, decodedLength));
					encodedOffset += encodedLength;
				}
			}

			// Create the final export list
			List<BundleExport> exports = [];
			{
				int packetIdx = 0;
				int packetOffset = 0;

				foreach (ExportInfo exportInfo in exportInfos)
				{
					if (packetOffset + exportInfo.Length > packets[packetIdx].DecodedLength)
					{
						packetIdx++;
						packetOffset = 0;
					}

					exports.Add(new BundleExport(exportInfo.TypeIdx, packetIdx, packetOffset, exportInfo.Length, exportInfo.References));
					packetOffset += exportInfo.Length;
				}
			}

			return new BundleHeader([.. types], [.. imports], [.. exports], [.. packets]);
		}
	}

	/// <summary>
	/// Collection of node types in a bundle
	/// </summary>
	public readonly struct BundleTypeCollection : IReadOnlyList<BlobType>
	{
		readonly BlobType[] _types;

		/// <inheritdoc/>
		public int Count => _types.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleTypeCollection() : this([]) { }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleTypeCollection(BlobType[] types) => _types = types;

		/// <summary>
		/// Reads a type collection from a block of memory
		/// </summary>
		public static BundleTypeCollection Read(ReadOnlyMemory<byte> data)
		{
			BlobType[] types = new BlobType[data.Length / BlobType.NumBytes];
			for (int idx = 0; idx < types.Length; idx++)
			{
				types[idx] = BlobType.Read(data.Span.Slice(idx * BlobType.NumBytes));
			}

			return new BundleTypeCollection(types);
		}

		/// <summary>
		/// Reads a type collection from a stream
		/// </summary>
		public static async Task<BundleTypeCollection> ReadAsync(Stream stream, int length, CancellationToken cancellationToken)
		{
			byte[] data = new byte[length];
			await stream.ReadFixedLengthBytesAsync(data, cancellationToken);
			return Read(data);
		}

		/// <summary>
		/// Gets the size of memory required to serialize a collection of types
		/// </summary>
		/// <returns>Number of bytes required to serialize the types</returns>
		public int Measure() => _types.Length * BlobType.NumBytes;

		/// <summary>
		/// Serializes a set of types to a fixed block of memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void Write(IMemoryWriter writer)
		{
			Span<byte> span = writer.GetSpanAndAdvance(_types.Length * BlobType.NumBytes);
			foreach (BlobType type in _types)
			{
				type.Write(span);
				span = span.Slice(BlobType.NumBytes);
			}
		}

		/// <inheritdoc/>
		public BlobType this[int index] => _types[index];

		/// <inheritdoc/>
		public IEnumerator<BlobType> GetEnumerator() => ((IEnumerable<BlobType>)_types).GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}

	/// <summary>
	/// Collection of imported node references
	/// </summary>
	public readonly struct BundleImportCollection : IReadOnlyList<BlobLocator>
	{
		readonly BlobLocator[] _imports;

		/// <inheritdoc/>
		public int Count => _imports.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleImportCollection() : this([]) { }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleImportCollection(BlobLocator[] imports) => _imports = imports;

		/// <summary>
		/// Reads a collection from memory
		/// </summary>
		public static BundleImportCollection Read(ReadOnlyMemory<byte> data)
		{
			ReadOnlySpan<byte> span = data.Span;
			int count = (span.Length == 0) ? 0 : BinaryPrimitives.ReadInt32LittleEndian(span) / sizeof(int);

			BlobLocator[] imports = new BlobLocator[count];
			for (int idx = 0; idx < count; idx++)
			{
				int offset = BinaryPrimitives.ReadInt32LittleEndian(span.Slice(idx * sizeof(int)));
				int length = span.Slice(offset).IndexOf((byte)0);
				imports[idx] = new BlobLocator(new Utf8String(data.Slice(offset, length)));
			}

			return new BundleImportCollection(imports);
		}

		/// <summary>
		/// Reads a collection from a stream
		/// </summary>
		public static async Task<BundleImportCollection> ReadAsync(Stream stream, int length, CancellationToken cancellationToken)
		{
			byte[] data = new byte[length];
			await stream.ReadFixedLengthBytesAsync(data, cancellationToken);
			return Read(data);
		}

		/// <inheritdoc/>
		public BlobLocator this[int index] => _imports[index];

		/// <inheritdoc/>
		public IEnumerator<BlobLocator> GetEnumerator() => _imports.AsEnumerable().GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <summary>
		/// Measure the size of memory required to store a collection of import locators
		/// </summary>
		/// <returns>Size in bytes of the output buffer</returns>
		public int Measure() => _imports.Sum(x => sizeof(int) + x.Path.Length + 1);

		/// <summary>
		/// Serialize a collection of locators to memory
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public void Write(IMemoryWriter writer)
		{
			int offset = _imports.Length * sizeof(int);
			foreach (BlobLocator import in _imports)
			{
				writer.WriteInt32(offset);
				offset += import.Path.Length + 1;
			}
			foreach (BlobLocator import in _imports)
			{
				writer.WriteNullTerminatedUtf8String(import.Path);
			}
		}
	}

	/// <summary>
	/// Descriptor for a compression packet
	/// </summary>
	public readonly struct BundlePacket
	{
		/// <summary>
		/// Size of this structure when serialized
		/// </summary>
		public const int NumBytes = 16;

		/// <summary>
		/// Compression format for this packet
		/// </summary>
		public BundleCompressionFormat CompressionFormat { get; }

		/// <summary>
		/// Offset of the packet within the payload stream
		/// </summary>
		public int EncodedOffset { get; }

		/// <summary>
		/// Encoded length of the packet
		/// </summary>
		public int EncodedLength { get; }

		/// <summary>
		/// Decoded length of the packet
		/// </summary>
		public int DecodedLength { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="compressionFormat">Compression format for the packet</param>
		/// <param name="encodedOffset">Offset of the data within the payload stream</param>
		/// <param name="encodedLength">Size of the encoded data</param>
		/// <param name="decodedLength">Size of the decoded data</param>
		public BundlePacket(BundleCompressionFormat compressionFormat, int encodedOffset, int encodedLength, int decodedLength)
		{
			CompressionFormat = compressionFormat;
			EncodedOffset = encodedOffset;
			EncodedLength = encodedLength;
			DecodedLength = decodedLength;
		}

		/// <summary>
		/// Read from a byte array
		/// </summary>
		public static BundlePacket Read(ReadOnlySpan<byte> span)
		{
			BundleCompressionFormat compressionFormat = (BundleCompressionFormat)BinaryPrimitives.ReadInt32LittleEndian(span);
			int encodedOffset = BinaryPrimitives.ReadInt32LittleEndian(span.Slice(4));
			int encodedLength = BinaryPrimitives.ReadInt32LittleEndian(span.Slice(8));
			int decodedLength = BinaryPrimitives.ReadInt32LittleEndian(span.Slice(12));
			return new BundlePacket(compressionFormat, encodedOffset, encodedLength, decodedLength);
		}

		/// <summary>
		/// Serialize the struct to memory
		/// </summary>
		public void Write(Span<byte> span)
		{
			BinaryPrimitives.WriteInt32LittleEndian(span, (int)CompressionFormat);
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(4), EncodedOffset);
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(8), EncodedLength);
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(12), DecodedLength);
		}
	}

	/// <summary>
	/// Collection of information about packets in a bundle
	/// </summary>
	public readonly struct BundlePacketCollection : IReadOnlyCollection<BundlePacket>
	{
		readonly BundlePacket[] _packets;

		/// <inheritdoc/>
		public int Count => _packets?.Length ?? 0;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundlePacketCollection() : this([]) { }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundlePacketCollection(BundlePacket[] packets) => _packets = packets;

		/// <summary>
		/// Reads a collection from memory
		/// </summary>
		public static BundlePacketCollection Read(ReadOnlyMemory<byte> data)
		{
			int count = data.Length / BundlePacket.NumBytes;

			BundlePacket[] packets = new BundlePacket[count];
			for (int idx = 0; idx < count; idx++)
			{
				packets[idx] = BundlePacket.Read(data.Span.Slice(idx * BundlePacket.NumBytes));
			}

			return new BundlePacketCollection(packets);
		}

		/// <summary>
		/// Reads a collection from a stream
		/// </summary>
		public static async Task<BundlePacketCollection> ReadAsync(Stream stream, int length, CancellationToken cancellationToken)
		{
			byte[] data = new byte[length];
			await stream.ReadFixedLengthBytesAsync(data, cancellationToken);
			return Read(data);
		}

		/// <inheritdoc/>
		public BundlePacket this[int index] => _packets[index];

		/// <inheritdoc/>
		public IEnumerator<BundlePacket> GetEnumerator() => _packets.AsEnumerable().GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <summary>
		/// Measure the size of memory required to store a collection of packets
		/// </summary>
		/// <returns>Size in bytes of the output buffer</returns>
		public int Measure() => BundlePacket.NumBytes * _packets.Length;

		/// <summary>
		/// Serialize a collection of packets to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void Write(IMemoryWriter writer)
		{
			Span<byte> span = writer.GetMemoryAndAdvance(_packets.Length * BundlePacket.NumBytes).Span;
			foreach (BundlePacket packet in _packets)
			{
				packet.Write(span);
				span = span.Slice(BundlePacket.NumBytes);
			}
		}
	}

	/// <summary>
	/// Entry for a node exported from an object
	/// </summary>
	public readonly struct BundleExport
	{
		/// <summary>
		/// Number of bytes in a serialized export object
		/// </summary>
		public const int NumHeaderBytes = 32;

		/// <summary>
		/// Raw data for this export
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Raw data for the header
		/// </summary>
		public ReadOnlyMemory<byte> HeaderData => Data.Slice(0, NumHeaderBytes);

		/// <summary>
		/// Hash of the node data
		/// </summary>
		public IoHash Hash => new(Data.Slice(0, 20).Span);

		/// <summary>
		/// Type id of the node. Can be used to look up the type information from the bundle header.
		/// </summary>
		public int TypeIdx => BinaryPrimitives.ReadUInt16LittleEndian(Data.Slice(20).Span);

		/// <summary>
		/// Packet containing this export's data
		/// </summary>
		public int Packet => BinaryPrimitives.ReadUInt16LittleEndian(Data.Slice(22).Span);

		/// <summary>
		/// Offset within the packet of the node data
		/// </summary>
		public int Offset => BinaryPrimitives.ReadInt32LittleEndian(Data.Slice(24).Span);

		/// <summary>
		/// Length of the node
		/// </summary>
		public int Length => BinaryPrimitives.ReadInt32LittleEndian(Data.Slice(28).Span);

		/// <summary>
		/// References to other nodes
		/// </summary>
		public BundleExportRefCollection References => new(Data.Slice(32));

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExport(ReadOnlyMemory<byte> data)
		{
			Data = data;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExport(int typeIdx, int packetIdx, int offset, int length, IReadOnlyList<BundleExportRef> references)
		{
			byte[] data = new byte[NumHeaderBytes + (BundleExportRef.NumBytes * references.Count)];
			WriteHeader(data, typeIdx, packetIdx, offset, length);

			for (int idx = 0; idx < references.Count; idx++)
			{
				references[idx].CopyTo(data.AsSpan(NumHeaderBytes + (BundleExportRef.NumBytes * idx)));
			}

			Data = data;
		}

		/// <summary>
		/// Writes a new export to a block of memory
		/// </summary>
		public static void WriteHeader(Span<byte> data, int typeIdx, int packet, int offset, int length)
		{
			IoHash.Zero.CopyTo(data); // Previously hash of export
			BinaryPrimitives.WriteUInt16LittleEndian(data.Slice(20), (ushort)typeIdx);
			BinaryPrimitives.WriteUInt16LittleEndian(data.Slice(22), (ushort)packet);
			BinaryPrimitives.WriteInt32LittleEndian(data.Slice(24), offset);
			BinaryPrimitives.WriteInt32LittleEndian(data.Slice(28), length);
		}
	}

	/// <summary>
	/// Entry for a node exported from an object
	/// </summary>
	public readonly struct BundleExportCollection : IReadOnlyList<BundleExport>
	{
		readonly BundleExport[] _exports;

		/// <inheritdoc/>
		public int Count => _exports.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExportCollection() : this([]) { }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExportCollection(BundleExport[] exports) => _exports = exports;

		/// <summary>
		/// Constructor
		/// </summary>
		public static BundleExportCollection Read(ReadOnlyMemory<byte> data, ReadOnlyMemory<byte> refs, BundleVersion version)
		{
			int count = data.Length / BundleExport.NumHeaderBytes;

			if (version < BundleVersion.ImportHashes && refs.Length > 0)
			{
				const int OldRefNumBytes = 4;

				int headerLength = BinaryPrimitives.ReadInt32LittleEndian(refs.Span);
				int refCount = (refs.Length - headerLength) / OldRefNumBytes;

				byte[] newRefs = new byte[headerLength + (refCount * BundleExportRef.NumBytes)];
				for (int headerOffset = 0; headerOffset < headerLength; headerOffset += sizeof(int))
				{
					int index = (BinaryPrimitives.ReadInt32LittleEndian(refs.Slice(headerOffset).Span) - headerLength) / OldRefNumBytes;
					BinaryPrimitives.WriteInt32LittleEndian(newRefs.AsSpan(headerOffset), headerLength + (index * BundleExportRef.NumBytes));
				}
				for (int index = 0; index < refCount; index++)
				{
					ReadOnlyMemory<byte> source = refs.Slice(headerLength + (index * OldRefNumBytes), OldRefNumBytes);
					source.CopyTo(newRefs.AsMemory(headerLength + (index * BundleExportRef.NumBytes)));
				}

				refs = newRefs;
			}

			BundleExport[] exports = new BundleExport[count];
			for (int index = 0; index < count; index++)
			{
				ReadOnlyMemory<byte> exportData = data.Slice(index * BundleExport.NumHeaderBytes, BundleExport.NumHeaderBytes);

				BundleExportRefCollection exportRefs = new();
				if (refs.Length > 0)
				{
					int minOffset = BinaryPrimitives.ReadInt32LittleEndian(refs.Span.Slice(index * sizeof(int)));
					int maxOffset = (index == count - 1) ? refs.Length : BinaryPrimitives.ReadInt32LittleEndian(refs.Span.Slice((index + 1) * sizeof(int)));
					exportRefs = new BundleExportRefCollection(refs.Slice(minOffset, maxOffset - minOffset));
				}

				byte[] export = new byte[BundleExport.NumHeaderBytes + exportRefs.Data.Length];
				exportData.CopyTo(export);
				exportRefs.Data.CopyTo(export.AsMemory(BundleExport.NumHeaderBytes));

				exports[index] = new BundleExport(export);
			}

			return new BundleExportCollection(exports);
		}

		/// <summary>
		/// Reads an export collection from a stream
		/// </summary>
		public static async Task<BundleExportCollection> ReadAsync(Stream stream, int sectionLength, CancellationToken cancellationToken)
		{
			BundleExport[] exports = [];
			if (sectionLength > 0)
			{
				const int MaxPageLength = 16384;

				// Read the offsets of each entry
				byte[] headerBuffer = new byte[sizeof(int)];
				await stream.ReadFixedLengthBytesAsync(headerBuffer, cancellationToken);

				int headerLength = BinaryPrimitives.ReadInt32LittleEndian(headerBuffer);
				Array.Resize(ref headerBuffer, headerLength);

				await stream.ReadFixedLengthBytesAsync(headerBuffer.AsMemory(sizeof(int)), cancellationToken);

				// Create the packet array
				int exportCount = headerLength / sizeof(int);
				exports = new BundleExport[exportCount];

				// Read the exports in pages
				int exportIdx = 0;
				for (int exportOffset = headerLength; exportOffset < sectionLength;)
				{
					int pageOffset = exportOffset;

					// Find the exports to read into this page
					int nextPageOffset = sectionLength;
					for (int nextExportIdx = exportIdx + 1; nextExportIdx < exportCount; nextExportIdx++)
					{
						int nextExportOffset = BinaryPrimitives.ReadInt32LittleEndian(headerBuffer.AsSpan(nextExportIdx * sizeof(int)));
						if (nextExportOffset > pageOffset + MaxPageLength)
						{
							nextPageOffset = nextExportOffset;
							break;
						}
					}

					// Read the page data
					byte[] pageBuffer = new byte[nextPageOffset - pageOffset];
					await stream.ReadFixedLengthBytesAsync(pageBuffer, cancellationToken);

					// Parse the individual exports
					for (; exportOffset < nextPageOffset; exportIdx++)
					{
						int startExportOffset = exportOffset;
						if (exportIdx + 1 < exportCount)
						{
							exportOffset = BinaryPrimitives.ReadInt32LittleEndian(headerBuffer.AsSpan((exportIdx + 1) * sizeof(int)));
						}
						else
						{
							exportOffset = nextPageOffset;
						}
						exports[exportIdx] = new BundleExport(pageBuffer.AsMemory(startExportOffset - pageOffset, exportOffset - startExportOffset));
					}
				}
			}
			return new BundleExportCollection(exports);
		}

		/// <inheritdoc/>
		public BundleExport this[int index] => _exports[index];

		/// <inheritdoc/>
		public IEnumerator<BundleExport> GetEnumerator() => _exports.AsEnumerable().GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <summary>
		/// Serializes this collection as a single section
		/// </summary>
		public void Write(IMemoryWriter writer)
		{
			int offset = _exports.Length * sizeof(int);
			Span<byte> index = writer.GetSpanAndAdvance(offset);

			foreach (BundleExport export in _exports)
			{
				BinaryPrimitives.WriteInt32LittleEndian(index, offset);
				index = index.Slice(sizeof(int));

				writer.WriteFixedLengthBytes(export.Data.Span);
				offset += export.Data.Length;
			}
		}

		/// <summary>
		/// Measure the size of memory required to store a collection of exports
		/// </summary>
		/// <returns>Size in bytes of the output buffer</returns>
		public int MeasureHeaders() => BundleExport.NumHeaderBytes * _exports.Length;

		/// <summary>
		/// Serialize a collection of exports to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void WriteHeaders(IMemoryWriter writer)
		{
			foreach (BundleExport export in _exports)
			{
				writer.WriteFixedLengthBytes(export.HeaderData.Span);
			}
		}

		/// <summary>
		/// Measure the size of memory required to store a collection of export refs
		/// </summary>
		/// <returns>Size in bytes of the output buffer</returns>
		public int MeasureRefs() => (sizeof(int) * _exports.Length) + _exports.Sum(x => x.References.Data.Length);

		/// <summary>
		/// Serialize a collection of export refs to memory
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void WriteRefs(IMemoryWriter writer)
		{
			int offset = sizeof(int) * _exports.Length;
			foreach (BundleExport export in _exports)
			{
				writer.WriteInt32(offset);
				offset += export.References.Data.Length;
			}
			foreach (BundleExport export in _exports)
			{
				writer.WriteFixedLengthBytes(export.References.Data.Span);
			}
		}
	}

	/// <summary>
	/// Reference to a node in another bundle
	/// </summary>
	/// <param name="ImportIdx">Index into the import table of the blob containing the referenced node. Can be -1 for references within the same bundle.</param>
	/// <param name="NodeIdx">Node imported from the bundle</param>
	public readonly record struct BundleExportRef(int ImportIdx, int NodeIdx)
	{
		/// <summary>
		/// Number of bytes in the serialized object
		/// </summary>
		public const int NumBytes = 4 + IoHash.NumBytes;

		/// <summary>
		/// Deserialize this object from memory
		/// </summary>
		public static BundleExportRef Read(ReadOnlySpan<byte> data)
		{
			int importIdx = BinaryPrimitives.ReadInt16LittleEndian(data);
			int nodeIdx = BinaryPrimitives.ReadUInt16LittleEndian(data.Slice(2));
			return new BundleExportRef(importIdx, nodeIdx);
		}

		/// <summary>
		/// Serialize this object to memory
		/// </summary>
		public void CopyTo(Span<byte> data)
		{
			if (ImportIdx > 0xffff)
			{
				throw new InvalidDataException($"Import index is out of range ({ImportIdx})");
			}
			if (NodeIdx > 0xffff)
			{
				throw new InvalidDataException($"Node index is out of range ({NodeIdx})");
			}

			BinaryPrimitives.WriteInt16LittleEndian(data, (short)ImportIdx);
			BinaryPrimitives.WriteUInt16LittleEndian(data.Slice(2), (ushort)NodeIdx);
		}
	}

	/// <summary>
	/// Collection of information about exported nodes
	/// </summary>
	public readonly struct BundleExportRefCollection : IReadOnlyList<BundleExportRef>
	{
		/// <summary>
		/// Data used to store this collection
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExportRefCollection(ReadOnlyMemory<byte> data) => Data = data;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleExportRefCollection(IReadOnlyList<BundleExportRef> exportRefs)
		{
			byte[] data = new byte[exportRefs.Count * BundleExportRef.NumBytes];
			for (int idx = 0; idx < exportRefs.Count; idx++)
			{
				exportRefs[idx].CopyTo(data.AsSpan(idx * BundleExportRef.NumBytes));
			}
			Data = data;
		}

		/// <inheritdoc/>
		public int Count => Data.Length / BundleExportRef.NumBytes;

		/// <inheritdoc/>
		public BundleExportRef this[int index]
		{
			get
			{
				Debug.Assert(index < Count);
				return BundleExportRef.Read(Data.Span.Slice(index * BundleExportRef.NumBytes));
			}
		}

		/// <inheritdoc/>
		public IEnumerator<BundleExportRef> GetEnumerator()
		{
			ReadOnlyMemory<byte> data = Data;
			while (data.Length > 0)
			{
				yield return BundleExportRef.Read(data.Span);
				data = data.Slice(BundleExportRef.NumBytes);
			}
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
	}
}
