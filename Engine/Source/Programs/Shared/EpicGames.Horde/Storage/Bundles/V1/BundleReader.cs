// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Bundles.V1
{
	/// <summary>
	/// Computed information about a bundle
	/// </summary>
	public class BundleInfo
	{
		/// <summary>
		/// Locator for the bundle
		/// </summary>
		public BlobLocator Locator { get; }

		/// <summary>
		/// Bundle header
		/// </summary>
		public BundleHeader Header { get; }

		/// <summary>
		/// Length of the header. Required to offset packets from the start of the bundle
		/// </summary>
		public int HeaderLength { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleInfo(BlobLocator locator, BundleHeader header, int headerLength)
		{
			Locator = locator;
			Header = header;
			HeaderLength = headerLength;
		}
	}

	/// <summary>
	/// Writes nodes from bundles in an <see cref="IStorageNamespace"/> instance.
	/// </summary>
	public class BundleReader
	{
		/// <summary>
		/// Queued set of requests from a particular bundle
		/// </summary>
		[DebuggerDisplay("{Locator}")]
		class QueuedBundle
		{
			public BlobLocator Locator { get; }
			public TaskCompletionSource<BundleInfo> BundleInfo { get; } = new TaskCompletionSource<BundleInfo>(TaskCreationOptions.RunContinuationsAsynchronously);
			public List<QueuedPacket> QueuedPackets { get; } = []; // Sorted by index
			public int InfoRefCount { get; set; } // Ref count for reading the header
			public PendingRead? PendingRead { get; set; }
			public Task? WorkerTask { get; set; }
			public CancellationTokenSource CancellationSource { get; set; } = new CancellationTokenSource();
			public bool Complete { get; set; }

			public QueuedBundle(BlobLocator locator)
			{
				Locator = locator;
			}

			public QueuedPacket AddPacket(int packetIdx)
			{
				QueuedPacket packet = new(packetIdx);

				int insertIdx = QueuedPackets.BinarySearch(packet);
				if (insertIdx < 0)
				{
					insertIdx = ~insertIdx;
				}
				QueuedPackets.Insert(insertIdx, packet);

				return packet;
			}

			public void RemovePacket(QueuedPacket packet) => QueuedPackets.Remove(packet);
		}

		/// <summary>
		/// Encoded bundle packet queued to be read
		/// </summary>
		class QueuedPacket : IComparable<QueuedPacket>
		{
			public readonly int PacketIdx;
			public readonly TaskCompletionSource<ReadOnlyMemory<byte>> CompletionSource = new(TaskCreationOptions.RunContinuationsAsynchronously);

			public QueuedPacket(int packetIdx) => PacketIdx = packetIdx;
			public int CompareTo(QueuedPacket? other) => PacketIdx - other?.PacketIdx ?? 0;
		}

		/// <summary>
		/// Information about a pending read
		/// </summary>
		sealed class PendingRead : IDisposable
		{
			public int MinPacketIdx { get; }
			public int MaxPacketIdx { get; }
			public CancellationTokenSource CancellationSource { get; }

			public PendingRead(int minPacketIdx, int maxPacketIdx, CancellationToken cancellationToken)
			{
				MinPacketIdx = minPacketIdx;
				MaxPacketIdx = maxPacketIdx;
				CancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
			}

			public void Dispose()
			{
				CancellationSource.Dispose();
			}
		}

		// Size of data to fetch by default. This is larger than the minimum request size to reduce number of reads.
		const int DefaultFetchSize = 15 * 1024 * 1024;

		// When reader is uncached, use a smaller default fetch size
		const int DefaultUncachedFetchSize = 1 * 1024 * 1024;

		readonly BundleStorageNamespace _store;
		readonly BundleCache _cache;
		readonly ILogger _logger;

		readonly Lock _queueLock = new();
		readonly List<QueuedBundle> _queuedBundles = [];
		readonly Dictionary<string, Task<ReadOnlyMemory<byte>>> _decodeTasks = new(StringComparer.Ordinal);

		int _numHeaderReads;
		int _numPacketReads;
		long _numBytesRead;
		long _decodeTimeTicks;

		/// <summary>
		/// Accessor for the cache
		/// </summary>
		public BundleCache Cache => _cache;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="store"></param>
		/// <param name="cache">Cache for data</param>
		/// <param name="logger">Logger for output</param>
		public BundleReader(BundleStorageNamespace store, BundleCache cache, ILogger logger)
		{
			_store = store;
			_cache = cache;
			_logger = logger;
		}

		#region Bundles

		/// <summary>
		/// Reads a bundle header
		/// </summary>
		/// <param name="locator">Locator for the bundle</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the bundle</returns>
		public async Task<BundleHeader> ReadHeaderAsync(BlobLocator locator, CancellationToken cancellationToken)
		{
			// Check for a cached value first
			if (_cache.TryGetCachedHeader(locator, out BundleInfo? cachedBundleInfo))
			{
				return cachedBundleInfo.Header;
			}

			// Find a registered bundle info with the given locator
			QueuedBundle? bundle = null;
			lock (_queueLock)
			{
				bundle = FindOrAddBundle(locator);
				bundle.InfoRefCount++;
			}

			// Wait for the read to complete
			try
			{
				BundleInfo bundleInfo = await bundle.BundleInfo.Task.WaitAsync(cancellationToken);
				return bundleInfo.Header;
			}
			finally
			{
				await CancelBundleRequestAsync(bundle, () => bundle.InfoRefCount--);
			}
		}

		static string GetDecodeTaskKey(BlobLocator locator, int packetIdx) => $"{locator}#{packetIdx}";

		/// <summary>
		/// Reads decoded packet data from a bundle
		/// </summary>
		/// <param name="locator">Locator for the bundle to read</param>
		/// <param name="packetIdx">Index of the bundle packet</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Data for the packet</returns>
		public async Task<ReadOnlyMemory<byte>> ReadPacketAsync(BlobLocator locator, int packetIdx, CancellationToken cancellationToken)
		{
			ReadOnlyMemory<byte> cachedDecodedPacket;
			if (_cache.TryGetCachedDecodedPacket(locator, packetIdx, out cachedDecodedPacket))
			{
				return cachedDecodedPacket;
			}

			// Create an async task to read the data
			Task<ReadOnlyMemory<byte>>? decodeTask;
			lock (_queueLock)
			{
				if (_cache.TryGetCachedDecodedPacket(locator, packetIdx, out cachedDecodedPacket))
				{
					return cachedDecodedPacket;
				}

				string decodedCacheKey = GetDecodeTaskKey(locator, packetIdx);
				if (!_decodeTasks.TryGetValue(decodedCacheKey, out decodeTask))
				{
					decodeTask = Task.Run(() => ReadAndDecodePacketAsync(locator, packetIdx, CancellationToken.None), CancellationToken.None);
					_decodeTasks.Add(decodedCacheKey, decodeTask);
				}
			}

			return await decodeTask.WaitAsync(cancellationToken);
		}

		async Task<ReadOnlyMemory<byte>> ReadAndDecodePacketAsync(BlobLocator locator, int packetIdx, CancellationToken cancellationToken)
		{
			BundleHeader header = await ReadHeaderAsync(locator, cancellationToken);
			ReadOnlyMemory<byte> encodedPacket = await ReadEncodedPacketAsync(locator, packetIdx, cancellationToken);

			BundlePacket packet = header.Packets[packetIdx];

			byte[] decodedPacket = new byte[packet.DecodedLength];
			Stopwatch decodeTimer = Stopwatch.StartNew();
			BundleData.Decompress(packet.CompressionFormat, encodedPacket, decodedPacket);
			Interlocked.Add(ref _decodeTimeTicks, decodeTimer.ElapsedTicks);
			_cache.AddCachedDecodedPacket(locator, packetIdx, decodedPacket);

			lock (_queueLock)
			{
				string decodedCacheKey = GetDecodeTaskKey(locator, packetIdx);
				_decodeTasks.Remove(decodedCacheKey);
			}

			return decodedPacket;
		}

		/// <summary>
		/// Reads encoded packet data from a bundle
		/// </summary>
		/// <param name="locator">Locator for the bundle to read</param>
		/// <param name="packetIdx">Index of the bundle packet</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Dtaa for the packet</returns>
		public async Task<ReadOnlyMemory<byte>> ReadEncodedPacketAsync(BlobLocator locator, int packetIdx, CancellationToken cancellationToken)
		{
			// Register a read for the packet
			QueuedBundle? bundle = null;
			QueuedPacket? packet = null;
			lock (_queueLock)
			{
				if (_cache.TryGetCachedEncodedPacket(locator, packetIdx, out ReadOnlyMemory<byte> cachedEncodedPacket))
				{
					return cachedEncodedPacket;
				}

				bundle = FindOrAddBundle(locator);
				packet = bundle.AddPacket(packetIdx);
			}

			// Wait for the read to complete
			using (CancellationTokenRegistration registration = cancellationToken.Register(() => packet.CompletionSource.TrySetCanceled()))
			{
				try
				{
					return await packet.CompletionSource.Task;
				}
				finally
				{
					await CancelBundleRequestAsync(bundle, () => bundle.QueuedPackets.Remove(packet));
				}
			}
		}

		QueuedBundle FindOrAddBundle(BlobLocator locator)
		{
			QueuedBundle? bundle = _queuedBundles.FirstOrDefault(x => x.Locator == locator);
			if (bundle == null)
			{
				bundle = new QueuedBundle(locator);
				bundle.WorkerTask = Task.Run(() => HandleBundleRequestsAsync(bundle), bundle.CancellationSource.Token);
				_queuedBundles.Add(bundle);
			}
			return bundle;
		}

		async Task HandleBundleRequestsAsync(QueuedBundle bundle)
		{
			// Read the bundle header
			BundleInfo? bundleInfo;
			if (!_cache.TryGetCachedHeader(bundle.Locator, out bundleInfo))
			{
				try
				{
					bundleInfo = await ReadBundleInfoAsync(bundle, bundle.CancellationSource.Token);
					_cache.AddCachedHeader(bundle.Locator, bundleInfo);
				}
				catch (Exception ex)
				{
					bundle.BundleInfo.SetException(ex);
					return;
				}
			}
			bundle.BundleInfo.SetResult(bundleInfo);

			// Serve any packet read requests
			for (; ; )
			{
				// Create the next read
				PendingRead? pendingRead;
				lock (_queueLock)
				{
					// Dispose of the previous pending read
					bundle.PendingRead?.Dispose();
					bundle.PendingRead = null;

					// If there's nothing left to read, dispose of it
					if (bundle.QueuedPackets.Count == 0)
					{
						bundle.Complete = true;
						_queuedBundles.Remove(bundle);
						break;
					}

					// Figure out the range of packets to read
					int minPacketIdx = bundle.QueuedPackets[0].PacketIdx;
					int maxPacketIdx = minPacketIdx;

					long length = 0;
					for (int packetIdx = minPacketIdx; packetIdx < bundleInfo.Header.Packets.Count; packetIdx++)
					{
						length += bundleInfo.Header.Packets[packetIdx].EncodedLength;
						if (length > DefaultFetchSize)
						{
							break;
						}
						maxPacketIdx = packetIdx;
					}

					// Create the new read
					pendingRead = new PendingRead(minPacketIdx, maxPacketIdx, bundle.CancellationSource.Token);
					bundle.PendingRead = pendingRead;
				}

				// Execute the read
				try
				{
					await ReadEncodedPacketsAsync(bundle, bundle.PendingRead.MinPacketIdx, bundle.PendingRead.MaxPacketIdx, pendingRead.CancellationSource.Token);
				}
				catch (OperationCanceledException ex)
				{
					_logger.LogTrace(ex, "Read from bundle was cancelled");
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error reading from bundle: {Message}", ex.Message);

					lock (_queueLock)
					{
						List<QueuedPacket> packets = bundle.QueuedPackets;
						for (int idx = 0; idx < packets.Count && packets[idx].PacketIdx <= pendingRead.MaxPacketIdx; idx++)
						{
							if (packets[idx].PacketIdx >= pendingRead.MinPacketIdx)
							{
								packets[idx].CompletionSource.TrySetException(ex);
								packets.RemoveAt(idx--);
							}
						}
					}
				}
			}

			// Dispose the cancellation source used for this bundle
			lock (_queueLock)
			{
				bundle.CancellationSource.Dispose();
				bundle.CancellationSource = null!;
			}
		}

		async Task CancelBundleRequestAsync(QueuedBundle bundle, Action updateAction)
		{
			// Remove this read request
			Task? workerTask = null;
			lock (_queueLock)
			{
				updateAction();

				// If there's nothing required any more, remove the bundle from the queue
				if (bundle.InfoRefCount == 0 && bundle.QueuedPackets.Count == 0)
				{
					workerTask = bundle.WorkerTask;
					_queuedBundles.Remove(bundle);
				}
			}

			// If this was the last thing using the queued bundle, wait for the worker task to finish
			if (workerTask != null)
			{
				await workerTask;

				try
				{
					await bundle.BundleInfo.Task; // Avoid unobserved cancellation exceptions
				}
				catch { }
			}
		}

		#endregion

		#region Reading

		/// <summary>
		/// Reads a bundle header from the queue
		/// </summary>
		/// <param name="bundle">Bundle header to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async Task<BundleInfo> ReadBundleInfoAsync(QueuedBundle bundle, CancellationToken cancellationToken)
		{
			Interlocked.Increment(ref _numHeaderReads);

			int prefetchSize = _cache != null ? DefaultFetchSize : DefaultUncachedFetchSize;
			for (; ; )
			{
				await using (Stream stream = await _store.Backend.OpenBlobAsync(bundle.Locator, 0, prefetchSize, cancellationToken))
				{
					// Read the header data
					byte[] prelude = new byte[BundleSignature.NumBytes];
					await stream.ReadFixedLengthBytesAsync(prelude, cancellationToken);

					// Make sure we've read enough to hold the header
					BundleSignature signature = BundleSignature.Read(prelude);
					if (signature.HeaderLength > prefetchSize)
					{
						prefetchSize = signature.HeaderLength;
						continue;
					}

					// Parse the header and construct the bundle info from it
					BundleHeader header = await BundleHeader.ReadAsync(signature, stream, cancellationToken);

					// Construct the bundle info
					BundleInfo bundleInfo = new(bundle.Locator, header, signature.HeaderLength);

					// Also add any encoded packets we prefetched
					int packetOffset = signature.HeaderLength;
					for (int packetIdx = 0; packetIdx < header.Packets.Count; packetIdx++)
					{
						BundlePacket packet = header.Packets[packetIdx];

						int packetLength = packet.EncodedLength;
						if (packetOffset + packetLength > prefetchSize)
						{
							break;
						}
						packetOffset += packetLength;

						await ReadEncodedPacketFromStreamAsync(bundle, packetIdx, packet, stream, cancellationToken);
					}

					Interlocked.Add(ref _numBytesRead, packetOffset);
					return bundleInfo;
				}
			}
		}

		async Task ReadEncodedPacketsAsync(QueuedBundle bundle, int minPacketIdx, int maxPacketIdx, CancellationToken cancellationToken)
		{
			BundleInfo bundleInfo = await bundle.BundleInfo.Task;

			BundlePacket minPacket = bundleInfo.Header.Packets[minPacketIdx];
			int minOffset = minPacket.EncodedOffset;

			BundlePacket maxPacket = bundleInfo.Header.Packets[maxPacketIdx];
			int maxOffset = maxPacket.EncodedOffset + maxPacket.EncodedLength;

			Interlocked.Increment(ref _numPacketReads);
			Interlocked.Add(ref _numBytesRead, maxOffset - minOffset);

			await using (Stream stream = await _store.Backend.OpenBlobAsync(bundleInfo.Locator, bundleInfo.HeaderLength + minOffset, maxOffset - minOffset, cancellationToken))
			{
				// Copy all the packets that have been read into separate buffers, so we can cache them indidually.
				for (int packetIdx = minPacketIdx; packetIdx <= maxPacketIdx; packetIdx++)
				{
					BundlePacket packet = bundleInfo.Header.Packets[packetIdx];
					await ReadEncodedPacketFromStreamAsync(bundle, packetIdx, packet, stream, cancellationToken);
				}
			}
		}

		async Task ReadEncodedPacketFromStreamAsync(QueuedBundle bundle, int packetIdx, BundlePacket packet, Stream stream, CancellationToken cancellationToken)
		{
			byte[] data = new byte[packet.EncodedLength];
			await stream.ReadFixedLengthBytesAsync(data, cancellationToken);

			lock (_queueLock)
			{
				_cache.AddCachedEncodedPacket(bundle.Locator, packetIdx, data); // Note: do this while holding the lock, to avoid races with new encodes being added

				List<QueuedPacket> packets = bundle.QueuedPackets;
				for (int idx = 0; idx < packets.Count && packets[idx].PacketIdx <= packetIdx; idx++)
				{
					if (packets[idx].PacketIdx == packetIdx)
					{
						packets[idx].CompletionSource.TrySetResult(data);
						packets.RemoveAt(idx--);
					}
				}
			}
		}

		#endregion

		/// <summary>
		/// Reads a node from a bundle
		/// </summary>
		/// <param name="bundleLocator">Locator for the bundle</param>
		/// <param name="exportIdx">Index of the export</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Node data read from the given bundle</returns>
		public async ValueTask<BlobData> ReadNodeDataAsync(BlobLocator bundleLocator, int exportIdx, CancellationToken cancellationToken = default)
		{
			BundleHeader header = await ReadHeaderAsync(bundleLocator, cancellationToken);
			BundleExport export = header.Exports[exportIdx];

			Dictionary<BlobLocator, BundleHandle> locatorToBundleHandle = [];

			List<IBlobRef> imports = new(export.References.Count);
			foreach (BundleExportRef reference in export.References)
			{
				BlobLocator importBlob;
				if (reference.ImportIdx == -1)
				{
					importBlob = bundleLocator;
				}
				else
				{
					importBlob = header.Imports[reference.ImportIdx];
				}

				BundleHandle? importBundle;
				if (!locatorToBundleHandle.TryGetValue(importBlob, out importBundle))
				{
					importBundle = new FlushedBundleHandle(_store, importBlob);
					locatorToBundleHandle.Add(importBlob, importBundle);
				}

				Debug.Assert(importBlob.IsValid());
				imports.Add(new FlushedNodeHandle(this, importBlob, importBundle, reference.NodeIdx));
			}

			ReadOnlyMemory<byte> nodeData = ReadOnlyMemory<byte>.Empty;
			if (export.Length > 0)
			{
				ReadOnlyMemory<byte> packetData = await ReadPacketAsync(bundleLocator, export.Packet, cancellationToken);
				nodeData = packetData.Slice(export.Offset, export.Length);
			}

			BlobType nodeType = header.Types[export.TypeIdx];
			return new BlobData(nodeType, nodeData, imports);
		}

		/// <summary>
		/// Gets stats for the reader
		/// </summary>
		public void GetStats(StorageStats stats)
		{
			if (_numBytesRead > 0)
			{
				stats.Add("Num bytes read (v1)", _numBytesRead);
			}
			if (_numHeaderReads > 0)
			{
				stats.Add("Num header reads (v1)", _numHeaderReads);
			}
			if (_numPacketReads > 0)
			{
				stats.Add("Num packet reads (v1)", _numPacketReads);
			}
			if (_decodeTimeTicks > 0)
			{
				stats.Add("Decode time (ms) (v1)", (_decodeTimeTicks * 1000) / Stopwatch.Frequency);
			}
		}
	}
}
