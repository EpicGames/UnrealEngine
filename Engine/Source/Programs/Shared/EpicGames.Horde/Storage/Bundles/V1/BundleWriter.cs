// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Text;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Bundles.V1
{
	/// <summary>
	/// Implementation of <see cref="IBlobRef"/> for nodes which can be read from storage
	/// </summary>
	sealed class FlushedNodeHandle : IBlobRef
	{
		readonly BundleReader _reader;

		public BlobLocator BundleLocator { get; }
		public int ExportIdx { get; }

		/// <inheritdoc/>
		public BundleHandle Outer { get; }

		public IBlobRef Innermost => this;

		/// <summary>
		/// Constructor
		/// </summary>
		public FlushedNodeHandle(BundleReader reader, BlobLocator bundleLocator, BundleHandle bundleHandle, int exportIdx)
		{
			Debug.Assert(!bundleLocator.CanUnwrap());

			_reader = reader;
			BundleLocator = bundleLocator;
			Outer = bundleHandle;
			ExportIdx = exportIdx;
		}

		public FlushedNodeHandle(BundleReader reader, BlobLocator bundleLocator, BundleHandle bundleHandle, ReadOnlySpan<byte> fragment)
		{
			Debug.Assert(!bundleLocator.CanUnwrap());

			_reader = reader;
			BundleLocator = bundleLocator;
			Outer = bundleHandle;

			if (!Utf8Parser.TryParse(fragment, out int exportIdx, out int bytesConsumed) || bytesConsumed != fragment.Length)
			{
				throw new ArgumentException($"Fragment {Encoding.UTF8.GetString(fragment)} is not valid for a bundle export");
			}

			ExportIdx = exportIdx;
		}

		/// <inheritdoc/>
		public bool TryAppendIdentifier(Utf8StringBuilder builder)
		{
			builder.Append(ExportIdx);
			return true;
		}

		/// <inheritdoc/>
		public bool TryGetLocator(out BlobLocator locator)
		{
			locator = new BlobLocator($"{BundleLocator}#{ExportIdx}");
			return true;
		}

		/// <inheritdoc/>
		public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default) => _reader.ReadNodeDataAsync(BundleLocator, ExportIdx, cancellationToken);

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken = default) => new();

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is FlushedNodeHandle other && BundleLocator == other.BundleLocator && ExportIdx == other.ExportIdx;

		/// <inheritdoc/>
		public override int GetHashCode() => HashCode.Combine(BundleLocator, ExportIdx);
	}

	/// <summary>
	/// Writes nodes of a tree to an <see cref="IStorageNamespace"/>, packed into bundles. Each <see cref="BundleWriter"/> instance is single threaded,
	/// but multiple instances may be written to in parallel.
	/// </summary>
	public sealed class BundleWriter : BlobWriter
	{
		// Information about a unique output node. Note that multiple node refs may de-duplicate to the same output node.
		internal class PendingNode : IHashedBlobRef
		{
			readonly BundleReader _reader;

			object LockObject => _reader;

			FlushedNodeHandle? _flushedHandle;
			PendingBundle? _pendingBundle;

			public readonly BlobType BlobType;
			public IoHash Hash { get; }
			public readonly int Packet;
			public readonly int Offset;
			public readonly int Length;
			public readonly IBlobRef[] Imports;
			public readonly AliasInfo[] Aliases;

			public IBlobRef Innermost => this;
			public PendingBundle? PendingBundle => _pendingBundle;
			public FlushedNodeHandle? FlushedNodeHandle => _flushedHandle;

			public BundleHandle Outer => (_flushedHandle != null) ? _flushedHandle.Outer : throw new NotSupportedException();

			public PendingNode(BundleReader reader, BlobType blobType, IoHash hash, int packet, int offset, int length, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, PendingBundle pendingBundle)
			{
				_reader = reader;

				BlobType = blobType;
				Hash = hash;
				Packet = packet;
				Offset = offset;
				Length = length;
				Imports = [.. imports.Select(x => x.Innermost)];
				Aliases = [.. aliases];

				_pendingBundle = pendingBundle;
			}

			/// <inheritdoc/>
			public bool TryAppendIdentifier(Utf8StringBuilder builder)
			{
				return _flushedHandle?.TryAppendIdentifier(builder) ?? false;
			}

			public void MarkAsWritten(FlushedNodeHandle flushedHandle)
			{
				lock (LockObject)
				{
					Debug.Assert(_flushedHandle == null);
					_flushedHandle = flushedHandle;
					_pendingBundle = null;
				}
			}

			/// <inheritdoc/>
			public async ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
			{
				if (_flushedHandle == null)
				{
					lock (LockObject)
					{
						if (_flushedHandle == null)
						{
							ReadOnlyMemory<byte> data = _pendingBundle!.GetNodeData(Packet, Offset, Length);
							return new BlobData(BlobType, data, Imports);
						}
					}
				}

				return await _flushedHandle!.ReadBlobDataAsync(cancellationToken);
			}

			/// <inheritdoc/>
			public async ValueTask FlushAsync(CancellationToken cancellationToken = default)
			{
				if (_flushedHandle != null)
				{
					return;
				}

				foreach (IBlobRef import in Imports)
				{
					await import.FlushAsync(cancellationToken);
				}

				PendingBundle? pendingBundle = _pendingBundle;
				if (pendingBundle != null)
				{
					await pendingBundle.FlushAsync(cancellationToken);
				}
			}

			/// <inheritdoc/>
			public bool TryGetLocator(out BlobLocator locator)
			{
				if (_flushedHandle == null)
				{
					locator = default;
					return false;
				}
				return _flushedHandle.TryGetLocator(out locator);
			}

			/// <inheritdoc/>
			public override bool Equals(object? obj) => ReferenceEquals(this, obj);

			/// <inheritdoc/>
			public override int GetHashCode() => RuntimeHelpers.GetHashCode(this);
		}

		// Information about a bundle being built. Metadata operations are synchronous, compression/writes are asynchronous.
		internal class PendingBundle : IDisposable
		{
			static int s_lastBundleId = 0;
			public int BundleId { get; } = Interlocked.Increment(ref s_lastBundleId);

			readonly BundleReader _treeReader;
			readonly BundleWriter _treeWriter;
			readonly int _maxPacketSize;
			readonly BundleCompressionFormat _compressionFormat;

			readonly Lock _lockObject = new();

			// The current packet being built
			IMemoryOwner<byte>? _currentPacket;

			// Index of the current packet being written to
			int _currentPacketIdx;

			// Length of the packet that has been written
			int _currentPacketLength;

			// Total size of compressed data in the current bundle
			long _compressedLength;

			// Queue of nodes for the current bundle
			readonly List<PendingNode> _queue = [];

			// Number of references in the queue
			int _queuedRefs = 0;

			// List of packets in the current bundle
			readonly List<BundlePacket> _packets = [];

			// List of compressed packets
			readonly ChunkedMemoryWriter _encodedPacketWriter;

			// Set of all direct dependencies from this bundle
			readonly HashSet<Task> _dependencies = [];

			// Total size of compressed data in the current bundle
			public long CompressedLength => _compressedLength;

			// Total size of uncompressed data in the current bundle
			public long UncompressedLength { get; private set; }

			// Task used to compress data in the background
			Task _compressPacketsTask = Task.CompletedTask;

			// Event which is signalled after the bundle is written to storage
			readonly TaskCompletionSource<bool> _completeEvent = new(TaskCreationOptions.RunContinuationsAsynchronously);

			// Task signalled after the write is complete
			public Task CompleteTask => _completeEvent.Task;

			public PendingBundle(BundleReader treeReader, BundleWriter treeWriter, int maxPacketSize, BundleCompressionFormat compressionFormat)
			{
				_treeReader = treeReader;
				_treeWriter = treeWriter;
				_maxPacketSize = maxPacketSize;
				_compressionFormat = compressionFormat;
				_encodedPacketWriter = new ChunkedMemoryWriter(1024 * 1024);
			}

			/// <inheritdoc/>
			public void Dispose()
			{
				if (!CompleteTask.IsCompleted)
				{
					_completeEvent.TrySetException(new ObjectDisposedException($"Bundle {BundleId} has been disposed. Make sure to call FlushAsync() on forked writers that may have dependencies before disposing.", $"Bundle {BundleId}"));
				}

				_currentPacket?.Dispose();
				_currentPacket = null;
				_encodedPacketWriter.Dispose();
			}

			// Whether this bundle is full
			public bool IsFull() => (_queue.Count + 1000) >= BundleHeader.MaxExports || (_queuedRefs + 1000) >= BundleHeader.MaxExportRefs;

			// Whether this bundle has finished writing
			public bool IsComplete() => CompleteTask.IsCompleted;

			// Add a dependency onto another bundle
			public void AddDependencyOn(PendingBundle bundle, ILogger? traceLogger)
			{
				if (bundle != this)
				{
					if (_dependencies.Add(bundle.CompleteTask))
					{
						traceLogger?.LogInformation("Added dependency from bundle {BundleId} on {OtherBundleId}", BundleId, bundle.BundleId);
					}
				}
			}

			// Starts writing a node to the buffer
			public Memory<byte> GetBuffer(int usedSize, int desiredSize)
			{
				if (_currentPacket == null)
				{
					_currentPacket = MemoryPool<byte>.Shared.Rent(Math.Max(_maxPacketSize, desiredSize));
				}
				else if (_currentPacketLength + desiredSize > _currentPacket.Memory.Length)
				{
					IMemoryOwner<byte> nextPacket = MemoryPool<byte>.Shared.Rent(Math.Max(_maxPacketSize, desiredSize));
					if (usedSize > 0)
					{
						_currentPacket.Memory.Slice(_currentPacketLength, usedSize).CopyTo(nextPacket.Memory);
					}
					FlushPacket();
					_currentPacket = nextPacket;
				}
				return _currentPacket.Memory.Slice(_currentPacketLength, desiredSize);
			}

			// Finish a node write
			public PendingNode WriteNode(BlobType blobType, int size, IReadOnlyList<IBlobRef> refs, IReadOnlyList<AliasInfo> aliases)
			{
				IoHash hash = IoHash.Compute(GetBuffer(size, size).Span);

				PendingNode pendingNode = new(_treeReader, blobType, hash, _currentPacketIdx, _currentPacketLength, (int)size, refs, aliases, this);
				_currentPacketLength += pendingNode.Length;
				UncompressedLength += pendingNode.Length;

				_queue.Add(pendingNode);
				_queuedRefs += pendingNode.Imports.Length;

				return pendingNode;
			}

			public ReadOnlyMemory<byte> GetNodeData(int packetIdx, int offset, int length)
			{
				lock (_lockObject)
				{
					if (packetIdx < _packets.Count)
					{
						BundlePacket packet = _packets[packetIdx];

						ReadOnlySequence<byte> sequence = _encodedPacketWriter.AsSequence(packet.EncodedOffset, packet.EncodedLength);
						if (!sequence.IsSingleSegment)
						{
							throw new InvalidOperationException();
						}

						using (IMemoryOwner<byte> decodeBuffer = MemoryPool<byte>.Shared.Rent(packet.DecodedLength))
						{
							BundleData.Decompress(_compressionFormat, sequence.First, decodeBuffer.Memory);
							return decodeBuffer.Memory.Slice(offset, length).ToArray();
						}
					}
					else if (packetIdx == _currentPacketIdx)
					{
						return _currentPacket!.Memory.Slice(offset, length).ToArray();
					}
					else
					{
						return ReadOnlyMemory<byte>.Empty;
					}
				}
			}

			// Compresses the current packet and schedule it to be written to storage
			void FlushPacket()
			{
				if (_currentPacket != null)
				{
					IMemoryOwner<byte> currentPacket = _currentPacket;
					int currentPacketLength = _currentPacketLength;

					if (currentPacketLength > 0)
					{
						_compressPacketsTask = _compressPacketsTask.ContinueWith(x => CompressPacket(currentPacket, currentPacketLength), TaskScheduler.Default);
						_currentPacketIdx++;
					}
					else
					{
						currentPacket.Dispose();
					}

					_currentPacket = null;
					_currentPacketLength = 0;
				}
			}

			void CompressPacket(IMemoryOwner<byte> packetData, int length)
			{
				lock (_lockObject)
				{
					int encodedLength = BundleData.Compress(_compressionFormat, packetData.Memory.Slice(0, length), _encodedPacketWriter);

					int encodedOffset = (_packets.Count == 0) ? 0 : _packets[^1].EncodedOffset + _packets[^1].EncodedLength;
					BundlePacket packet = new(_compressionFormat, encodedOffset, encodedLength, length);
					_packets.Add(packet);
					Interlocked.Add(ref _compressedLength, encodedLength);

					packetData.Dispose();
				}
			}

			// Wait for the bundle's dependencies to complete
			public async Task FlushDependenciesAsync(ILogger? traceLogger, CancellationToken cancellationToken)
			{
				FlushPacket();
				await _compressPacketsTask.WaitAsync(cancellationToken);

				if (traceLogger != null)
				{
					foreach (Task dependency in _dependencies)
					{
						if (!dependency.IsCompleted)
						{
							traceLogger.LogInformation("Bundle {BundleId} is stalling waiting for dependencies to flush first", BundleId);
						}
					}
				}

				await Task.WhenAll(_dependencies).WaitAsync(cancellationToken);
			}

			// Mark the bundle as complete
			public async Task WriteAsync(BundleStorageNamespace store, string? basePath, ILogger? traceLogger)
			{
				traceLogger?.LogInformation("Marking bundle {BundleId} as complete ({NumNodes} nodes); adding to write queue.", BundleId, _queue.Count);

				await FlushDependenciesAsync(traceLogger, CancellationToken.None);

				try
				{
					(BundleHeader header, List<ReadOnlyMemory<byte>> packets) = CreateBundle();

					// Write the bundle to storage
					HashSet<BlobLocator> imports = [];
					foreach (BlobLocator import in header.Imports)
					{
						imports.Add(import.BaseLocator);
					}

					// Create the output sequence
					ReadOnlySequenceBuilder<byte> sequence = new();
					header.AppendTo(sequence);

					foreach (ReadOnlyMemory<byte> packet in packets)
					{
						sequence.Append(packet);
					}

					// Write it
					BlobLocator locator;
					using (ReadOnlySequenceStream stream = new(sequence.Construct()))
					{
						locator = await store.Backend.WriteBlobAsync(stream, imports, basePath, CancellationToken.None);
					}

					traceLogger?.LogInformation("Written bundle {BundleId} as {Locator}", BundleId, locator);

					FlushedBundleHandle handle = new(store, locator);
					for (int idx = 0; idx < _queue.Count; idx++)
					{
						PendingNode node = _queue[idx];

						FlushedNodeHandle flushedHandle = new(_treeReader, locator, handle, idx);
						_queue[idx].MarkAsWritten(flushedHandle);

						foreach (AliasInfo alias in node.Aliases)
						{
							await store.AddAliasAsync(alias.Name, flushedHandle, alias.Rank, alias.Data, CancellationToken.None);
						}
					}

					_completeEvent.SetResult(true);
				}
				catch (Exception ex)
				{
					_completeEvent.SetException(ex);
				}
			}

			// Flush the current write state
			public Task FlushAsync(CancellationToken cancellationToken) => _treeWriter.FlushAsync(cancellationToken);

			// Mark the bundle as complete and create a bundle with the current state
			public (BundleHeader, List<ReadOnlyMemory<byte>>) CreateBundle()
			{
				lock (_lockObject)
				{
					return CreateBundleInternal();
				}
			}

			(BundleHeader, List<ReadOnlyMemory<byte>>) CreateBundleInternal()
			{
				// List of imported blobs
				List<BlobLocator> imports = [];
				Dictionary<BlobLocator, int> importToIndex = [];

				// List of types in the bundle
				List<BlobType> types = [];
				Dictionary<BlobType, int> typeToIndex = [];

				// Map of node handle to reference
				Dictionary<IBlobRef, BundleExportRef> nodeHandleToExportRef = [];
				for (int exportIdx = 0; exportIdx < _queue.Count; exportIdx++)
				{
					IBlobRef handle = _queue[exportIdx];
					nodeHandleToExportRef[handle] = new BundleExportRef(-1, exportIdx);
				}

				// Create the export list
				List<BundleExport> exports = new(_queue.Count);
				foreach (PendingNode nodeInfo in _queue)
				{
					int typeIdx = FindOrAddItemIndex(nodeInfo.BlobType, types, typeToIndex);

					List<BundleExportRef> exportRefs = [];
					foreach (IBlobRef import in nodeInfo.Imports)
					{
						BundleExportRef exportRef;
						if (!nodeHandleToExportRef.TryGetValue(import, out exportRef))
						{
							FlushedNodeHandle? flushedHandle = import as FlushedNodeHandle;
							if (flushedHandle == null)
							{
								flushedHandle = (import as PendingNode)?.FlushedNodeHandle;
								if (flushedHandle == null)
								{
									throw new InvalidOperationException($"Node {import.GetLocator()} is not a bundle node");
								}
							}

							int importIdx = FindOrAddItemIndex(flushedHandle.BundleLocator, imports, importToIndex);
							exportRef = new BundleExportRef(importIdx, flushedHandle.ExportIdx);
						}
						exportRefs.Add(exportRef);
					}

					BundleExport export = new(typeIdx, nodeInfo.Packet, nodeInfo.Offset, nodeInfo.Length, exportRefs);
					exports.Add(export);
				}

				// Get the memory for each packet
				List<ReadOnlyMemory<byte>> packetData = new(_packets.Count);
				foreach (ReadOnlyMemory<byte> segment in _encodedPacketWriter.AsSequence())
				{
					int offset = 0;
					while (packetData.Count < _packets.Count)
					{
						BundlePacket packet = _packets[packetData.Count];
						int next = offset + packet.EncodedLength;

						if (next > segment.Length)
						{
							break;
						}
						packetData.Add(segment.Slice(offset, packet.EncodedLength));
						offset = next;
					}
				}

				// Create the bundle
				BundleHeader header = new([.. types], [.. imports], [.. exports], [.. _packets]);
				return (header, packetData);
			}

			static int FindOrAddItemIndex<TItem>(TItem item, List<TItem> items, Dictionary<TItem, int> itemToIndex) where TItem : notnull
			{
				int index;
				if (!itemToIndex.TryGetValue(item, out index))
				{
					index = items.Count;
					items.Add(item);
					itemToIndex.Add(item, index);
				}
				return index;
			}
		}

		class WriteQueue
		{
			long _memoryFootprint;
			readonly BundleStorageNamespace _store;
			readonly string? _basePath;
			readonly long _maxMemoryFootprint;
			readonly ILogger? _traceLogger;
			readonly AsyncEvent _completeEvent = new();
			int _refCount;
			readonly List<Task> _writeTasks = [];

			public WriteQueue(BundleStorageNamespace store, string? basePath, long maxMemoryFootprint, ILogger? traceLogger)
			{
				_store = store;
				_basePath = basePath;
				_maxMemoryFootprint = maxMemoryFootprint;
				_refCount = 1;
				_traceLogger = traceLogger;
			}

			public void AddRef()
			{
				Interlocked.Increment(ref _refCount);
			}

			public async ValueTask ReleaseAsync()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					await FlushAsync(CancellationToken.None);
				}
			}

			public async Task AddAsync(PendingBundle pendingBundle, CancellationToken cancellationToken)
			{
				lock (_writeTasks)
				{
					_writeTasks.RemoveCompleteTasks();
				}

				await pendingBundle.FlushDependenciesAsync(_traceLogger, cancellationToken);

				for (; ; )
				{
					Task completeTask = _completeEvent.Task;

					long memoryFootprint = Interlocked.CompareExchange(ref _memoryFootprint, 0, 0);
					long newMemoryFootprint = memoryFootprint + pendingBundle.CompressedLength;

					if (memoryFootprint > 0 && newMemoryFootprint > _maxMemoryFootprint)
					{
						await completeTask;
					}
					else if (Interlocked.CompareExchange(ref _memoryFootprint, newMemoryFootprint, memoryFootprint) == memoryFootprint)
					{
						break;
					}
				}

				lock (_writeTasks)
				{
					_writeTasks.Add(WriteAsync(pendingBundle));
				}
			}

			public async Task FlushAsync(CancellationToken cancellationToken)
			{
				Task[] writeTasks;
				lock (_writeTasks)
				{
					writeTasks = [.. _writeTasks];
				}
				await Task.WhenAll(writeTasks).WaitAsync(cancellationToken);
			}

			async Task WriteAsync(PendingBundle pendingBundle)
			{
				await pendingBundle.WriteAsync(_store, _basePath, _traceLogger);
				Interlocked.Add(ref _memoryFootprint, -pendingBundle.CompressedLength);
				pendingBundle.Dispose();
				_completeEvent.Pulse();
			}
		}

		static readonly BundleOptions s_defaultOptions = new();

		readonly BundleStorageNamespace _store;
		readonly BundleReader _reader;
		readonly BundleOptions _bundleOptions;
		readonly string? _basePath;

		readonly WriteQueue _writeQueue;

		PendingBundle? _currentBundle;

		bool _disposed;

		internal ILogger? TraceLogger { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="store">Store to write data to</param>
		/// <param name="reader">Reader for serialized node data</param>
		/// <param name="basePath">Base path for new nodes</param>
		/// <param name="bundleOptions">Options for the writer</param>
		/// <param name="blobSerializerOptions"></param>
		/// <param name="traceLogger">Optional logger for trace information</param>
		public BundleWriter(BundleStorageNamespace store, BundleReader reader, string? basePath, BundleOptions? bundleOptions = null, BlobSerializerOptions? blobSerializerOptions = null, ILogger? traceLogger = null)
			: this(store, reader, basePath, bundleOptions, null, blobSerializerOptions, traceLogger)
		{
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other"></param>
		public BundleWriter(BundleWriter other)
			: this(other._store, other._reader, other._basePath, other._bundleOptions, other._writeQueue, other.Options, other.TraceLogger)
		{
			_writeQueue.AddRef();
		}

		/// <summary>
		/// Internal constructor
		/// </summary>
		private BundleWriter(BundleStorageNamespace store, BundleReader reader, string? basePath, BundleOptions? bundleOptions, WriteQueue? writeQueue, BlobSerializerOptions? blobSerializerOptions, ILogger? traceLogger = null)
			: base(blobSerializerOptions)
		{
			_store = store;
			_reader = reader;
			_basePath = basePath;
			_bundleOptions = bundleOptions ?? s_defaultOptions;
			_writeQueue = writeQueue ?? new WriteQueue(store, basePath, _bundleOptions.MaxWriteQueueLength, traceLogger);
			TraceLogger = traceLogger;
		}

		/// <inheritdoc/>
		public override IBlobWriter Fork() => new BundleWriter(this);

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			if (!_disposed)
			{
				await FlushAsync();
				await _writeQueue.ReleaseAsync();
				_disposed = true;
			}

			_currentBundle?.Dispose();
			_currentBundle = null;
		}

		/// <summary>
		/// Mark this writer as complete, allowing its data to be serialized.
		/// </summary>
		public async Task CompleteAsync(CancellationToken cancellationToken)
		{
			if (_currentBundle != null)
			{
				await _writeQueue.AddAsync(_currentBundle, cancellationToken);
				_currentBundle = null;
			}
		}

		/// <summary>
		/// Gets an output buffer for writing.
		/// </summary>
		/// <param name="usedSize">Current size in the existing buffer that has been written to</param>
		/// <param name="desiredSize">Desired size of the returned buffer</param>
		/// <returns>Buffer to be written into.</returns>
		public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
		{
			PendingBundle currentBundle = GetCurrentBundle();
			return currentBundle.GetBuffer(usedSize, desiredSize);
		}

		/// <summary>
		/// Finish writing a node.
		/// </summary>
		/// <param name="type">Type of the node that was written</param>
		/// <param name="size">Used size of the buffer</param>
		/// <param name="imports">References to other nodes</param>
		/// <param name="aliases">Aliases for the node</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the written node</returns>
		public override async ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken = default)
		{
			PendingBundle currentBundle = GetCurrentBundle();

			// Append this node data
			PendingNode pendingNode = currentBundle.WriteNode(type, size, imports, aliases);
			TraceLogger?.LogInformation("Added new node for {NodeKey} in bundle {BundleId}", pendingNode, currentBundle.BundleId);

			// Add dependencies on all bundles containing a dependent node
			foreach (IBlobRef import in imports)
			{
				PendingNode? pendingReference = import.Innermost as PendingNode;
				if (pendingReference?.PendingBundle != null)
				{
					currentBundle.AddDependencyOn(pendingReference.PendingBundle, TraceLogger);
				}
			}

			// If the bundle is full, start the process of writing it to disk
			if (currentBundle.UncompressedLength > _bundleOptions.MaxBlobSize || currentBundle.IsFull())
			{
				await CompleteAsync(cancellationToken);
			}

			return pendingNode;
		}

		PendingBundle GetCurrentBundle()
		{
			if (_currentBundle == null)
			{
				int bufferSize = (int)(_bundleOptions.MinCompressionPacketSize * 1.2);
				_currentBundle = new PendingBundle(_reader, this, bufferSize, _bundleOptions.CompressionFormat);
			}
			return _currentBundle;
		}

		/// <summary>
		/// Flushes all the current nodes to storage
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public override async Task FlushAsync(CancellationToken cancellationToken = default)
		{
			ObjectDisposedException.ThrowIf(_disposed, this);

			await CompleteAsync(cancellationToken);
			await _writeQueue.FlushAsync(cancellationToken);
		}
	}
}
