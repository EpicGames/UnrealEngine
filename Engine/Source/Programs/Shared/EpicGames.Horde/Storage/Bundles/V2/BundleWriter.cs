// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Implements the primary storage writer interface for V2 bundles. Writes exports into packets, and flushes them to storage in bundles.
	/// </summary>
	public sealed class BundleWriter : BlobWriter
	{
		// Packet that is still being built, but may be redirected to a flushed packet
		internal sealed class PendingPacketHandle : PacketHandle
		{
			readonly PendingBundleHandle _bundle;
			FlushedPacketHandle? _flushedHandle;

			public override BundleHandle Bundle => _flushedHandle?.Bundle ?? _bundle;
			public FlushedPacketHandle? FlushedHandle => _flushedHandle;

			public PendingPacketHandle(PendingBundleHandle bundle) => _bundle = bundle;

			public override ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> _bundle.FlushAsync(cancellationToken);

			public void CompletePacket(FlushedPacketHandle flushedHandle)
				=> _flushedHandle = flushedHandle;

			public override async ValueTask<BlobData> ReadExportAsync(int exportIdx, CancellationToken cancellationToken = default)
			{
				lock (_bundle.LockObject)
				{
					if (_flushedHandle == null)
					{
						return _bundle.GetPendingExport(exportIdx);
					}
				}
				return await _flushedHandle!.ReadExportAsync(exportIdx, cancellationToken);
			}

			public override bool TryAppendIdentifier(Utf8StringBuilder builder)
				=> _flushedHandle?.TryAppendIdentifier(builder) ?? false;
		}

		// Bundle that has not yet been written to storage.
		internal sealed class PendingBundleHandle : BundleHandle, IDisposable
		{
			public readonly Lock LockObject = new();

			readonly BundleWriter _outer;
			readonly BundleStorageNamespace _storageNamespace;
			readonly BundleCache _bundleCache;
			readonly BundleOptions _bundleOptions;

			FlushedBundleHandle? _flushedHandle;

			PacketWriter? _packetWriter;
			PendingPacketHandle? _packetHandle;
			List<(ExportHandle, AliasInfo)>? _pendingExportAliases;
			RefCountedMemoryWriter? _encodedPacketWriter;
			readonly HashSet<BlobLocator> _bundleImports = [];

			public Task? _lockTask;
			public Task? _writeTask;

			public FlushedBundleHandle? FlushedHandle => _flushedHandle;

			/// <summary>
			/// Compressed length of this bundle
			/// </summary>
			public int Length => _encodedPacketWriter?.Length ?? throw new InvalidOperationException("Bundle has been flushed");

			public PendingBundleHandle(BundleWriter outer, BlobLocator locator)
				: base(locator)
			{
				_outer = outer;
				_storageNamespace = _outer._storageNamespace;
				_bundleCache = _outer._bundleCache;
				_bundleOptions = _outer._bundleOptions;

				_encodedPacketWriter = new RefCountedMemoryWriter(_bundleCache.Allocator, 65536, nameof(PendingBundleHandle));

				StartPacket();
			}

			public void Dispose() => ReleaseResources();

			void ReleaseResources()
			{
				_packetWriter?.Dispose();
				_packetWriter = null;
				_encodedPacketWriter?.Dispose();
				_encodedPacketWriter = null;

				// Also clear out any arrays that can be GC'd
				_packetHandle = null;
				_pendingExportAliases = null;
			}

			public Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
			{
				RuntimeAssert(_packetWriter != null);
				return _packetWriter!.GetOutputBuffer(usedSize, desiredSize);
			}

			public BlobData GetPendingExport(int exportIdx)
			{
				RuntimeAssert(_packetWriter != null);
				return _packetWriter.GetExport(exportIdx);
			}

			public HashedExportHandle CompleteExport(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases)
			{
				RuntimeAssert(_packetWriter != null);
				RuntimeAssert(_packetHandle != null);

				IoHash hash = IoHash.Compute(_packetWriter.GetOutputBuffer(size, size).Span);

				int exportIdx = _packetWriter.CompleteExport(size, type, imports);
				HashedExportHandle exportHandle = new(hash, _packetHandle, exportIdx);

				if (aliases.Count > 0)
				{
					_pendingExportAliases ??= [];
					_pendingExportAliases.AddRange(aliases.Select(x => ((ExportHandle)exportHandle, x)));
				}

				if (_packetWriter.Length > Math.Min(_bundleOptions.MinCompressionPacketSize, _bundleOptions.MaxBlobSize))
				{
					FinishPacket();
					StartPacket();
				}

				return exportHandle;
			}

			void StartPacket()
			{
				RuntimeAssert(_packetHandle == null);
				RuntimeAssert(_packetWriter == null);

				_packetHandle = new PendingPacketHandle(this);
				_packetWriter = new PacketWriter(this, _packetHandle, _bundleCache.Allocator, LockObject);
			}

			void FinishPacket()
			{
				RuntimeAssert(_packetHandle != null);
				RuntimeAssert(_packetWriter != null);
				RuntimeAssert(_encodedPacketWriter != null);

				if (_packetWriter.GetExportCount() > 0)
				{
					int packetOffset = _encodedPacketWriter.Length;
					Packet packet = _packetWriter.CompletePacket();
					packet.Encode(_bundleOptions.CompressionFormat, _encodedPacketWriter);
					int packetLength = _encodedPacketWriter.Length - packetOffset;

					// Point the packet handle to the encoded data
					lock (LockObject)
					{
						FlushedPacketHandle flushedPacketHandle = new(_storageNamespace, this, packetOffset, packetLength, _bundleCache);
						_packetHandle.CompletePacket(flushedPacketHandle);
					}
				}

				_bundleImports.UnionWith(_packetWriter.BundleImports);

				_packetWriter.Dispose();
				_packetWriter = null;

				_packetHandle = null;
			}

			public override async ValueTask FlushAsync(CancellationToken cancellationToken = default)
			{
				if (_flushedHandle == null)
				{
					Task writeTask = await _outer.StartWriteAsync(this, cancellationToken);
					await writeTask.WaitAsync(cancellationToken);
				}
			}

			// Handle writing the bundle to storage
			public async Task WriteAsync(CancellationToken cancellationToken = default)
			{
				if (_encodedPacketWriter == null)
				{
					return;
				}

				if (_packetWriter != null)
				{
					FinishPacket();
				}

				if (_encodedPacketWriter.Length == 0)
				{
					return;
				}

				// Write the bundle data
				FlushedBundleHandle flushedHandle;
				using (ReadOnlySequenceStream stream = new(_encodedPacketWriter.AsSequence()))
				{
					await _storageNamespace.Backend.WriteBlobAsync(Locator, stream, _bundleImports, cancellationToken);
					flushedHandle = new FlushedBundleHandle(_storageNamespace, Locator);
				}

				// Release all the intermediate data
				List<(ExportHandle, AliasInfo)>? pendingExportAliases = _pendingExportAliases;
				lock (LockObject)
				{
					_flushedHandle = flushedHandle;
					ReleaseResources();
				}

				// TODO: put all the encoded packets into the cache using the final handles

				// Add all the aliases
				if (pendingExportAliases != null)
				{
					foreach ((ExportHandle exportHandle, AliasInfo aliasInfo) in pendingExportAliases)
					{
						await _storageNamespace.AddAliasAsync(aliasInfo.Name, exportHandle, aliasInfo.Rank, aliasInfo.Data, cancellationToken);
					}
				}
			}

			/// <inheritdoc/>
			public override async Task<Stream> OpenAsync(int offset, int? length, CancellationToken cancellationToken = default)
			{
				if (_flushedHandle == null)
				{
					IReadOnlyMemoryOwner<byte> owner = await ReadAsync(offset, length, cancellationToken);
					return owner.AsStream();
				}

				return await _flushedHandle.OpenAsync(offset, length, cancellationToken);
			}

			/// <inheritdoc/>
			public override async ValueTask<IReadOnlyMemoryOwner<byte>> ReadAsync(int offset, int? length, CancellationToken cancellationToken = default)
			{
				if (_flushedHandle == null)
				{
					lock (LockObject)
					{
#pragma warning disable CA1508 // Avoid dead conditional code - need to re-check after locking.
						if (_flushedHandle == null)
#pragma warning restore CA1508
						{
							Debug.Assert(_encodedPacketWriter != null);
							int fetchLength = length ?? (_encodedPacketWriter.Length - offset);

							IRefCountedHandle<ReadOnlyMemory<byte>> handle = _encodedPacketWriter.AsRefCountedMemory(offset, fetchLength);
							return ReadOnlyMemoryOwner.Create(handle.Target, handle);
						}
					}
				}

				return await _flushedHandle.ReadAsync(offset, length, cancellationToken);
			}
		}

		readonly BundleStorageNamespace _storageNamespace;
		readonly string? _basePath;
		readonly BundleOptions _bundleOptions;
		readonly BundleCache _bundleCache;
		readonly ILogger _logger;
		readonly CancellationToken _cancellationToken;

		readonly string _locatorPrefix;
		int _locatorSuffix;

		PendingBundleHandle _currentBundle;

		readonly Lock _lockObject = new();
		Task _flushTask = Task.CompletedTask;

		/// <summary>
		/// Constructor
		/// </summary>
		public BundleWriter(BundleStorageNamespace storageNamespace, string? basePath, BundleCache bundleCache, BundleOptions bundleOptions, BlobSerializerOptions? blobOptions, ILogger logger, CancellationToken cancellationToken)
			: base(blobOptions)
		{
			string instanceSuffix = Guid.NewGuid().ToString("n");

			_storageNamespace = storageNamespace;
			_basePath = basePath;
			_bundleOptions = bundleOptions;
			_bundleCache = bundleCache;
			_logger = logger;
			_cancellationToken = cancellationToken;

			if (String.IsNullOrEmpty(basePath))
			{
				_locatorPrefix = instanceSuffix;
			}
			else if (basePath.EndsWith('/'))
			{
				_locatorPrefix = $"{basePath}{instanceSuffix}";
			}
			else
			{
				_locatorPrefix = $"{basePath}/{instanceSuffix}";
			}

			_currentBundle = new PendingBundleHandle(this, GetNextLocator());
		}

		BlobLocator GetNextLocator()
			=> new($"{_locatorPrefix}_{Interlocked.Increment(ref _locatorSuffix)}");

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await FlushAsync();
			_currentBundle.Dispose();
		}

		async Task WriteCurrentBundleAsync(CancellationToken cancellationToken = default)
		{
			await StartWriteAsync(_currentBundle, cancellationToken);
			_currentBundle = new PendingBundleHandle(this, GetNextLocator());
		}

		async Task<Task> StartWriteAsync(PendingBundleHandle bundle, CancellationToken cancellationToken)
		{
			lock (bundle.LockObject)
			{
				Task lockTask = bundle._lockTask ??= _bundleCache.WriteSemaphore.WaitAsync(_cancellationToken);
				bundle._writeTask ??= Task.Run(() => HandleBundleWriteAsync(lockTask, bundle), CancellationToken.None);
			}

			try
			{
				await bundle._lockTask.WaitAsync(cancellationToken);
			}
			catch
			{
				await bundle._writeTask; // Allow this exception to be observed as well, since we're discarding it
				throw;
			}

			lock (_lockObject)
			{
				_flushTask = Task.WhenAll(_flushTask, bundle._writeTask);
			}

			return bundle._writeTask;
		}

		async Task HandleBundleWriteAsync(Task lockTask, PendingBundleHandle bundle)
		{
			// Wait until we've got the semaphore
			await lockTask;

			// Execute the write
			try
			{
				await bundle.WriteAsync(_cancellationToken);
			}
			catch (OperationCanceledException)
			{
				_logger.LogDebug("Write of {Locator} was cancelled", bundle.Locator);
				throw;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error writing {Locator}: {Message}", bundle.Locator, ex.Message);
				throw;
			}
			finally
			{
				bundle.Dispose();
				_bundleCache.WriteSemaphore.Release();
			}
		}

		/// <inheritdoc/>
		public override async Task FlushAsync(CancellationToken cancellationToken = default)
		{
			await WriteCurrentBundleAsync(cancellationToken);

			Task flushTask;
			lock (_lockObject)
			{
				flushTask = _flushTask;
			}

			await flushTask.WaitAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public override IBlobWriter Fork()
			=> new BundleWriter(_storageNamespace, _basePath, _bundleCache, _bundleOptions, Options, _logger, _cancellationToken);

		/// <inheritdoc/>
		public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
			=> _currentBundle.GetOutputBuffer(usedSize, desiredSize);

		/// <inheritdoc/>
		public override async ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> references, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken = default)
		{
			HashedExportHandle exportHandle = _currentBundle.CompleteExport(type, size, references, aliases);
			if (_currentBundle.Length > _bundleOptions.MaxBlobSize)
			{
				await WriteCurrentBundleAsync(cancellationToken);
			}
			return exportHandle;
		}

		/// <summary>
		/// Helper method to check a precondition is valid at runtime, regardless of build configuration.
		/// </summary>
		static void RuntimeAssert([DoesNotReturnIf(false)] bool condition, [CallerArgumentExpression(nameof(condition))] string? message = null)
		{
			if (!condition)
			{
				throw new InvalidOperationException($"Condition failed: {message}");
			}
		}
	}
}
