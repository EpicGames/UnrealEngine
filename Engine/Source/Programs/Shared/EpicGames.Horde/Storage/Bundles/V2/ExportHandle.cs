// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Text;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Handle class for blobs exported from a bundle
	/// </summary>
	public class ExportHandle : IBlobRef
	{
		static readonly Utf8String s_fragmentPrefix = new("exp=");

		/// <summary>
		/// Accessor for the packet that this export is in
		/// </summary>
		public PacketHandle Packet => _packet;

		/// <summary>
		/// Export index within the packet
		/// </summary>
		public int ExportIdx => _exportIdx;

		readonly PacketHandle _packet;
		readonly int _exportIdx;

		/// <inheritdoc/>
		public IBlobRef Innermost => this;

		/// <summary>
		/// Constructor
		/// </summary>
		public ExportHandle(PacketHandle packet, int exportIdx)
		{
			_packet = packet;
			_exportIdx = exportIdx;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ExportHandle(PacketHandle packet, ReadOnlySpan<byte> fragment)
		{
			_packet = packet;
			if (!TryParse(fragment, out _exportIdx))
			{
				throw new ArgumentException("Invalid fragment");
			}
		}

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken = default)
			=> Packet.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
			=> Packet.ReadExportAsync(ExportIdx, cancellationToken);

		/// <summary>
		/// Attempt to parse an export index from the given fragment
		/// </summary>
		public static bool TryParse(ReadOnlySpan<byte> fragment, out int exportIdx)
		{
			if (!fragment.StartsWith(s_fragmentPrefix))
			{
				exportIdx = 0;
				return false;
			}

			fragment = fragment.Slice(s_fragmentPrefix.Length);
			if (!Utf8Parser.TryParse(fragment, out exportIdx, out int numBytesRead) || numBytesRead != fragment.Length)
			{
				exportIdx = 0;
				return false;
			}

			return true;
		}

		/// <summary>
		/// Get an identifier for this export within the outer packet
		/// </summary>
		public Utf8String GetIdentifier()
		{
			Utf8StringBuilder builder = new();
			AppendIdentifier(builder);
			return builder.ToUtf8String();
		}

		/// <summary>
		/// Append a locator for this export to the given string builder
		/// </summary>
		public bool TryAppendLocator(Utf8StringBuilder builder)
		{
			if (Packet.TryAppendLocator(builder))
			{
				builder.Append('&');
				AppendIdentifier(builder);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Appends an export identifier to the given string builder
		/// </summary>
		public void AppendIdentifier(Utf8StringBuilder builder)
		{
			builder.Append(s_fragmentPrefix);
			builder.Append(_exportIdx);
		}

		/// <inheritdoc/>
		public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
		{
			Utf8StringBuilder builder = new();
			if (TryAppendLocator(builder))
			{
				locator = new BlobLocator(builder.ToUtf8String());
				return true;
			}
			else
			{
				locator = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
			=> obj is ExportHandle other && Packet.Equals(other.Packet) && ExportIdx == other.ExportIdx;

		/// <inheritdoc/>
		public override int GetHashCode()
			=> HashCode.Combine(Packet, ExportIdx);
	}

	/// <summary>
	/// Implementation of <see cref="ExportHandle"/> which also contains a hash
	/// </summary>
	public class HashedExportHandle : ExportHandle, IHashedBlobRef
	{
		readonly IoHash _hash;

		/// <inheritdoc/>
		public IoHash Hash => _hash;

		/// <summary>
		/// Constructor
		/// </summary>
		public HashedExportHandle(IoHash hash, PacketHandle packet, int exportIdx)
			: base(packet, exportIdx)
		{
			_hash = hash;
		}
	}
}
