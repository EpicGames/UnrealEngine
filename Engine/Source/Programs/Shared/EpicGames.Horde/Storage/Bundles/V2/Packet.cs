// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Accessor for data structures stored into a serialized bundle packet.
	/// </summary>
	/// <remarks>
	/// <code>
	/// Each raw packet contains:
	///  - 8 bytes: Standard bundle signature. The length field specifies the size of the following data, including the signature itself.
	///  - 4 bytes: Decoded packet length
	///  - 1 byte: Compression format
	///  - ?? bytes: Compressed packet data
	/// 
	/// After decoding, the packet contains the following:
	///  - 8 bytes: Standard bundle signature. The length field specifies the size of the following data, including the signature itself.
	///  - 4 bytes: offset of type table from the start of the packet
	///  - 4 bytes: offset of import table from the start of the packet
	///  - 4 bytes: offset of export table from the start of the packet
	///	
	/// The type table is constructed as:
	///  - 4 bytes: Number of entries
	///  - 20 bytes * Number of entries: BlobType data
	///	
	/// The import table is constructed as:
	///  - 4 bytes: Number of entries
	///  - 4 bytes * (Number of entries + 1): Offset of each entry from the start of the packet, with a sentinel value for the end of the last entry. Length of each entry is implicit by next entry - this entry.
	///	
	/// The export index is constructed as:
	///  - 4 bytes: Number of entries
	///  - 4 bytes * (Number of entries + 1): Offset of each entry from the start of the packet, with a sentinel value for the end of the last entry. Length of each entry is implicit by next entry - this entry.
	///	
	/// Each import is written as:
	///  - VarInt: Base import index, with a +1 bias, or zero for 'none'
	///  - Utf8 string containing fragment on top of base import, without a null terminator.
	///	
	/// Each export is written as:
	///  - 4 bytes: Length of payload
	///  - ?? bytes: Payload data
	///  - VarInt: Type index
	///  - VarInt: Number of imports
	///  - VarInt * Number of imports: Import index
	/// </code>
	/// </remarks>
	[DebuggerTypeProxy(typeof(Packet.DebugProxy))]
	public sealed class Packet
	{
		/// <summary>
		/// Type for packet blobs
		/// </summary>
		public static BlobType BlobType { get; } = new BlobType("{CD9A04EF-47D3-CAC1-492A-05A651E63081}", 1);

		class DebugProxy
		{
			public BlobType[] Types { get; }
			public PacketImport[] Imports { get; }
			public PacketExport[] Exports { get; }

			public DebugProxy(Packet packet)
			{
				Types = [.. Enumerable.Range(0, packet.GetTypeCount()).Select(packet.GetType)];
				Imports = [.. Enumerable.Range(0, packet.GetImportCount()).Select(packet.GetImport)];
				Exports = [.. Enumerable.Range(0, packet.GetExportCount()).Select(packet.GetExport)];
			}
		}

		readonly ReadOnlyMemory<byte> _data;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">Data for the packet</param>
		public Packet(ReadOnlyMemory<byte> data) => _data = data;

		/// <summary>
		/// Accessor for the underlying packet data
		/// </summary>
		public ReadOnlyMemory<byte> Data => _data;

		/// <summary>
		/// Length of this packet
		/// </summary>
		public int Length => _data.Length;

		/// <summary>
		/// Gets the number of types in this packet
		/// </summary>
		public int GetTypeCount()
		{
			ReadOnlySpan<byte> span = _data.Span;
			int typeTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[8..]);
			return BinaryPrimitives.ReadInt32LittleEndian(span[typeTableOffset..]);
		}

		/// <summary>
		/// Gets a type from the packet
		/// </summary>
		public BlobType GetType(int typeIdx)
		{
			ReadOnlySpan<byte> span = _data.Span;
			int typeTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[8..]);

			int typeCount = BinaryPrimitives.ReadInt32LittleEndian(span[typeTableOffset..]);
			if (typeIdx < 0 || typeIdx >= typeCount)
			{
				throw new IndexOutOfRangeException();
			}

			int typeOffset = typeTableOffset + sizeof(int) + typeIdx * BlobType.NumBytes;
			return BlobType.Read(span[typeOffset..]);
		}

		/// <summary>
		/// Gets the number of imports in this packet
		/// </summary>
		public int GetImportCount()
		{
			ReadOnlySpan<byte> span = _data.Span;
			int importTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[12..]);
			return BinaryPrimitives.ReadInt32LittleEndian(span[importTableOffset..]);
		}

		/// <summary>
		/// Gets the locator for a particular import
		/// </summary>
		/// <param name="importIdx">Index of the import to retrieve</param>
		public PacketImport GetImport(int importIdx)
		{
			ReadOnlySpan<byte> span = _data.Span;
			int importTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[12..]);

			int importCount = BinaryPrimitives.ReadInt32LittleEndian(span[importTableOffset..]);
			if (importIdx < 0 || importIdx >= importCount)
			{
				throw new IndexOutOfRangeException();
			}

			int importEntryOffset = importTableOffset + sizeof(int) + importIdx * sizeof(int);

			int importOffset = BinaryPrimitives.ReadInt32LittleEndian(span[importEntryOffset..]);
			int importLength = BinaryPrimitives.ReadInt32LittleEndian(span[(importEntryOffset + 4)..]) - importOffset;

			return PacketImport.Read(_data.Slice(importOffset, importLength));
		}

		/// <summary>
		/// Gets the number of exports in this packet
		/// </summary>
		public int GetExportCount()
		{
			ReadOnlySpan<byte> span = _data.Span;
			int exportTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[16..]);
			return BinaryPrimitives.ReadInt32LittleEndian(span[exportTableOffset..]);
		}

		/// <summary>
		/// Gets the bulk data for a particular export
		/// </summary>
		/// <param name="exportIdx"></param>
		public PacketExport GetExport(int exportIdx)
		{
			ReadOnlySpan<byte> span = _data.Span;
			int exportTableOffset = BinaryPrimitives.ReadInt32LittleEndian(span[16..]);

			int exportCount = BinaryPrimitives.ReadInt32LittleEndian(span[exportTableOffset..]);
			if (exportIdx < 0 || exportIdx >= exportCount)
			{
				throw new IndexOutOfRangeException();
			}

			int exportEntryOffset = exportTableOffset + sizeof(int) + exportIdx * sizeof(int);

			int exportOffset = BinaryPrimitives.ReadInt32LittleEndian(span[exportEntryOffset..]);
			int exportLength = BinaryPrimitives.ReadInt32LittleEndian(span[(exportEntryOffset + 4)..]) - exportOffset;

			return new PacketExport(_data.Slice(exportOffset, exportLength));
		}

		/// <summary>
		/// Encodes a packet
		/// </summary>
		/// <param name="format">Compression format for the encoded data</param>
		/// <param name="writer">Writer for the encoded data</param>
		public void Encode(BundleCompressionFormat format, IMemoryWriter writer)
		{
			Span<byte> signatureSpan = writer.GetSpanAndAdvance(BundleSignature.NumBytes);
			writer.WriteInt32(_data.Length);
			writer.WriteUInt8((byte)format);

			int encodedLength = BundleData.Compress(format, _data, writer);

			BundleSignature signature = new(BundleVersion.LatestV2, BundleSignature.NumBytes + encodedLength + sizeof(int) + 1);
			signature.Write(signatureSpan);
		}

		/// <summary>
		/// Decodes this packet
		/// </summary>
		public static IRefCountedHandle<Packet> Decode(ReadOnlyMemory<byte> data, IMemoryAllocator<byte> allocator, object? allocationTag)
		{
			BundleSignature signature = BundleSignature.Read(data.Span);
			if (signature.Version <= BundleVersion.LatestV1 || signature.Version > BundleVersion.LatestV2)
			{
				throw new InvalidOperationException($"Cannot read bundle packet; unsupported version {(int)signature.Version}");
			}

			data = data.Slice(0, signature.HeaderLength);
			ReadOnlySpan<byte> span = data.Span.Slice(BundleSignature.NumBytes);

			int decodedLength = BinaryPrimitives.ReadInt32LittleEndian(span);
			span = span[4..];

			BundleCompressionFormat format = (BundleCompressionFormat)span[0];
			span = span[1..];

			IMemoryOwner<byte> owner = allocator.Alloc(decodedLength, allocationTag);
			Memory<byte> memory = owner.Memory.Slice(0, decodedLength);

			BundleData.Decompress(format, data.Slice(data.Length - span.Length), memory);

			Packet packet = new(memory);
			return RefCountedHandle.Create(packet, owner);
		}
	}

	/// <summary>
	/// Raw data for an imported node
	/// </summary>
	/// <param name="BaseIdx">Base index for this locator</param>
	/// <param name="Fragment">The utf8 fragment appended to the base index</param>
	public readonly record struct PacketImport(int BaseIdx, Utf8String Fragment)
	{
		/// <summary>
		/// Bias for indexes into the import table
		/// </summary>
		public const int Bias = 3;

		/// <summary>
		/// 
		/// </summary>
		public const int InvalidBaseIdx = -1;

		/// <summary>
		/// 
		/// </summary>
		public const int CurrentPacketBaseIdx = -2;

		/// <summary>
		/// 
		/// </summary>
		public const int CurrentBundleBaseIdx = -3;

		/// <summary>
		/// Reads an import from a block of memory
		/// </summary>
		public static PacketImport Read(ReadOnlyMemory<byte> data)
		{
			ReadOnlySpan<byte> span = data.Span;
			int baseIdx = (int)VarInt.ReadUnsigned(span, out int bytesRead) - Bias;
			Utf8String fragment = new(data.Slice(bytesRead));
			return new PacketImport(baseIdx, fragment);
		}
	}

	/// <summary>
	/// Data for an exported node in a packet
	/// </summary>
	[DebuggerTypeProxy(typeof(PacketExport.DebugProxy))]
	public readonly struct PacketExport
	{
		class DebugProxy
		{
			public int TypeIdx { get; }
			public int[] Imports { get; }
			public ReadOnlyMemory<byte> Payload { get; }

			public DebugProxy(PacketExport export)
			{
				PacketExportHeader header = export.GetHeader();
				TypeIdx = header.TypeIdx;
				Imports = header.Imports;
				Payload = export.GetPayload();
			}
		}

		/// <summary>
		/// Data for this export
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public PacketExport(ReadOnlyMemory<byte> data) => Data = data;

		/// <summary>
		/// Gets the header for this export
		/// </summary>
		public PacketExportHeader GetHeader()
		{
			ReadOnlySpan<byte> span = Data.Span;
			int payloadLength = BinaryPrimitives.ReadInt32LittleEndian(span);
			return new PacketExportHeader(span.Slice(sizeof(int) + payloadLength));
		}

		/// <summary>
		/// Gets the payload for this export
		/// </summary>
		public ReadOnlyMemory<byte> GetPayload()
		{
			ReadOnlySpan<byte> span = Data.Span;
			int payloadLength = BinaryPrimitives.ReadInt32LittleEndian(span);
			return Data.Slice(sizeof(int), payloadLength);
		}
	}

	/// <summary>
	/// Data for an exported node in a bundle
	/// </summary>
	public readonly struct PacketExportHeader
	{
		/// <summary>
		/// Index of the type for this export
		/// </summary>
		public int TypeIdx { get; }

		/// <summary>
		/// Index of imports for this export
		/// </summary>
		public int[] Imports { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public PacketExportHeader(ReadOnlySpan<byte> span)
		{
			int bytesRead;

			TypeIdx = (int)VarInt.ReadUnsigned(span, out bytesRead);
			span = span[bytesRead..];

			int numImports = (int)VarInt.ReadUnsigned(span, out bytesRead);
			span = span[bytesRead..];

			Imports = new int[numImports];
			for (int idx = 0; idx < numImports; idx++)
			{
				Imports[idx] = (int)VarInt.ReadUnsigned(span, out bytesRead);
				span = span[bytesRead..];
			}
		}
	}
}
