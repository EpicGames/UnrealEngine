// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Text;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using BitFaster.Caching;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Base class for packet handles
	/// </summary>
	public abstract class PacketHandle
	{
		/// <summary>
		/// Bundle containing this packet
		/// </summary>
		public abstract BundleHandle Bundle { get; }

		/// <inheritdoc cref="IBlobRef.FlushAsync(CancellationToken)"/>
		public abstract ValueTask FlushAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Reads an export from this packet
		/// </summary>
		public abstract ValueTask<BlobData> ReadExportAsync(int exportIdx, CancellationToken cancellationToken = default);

		/// <summary>
		/// Append the identifier for this packet to the given string builder
		/// </summary>
		public abstract bool TryAppendIdentifier(Utf8StringBuilder builder);

		/// <summary>
		/// Appends the locator to the given string builder
		/// </summary>
		public bool TryAppendLocator(Utf8StringBuilder builder)
		{
			BlobLocator locator;
			if (Bundle.TryGetLocator(out locator))
			{
				builder.Append(locator.Path);
				builder.Append('#');
				return TryAppendIdentifier(builder);
			}
			return false;
		}
	}

	/// <summary>
	/// Handle to an packet within a bundle. 
	/// </summary>
	class FlushedPacketHandle : PacketHandle
	{
		readonly BundleStorageNamespace _storageNamespace;
		readonly BundleHandle _outer;
		readonly int _packetOffset;
		readonly int _packetLength;
		readonly BundleCache _cache;

		static readonly Utf8String s_fragmentPrefix = new("pkt=");

		/// <inheritdoc/>
		public override BundleHandle Bundle => _outer;

		/// <summary>
		/// Offset of the packet within the bundle
		/// </summary>
		public int PacketOffset => _packetOffset;

		/// <summary>
		/// Length of the packet within the bundle
		/// </summary>
		public int PacketLength => _packetLength;

		/// <summary>
		/// Constructor
		/// </summary>
		public FlushedPacketHandle(BundleStorageNamespace storageNamespace, BundleHandle outer, int packetOffset, int packetLength, BundleCache cache)
		{
			_storageNamespace = storageNamespace;

			_outer = outer;
			_packetOffset = packetOffset;
			_packetLength = packetLength;
			_cache = cache;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FlushedPacketHandle(BundleStorageNamespace storageNamespace, BundleHandle outer, ReadOnlySpan<byte> fragment, BundleCache cache)
		{
			_storageNamespace = storageNamespace;

			_outer = outer;
			_cache = cache;

			if (!TryParse(fragment, out _packetOffset, out _packetLength))
			{
				throw new FormatException($"Cannot parse fragment {Encoding.UTF8.GetString(fragment)} relative to {outer}");
			}
		}

		/// <summary>
		/// Parse a fragment containing an offset and length
		/// </summary>
		static bool TryParse(ReadOnlySpan<byte> fragment, out int packetOffset, out int packetLength)
		{
			if (!fragment.StartsWith(s_fragmentPrefix.Span))
			{
				packetOffset = packetLength = 0;
				return false;
			}

			fragment = fragment.Slice(s_fragmentPrefix.Length);
			if (!Utf8Parser.TryParse(fragment, out packetOffset, out int numBytesRead))
			{
				packetOffset = packetLength = 0;
				return false;
			}

			fragment = fragment[numBytesRead..];
			if (fragment.Length == 0 || fragment[0] != (byte)',')
			{
				packetOffset = packetLength = 0;
				return false;
			}

			fragment = fragment[1..];
			if (!Utf8Parser.TryParse(fragment, out packetLength, out numBytesRead) || numBytesRead != fragment.Length)
			{
				packetOffset = packetLength = 0;
				return false;
			}

			return true;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		Utf8String GetLocator()
		{
			Utf8StringBuilder builder = new();
			if (!TryAppendLocator(builder))
			{
				throw new NotImplementedException();
			}
			return builder.ToUtf8String();
		}

		/// <inheritdoc/>
		public override ValueTask FlushAsync(CancellationToken cancellationToken = default) => default;

		/// <summary>
		/// Reads an export from this packet
		/// </summary>
		/// <param name="exportIdx">Index of the export</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public override async ValueTask<BlobData> ReadExportAsync(int exportIdx, CancellationToken cancellationToken = default)
		{
			try
			{
				using Lifetime<PacketReader> packetReaderHandle = await GetPacketReaderAsync(cancellationToken);
				return packetReaderHandle.Value.ReadExport(exportIdx);
			}
			catch (OperationCanceledException)
			{
				throw;
			}
			catch (ObjectNotFoundException ex)
			{
				Utf8String locator = GetLocator();
				throw new ObjectNotFoundException(ex.Key, $"Unable to read {locator}: {ex.Message}", ex);
			}
			catch (Exception ex)
			{
				Utf8String locator = GetLocator();
				throw new StorageException($"Unable to read {locator}: {ex.Message}", ex);
			}
		}

		/// <inheritdoc/>
		public override bool TryAppendIdentifier(Utf8StringBuilder builder)
		{
			AppendIdentifier(builder, _packetOffset, _packetLength);
			return true;
		}

		/// <summary>
		/// Appends an identifier for a packet to the given buffer
		/// </summary>
		public static void AppendIdentifier(Utf8StringBuilder builder, int packetOffset, int packetLength)
		{
			builder.Append(s_fragmentPrefix);
			builder.Append(packetOffset);
			builder.Append((byte)',');
			builder.Append(packetLength);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
			=> obj is FlushedPacketHandle other && _outer.Equals(other._outer) && _packetOffset == other._packetOffset;

		/// <inheritdoc/>
		public override int GetHashCode()
			=> HashCode.Combine(_outer, _packetOffset);

		#region Packet reader access

		async ValueTask<Lifetime<PacketReader>> GetPacketReaderAsync(CancellationToken cancellationToken = default)
		{
			PacketReaderCacheKey cacheKey = new(_outer, _packetOffset);
			return await _cache.PacketReaderCache.ScopedGetOrAddAsync(cacheKey, CreatePacketReaderAsync, cancellationToken);
		}

		internal PacketReader CreatePacketReader(ReadOnlyMemory<byte> encodedData)
		{
			PacketReaderCacheKey cacheKey = new(_outer, _packetOffset);
			IRefCountedHandle<Packet> packet = Packet.Decode(encodedData, _cache.Allocator, cacheKey);
			return new PacketReader(_storageNamespace, _cache, _outer, this, packet.Target, packet);
		}

		async Task<Scoped<PacketReader>> CreatePacketReaderAsync(PacketReaderCacheKey cacheKey, CancellationToken cancellationToken)
		{
			using IReadOnlyMemoryOwner<byte> encodedData = await ReadEncodedPacketAsync(cancellationToken);
			IRefCountedHandle<Packet> packet = Packet.Decode(encodedData.Memory, _cache.Allocator, cacheKey);
			Interlocked.Add(ref _storageNamespace.PacketReaderStats._numDecodedBytesRead, packet.Target.Length);
#pragma warning disable CA2000
			PacketReader reader = new(_storageNamespace, _cache, _outer, this, packet.Target, packet);
			return new Scoped<PacketReader>(reader);
#pragma warning restore CA2000
		}

		async ValueTask<IReadOnlyMemoryOwner<byte>> ReadEncodedPacketAsync(CancellationToken cancellationToken)
		{
			Interlocked.Add(ref _storageNamespace.PacketReaderStats._numEncodedBytesRead, _packetLength);

			int minPageIdx = _packetOffset / _cache.BundlePageSize;
			int maxPageIdx = ((_packetOffset + _packetLength) + (_cache.BundlePageSize - 1)) / _cache.BundlePageSize;

			// If the whole packet is contained within on page, just return that.
			if (maxPageIdx == minPageIdx + 1)
			{
				Lifetime<IReadOnlyMemoryOwner<byte>> pageData = await ReadBundlePageAsync(minPageIdx, cancellationToken);
				return ReadOnlyMemoryOwner.Create(pageData.Value.Memory.Slice(_packetOffset - (minPageIdx * _cache.BundlePageSize), _packetLength), pageData);
			}

			// Otherwise read sections from each page that contributes to the output and copy into a shared buffer
			IMemoryOwner<byte> owner = PoolAllocator.Shared.Alloc(_packetLength, null);
			try
			{
				await Parallel.ForEachAsync(Enumerable.Range(minPageIdx, maxPageIdx - minPageIdx), cancellationToken, async (pageIdx, ctx) => await ReadBundlePageAsync(pageIdx, owner.Memory, ctx));
				return ReadOnlyMemoryOwner.Create<byte>(owner.Memory.Slice(0, _packetLength), owner);
			}
			catch
			{
				owner.Dispose();
				throw;
			}
		}

		async ValueTask<Lifetime<IReadOnlyMemoryOwner<byte>>> ReadBundlePageAsync(int pageIdx, CancellationToken cancellationToken)
		{
			BundlePageCacheKey bundlePageCacheKey = new(_outer, pageIdx);
			return await _cache.BundlePageCache.ScopedGetOrAddAsync(bundlePageCacheKey, ReadBundlePageInternalAsync, cancellationToken);
		}

		async Task ReadBundlePageAsync(int pageIdx, Memory<byte> targetMemory, CancellationToken cancellationToken)
		{
			using Lifetime<IReadOnlyMemoryOwner<byte>> pageData = await ReadBundlePageAsync(pageIdx, cancellationToken);

			int pageBase = pageIdx * _cache.BundlePageSize;

			int minOffset = Math.Max(_packetOffset, pageBase);
			int maxOffset = Math.Min(_packetOffset + _packetLength, pageBase + _cache.BundlePageSize);

			ReadOnlyMemory<byte> sourceMemory = pageData.Value.Memory.Slice(minOffset - pageBase, maxOffset - minOffset);
			sourceMemory.CopyTo(targetMemory.Slice(minOffset - _packetOffset));
		}

		async Task<Scoped<IReadOnlyMemoryOwner<byte>>> ReadBundlePageInternalAsync(BundlePageCacheKey key, CancellationToken cancellationToken)
		{
			IReadOnlyMemoryOwner<byte> owner = await _outer.ReadAsync(key.Index * _cache.BundlePageSize, _cache.BundlePageSize, cancellationToken);
			Interlocked.Add(ref _storageNamespace.PacketReaderStats._numBytesRead, owner.Memory.Length);
			return new Scoped<IReadOnlyMemoryOwner<byte>>(owner);
		}

		#endregion
	}
}
