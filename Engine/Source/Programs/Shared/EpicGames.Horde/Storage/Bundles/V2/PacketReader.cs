// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Counters used for tracking operations performed by a BundleStorageNamespace
	/// </summary>
	class PacketReaderStats
	{
		public long _numBytesRead;
		public long _numEncodedBytesRead;
		public long _numDecodedBytesRead;

		public void GetStats(StorageStats stats)
		{
			stats.Add("bundle.packet_reader.num_bytes_read", _numBytesRead);
			stats.Add("bundle.packet_reader.num_encoded_bytes_read", _numEncodedBytesRead);
			stats.Add("bundle.packet_reader.num_decoded_bytes_read", _numDecodedBytesRead);
		}
	}

	readonly record struct PacketReaderCacheKey(BundleHandle Bundle, int Offset)
	{
		public override string ToString()
			=> $"packet-reader:{Bundle}@{Offset}";
	}

	/// <summary>
	/// Utility class for constructing BlobData objects from a packet, caching any computed handles to other blobs.
	/// </summary>
	sealed class PacketReader : IDisposable
	{
		readonly BundleStorageNamespace _storageNamespace;
		readonly BundleCache _cache;
		readonly BundleHandle _bundleHandle;
		readonly FlushedPacketHandle _packetHandle;

		Packet _decodedPacket;
		object?[] _cachedImportHandles;
		IRefCountedHandle _memoryOwner;

		/// <summary>
		/// Accessor for the underlying packet data
		/// </summary>
		public Packet Packet => _decodedPacket;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="storageNamespace"></param>
		/// <param name="cache"></param>
		/// <param name="bundleHandle"></param>
		/// <param name="packetHandle"></param>
		/// <param name="decodedPacket">Data for the packet</param>
		/// <param name="memoryOwner">Owner for the packet data</param>
		public PacketReader(BundleStorageNamespace storageNamespace, BundleCache cache, BundleHandle bundleHandle, FlushedPacketHandle packetHandle, Packet decodedPacket, IRefCountedHandle memoryOwner)
		{
			_storageNamespace = storageNamespace;
			_cache = cache;
			_bundleHandle = bundleHandle;
			_packetHandle = packetHandle;
			_decodedPacket = decodedPacket;
			_memoryOwner = memoryOwner;
			_cachedImportHandles = new object?[_decodedPacket.GetImportCount()];
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_memoryOwner != null)
			{
				_memoryOwner.Dispose();
				_memoryOwner = null!;
			}

			_decodedPacket = null!;
			_cachedImportHandles = null!;
		}

		/// <summary>
		/// Reads this packet in its entirety
		/// </summary>
		public BlobData Read()
		{
			IBlobRef[] imports = new IBlobRef[_decodedPacket.GetImportCount()];
			for (int idx = 0; idx < imports.Length; idx++)
			{
				imports[idx] = GetImportedBlobHandle(idx);
			}
			return new BlobDataWithOwner(Packet.BlobType, _decodedPacket.Data, imports, _memoryOwner.AddRef());
		}

		/// <summary>
		/// Reads an export from this packet
		/// </summary>
		/// <param name="exportIdx">Index of the export</param>
		public BlobData ReadExport(int exportIdx)
		{
			PacketExport export = _decodedPacket.GetExport(exportIdx);
			PacketExportHeader exportHeader = export.GetHeader();

			BlobType type = _decodedPacket.GetType(exportHeader.TypeIdx);

			IBlobRef[] imports = new IBlobRef[exportHeader.Imports.Length];
			for (int idx = 0; idx < exportHeader.Imports.Length; idx++)
			{
				imports[idx] = GetImportedBlobHandle(exportHeader.Imports[idx]);
			}

			if (_memoryOwner == null)
			{
				return new BlobData(type, export.GetPayload(), imports);
			}
			else
			{
				return new BlobDataWithOwner(type, export.GetPayload(), imports, _memoryOwner.AddRef());
			}
		}

		/// <summary>
		/// Reads an export from this packet
		/// </summary>
		/// <param name="exportIdx">Index of the export</param>
		public IReadOnlyMemoryOwner<byte> ReadExportBody(int exportIdx)
		{
			PacketExport export = _decodedPacket.GetExport(exportIdx);
			return ReadOnlyMemoryOwner.Create(export.GetPayload(), _memoryOwner?.AddRef());
		}

		/// <summary>
		/// Gets an import handle for the packet
		/// </summary>
		IBlobRef GetImportedBlobHandle(int blobIdx)
		{
			IBlobRef? blobHandle = _cachedImportHandles[blobIdx] as IBlobRef;
			if (blobHandle is null)
			{
				PacketImport blobImportInfo = _decodedPacket.GetImport(blobIdx);
				Trace.Assert(blobImportInfo.BaseIdx != -1);

				int packetIdx = blobImportInfo.BaseIdx;
				PacketHandle packetHandle = GetImportedPacketHandle(packetIdx);

				blobHandle = new ExportHandle(packetHandle, blobImportInfo.Fragment);
				_cachedImportHandles[blobIdx] = blobHandle;
			}
			return blobHandle;
		}

		PacketHandle GetImportedPacketHandle(int packetIdx)
		{
			if (packetIdx == PacketImport.CurrentPacketBaseIdx)
			{
				return _packetHandle;
			}

			PacketHandle? packetHandle = _cachedImportHandles[packetIdx] as PacketHandle;
			if (packetHandle is null)
			{
				PacketImport packetImport = _decodedPacket.GetImport(packetIdx);
				Trace.Assert(packetImport.BaseIdx != -1);

				int bundleIdx = packetImport.BaseIdx;
				BundleHandle bundleHandle = GetImportedBundleHandle(bundleIdx);

				packetHandle = new FlushedPacketHandle(_storageNamespace, bundleHandle, packetImport.Fragment, _cache);
				_cachedImportHandles[packetIdx] = packetHandle;
			}
			return packetHandle;
		}

		BundleHandle GetImportedBundleHandle(int bundleIdx)
		{
			if (bundleIdx == PacketImport.CurrentBundleBaseIdx)
			{
				return _bundleHandle;
			}

			BundleHandle? bundleHandle = _cachedImportHandles[bundleIdx] as BundleHandle;
			if (bundleHandle is null)
			{
				PacketImport bundleImport = _decodedPacket.GetImport(bundleIdx);
				Trace.Assert(bundleImport.BaseIdx == -1);

				bundleHandle = new FlushedBundleHandle(_storageNamespace, new BlobLocator(bundleImport.Fragment.Clone()));
				_cachedImportHandles[bundleIdx] = bundleHandle;
			}
			return bundleHandle;
		}
	}
}
