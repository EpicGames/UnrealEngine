// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Bundles.V2
{
	/// <summary>
	/// Writes exports into a new bundle packet
	/// </summary>
	public sealed class PacketWriter : IDisposable
	{
		const string AllocationTag = nameof(PacketWriter);

		readonly BundleHandle _bundleHandle;
		readonly PacketHandle _packetHandle;
		readonly IMemoryAllocator<byte> _allocator;
		readonly Lock _lockObject;

		IRefCountedHandle<Memory<byte>> _bufferHandle;
		Memory<byte> _buffer;

		int _length;

		readonly List<BlobType> _types = [];
		readonly List<PacketImport> _imports = [];
		readonly List<object> _importHandles = [];
		readonly Dictionary<object, int> _importMap = [];
		readonly List<int> _exportOffsets = [];
		readonly HashSet<int> _bundleImportIndices = [];

		/// <summary>
		/// Current length of the packet
		/// </summary>
		public int Length => _length;

		/// <summary>
		/// Enumerate all imported bundle locators
		/// </summary>
		public IEnumerable<BlobLocator> BundleImports
			=> _bundleImportIndices.Select(x => GetImportLocator(x));

		/// <summary>
		/// Constructor
		/// </summary>
		public PacketWriter(BundleHandle bundleHandle, PacketHandle packetHandle, IMemoryAllocator<byte> allocator, Lock lockObject)
		{
			_bundleHandle = bundleHandle;
			_packetHandle = packetHandle;
			_allocator = allocator;
			_lockObject = lockObject;

			for (int idx = 0; idx < PacketImport.Bias; idx++)
			{
				_importHandles.Add(null!);
			}

			_importHandles[PacketImport.CurrentBundleBaseIdx + PacketImport.Bias] = _bundleHandle;
			_importHandles[PacketImport.CurrentPacketBaseIdx + PacketImport.Bias] = _packetHandle;

			_importMap[_bundleHandle] = PacketImport.CurrentBundleBaseIdx;
			_importMap[_packetHandle] = PacketImport.CurrentPacketBaseIdx;

			_bufferHandle = RefCountedHandle.Create(_allocator.Alloc(1024, AllocationTag));
			_buffer = _bufferHandle.Target;

			BundleSignature signature = new(BundleVersion.LatestV2, 0);
			signature.Write(_buffer.Span);

			_length = BundleSignature.NumBytes + (sizeof(int) * 3);
			_exportOffsets.Add(_length);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_buffer = Memory<byte>.Empty;

			if (_bufferHandle != null)
			{
				_bufferHandle.Dispose();
				_bufferHandle = null!;
			}
		}

		/// <summary>
		/// Gets the number of exports currently in this writer
		/// </summary>
		public int GetExportCount() => _exportOffsets.Count - 1;

		/// <summary>
		/// Writes a new blob to this packet
		/// </summary>
		/// <param name="size"></param>
		/// <param name="type"></param>
		/// <param name="imports"></param>
		public int CompleteExport(int size, BlobType type, IReadOnlyList<IBlobRef> imports)
		{
			int[] importIndices = new int[imports.Count];
			for (int idx = 0; idx < imports.Count; idx++)
			{
				importIndices[idx] = FindOrAddImport(imports[idx].Innermost);
			}

			int typeIdx = FindOrAddType(type);
			return CompleteExport(size, typeIdx, importIndices);
		}

		/// <summary>
		/// Complete the current export
		/// </summary>
		/// <param name="size">Size of data written to the export buffer</param>
		/// <param name="typeIdx">Index of the blob type</param>
		/// <param name="imports">Indexes of the imported blobs</param>
		internal int CompleteExport(int size, int typeIdx, int[] imports)
		{
			// Finalize the written export data, inserting the length at the start.
			BinaryPrimitives.WriteInt32LittleEndian(_buffer.Slice(_length).Span, size);
			_length += sizeof(int) + size;

			// Write the type and import list
			Span<byte> span = GetOutputBufferInternal(0, (imports.Length + 2) * 5).Span;
			int offset = 0;

			offset += VarInt.WriteUnsigned(span[offset..], typeIdx);
			offset += VarInt.WriteUnsigned(span[offset..], imports.Length);

			for (int idx = 0; idx < imports.Length; idx++)
			{
				offset += VarInt.WriteUnsigned(span[offset..], imports[idx]);
			}

			_length = Align(_length + offset, 4);

			// Write the next export offset to the buffer
			_exportOffsets.Add(_length);
			return _exportOffsets.Count - 2;
		}

		/// <summary>
		/// Reads data for a blob written to storage
		/// </summary>
		/// <param name="exportIdx"></param>
		/// <returns></returns>
		public BlobData GetExport(int exportIdx)
		{
			lock (_lockObject)
			{
				int offset = _exportOffsets[exportIdx];
				int length = _exportOffsets[exportIdx + 1] - offset;

				PacketExport export = new(_buffer.Slice(offset, length));
				PacketExportHeader header = export.GetHeader();

				IBlobRef[] imports = new IBlobRef[header.Imports.Length];
				for (int idx = 0; idx < header.Imports.Length; idx++)
				{
					int importIdx = header.Imports[idx];
					imports[idx] = (IBlobRef)_importHandles[importIdx + PacketImport.Bias];
				}

				IReadOnlyMemoryOwner<byte> body = ReadOnlyMemoryOwner.Create(export.GetPayload(), _bufferHandle.AddRef());
				return new BlobDataWithOwner(_types[header.TypeIdx], body.Memory, imports, body);
			}
		}

		/// <summary>
		/// Add a new blob type to be written
		/// </summary>
		/// <param name="type">Type to add</param>
		/// <returns>Index of the type</returns>
		public int FindOrAddType(BlobType type)
		{
			int idx = _types.IndexOf(type);
			if (idx == -1)
			{
				idx = _types.Count;
				_types.Add(type);
			}
			return idx;
		}

		int AddImport(int baseIdx, Utf8String fragment, object handle)
		{
			PacketImport import = new(baseIdx, fragment);
			int idx = _imports.Count;
			_imports.Add(import);
			_importMap.Add(handle, idx);

			Debug.Assert(idx + PacketImport.Bias == _importHandles.Count);
			_importHandles.Add(handle);

			return idx;
		}

		/// <summary>
		/// Adds a new imported blob locator
		/// </summary>
		/// <param name="handle">Handle to add</param>
		/// <returns>Index of the import</returns>
		public int FindOrAddImport(IBlobRef handle) => FindOrAddImportInternal(handle);

		int FindOrAddImportInternal(object handle)
		{
			int idx;
			if (_importMap.TryGetValue(handle, out idx))
			{
				return idx;
			}
			else if (handle is ExportHandle exportHandle)
			{
				int baseIdx = FindOrAddImportInternal(exportHandle.Packet);
				return AddImport(baseIdx, exportHandle.GetIdentifier(), handle);
			}
			else if (handle is PacketHandle packetHandle)
			{
				Utf8StringBuilder builder = new();
				if (!packetHandle.TryAppendIdentifier(builder))
				{
					throw new NotSupportedException("Referenced blob has not been flushed to storage yet");
				}

				int baseIdx = FindOrAddImportInternal(packetHandle.Bundle);
				return AddImport(baseIdx, builder.ToUtf8String(), handle);
			}
			else if (handle is BundleHandle bundleHandle)
			{
				BlobLocator locator;
				if (!bundleHandle.TryGetLocator(out locator))
				{
					throw new NotSupportedException("Referenced blob has not been flushed to storage yet");
				}

				int bundleImportIdx = AddImport(-1, locator.Path, handle);
				_bundleImportIndices.Add(bundleImportIdx);
				return bundleImportIdx;
			}
			else if (handle is IBlobRef blobHandle)
			{
				BlobLocator locator;
				if (!blobHandle.TryGetLocator(out locator))
				{
					throw new NotSupportedException("Referenced blob has not been flushed to storage yet");
				}
				return AddImport(-1, locator.Path, handle);
			}
			else
			{
				throw new NotSupportedException("Unknown blob type");
			}
		}

		/// <summary>
		/// Gets the import assigned to a particular index
		/// </summary>
		public IBlobRef GetImport(int importIdx)
			=> (_importHandles[importIdx + PacketImport.Bias] as IBlobRef) ?? throw new InvalidOperationException("Import is not a blob handle");

		/// <summary>
		/// Gets the import assigned to a particular index
		/// </summary>
		public BlobLocator GetImportLocator(int importIdx)
		{
			Utf8StringBuilder builder = new();
			BuildImportLocator(importIdx, builder);
			return new BlobLocator(builder.ToUtf8String());
		}

		void BuildImportLocator(int importIdx, Utf8StringBuilder builder)
		{
			PacketImport import = _imports[importIdx];
			if (import.BaseIdx != -1)
			{
				BuildImportLocator(import.BaseIdx, builder);
			}
			builder.Append(import.Fragment);
		}

		/// <summary>
		/// Gets data to write new export
		/// </summary>
		/// <param name="usedSize">Size of data in the current buffer that has been written to</param>
		/// <param name="desiredSize"></param>
		/// <returns></returns>
		public Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
		{
			int totalUsedSize = 0;
			if (usedSize > 0)
			{
				totalUsedSize = sizeof(int) + usedSize;
			}

			Memory<byte> memory = GetOutputBufferInternal(totalUsedSize, sizeof(int) + desiredSize);
			return memory[4..];
		}

		/// <summary>
		/// Aligns an offset to a power-of-2 boundary
		/// </summary>
		static int Align(int value, int alignment) => (value + (alignment - 1)) & ~(alignment - 1);

		/// <summary>
		/// Gets data to write new export
		/// </summary>
		Memory<byte> GetOutputBufferInternal(int usedSize, int desiredSize)
		{
			if (_length + desiredSize > _buffer.Length)
			{
				lock (_lockObject)
				{
					int newSize = (_length + desiredSize + 4096 + 16384) & ~16384;

					IRefCountedHandle<Memory<byte>> newBufferHandle = RefCountedHandle.Create(_allocator.Alloc(newSize, AllocationTag));
					_buffer.Slice(0, _length + usedSize).CopyTo(newBufferHandle.Target);
					_buffer = newBufferHandle.Target;
					_bufferHandle.Dispose();

					_bufferHandle = newBufferHandle;
				}
			}
			return _buffer.Slice(_length);
		}

		/// <summary>
		/// Gets an output span and updates the length accordingly
		/// </summary>
		Span<byte> GetOutputSpanAndAdvance(int length)
		{
			Span<byte> span = GetOutputBufferInternal(0, length).Span;
			_length += length;
			return span.Slice(0, length);
		}

		/// <summary>
		/// Mark the current packet as complete
		/// </summary>
		public Packet CompletePacket()
		{
			_length = Align(_length, 4);

			// Write the various lookup tables
			WriteTypeTable();
			WriteImportTable();
			WriteExportTable();

			// Write the final packet length
			BinaryPrimitives.WriteInt32LittleEndian(_buffer.Span.Slice(4), _length);
			return new Packet(_buffer.Slice(0, _length));
		}

		void WriteTypeTable()
		{
			// Write the type table offset in the header
			BinaryPrimitives.WriteInt32LittleEndian(_buffer.Span.Slice(8), _length);

			// Write the types to the end of the buffer
			Span<byte> span = GetOutputSpanAndAdvance(sizeof(int) + _types.Count * BlobType.NumBytes);

			BinaryPrimitives.WriteInt32LittleEndian(span, _types.Count);
			span = span[4..];

			foreach (BlobType type in _types)
			{
				type.Write(span);
				span = span[IoHash.NumBytes..];
			}
		}

		void WriteImportTable()
		{
			// Write the the import table offset in the header
			BinaryPrimitives.WriteInt32LittleEndian(_buffer.Span.Slice(12), _length);

			// Measure the length of all the imports
			int importDataLength = 0;
			foreach (PacketImport import in _imports)
			{
				importDataLength += VarInt.MeasureUnsigned(import.BaseIdx + PacketImport.Bias) + import.Fragment.Length;
			}
			importDataLength = Align(importDataLength, 4);

			// Allocate the buffers
			int importHeaderLength = (_imports.Count + 2) * sizeof(int);
			int importDataOffset = _length + importHeaderLength;
			Span<byte> span = GetOutputSpanAndAdvance(importHeaderLength + importDataLength);

			// Write all the import data
			BinaryPrimitives.WriteInt32LittleEndian(span, _imports.Count);
			span = span[4..];

			foreach (PacketImport import in _imports)
			{
				BinaryPrimitives.WriteInt32LittleEndian(span, importDataOffset);
				span = span[4..];

				Span<byte> baseSpan = _buffer.Span.Slice(importDataOffset);
				importDataOffset += VarInt.WriteUnsigned(baseSpan, import.BaseIdx + PacketImport.Bias);

				Span<byte> fragmentSpan = _buffer.Span.Slice(importDataOffset);
				import.Fragment.Span.CopyTo(fragmentSpan);
				importDataOffset += import.Fragment.Length;
			}
			BinaryPrimitives.WriteInt32LittleEndian(span, importDataOffset);
		}

		void WriteExportTable()
		{
			// Write the export table offset in the header
			BinaryPrimitives.WriteInt32LittleEndian(_buffer.Span.Slice(16), _length);

			// Write the exports
			Span<int> span = MemoryMarshal.Cast<byte, int>(GetOutputSpanAndAdvance((_exportOffsets.Count + 1) * sizeof(int)));

			span[0] = _exportOffsets.Count - 1;
			for (int idx = 0; idx < _exportOffsets.Count; idx++)
			{
				span[idx + 1] = _exportOffsets[idx];
			}
		}
	}
}
