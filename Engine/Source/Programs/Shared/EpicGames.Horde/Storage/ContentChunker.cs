// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Utilities;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Base class for implementations of a content defined chunker
	/// </summary>
	public abstract class ContentChunker
	{
		/// <summary>
		/// Source channel.
		/// </summary>
		public abstract ChannelWriter<ChunkerSource> SourceWriter { get; }

		/// <summary>
		/// Output channel. The order of items written to the source writer will be preserved in the output reader. Each 
		/// output item should be read completely (through calls to <see cref="ChunkerOutput.MoveNextAsync(CancellationToken)"/>)
		/// before reading the next.
		/// </summary>
		public abstract ChannelReader<ChunkerOutput> OutputReader { get; }
	}

	/// <summary>
	/// Base class for input to a <see cref="ContentChunker"/>. Allows the chunker to read data into a buffer as required.
	/// </summary>
	public abstract class ChunkerSource
	{
		/// <summary>
		/// Length of the input data
		/// </summary>
		public abstract long Length { get; }

		/// <summary>
		/// Optional user specified data to be propagated to the output
		/// </summary>
		public virtual object? UserData { get; } = null;

		/// <summary>
		/// Starts a task to read the next chunk of data. Note that this task may be called again before the task completes.
		/// </summary>
		/// <param name="memory">Buffer to store the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public abstract Task<Task> StartReadAsync(Memory<byte> memory, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Implementation of <see cref="ChunkerSource"/> for files on disk
	/// </summary>
	public class FileChunkerSource : ChunkerSource
	{
		readonly FileInfo _fileInfo;
		readonly object? _userData;
		long _offset;

		/// <inheritdoc/>
		public override object? UserData
			=> _userData;

		/// <summary>
		/// Constructor
		/// </summary>
		public FileChunkerSource(FileInfo fileInfo, object? userData)
		{
			_fileInfo = fileInfo;
			_userData = userData;
		}

		/// <inheritdoc/>
		public override long Length
			=> _fileInfo.Length;

		/// <inheritdoc/>
		public override Task<Task> StartReadAsync(Memory<byte> memory, CancellationToken cancellationToken = default)
		{
			long offset = _offset;
			_offset += memory.Length;
			return Task.FromResult(HandleReadAsync(offset, memory, cancellationToken));
		}

		/// <inheritdoc/>
		async Task HandleReadAsync(long offset, Memory<byte> memory, CancellationToken cancellationToken)
		{
			using PlatformFileLock? platformFileLock = await PlatformFileLock.CreateAsync(cancellationToken);

			FileStreamOptions options = new() { Mode = FileMode.Open, Access = FileAccess.Read, Options = FileOptions.Asynchronous };
			using FileStream stream = _fileInfo.Open(options);
			stream.Seek(offset, SeekOrigin.Begin);
			await stream.ReadExactlyAsync(memory, cancellationToken);
		}
	}

	/// <summary>
	/// Implementation of <see cref="ChunkerSource"/> for data in memory
	/// </summary>
	class MemoryChunkerSource : ChunkerSource
	{
		readonly ReadOnlyMemory<byte> _data;
		int _offset;

		/// <inheritdoc/>
		public override long Length => _data.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public MemoryChunkerSource(Memory<byte> data)
			=> _data = data;

		/// <inheritdoc/>
		public override Task<Task> StartReadAsync(Memory<byte> memory, CancellationToken cancellationToken = default)
		{
			_data.Slice(_offset, memory.Length).CopyTo(memory);
			_offset += memory.Length;
			return Task.FromResult(Task.CompletedTask);
		}
	}

	/// <summary>
	/// Enumerates chunks from an input file 
	/// </summary>
	public abstract class ChunkerOutput
	{
		/// <summary>
		/// Rolling hash for this chunk
		/// </summary>
		public abstract uint RollingHash { get; }

		/// <summary>
		/// Accessor for the chunk's data
		/// </summary>
		public abstract ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// User specified data from the <see cref="ChunkerSource"/>
		/// </summary>
		public abstract object? UserData { get; }

		/// <summary>
		/// Moves to the next output chunk
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if there was another output chunk</returns>
		public abstract ValueTask<bool> MoveNextAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Simple serial implementation of content chunking
	/// </summary>
	public class SerialBuzHashChunker : ContentChunker
	{
		class Output : ChunkerOutput
		{
			readonly SerialBuzHashChunker _pipeline;
			public readonly ChunkerSource Input;

			public bool _firstOutput = true;
			public uint _rollingHash;
			public long _inputOffset;
			public long _outputOffset;
			public ReadOnlyMemory<byte> _data;

			public override uint RollingHash => _rollingHash;
			public override ReadOnlyMemory<byte> Data => _data;
			public override object? UserData => Input.UserData;

			public Output(SerialBuzHashChunker pipeline, ChunkerSource input)
			{
				_pipeline = pipeline;
				Input = input;
			}

			/// <inheritdoc/>
			public override ValueTask<bool> MoveNextAsync(CancellationToken cancellationToken = default)
				=> _pipeline.MoveNextAsync(Input, this, cancellationToken);
		}

		readonly Memory<byte> _buffer;
		readonly Channel<ChunkerSource> _sourceChannel;
		readonly Channel<ChunkerOutput> _outputChannel;
		readonly LeafChunkedDataNodeOptions _options;
		readonly Queue<Output> _order = new();

		int _bufferOffset;
		int _bufferLength;
		int _chunkLength;

		/// <inheritdoc/>
		public override ChannelWriter<ChunkerSource> SourceWriter
			=> _sourceChannel.Writer;

		/// <inheritdoc/>
		public override ChannelReader<ChunkerOutput> OutputReader
			=> _outputChannel.Reader;

		/// <summary>
		/// Constructor
		/// </summary>
		public SerialBuzHashChunker(AsyncPipeline pipeline, Memory<byte> buffer, LeafChunkedDataNodeOptions options)
		{
			_buffer = buffer;
			_sourceChannel = Channel.CreateUnbounded<ChunkerSource>();
			_outputChannel = Channel.CreateUnbounded<ChunkerOutput>();
			_options = options;

			_ = pipeline.AddTask(ProcessInputAsync);
		}

		async Task ProcessInputAsync(CancellationToken cancellationToken)
		{
			ChannelReader<ChunkerSource> inputReader = _sourceChannel.Reader;
			ChannelWriter<ChunkerOutput> outputWriter = _outputChannel.Writer;

			while (await inputReader.WaitToReadAsync(cancellationToken))
			{
				while (inputReader.TryRead(out ChunkerSource? input))
				{
					Output output = new(this, input);
					if (input.Length > 0)
					{
						lock (_order)
						{
							_order.Enqueue(output);
						}
					}
					await outputWriter.WriteAsync(output, cancellationToken);
				}
			}

			outputWriter.Complete();
		}

		async ValueTask<bool> MoveNextAsync(ChunkerSource input, Output output, CancellationToken cancellationToken)
		{
			bool firstOutput = output._firstOutput;
			output._firstOutput = false;

			if (output._outputOffset == output.Input.Length)
			{
				output._data = ReadOnlyMemory<byte>.Empty;
				return firstOutput;
			}
			else
			{
				lock (_order)
				{
					while (_order.TryPeek(out Output? current) && current._outputOffset == current.Input.Length)
					{
						_order.Dequeue();
					}
					if (output != _order.Peek())
					{
						throw new InvalidOperationException();
					}
				}

				// Discard the previous chunk
				_bufferOffset += _chunkLength;
				_bufferLength -= _chunkLength;

				// Shrink the buffer once we don't have enough space for a full chunk
				if (_bufferLength < _options.MaxSize)
				{
					// Move the remaining buffer to the start
					if (_bufferOffset > 0)
					{
						Memory<byte> source = _buffer.Slice(_bufferOffset, _bufferLength);
						source.CopyTo(_buffer);
						_bufferOffset = 0;
					}

					// Top up the input buffer
					int readLength = (int)Math.Min(input.Length - output._inputOffset, _buffer.Length - _bufferLength);
					if (readLength > 0)
					{
						Task readTask = await input.StartReadAsync(_buffer.Slice(_bufferLength, readLength), cancellationToken);
						await readTask;

						output._inputOffset += readLength;
						_bufferLength += readLength;
					}
				}

				// Find the next chunk
				_chunkLength = BuzHash.FindChunkLength(_buffer.Span.Slice(_bufferOffset, _bufferLength), _options.MinSize, _options.MaxSize, _options.TargetSize, out output._rollingHash);
				output._data = _buffer.Slice(_bufferOffset, _chunkLength);
				output._outputOffset += _chunkLength;
				return true;
			}
		}
	}

	/// <summary>
	/// Parallel implementation of <see cref="SerialBuzHashChunker"/>
	/// </summary>
	public class ParallelBuzHashChunker : ContentChunker
	{
		const int ReadBlockSize = 256 * 1024;

		record class InputRequest(ChunkerSource Input, TaskCompletionSource<Block> FirstBlockTcs);
		record class BlockRequest(int BufferOffset, int Length, Task ReadTask, TaskCompletionSource<Block> BlockTcs, Task<Block>? NextBlock);
		record class Block(int BufferOffset, int Length, List<(int, uint)> Boundaries, Task<Block>? Next);

		class RingBuffer
		{
			readonly Lock _lockObject = new();

			int _head;
			int _size;

			readonly Memory<byte> _data;
			readonly int _wrapSize;
			readonly AsyncEvent _freeEvent = new();

			public RingBuffer(Memory<byte> data, int wrapSize)
			{
				_data = data;
				_wrapSize = wrapSize;
			}

			public Memory<byte> GetData(int offset, int length)
				=> _data.Slice(offset, length);

			public async Task<int> AllocAsync(int blockSize, CancellationToken cancellationToken)
			{
				if (blockSize > _wrapSize)
				{
					throw new ArgumentException($"Requested block size is too large ({blockSize} > {_wrapSize})", nameof(blockSize));
				}

				for (; ; )
				{
					Task freeTask = _freeEvent.Task;

					lock (_lockObject)
					{
						// Figure out how much of the buffer we need to allocate, wrapping round to the start if necessary
						int allocResult = _head;
						int allocLength = blockSize;
						if (_head + blockSize > _wrapSize)
						{
							allocResult = 0;
							allocLength += (_wrapSize - _head);
						}

						// Check if we have the space available to allocate
						if (_size + allocLength <= _wrapSize)
						{
							_head = allocResult + blockSize;
							_size += allocLength;
							return allocResult;
						}
					}

					// Wait for more space to become available
					await freeTask.WaitAsync(cancellationToken);
				}
			}

			public void Free(int offset, int length)
			{
				if (offset < 0)
				{
					throw new ArgumentException("Offset is out of range", nameof(offset));
				}
				if (length < 0 || offset + length > _wrapSize)
				{
					throw new ArgumentException("Length is out of range", nameof(length));
				}

				if (length > 0)
				{
					lock (_lockObject)
					{
						int end = offset + length;
						if (offset < _head)
						{
							_size = _head - end;
						}
						else
						{
							_size = (_wrapSize - end) + _head;
						}
					}
					_freeEvent.Set();
				}
			}
		}

		class LatestCdcOutput : ChunkerOutput
		{
			readonly ParallelBuzHashChunker _pipeline;
			readonly object? _userData;

			Task<Block> _blockTask;
			int _blockOffset;
			int _boundaryIdx;
			uint _rollingHash;
			ReadOnlyMemory<byte> _data;
			bool _firstChunk = true;

			public LatestCdcOutput(ParallelBuzHashChunker pipeline, object? userData, Task<Block> blockTask)
			{
				_pipeline = pipeline;
				_userData = userData;
				_blockTask = blockTask;
			}

			public override uint RollingHash
				=> _rollingHash;

			public override ReadOnlyMemory<byte> Data
				=> _data;

			public override object? UserData
				=> _userData;

			public override async ValueTask<bool> MoveNextAsync(CancellationToken cancellationToken = default)
			{
				RingBuffer ringBuffer = _pipeline._ringBuffer;
				LeafChunkedDataNodeOptions options = _pipeline._options;

				// Always need to return at least one chunk, even if the source data is empty. Use this flag to track whether
				// we've returned something.
				bool firstChunk = _firstChunk;
				_firstChunk = false;

				for (; ; )
				{
					Block block = await _blockTask.WaitAsync(cancellationToken);
					ReadOnlyMemory<byte> blockData = ringBuffer.GetData(block.BufferOffset, block.Length);

					// Release data from the current chunk
					_blockOffset += _data.Length;
					_data = ReadOnlyMemory<byte>.Empty;

					// Enumerate all the chunks from the cached boundaries
					for (; _boundaryIdx < block.Boundaries.Count; _boundaryIdx++)
					{
						(int boundaryOffset, _rollingHash) = block.Boundaries[_boundaryIdx];
						if (boundaryOffset >= _blockOffset + options.MaxSize)
						{
							_data = blockData.Slice(_blockOffset, options.MaxSize);
							return true;
						}
						if (boundaryOffset >= _blockOffset + options.MinSize)
						{
							_data = blockData.Slice(_blockOffset, boundaryOffset - _blockOffset);
							return true;
						}
					}

					// If this is the last block, return all the remaining data
					if (block.Next == null)
					{
						_data = blockData.Slice(_blockOffset);
						_ = BuzHash.FindChunkLength(_data.Span, options.MinSize, options.MaxSize, options.TargetSize, out _rollingHash);

						if (_data.Length == 0 && !firstChunk)
						{
							ringBuffer.Free(block.BufferOffset, block.Length);
							_blockTask = null!;
							return false;
						}

						return true;
					}

					// If we still have remaining data in the current block, find a chunk that spans this and the next one
					if (_blockOffset < block.Length)
					{
						Block nextBlock = await block.Next.WaitAsync(cancellationToken);

						// If the two blocks aren't contiguous, append data from the next block to make the max chunk size.
						// We allocate slack space in the buffer to ensure that we have enough space to support this.
						int appendLength = Math.Min((_blockOffset + options.MaxSize) - block.Length, nextBlock.Length);
						if (appendLength > 0)
						{
							int blockEnd = block.BufferOffset + block.Length;
							if (nextBlock.BufferOffset != blockEnd)
							{
								ReadOnlyMemory<byte> copyData = ringBuffer.GetData(nextBlock.BufferOffset, appendLength);
								copyData.CopyTo(ringBuffer.GetData(blockEnd, appendLength));
							}
							blockData = ringBuffer.GetData(block.BufferOffset, block.Length + appendLength);
						}

						int chunkLength = BuzHash.FindChunkLength(blockData.Slice(_blockOffset).Span, options.MinSize, options.MaxSize, options.TargetSize, out _rollingHash);
						_data = blockData.Slice(_blockOffset, chunkLength);

						return true;
					}

					// Release data from the current block
					ringBuffer.Free(block.BufferOffset, block.Length);

					// Move to the next block
					_boundaryIdx = 0;
					_blockOffset -= block.Length;
					_blockTask = block.Next;
				}
			}
		}

		readonly RingBuffer _ringBuffer;
		readonly LeafChunkedDataNodeOptions _options;

		readonly Channel<ChunkerSource> _inputChannel;
		readonly Channel<ChunkerOutput> _outputChannel;

		readonly Channel<InputRequest> _inputRequestChannel;
		readonly Channel<BlockRequest> _blockRequestChannel;

		/// <inheritdoc/>
		public override ChannelWriter<ChunkerSource> SourceWriter
			=> _inputChannel.Writer;

		/// <inheritdoc/>
		public override ChannelReader<ChunkerOutput> OutputReader
			=> _outputChannel.Reader;

		/// <summary>
		/// Constructor
		/// </summary>
		public ParallelBuzHashChunker(AsyncPipeline pipeline, LeafChunkedDataNodeOptions options)
			: this(pipeline, new byte[2 * 1024 * 1024], options)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public ParallelBuzHashChunker(AsyncPipeline pipeline, Memory<byte> buffer, LeafChunkedDataNodeOptions options)
		{
			if (buffer.Length < options.MaxSize * 3)
			{
				throw new ArgumentException($"Buffer must be large enough to contain three blocks of maximum size", nameof(buffer));
			}
			if (options.MaxSize > ReadBlockSize)
			{
				throw new ArgumentException($"Max chunk size must be less than {ReadBlockSize} bytes", nameof(options));
			}

			_ringBuffer = new RingBuffer(buffer, buffer.Length - options.MaxSize);
			_options = options;

			_inputChannel = Channel.CreateUnbounded<ChunkerSource>();
			_outputChannel = Channel.CreateUnbounded<ChunkerOutput>();

			_inputRequestChannel = Channel.CreateUnbounded<InputRequest>();//.CreateBounded<InputRequest>(new BoundedChannelOptions(20));
			_blockRequestChannel = Channel.CreateUnbounded<BlockRequest>();

			_ = pipeline.AddTask(ProcessInputAsync);
			_ = pipeline.AddTask(ProcessInputRequestsAsync);
			_ = pipeline.AddTasks(32, ProcessBlockRequestsAsync);
		}

		async Task ProcessInputAsync(CancellationToken cancellationToken)
		{
			ChannelReader<ChunkerSource> inputReader = _inputChannel.Reader;
			ChannelWriter<ChunkerOutput> outputWriter = _outputChannel.Writer;

			while (await inputReader.WaitToReadAsync(cancellationToken))
			{
				ChunkerSource? input;
				while (inputReader.TryRead(out input))
				{
					ChunkerOutput output = await AddAsync(input, cancellationToken);
					await outputWriter.WriteAsync(output, cancellationToken);
				}
			}

			_inputRequestChannel.Writer.Complete();
			outputWriter.Complete();
		}

		async Task<ChunkerOutput> AddAsync(ChunkerSource input, CancellationToken cancellationToken = default)
		{
			InputRequest inputRequest = new(input, new TaskCompletionSource<Block>(TaskCreationOptions.RunContinuationsAsynchronously));
			await _inputRequestChannel.Writer.WriteAsync(inputRequest, cancellationToken);
			return new LatestCdcOutput(this, input.UserData, inputRequest.FirstBlockTcs.Task);
		}

		async Task ProcessInputRequestsAsync(CancellationToken cancellationToken)
		{
			ChannelReader<InputRequest> reader = _inputRequestChannel.Reader;
			ChannelWriter<BlockRequest> writer = _blockRequestChannel.Writer;
			while (await reader.WaitToReadAsync(cancellationToken))
			{
				while (reader.TryRead(out InputRequest? inputRequest))
				{
					ChunkerSource input = inputRequest.Input;

					TaskCompletionSource<Block>? blockTcs = inputRequest.FirstBlockTcs;
					for (long sourceOffset = 0; blockTcs != null;)
					{
						// Read a block of data
						int readLength = (int)Math.Min(input.Length - sourceOffset, ReadBlockSize);

						int bufferOffset = await _ringBuffer.AllocAsync(readLength, cancellationToken);
						Memory<byte> data = _ringBuffer.GetData(bufferOffset, readLength);
						Task readTask = await input.StartReadAsync(data, cancellationToken);

						sourceOffset += readLength;

						// Post the request to handle the result
						TaskCompletionSource<Block>? nextBlockTcs;
						if (sourceOffset >= input.Length)
						{
							nextBlockTcs = null;
						}
						else
						{
							nextBlockTcs = new TaskCompletionSource<Block>(TaskCreationOptions.RunContinuationsAsynchronously);
						}

						BlockRequest blockRequest = new(bufferOffset, readLength, readTask, blockTcs, nextBlockTcs?.Task);
						await writer.WriteAsync(blockRequest, cancellationToken);

						blockTcs = nextBlockTcs;
					}
				}
			}
			writer.Complete();
		}

		async Task ProcessBlockRequestsAsync(CancellationToken cancellationToken)
		{
			ChannelReader<BlockRequest> reader = _blockRequestChannel.Reader;
			while (await reader.WaitToReadAsync(cancellationToken))
			{
				while (reader.TryRead(out BlockRequest? blockRequest))
				{
					// Wait until the data has been read
					await blockRequest.ReadTask;

					// Find the split points
					Memory<byte> data = _ringBuffer.GetData(blockRequest.BufferOffset, blockRequest.Length);
					List<(int, uint)> boundaries = BuzHash.FindCandidateSplitPoints(data.Span, _options.WindowSize, _options.Threshold);

					// Create the block
					Block block = new(blockRequest.BufferOffset, blockRequest.Length, boundaries, blockRequest.NextBlock);
					blockRequest.BlockTcs.SetResult(block);
				}
			}
		}
	}
}
