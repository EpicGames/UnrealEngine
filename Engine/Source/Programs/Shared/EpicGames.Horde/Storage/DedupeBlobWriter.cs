// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Nodes;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Index of known nodes that can be used for deduplication.
	/// </summary>
	public sealed class DedupeBlobWriter : BlobWriter
	{
		record BlobKey(IoHash Hash, BlobType Type);

		class DedupeCache
		{
			readonly int _maxKeys;
			readonly Queue<BlobKey> _blobKeys = new();
			readonly Dictionary<BlobKey, IHashedBlobRef> _blobKeyToHandle = [];

			public DedupeCache(int maxKeys)
			{
				_maxKeys = maxKeys;
				_blobKeys = new Queue<BlobKey>(maxKeys);
				_blobKeyToHandle = new Dictionary<BlobKey, IHashedBlobRef>(maxKeys);
			}

			internal void Add(BlobKey key, IHashedBlobRef handle)
			{
				BlobKey? prevKey;
				if (_blobKeys.Count == _maxKeys && _blobKeys.TryDequeue(out prevKey))
				{
					_blobKeyToHandle.Remove(prevKey);
				}
				_blobKeyToHandle.TryAdd(key, handle);
			}

			internal bool TryGetValue(BlobKey key, [NotNullWhen(true)] out IHashedBlobRef? handle) => _blobKeyToHandle.TryGetValue(key, out handle);
		}

		class WrappedHandle : IHashedBlobRef
		{
			public object _lockObject = new();
			public IHashedBlobRef? _inner;

			/// <inheritdoc/>
			public IBlobRef Innermost
				=> _inner!.Innermost;

			/// <inheritdoc/>
			public IoHash Hash
				=> _inner!.Hash;

			/// <inheritdoc/>
			public bool TryGetLocator(out BlobLocator locator)
				=> _inner!.TryGetLocator(out locator);

			/// <inheritdoc/>
			public ValueTask FlushAsync(CancellationToken cancellationToken)
				=> _inner!.FlushAsync(cancellationToken);

			/// <inheritdoc/>
			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
				=> _inner!.ReadBlobDataAsync(cancellationToken);

			/// <inheritdoc/>
			public override bool Equals(object? obj) => _inner is not null && obj is WrappedHandle other && _inner == other._inner;

			/// <inheritdoc/>
			public override int GetHashCode() => HashCode.Combine((_inner is null) ? 0 : _inner.GetHashCode(), 1);
		}

		/// <summary>
		/// Default value for maximum number of keys
		/// </summary>
		public const int DefaultMaxKeys = 512 * 1024;

		readonly BlobWriter _inner;
		readonly DedupeCache _cache;

		int _numHits;
		long _totalHitsSize;
		int _numMisses;
		long _totalMissesSize;
		int _numCacheAdds;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner"></param>
		/// <param name="maxKeys"></param>
		public DedupeBlobWriter(IBlobWriter inner, int maxKeys = DefaultMaxKeys)
			: base(inner.Options)
		{
			_inner = (BlobWriter)inner;
			_cache = new DedupeCache(maxKeys);
		}

		private DedupeBlobWriter(IBlobWriter inner, DedupeCache cache)
			: base(inner.Options)
		{
			_inner = (BlobWriter)inner;
			_cache = cache;
		}

		/// <inheritdoc/>
		public override ValueTask DisposeAsync() => _inner.DisposeAsync();

		/// <inheritdoc/>
		public override Task FlushAsync(CancellationToken cancellationToken = default) => _inner.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public override IBlobWriter Fork() => new DedupeBlobWriter(_inner.Fork(), _cache);

		/// <inheritdoc/>
		public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize) => _inner.GetOutputBuffer(usedSize, desiredSize);

		/// <summary>
		/// Add a blob to the cache
		/// </summary>
		/// <param name="type">Type of the blob</param>
		/// <param name="blobRef">Reference to the blob data</param>
		public void AddToCache(BlobType type, IHashedBlobRef blobRef)
		{
			_cache.Add(new BlobKey(blobRef.Hash, type), blobRef);
			Interlocked.Increment(ref _numCacheAdds);
		}

		/// <inheritdoc/>
		public override async ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = _inner.GetOutputBuffer(size, size).Slice(0, size);
			IoHash hash = IoHash.Compute(data.Span);
			BlobKey key = new(hash, type);

			WrappedHandle? wrappedHandle;
			lock (_cache)
			{
				IHashedBlobRef? handle;
				if (_cache.TryGetValue(key, out handle))
				{
					_numHits++;
					_totalHitsSize += size;
					return handle;
				}
				else
				{
					_numMisses++;
					_totalMissesSize += size;

					wrappedHandle = new WrappedHandle();
					_cache.Add(key, wrappedHandle);
				}
			}

			wrappedHandle._inner = await _inner.WriteBlobAsync(type, size, imports.ConvertAll(x => x.Innermost), aliases, cancellationToken);
			return wrappedHandle;
		}

		/// <summary>
		/// Gets stats for the copy operation
		/// </summary>
		public StorageStats GetStats()
		{
			StorageStats stats = new();
			stats.Add("Cache hits", _numHits);
			stats.Add("Cache hits size", _totalHitsSize);
			stats.Add("Cache misses", _numMisses);
			stats.Add("Cache misses size", _totalMissesSize);
			stats.Add("Cache adds", _numCacheAdds);
			return stats;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="IBlobWriter"/>
	/// </summary>
	public static class DedupeBlobWriterExtensions
	{
		/// <summary>
		/// Wraps a <see cref="IBlobWriter"/> with a <see cref="DedupeBlobWriter"/>
		/// </summary>
		public static DedupeBlobWriter WithDedupe(this IBlobWriter writer, int maxKeys = DedupeBlobWriter.DefaultMaxKeys) => new(writer, maxKeys);

		/// <summary>
		/// Creates a dedupe writer
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="maxKeys">Maximum number of keys to include in the cache</param>
		public static DedupeBlobWriter CreateDedupeBlobWriter(this IStorageNamespace store, int maxKeys = DedupeBlobWriter.DefaultMaxKeys)
		{
			IBlobWriter writer = store.CreateBlobWriter();
			return new DedupeBlobWriter(writer, maxKeys);
		}

		/// <summary>
		/// Creates a writer using a refname as a base path
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="refName">Ref name to use as a base path</param>
		/// <param name="maxKeys">Maximum number of keys to include in the cache</param>
		public static DedupeBlobWriter CreateDedupeBlobWriter(this IStorageNamespace store, RefName refName, int maxKeys = DedupeBlobWriter.DefaultMaxKeys)
		{
			IBlobWriter writer = store.CreateBlobWriter(refName.ToString());
			return new DedupeBlobWriter(writer, maxKeys);
		}

		/// <summary>
		/// Adds a directory tree to the cache
		/// </summary>
		/// <param name="dedupeWriter">Dedupe writer to operate on</param>
		/// <param name="directoryNodeRef">Reference to the directory to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task AddToCacheAsync(this DedupeBlobWriter dedupeWriter, IBlobRef<DirectoryNode> directoryNodeRef, CancellationToken cancellationToken = default)
		{
			DirectoryNode directoryNode;
			using (BlobData blobData = await directoryNodeRef.ReadBlobDataAsync(cancellationToken))
			{
				IHashedBlobRef hashedBlobRef = HashedBlobRef.Create(IoHash.Compute(blobData.Data.Span), directoryNodeRef);
				dedupeWriter.AddToCache(blobData.Type, hashedBlobRef);
				directoryNode = BlobSerializer.Deserialize<DirectoryNode>(blobData);
			}

			foreach (DirectoryEntry directoryEntry in directoryNode.Directories)
			{
				await AddToCacheAsync(dedupeWriter, directoryEntry.Handle, cancellationToken);
			}
			foreach (FileEntry fileEntry in directoryNode.Files)
			{
				await AddToCacheAsync(dedupeWriter, fileEntry.Target, cancellationToken);
			}
		}

		/// <summary>
		/// Adds a chunked data stream to the cache
		/// </summary>
		/// <param name="dedupeWriter">Dedupe writer to operate on</param>
		/// <param name="dataNodeRef">Reference to a data node in the stream</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task AddToCacheAsync(this DedupeBlobWriter dedupeWriter, ChunkedDataNodeRef dataNodeRef, CancellationToken cancellationToken = default)
		{
			if (dataNodeRef.Type == ChunkedDataNodeType.Leaf)
			{
				BlobType blobType = LeafChunkedDataNodeConverter.BlobType;
				dedupeWriter.AddToCache(blobType, dataNodeRef);
			}
			else if (dataNodeRef.Type == ChunkedDataNodeType.Interior)
			{
				using BlobData blobData = await dataNodeRef.ReadBlobDataAsync(cancellationToken);
				dedupeWriter.AddToCache(blobData.Type, dataNodeRef);

				InteriorChunkedDataNode interiorNode = BlobSerializer.Deserialize<InteriorChunkedDataNode>(blobData);
				foreach (ChunkedDataNodeRef childNodeRef in interiorNode.Children)
				{
					await AddToCacheAsync(dedupeWriter, childNodeRef, cancellationToken);
				}
			}
		}
	}
}
