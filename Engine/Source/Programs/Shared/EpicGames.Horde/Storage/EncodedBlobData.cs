// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Stores a <see cref="BlobData"/> object in a fixed memory buffer for transport over the wire.
	/// </summary>
	public class EncodedBlobData
	{
		enum Version
		{
			Initial,
		}

		static readonly Version s_currentVersion = Enum.GetValues(typeof(Version)).Cast<Version>().Max();

		readonly struct LocatorCollection : IReadOnlyList<BlobLocator>
		{
			public readonly JaggedReadOnlyMemoryArray Array;

			public LocatorCollection(JaggedReadOnlyMemoryArray array) => Array = array;
			public BlobLocator this[int index] => new(new Utf8String(Array[index]));
			public int Count => Array.Count;
			public IEnumerator<BlobLocator> GetEnumerator() => Array.Select(x => new BlobLocator(new Utf8String(x))).GetEnumerator();

			IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();
		}

		readonly LocatorCollection _imports;

		/// <summary>
		/// Type of the blob
		/// </summary>
		public BlobType Type { get; }

		/// <summary>
		/// The blob data payload
		/// </summary>
		public ReadOnlyMemory<byte> Payload { get; }

		/// <summary>
		/// References to other blobs
		/// </summary>
		public IReadOnlyList<BlobLocator> Imports => _imports;

		/// <summary>
		/// Constructor
		/// </summary>
		public EncodedBlobData(ReadOnlyMemory<byte> memory)
		{
			ReadOnlySpan<byte> span = memory.Span;

			if (span[0] != 'U' || span[1] != 'B' || span[2] != 'D')
			{
				throw new InvalidDataException("Invalid signature data at start of blob");
			}

			Version version = (Version)span[3];
			int offset = 4;

			if (version == Version.Initial)
			{
				Type = BlobType.Read(span.Slice(offset));
				offset += BlobType.NumBytes;

				_imports = new LocatorCollection(new JaggedReadOnlyMemoryArray(memory.Slice(offset)));
				offset += _imports.Array.Data.Length;

				Payload = memory.Slice(offset);
			}
			else
			{
				throw new InvalidDataException($"Unsupported blob data version ({(int)version}");
			}
		}

		/// <summary>
		/// Encode a blob data object to a byte aray
		/// </summary>
		public static byte[] Create(BlobData blobData)
		{
			BlobLocator[] locators = new BlobLocator[blobData.Imports.Count];
			for (int idx = 0; idx < blobData.Imports.Count; idx++)
			{
				locators[idx] = blobData.Imports[idx].GetLocator();
			}

			return Create(blobData.Type, locators, blobData.Data);
		}

		/// <summary>
		/// Encode a blob data object to a byte aray
		/// </summary>
		public static byte[] Create(BlobType type, BlobLocator[] locators, ReadOnlyMemory<byte> data)
		{
			int headerLength = GetHeaderLength(locators);

			byte[] output = new byte[headerLength + data.Length];
			WriteHeader(type, locators, output);
			data.CopyTo(output.AsMemory(headerLength));

			return output;
		}

		/// <summary>
		/// Gets the length of an encoded blob data object
		/// </summary>
		public static int GetHeaderLength(BlobLocator[] locators)
		{
			int length = sizeof(int) + BlobType.NumBytes + (sizeof(int) * (locators.Length + 1));
			foreach (BlobLocator locator in locators)
			{
				length += locator.Path.Length;
			}
			return length;
		}

		/// <summary>
		/// Encodes blob data into an existing buffer
		/// </summary>
		public static void WriteHeader(BlobType type, BlobLocator[] locators, Span<byte> output)
		{
			output[0] = (byte)'U';
			output[1] = (byte)'B';
			output[2] = (byte)'D';
			output[3] = (byte)s_currentVersion;
			output = output.Slice(4);

			type.Write(output);
			output = output.Slice(BlobType.NumBytes);

			// Write all the offsets
			int offset = sizeof(int) * (locators.Length + 1);
			foreach (BlobLocator locator in locators)
			{
				BinaryPrimitives.WriteInt32LittleEndian(output, offset);
				output = output.Slice(sizeof(int));
				offset += locator.Path.Length;
			}

			// Sentinel offset
			BinaryPrimitives.WriteInt32LittleEndian(output, offset);
			output = output.Slice(sizeof(int));

			// Write all the locator data
			foreach (BlobLocator locator in locators)
			{
				locator.Path.Span.CopyTo(output);
				output = output.Slice(locator.Path.Length);
			}
		}
	}
}
