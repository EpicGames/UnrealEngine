// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using Dapper;
using EpicGames.Core;
using Microsoft.Data.Sqlite;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Database of file metadata, storing timestamps for trees of files and hashes for ranges within them. Implemented as a SQLite database.
	/// </summary>
	public sealed class FileMetadataDb : IDisposable
	{
		/// <summary>
		/// Id value for the root directory
		/// </summary>
		public const int RootDirectoryId = 0;

		class IoHashTypeMapper : SqlMapper.TypeHandler<IoHash>
		{
			public override IoHash Parse(object value)
			{
				return IoHash.Parse((string)value);
			}

			public override void SetValue(IDbDataParameter parameter, IoHash value)
			{
				parameter.DbType = DbType.AnsiStringFixedLength;
				parameter.Value = value.ToString();
			}
		}

		static readonly IoHashTypeMapper s_hashTypeMapper = new();

		readonly SqliteConnection _connection;

		FileMetadataDb(SqliteConnection connection)
		{
			_connection = connection;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_connection.Dispose();
		}

		/// <summary>
		/// Creates a new in-memory database for file metadata
		/// </summary>
		public static async Task<FileMetadataDb> CreateInMemoryAsync(CancellationToken cancellationToken = default)
		{
			return await CreateAsync("Data Source=:memory:", cancellationToken);
		}

		/// <summary>
		/// Creates a new database for metadata backed by a file on disk
		/// </summary>
		public static async Task<FileMetadataDb> CreateFromFileAsync(FileReference file, CancellationToken cancellationToken = default)
		{
			return await CreateAsync($"Data Source={file}", cancellationToken);
		}

		static async Task<FileMetadataDb> CreateAsync(string connectionString, CancellationToken cancellationToken)
		{
			SqlMapper.AddTypeHandler<IoHash>(s_hashTypeMapper);

			SqliteConnection? connection = new(connectionString);
			try
			{
				await connection.OpenAsync(cancellationToken);

				int version = await connection.ExecuteScalarAsync<int>("PRAGMA user_version;");
				if (version == 0)
				{
					// Configure the directories table
					await connection.ExecuteAsync(
						"CREATE TABLE Directories (" +
							" id INTEGER NOT NULL PRIMARY KEY AUTOINCREMENT," +
							" parentDirectoryId INTEGER," +
							" name TEXT NOT NULL," +
							" FOREIGN KEY (parentDirectoryId) REFERENCES Directories(id));", cancellationToken);

					await connection.ExecuteAsync(
						"CREATE INDEX DirectoryParents ON Directories(parentDirectoryId);", cancellationToken);

					await connection.ExecuteAsync(
						"INSERT INTO Directories VALUES (0, NULL, \"\");", cancellationToken);

					// Configure the files table
					await connection.ExecuteAsync(
						"CREATE TABLE Files (" +
							" id INTEGER NOT NULL PRIMARY KEY AUTOINCREMENT," +
							" directoryId INTEGER NOT NULL," +
							" name TEXT NOT NULL," +
							" time INTEGER NOT NULL," +
							" length INTEGER NOT NULL," +
							" FOREIGN KEY(directoryId) REFERENCES Directories(id));", cancellationToken);

					await connection.ExecuteAsync(
						"CREATE INDEX FileParents ON Files(directoryId);", cancellationToken);

					// Configure the chunks table
					await connection.ExecuteAsync(
						"CREATE TABLE Chunks (" +
							" id INTEGER NOT NULL PRIMARY KEY AUTOINCREMENT," +
							" fileId INTEGER NOT NULL," +
							" offset INTEGER NOT NULL," +
							" length INTEGER NOT NULL," +
							" hash VARCHAR NOT NULL," +
							" FOREIGN KEY(fileId) REFERENCES Files(id));", cancellationToken);

					await connection.ExecuteAsync(
						"CREATE INDEX ChunkParents ON Chunks(fileId);", cancellationToken);

					await connection.ExecuteAsync(
						"CREATE INDEX ChunkHashes ON Chunks(hash);", cancellationToken);

					// Set the DB version number to 1
					await connection.ExecuteAsync(
						"PRAGMA user_version = 1;", cancellationToken);

					version = 1;
				}
				return new FileMetadataDb(connection);
			}
			catch
			{
				await connection.DisposeAsync();
				throw;
			}
		}

		#region Files

		/// <summary>
		/// Adds a single file to the database
		/// </summary>
		public async Task<int> AddFileAsync(int directoryId, string name, DateTime time, long length)
		{
			FileRow row = new(directoryId, name, time, length);
			return await _connection.ExecuteScalarAsync<int>("INSERT INTO Files (DirectoryId, Name, Time, Length) VALUES (@DirectoryId, @Name, @Time, @Length); SELECT last_insert_rowid();", row);
		}

		/// <summary>
		/// Adds a set of files to the database
		/// </summary>
		public async Task AddFilesAsync(IEnumerable<FileRow> files)
		{
			await _connection.ExecuteAsync("INSERT INTO Files (DirectoryId, Name, Time, Length) VALUES (@DirectoryId, @Name, @Time, @Length);", files);
		}

		/// <summary>
		/// Finds all the files in a particular directory
		/// </summary>
		public Task<FileRow> GetFileAsync(int fileId)
		{
			return _connection.QuerySingleAsync<FileRow>("SELECT * FROM Files WHERE Id = @FileId", new { FileId = fileId });
		}

		/// <summary>
		/// Finds all the files in a particular directory
		/// </summary>
		public Task<IEnumerable<FileRow>> FindFilesInDirectoryAsync(int directoryId)
		{
			return _connection.QueryAsync<FileRow>("SELECT * FROM Files WHERE DirectoryId = @DirectoryId", new { DirectoryId = directoryId });
		}

		/// <summary>
		/// Finds all the files in a particular directory
		/// </summary>
		public Task<IEnumerable<FileRow>> FindFilesInDirectoriesAsync(IEnumerable<int> directoryIds)
		{
			return _connection.QueryAsync<FileRow>("SELECT * FROM Files WHERE DirectoryId IN (@DirectoryIds)", new { DirectoryIds = directoryIds });
		}

		/// <summary>
		/// Gets the full name of a file
		/// </summary>
		public async Task<string> GetFilePathAsync(int id)
		{
			StringBuilder builder = new();
			await GetFilePathAsync(id, builder);
			return builder.ToString();
		}

		/// <summary>
		/// Gets the full name of a directory
		/// </summary>
		public async Task GetFilePathAsync(int id, StringBuilder builder)
		{
			FileRow file = await GetFileAsync(id);
			await GetDirectoryPathAsync(file.DirectoryId, builder);
			builder.Append(file.Name);
		}

		/// <summary>
		/// Removes a file and all its chunk metadata
		/// </summary>
		public async Task RemoveFileAsync(int fileId)
		{
			await _connection.ExecuteAsync("DELETE FROM Chunks WHERE FileId = @FileId; DELETE FROM Files WHERE FileId = @FileId;", fileId);
		}

		#endregion

		#region Chunks

		/// <summary>
		/// Adds a record for a new file chunk
		/// </summary>
		public async Task<int> AddChunkAsync(int fileId, long offset, long length, IoHash hash)
		{
			ChunkRow row = new(fileId, offset, length, hash);
			return await _connection.ExecuteScalarAsync<int>("INSERT INTO Chunks (FileId, Offset, Length, Hash) VALUES (@FileId, @Offset, @Length, @Hash); SELECT last_insert_rowid();", row);
		}

		/// <summary>
		/// Adds multiple file chunk records
		/// </summary>
		public async Task AddChunksAsync(IEnumerable<ChunkRow> chunks)
		{
			await _connection.ExecuteScalarAsync<int>("INSERT INTO Chunks (FileId, Offset, Length, Hash) VALUES (@FileId, @Offset, @Length, @Hash);", chunks);
		}

		/// <summary>
		/// Gets a chunk row
		/// </summary>
		public async Task<ChunkRow> GetChunkAsync(int chunkId)
		{
			return await _connection.QuerySingleAsync<ChunkRow>("SELECT * FROM Chunks WHERE id = @ChunkId;", new { ChunkId = chunkId });
		}

		/// <summary>
		/// Find all chunks with a particular hash and length
		/// </summary>
		public async Task<IEnumerable<ChunkRow>> FindChunksAsync(IoHash hash, long length)
		{
			return await _connection.QueryAsync<ChunkRow>("SELECT * FROM Chunks WHERE hash = @Hash AND length = @Length;", new { Hash = hash, Length = length });
		}

		/// <summary>
		/// Finds all the chunks within a particular file
		/// </summary>
		public async Task<IEnumerable<ChunkRow>> FindChunksForFileAsync(int fileId)
		{
			return await _connection.QueryAsync<ChunkRow>("SELECT * FROM Chunks WHERE fileId = @FileId;", new { FileId = fileId });
		}

		/// <summary>
		/// Remove all chunks for a particular file
		/// </summary>
		public async Task RemoveChunksForFileAsync(int fileId)
		{
			await _connection.ExecuteAsync("DELETE FROM Chunks WHERE FileId = @FileId; DELETE FROM Files WHERE FileId = @FileId;", fileId);
		}

		/// <summary>
		/// Remove all chunks for a set of files
		/// </summary>
		public async Task RemoveChunksForFilesAsync(IEnumerable<int> fileIds)
		{
			await _connection.ExecuteAsync("DELETE FROM Chunks WHERE FileId IN (@FileIds);", new { FileIds = fileIds });
		}

		#endregion

		#region Directories

		/// <summary>
		/// Adds a new directory to the collection
		/// </summary>
		public async Task<int> AddDirectoryAsync(int parentDirectoryId, string name)
		{
			DirectoryRow row = new() { ParentDirectoryId = parentDirectoryId, Name = name };
			return await _connection.ExecuteScalarAsync<int>("INSERT INTO Directories (ParentDirectoryId, Name) VALUES (@ParentDirectoryId, @Name); SELECT last_insert_rowid();", row);
		}

		/// <summary>
		/// Adds multiple directories to the collection
		/// </summary>
		public async Task AddDirectoriesAsync(IEnumerable<DirectoryRow> dirs)
		{
			await _connection.ExecuteAsync("INSERT INTO Directories (ParentDirectoryId, Name) VALUES (@ParentDirectoryId, @Name);", dirs);
		}

		/// <summary>
		/// Gets the definition for a particular directory
		/// </summary>
		public async Task<DirectoryRow> GetDirectoryAsync(int id)
		{
			return await _connection.QuerySingleAsync<DirectoryRow>("SELECT * FROM Directories WHERE Id = @Id;", new { Id = id });
		}

		/// <summary>
		/// Gets the full name of a directory
		/// </summary>
		public async Task<string> GetDirectoryPathAsync(int id)
		{
			StringBuilder builder = new();
			await GetDirectoryPathAsync(id, builder);
			return builder.ToString();
		}

		/// <summary>
		/// Gets the full name of a directory
		/// </summary>
		public async Task GetDirectoryPathAsync(int id, StringBuilder builder)
		{
			if (id != RootDirectoryId)
			{
				DirectoryRow row = await GetDirectoryAsync(id);
				await GetDirectoryPathAsync(row.ParentDirectoryId, builder);
				builder.Append(row.Name);
				builder.Append('/');
			}
		}

		/// <summary>
		/// Finds all directories within a given parent directory
		/// </summary>
		public async Task<IEnumerable<DirectoryRow>> GetDirectoriesAsync(int parentDirectoryId)
		{
			return await _connection.QueryAsync<DirectoryRow>("SELECT * FROM Directories WHERE ParentDirectoryId = @ParentDirectoryId;", new { ParentDirectoryId = parentDirectoryId });
		}

		/// <summary>
		/// Removes a directory and all its subdirectories
		/// </summary>
		public async Task RemoveDirectoryAsync(int directoryId)
		{
			await RemoveDirectoryContentsAsync(directoryId);
			await _connection.ExecuteAsync("DELETE FROM Directories WHERE Id = @Id;", new { Id = directoryId });
		}

		/// <summary>
		/// Removes a directory and all its subdirectories
		/// </summary>
		public async Task RemoveDirectoriesAsync(IEnumerable<int> directoryIds)
		{
			await RemoveDirectoryContentsAsync(directoryIds);
			await _connection.ExecuteAsync("DELETE FROM Directories WHERE Id IN (@DirectoryIds);", new { DirectoryIds = directoryIds });
		}

		/// <summary>
		/// Removes the contents of a directory, without removing the directory itself
		/// </summary>
		public async Task RemoveDirectoryContentsAsync(int directoryId)
		{
			await RemoveDirectoryContentsAsync([directoryId]);
		}

		/// <summary>
		/// Removes all subdirectories and files starting at the given roots
		/// </summary>
		public async Task RemoveDirectoryContentsAsync(IEnumerable<int> directoryIds)
		{
			object directoryIdList = new { DirectoryIds = directoryIds };

			IEnumerable<int> subDirectoryIds = await _connection.QueryAsync<int>("SELECT Id FROM Directories WHERE ParentDirectoryId IN (@DirectoryIds);", directoryIdList);
			if (subDirectoryIds.Any())
			{
				await RemoveDirectoryContentsAsync(subDirectoryIds);
				await _connection.ExecuteAsync("DELETE FROM Directories WHERE ParentDirectoryId IN (@DirectoryIds);", directoryIdList);
			}

			IEnumerable<int> fileIds = await _connection.QueryAsync<int>("SELECT (Id) FROM Files WHERE DirectoryId IN (@DirectoryIds);", directoryIdList);
			if (fileIds.Any())
			{
				await RemoveChunksForFilesAsync(fileIds);
				await _connection.ExecuteAsync("DELETE FROM Files WHERE DirectoryId IN (@DirectoryIds);", directoryIdList);
			}
		}

		#endregion
	}

	/// <summary>
	/// Metadata for a file
	/// </summary>
	/// <param name="Id">Unique id for this file</param>
	/// <param name="DirectoryId">Identifier for the directory containing this file</param>
	/// <param name="Name">Name of the file</param>
	/// <param name="Time">Last modified timestamp for the file</param>
	/// <param name="Length">Length of the file</param>
	public record class FileRow(int Id, int DirectoryId, string Name, DateTime Time, long Length)
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public FileRow()
			: this(-1, -1, String.Empty, DateTime.MinValue, 0)
		{ }

		/// <summary>
		/// Constructor for new file rows
		/// </summary>
		public FileRow(int directoryId, string name, DateTime time, long length)
			: this(-1, directoryId, name, time, length)
		{ }
	}

	/// <summary>
	/// Metadata for a directory
	/// </summary>
	/// <param name="Id">Unique id for this directory</param>
	/// <param name="ParentDirectoryId">Parent directory identifier</param>
	/// <param name="Name">Name of the directory</param>
	public record class DirectoryRow(int Id, int ParentDirectoryId, string Name)
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public DirectoryRow()
			: this(-1, -1, String.Empty)
		{ }

		/// <summary>
		/// Constructor for new directory rows
		/// </summary>
		public DirectoryRow(int parentDirectoryId, string name)
			: this(-1, parentDirectoryId, name)
		{ }
	}

	/// <summary>
	/// Metadata for a file chunk
	/// </summary>
	/// <param name="Id">Unique id for the row</param>
	/// <param name="FileId">Id of the file that this chunk belongs to</param>
	/// <param name="Offset">Starting offset within the file of this chunk</param>
	/// <param name="Length">Length of the chunk</param>
	/// <param name="Hash">Hash of the chunk data</param>
	public record class ChunkRow(int Id, int FileId, long Offset, long Length, IoHash Hash)
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public ChunkRow()
			: this(-1, -1, 0, 0, IoHash.Zero)
		{ }

		/// <summary>
		/// Constructor for new chunk rows
		/// </summary>
		public ChunkRow(int fileId, long offset, long length, IoHash hash)
			: this(-1, fileId, offset, length, hash)
		{ }
	}
}
