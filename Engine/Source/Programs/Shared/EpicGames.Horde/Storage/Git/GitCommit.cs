// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Git
{
	/// <summary>
	/// Property names for git commits
	/// </summary>
	public static class GitCommitProperty
	{
		/// <summary>
		/// The root of the tree for this commit
		/// </summary>
		public static Utf8String Tree { get; } = new Utf8String("tree");

		/// <summary>
		/// Author of the commit
		/// </summary>
		public static Utf8String Author { get; } = new Utf8String("author");

		/// <summary>
		/// Person that committed the change
		/// </summary>
		public static Utf8String Committer { get; } = new Utf8String("committer");

		/// <summary>
		/// Parent commit
		/// </summary>
		public static Utf8String Parent { get; } = new Utf8String("parent");
	}

	/// <summary>
	/// Representation of a Git commit object
	/// </summary>
	public class GitCommit
	{
		/// <summary>
		/// Properties for the commit
		/// </summary>
		public List<KeyValuePair<Utf8String, Utf8String>> Properties { get; } = [];

		/// <summary>
		/// Commit messages
		/// </summary>
		public Utf8String Message { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="tree"></param>
		/// <param name="message"></param>
		public GitCommit(Sha1Hash tree, Utf8String message)
		{
			AddProperty(GitCommitProperty.Tree, tree.ToUtf8String());
			Message = message;
		}

		/// <summary>
		/// Adds a new property to the collection
		/// </summary>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public void AddProperty(Utf8String name, Utf8String value)
		{
			Properties.Add(new KeyValuePair<Utf8String, Utf8String>(name, value));
		}

		/// <summary>
		/// Gets a property value with the given key name
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <returns>The property value</returns>
		public Utf8String GetProperty(Utf8String name)
		{
			foreach (KeyValuePair<Utf8String, Utf8String> pair in Properties)
			{
				if (pair.Key == name)
				{
					return pair.Value;
				}
			}
			return Utf8String.Empty;
		}

		/// <summary>
		/// Gets property values with the given key name
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <returns>The property value</returns>
		public IEnumerable<Utf8String> FindProperties(Utf8String name)
		{
			foreach (KeyValuePair<Utf8String, Utf8String> pair in Properties)
			{
				if (pair.Key == name)
				{
					yield return pair.Value;
				}
			}
		}

		/// <summary>
		/// The tree for this commit
		/// </summary>
		public Sha1Hash GetTree() => Sha1Hash.Parse(GetProperty(GitCommitProperty.Tree));

		/// <summary>
		/// Parents of this commit
		/// </summary>
		public IEnumerable<Sha1Hash> GetParents() => FindProperties(GitCommitProperty.Parent).Select(x => Sha1Hash.Parse(x));

		/// <summary>
		/// Serializes this object
		/// </summary>
		/// <returns></returns>
		public byte[] Serialize()
		{
			int size = Message.Length + 1 + Properties.Sum(x => x.Key.Length + x.Value.Length + 2);

			Span<byte> header = GitObject.WriteHeader(GitObjectType.Commit, size, stackalloc byte[32]);

			byte[] data = new byte[header.Length + size];
			header.CopyTo(data);

			Span<byte> output = data.AsSpan(header.Length);
			foreach (KeyValuePair<Utf8String, Utf8String> pair in Properties)
			{
				pair.Key.Span.CopyTo(output);
				output = output.Slice(pair.Key.Length);

				output[0] = (byte)' ';
				output = output.Slice(1);

				pair.Value.Span.CopyTo(output);
				output = output.Slice(pair.Value.Length);

				output[0] = (byte)'\n';
				output = output.Slice(1);
			}

			output[0] = (byte)'\n';
			output = output.Slice(1);

			Message.Span.CopyTo(output);
			Debug.Assert(output.Length == Message.Length);

			return data;
		}
	}
}
