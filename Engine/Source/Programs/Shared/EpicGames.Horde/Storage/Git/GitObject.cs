// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Git
{
	/// <summary>
	/// Type fields for git objects
	/// </summary>
	public static class GitObjectType
	{
		/// <summary>
		/// Blob objects
		/// </summary>
		public static Utf8String Blob { get; } = new Utf8String("blob");

		/// <summary>
		/// Tree objects
		/// </summary>
		public static Utf8String Tree { get; } = new Utf8String("tree");

		/// <summary>
		/// Commit objects
		/// </summary>
		public static Utf8String Commit { get; } = new Utf8String("commit");
	}

	/// <summary>
	/// Utility methods for manipulating Git objects
	/// </summary>
	public static class GitObject
	{
		/// <summary>
		/// Writes the header for an object to a stream
		/// </summary>
		/// <param name="stream"></param>
		/// <param name="type">Type of the object</param>
		/// <param name="size">Size of the object</param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public static async Task WriteHeaderAsync(Stream stream, Utf8String type, long size, CancellationToken cancellationToken = default)
		{
			byte[] header = new byte[32];
			int length = WriteHeader(type, size, header).Length;
			await stream.WriteAsync(header.AsMemory(0, length), cancellationToken);
		}

		/// <summary>
		/// Appends data for a header to the hash
		/// </summary>
		/// <param name="type">Type of the object</param>
		/// <param name="size">Size of the object</param>
		/// <param name="hash">Hash for the header data</param>
		public static void WriteHeader(Utf8String type, long size, IncrementalHash hash)
		{
			Span<byte> header = WriteHeader(type, size, stackalloc byte[32]);
			hash.AppendData(header);
		}

		/// <summary>
		/// Writes a header for an object
		/// </summary>
		/// <param name="type">Type of the object</param>
		/// <param name="size">Size of the object</param>
		/// <param name="buffer">Buffer to receive the data</param>
		/// <returns></returns>
		public static Span<byte> WriteHeader(Utf8String type, long size, Span<byte> buffer)
		{
			type.Span.CopyTo(buffer);
			buffer[type.Length] = (byte)' ';

			int start = type.Length + 1;
			int end = start;
			for (; ; )
			{
				buffer[end++] = (byte)('0' + (size % 10));
				size /= 10;
				if (size == 0)
				{
					break;
				}
			}

			buffer.Slice(start, end - start).Reverse();
			buffer[end] = 0;

			return buffer.Slice(0, end + 1);
		}
	}
}
