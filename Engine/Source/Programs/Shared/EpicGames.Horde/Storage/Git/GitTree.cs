// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Cryptography;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Git
{
	/// <summary>
	/// Valid file modes for tree entries
	/// </summary>
	public static class GitFileMode
	{
		/// <summary>
		/// Regular file
		/// </summary>
		public static Utf8String File { get; } = new Utf8String("100644");

		/// <summary>
		/// Executable file
		/// </summary>
		public static Utf8String ExecutableFile { get; } = new Utf8String("100755");

		/// <summary>
		/// Child tree
		/// </summary>
		public static Utf8String Tree { get; } = new Utf8String("40000");
	}

	/// <summary>
	/// Representation of a Git tree object
	/// </summary>
	public class GitTree
	{
		/// <summary>
		/// Entries for this tree. Sort be sorted for consistency.
		/// </summary>
		public List<GitTreeEntry> Entries { get; } = [];

		/// <summary>
		/// Gets the hash of this tree object.
		/// </summary>
		/// <returns>Hash of the object</returns>
		public Sha1Hash GetHash()
		{
			Span<byte> buffer = stackalloc byte[32];
			using (IncrementalHash hash = IncrementalHash.CreateHash(HashAlgorithmName.SHA1))
			{
				int size = GetPayloadSize();
				GitObject.WriteHeader(GitObjectType.Tree, size, hash);

				foreach (GitTreeEntry entry in Entries)
				{
					hash.AppendData(entry.Mode.Span);

					buffer[0] = (byte)' ';
					hash.AppendData(buffer.Slice(0, 1));

					hash.AppendData(entry.Name.Span);

					buffer[0] = 0;
					entry.Hash.CopyTo(buffer.Slice(1));
					hash.AppendData(buffer.Slice(0, Sha1Hash.NumBytes + 1));
				}

				hash.GetHashAndReset(buffer);
				return new Sha1Hash(buffer);
			}
		}

		/// <summary>
		/// Serializes this object
		/// </summary>
		/// <returns></returns>
		public byte[] Serialize()
		{
			int size = GetPayloadSize();

			Span<byte> header = GitObject.WriteHeader(GitObjectType.Tree, size, stackalloc byte[32]);

			byte[] data = new byte[header.Length + size];
			header.CopyTo(data);

			Span<byte> output = data.AsSpan(header.Length);
			foreach (GitTreeEntry entry in Entries)
			{
				entry.Mode.Span.CopyTo(output);
				output[entry.Mode.Length] = (byte)' ';
				output = output.Slice(entry.Mode.Length + 1);

				entry.Name.Span.CopyTo(output);
				output[entry.Name.Length] = 0;
				output = output.Slice(entry.Name.Length + 1);

				entry.Hash.CopyTo(output);
				output = output.Slice(Sha1Hash.NumBytes);
			}

			return data;
		}

		int GetPayloadSize()
		{
			int size = 0;
			foreach (GitTreeEntry entry in Entries)
			{
				size += entry.Mode.Length + 1 + entry.Name.Length + 1 + Sha1Hash.NumBytes;
			}
			return size;
		}
	}

	/// <summary>
	/// Entry for a Git tree
	/// </summary>
	public class GitTreeEntry
	{
		/// <summary>
		/// Mode for this entry. Can be any values from <see cref="GitFileMode"/>
		/// </summary>
		public Utf8String Mode { get; }

		/// <summary>
		/// Name of this entry
		/// </summary>
		public Utf8String Name { get; }

		/// <summary>
		/// Hash of the object for this entry
		/// </summary>
		public Sha1Hash Hash { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GitTreeEntry(Utf8String mode, Utf8String name, Sha1Hash hash)
		{
			Mode = mode;
			Name = name;
			Hash = hash;
		}
	}
}
