// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Factory for constructing storage namespace instances with an HTTP backend
	/// </summary>
	public sealed class HttpStorageClient : IStorageClient
	{
		readonly HttpStorageBackendFactory _backendFactory;
		readonly BundleCache _bundleCache;
		readonly IOptionsSnapshot<HordeOptions> _hordeOptions;
		readonly ILogger<BundleStorageNamespace> _clientLogger;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpStorageClient(HttpStorageBackendFactory backendFactory, BundleCache bundleCache, IOptionsSnapshot<HordeOptions> hordeOptions, ILogger<BundleStorageNamespace> clientLogger)
		{
			_backendFactory = backendFactory;
			_bundleCache = bundleCache;
			_hordeOptions = hordeOptions;
			_clientLogger = clientLogger;
		}

		/// <summary>
		/// Creates a new HTTP storage namespace
		/// </summary>
		/// <param name="basePath">Base path for all requests</param>
		/// <param name="accessToken">Custom access token to use for requests</param>
		/// <param name="withBackendCache"></param>
		public IStorageNamespace GetNamespaceWithPath(string basePath, string? accessToken = null, bool withBackendCache = true)
		{
			IStorageBackend backend = _backendFactory.CreateBackend(basePath, accessToken, withBackendCache);
			return new BundleStorageNamespace(backend, _bundleCache, _hordeOptions.Value.Bundle, _clientLogger);
		}

		/// <summary>
		/// Creates a new HTTP storage namespace
		/// </summary>
		/// <param name="namespaceId">Namespace to create a client for</param>
		/// <param name="accessToken">Custom access token to use for requests</param>
		/// <param name="withBackendCache">Whether to enable the backend cache, which caches full bundles to disk</param>
		public IStorageNamespace GetNamespace(NamespaceId namespaceId, string? accessToken = null, bool withBackendCache = true) => GetNamespaceWithPath($"api/v1/storage/{namespaceId}", accessToken, withBackendCache);

		/// <inheritdoc/>
		IStorageNamespace? IStorageClient.TryGetNamespace(NamespaceId namespaceId) => GetNamespace(namespaceId);
	}
}
