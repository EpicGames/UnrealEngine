// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

#pragma warning disable CA1716 // Do not use 'Imports' as identifier

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Interface for reading nodes from storage
	/// </summary>
	public interface IBlobReader : IMemoryReader
	{
		/// <summary>
		/// Type to deserialize
		/// </summary>
		BlobType Type { get; }

		/// <summary>
		/// Version of the current node, as specified via <see cref="BlobType"/>
		/// </summary>
		int Version { get; }

		/// <summary>
		/// Locations of all referenced nodes. These handles do not have valid hashes.
		/// </summary>
		IReadOnlyList<IBlobRef> Imports { get; }

		/// <summary>
		/// Gets the next serialized blob handle
		/// </summary>
		IHashedBlobRef ReadBlobRef();

		/// <summary>
		/// Gets the next serialized blob handle
		/// </summary>
		IHashedBlobRef<T> ReadBlobRef<T>();
	}

	/// <summary>
	/// Reader for blob objects
	/// </summary>
	public sealed class BlobReader : MemoryReader, IBlobReader
	{
		/// <summary>
		/// Type to deserialize
		/// </summary>
		public BlobType Type => _blobData.Type;

		/// <summary>
		/// Version of the current node, as specified via <see cref="BlobType"/>
		/// </summary>
		public int Version => Type.Version;

		/// <summary>
		/// Total length of the data in this node
		/// </summary>
		public int Length => _blobData.Data.Length;

		/// <summary>
		/// Amount of data remaining to be read
		/// </summary>
		public int RemainingLength => RemainingMemory.Length;

		/// <summary>
		/// Raw data for this blob
		/// </summary>
		public ReadOnlyMemory<byte> Data => _blobData.Data;

		/// <summary>
		/// Locations of all referenced nodes.
		/// </summary>
		public IReadOnlyList<IBlobRef> Imports => _blobData.Imports;

		readonly BlobData _blobData;
		readonly BlobSerializerOptions _options;
		int _importIdx;

		/// <summary>
		/// Constructor
		/// </summary>
		public BlobReader(BlobData blobData, BlobSerializerOptions? options)
			: base(blobData.Data)
		{
			_blobData = blobData;
			_options = options ?? BlobSerializerOptions.Default;
		}

		/// <summary>
		/// Gets the next serialized blob reference
		/// </summary>
		public IHashedBlobRef ReadBlobRef()
		{
			IBlobRef import = Imports[_importIdx++];
			IoHash hash = this.ReadIoHash();
			return HashedBlobRef.Create(hash, import);
		}

		/// <summary>
		/// Gets the next serialized blob reference
		/// </summary>
		public IHashedBlobRef<T> ReadBlobRef<T>()
		{
			IBlobRef import = Imports[_importIdx++];
			IoHash hash = this.ReadIoHash();
			return HashedBlobRef.Create<T>(hash, import, _options);
		}
	}
}
