// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Reference to another node in storage. This type is similar to <see cref="IHashedBlobRef"/>, but without a hash.
	/// </summary>
	public interface IBlobRef
	{
		/// <summary>
		/// Accessor for the innermost import
		/// </summary>
		IBlobRef Innermost { get; }

		/// <summary>
		/// Flush the referenced data to underlying storage
		/// </summary>
		ValueTask FlushAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Reads the blob's data
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to get a path for this blob.
		/// </summary>
		/// <param name="locator">Receives the blob path on success.</param>
		/// <returns>True if a path was available, false if the blob has not yet been flushed to storage.</returns>
		bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator);
	}

	/// <summary>
	/// Typed interface to a particular blob handle
	/// </summary>
	/// <typeparam name="T">Type of the deserialized blob</typeparam>
	public interface IBlobRef<out T> : IBlobRef
	{
		/// <summary>
		/// Options for deserializing the blob
		/// </summary>
		BlobSerializerOptions? SerializerOptions { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IBlobRef"/>
	/// </summary>
	public static class BlobRefExtensions
	{
		class TypedBlobRef<T> : IBlobRef<T>
		{
			readonly IBlobRef _inner;

			/// <inheritdoc/>
			public BlobSerializerOptions? SerializerOptions { get; }

			public IBlobRef Innermost => throw new NotImplementedException();

			public TypedBlobRef(IBlobRef inner, BlobSerializerOptions? serializerOptions)
			{
				_inner = inner;
				SerializerOptions = serializerOptions;
			}

			public ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> _inner.FlushAsync(cancellationToken);

			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
				=> _inner.ReadBlobDataAsync(cancellationToken);

			public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
				=> _inner.TryGetLocator(out locator);
		}

		/// <summary>
		/// Create a typed blob reference
		/// </summary>
		/// <typeparam name="T">Target type</typeparam>
		/// <param name="blobRef">Blob referfence to wrap</param>
		/// <param name="serializerOptions">Options for deserializing the blob</param>
		public static IBlobRef<T> ForType<T>(this IBlobRef blobRef, BlobSerializerOptions? serializerOptions = null)
			=> new TypedBlobRef<T>(blobRef, serializerOptions);

		/// <summary>
		/// Gets a path to this blob that can be used to describe blob references over the wire.
		/// </summary>
		/// <param name="import">Handle to query</param>
		public static BlobLocator GetLocator(this IBlobRef import)
		{
			BlobLocator locator;
			if (!import.TryGetLocator(out locator))
			{
				throw new InvalidOperationException("Blob has not yet been written to storage");
			}
			return locator;
		}
	}
}
