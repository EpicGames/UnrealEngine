// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

#pragma warning disable CA1716 // Do not use 'imports' as variable name

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Interface for a writer of node objects
	/// </summary>
	public interface IBlobWriter : IMemoryWriter, IAsyncDisposable
	{
		/// <summary>
		/// Options for serialization
		/// </summary>
		BlobSerializerOptions Options { get; }

		/// <summary>
		/// Accessor for the memory written to the current blob
		/// </summary>
		ReadOnlyMemory<byte> WrittenMemory { get; }

		/// <summary>
		/// Adds an alias to the blob currently being written
		/// </summary>
		/// <param name="name">Name of the alias</param>
		/// <param name="rank">Rank to use when finding blobs by alias</param>
		/// <param name="data">Inline data to store with the alias</param>
		void AddAlias(string name, int rank, ReadOnlyMemory<byte> data = default);

		/// <summary>
		/// Flush any pending nodes to storage
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task FlushAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Create another writer instance, allowing multiple threads to write in parallel.
		/// </summary>
		/// <returns>New writer instance</returns>
		IBlobWriter Fork();

		/// <summary>
		/// Finish writing a blob that has been written into the output buffer.
		/// </summary>
		/// <param name="type">Type of the node that was written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the written node</returns>
		ValueTask<IHashedBlobRef> CompleteAsync(BlobType type, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finish writing a blob that has been written into the output buffer.
		/// </summary>
		/// <param name="type">Type of the node that was written</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the written node</returns>
		ValueTask<IHashedBlobRef<T>> CompleteAsync<T>(BlobType type, CancellationToken cancellationToken = default);

		/// <summary>
		/// Writes a reference to another blob. NOTE: This does not write anything to the underlying output stream, which prevents the data forming a Merkle tree
		/// unless guaranteed uniqueness via a hash being written separately.
		/// </summary>
		/// <param name="handle">Referenced blob</param>
		void WriteBlobHandleDangerous(IBlobRef handle);

		/// <summary>
		/// Writes a reference to another blob. The blob's hash is serialized to the output stream.
		/// </summary>
		/// <param name="blobRef">Referenced blob</param>
		void WriteBlobRef(IHashedBlobRef blobRef);
	}

	/// <summary>
	/// Information about an alias to be added alongside a blob
	/// </summary>
	/// <param name="Name">Name of the alias</param>
	/// <param name="Rank">Rank of the alias</param>
	/// <param name="Data">Inline data to be stored for the alias</param>
	public record class AliasInfo(string Name, int Rank, ReadOnlyMemory<byte> Data);

	/// <summary>
	/// Base class for <see cref="IBlobWriter"/> implementations.
	/// </summary>
	public abstract class BlobWriter : IBlobWriter
	{
		Memory<byte> _memory;
		readonly List<AliasInfo> _aliases = [];
		readonly List<IBlobRef> _imports = [];
		readonly BlobSerializerOptions _options;
		int _length;

		/// <inheritdoc/>
		public int Length => _length;

		/// <inheritdoc/>
		public BlobSerializerOptions Options => _options;

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> WrittenMemory => _memory.Slice(0, _length);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="options"></param>
		protected BlobWriter(BlobSerializerOptions? options)
		{
			_options = options ?? BlobSerializerOptions.Default;
		}

		/// <summary>
		/// Computes the hash of the written data
		/// </summary>
		public IoHash ComputeHash() => IoHash.Compute(_memory.Span.Slice(0, _length));

		/// <inheritdoc/>
		public void WriteBlobHandleDangerous(IBlobRef target)
		{
			_imports.Add(target);
		}

		/// <summary>
		/// Writes a handle to another node
		/// </summary>
		public void WriteBlobRef(IHashedBlobRef target)
		{
			this.WriteIoHash(target.Hash);
			WriteBlobHandleDangerous(target);
		}

		/// <inheritdoc/>
		public Span<byte> GetSpan(int sizeHint = 0) => GetMemory(sizeHint).Span;

		/// <inheritdoc/>
		public Memory<byte> GetMemory(int sizeHint = 0)
		{
			int newLength = _length + Math.Max(sizeHint, 1);
			if (newLength > _memory.Length)
			{
				newLength = _length + Math.Max(sizeHint, 1024);
				_memory = GetOutputBuffer(_length, Math.Max(_memory.Length * 2, newLength));
			}
			return _memory.Slice(_length);
		}

		/// <inheritdoc/>
		public void Advance(int length) => _length += length;

		/// <summary>
		/// Request a new buffer to write to
		/// </summary>
		/// <param name="usedSize">Size of data written to the current buffer</param>
		/// <param name="desiredSize">Desired size for the buffer</param>
		/// <returns>New buffer</returns>
		public abstract Memory<byte> GetOutputBuffer(int usedSize, int desiredSize);

		/// <summary>
		/// Write the current blob to storage
		/// </summary>
		/// <param name="type">Type of the blob</param>
		/// <param name="size">Size of the blob to write</param>
		/// <param name="imports">References to other blobs</param>
		/// <param name="aliases">Aliases for the new blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New buffer</returns>
		public abstract ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken);

		/// <inheritdoc/>
		public void AddAlias(string name, int rank, ReadOnlyMemory<byte> data)
			=> _aliases.Add(new AliasInfo(name, rank, data));

		/// <inheritdoc/>
		public abstract Task FlushAsync(CancellationToken cancellationToken = default);

		/// <inheritdoc/>
		public abstract IBlobWriter Fork();

		/// <inheritdoc/>
		public async ValueTask<IHashedBlobRef> CompleteAsync(BlobType type, CancellationToken cancellationToken = default)
		{
			IHashedBlobRef blobRef = await WriteBlobAsync(type, _length, _imports, _aliases, cancellationToken);

			_memory = default;
			_length = 0;
			_imports.Clear();
			_aliases.Clear();

			return blobRef;
		}

		/// <inheritdoc/>
		public async ValueTask<IHashedBlobRef<T>> CompleteAsync<T>(BlobType type, CancellationToken cancellationToken = default)
		{
			IoHash hash = IoHash.Compute(_memory.Span.Slice(0, _length));
			IHashedBlobRef blobRef = await CompleteAsync(type, cancellationToken);
			return HashedBlobRef.Create<T>(hash, blobRef, _options);
		}

		/// <inheritdoc/>
		public abstract ValueTask DisposeAsync();
	}

	/// <summary>
	/// Implementation of <see cref="IBlobWriter"/> which discards any written data
	/// </summary>
	public sealed class NullBlobWriter : BlobWriter
	{
		class BlobRef : IHashedBlobRef
		{
			public IoHash Hash { get; }

			public IBlobRef Innermost
				=> throw new NotImplementedException();

			public BlobRef(IoHash hash)
				=> Hash = hash;

			public ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> default;

			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
				=> throw new NotSupportedException();

			public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
			{
				locator = new BlobLocator(Hash.ToUtf8String());
				return true;
			}
		}

		readonly ChunkedMemoryWriter _memoryWriter;
		readonly BlobSerializerOptions _options;

		/// <summary>
		/// Constructor
		/// </summary>
		public NullBlobWriter(BlobSerializerOptions options)
			: base(options)
		{
			_memoryWriter = new ChunkedMemoryWriter();
			_options = options;
		}

		/// <inheritdoc/>
		public override ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);
			_memoryWriter.Dispose();
			return new ValueTask();
		}

		/// <inheritdoc/>
		public override ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken)
		{
			Memory<byte> memory = _memoryWriter.GetMemoryAndAdvance(size);
			IoHash hash = IoHash.Compute(memory.Span);
			_memoryWriter.Clear();
			return new ValueTask<IHashedBlobRef>(new BlobRef(hash));
		}

		/// <inheritdoc/>
		public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
			=> _memoryWriter.GetMemory(usedSize, desiredSize);

		/// <inheritdoc/>
		public override Task FlushAsync(CancellationToken cancellationToken = default)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public override IBlobWriter Fork()
			=> new NullBlobWriter(_options);
	}

	/// <summary>
	/// Implementation of <see cref="IBlobWriter"/> which just buffers data in memory
	/// </summary>
	public class MemoryBlobWriter : BlobWriter
	{
		class BlobRef : IHashedBlobRef
		{
			readonly int _index;
			readonly IoHash _hash;
			readonly BlobData _data;

			public BlobRef(int index, BlobData data)
			{
				_index = index;
				_hash = IoHash.Compute(data.Data.Span);
				_data = data;
			}

			public IoHash Hash
				=> _hash;

			public int Index
				=> _index;

			public IBlobRef Innermost
				=> this;

			public ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> default;

			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
				=> new(_data);

			public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
				=> throw new NotSupportedException();
		}

		readonly ChunkedMemoryWriter _memoryWriter;
		int _nextIndex;

		/// <summary>
		/// Constructor
		/// </summary>
		public MemoryBlobWriter(BlobSerializerOptions options)
			: base(options)
		{
			_memoryWriter = new ChunkedMemoryWriter();
		}

		/// <inheritdoc/>
		public override ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);
			_memoryWriter.Dispose();
			return new ValueTask();
		}

		/// <summary>
		/// Clears the contents of this writer
		/// </summary>
		public void Clear()
			=> _memoryWriter.Clear();

		/// <inheritdoc/>
		public override Task FlushAsync(CancellationToken cancellationToken = default)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public override IBlobWriter Fork()
			=> new MemoryBlobWriter(Options);

		/// <inheritdoc/>
		public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
			=> _memoryWriter.GetMemory(usedSize, desiredSize);

		/// <inheritdoc/>
		public override ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken)
		{
			Memory<byte> memory = _memoryWriter.GetMemoryAndAdvance(size);
			BlobData data = new(type, memory.ToArray(), [.. imports]);
			return new ValueTask<IHashedBlobRef>(new BlobRef(++_nextIndex, data));
		}

		/// <summary>
		/// Helper function to get the index of a blob
		/// </summary>
		public static int GetIndex(IHashedBlobRef handle)
			=> ((BlobRef)handle.Innermost).Index;
	}
}
