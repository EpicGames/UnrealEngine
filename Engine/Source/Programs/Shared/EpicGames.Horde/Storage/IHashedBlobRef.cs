// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Handle to a node. Can be used to reference nodes that have not been flushed yet.
	/// </summary>
	public interface IHashedBlobRef : IBlobRef
	{
		/// <summary>
		/// Hash of the target node
		/// </summary>
		IoHash Hash { get; }
	}

	/// <summary>
	/// Typed interface to a particular blob handle
	/// </summary>
	/// <typeparam name="T">Type of the deserialized blob</typeparam>
	public interface IHashedBlobRef<out T> : IHashedBlobRef, IBlobRef<T>
	{
	}

	/// <summary>
	/// Contains the value for a blob ref
	/// </summary>
	public record class HashedBlobRefValue(IoHash Hash, BlobLocator Locator);

	/// <summary>
	/// Helper methods for creating blob handles
	/// </summary>
	public static class HashedBlobRef
	{
		class HashedBlobRefImpl : IHashedBlobRef
		{
			readonly IoHash _hash;
			readonly IBlobRef _handle;

			public IBlobRef Innermost => _handle.Innermost;
			public IoHash Hash => _hash;

			public HashedBlobRefImpl(IoHash hash, IBlobRef handle)
			{
				_hash = hash;
				_handle = handle;
			}

			public ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> _handle.FlushAsync(cancellationToken);

			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
				=> _handle.ReadBlobDataAsync(cancellationToken);

			public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
				=> _handle.TryGetLocator(out locator);
		}

		class HashedBlobRefImpl<T> : HashedBlobRefImpl, IHashedBlobRef<T>
		{
			readonly BlobSerializerOptions _options;

			public BlobSerializerOptions SerializerOptions => _options;

			public HashedBlobRefImpl(IoHash hash, IBlobRef handle, BlobSerializerOptions options)
				: base(hash, handle)
			{
				_options = options;
			}
		}

		/// <summary>
		/// Create an untyped blob handle
		/// </summary>
		/// <param name="handle">Imported blob interface</param>
		/// <param name="hash">Hash of the blob</param>
		/// <returns>Handle to the blob</returns>
		public static IHashedBlobRef Create(IoHash hash, IBlobRef handle)
			=> new HashedBlobRefImpl(hash, handle);

		/// <summary>
		/// Create a typed blob handle
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="blobRef">Existing blob reference</param>
		/// <param name="options">Options for deserializing the target blob</param>
		/// <returns>Handle to the blob</returns>
		public static IHashedBlobRef<T> Create<T>(IHashedBlobRef blobRef, BlobSerializerOptions? options = null)
			=> new HashedBlobRefImpl<T>(blobRef.Hash, blobRef, options ?? BlobSerializerOptions.Default);

		/// <summary>
		/// Create a typed blob handle
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="hash">Hash of the blob</param>
		/// <param name="handle">Imported blob interface</param>
		/// <param name="options">Options for deserializing the target blob</param>
		/// <returns>Handle to the blob</returns>
		public static IHashedBlobRef<T> Create<T>(IoHash hash, IBlobRef handle, BlobSerializerOptions? options = null)
			=> new HashedBlobRefImpl<T>(hash, handle, options ?? BlobSerializerOptions.Default);
	}

	/// <summary>
	/// Extension methods for <see cref="IHashedBlobRef"/>
	/// </summary>
	public static class HashedBlobRefExtensions
	{
		/// <summary>
		/// Gets a BlobRefValue from an IBlobRef
		/// </summary>
		public static HashedBlobRefValue GetRefValue(this IHashedBlobRef blobRef)
		{
			return new HashedBlobRefValue(blobRef.Hash, blobRef.GetLocator());
		}
	}
}
