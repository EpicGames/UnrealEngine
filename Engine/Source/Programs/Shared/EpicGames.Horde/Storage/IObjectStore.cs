// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Interface for a object storage service.
	/// </summary>
	public interface IObjectStore
	{
		/// <summary>
		/// Whether this storage backend supports HTTP redirects for reads and writes
		/// </summary>
		bool SupportsRedirects { get; }

		/// <summary>
		/// Attempts to open a read stream for the given path.
		/// </summary>
		/// <param name="key">Relative path within the bucket</param>
		/// <param name="offset">Offset to start reading from</param>
		/// <param name="length">Length of data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default);

		/// <summary>
		/// Reads an object into memory and returns a handle to it.
		/// </summary>
		/// <param name="key">Path to the file</param>
		/// <param name="offset">Offset of the data to retrieve</param>
		/// <param name="length">Length of the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the data. Must be disposed by the caller.</returns>
		Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default);

		/// <summary>
		/// Writes a stream to the storage backend. If the stream throws an exception during read, the write will be aborted.
		/// </summary>
		/// <param name="key">Path to write to</param>
		/// <param name="stream">Stream to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to the uploaded object</returns>
		Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default);

		/// <summary>
		/// Tests whether the given path exists
		/// </summary>
		/// <param name="key">Relative path within the bucket</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the object exists</returns>
		Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the size of a particular object
		/// </summary>
		/// <param name="key">Relative path within the bucket</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Size of the object, or -1 if it does not exist</returns>
		Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a file with the given path
		/// </summary>
		/// <param name="key">Relative path within the bucket</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a HTTP redirect for a read request
		/// </summary>
		/// <param name="key">Path to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to upload the data to</returns>
		ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a HTTP redirect for a write request
		/// </summary>
		/// <param name="key">Path to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path for retrieval, and URI to upload the data to</returns>
		ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets stats for this storage backend
		/// </summary>
		void GetStats(StorageStats stats);
	}

	/// <summary>
	/// Typed <see cref="IObjectStore"/> instance for dependency injection
	/// </summary>
	public interface IObjectStore<T> : IObjectStore
	{
	}

	/// <summary>
	/// Extension methods for <see cref="IObjectStore"/>
	/// </summary>
	public static class ObjectStoreExtensions
	{
		sealed class TypedObjectStore<T> : IObjectStore<T>
		{
			readonly IObjectStore _inner;

			public bool SupportsRedirects => _inner.SupportsRedirects;

			public TypedObjectStore(IObjectStore inner) => _inner = inner;

			public Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken) => _inner.OpenAsync(key, offset, length, cancellationToken);
			public Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken) => _inner.ReadAsync(key, offset, length, cancellationToken);
			public Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken) => _inner.WriteAsync(key, stream, cancellationToken);
			public Task DeleteAsync(ObjectKey path, CancellationToken cancellationToken) => _inner.DeleteAsync(path, cancellationToken);
			public Task<bool> ExistsAsync(ObjectKey path, CancellationToken cancellationToken) => _inner.ExistsAsync(path, cancellationToken);
			public Task<long> GetSizeAsync(ObjectKey path, CancellationToken cancellationToken) => _inner.GetSizeAsync(path, cancellationToken);
			public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey path, CancellationToken cancellationToken = default) => _inner.TryGetReadRedirectAsync(path, cancellationToken);
			public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey path, CancellationToken cancellationToken = default) => _inner.TryGetWriteRedirectAsync(path, cancellationToken);
			public void GetStats(StorageStats stats) => _inner.GetStats(stats);
		}

		/// <summary>
		/// Creates a typed wrapper around the given object store
		/// </summary>
		/// <param name="store">Store to wrap</param>
		public static IObjectStore<T> ForType<T>(this IObjectStore store)
			=> new TypedObjectStore<T>(store);

		/// <summary>
		/// Attempts to open a read stream for the given path.
		/// </summary>
		/// <param name="store">Store to read from</param>
		/// <param name="key">Relative path within the bucket</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static Task<Stream> OpenAsync(this IObjectStore store, ObjectKey key, CancellationToken cancellationToken = default)
			=> store.OpenAsync(key, 0, null, cancellationToken);

		/// <summary>
		/// Reads an object into memory and returns a handle to it.
		/// </summary>
		/// <param name="store">Store to read from</param>
		/// <param name="key">Path to the file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the data. Must be disposed by the caller.</returns>
		public static Task<IReadOnlyMemoryOwner<byte>> ReadAsync(this IObjectStore store, ObjectKey key, CancellationToken cancellationToken = default)
			=> store.ReadAsync(key, 0, null, cancellationToken);

		/// <summary>
		/// Writes a stream to the storage backend. If the stream throws an exception during read, the write will be aborted.
		/// </summary>
		/// <param name="store">Store to write to</param>
		/// <param name="locator">Path to write to</param>
		/// <param name="data">Data to write</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to the uploaded object</returns>
		public static async Task WriteAsync(this IObjectStore store, ObjectKey locator, ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default)
		{
			using ReadOnlyMemoryStream stream = new(data);
			await store.WriteAsync(locator, stream, cancellationToken);
		}
	}
}
