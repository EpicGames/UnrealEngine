// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

#pragma warning disable CA1716

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Interface for a low-level storage backend.
	/// </summary>
	public interface IStorageBackend
	{
		#region Blobs

		/// <summary>
		/// Whether this storage backend supports HTTP redirects for reads and writes
		/// </summary>
		bool SupportsRedirects { get; }

		/// <summary>
		/// Attempts to open a read stream for the given path.
		/// </summary>
		/// <param name="locator">Relative path within the bucket</param>
		/// <param name="offset">Offset to start reading from</param>
		/// <param name="length">Length of data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default);

		/// <summary>
		/// Reads an object into memory and returns a handle to it.
		/// </summary>
		/// <param name="locator">Path to the file</param>
		/// <param name="offset">Offset of the data to retrieve</param>
		/// <param name="length">Length of the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the data. Must be disposed by the caller.</returns>
		Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default);

		/// <summary>
		/// Writes a stream to the storage backend. If the stream throws an exception during read, the write will be aborted.
		/// </summary>
		/// <param name="stream">Data stream</param>
		/// <param name="imports">List of referenced blobs</param>
		/// <param name="prefix">Path prefix for the uploaded data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to the uploaded object</returns>
		Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Writes a stream to the storage backend. If the stream throws an exception during read, the write will be aborted.
		/// </summary>s
		/// <param name="locator">Locator for the new blob</param>
		/// <param name="stream">Data stream</param>
		/// <param name="imports">Imported blobs. If omitted, the backend will parse them from the stream data.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to the uploaded object</returns>
		Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a HTTP redirect for a read request
		/// </summary>
		/// <param name="locator">Path to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path to upload the data to</returns>
		ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a HTTP redirect for a write request
		/// </summary>
		/// <param name="locator">Path to write to</param>
		/// <param name="imports">Imports for this blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path for retrieval, and URI to upload the data to</returns>
		ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a HTTP redirect for a write request
		/// </summary>
		/// <param name="imports">Imports for this blob</param>
		/// <param name="prefix">Prefix for the uploaded data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Path for retrieval, and URI to upload the data to</returns>
		ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default);

		#endregion

		#region Aliases

		/// <summary>
		/// Finds blobs with the given alias. Unlike refs, aliases do not serve as GC roots.
		/// </summary>
		/// <param name="name">Alias for the blob</param>
		/// <param name="maxResults">Maximum number of aliases to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Blobs matching the given handle</returns>
		Task<BlobAliasLocator[]> FindAliasesAsync(string name, int? maxResults = null, CancellationToken cancellationToken = default);

		#endregion

		#region Refs

		/// <summary>
		/// Reads data for a ref from the store
		/// </summary>
		/// <param name="name">The ref name</param>
		/// <param name="cacheTime">Minimum coherency for any cached value to be returned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Blob pointed to by the ref</returns>
		Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default);

		#endregion

		/// <summary>
		/// Batch request to update metadata
		/// </summary>
		/// <param name="request">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets stats for this storage backend
		/// </summary>
		void GetStats(StorageStats stats);
	}

	/// <summary>
	/// Utility methods for storage backend implementations
	/// </summary>
	public static class StorageHelpers
	{
		/// <summary>
		/// Unique session id used for unique ids
		/// </summary>
		static readonly string s_sessionPrefix = $"{Guid.NewGuid():n}_";

		/// <summary>
		/// Incremented value used for each supplied id
		/// </summary>
		static int s_increment;

		/// <summary>
		/// Creates a unique name with a given prefix
		/// </summary>
		/// <param name="prefix">The prefix to use</param>
		/// <returns>Unique name generated with the given prefix</returns>
		public static BlobLocator CreateUniqueLocator(string? prefix)
		{
			StringBuilder builder = new(prefix);
			if (builder.Length > 0 && builder[^1] != '/')
			{
				builder.Append('/');
			}
			builder.Append(s_sessionPrefix);
			builder.Append(Interlocked.Increment(ref s_increment));
			return new BlobLocator(builder.ToString());
		}
	}

	/// <summary>
	/// Extension methods for <see cref="IStorageBackend"/>
	/// </summary>
	public static class StorageBackendExtensions
	{
		/// <summary>
		/// Attempts to open a read stream for the given path.
		/// </summary>
		/// <param name="storageBackend">Backend to read from</param>
		/// <param name="locator">Object name within the store</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Stream for the object</returns>
		public static Task<Stream> OpenBlobAsync(this IStorageBackend storageBackend, BlobLocator locator, CancellationToken cancellationToken = default) => storageBackend.OpenBlobAsync(locator, 0, null, cancellationToken);

		/// <summary>
		/// Attempts to open a read stream for the given path.
		/// </summary>
		/// <param name="storageBackend">Backend to read from</param>
		/// <param name="locator">Object name within the store</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Stream for the object</returns>
		public static Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(this IStorageBackend storageBackend, BlobLocator locator, CancellationToken cancellationToken = default) => storageBackend.ReadBlobAsync(locator, 0, null, cancellationToken);

		/// <summary>
		/// Reads an object as an array of bytes
		/// </summary>
		/// <param name="storageBackend">Backend to read from</param>
		/// <param name="locator">Object name within the store</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Contents of the object</returns>
		public static async Task<byte[]> ReadBytesAsync(this IStorageBackend storageBackend, BlobLocator locator, CancellationToken cancellationToken = default)
		{
			using IReadOnlyMemoryOwner<byte> storageObject = await storageBackend.ReadBlobAsync(locator, cancellationToken);
			return storageObject.Memory.ToArray();
		}

		/// <summary>
		/// Writes a block of memory to storage
		/// </summary>
		/// <param name="storageBackend">Backend to read from</param>
		/// <param name="data">Data to be written</param>
		/// <param name="imports"></param>
		/// <param name="prefix">Prefix for the uploaded data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<BlobLocator> WriteBytesAsync(this IStorageBackend storageBackend, ReadOnlyMemory<byte> data, IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default)
		{
			using (ReadOnlyMemoryStream stream = new(data))
			{
				return await storageBackend.WriteBlobAsync(stream, imports, prefix, cancellationToken);
			}
		}
	}
}
