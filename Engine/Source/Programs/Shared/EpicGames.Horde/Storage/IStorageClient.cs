// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Client for the storage system
	/// </summary>
	public interface IStorageClient
	{
		/// <summary>
		/// Creates a storage namespace for the given id
		/// </summary>
		/// <param name="namespaceId">Namespace to manipulate</param>
		/// <returns>Storage namespace instance. May be null if the namespace does not exist.</returns>
		IStorageNamespace? TryGetNamespace(NamespaceId namespaceId);
	}

	/// <summary>
	/// Extension methods for <see cref="IStorageClient"/>
	/// </summary>
	public static class StorageClientExtensions
	{
		/// <summary>
		/// Creates a new storage namespace, throwing an exception if it does not exist
		/// </summary>
		public static IStorageNamespace GetNamespace(this IStorageClient storageClient, NamespaceId namespaceId)
		{
			IStorageNamespace? storageNamespace = storageClient.TryGetNamespace(namespaceId);
			if (storageNamespace == null)
			{
				throw new InvalidOperationException($"No namespace '{namespaceId}' is configured");
			}
			return storageNamespace;
		}
	}
}
