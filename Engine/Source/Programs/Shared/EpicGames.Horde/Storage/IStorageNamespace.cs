// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Options for a new ref
	/// </summary>
	/// <param name="Lifetime">Time until a ref is expired</param>
	/// <param name="Extend">Whether to extend the remaining lifetime of a ref whenever it is fetched. Defaults to true.</param>
	public record class RefOptions(TimeSpan? Lifetime = null, bool? Extend = null);

	/// <summary>
	/// Interface for the storage system.
	/// </summary>
	public interface IStorageNamespace
	{
		/// <summary>
		/// Creates a writer for updating the namespace
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for async writing. The writer will flush on dispose unless this cancellation token is signalled.</param>
		/// <returns>New writer instance</returns>
		IStorageWriter CreateWriter(CancellationToken cancellationToken = default);

		#region Blobs

		/// <summary>
		/// Creates a new blob handle by parsing a locator
		/// </summary>
		/// <param name="locator">Path to the blob</param>
		/// <returns>New handle to the blob</returns>
		IBlobRef CreateBlobRef(BlobLocator locator);

		/// <summary>
		/// Creates a new writer for storage blobs
		/// </summary>
		/// <param name="basePath">Base path for any nodes written from the writer.</param>
		/// <param name="serializerOptions">Options for serializing classes</param>
		/// <param name="cancellationToken">Cancellation token used for any buffered blob writes. Blob writers will flush on close, unless this cancellation token is signalled.</param>
		/// <returns>New writer instance. Must be disposed after use.</returns>
		IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default);

		#endregion

		#region Aliases

		/// <summary>
		/// Finds blobs with the given alias. Unlike refs, aliases do not serve as GC roots.
		/// </summary>
		/// <param name="name">Alias for the blob</param>
		/// <param name="maxResults">Maximum number of aliases to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Blobs matching the given handle</returns>
		Task<BlobAlias[]> FindAliasesAsync(string name, int? maxResults = null, CancellationToken cancellationToken = default);

		#endregion

		#region Refs

		/// <summary>
		/// Reads data for a ref from the store
		/// </summary>
		/// <param name="name">The ref name</param>
		/// <param name="cacheTime">Minimum coherency for any cached value to be returned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Blob pointed to by the ref</returns>
		Task<IHashedBlobRef?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default);

		#endregion

		/// <summary>
		/// Gets a snapshot of the stats for the storage namespace.
		/// </summary>
		void GetStats(StorageStats stats);
	}

	/// <summary>
	/// Indicates the maximum age of a entry returned from a cache in the hierarchy
	/// </summary>
	/// <param name="Utc">Oldest allowed timestamp for a returned result</param>
	public readonly record struct RefCacheTime(DateTime Utc)
	{
		/// <summary>
		/// Maximum age for a cached value to be returned
		/// </summary>
		public readonly TimeSpan MaxAge => DateTime.UtcNow - Utc;

		/// <summary>
		/// Sets the earliest time at which the entry must have been valid
		/// </summary>
		/// <param name="age">Maximum age of any returned cache value. Taken from the moment that this object was created.</param>
		public RefCacheTime(TimeSpan age) : this(DateTime.UtcNow - age) { }

		/// <summary>
		/// Tests whether this value is set
		/// </summary>
		public readonly bool IsSet() => Utc != default;

		/// <summary>
		/// Determines if this cache time deems a particular cache entry stale
		/// </summary>
		/// <param name="entryTime">Time at which the cache entry was valid</param>
		/// <param name="cacheTime">Maximum cache time to test against</param>
		public static bool IsStaleCacheEntry(DateTime entryTime, RefCacheTime cacheTime) => cacheTime.IsSet() && cacheTime.Utc < entryTime;

		/// <summary>
		/// Implicit conversion operator from datetime values.
		/// </summary>
		public static implicit operator RefCacheTime(DateTime time) => new(time);

		/// <summary>
		/// Implicit conversion operator from timespan values.
		/// </summary>
		public static implicit operator RefCacheTime(TimeSpan age) => new(age);
	}

	/// <summary>
	/// Stats for the storage system
	/// </summary>
	public class StorageStats
	{
		/// <summary>
		/// Stat name to value
		/// </summary>
		public List<(string, long)> Values { get; } = [];

		/// <summary>
		/// Add a new stat to the list
		/// </summary>
		public void Add(string name, long value) => Values.Add((name, value));

		/// <summary>
		/// Prints the table of stats to the logger
		/// </summary>
		public void Print(ILogger logger)
		{
			foreach ((string key, long value) in Values)
			{
				logger.LogInformation("{Key}: {Value:n0}", key, value);
			}
		}

		/// <summary>
		/// Subtract a base set of stats from this one
		/// </summary>
		public static StorageStats GetDelta(StorageStats initial, StorageStats finish)
		{
			StorageStats result = new();

			Dictionary<string, long> initialValues = initial.Values.ToDictionary(x => x.Item1, x => x.Item2, StringComparer.Ordinal);
			foreach ((string name, long value) in finish.Values.ToArray())
			{
				initialValues.TryGetValue(name, out long otherValue);
				result.Add(name, value - otherValue);
			}

			return result;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="IStorageNamespace"/>
	/// </summary>
	public static class StorageNamespaceExtensions
	{
		#region Blobs

		/// <summary>
		/// Creates a new blob handle by parsing a locator
		/// </summary>
		/// <param name="storageNamespace">Storage namespace to operate on</param>
		/// <param name="locator">Path to the blob</param>
		/// <param name="serializerOptions">Options for deserializing the blob</param>
		/// <returns>New handle to the blob</returns>
		public static IBlobRef<T> CreateBlobRef<T>(this IStorageNamespace storageNamespace, BlobLocator locator, BlobSerializerOptions? serializerOptions = null)
			=> storageNamespace.CreateBlobRef(locator).ForType<T>(serializerOptions);

		/// <summary>
		/// Creates a new blob reference from a locator and hash
		/// </summary>
		/// <param name="storageNamespace">Storage namespace to operate on</param>
		/// <param name="hash">Hash of the target blob</param>
		/// <param name="locator">Path to the blob</param>
		/// <returns>New handle to the blob</returns>
		public static IHashedBlobRef CreateBlobRef(this IStorageNamespace storageNamespace, IoHash hash, BlobLocator locator)
			=> HashedBlobRef.Create(hash, storageNamespace.CreateBlobRef(locator));

		/// <summary>
		/// Creates a new blob reference from a locator and hash
		/// </summary>
		/// <param name="storageNamespace">Storage namespace to operate on</param>
		/// <param name="hash">Hash of the target blob</param>
		/// <param name="locator">Path to the blob</param>
		/// <param name="serializerOptions">Options for deserializing the blob</param>
		/// <returns>New handle to the blob</returns>
		public static IHashedBlobRef<T> CreateBlobRef<T>(this IStorageNamespace storageNamespace, IoHash hash, BlobLocator locator, BlobSerializerOptions? serializerOptions = null)
			=> HashedBlobRef.Create<T>(hash, storageNamespace.CreateBlobRef(locator), serializerOptions);

		/// <summary>
		/// Create a blob ref from a RefValue
		/// </summary>
		public static IHashedBlobRef CreateBlobRef(this IStorageNamespace storageNamespace, HashedBlobRefValue refValue)
			=> storageNamespace.CreateBlobRef(refValue.Hash, refValue.Locator);

		/// <summary>
		/// Create a typed blob ref from a RefValue
		/// </summary>
		public static IHashedBlobRef<T> CreateBlobRef<T>(this IStorageNamespace storageNamespace, HashedBlobRefValue refValue, BlobSerializerOptions? options)
			=> storageNamespace.CreateBlobRef<T>(refValue.Hash, refValue.Locator, options);

		class NamedBlobRef : IBlobRef
		{
			readonly IStorageNamespace _storageNamespace;
			readonly RefName _name;
			readonly RefCacheTime _cacheTime;

			public NamedBlobRef(IStorageNamespace storageNamespace, RefName name, RefCacheTime cacheTime = default)
			{
				_storageNamespace = storageNamespace;
				_name = name;
				_cacheTime = cacheTime;
			}

			public IBlobRef Innermost => this;

			public ValueTask FlushAsync(CancellationToken cancellationToken = default)
				=> default;

			public async ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
			{
				IHashedBlobRef handle = await _storageNamespace.ReadRefAsync(_name, _cacheTime, cancellationToken);
				return await handle.ReadBlobDataAsync(cancellationToken);
			}

			public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
			{
				locator = default;
				return false;
			}
		}

		/// <summary>
		/// Creates a new blob ref from a ref name
		/// </summary>
		/// <param name="storageNamespace">The store instance to read from</param>
		/// <param name="name">Name of the reference</param>
		/// <param name="cacheTime">Maximum age for cached responses</param>
		/// <returns>New handle to the blob</returns>
		public static IBlobRef CreateBlobRef(this IStorageNamespace storageNamespace, RefName name, RefCacheTime cacheTime = default)
			=> new NamedBlobRef(storageNamespace, name, cacheTime);

		/// <summary>
		/// Creates a new blob ref from a ref name
		/// </summary>
		/// <param name="storageNamespace">The store instance to read from</param>
		/// <param name="name">Name of the reference</param>
		/// <param name="cacheTime">Maximum age for cached responses</param>
		/// <param name="serializerOptions">Options for deserializing the blob</param>
		/// <returns>New handle to the blob</returns>
		public static IBlobRef<T> CreateBlobRef<T>(this IStorageNamespace storageNamespace, RefName name, RefCacheTime cacheTime = default, BlobSerializerOptions? serializerOptions = null)
			=> CreateBlobRef(storageNamespace, name, cacheTime).ForType<T>(serializerOptions);

		/// <summary>
		/// Creates a writer using a refname as a base path
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="refName">Ref name to use as a base path</param>
		public static IBlobWriter CreateBlobWriter(this IStorageNamespace store, RefName refName)
			=> store.CreateBlobWriter(refName.ToString());

		#endregion

		#region Aliases

		/// <summary>
		/// Adds an alias to a given blob
		/// </summary>
		/// <param name="store">The store instance to write to</param>
		/// <param name="name">Alias for the blob</param>
		/// <param name="handle">Locator for the blob</param>
		/// <param name="rank">Rank for this alias. In situations where an alias has multiple mappings, the alias with the highest rank will be returned by default.</param>
		/// <param name="data">Additional data to be stored inline with the alias</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task AddAliasAsync(this IStorageNamespace store, string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
		{
			await using IStorageWriter writer = store.CreateWriter(cancellationToken);
			await writer.AddAliasAsync(name, handle, rank, data, cancellationToken);
		}

		/// <summary>
		/// Removes an alias from a blob
		/// </summary>
		/// <param name="store">The store instance to write to</param>
		/// <param name="name">Name of the alias</param>
		/// <param name="handle">Locator for the blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task RemoveAliasAsync(this IStorageNamespace store, string name, IBlobRef handle, CancellationToken cancellationToken = default)
		{
			await using IStorageWriter writer = store.CreateWriter(cancellationToken);
			await writer.RemoveAliasAsync(name, handle, cancellationToken);
		}

		/// <summary>
		/// Finds blobs with the given alias. Unlike refs, aliases do not serve as GC roots.
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="name">Alias for the blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Blobs matching the given handle</returns>
		public static async Task<BlobAlias?> FindAliasAsync(this IStorageNamespace store, string name, CancellationToken cancellationToken = default)
		{
			BlobAlias[] aliases = await store.FindAliasesAsync(name, 1, cancellationToken);
			return aliases.FirstOrDefault();
		}

		#endregion

		#region Refs

		/// <summary>
		/// Checks if the given ref exists
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="name">Name of the reference to look for</param>
		/// <param name="cacheTime">Minimum coherency for any cached value to be returned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the ref exists, false if it did not exist</returns>
		public static async Task<bool> RefExistsAsync(this IStorageNamespace store, RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			IBlobRef? target = await store.TryReadRefAsync(name, cacheTime, cancellationToken);
			return target != null;
		}

		/// <summary>
		/// Reads a ref from the store, throwing an exception if it does not exist
		/// </summary>
		/// <param name="store">The store instance to read from</param>
		/// <param name="name">Id for the ref</param>
		/// <param name="cacheTime">Minimum coherency of any cached result</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The ref target</returns>
		public static async Task<IHashedBlobRef> ReadRefAsync(this IStorageNamespace store, RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			IHashedBlobRef? refTarget = await store.TryReadRefAsync(name, cacheTime, cancellationToken);
			return refTarget ?? throw new RefNameNotFoundException(name);
		}

		/// <summary>
		/// Writes a new ref to the store
		/// </summary>
		/// <param name="store">The store instance to write to</param>
		/// <param name="name">Ref to write</param>
		/// <param name="target">Handle to the target blob</param>
		/// <param name="options">Options for the new ref</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Unique identifier for the blob</returns>
		public static async Task AddRefAsync(this IStorageNamespace store, RefName name, IHashedBlobRef target, RefOptions? options = null, CancellationToken cancellationToken = default)
		{
			await using IStorageWriter writer = store.CreateWriter(cancellationToken);
			await writer.AddRefAsync(name, target, options, cancellationToken);
		}

		/// <summary>
		/// Reads data for a ref from the store
		/// </summary>
		/// <param name="store">The store instance to write to</param>
		/// <param name="name">The ref identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task RemoveRefAsync(this IStorageNamespace store, RefName name, CancellationToken cancellationToken = default)
		{
			await using IStorageWriter writer = store.CreateWriter(cancellationToken);
			await writer.RemoveRefAsync(name, cancellationToken);
		}

		/// <inheritdoc cref="AddRefAsync(IStorageNamespace, RefName, IHashedBlobRef, RefOptions?, CancellationToken)"/>
		[Obsolete("Use AddRefAsync() instead")]
		public static Task WriteRefAsync(this IStorageNamespace store, RefName name, IHashedBlobRef target, RefOptions? options = null, CancellationToken cancellationToken = default)
			=> AddRefAsync(store, name, target, options, cancellationToken);

		/// <inheritdoc cref="RemoveRefAsync(IStorageNamespace, RefName, CancellationToken)"/>
		[Obsolete("Use RemoveRefAsync() instead")]
		public static Task DeleteRefAsync(this IStorageNamespace store, RefName name, CancellationToken cancellationToken = default)
			=> RemoveRefAsync(store, name, cancellationToken);

		#endregion

		/// <summary>
		/// Gets a snapshot of the stats for the storage namespace.
		/// </summary>
		public static StorageStats GetStats(this IStorageNamespace store)
		{
			StorageStats stats = new();
			store.GetStats(stats);
			return stats;
		}
	}
}
