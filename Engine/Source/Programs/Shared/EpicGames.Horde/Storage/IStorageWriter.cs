// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Interface for a batching storage writer
	/// </summary>
	public interface IStorageWriter : IAsyncDisposable
	{
		#region Blobs

		/// <summary>
		/// Creates a new writer for storage blobs
		/// </summary>
		/// <param name="basePath">Base path for any nodes written from the writer.</param>
		/// <param name="serializerOptions">Options for serializing classes</param>
		/// <returns>New writer instance. Must be disposed after use.</returns>
		IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null);

		#endregion

		#region Aliases

		/// <summary>
		/// Adds an alias to a given blob
		/// </summary>
		/// <param name="name">Alias for the blob</param>
		/// <param name="handle">Locator for the blob</param>
		/// <param name="rank">Rank for this alias. In situations where an alias has multiple mappings, the alias with the highest rank will be returned by default.</param>
		/// <param name="data">Additional data to be stored inline with the alias</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task AddAliasAsync(string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default);

		/// <summary>
		/// Removes an alias from a blob
		/// </summary>
		/// <param name="name">Name of the alias</param>
		/// <param name="handle">Locator for the blob</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task RemoveAliasAsync(string name, IBlobRef handle, CancellationToken cancellationToken = default);

		#endregion

		#region Refs

		/// <summary>
		/// Adds a new ref to the store
		/// </summary>
		/// <param name="name">Ref to write</param>
		/// <param name="target">Handle to the target blob</param>
		/// <param name="options">Options for the new ref</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Unique identifier for the blob</returns>
		Task AddRefAsync(RefName name, IHashedBlobRef target, RefOptions? options = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Removes a ref from the store
		/// </summary>
		/// <param name="name">The ref identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task RemoveRefAsync(RefName name, CancellationToken cancellationToken = default);

		#endregion

		/// <summary>
		/// Flush any buffered writes
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task FlushAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Default base implementation of <see cref="IStorageWriter"/> which combines metadata updates into a <see cref="UpdateMetadataRequest"/>
	/// </summary>
	public abstract class StorageWriter : IStorageWriter
	{
		readonly IStorageBackend _storageBackend;
		readonly CancellationToken _cancellationToken;
		UpdateMetadataRequest _request = new();

		/// <summary>
		/// Constructor
		/// </summary>
		protected StorageWriter(IStorageBackend storageBackend, CancellationToken cancellationToken = default)
		{
			_storageBackend = storageBackend;
			_cancellationToken = cancellationToken;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await FlushAsync(_cancellationToken);
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null)
			=> CreateBlobWriter(basePath, serializerOptions, _cancellationToken);

		/// <inheritdoc cref="IStorageWriter.CreateBlobWriter(String?, BlobSerializerOptions?)"/>
		protected abstract IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default);

		#region Aliases

		/// <inheritdoc/>
		public async Task AddAliasAsync(string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
		{
			await handle.FlushAsync(cancellationToken);
			_request.AddAliases.Add(new AddAliasRequest { Name = name, Rank = rank, Data = data.ToArray(), Target = handle.GetLocator() });
			await CheckForFlushAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task RemoveAliasAsync(string name, IBlobRef handle, CancellationToken cancellationToken = default)
		{
			await handle.FlushAsync(cancellationToken);
			_request.RemoveAliases.Add(new RemoveAliasRequest { Name = name, Target = handle.GetLocator() });
			await CheckForFlushAsync(cancellationToken);
		}

		#endregion

		#region Refs

		/// <inheritdoc/>
		public async Task AddRefAsync(RefName name, IHashedBlobRef target, RefOptions? options = null, CancellationToken cancellationToken = default)
		{
			await target.FlushAsync(cancellationToken);
			options = (options != null) ? new RefOptions(options.Lifetime, options.Extend) : null;
			_request.AddRefs.Add(new AddRefRequest { RefName = name, Hash = target.Hash, Target = target.GetLocator(), Options = options });
			await CheckForFlushAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task RemoveRefAsync(RefName name, CancellationToken cancellationToken = default)
		{
			_request.RemoveRefs.Add(new RemoveRefRequest { RefName = name });
			await CheckForFlushAsync(cancellationToken);
		}

		#endregion

		async Task CheckForFlushAsync(CancellationToken cancellationToken = default)
		{
			int numItems = _request.AddAliases.Count + _request.RemoveAliases.Count + _request.AddRefs.Count + _request.RemoveRefs.Count;
			if (numItems >= 100)
			{
				await FlushAsync(cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task FlushAsync(CancellationToken cancellationToken = default)
		{
			if (_request.AddAliases.Count > 0 || _request.RemoveAliases.Count > 0 || _request.AddRefs.Count > 0 || _request.RemoveRefs.Count > 0)
			{
				await _storageBackend.UpdateMetadataAsync(_request, cancellationToken);
				_request = new UpdateMetadataRequest();
			}
		}
	}
}
