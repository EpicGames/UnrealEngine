// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Backends;

namespace EpicGames.Horde.Storage.Clients
{
	/// <summary>
	/// Base class for storage namespaces that wrap a diirect key/value type store without any merging/splitting.
	/// </summary>
	public sealed class KeyValueStorageNamespace : IStorageNamespace
	{
		class Handle : IBlobRef
		{
			readonly KeyValueStorageNamespace _keyValueStorageNamespace;
			readonly BlobLocator _locator;

			/// <inheritdoc/>
			public IBlobRef Innermost => this;

			/// <summary>
			/// Constructor
			/// </summary>
			public Handle(KeyValueStorageNamespace keyValueStorageNamespace, BlobLocator locator)
			{
				_keyValueStorageNamespace = keyValueStorageNamespace;
				_locator = locator;
			}

			/// <inheritdoc/>
			public ValueTask FlushAsync(CancellationToken cancellationToken = default) => default;

			/// <inheritdoc/>
			public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default) => _keyValueStorageNamespace.ReadBlobAsync(_locator, cancellationToken);

			/// <inheritdoc/>
			public bool TryGetLocator(out BlobLocator locator)
			{
				locator = _locator;
				return true;
			}

			/// <inheritdoc/>
			public override bool Equals(object? obj) => obj is Handle other && _locator == other._locator;

			/// <inheritdoc/>
			public override int GetHashCode() => _locator.GetHashCode();
		}

		class StorageWriterImpl : StorageWriter
		{
			readonly KeyValueStorageNamespace _keyValueStorageNamespace;

			public StorageWriterImpl(KeyValueStorageNamespace keyValueStorageNamespace, IStorageBackend storageBackend, CancellationToken cancellationToken)
				: base(storageBackend, cancellationToken)
			{
				_keyValueStorageNamespace = keyValueStorageNamespace;
			}

			/// <inheritdoc/>
			protected override IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? serializerOptions = null, CancellationToken cancellationToken = default)
				=> _keyValueStorageNamespace.CreateBlobWriter(basePath, serializerOptions, cancellationToken);
		}

		class BlobWriterImpl : BlobWriter
		{
			readonly KeyValueStorageNamespace _outer;
			readonly string _basePath;
			byte[] _data = [];
			int _offset;

			/// <summary>
			/// Constructor
			/// </summary>
			public BlobWriterImpl(KeyValueStorageNamespace outer, string? basePath, BlobSerializerOptions? options)
				: base(options)
			{
				_outer = outer;
				_basePath = basePath ?? String.Empty;

				if (!_basePath.EndsWith('/'))
				{
					_basePath += "/";
				}
			}

			/// <inheritdoc/>
			public override ValueTask DisposeAsync() => new();

			/// <inheritdoc/>
			public override Task FlushAsync(CancellationToken cancellationToken = default) => Task.CompletedTask;

			/// <inheritdoc/>
			public override IBlobWriter Fork() => new BlobWriterImpl(_outer, _basePath, Options);

			/// <inheritdoc/>
			public override Memory<byte> GetOutputBuffer(int usedSize, int desiredSize)
			{
				if (_offset + desiredSize > _data.Length)
				{
					byte[] newData = new byte[(desiredSize + 4095) & ~4095];
					_data.AsSpan(_offset, usedSize).CopyTo(newData);
					_data = newData;
					_offset = 0;
				}
				return _data.AsMemory(_offset);
			}

			/// <inheritdoc/>
			public override async ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, int size, IReadOnlyList<IBlobRef> imports, IReadOnlyList<AliasInfo> aliases, CancellationToken cancellationToken = default)
			{
				ReadOnlyMemory<byte> data = _data.AsMemory(_offset, size);
				_offset += size;

				using ReadOnlyMemoryStream stream = new(data);

				IHashedBlobRef handle = await _outer.WriteBlobAsync(type, stream, imports, _basePath, cancellationToken);
				foreach (AliasInfo aliasInfo in aliases)
				{
					await _outer.AddAliasAsync(aliasInfo.Name, handle, aliasInfo.Rank, aliasInfo.Data, cancellationToken);
				}
				return handle;
			}
		}

		/// <inheritdoc/>
		public bool SupportsRedirects => _backend.SupportsRedirects;

		readonly IStorageBackend _backend;

		/// <summary>
		/// Constructor
		/// </summary>
		public KeyValueStorageNamespace(IStorageBackend backend)
		{
			_backend = backend;
		}

		/// <summary>
		/// Create an in-memory storage namespace
		/// </summary>
		public static KeyValueStorageNamespace CreateInMemory() => new(new MemoryStorageBackend());

		/// <inheritdoc/>
		public IStorageWriter CreateWriter(CancellationToken cancellationToken)
			=> new StorageWriterImpl(this, _backend, cancellationToken);

		#region Blobs

		/// <inheritdoc/>
		public IBlobRef CreateBlobRef(BlobLocator locator)
			=> new Handle(this, locator);

		/// <inheritdoc/>
		public IBlobWriter CreateBlobWriter(string? basePath = null, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default)
			=> new BlobWriterImpl(this, basePath, options);

		/// <inheritdoc/>
		public async ValueTask<BlobData> ReadBlobAsync(BlobLocator locator, CancellationToken cancellationToken = default)
		{
			IReadOnlyMemoryOwner<byte> owner = await _backend.ReadBlobAsync(locator, cancellationToken);
			EncodedBlobData encodedData = new(owner.Memory.ToArray());

			return new BlobDataWithOwner(encodedData.Type, encodedData.Payload, [.. encodedData.Imports.Select(x => CreateBlobRef(x))], owner);
		}

		/// <inheritdoc/>
		public async ValueTask<IHashedBlobRef> WriteBlobAsync(BlobType type, Stream stream, IReadOnlyList<IBlobRef> imports, string? basePath = null, CancellationToken cancellationToken = default)
		{
			BlobLocator[] locators = [.. imports.ConvertAll(x => x.GetLocator())];
			byte[] payload = await stream.ReadAllBytesAsync(cancellationToken);
			byte[] encodedData = EncodedBlobData.Create(type, locators, payload);

			using ReadOnlyMemoryStream encodedStream = new(encodedData);
			BlobLocator locator = await _backend.WriteBlobAsync(encodedStream, locators, basePath, cancellationToken);

			IoHash hash = IoHash.Compute(payload);
			return HashedBlobRef.Create(hash, CreateBlobRef(locator));
		}

		#endregion

		#region Aliases

		/// <inheritdoc/>
		public async Task<BlobAlias[]> FindAliasesAsync(string name, int? maxResults = null, CancellationToken cancellationToken = default)
		{
			BlobAliasLocator[] locators = await _backend.FindAliasesAsync(name, maxResults, cancellationToken);
			return [.. locators.Select(x => new BlobAlias(CreateBlobRef(x.Target), x.Rank, x.Data))];
		}

		#endregion

		#region Refs

		/// <inheritdoc/>
		public async Task<IHashedBlobRef?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
		{
			HashedBlobRefValue? value = await _backend.TryReadRefAsync(name, cacheTime, cancellationToken);
			if (value == null)
			{
				return null;
			}
			return this.CreateBlobRef(value.Hash, value.Locator);
		}

		#endregion

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
			=> _backend.GetStats(stats);
	}
}
