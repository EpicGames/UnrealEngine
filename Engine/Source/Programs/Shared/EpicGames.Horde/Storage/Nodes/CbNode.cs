// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// A node containing arbitrary compact binary data
	/// </summary>
	[BlobConverter(typeof(CbNodeConverter))]
	public class CbNode
	{
		/// <summary>
		/// Static accessor for the blob type guid
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{34A0793F-42F4-8364-A798-32862932841C}");

		/// <summary>
		/// The compact binary object
		/// </summary>
		public CbObject Object { get; set; }

		/// <summary>
		/// Imported nodes
		/// </summary>
		public IReadOnlyList<IBlobRef> Imports { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="obj">The compact binary object</param>
		/// <param name="imports">List of imports for attachments</param>
		public CbNode(CbObject obj, IReadOnlyList<IBlobRef> imports)
		{
			Object = obj;
			Imports = imports;
		}
	}

	class CbNodeConverter : BlobConverter<CbNode>
	{
		public static BlobType BlobType { get; } = new BlobType(CbNode.BlobTypeGuid, 1);

		/// <inheritdoc/>
		public override CbNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			CbObject obj = new(reader.GetMemory().ToArray());
			return new CbNode(obj, [.. reader.Imports]);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, CbNode value, BlobSerializerOptions options)
		{
			writer.WriteFixedLengthBytes(value.Object.GetView().Span);
			return BlobType;
		}
	}
}
