// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.IO.Pipelines;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Representation of a data stream, split into chunks along content-aware boundaries using a rolling hash (<see cref="BuzHash"/>).
	/// Chunks are pushed into a tree hierarchy as data is appended to the root, with nodes of the tree also split along content-aware boundaries with <see cref="IoHash.NumBytes"/> granularity.
	/// Once a chunk has been written to storage, it is treated as immutable.
	/// </summary>
	[BlobConverter(typeof(ChunkedDataNodeConverter))]
	public abstract class ChunkedDataNode
	{
		/// <summary>
		/// Copies the contents of this node and its children to the given output stream
		/// </summary>
		/// <param name="outputStream">The output stream to receive the data</param>
		/// <param name="options">Options for controling serialization</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public abstract Task CopyToStreamAsync(Stream outputStream, BlobSerializerOptions? options = null, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for chunked data node types
	/// </summary>
	public static class ChunkedDataNodeExtensions
	{
		/// <summary>
		/// Copy the contents of the node to the output stream without creating the intermediate FileNodes
		/// </summary>
		/// <param name="handle">Handle to the data to read</param>
		/// <param name="length">Length of the stream</param>
		public static Stream OpenAsStream(this IBlobRef<ChunkedDataNode> handle, long? length = null)
			=> new ChunkedDataStream(handle, length);

		/// <summary>
		/// Copy the contents of the node to the output stream without creating the intermediate FileNodes
		/// </summary>
		/// <param name="handle">Handle to the data to read</param>
		/// <param name="outputStream">The output stream to receive the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task CopyToStreamAsync(this IBlobRef<ChunkedDataNode> handle, Stream outputStream, CancellationToken cancellationToken = default)
		{
			using BlobData blobData = await handle.ReadBlobDataAsync(cancellationToken);
			if (blobData.Type.Guid == LeafChunkedDataNode.BlobTypeGuid)
			{
				await LeafChunkedDataNode.CopyToStreamAsync(blobData, outputStream, cancellationToken);
			}
			else if (blobData.Type.Guid == InteriorChunkedDataNode.BlobTypeGuid)
			{
				await InteriorChunkedDataNode.CopyToStreamAsync(blobData, outputStream, cancellationToken);
			}
			else
			{
				throw new ArgumentException($"Unexpected {nameof(ChunkedDataNode)} type found.");
			}
		}

		/// <summary>
		/// Extracts the contents of this node to a file
		/// </summary>
		/// <param name="handle">Handle to the data to read</param>
		/// <param name="file">File to write with the contents of this node</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task CopyToFileAsync(this IBlobRef<ChunkedDataNode> handle, FileInfo file, CancellationToken cancellationToken)
		{
			if (file.Exists && (file.Attributes & FileAttributes.ReadOnly) != 0)
			{
				file.Attributes &= ~FileAttributes.ReadOnly;
			}
			using (FileStream stream = file.Open(FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				await CopyToStreamAsync(handle, stream, cancellationToken);
			}
		}

		/// <summary>
		/// Copy the contents of the node to the output stream without creating the intermediate FileNodes
		/// </summary>
		/// <param name="handle">Handle to the data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<byte[]> ReadAllBytesAsync(this IBlobRef<ChunkedDataNode> handle, CancellationToken cancellationToken = default)
		{
			using Stream stream = OpenAsStream(handle);
			return await stream.ReadAllBytesAsync(cancellationToken);
		}
	}

	class ChunkedDataNodeConverter : BlobConverter<ChunkedDataNode>
	{
		/// <inheritdoc/>
		public override ChunkedDataNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			if (reader.Type.Guid == LeafChunkedDataNode.BlobTypeGuid)
			{
				return options.GetConverter<LeafChunkedDataNode>().Read(reader, options);
			}
			else if (reader.Type.Guid == InteriorChunkedDataNode.BlobTypeGuid)
			{
				return options.GetConverter<InteriorChunkedDataNode>().Read(reader, options);
			}
			else
			{
				throw new NotSupportedException();
			}
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, ChunkedDataNode value, BlobSerializerOptions options)
		{
			if (value is LeafChunkedDataNode leafNode)
			{
				return options.GetConverter<LeafChunkedDataNode>().Write(writer, leafNode, options);
			}
			else if (value is InteriorChunkedDataNode interiorNode)
			{
				return options.GetConverter<InteriorChunkedDataNode>().Write(writer, interiorNode, options);
			}
			else
			{
				throw new NotSupportedException();
			}
		}
	}

	/// <summary>
	/// Type of a chunked data node
	/// </summary>
	public enum ChunkedDataNodeType
	{
		/// <summary>
		/// Unknown node type
		/// </summary>
		Unknown,

		/// <summary>
		/// Leaf node
		/// </summary>
		Leaf,

		/// <summary>
		/// An interior node
		/// </summary>
		Interior
	}

	/// <summary>
	/// Reference to a chunked data node
	/// </summary>
	/// <param name="Type">Type of the referenced node</param>
	/// <param name="Length">Length of the data stream within this node</param>
	/// <param name="RollingHash">Rolling hash for this chunk. Only serialized for leaf node references.</param>
	/// <param name="Inner">Handle to the target node</param>
	public record class ChunkedDataNodeRef(ChunkedDataNodeType Type, long Length, uint RollingHash, IHashedBlobRef<ChunkedDataNode> Inner) : IHashedBlobRef<ChunkedDataNode>
	{
		/// <inheritdoc/>
		public IoHash Hash => Inner.Hash;

		/// <inheritdoc/>
		public BlobSerializerOptions? SerializerOptions => Inner.SerializerOptions;

		/// <inheritdoc/>
		public IBlobRef Innermost => Inner.Innermost;

		/// <summary>
		/// Leaf node constructor
		/// </summary>
		public ChunkedDataNodeRef(long length, uint rollingHash, IHashedBlobRef<LeafChunkedDataNode> handle)
			: this(ChunkedDataNodeType.Leaf, length, rollingHash, handle)
		{
		}

		/// <summary>
		/// Interior node constructor
		/// </summary>
		public ChunkedDataNodeRef(long length, IHashedBlobRef<InteriorChunkedDataNode> handle)
			: this(ChunkedDataNodeType.Interior, length, 0, handle)
		{
		}

		/// <summary>
		/// Gets the target interior node handle
		/// </summary>
		public IHashedBlobRef<InteriorChunkedDataNode> GetInteriorHandle()
		{
			if (Type != ChunkedDataNodeType.Interior)
			{
				throw new InvalidOperationException("Node is not an interior node");
			}
			return HashedBlobRef.Create<InteriorChunkedDataNode>(Inner.Hash, Inner, Inner.SerializerOptions);
		}

		/// <summary>
		/// Gets the target leaf node handle
		/// </summary>
		public IHashedBlobRef<LeafChunkedDataNode> GetLeafHandle()
		{
			if (Type != ChunkedDataNodeType.Leaf)
			{
				throw new InvalidOperationException("Node is not a leaf node");
			}
			return HashedBlobRef.Create<LeafChunkedDataNode>(Inner.Hash, Inner, Inner.SerializerOptions);
		}

		/// <summary>
		/// Read the node which is the target of this ref
		/// </summary>
		public async ValueTask<ChunkedDataNode> ReadBlobAsync(BlobSerializerOptions? options = null, CancellationToken cancellationToken = default)
		{
			if (Type == ChunkedDataNodeType.Leaf)
			{
				return await Inner.ReadBlobAsync<LeafChunkedDataNode>(options, cancellationToken);
			}
			else if (Type == ChunkedDataNodeType.Interior)
			{
				return await Inner.ReadBlobAsync<InteriorChunkedDataNode>(options, cancellationToken);
			}
			else
			{
				throw new InvalidDataException();
			}
		}

		/// <summary>
		/// Enumerates all the leaf nodes for this node
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async IAsyncEnumerable<LeafChunkedDataNodeRef> EnumerateLeafNodesAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			Stack<ChunkedDataNodeRef> stack = new();
			stack.Push(this);

			while (stack.TryPop(out ChunkedDataNodeRef? dataRef))
			{
				if (dataRef.Type == ChunkedDataNodeType.Leaf)
				{
					long length = dataRef.Length;
					if (dataRef.Length < 0)
					{
						// Backwards compatibility hack for v2 format
						using BlobData data = await dataRef.ReadBlobDataAsync(cancellationToken);
						length = data.Data.Length;
					}

					yield return new LeafChunkedDataNodeRef(length, dataRef.GetLeafHandle());
				}
				else
				{
					using BlobData data = await dataRef.ReadBlobDataAsync(cancellationToken);
					if (data.Type.Guid == LeafChunkedDataNodeConverter.BlobType.Guid)
					{
						yield return new LeafChunkedDataNodeRef(dataRef.Length, dataRef.GetLeafHandle());
					}
					else
					{
						InteriorChunkedDataNode interiorNode = BlobSerializer.Deserialize<InteriorChunkedDataNode>(data);
						for (int idx = interiorNode.Children.Count - 1; idx >= 0; idx--) // Push onto stack in reverse order for correct traversal order
						{
							stack.Push(interiorNode.Children[idx]);
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken = default)
			=> Inner.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
			=> Inner.ReadBlobDataAsync(cancellationToken);

		/// <inheritdoc/>
		public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
			=> Inner.TryGetLocator(out locator);
	}

	/// <summary>
	/// Reference to a <see cref="LeafChunkedDataNode"/>
	/// </summary>
	/// <param name="Length">Length of the leaf data</param>
	/// <param name="Inner">Reference to the blob</param>
	public record class LeafChunkedDataNodeRef(long Length, IHashedBlobRef<LeafChunkedDataNode> Inner) : IHashedBlobRef<LeafChunkedDataNode>
	{
		/// <inheritdoc/>
		public BlobSerializerOptions? SerializerOptions => Inner.SerializerOptions;

		/// <inheritdoc/>
		public IBlobRef Innermost => Inner.Innermost;

		/// <inheritdoc/>
		public IoHash Hash => Inner.Hash;

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken = default)
			=> Inner.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public ValueTask<BlobData> ReadBlobDataAsync(CancellationToken cancellationToken = default)
			=> Inner.ReadBlobDataAsync(cancellationToken);

		/// <inheritdoc/>
		public bool TryGetLocator([NotNullWhen(true)] out BlobLocator locator)
			=> Inner.TryGetLocator(out locator);
	}

	/// <summary>
	/// Stores the flat list of chunks produced from chunking a single data stream
	/// </summary>
	/// <param name="Hash">Hash of the data</param>
	/// <param name="LeafHandles">Handles to the leaf chunks</param>
	public record class LeafChunkedData(IoHash Hash, List<ChunkedDataNodeRef> LeafHandles);

	/// <summary>
	/// File node that contains a chunk of data
	/// </summary>
	[BlobConverter(typeof(LeafChunkedDataNodeConverter))]
	public sealed class LeafChunkedDataNode : ChunkedDataNode
	{
		/// <summary>
		/// Guid for the blob type
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{B27AFB68-4A4B-9E20-8A78-D8A439D49840}");

		/// <summary>
		/// Data for this node
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Create an empty leaf node
		/// </summary>
		public LeafChunkedDataNode(ReadOnlyMemory<byte> data)
		{
			Data = data;
		}

		/// <inheritdoc/>
		public override async Task CopyToStreamAsync(Stream outputStream, BlobSerializerOptions? options, CancellationToken cancellationToken)
		{
			await outputStream.WriteAsync(Data, cancellationToken);
		}

		/// <summary>
		/// Copy the contents of the node to the output stream without creating the intermediate FileNodes
		/// </summary>
		/// <param name="nodeData">The raw node data</param>
		/// <param name="outputStream">The output stream to receive the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task CopyToStreamAsync(BlobData nodeData, Stream outputStream, CancellationToken cancellationToken)
		{
			// Keep this code in sync with the constructor
			await outputStream.WriteAsync(nodeData.Data, cancellationToken);
		}

		/// <summary>
		/// Creates nodes from the given file
		/// </summary>
		/// <param name="writer">Writer for output nodes</param>
		/// <param name="file">File info</param>
		/// <param name="options">Options for finding chunk boundaries</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hash of the full file data</returns>
		public static async Task<LeafChunkedData> CreateFromFileAsync(IBlobWriter writer, FileReference file, LeafChunkedDataNodeOptions options, CancellationToken cancellationToken)
		{
			using (FileStream stream = FileReference.Open(file, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				return await CreateFromStreamAsync(writer, stream, options, null, cancellationToken);
			}
		}

		/// <summary>
		/// Creates nodes from the given file
		/// </summary>
		/// <param name="writer">Writer for output nodes</param>
		/// <param name="file">File info</param>
		/// <param name="options">Options for finding chunk boundaries</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hash of the full file data</returns>
		public static async Task<LeafChunkedData> CreateFromFileAsync(IBlobWriter writer, FileInfo file, LeafChunkedDataNodeOptions options, CancellationToken cancellationToken)
		{
			using (FileStream stream = file.Open(FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				return await CreateFromStreamAsync(writer, stream, options, null, cancellationToken);
			}
		}

		/// <summary>
		/// Creates nodes from the given file
		/// </summary>
		/// <param name="writer">Writer for output nodes</param>
		/// <param name="stream">Stream to read from</param>
		/// <param name="options">Options for finding chunk boundaries</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hash of the full file data</returns>
		public static Task<LeafChunkedData> CreateFromStreamAsync(IBlobWriter writer, Stream stream, LeafChunkedDataNodeOptions options, CancellationToken cancellationToken)
		{
			return CreateFromStreamAsync(writer, stream, options, null, cancellationToken);
		}

		/// <summary>
		/// Creates nodes from the given file
		/// </summary>
		/// <param name="writer">Writer for output nodes</param>
		/// <param name="stream">Stream to read from</param>
		/// <param name="options">Options for finding chunk boundaries</param>
		/// <param name="updateStats">Stats for the copy operation</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hash of the full file data</returns>
		internal static async Task<LeafChunkedData> CreateFromStreamAsync(IBlobWriter writer, Stream stream, LeafChunkedDataNodeOptions options, UpdateStats? updateStats, CancellationToken cancellationToken)
		{
			using Blake3.Hasher hasher = Blake3.Hasher.New();
			using IMemoryOwner<byte> readBuffer = MemoryPool<byte>.Shared.Rent(options.MaxSize);

			List<ChunkedDataNodeRef> leafNodeRefs = [];

			int size = 0;
			int sizeSinceProgressUpdate = 0;
			for (; ; )
			{
				size += await stream.ReadGreedyAsync(readBuffer.Memory.Slice(size), cancellationToken);
				if (size == 0 && leafNodeRefs.Count > 0)
				{
					break;
				}

				uint rollingHash;
				int nextLength = GetChunkLength(readBuffer.Memory.Span.Slice(0, size), options, out rollingHash);

				ReadOnlyMemory<byte> nextBlobData = readBuffer.Memory.Slice(0, nextLength);
				writer.WriteFixedLengthBytes(nextBlobData.Span);
				hasher.Update(nextBlobData.Span);

				sizeSinceProgressUpdate += nextLength;
				if (sizeSinceProgressUpdate > 512 * 1024)
				{
					updateStats?.Update(0, sizeSinceProgressUpdate);
					sizeSinceProgressUpdate = 0;
				}

				IHashedBlobRef<LeafChunkedDataNode> blobHandle = await writer.CompleteAsync<LeafChunkedDataNode>(LeafChunkedDataNodeConverter.BlobType, cancellationToken);
				leafNodeRefs.Add(new ChunkedDataNodeRef(nextLength, rollingHash, blobHandle));

				readBuffer.Memory.Slice(nextLength, size - nextLength).CopyTo(readBuffer.Memory);
				size -= nextLength;
			}

			updateStats?.Update(1, sizeSinceProgressUpdate);

			IoHash hash = IoHash.FromBlake3(hasher);
			return new LeafChunkedData(hash, leafNodeRefs);
		}

		internal static async Task<LeafChunkedData> CreateFromChunkerOutputAsync(IBlobWriter writer, ChunkerOutput output, UpdateStats? updateStats, CancellationToken cancellationToken)
		{
			using Blake3.Hasher hasher = Blake3.Hasher.New();

			List<ChunkedDataNodeRef> leafNodeRefs = [];

			int sizeSinceProgressUpdate = 0;
			while (await output.MoveNextAsync(cancellationToken))
			{
				ReadOnlyMemory<byte> data = output.Data;
				writer.WriteFixedLengthBytes(data.Span);
				hasher.Update(data.Span);

				sizeSinceProgressUpdate += data.Length;
				if (sizeSinceProgressUpdate > 512 * 1024)
				{
					updateStats?.Update(0, sizeSinceProgressUpdate);
					sizeSinceProgressUpdate = 0;
				}

				IHashedBlobRef<LeafChunkedDataNode> blobHandle = await writer.CompleteAsync<LeafChunkedDataNode>(LeafChunkedDataNodeConverter.BlobType, cancellationToken);
				leafNodeRefs.Add(new ChunkedDataNodeRef(data.Length, output.RollingHash, blobHandle));
			}

			updateStats?.Update(1, sizeSinceProgressUpdate);

			IoHash hash = IoHash.FromBlake3(hasher);
			return new LeafChunkedData(hash, leafNodeRefs);
		}

		/// <summary>
		/// Determines how much data to append to an existing leaf node
		/// </summary>
		/// <param name="inputData">Data to be appended</param>
		/// <param name="options">Options for chunking the data</param>
		/// <param name="rollingHash">Receives the rolling hash at the end of this chunk</param>
		/// <returns>The number of bytes to append</returns>
		public static int GetChunkLength(ReadOnlySpan<byte> inputData, LeafChunkedDataNodeOptions options, out uint rollingHash)
		{
			return BuzHash.FindChunkLength(inputData, options.MinSize, options.MaxSize, options.TargetSize, out rollingHash);
		}
	}

	class LeafChunkedDataNodeConverter : BlobConverter<LeafChunkedDataNode>
	{
		/// <summary>
		/// Static accessor for the blob type
		/// </summary>
		public static BlobType BlobType { get; } = new BlobType(LeafChunkedDataNode.BlobTypeGuid, 1);

		public override LeafChunkedDataNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			byte[] data = reader.GetMemory().ToArray();
			return new LeafChunkedDataNode(data);
		}

		public override BlobType Write(IBlobWriter writer, LeafChunkedDataNode value, BlobSerializerOptions options)
		{
			writer.WriteFixedLengthBytes(value.Data.Span);
			return BlobType;
		}
	}

	/// <summary>
	/// Options for creating interior nodes
	/// </summary>
	/// <param name="MinChildCount">Minimum number of children in each node</param>
	/// <param name="TargetChildCount">Target number of children in each node</param>
	/// <param name="MaxChildCount">Maximum number of children in each node</param>
	/// <param name="SliceThreshold">Threshold hash value for splitting interior nodes</param>
	public record class InteriorChunkedDataNodeOptions(int MinChildCount, int TargetChildCount, int MaxChildCount, uint SliceThreshold)
	{
		/// <summary>
		/// Default settings
		/// </summary>
		public static InteriorChunkedDataNodeOptions Default { get; } = new InteriorChunkedDataNodeOptions(1, 100, 500);

		/// <summary>
		/// Constructor
		/// </summary>
		public InteriorChunkedDataNodeOptions(int minChildCount, int targetChildCount, int maxChildCount)
			: this(minChildCount, targetChildCount, maxChildCount, (uint)((1UL << 32) / (uint)targetChildCount))
		{
		}
	}

	/// <summary>
	/// An interior file node
	/// </summary>
	[BlobConverter(typeof(InteriorChunkedDataNodeConverter))]
	public class InteriorChunkedDataNode : ChunkedDataNode
	{
		/// <summary>
		/// Static accessor for the blob type guid
		/// </summary>
		public static Guid BlobTypeGuid { get; } = Guid.Parse("{F4DEDDBC-4C7A-70CB-11F0-4783B9CDCCAF}");

		/// <summary>
		/// Child nodes
		/// </summary>
		public IReadOnlyList<ChunkedDataNodeRef> Children { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="children"></param>
		public InteriorChunkedDataNode(IReadOnlyList<ChunkedDataNodeRef> children)
		{
			Children = children;
		}

		/// <summary>
		/// Create a tree of nodes from the given list of handles, splitting nodes in each layer based on the hash of the last node.
		/// </summary>
		/// <param name="leafChunkedData">List of leaf handles</param>
		/// <param name="options">Options for splitting the tree</param>
		/// <param name="writer">Output writer for new interior nodes</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the root node of the tree</returns>
		public static async Task<ChunkedData> CreateTreeAsync(LeafChunkedData leafChunkedData, InteriorChunkedDataNodeOptions options, IBlobWriter writer, CancellationToken cancellationToken)
		{
			ChunkedDataNodeRef rootRef = await CreateTreeAsync(leafChunkedData.LeafHandles, options, writer, cancellationToken);
			return new ChunkedData(leafChunkedData.Hash, rootRef);
		}

		/// <summary>
		/// Create a tree of nodes from the given list of handles, splitting nodes in each layer based on the hash of the last node.
		/// </summary>
		/// <param name="nodeRefs">List of leaf nodes</param>
		/// <param name="chunkingOptions">Options for splitting the tree</param>
		/// <param name="writer">Output writer for new interior nodes</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the root node of the tree</returns>
		public static async Task<ChunkedDataNodeRef> CreateTreeAsync(List<ChunkedDataNodeRef> nodeRefs, InteriorChunkedDataNodeOptions chunkingOptions, IBlobWriter writer, CancellationToken cancellationToken)
		{
			await using MemoryBlobWriter memoryWriter = new(writer.Options);

			List<ChunkedDataNodeRef> handleBuffer = [];

			List<(InteriorChunkedDataNode, long)> interiorNodes = [];
			while (nodeRefs.Count > 1)
			{
				interiorNodes.Clear();
				CreateTreeLayer(nodeRefs, chunkingOptions, interiorNodes);

				handleBuffer.Clear();
				foreach ((InteriorChunkedDataNode interiorNode, long interiorLength) in interiorNodes)
				{
					IHashedBlobRef<InteriorChunkedDataNode> interiorHandle = await memoryWriter.WriteBlobAsync(interiorNode, cancellationToken);
					handleBuffer.Add(new ChunkedDataNodeRef(interiorLength, interiorHandle));
				}

				nodeRefs = handleBuffer;
			}

			return await OrderTreeAsync(nodeRefs[0], writer, cancellationToken);
		}

		static async Task<ChunkedDataNodeRef> OrderTreeAsync(ChunkedDataNodeRef source, IBlobWriter writer, CancellationToken cancellationToken)
		{
			if (source.Type != ChunkedDataNodeType.Interior)
			{
				return source;
			}

			InteriorChunkedDataNode sourceNode = await source.ReadBlobAsync<InteriorChunkedDataNode>(cancellationToken: cancellationToken);

			List<ChunkedDataNodeRef> children = [.. sourceNode.Children];
			for (int idx = children.Count - 1; idx >= 0; idx--)
			{
				children[idx] = await OrderTreeAsync(children[idx], writer, cancellationToken);
			}

			InteriorChunkedDataNode targetNode = new(children);
			IHashedBlobRef<InteriorChunkedDataNode> targetHandle = await writer.WriteBlobAsync(targetNode, cancellationToken);

			return new ChunkedDataNodeRef(source.Length, targetHandle);
		}

		/// <summary>
		/// Split a list of leaf handles into a layer of interior nodes
		/// </summary>
		static void CreateTreeLayer(List<ChunkedDataNodeRef> nodeRefs, InteriorChunkedDataNodeOptions options, List<(InteriorChunkedDataNode, long)> interiorNodes)
		{
			Span<byte> buffer = stackalloc byte[IoHash.NumBytes];

			for (int index = 0; index < nodeRefs.Count;)
			{
				int minIndex = index;
				int maxIndex = Math.Min(minIndex + options.MaxChildCount, nodeRefs.Count);

				index = Math.Min(index + options.MinChildCount, nodeRefs.Count);
				for (; index < maxIndex; index++)
				{
					nodeRefs[index].Hash.CopyTo(buffer);

					uint value = BinaryPrimitives.ReadUInt32LittleEndian(buffer);
					if (value < options.SliceThreshold)
					{
						break;
					}
				}

				ChunkedDataNodeRef[] children = new ChunkedDataNodeRef[index - minIndex];
				for (int childIndex = minIndex; childIndex < index; childIndex++)
				{
					children[childIndex - minIndex] = nodeRefs[childIndex];
				}

				long interiorLength = children.Sum(x => x.Length);
				interiorNodes.Add((new InteriorChunkedDataNode(children), interiorLength));
			}
		}

		/// <inheritdoc/>
		public override async Task CopyToStreamAsync(Stream outputStream, BlobSerializerOptions? options, CancellationToken cancellationToken)
		{
			foreach (ChunkedDataNodeRef childNodeRef in Children)
			{
				ChunkedDataNode childNode = await childNodeRef.ReadBlobAsync(options, cancellationToken);
				await childNode.CopyToStreamAsync(outputStream, options, cancellationToken);
			}
		}

		/// <summary>
		/// Copy the contents of the node to the output stream without creating the intermediate FileNodes
		/// </summary>
		/// <param name="nodeData">Source data</param>
		/// <param name="outputStream">The output stream to receive the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task CopyToStreamAsync(BlobData nodeData, Stream outputStream, CancellationToken cancellationToken)
		{
			BlobReader nodeReader = new(nodeData, null);
			while (nodeReader.GetMemory(0).Length > 0)
			{
				IHashedBlobRef<ChunkedDataNode> handle = nodeReader.ReadBlobRef<ChunkedDataNode>();

				ChunkedDataNodeType type = ChunkedDataNodeType.Unknown;
				if (nodeReader.Version >= 2)
				{
					type = (ChunkedDataNodeType)nodeReader.ReadUnsignedVarInt();
				}
				if (nodeReader.Version >= 3)
				{
					_ = nodeReader.ReadUnsignedVarInt(); // Length
				}
				if (nodeReader.Version >= (int)HordeApiVersion.AddRollingHashesForLeafNodes && type == ChunkedDataNodeType.Leaf)
				{
					_ = nodeReader.ReadUInt32(); // Rolling hash
				}

				await handle.CopyToStreamAsync(outputStream, cancellationToken);
			}
		}
	}

	/// <summary>
	/// Converter for interior node types
	/// </summary>
	public class InteriorChunkedDataNodeConverter : BlobConverter<InteriorChunkedDataNode>
	{
		readonly HordeApiVersion _writeVersion;

		/// <summary>
		/// Constructor
		/// </summary>
		public InteriorChunkedDataNodeConverter() : this(HordeApiVersion.Latest)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="writeVersion">Version number for serialized data</param>
		public InteriorChunkedDataNodeConverter(HordeApiVersion writeVersion)
		{
			_writeVersion = writeVersion;
		}

		/// <inheritdoc/>
		public override InteriorChunkedDataNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			// Keep this code in sync with CopyToStreamAsync
			List<ChunkedDataNodeRef> children = [];
			while (reader.GetMemory().Length > 0)
			{
				IHashedBlobRef<ChunkedDataNode> handle = reader.ReadBlobRef<ChunkedDataNode>();

				ChunkedDataNodeType type = ChunkedDataNodeType.Unknown;
				if (reader.Version >= 2) // Pre sync with HordeApiVersion
				{
					type = (ChunkedDataNodeType)reader.ReadUnsignedVarInt();
				}

				long length = -1;
				if (reader.Version >= 3) // Pre sync with HordeApiVersion
				{
					length = (long)reader.ReadUnsignedVarInt();
				}

				uint rollingHash = 0;
				if (reader.Version >= (int)HordeApiVersion.AddRollingHashesForLeafNodes && type == ChunkedDataNodeType.Leaf)
				{
					rollingHash = reader.ReadUInt32();
				}

				children.Add(new ChunkedDataNodeRef(type, length, rollingHash, handle));
			}
			return new InteriorChunkedDataNode(children);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, InteriorChunkedDataNode value, BlobSerializerOptions options)
		{
			foreach (ChunkedDataNodeRef child in value.Children)
			{
				writer.WriteBlobRef(child);
				writer.WriteUnsignedVarInt((int)child.Type);

				if (_writeVersion >= HordeApiVersion.AddLengthsToInteriorNodes)
				{
					writer.WriteUnsignedVarInt((ulong)child.Length);
				}
				if (_writeVersion >= HordeApiVersion.AddRollingHashesForLeafNodes && child.Type == ChunkedDataNodeType.Leaf)
				{
					writer.WriteUInt32(child.RollingHash);
				}
			}
			return GetBlobType(_writeVersion);
		}

		/// <summary>
		/// Gets the blob type for a particular Horde api version
		/// </summary>
		public static BlobType GetBlobType(HordeApiVersion version)
		{
			int blobVersion;
			if (version >= HordeApiVersion.AddRollingHashesForLeafNodes)
			{
				blobVersion = (int)HordeApiVersion.AddRollingHashesForLeafNodes;
			}
			else if (version >= HordeApiVersion.AddLengthsToInteriorNodes)
			{
				blobVersion = 3;
			}
			else
			{
				blobVersion = 2;
			}
			return new BlobType(InteriorChunkedDataNode.BlobTypeGuid, blobVersion);
		}
	}

	/// <summary>
	/// Stream which returns the content of a chunked data stream
	/// </summary>
	class ChunkedDataStream : Stream
	{
		/// <inheritdoc/>
		public override bool CanRead => _readStream.CanRead;

		/// <inheritdoc/>
		public override bool CanSeek => _readStream.CanSeek;

		/// <inheritdoc/>
		public override bool CanWrite => _readStream.CanWrite;

		/// <inheritdoc/>
		public override long Length => _length ?? throw new NotSupportedException();

		/// <inheritdoc/>
		public override long Position { get => _readStream.Position; set => throw new NotImplementedException(); }

		readonly Pipe _pipe;
		readonly Stream _readStream;
		readonly BackgroundTask _writeTask;
		readonly long? _length;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="chunkedDataNodeRef">Reference to the chunked data node</param>
		public ChunkedDataStream(ChunkedDataNodeRef chunkedDataNodeRef)
			: this(chunkedDataNodeRef, chunkedDataNodeRef.Length)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="entry">The file entry to copy from</param>
		/// <param name="length">Length of the stream. May be null.</param>
		public ChunkedDataStream(IBlobRef<ChunkedDataNode> entry, long? length)
		{
			_pipe = new Pipe();
			_readStream = _pipe.Reader.AsStream();
			_writeTask = BackgroundTask.StartNew(ctx => WriteDataAsync(entry, _pipe.Writer, ctx));
			_length = length;
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			await _writeTask.DisposeAsync();

			_readStream.Dispose();
		}

		static async Task WriteDataAsync(IBlobRef<ChunkedDataNode> entry, PipeWriter pipeWriter, CancellationToken cancellationToken)
		{
			using Stream stream = pipeWriter.AsStream();
			await entry.CopyToStreamAsync(stream, cancellationToken);
		}

		/// <inheritdoc/>
		public override ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default) => _readStream.ReadAsync(buffer, cancellationToken);

		/// <inheritdoc/>
		public override void Flush()
		{
		}

		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => _readStream.Read(buffer, offset, count);

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();
	}
}
