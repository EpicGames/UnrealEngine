// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Options for creating file nodes
	/// </summary>
	public class ChunkingOptions
	{
		/// <summary>
		/// Options for creating leaf nodes
		/// </summary>
		public LeafChunkedDataNodeOptions LeafOptions { get; set; }

		/// <summary>
		/// Options for creating interior nodes
		/// </summary>
		public InteriorChunkedDataNodeOptions InteriorOptions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ChunkingOptions()
		{
			LeafOptions = LeafChunkedDataNodeOptions.Default;
			InteriorOptions = InteriorChunkedDataNodeOptions.Default;
		}
	}

	/// <summary>
	/// Options for creating a specific type of file nodes
	/// </summary>
	/// <param name="MinSize">Minimum chunk size</param>
	/// <param name="MaxSize">Maximum chunk size. Chunks will be split on this boundary if another match is not found.</param>
	/// <param name="TargetSize">Target chunk size for content-slicing</param>
	public record class LeafChunkedDataNodeOptions(int MinSize, int MaxSize, int TargetSize)
	{
		/// <summary>
		/// Default settings
		/// </summary>
		public static LeafChunkedDataNodeOptions Default { get; } = new LeafChunkedDataNodeOptions(32 * 1024, 256 * 1024, 64 * 1024);

		/// <summary>
		/// Window size to use when scanning for split points
		/// </summary>
		public int WindowSize => MinSize;

		/// <summary>
		/// Accessor for the BuzHash chunking threshold
		/// </summary>
		public uint Threshold => BuzHash.GetThreshold(TargetSize - MinSize);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="size">Fixed size chunks to use</param>
		public LeafChunkedDataNodeOptions(int size)
			: this(size, size, size)
		{
		}
	}

	/// <summary>
	/// Utility class for generating FileNode data directly into <see cref="IBlobWriter"/> instances, without constructing node representations first.
	/// </summary>
	public sealed class LeafChunkedDataWriter : IDisposable
	{
		/// <summary>
		/// Default buffer length when calling CreateAsync/AppendAsync
		/// </summary>
		public const int DefaultBufferLength = 32 * 1024;

		readonly IBlobWriter _writer;
		readonly LeafChunkedDataNodeOptions _leafChunkOptions;
		readonly Blake3.Hasher _hasher;

		// Tree state
		long _totalLength;
		readonly List<ChunkedDataNodeRef> _leafHandles = [];

		// Leaf node state
		uint _leafHash;

		/// <summary>
		/// Length of the file so far
		/// </summary>
		public long Length => _totalLength;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="leafNodeWriter">Writer for new nodes</param>
		public LeafChunkedDataWriter(IBlobWriter leafNodeWriter)
			: this(leafNodeWriter, LeafChunkedDataNodeOptions.Default)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="leafNodeWriter">Writer for new nodes</param>
		/// <param name="leafChunkOptions">Chunking options</param>
		public LeafChunkedDataWriter(IBlobWriter leafNodeWriter, LeafChunkedDataNodeOptions leafChunkOptions)
		{
			_writer = leafNodeWriter;
			_leafChunkOptions = leafChunkOptions;
			_hasher = Blake3.Hasher.New();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_hasher.Dispose();
		}

		/// <summary>
		/// Reset the current state
		/// </summary>
		public void Reset()
		{
			_leafHandles.Clear();
			ResetLeafState();
			_totalLength = 0;
			_hasher.Reset();
		}

		/// <summary>
		/// Resets the state of the current leaf node
		/// </summary>
		void ResetLeafState()
		{
			_leafHash = 0;
		}

		/// <summary>
		/// Creates data for the given file
		/// </summary>
		/// <param name="fileInfo">File to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<LeafChunkedData> CreateAsync(FileInfo fileInfo, CancellationToken cancellationToken)
		{
			return await CreateAsync(fileInfo, DefaultBufferLength, cancellationToken);
		}

		/// <summary>
		/// Creates data for the given file
		/// </summary>
		/// <param name="fileInfo">File to append</param>
		/// <param name="bufferLength">Size of the read buffer</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<LeafChunkedData> CreateAsync(FileInfo fileInfo, int bufferLength, CancellationToken cancellationToken)
		{
			using (FileStream stream = fileInfo.OpenRead())
			{
				return await CreateAsync(stream, bufferLength, cancellationToken);
			}
		}

		/// <summary>
		/// Creates data from the given stream
		/// </summary>
		/// <param name="stream">Stream to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<LeafChunkedData> CreateAsync(Stream stream, CancellationToken cancellationToken)
		{
			return await CreateAsync(stream, DefaultBufferLength, cancellationToken);
		}

		/// <summary>
		/// Creates data from the given stream
		/// </summary>
		/// <param name="stream">Stream to append</param>
		/// <param name="bufferLength">Size of the read buffer</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<LeafChunkedData> CreateAsync(Stream stream, int bufferLength, CancellationToken cancellationToken)
		{
			Reset();
			await AppendAsync(stream, bufferLength, cancellationToken);
			return await CompleteAsync(cancellationToken);
		}

		/// <summary>
		/// Creates data from the given data
		/// </summary>
		/// <param name="data">Stream to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<LeafChunkedData> CreateAsync(ReadOnlyMemory<byte> data, CancellationToken cancellationToken)
		{
			Reset();
			await AppendAsync(data, cancellationToken);
			return await CompleteAsync(cancellationToken);
		}

		/// <summary>
		/// Appends data to the current file
		/// </summary>
		/// <param name="stream">Stream containing data to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AppendAsync(Stream stream, CancellationToken cancellationToken)
		{
			await AppendAsync(stream, DefaultBufferLength, cancellationToken);
		}

		/// <summary>
		/// Appends data to the current file
		/// </summary>
		/// <param name="stream">Stream containing data to append</param>
		/// <param name="bufferLength">Size of the read buffer</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AppendAsync(Stream stream, int bufferLength, CancellationToken cancellationToken)
		{
			await stream.ReadAllBytesAsync(bufferLength, async (x) => await AppendAsync(x, cancellationToken), cancellationToken);
		}

		/// <summary>
		/// Appends data to the current file
		/// </summary>
		/// <param name="data">Data to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AppendAsync(ReadOnlyMemory<byte> data, CancellationToken cancellationToken)
		{
			_hasher.Update(data.Span);

			for (; ; )
			{
				int appendLength = AppendToLeafNode(_writer.WrittenMemory.Span, data.Span, ref _leafHash, _leafChunkOptions);
				_writer.WriteFixedLengthBytes(data.Slice(0, appendLength).Span);

				data = data.Slice(appendLength);

				_totalLength += appendLength;

				if (data.Length == 0)
				{
					break;
				}

				await FlushLeafNodeAsync(cancellationToken);
			}
		}

		/// <summary>
		/// Determines how much data to append to an existing leaf node
		/// </summary>
		/// <param name="currentData">Current data in the leaf node</param>
		/// <param name="appendData">Data to be appended</param>
		/// <param name="rollingHash">Current BuzHash of the data</param>
		/// <param name="options">Options for chunking the data</param>
		/// <returns>The number of bytes to append</returns>
		static int AppendToLeafNode(ReadOnlySpan<byte> currentData, ReadOnlySpan<byte> appendData, ref uint rollingHash, LeafChunkedDataNodeOptions options)
		{
			// If the target option sizes are fixed, just chunk the data along fixed boundaries
			if (options.MinSize == options.TargetSize && options.MaxSize == options.TargetSize)
			{
				return Math.Min(appendData.Length, options.MaxSize - (int)currentData.Length);
			}

			// Cap the append data span to the maximum amount we can add
			int maxAppendLength = options.MaxSize - currentData.Length;
			if (maxAppendLength < appendData.Length)
			{
				appendData = appendData.Slice(0, maxAppendLength);
			}

			// Length of the data to be appended
			int appendLength = 0;

			// Fast path for appending data to the buffer up to the chunk window size
			int windowSize = options.MinSize;
			if (currentData.Length < windowSize)
			{
				appendLength = Math.Min(windowSize - (int)currentData.Length, appendData.Length);
				rollingHash = BuzHash.Add(rollingHash, appendData.Slice(0, appendLength));
			}

			// Get the threshold for the rolling hash
			uint rollingHashThreshold = (uint)((1L << 32) / (options.TargetSize - options.MinSize));

			// Step through the part of the data where the tail of the window is in currentData, and the head of the window is in appendData.
			if (appendLength < appendData.Length && windowSize > appendLength)
			{
				int overlap = windowSize - appendLength;
				int overlapLength = Math.Min(appendData.Length - appendLength, overlap);

				ReadOnlySpan<byte> tailSpan = currentData.Slice(currentData.Length - overlap, overlapLength);
				ReadOnlySpan<byte> headSpan = appendData.Slice(appendLength, overlapLength);

				int count = BuzHash.Update(tailSpan, headSpan, windowSize, rollingHashThreshold, ref rollingHash);
				if (count != -1)
				{
					appendLength += count;
					return appendLength;
				}

				appendLength += headSpan.Length;
			}

			// Step through the rest of the data which is completely contained in appendData.
			if (appendLength < appendData.Length)
			{
				Debug.Assert(appendLength >= windowSize);

				ReadOnlySpan<byte> tailSpan = appendData.Slice(appendLength - windowSize, appendData.Length - windowSize);
				ReadOnlySpan<byte> headSpan = appendData.Slice(appendLength);

				int count = BuzHash.Update(tailSpan, headSpan, windowSize, rollingHashThreshold, ref rollingHash);
				if (count != -1)
				{
					appendLength += count;
					return appendLength;
				}

				appendLength += headSpan.Length;
			}

			return appendLength;
		}

		/// <summary>
		/// Complete the current file, and write all open nodes to the underlying writer
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the root node</returns>
		public async Task<LeafChunkedData> CompleteAsync(CancellationToken cancellationToken)
		{
			IoHash hash = IoHash.FromBlake3(_hasher);
			await FlushLeafNodeAsync(cancellationToken);
			LeafChunkedData leafChunkedData = new(hash, [.. _leafHandles]);

			_leafHandles.Clear();
			_hasher.Reset();
			_totalLength = 0;

			return leafChunkedData;
		}

		/// <summary>
		/// Writes the contents of the current leaf node to storage
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the written leaf node</returns>
		async ValueTask FlushLeafNodeAsync(CancellationToken cancellationToken)
		{
			int leafLength = _writer.WrittenMemory.Length;
			IHashedBlobRef<LeafChunkedDataNode> leafHandle = await _writer.CompleteAsync<LeafChunkedDataNode>(LeafChunkedDataNodeConverter.BlobType, cancellationToken);
			_leafHandles.Add(new ChunkedDataNodeRef(leafLength, _leafHash, leafHandle));
			ResetLeafState();
		}
	}

	/// <summary>
	/// Describes a chunked data stream
	/// </summary>
	/// <param name="StreamHash">Hash of the stream as a contiguous buffer</param>
	/// <param name="Root">Handle to the root chunk containing the data</param>
	public record class ChunkedData(IoHash StreamHash, ChunkedDataNodeRef Root)
	{
		static readonly ChunkingOptions s_defaultChunkingOptions = new();

		/// <summary>
		/// Creates chunked data from a block of memory
		/// </summary>
		/// <param name="blobWriter">Writer for output blobs</param>
		/// <param name="data">Data to create from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task<ChunkedData> CreateAsync(IBlobWriter blobWriter, ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default)
			=> CreateAsync(blobWriter, data, s_defaultChunkingOptions, cancellationToken);

		/// <summary>
		/// Creates chunked data from a block of memory
		/// </summary>
		/// <param name="blobWriter">Writer for output blobs</param>
		/// <param name="data">Data to create from</param>
		/// <param name="chunkingOptions">Options controlling the chunking</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<ChunkedData> CreateAsync(IBlobWriter blobWriter, ReadOnlyMemory<byte> data, ChunkingOptions chunkingOptions, CancellationToken cancellationToken = default)
		{
			using ChunkedDataWriter chunkedWriter = new(blobWriter, chunkingOptions);
			await chunkedWriter.AppendAsync(data, cancellationToken);
			return await chunkedWriter.FlushAsync(cancellationToken);
		}
	}

	/// <summary>
	/// Writes chunked data to an output writer
	/// </summary>
	public sealed class ChunkedDataWriter : IDisposable
	{
		readonly IBlobWriter _writer;
		readonly ChunkingOptions _chunkingOptions;
		readonly LeafChunkedDataWriter _leafWriter;

		/// <summary>
		/// Length of the current stream
		/// </summary>
		public long Length => _leafWriter.Length;

		/// <summary>
		/// Constructor
		/// </summary>
		public ChunkedDataWriter(IBlobWriter writer, ChunkingOptions chunkingOptions)
		{
			_writer = writer;
			_chunkingOptions = chunkingOptions;
			_leafWriter = new LeafChunkedDataWriter(writer, chunkingOptions.LeafOptions);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_leafWriter.Dispose();
		}

		/// <summary>
		/// Reset the current state
		/// </summary>
		public void Reset()
		{
			_leafWriter.Reset();
		}

		/// <summary>
		/// Creates data for the given file
		/// </summary>
		/// <param name="fileInfo">File to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ChunkedData> CreateAsync(FileInfo fileInfo, CancellationToken cancellationToken)
		{
			Reset();
			LeafChunkedData leafChunkedData = await _leafWriter.CreateAsync(fileInfo, cancellationToken);
			return await InteriorChunkedDataNode.CreateTreeAsync(leafChunkedData, _chunkingOptions.InteriorOptions, _writer, cancellationToken);
		}

		/// <summary>
		/// Creates data from the given data
		/// </summary>
		/// <param name="data">Stream to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<ChunkedData> CreateAsync(ReadOnlyMemory<byte> data, CancellationToken cancellationToken)
		{
			Reset();
			await _leafWriter.AppendAsync(data, cancellationToken);
			return await CompleteAsync(cancellationToken);
		}

		/// <summary>
		/// Appends data to the current file
		/// </summary>
		/// <param name="data">Data to append</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AppendAsync(ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default)
		{
			await _leafWriter.AppendAsync(data, cancellationToken);
		}

		/// <summary>
		/// Complete the current file, and write all open nodes to the underlying writer
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the root node</returns>
		public async Task<ChunkedData> CompleteAsync(CancellationToken cancellationToken = default)
		{
			LeafChunkedData leafChunkedData = await _leafWriter.CompleteAsync(cancellationToken);
			return await InteriorChunkedDataNode.CreateTreeAsync(leafChunkedData, _chunkingOptions.InteriorOptions, _writer, cancellationToken);
		}

		/// <summary>
		/// Complete the current file, and write all open nodes to the underlying writer
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Handle to the root node</returns>
		public async Task<ChunkedData> FlushAsync(CancellationToken cancellationToken = default)
		{
			ChunkedData chunkedData = await CompleteAsync(cancellationToken);
			await _writer.FlushAsync(cancellationToken);
			return chunkedData;
		}
	}
}
