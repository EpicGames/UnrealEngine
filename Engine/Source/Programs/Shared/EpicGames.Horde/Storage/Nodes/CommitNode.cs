// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// A node representing commit metadata
	/// </summary>
	[BlobConverter(typeof(CommitNodeConverter))]
	public class CommitNode
	{
		/// <summary>
		/// Static accessor for the blob type guid
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{64D50724-41C0-6B22-1CB5-90A8171824D6}");

		/// <summary>
		/// The commit number
		/// </summary>
		public int Number { get; set; }

		/// <summary>
		/// Reference to the parent commit
		/// </summary>
		public IHashedBlobRef<CommitNode>? Parent { get; set; }

		/// <summary>
		/// Human readable name of the author of this change
		/// </summary>
		public string Author { get; set; }

		/// <summary>
		/// Optional unique identifier for the author. May be an email address, user id, etc...
		/// </summary>
		public string? AuthorId { get; set; }

		/// <summary>
		/// Human readable name of the committer of this change
		/// </summary>
		public string? Committer { get; set; }

		/// <summary>
		/// Optional unique identifier for the committer. May be an email address, user id, etc...
		/// </summary>
		public string? CommitterId { get; set; }

		/// <summary>
		/// Message for this commit
		/// </summary>
		public string Message { get; set; }

		/// <summary>
		/// Time that this commit was created
		/// </summary>
		public DateTime Time { get; set; }

		/// <summary>
		/// Contents of the tree at this commit
		/// </summary>
		public DirectoryNodeRef Contents { get; set; }

		/// <summary>
		/// Metadata for this commit, keyed by arbitrary GUID
		/// </summary>
		public Dictionary<Guid, IHashedBlobRef> Metadata { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public CommitNode(int number, IHashedBlobRef<CommitNode>? parent, string author, string? authorId, string? committer, string? commiterId, string message, DateTime time, DirectoryNodeRef contents, Dictionary<Guid, IHashedBlobRef> metadata)
		{
			Number = number;
			Parent = parent;
			Author = author;
			AuthorId = authorId;
			Committer = committer;
			CommitterId = commiterId;
			Message = message;
			Time = time;
			Contents = contents;
			Metadata = metadata;
		}
	}

	class CommitNodeConverter : BlobConverter<CommitNode>
	{
		static readonly BlobType s_blobType = new(CommitNode.BlobTypeGuid, 1);

		public override CommitNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			int number = (int)reader.ReadUnsignedVarInt();

			IHashedBlobRef<CommitNode>? parent;
			if (reader.ReadBoolean())
			{
				parent = reader.ReadBlobRef<CommitNode>();
			}
			else
			{
				parent = null;
			}

			string author = reader.ReadString();
			string? authorId = reader.ReadOptionalString();
			string? committer = reader.ReadOptionalString();
			string? committerId = reader.ReadOptionalString();
			string message = reader.ReadString();
			DateTime time = reader.ReadDateTime();

			IHashedBlobRef<DirectoryNode> contentsNode = reader.ReadBlobRef<DirectoryNode>();
			long length = (long)reader.ReadUnsignedVarInt();
			DirectoryNodeRef contents = new(length, contentsNode);

			Dictionary<Guid, IHashedBlobRef> metadata = reader.ReadDictionary(() => reader.ReadGuidUnrealOrder(), () => reader.ReadBlobRef());

			return new CommitNode(number, parent, author, authorId, committer, committerId, message, time, contents, metadata);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, CommitNode value, BlobSerializerOptions options)
		{
			writer.WriteUnsignedVarInt(value.Number);
			writer.WriteBoolean(value.Parent != null);
			if (value.Parent != null)
			{
				writer.WriteBlobRef(value.Parent);
			}
			writer.WriteString(value.Author);
			writer.WriteOptionalString(value.AuthorId);
			writer.WriteOptionalString(value.Committer);
			writer.WriteOptionalString(value.CommitterId);
			writer.WriteString(value.Message);
			writer.WriteDateTime(value.Time);

			writer.WriteBlobRef(value.Contents.Handle);
			writer.WriteUnsignedVarInt((ulong)value.Contents.Length);

			writer.WriteDictionary(value.Metadata, key => writer.WriteGuidUnrealOrder(key), value => writer.WriteBlobRef(value));

			return s_blobType;
		}
	}
}
