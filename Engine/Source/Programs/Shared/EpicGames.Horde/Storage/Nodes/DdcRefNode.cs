// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// A node containing ref data
	/// </summary>
	[BlobConverter(typeof(DdcRefNodeConverter))]
	public class DdcRefNode
	{
		/// <summary>
		/// Static accessor for the blob type guid
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{0C7E5F25-4B55-454B-63F4-4A9B74D00651}");

		/// <summary>
		/// Hash of the root node
		/// </summary>
		public IoHash RootHash { get; }

		/// <summary>
		/// References to attachments. We embed this in the ref node to ensure any aliased blobs have a hard reference from the root.
		/// </summary>
		public List<IHashedBlobRef> References { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public DdcRefNode(IoHash rootHash)
		{
			RootHash = rootHash;
			References = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public DdcRefNode(IoHash rootHash, IEnumerable<IHashedBlobRef> references)
		{
			RootHash = rootHash;
			References = [.. references];
		}
	}

	class DdcRefNodeConverter : BlobConverter<DdcRefNode>
	{
		static readonly BlobType s_blobType = new(DdcRefNode.BlobTypeGuid, 1);

		public override DdcRefNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			IoHash rootHash = reader.ReadIoHash();
			List<IHashedBlobRef> references = reader.ReadList(x => reader.ReadBlobRef());
			return new DdcRefNode(rootHash, references);
		}

		public override BlobType Write(IBlobWriter writer, DdcRefNode value, BlobSerializerOptions options)
		{
			writer.WriteIoHash(value.RootHash);
			writer.WriteList(value.References, x => writer.WriteBlobRef(x));
			return s_blobType;
		}
	}
}
