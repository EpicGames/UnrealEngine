// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Stats reported for copy operations
	/// </summary>
	public interface IExtractStats
	{
		/// <summary>
		/// Number of files that have been copied
		/// </summary>
		int NumFiles { get; }

		/// <summary>
		/// Total size of data to be copied
		/// </summary>
		long ExtractSize { get; }

		/// <summary>
		/// Processing speed, in bytes per second
		/// </summary>
		double ExtractRate { get; }

		/// <summary>
		/// Total size of the data downloaded
		/// </summary>
		long DownloadSize { get; }

		/// <summary>
		/// Download speed, in bytes per second
		/// </summary>
		double DownloadRate { get; }
	}

	/// <summary>
	/// Progress logger for writing copy stats
	/// </summary>
	public class ExtractStatsLogger : IProgress<IExtractStats>
	{
		readonly int _totalCount;
		readonly long _totalSize;
		readonly ILogger _logger;

		/// <summary>
		/// Whether to print out separate stats for download speed
		/// </summary>
		public bool ShowDownloadStats { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ExtractStatsLogger(ILogger logger)
			=> _logger = logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ExtractStatsLogger(int totalCount, long totalSize, ILogger logger)
		{
			_totalCount = totalCount;
			_totalSize = totalSize;
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Report(IExtractStats stats)
		{
			if (ShowDownloadStats)
			{
				double downloadRate = (stats.DownloadRate * 8.0) / (1024.0 * 1024.0);
				_logger.LogInformation("Downloaded {TotalSize:n1}mb, {Rate:n0} mbps", stats.DownloadSize / (1024.0 * 1024.0), downloadRate);
			}

			if (_totalCount > 0 && _totalSize > 0)
			{
				_logger.LogInformation("Written {NumFiles:n0}/{TotalFiles:n0} files ({Size:n1}/{TotalSize:n1}mb, {Rate:n1}mb/s, {Pct}%)", stats.NumFiles, _totalCount, stats.ExtractSize / (1024.0 * 1024.0), _totalSize / (1024.0 * 1024.0), stats.ExtractRate / (1024.0 * 1024.0), (int)((Math.Max(stats.ExtractSize, 1) * 100) / Math.Max(_totalSize, 1)));
			}
			else if (_totalCount > 0)
			{
				_logger.LogInformation("Written {NumFiles:n0}/{TotalFiles:n0} files ({Size:n1}mb, {Rate:n1}mb/s)", stats.NumFiles, _totalCount, stats.ExtractSize / (1024.0 * 1024.0), stats.ExtractRate / (1024.0 * 1024.0));
			}
			else if (_totalSize > 0)
			{
				_logger.LogInformation("Written {NumFiles:n0} files ({Size:n1}/{TotalSize:n1}mb, {Rate:n1}mb/s, {Pct}%)", stats.NumFiles, stats.ExtractSize / (1024.0 * 1024.0), _totalSize / (1024.0 * 1024.0), stats.ExtractRate / (1024.0 * 1024.0), (int)((Math.Max(stats.ExtractSize, 1) * 100) / Math.Max(_totalSize, 1)));
			}
			else
			{
				_logger.LogInformation("Written {NumFiles:n0} files ({Size:n1}mb, {Rate:n1}mb/s)", stats.NumFiles, stats.ExtractSize / (1024.0 * 1024.0), stats.ExtractRate / (1024.0 * 1024.0));
			}
		}
	}

	/// <summary>
	/// Options for extracting data
	/// </summary>
	public class ExtractOptions
	{
		/// <summary>
		/// Number of async tasks to spawn for reading
		/// </summary>
		public int? NumReadTasks { get; set; }

		/// <summary>
		/// Default number of read tasks to use
		/// </summary>
		public static int DefaultNumReadTasks { get; } = 16;

		/// <summary>
		/// Number of async tasks to spawn for decoding data
		/// </summary>
		public int? NumDecodeTasks { get; set; }

		/// <summary>
		/// Default number of decode tasks to use
		/// </summary>
		public static int DefaultNumDecodeTasks { get; } = Math.Min(Environment.ProcessorCount, 16);

		/// <summary>
		/// Number of async tasks to spawn for writing output
		/// </summary>
		public int? NumWriteTasks { get; set; }

		/// <summary>
		/// Default number of write tasks to use
		/// </summary>
		public static int DefaultNumWriteTasks(long downloadSize)
			=> Math.Min(1 + (int)(downloadSize / (16 * 1024 * 1024)), 16);

		/// <summary>
		/// Output for progress updates
		/// </summary>
		public IProgress<IExtractStats>? Progress { get; set; }

		/// <summary>
		/// Frequency that the progress object is updated
		/// </summary>
		public TimeSpan ProgressUpdateFrequency { get; set; } = TimeSpan.FromSeconds(5.0);

		/// <summary>
		/// Whether to hash downloaded data to ensure it's correct
		/// </summary>
		public bool VerifyOutput { get; set; }

		/// <summary>
		/// Output verbose logging about operations being performed
		/// </summary>
		public bool VerboseOutput { get; set; }

		/// <summary>
		/// Maximum number of exports to write in a single request
		/// </summary>
		public int? MaxBatchSize { get; set; }
	}

	/// <summary>
	/// Extension methods for extracting data from directory nodes
	/// </summary>
	public static class DirectoryNodeExtract
	{
#pragma warning disable IDE0060
		static void TraceBlobRead(string type, string path, IBlobRef handle, ILogger logger)
		{
			//			logger.LogTrace(KnownLogEvents.Horde_BlobRead, "Blob [{Type,-20}] Path=\"{Path}\", Locator={Locator}", type, path, handle.GetLocator());
		}
#pragma warning restore IDE0060

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryRef">Directory to update</param>
		/// <param name="directoryInfo"></param>
		/// <param name="logger"></param>
		/// <param name="cancellationToken"></param>
		[Obsolete("Use an ExtractAsync overload that takes an ExtractOptions object")]
		public static Task ExtractAsync(this IBlobRef<DirectoryNode> directoryRef, DirectoryInfo directoryInfo, ILogger logger, CancellationToken cancellationToken = default)
			=> ExtractAsync(directoryRef, directoryInfo, new ExtractOptions(), logger, cancellationToken);

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryRef">Directory to extract</param>
		/// <param name="directoryInfo">Direcotry to write to</param>
		/// <param name="progress">Sink for progress updates</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[Obsolete("Use an ExtractAsync overload that takes an ExtractOptions object")]
		public static Task ExtractAsync(this IBlobRef<DirectoryNode> directoryRef, DirectoryInfo directoryInfo, IProgress<IExtractStats>? progress, ILogger logger, CancellationToken cancellationToken = default)
		{
			return ExtractAsync(directoryRef, directoryInfo, new ExtractOptions { Progress = progress }, logger, cancellationToken);
		}

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryRef">Directory to extract</param>
		/// <param name="directoryInfo">Direcotry to write to</param>
		/// <param name="progress">Sink for progress updates</param>
		/// <param name="frequency">Frequency for progress updates</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[Obsolete("Use an ExtractAsync overload that takes an ExtractOptions object")]
		public static async Task ExtractAsync(this IBlobRef<DirectoryNode> directoryRef, DirectoryInfo directoryInfo, IProgress<IExtractStats>? progress, TimeSpan frequency, ILogger logger, CancellationToken cancellationToken = default)
		{
			DirectoryNode directoryNode = await directoryRef.ReadBlobAsync(cancellationToken);
			await ExtractAsync(directoryNode, directoryInfo, new ExtractOptions { Progress = progress, ProgressUpdateFrequency = frequency }, logger, cancellationToken);
		}

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryNode">Directory to extract</param>
		/// <param name="directoryInfo">Direcotry to write to</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task ExtractAsync(this DirectoryNode directoryNode, DirectoryInfo directoryInfo, ILogger logger, CancellationToken cancellationToken = default)
			=> ExtractAsync(directoryNode, directoryInfo, new ExtractOptions(), logger, cancellationToken);

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryRef">Directory to extract</param>
		/// <param name="directoryInfo">Direcotry to write to</param>
		/// <param name="options">Options for the download</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task ExtractAsync(this IBlobRef<DirectoryNode> directoryRef, DirectoryInfo directoryInfo, ExtractOptions? options, ILogger logger, CancellationToken cancellationToken = default)
		{
			DirectoryNode directoryNode = await directoryRef.ReadBlobAsync(cancellationToken);
			await ExtractAsync(directoryNode, directoryInfo, options, logger, cancellationToken);
		}

		/// <summary>
		/// Utility function to allow extracting a packed directory to disk
		/// </summary>
		/// <param name="directoryNode">Directory to extract</param>
		/// <param name="directoryInfo">Direcotry to write to</param>
		/// <param name="options">Options for the download</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task ExtractAsync(this DirectoryNode directoryNode, DirectoryInfo directoryInfo, ExtractOptions? options, ILogger logger, CancellationToken cancellationToken = default)
		{
			options ??= new ExtractOptions();

			int numReadTasks = options.NumReadTasks ?? ExtractOptions.DefaultNumReadTasks;
			int numDecodeTasks = options.NumDecodeTasks ?? ExtractOptions.DefaultNumDecodeTasks;
			int numWriteTasks = options.NumWriteTasks ?? Math.Min(1 + (int)(directoryNode.Length / (16 * 1024 * 1024)), 16);
			int maxBatchSize = options.MaxBatchSize ?? BatchChunkReader.DefaultMaxBatchSize;

			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				using BatchOutputWriter outputWriter = new(logger);
				outputWriter.VerifyOutput = options.VerifyOutput;
				outputWriter.VerboseOutput = options.VerboseOutput;

				using BatchChunkReader chunkReader = new(outputWriter.RequestWriter) { MaxBatchSize = maxBatchSize };
				chunkReader.VerifyHashes = options.VerifyOutput;

				logger.LogDebug("Using {NumReadTasks} read tasks, {NumDecodeTasks} decode tasks, {NumWriteTasks} write tasks, {MaxBatchSize} max batch size", numReadTasks, numDecodeTasks, numWriteTasks, chunkReader.MaxBatchSize);

				Channel<ChunkReadRequest> blobRequests = Channel.CreateBounded<ChunkReadRequest>(new BoundedChannelOptions(chunkReader.MinQueueLength * 2));
				_ = pipeline.AddTask(ctx => FindOutputChunksRootAsync(directoryInfo, directoryNode, blobRequests.Writer, options, logger, ctx));

				Task[] readTasks = chunkReader.AddToPipeline(pipeline, numReadTasks, numDecodeTasks, blobRequests.Reader);
				_ = Task.WhenAll(readTasks).ContinueWith(_ => outputWriter.RequestWriter.TryComplete(), TaskScheduler.Default);

				Task[] writeTasks = outputWriter.AddToPipeline(pipeline, numWriteTasks);

				if (options.Progress != null)
				{
					_ = pipeline.AddTask(ctx => UpdateStatsAsync(chunkReader, outputWriter, Task.WhenAll(writeTasks), options.Progress, options.ProgressUpdateFrequency, ctx));
				}

				await pipeline.WaitForCompletionAsync();

				cancellationToken.ThrowIfCancellationRequested();
			}
		}

		#region Enumerate chunks

		static async Task FindOutputChunksRootAsync(DirectoryInfo rootDir, DirectoryNode node, ChannelWriter<ChunkReadRequest> writer, ExtractOptions options, ILogger logger, CancellationToken cancellationToken)
		{
			await FindOutputChunksForDirectoryAsync(rootDir, "", node, writer, options, logger, cancellationToken);
			writer.Complete();
		}

		static async Task FindOutputChunksForDirectoryAsync(DirectoryInfo rootDir, string path, DirectoryNode node, ChannelWriter<ChunkReadRequest> writer, ExtractOptions options, ILogger logger, CancellationToken cancellationToken)
		{
			foreach (FileEntry fileEntry in node.Files)
			{
				string filePath = CombinePaths(path, fileEntry.Name);
				FileInfo fileInfo = new(Path.Combine(rootDir.FullName, filePath));
				OutputFile outputFile = new(filePath, fileInfo, fileEntry);

				await FindOutputChunksForFileAsync(outputFile, writer, options, logger, cancellationToken);
			}

			foreach (DirectoryEntry directoryEntry in node.Directories)
			{
				string subPath = CombinePaths(path, directoryEntry.Name);
				TraceBlobRead("Directory", subPath, directoryEntry.Handle, logger);
				DirectoryNode subDirectoryNode = await directoryEntry.Handle.ReadBlobAsync(cancellationToken);

				await FindOutputChunksForDirectoryAsync(rootDir, subPath, subDirectoryNode, writer, options, logger, cancellationToken);
			}
		}

		internal static async Task FindOutputChunksForFileAsync(OutputFile outputFile, ChannelWriter<ChunkReadRequest> writer, ExtractOptions options, ILogger logger, CancellationToken cancellationToken)
		{
			long offset = 0;
			ChunkReadRequest? bufferedRequest = null;

			await foreach (LeafChunkedDataNodeRef leafNodeRef in outputFile.FileEntry.Target.EnumerateLeafNodesAsync(cancellationToken))
			{
				if (options.VerboseOutput)
				{
					logger.LogInformation("Queuing {Path},{Offset} ({Locator})", outputFile.Path, offset, leafNodeRef.GetLocator());
				}

				outputFile.IncrementRemaining();
				ChunkReadRequest request = new(outputFile, offset, leafNodeRef);
				offset += leafNodeRef.Length;

				if (bufferedRequest != null)
				{
					await writer.WriteAsync(bufferedRequest, cancellationToken);
				}

				bufferedRequest = request;
			}

			if (bufferedRequest != null)
			{
				await writer.WriteAsync(bufferedRequest, cancellationToken);
			}
		}

		#endregion

		#region Stats

		class ExtractStats : IExtractStats
		{
			public int NumFiles { get; set; }
			public long ExtractSize { get; set; }
			public double ExtractRate { get; set; }
			public long DownloadSize { get; set; }
			public double DownloadRate { get; set; }
		}

		internal static async Task UpdateStatsAsync(BatchChunkReader chunkReader, BatchOutputWriter outputWriter, Task writeTask, IProgress<IExtractStats> progress, TimeSpan frequency, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();
			double lastOutputTime = 0.0;

			// Store previous download progress so we can sample the download rate over time
			double rateAverageOverTime = 5.0;
			Queue<(double, ExtractStats)> samples = new();
			samples.Enqueue((0.0, new ExtractStats()));

			Task? completeTask = null;
			while (completeTask != writeTask)
			{
				completeTask = await Task.WhenAny(writeTask, Task.Delay(frequency / 4.0, cancellationToken));

				double timeSeconds = timer.Elapsed.TotalSeconds;

				ExtractStats stats = new();
				stats.NumFiles = outputWriter.WrittenFiles;
				stats.ExtractSize = outputWriter.WrittenBytes;
				samples.Enqueue((timeSeconds, stats));

				BatchReaderStats batchStats = chunkReader.GetStats();
				stats.DownloadSize = batchStats.BytesRead;

				(double TimeSeconds, ExtractStats Stats) firstSample;
				while (samples.TryPeek(out firstSample) && samples.Count > 2 && timeSeconds > firstSample.TimeSeconds + rateAverageOverTime)
				{
					samples.Dequeue();
				}

				double elapsedSeconds = timeSeconds - firstSample.TimeSeconds;
				if (elapsedSeconds > 1.0)
				{
					stats.ExtractRate = (stats.ExtractSize - firstSample.Stats.ExtractSize) / elapsedSeconds;
					stats.DownloadRate = (stats.DownloadSize - firstSample.Stats.DownloadSize) / elapsedSeconds;
				}

				if (timeSeconds > lastOutputTime + frequency.TotalSeconds || completeTask == writeTask)
				{
					progress.Report(stats);
					lastOutputTime = timeSeconds;
				}
			}
		}

		#endregion

		static string CombinePaths(string basePath, string nextPath)
		{
			if (basePath.Length > 0)
			{
				return $"{basePath}/{nextPath}";
			}
			else
			{
				return nextPath;
			}
		}
	}
}
