// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Stats reported for copy operations
	/// </summary>
	public interface IUpdateStats
	{
		/// <summary>
		/// Number of files that have been copied
		/// </summary>
		int Count { get; }

		/// <summary>
		/// Total size of data to be copied
		/// </summary>
		long Size { get; }

		/// <summary>
		/// Processing speed, in bytes per second
		/// </summary>
		double Rate { get; }
	}

	/// <summary>
	/// Reports progress info back to callers
	/// </summary>
	class UpdateStats : IUpdateStats
	{
		readonly Lock _lockObject = new();
		readonly Stopwatch _timer = Stopwatch.StartNew();
		readonly IProgress<IUpdateStats>? _progress;
		long _lastTotalSize;

		public int Count { get; set; }
		public long Size { get; set; }
		public double Rate { get; set; }

		public UpdateStats(IProgress<IUpdateStats>? progress)
		{
			_progress = progress;
		}

		public void Update(int count, long size)
		{
			if (_progress != null)
			{
				lock (_lockObject)
				{
					Count += count;
					Size += size;
					if (_timer.Elapsed > TimeSpan.FromSeconds(5.0))
					{
						Rate = (Size - _lastTotalSize) / _timer.Elapsed.TotalSeconds;
						_lastTotalSize = Size;

						FlushInternal();
					}
				}
			}
		}

		public void Flush()
		{
			if (_progress != null)
			{
				lock (_lockObject)
				{
					FlushInternal();
				}
			}
		}

		void FlushInternal()
		{
			_progress!.Report(this);
			_timer.Restart();
		}
	}

	/// <summary>
	/// Progress logger for writing copy stats
	/// </summary>
	public class UpdateStatsLogger : IProgress<IUpdateStats>
	{
		readonly int _totalCount;
		readonly long _totalSize;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UpdateStatsLogger(ILogger logger)
			=> _logger = logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UpdateStatsLogger(int totalCount, long totalSize, ILogger logger)
		{
			_totalCount = totalCount;
			_totalSize = totalSize;
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Report(IUpdateStats stats)
		{
			if (_totalCount > 0 && _totalSize > 0)
			{
				_logger.LogInformation("Copied {NumFiles:n0}/{TotalFiles:n0} files ({Size:n1}/{TotalSize:n1}mb, {Rate:n1}mb/s, {Pct}%)", stats.Count, _totalCount, stats.Size / (1024.0 * 1024.0), _totalSize / (1024.0 * 1024.0), stats.Rate / (1024.0 * 1024.0), (int)((Math.Max(stats.Size, 1) * 100) / Math.Max(_totalSize, 1)));
			}
			else if (_totalCount > 0)
			{
				_logger.LogInformation("Copied {NumFiles:n0}/{TotalFiles:n0} files ({Size:n1}mb, {Rate:n1}mb/s)", stats.Count, _totalCount, stats.Size / (1024.0 * 1024.0), stats.Rate / (1024.0 * 1024.0));
			}
			else if (_totalSize > 0)
			{
				_logger.LogInformation("Copied {NumFiles:n0} files ({Size:n1}/{TotalSize:n1}mb, {Rate:n1}mb/s, {Pct}%)", stats.Count, stats.Size / (1024.0 * 1024.0), _totalSize / (1024.0 * 1024.0), stats.Rate / (1024.0 * 1024.0), (int)((Math.Max(stats.Size, 1) * 100) / Math.Max(_totalSize, 1)));
			}
			else
			{
				_logger.LogInformation("Copied {NumFiles:n0} files ({Size:n1}mb, {Rate:n1}mb/s)", stats.Count, stats.Size / (1024.0 * 1024.0), stats.Rate / (1024.0 * 1024.0));
			}
		}
	}

	/// <summary>
	/// Describes an update to a file in a directory tree
	/// </summary>
	/// <param name="Path">Path to the file</param>
	/// <param name="Length">Length of the file data</param>
	/// <param name="ModTime">Last modified time for the file</param>
	/// <param name="Flags">Flags for the new file entry</param>
	/// <param name="StreamHash">Hash of the entire stream</param>
	/// <param name="Nodes">Chunked data for the file</param>
	/// <param name="CustomData"></param>
	public record class FileUpdate(string Path, FileEntryFlags Flags, long Length, IoHash StreamHash, List<ChunkedDataNodeRef> Nodes, DateTime ModTime = default, ReadOnlyMemory<byte> CustomData = default)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public FileUpdate(string path, FileEntryFlags flags, long length, ChunkedData chunkedData, DateTime modTime = default, ReadOnlyMemory<byte> customData = default)
			: this(path, flags, length, chunkedData.StreamHash, [chunkedData.Root], modTime, customData)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FileUpdate(string path, FileEntryFlags flags, long length, LeafChunkedData chunkedData, DateTime modTime = default, ReadOnlyMemory<byte> customData = default)
			: this(path, flags, length, chunkedData.Hash, chunkedData.LeafHandles, modTime, customData)
		{
		}

		/// <summary>
		/// Writes interior node data to the given writer
		/// </summary>
		public async ValueTask WriteInteriorNodesAsync(IBlobWriter writer, InteriorChunkedDataNodeOptions interiorNodeOptions, CancellationToken cancellationToken)
		{
			if (Nodes.Count > 1)
			{
				ChunkedDataNodeRef rootRef = await InteriorChunkedDataNode.CreateTreeAsync(Nodes, interiorNodeOptions, writer, cancellationToken);
				Nodes.Clear();
				Nodes.Add(rootRef);
			}
		}
	}

	/// <summary>
	/// Describes an update to a directory node
	/// </summary>
	public class DirectoryUpdate
	{
		class ReverseStringComparer : IComparer<string>
		{
			public static ReverseStringComparer Instance { get; } = new ReverseStringComparer();

			public int Compare(string? x, string? y)
				=> -StringComparer.Ordinal.Compare(x, y);
		}

		/// <summary>
		/// Directories to be updated
		/// </summary>
		public SortedDictionary<string, DirectoryUpdate?> Directories { get; } = new SortedDictionary<string, DirectoryUpdate?>(ReverseStringComparer.Instance);

		/// <summary>
		/// Files to be updated
		/// </summary>
		public SortedDictionary<string, FileUpdate?> Files { get; } = new SortedDictionary<string, FileUpdate?>(ReverseStringComparer.Instance);

		/// <summary>
		/// Reset this instance
		/// </summary>
		public void Clear()
		{
			Directories.Clear();
			Files.Clear();
		}

		/// <summary>
		/// Adds a file by path to this object
		/// </summary>
		/// <param name="path">Path to add to</param>
		/// <param name="fileUpdate">Content for the file</param>
		public void AddFile(string path, FileUpdate? fileUpdate)
		{
			string[] fragments = path.Split(['/', '\\']);

			DirectoryUpdate lastDir = this;
			for (int idx = 0; idx + 1 < fragments.Length; idx++)
			{
				DirectoryUpdate? nextDir;
				if (!lastDir.Directories.TryGetValue(fragments[idx], out nextDir) || nextDir == null)
				{
					if (fileUpdate == null)
					{
						return;
					}

					nextDir = new DirectoryUpdate();
					lastDir.Directories.Add(fragments[idx], nextDir);
				}
				lastDir = nextDir;
			}

			lastDir.Files[fragments[^1]] = fileUpdate;
		}

		/// <summary>
		/// Adds a file to the tree
		/// </summary>
		/// <param name="path">Path to the file</param>
		/// <param name="flags">Flags for the new file entry</param>
		/// <param name="length">Length of the file</param>
		/// <param name="modTime">Last modified time for the file</param>
		/// <param name="chunkedData">Chunked data instance</param>
		/// <param name="customData"></param>
		public FileUpdate AddFile(string path, FileEntryFlags flags, long length, LeafChunkedData chunkedData, DateTime modTime = default, ReadOnlyMemory<byte> customData = default)
		{
			string name = path;
			for (int idx = path.Length - 1; idx >= 0; idx--)
			{
				if (path[idx] == Path.DirectorySeparatorChar || path[idx] == Path.AltDirectorySeparatorChar)
				{
					name = path.Substring(idx + 1);
					break;
				}
			}

			FileUpdate update = new(name, flags, length, chunkedData, modTime, customData);
			AddFile(path, update);
			return update;
		}

		/// <summary>
		/// Adds a filtered list of files from disk
		/// </summary>
		/// <param name="files">Files to add</param>
		public void AddFiles(IEnumerable<FileUpdate> files)
		{
			IEnumerator<FileUpdate> enumerator = files.GetEnumerator();
			if (enumerator.MoveNext())
			{
				AddFiles(String.Empty, enumerator);
			}
		}

		bool AddFiles(ReadOnlySpan<char> prefix, IEnumerator<FileUpdate> files)
		{
			for (; ; )
			{
				FileUpdate fileUpdate = files.Current;
				if (fileUpdate.Path.Length < prefix.Length || !fileUpdate.Path.AsSpan().StartsWith(prefix, FileSystemReference.Comparison))
				{
					return true;
				}

				int nextDirLength = fileUpdate.Path.AsSpan(prefix.Length).IndexOfAny(Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar);
				if (nextDirLength == -1)
				{
					string fileName = fileUpdate.Path.Substring(prefix.Length);
					Files[fileName] = fileUpdate;

					if (!files.MoveNext())
					{
						return false;
					}
				}
				else
				{
					string dirName = fileUpdate.Path.Substring(prefix.Length, nextDirLength);

					DirectoryUpdate? nextTree;
					if (!Directories.TryGetValue(dirName, out nextTree) || nextTree == null)
					{
						nextTree = new DirectoryUpdate();
						Directories[dirName] = nextTree;
					}

					ReadOnlySpan<char> nextPrefix = fileUpdate.Path.AsSpan(0, prefix.Length + nextDirLength + 1);
					if (!nextTree.AddFiles(nextPrefix, files))
					{
						return false;
					}
				}
			}
		}

		/// <summary>
		/// Writes interior node data to the given writer
		/// </summary>
		public async ValueTask WriteInteriorNodesAsync(IBlobWriter writer, InteriorChunkedDataNodeOptions interiorNodeOptions, CancellationToken cancellationToken)
		{
			foreach (DirectoryUpdate? directoryUpdate in Directories.Values)
			{
				if (directoryUpdate != null)
				{
					await directoryUpdate.WriteInteriorNodesAsync(writer, interiorNodeOptions, cancellationToken);
				}
			}
			foreach (FileUpdate? fileUpdate in Files.Values)
			{
				if (fileUpdate != null)
				{
					await fileUpdate.WriteInteriorNodesAsync(writer, interiorNodeOptions, cancellationToken);
				}
			}
		}
	}

	/// <summary>
	/// Extension methods for directory node updates
	/// </summary>
	public static class DirectoryNodeUpdateExtensions
	{
		/// <summary>
		/// Writes a tree of files to a storage writer
		/// </summary>
		public static async Task<IHashedBlobRef<DirectoryNode>> AddFilesAsync(this IBlobWriter writer, DirectoryReference baseDir, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			DirectoryNode outputNode = new();
			await outputNode.AddFilesAsync(baseDir, DirectoryReference.EnumerateFiles(baseDir, "*", SearchOption.AllDirectories), writer, options, progress, cancellationToken);
			return await writer.WriteBlobAsync(outputNode, cancellationToken: cancellationToken);
		}

		/// <summary>
		/// Writes a tree of files to a storage writer
		/// </summary>
		public static async Task<IHashedBlobRef<DirectoryNode>> AddFilesAsync(this IBlobWriter writer, DirectoryInfo baseDir, IReadOnlyList<FileInfo> files, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			DirectoryNode outputNode = new();
			await outputNode.AddFilesAsync(baseDir, files, writer, options, progress, cancellationToken);
			return await writer.WriteBlobAsync(outputNode, cancellationToken: cancellationToken);
		}

		/// <summary>
		/// Writes a tree of files to a storage writer
		/// </summary>
		public static async Task<IHashedBlobRef<DirectoryNode>> AddFilesAsync(this IBlobWriter writer, DirectoryReference baseDir, IReadOnlyList<FileReference> files, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			DirectoryNode outputNode = new();
			await outputNode.AddFilesAsync(baseDir, files, writer, options, progress, cancellationToken);
			return await writer.WriteBlobAsync(outputNode, cancellationToken: cancellationToken);
		}

		/// <inheritdoc cref="AddFilesAsync(DirectoryNode, DirectoryReference, IEnumerable{FileInfo}, IBlobWriter, ChunkingOptions?, IProgress{IUpdateStats}?, CancellationToken)"/>
		public static async Task AddFilesAsync(this DirectoryNode directoryNode, DirectoryInfo directoryInfo, IBlobWriter writer, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			await directoryNode.AddFilesAsync(new DirectoryReference(directoryInfo), directoryInfo.EnumerateFiles("*", SearchOption.AllDirectories).ToList(), writer, options, progress, cancellationToken);
		}

		/// <inheritdoc cref="AddFilesAsync(DirectoryNode, DirectoryReference, IEnumerable{FileInfo}, IBlobWriter, ChunkingOptions?, IProgress{IUpdateStats}?, CancellationToken)"/>
		public static Task AddFilesAsync(this DirectoryNode directoryNode, DirectoryReference baseDir, IEnumerable<FileReference> files, IBlobWriter writer, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			return directoryNode.AddFilesAsync(baseDir, files.Select(x => x.ToFileInfo()).ToList(), writer, options, progress, cancellationToken);
		}

		/// <inheritdoc cref="AddFilesAsync(DirectoryNode, DirectoryReference, IEnumerable{FileInfo}, IBlobWriter, ChunkingOptions?, IProgress{IUpdateStats}?, CancellationToken)"/>
		public static Task AddFilesAsync(this DirectoryNode directoryNode, DirectoryInfo baseDir, IEnumerable<FileInfo> files, IBlobWriter writer, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			return directoryNode.AddFilesAsync(new DirectoryReference(baseDir), files.ToList(), writer, options, progress, cancellationToken);
		}

		/// <summary>
		/// Updates this tree of directory objects
		/// </summary>
		/// <param name="directoryNode">Directory to update</param>
		/// <param name="updates">Files to add</param>
		/// <param name="writer">Writer for new node data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static Task UpdateAsync(this DirectoryNode directoryNode, IEnumerable<FileUpdate> updates, IBlobWriter writer, CancellationToken cancellationToken = default)
		{
			DirectoryUpdate update = new();
			update.AddFiles(updates);
			return directoryNode.UpdateAsync(update, writer, cancellationToken);
		}

		/// <summary>
		/// Updates this tree of directory objects
		/// </summary>
		/// <param name="directoryNode">Directory to update</param>
		/// <param name="update">Files to add</param>
		/// <param name="writer">Writer for new node data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task UpdateAsync(this DirectoryNode directoryNode, DirectoryUpdate update, IBlobWriter writer, CancellationToken cancellationToken = default)
		{
			foreach ((string name, DirectoryUpdate? directory) in update.Directories)
			{
				if (directory == null)
				{
					directoryNode.DeleteDirectory(name);
				}
				else
				{
					DirectoryNode? childNode = await directoryNode.TryOpenDirectoryAsync(name, cancellationToken);
					childNode ??= new DirectoryNode();
					await childNode.UpdateAsync(directory, writer, cancellationToken);
					IHashedBlobRef<DirectoryNode> handle = await writer.WriteBlobAsync<DirectoryNode>(childNode, cancellationToken);
					directoryNode.DeleteDirectory(name);
					directoryNode.AddDirectory(new DirectoryEntry(name, childNode.Length, handle));
				}
			}
			foreach ((string name, FileUpdate? file) in update.Files)
			{
				directoryNode.DeleteFile(name);
				if (file != null)
				{
					await file.WriteInteriorNodesAsync(writer, new ChunkingOptions().InteriorOptions, cancellationToken);
					directoryNode.AddFile(new FileEntry(name, file.Flags, file.Length, file.StreamHash, file.Nodes[0], file.ModTime, file.CustomData));
				}
			}
		}

		/// <summary>
		/// Adds files from a directory to the storage
		/// </summary>
		/// <param name="directoryNode">Directory to add to</param>
		/// <param name="baseDir">Base directory to base paths relative to</param>
		/// <param name="files">Files to add</param>
		/// <param name="options">Options for chunking file content</param>
		/// <param name="writer">Writer for new node data</param>
		/// <param name="progress">Feedback interface for progress updates</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task AddFilesAsync(this DirectoryNode directoryNode, DirectoryReference baseDir, IEnumerable<FileInfo> files, IBlobWriter writer, ChunkingOptions? options = null, IProgress<IUpdateStats>? progress = null, CancellationToken cancellationToken = default)
		{
			options ??= new ChunkingOptions();

			UpdateStats? updateStats = null;
			if (progress != null)
			{
				updateStats = new UpdateStats(progress);
			}

			DirectoryUpdate update = new();

			// Process the input sequence in 2gb batches
			const long MaxBatchSize = 2 * 1024 * 1024 * 1024L;
			using (IEnumerator<FileInfo> fileEnumerator = files.GetEnumerator())
			{
				List<FileInfo> batch = [];
				for (bool moreData = fileEnumerator.MoveNext(); moreData;)
				{
					batch.Clear();

					// Take the next batch of files
					long batchSize = 0;
					while (batchSize < MaxBatchSize && moreData)
					{
						FileInfo file = fileEnumerator.Current;
						batch.Add(file);
						batchSize += file.Length;
						moreData = fileEnumerator.MoveNext();
					}

					// Partition them up into parallel writers
					List<(int Start, int Count)> partitions = ComputePartitions(batch, batchSize);
					LeafChunkedData[] leafChunkedFiles = new LeafChunkedData[batch.Count];
					if (partitions.Count == 1)
					{
						await CreateLeafChunkNodesAsync(writer, batch, leafChunkedFiles, 0, batch.Count, updateStats, options, cancellationToken);
					}
					else
					{
						await Parallel.ForEachAsync(partitions, cancellationToken, (filePartition, ctx) => CreateLeafChunkNodesAsync(writer, batch, leafChunkedFiles, filePartition.Start, filePartition.Count, updateStats, options, cancellationToken));
					}

					// Write all the interior nodes and generate the directory update
					for (int idx = 0; idx < batch.Count; idx++)
					{
						FileInfo file = batch[idx];

						FileEntryFlags flags = FileEntry.GetPermissions(file);
						FileUpdate entry = new(file.Name, flags, file.Length, leafChunkedFiles[idx], file.LastWriteTimeUtc);
						update.AddFile(new FileReference(file).MakeRelativeTo(baseDir), entry);
					}
				}
			}

			// Add all the new entries to the tree
			await directoryNode.UpdateAsync(update, writer, cancellationToken);
		}

		static List<(int Index, int Count)> ComputePartitions(IReadOnlyList<FileInfo> files, long totalSize)
		{
			// Maximum number of streams to write in parallel
			const int MaxPartitions = 16;

			// Minimum size of output payload for each writer
			const long MinSizePerPartition = 1024 * 1024;

			// Calculate the number of writers
			int numPartitions = 1 + (int)Math.Min(totalSize / MinSizePerPartition, MaxPartitions);

			// Create the partitions
			List<(int Index, int Count)> partitions = [];

			long writtenSize = 0;
			long remainingSizeForPartition = 0;

			int startIndex = 0;
			for (int index = 0; index < files.Count; index++)
			{
				FileInfo file = files[index];
				if (file.Length >= remainingSizeForPartition && partitions.Count < numPartitions)
				{
					remainingSizeForPartition = (totalSize - writtenSize) / (numPartitions - partitions.Count);
					partitions.Add((startIndex, index - startIndex));
					startIndex = index;
				}

				writtenSize += file.Length;
				remainingSizeForPartition -= file.Length;
			}

			partitions.Add((startIndex, files.Count - startIndex));
			return partitions;
		}

		static async ValueTask CreateLeafChunkNodesAsync(IBlobWriter writer, IReadOnlyList<FileInfo> files, LeafChunkedData[] leafChunks, int start, int count, UpdateStats? updateStats, ChunkingOptions options, CancellationToken cancellationToken)
		{
			await using IBlobWriter writerFork = writer.Fork();

			byte[] buffer = new byte[20 * 1024 * 1024];
			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				ContentChunker chunker = new ParallelBuzHashChunker(pipeline, buffer, options.LeafOptions);
				_ = pipeline.AddTask(ctx => WriteInputsAsync(files, start, count, chunker.SourceWriter, ctx));
				_ = pipeline.AddTask(ctx => HandleOutputsAsync(chunker.OutputReader, writerFork, leafChunks, updateStats, ctx));
				await pipeline.WaitForCompletionAsync();
			}

			await writerFork.FlushAsync(cancellationToken);
		}

		static async Task WriteInputsAsync(IReadOnlyList<FileInfo> files, int start, int count, ChannelWriter<ChunkerSource> sourceWriter, CancellationToken cancellationToken)
		{
			for (int idx = start; idx < start + count; idx++)
			{
				FileChunkerSource source = new(files[idx], idx);
				await sourceWriter.WriteAsync(source, cancellationToken);
			}
			sourceWriter.Complete();
		}

		static async Task HandleOutputsAsync(ChannelReader<ChunkerOutput> outputReader, IBlobWriter writerFork, LeafChunkedData[] leafChunks, UpdateStats? updateStats, CancellationToken cancellationToken)
		{
			while (await outputReader.WaitToReadAsync(cancellationToken))
			{
				ChunkerOutput? output;
				if (outputReader.TryRead(out output))
				{
					int idx = (int)output.UserData!;
					leafChunks[idx] = await LeafChunkedDataNode.CreateFromChunkerOutputAsync(writerFork, output, updateStats, cancellationToken);
				}
			}
		}

		/// <summary>
		/// Copies entries from a zip file
		/// </summary>
		/// <param name="directoryNode">Directory to update</param>
		/// <param name="stream">Input stream</param>
		/// <param name="writer">Writer for new nodes</param>
		/// <param name="options"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task CopyFromZipStreamAsync(this DirectoryNode directoryNode, Stream stream, IBlobWriter writer, ChunkingOptions options, CancellationToken cancellationToken = default)
		{
			// Create all the leaf nodes
			List<(ZipArchiveEntry, LeafChunkedData)> entries = [];
			using (ZipArchive archive = new(stream, ZipArchiveMode.Read, true))
			{
				foreach (ZipArchiveEntry entry in archive.Entries)
				{
					if (entry.Name.Length > 0)
					{
#pragma warning disable CA1849 // ZipArchiveEntry.Open() has no async equivalent
						using (Stream entryStream = entry.Open())
#pragma warning restore CA1849
						{
							LeafChunkedData leafChunkedData = await LeafChunkedDataNode.CreateFromStreamAsync(writer, entryStream, options.LeafOptions, cancellationToken);
							entries.Add((entry, leafChunkedData));
						}
					}
				}
			}

			// Create all the interior nodes
			List<FileUpdate> updates = [];
			foreach ((ZipArchiveEntry entry, LeafChunkedData leafChunkedFile) in entries)
			{
				FileEntryFlags flags = FileEntryFlags.None;
				if ((entry.ExternalAttributes & (0b_001_001_001 << 16)) != 0)
				{
					flags |= FileEntryFlags.Executable;
				}

				ChunkedData chunkedFile = await InteriorChunkedDataNode.CreateTreeAsync(leafChunkedFile, options.InteriorOptions, writer, cancellationToken);
				updates.Add(new FileUpdate(entry.FullName, flags, entry.Length, chunkedFile, entry.LastWriteTime.UtcDateTime));
			}

			// Update the tree
			await directoryNode.UpdateAsync(updates, writer, cancellationToken);
		}
	}
}
