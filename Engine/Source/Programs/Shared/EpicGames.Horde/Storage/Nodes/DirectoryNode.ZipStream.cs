// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.IO.Compression;
using System.IO.Pipelines;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Stream which zips a directory node tree dynamically
	/// </summary>
	class DirectoryNodeZipStream : Stream
	{
		readonly Pipe _pipe;
		readonly ILogger? _logger;
		readonly BackgroundTask _backgroundTask;

		long _position;
		ReadOnlySequence<byte> _current = ReadOnlySequence<byte>.Empty;

		/// <inheritdoc/>
		public override bool CanRead => true;

		/// <inheritdoc/>
		public override bool CanSeek => false;

		/// <inheritdoc/>
		public override bool CanWrite => false;

		/// <inheritdoc/>
		public override long Length => throw new NotImplementedException();

		/// <inheritdoc/>
		public override long Position { get => _position; set => throw new NotImplementedException(); }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="node">Root node to copy from</param>
		/// <param name="filter">Filter for files to include in the zip</param>
		/// <param name="logger">Optional logger for debug tracing</param>
		public DirectoryNodeZipStream(IBlobRef<DirectoryNode> node, FileFilter? filter, ILogger? logger)
		{
			_pipe = new Pipe();
			_backgroundTask = BackgroundTask.StartNew(ctx => CopyToPipeAsync(node, filter, _pipe.Writer, logger, ctx));
			_logger = logger;
		}

		/// <inheritdoc/>
		protected override void Dispose(bool disposing)
		{
#pragma warning disable VSTHRD002
			if (disposing)
			{
				_backgroundTask.DisposeAsync().AsTask().Wait();
			}
#pragma warning restore VSTHRD002

			base.Dispose(disposing);
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();

			await base.DisposeAsync();
		}

		/// <inheritdoc/>
		public override async ValueTask<int> ReadAsync(Memory<byte> buffer, CancellationToken cancellationToken = default)
		{
			while (_current.Length == 0)
			{
				ReadResult result = await _pipe.Reader.ReadAsync(cancellationToken);
				_current = result.Buffer;

				if (result.IsCompleted && _current.Length == 0)
				{
					// Wait for the background thread to finish; it may error/have errored, and we want to re-throw on this thread before completing the read.
					await _backgroundTask.StopAsync(cancellationToken);
					_logger?.LogInformation("Zip file pipe was read to end");
					return 0;
				}
			}

			int initialSize = buffer.Length;
			while (buffer.Length > 0 && _current.Length > 0)
			{
				int copy = Math.Min(buffer.Length, _current.First.Length);
				_current.First.Slice(0, copy).CopyTo(buffer);
				_current = _current.Slice(copy);
				buffer = buffer.Slice(copy);
			}

			if (_current.Length == 0)
			{
				_pipe.Reader.AdvanceTo(_current.End);
			}

			int length = initialSize - buffer.Length;
			_position += length;
			return length;
		}

		static async Task CopyToPipeAsync(IBlobRef<DirectoryNode> node, FileFilter? filter, PipeWriter writer, ILogger? logger, CancellationToken cancellationToken)
		{
			using Stream outputStream = writer.AsStream();
			using ZipArchive archive = new(outputStream, ZipArchiveMode.Create);
			await CopyFilesAsync(node, "", filter, archive, logger, cancellationToken);
		}

		static async Task CopyFilesAsync(IBlobRef<DirectoryNode> directoryRef, string prefix, FileFilter? filter, ZipArchive archive, ILogger? logger, CancellationToken cancellationToken)
		{
			DirectoryNode directory = await directoryRef.ReadBlobAsync(cancellationToken);

			int numDirs = directory.Directories.Count;
			int numFiles = directory.Files.Count;

			int numCopiedDirs = 0;
			foreach (DirectoryEntry directoryEntry in directory.Directories)
			{
				string directoryPath = $"{prefix}{directoryEntry.Name}/";
				if (filter == null || filter.PossiblyMatches(directoryPath))
				{
					await CopyFilesAsync(directoryEntry.Handle, directoryPath, filter, archive, logger, cancellationToken);
				}
				numCopiedDirs++;
			}

			int numCopiedFiles = 0;
			foreach (FileEntry fileEntry in directory.Files)
			{
				string filePath = $"{prefix}{fileEntry}";
				if (filter == null || filter.Matches(filePath))
				{
					ZipArchiveEntry entry = archive.CreateEntry(filePath);

					if ((fileEntry.Flags & FileEntryFlags.Executable) != 0)
					{
						entry.ExternalAttributes |= 0b_111_111_101 << 16; // rwx rwx r-x
					}
					else
					{
						entry.ExternalAttributes |= 0b_110_110_100 << 16; // rw- rw- r--
					}

#pragma warning disable CA1849 // ZipArchiveEntry.Open() has no async equivalent
					using Stream entryStream = entry.Open();
#pragma warning restore CA1849
					await fileEntry.CopyToStreamAsync(entryStream, cancellationToken);
				}
				numCopiedFiles++;
			}

			logger?.LogInformation("Zip file path {Prefix} has {NumCopiedDirectories}/{NumDirectories} directories, {NumCopiedFiles}/{NumFiles} files", prefix, numCopiedDirs, numDirs, numCopiedFiles, numFiles);
		}

		/// <inheritdoc/>
		public override void Flush()
		{
		}

#pragma warning disable VSTHRD002
		/// <inheritdoc/>
		public override int Read(byte[] buffer, int offset, int count) => ReadAsync(buffer.AsMemory(offset, count)).AsTask().Result;
#pragma warning restore VSTHRD002

		/// <inheritdoc/>
		public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void SetLength(long value) => throw new NotSupportedException();

		/// <inheritdoc/>
		public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();
	}

	/// <summary>
	/// Extension methods for <see cref="DirectoryNodeZipStream"/>
	/// </summary>
	public static class DirectoryNodeZipStreamExtensions
	{
		/// <summary>
		/// Returns a stream containing the zipped contents of this directory
		/// </summary>
		/// <param name="directoryRef">The directory to zip</param>
		/// <param name="filter">Filter for files to include in the zip</param>
		/// <param name="logger">Logger for diagnostic output</param>
		/// <returns>Stream containing zipped archive data</returns>
		public static Stream AsZipStream(this IBlobRef<DirectoryNode> directoryRef, FileFilter? filter = null, ILogger? logger = null)
			=> new DirectoryNodeZipStream(directoryRef, filter, logger);
	}
}
