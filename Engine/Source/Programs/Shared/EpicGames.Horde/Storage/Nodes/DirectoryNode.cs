// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Flags for a directory node
	/// </summary>
	public enum DirectoryFlags
	{
		/// <summary>
		/// No flags specified
		/// </summary>
		None = 0,
	}

	/// <summary>
	/// A directory node
	/// </summary>
	[BlobConverter(typeof(DirectoryNodeConverter))]
	public class DirectoryNode
	{
		/// <summary>
		/// Type of serialized directory node blobs
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{0714EC11-4D07-291A-8AE7-7F86799980D6}");

		readonly SortedDictionary<string, FileEntry> _nameToFileEntry = new(StringComparer.Ordinal);
		readonly SortedDictionary<string, DirectoryEntry> _nameToDirectoryEntry = new(StringComparer.Ordinal);

		/// <summary>
		/// Total size of this directory
		/// </summary>
		public long Length => _nameToFileEntry.Values.Sum(x => x.Length) + _nameToDirectoryEntry.Values.Sum(x => x.Length);

		/// <summary>
		/// Flags for this directory 
		/// </summary>
		public DirectoryFlags Flags { get; }

		/// <summary>
		/// All the files within this directory
		/// </summary>
		public IReadOnlyCollection<FileEntry> Files => _nameToFileEntry.Values;

		/// <summary>
		/// Map of name to file entry
		/// </summary>
		public IReadOnlyDictionary<string, FileEntry> NameToFile => _nameToFileEntry;

		/// <summary>
		/// All the subdirectories within this directory
		/// </summary>
		public IReadOnlyCollection<DirectoryEntry> Directories => _nameToDirectoryEntry.Values;

		/// <summary>
		/// Map of name to file entry
		/// </summary>
		public IReadOnlyDictionary<string, DirectoryEntry> NameToDirectory => _nameToDirectoryEntry;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="flags"></param>
		public DirectoryNode(DirectoryFlags flags = DirectoryFlags.None)
		{
			Flags = flags;
		}

		/// <summary>
		/// Clear the contents of this directory
		/// </summary>
		public void Clear()
		{
			_nameToFileEntry.Clear();
			_nameToDirectoryEntry.Clear();
		}

		/// <summary>
		/// Check whether an entry with the given name exists in this directory
		/// </summary>
		/// <param name="name">Name of the entry to search for</param>
		/// <returns>True if the entry exists</returns>
		public bool Contains(string name) => TryGetFileEntry(name, out _) || TryGetDirectoryEntry(name, out _);

		#region File operations

		/// <summary>
		/// Adds a new file entry to this directory
		/// </summary>
		/// <param name="entry">The entry to add</param>
		public void AddFile(FileEntry entry)
		{
			_nameToFileEntry[entry.Name] = entry;
		}

		/// <summary>
		/// Adds a new file with the given name
		/// </summary>
		/// <param name="name">Name of the new directory</param>
		/// <param name="flags">Flags for the new file</param>
		/// <param name="data">Chunked data for the file</param>
		/// <returns>The new directory object</returns>
		public FileEntry AddFile(string name, FileEntryFlags flags, ChunkedData data)
		{
			FileEntry entry = new(name, flags, data);
			AddFile(entry);
			return entry;
		}

		/// <summary>
		/// Attempts to get a file entry with the given name
		/// </summary>
		/// <param name="name">Name of the file</param>
		/// <returns>Entry for the given name</returns>
		public FileEntry GetFileEntry(string name) => _nameToFileEntry[name];

		/// <summary>
		/// Attempts to get a file entry with the given name
		/// </summary>
		/// <param name="name">Name of the file</param>
		/// <param name="entry">Entry for the file</param>
		/// <returns>True if the file was found</returns>
		public bool TryGetFileEntry(string name, [NotNullWhen(true)] out FileEntry? entry) => _nameToFileEntry.TryGetValue(name, out entry);

		/// <summary>
		/// Opens a file for reading
		/// </summary>
		/// <param name="name">Name of the file to open</param>
		/// <returns>Stream for the file</returns>
		public Stream OpenFile(string name) => GetFileEntry(name).OpenAsStream();

		/// <summary>
		/// Attempts to open a file for reading
		/// </summary>
		/// <param name="name">Name of the file</param>
		/// <returns>File stream, or null if the file does not exist</returns>
		public Stream? TryOpenFile(string name)
		{
			FileEntry? entry;
			if (TryGetFileEntry(name, out entry))
			{
				return entry.OpenAsStream();
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Deletes the file entry with the given name
		/// </summary>
		/// <param name="name">Name of the entry to delete</param>
		/// <returns>True if the entry was found, false otherwise</returns>
		public bool DeleteFile(string name) => _nameToFileEntry.Remove(name);

		/// <summary>
		/// Attempts to get a file entry from a path
		/// </summary>
		/// <param name="path">Path to the directory</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The directory with the given path, or null if it was not found</returns>
		public async ValueTask<FileEntry?> GetFileEntryByPathAsync(string path, CancellationToken cancellationToken = default)
		{
			FileEntry? fileEntry;

			int slashIdx = path.LastIndexOf('/');
			if (slashIdx == -1)
			{
				if (!TryGetFileEntry(path, out fileEntry))
				{
					return null;
				}
			}
			else
			{
				DirectoryNode? directoryNode = await GetDirectoryByPathAsync(path.Substring(0, slashIdx), cancellationToken);
				if (directoryNode == null)
				{
					return null;
				}
				if (!directoryNode.TryGetFileEntry(path.Substring(slashIdx + 1), out fileEntry))
				{
					return null;
				}
			}

			return fileEntry;
		}

		/// <summary>
		/// Attempts to get a directory entry from a path
		/// </summary>
		/// <param name="path">Path to the directory</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The directory with the given path, or null if it was not found</returns>
		public ValueTask<DirectoryNode?> GetDirectoryByPathAsync(string path, CancellationToken cancellationToken = default) => GetDirectoryByPathAsync(this, path, cancellationToken);

		static async ValueTask<DirectoryNode?> GetDirectoryByPathAsync(DirectoryNode directoryNode, string path, CancellationToken cancellationToken = default)
		{
			while (path.Length > 0)
			{
				string directoryName;

				int slashIdx = path.IndexOf('/', StringComparison.Ordinal);
				if (slashIdx == -1)
				{
					directoryName = path;
					path = String.Empty;
				}
				else
				{
					directoryName = path.Substring(0, slashIdx);
					path = path.Substring(slashIdx + 1);
				}

				DirectoryEntry? directoryEntry;
				if (!directoryNode.TryGetDirectoryEntry(directoryName, out directoryEntry))
				{
					return null;
				}

				directoryNode = await directoryEntry.Handle.ReadBlobAsync(cancellationToken);
			}
			return directoryNode;
		}

		/// <summary>
		/// Deletes a file with the given path
		/// </summary>
		/// <param name="path"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async ValueTask<bool> DeleteFileByPathAsync(string path, CancellationToken cancellationToken = default)
		{
			string remainingPath = path;
			for (DirectoryNode? directory = this; directory != null;)
			{
				int length = remainingPath.IndexOf('/', StringComparison.Ordinal);
				if (length == -1)
				{
					return directory.DeleteFile(remainingPath);
				}
				if (length > 0)
				{
					directory = await directory.TryOpenDirectoryAsync(remainingPath.Substring(0, length), cancellationToken);
				}
				remainingPath = remainingPath.Substring(length + 1);
			}
			return false;
		}

		#endregion

		#region Directory operations

		/// <summary>
		/// Adds a new directory with the given name
		/// </summary>
		/// <param name="entry">Name of the new directory</param>
		public void AddDirectory(DirectoryEntry entry)
		{
			if (TryGetFileEntry(entry.Name, out _))
			{
				throw new ArgumentException($"A file with the name '{entry.Name}' already exists in this directory", nameof(entry));
			}

			_nameToDirectoryEntry.Add(entry.Name, entry);
		}

		/// <summary>
		/// Get a directory entry with the given name
		/// </summary>
		/// <param name="name">Name of the directory</param>
		/// <returns>The entry with the given name</returns>
		public DirectoryEntry GetDirectoryEntry(string name) => _nameToDirectoryEntry[name];

		/// <summary>
		/// Attempts to get a directory entry with the given name
		/// </summary>
		/// <param name="name">Name of the directory</param>
		/// <param name="entry">Entry for the directory</param>
		/// <returns>True if the directory was found</returns>
		public bool TryGetDirectoryEntry(string name, [NotNullWhen(true)] out DirectoryEntry? entry) => _nameToDirectoryEntry.TryGetValue(name, out entry);

		/// <summary>
		/// Tries to get a directory with the given name
		/// </summary>
		/// <param name="name">Name of the new directory</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new directory object</returns>
		public async ValueTask<DirectoryNode> OpenDirectoryAsync(string name, CancellationToken cancellationToken = default)
		{
			DirectoryNode? directoryNode = await TryOpenDirectoryAsync(name, cancellationToken);
			if (directoryNode == null)
			{
				throw new DirectoryNotFoundException();
			}
			return directoryNode;
		}

		/// <summary>
		/// Tries to get a directory with the given name
		/// </summary>
		/// <param name="name">Name of the new directory</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new directory object</returns>
		public async ValueTask<DirectoryNode?> TryOpenDirectoryAsync(string name, CancellationToken cancellationToken = default)
		{
			if (TryGetDirectoryEntry(name, out DirectoryEntry? entry))
			{
				return await entry.Handle.ReadBlobAsync(cancellationToken);
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Deletes the file entry with the given name
		/// </summary>
		/// <param name="name">Name of the entry to delete</param>
		/// <returns>True if the entry was found, false otherwise</returns>
		public bool DeleteDirectory(string name) => _nameToDirectoryEntry.Remove(name);

		#endregion
	}

	class DirectoryNodeConverter : BlobConverter<DirectoryNode>
	{
		readonly HordeApiVersion _writeVersion;

		/// <summary>
		/// Constructor
		/// </summary>
		public DirectoryNodeConverter()
			: this(HordeApiVersion.AddRollingHashesForLeafNodes)
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public DirectoryNodeConverter(HordeApiVersion writeVersion)
			=> _writeVersion = writeVersion;

		/// <inheritdoc/>
		public override DirectoryNode Read(IBlobReader reader, BlobSerializerOptions options)
		{
			DirectoryNode directoryNode = new((DirectoryFlags)reader.ReadUnsignedVarInt());

			int fileCount = (int)reader.ReadUnsignedVarInt();
			for (int idx = 0; idx < fileCount; idx++)
			{
				IHashedBlobRef<ChunkedDataNode> targetHandle = reader.ReadBlobRef<ChunkedDataNode>();

				ChunkedDataNodeType targetType = ChunkedDataNodeType.Unknown;
				if (reader.Version >= 2) // Pre-unification with HordeApiVersion
				{
					targetType = (ChunkedDataNodeType)reader.ReadUnsignedVarInt();
				}

				uint rollingHash = 0;
				if (targetType == ChunkedDataNodeType.Leaf && reader.Version >= (int)HordeApiVersion.AddRollingHashesForLeafNodes)
				{
					rollingHash = reader.ReadUInt32();
				}

				string name = reader.ReadString();
				FileEntryFlags flags = (FileEntryFlags)reader.ReadUnsignedVarInt();
				long length = (long)reader.ReadUnsignedVarInt();
				IoHash streamHash = reader.ReadIoHash();
				ChunkedDataNodeRef target = new(targetType, length, rollingHash, targetHandle);

				DateTime modTime = default;
				if ((flags & FileEntryFlags.HasModTime) != 0)
				{
					modTime = new DateTime((long)reader.ReadUnsignedVarInt());
				}

				ReadOnlyMemory<byte> customData = default;
				if ((flags & FileEntryFlags.HasCustomData) != 0)
				{
					customData = reader.ReadVariableLengthBytes();
				}

				directoryNode.AddFile(new FileEntry(name, flags, length, streamHash, target, modTime, customData));
			}

			int directoryCount = (int)reader.ReadUnsignedVarInt();
			for (int idx = 0; idx < directoryCount; idx++)
			{
				IHashedBlobRef<DirectoryNode> directoryHandle = reader.ReadBlobRef<DirectoryNode>();
				long length = (long)reader.ReadUnsignedVarInt();
				string name = reader.ReadString();

				directoryNode.AddDirectory(new DirectoryEntry(name, length, directoryHandle));
			}

			return directoryNode;
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, DirectoryNode value, BlobSerializerOptions options)
		{
			writer.WriteUnsignedVarInt((ulong)value.Flags);

			writer.WriteUnsignedVarInt(value.Files.Count);
			foreach (FileEntry fileEntry in value.Files)
			{
				writer.WriteBlobRef(fileEntry.Target);
				writer.WriteUnsignedVarInt((int)fileEntry.Target.Type);

				if (_writeVersion >= HordeApiVersion.AddRollingHashesForLeafNodes && fileEntry.Target.Type == ChunkedDataNodeType.Leaf)
				{
					writer.WriteUInt32(fileEntry.Target.RollingHash);
				}

				FileEntryFlags flags = fileEntry.Flags & ~(FileEntryFlags.HasCustomData | FileEntryFlags.HasModTime);
				if (fileEntry.CustomData.Length > 0)
				{
					flags |= FileEntryFlags.HasCustomData;
				}
				if (_writeVersion >= HordeApiVersion.AddFileModTimes && fileEntry.ModTime != default)
				{
					flags |= FileEntryFlags.HasModTime;
				}

				writer.WriteString(fileEntry.Name);
				writer.WriteUnsignedVarInt((ulong)flags);
				writer.WriteUnsignedVarInt((ulong)fileEntry.Length);
				writer.WriteIoHash(fileEntry.Hash);

				if ((flags & FileEntryFlags.HasModTime) != 0)
				{
					writer.WriteUnsignedVarInt((ulong)fileEntry.ModTime.Ticks);
				}

				if ((flags & FileEntryFlags.HasCustomData) != 0)
				{
					writer.WriteVariableLengthBytes(fileEntry.CustomData.Span);
				}
			}

			writer.WriteUnsignedVarInt(value.Directories.Count);
			foreach (DirectoryEntry directoryEntry in value.Directories)
			{
				writer.WriteBlobRef(directoryEntry.Handle);
				writer.WriteUnsignedVarInt((ulong)directoryEntry.Length);
				writer.WriteString(directoryEntry.Name);
			}

			return GetBlobType(_writeVersion);
		}

		static BlobType GetBlobType(HordeApiVersion version)
		{
			int blobVersion;
			if (version >= HordeApiVersion.AddRollingHashesForLeafNodes)
			{
				blobVersion = (int)HordeApiVersion.AddRollingHashesForLeafNodes;
			}
			else if (version >= HordeApiVersion.AddFileModTimes)
			{
				blobVersion = (int)HordeApiVersion.AddFileModTimes;
			}
			else
			{
				blobVersion = 2;
			}
			return new BlobType(DirectoryNode.BlobTypeGuid, blobVersion);
		}
	}

	/// <summary>
	/// Reference to a directory node, including the target hash and length
	/// </summary>
	/// <param name="Length">Sum total of all the file lengths in this directory tree</param>
	/// <param name="Handle">Handle to the target node</param>
	public record class DirectoryNodeRef(long Length, IHashedBlobRef<DirectoryNode> Handle);

	/// <summary>
	/// Entry for a directory within a directory node
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public record class DirectoryEntry(string Name, long Length, IHashedBlobRef<DirectoryNode> Handle) : DirectoryNodeRef(Length, Handle)
	{
	}
}
