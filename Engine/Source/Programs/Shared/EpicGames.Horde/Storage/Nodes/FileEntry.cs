// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Flags for a file entry
	/// </summary>
	[Flags]
	public enum FileEntryFlags
	{
		/// <summary>
		/// No other flags set
		/// </summary>
		None = 0,

		/// <summary>
		/// Indicates that the referenced file is executable
		/// </summary>
		Executable = 4,

		/// <summary>
		/// File should be stored as read-only
		/// </summary>
		ReadOnly = 8,

		/// <summary>
		/// File contents are utf-8 encoded text. Client may want to replace line-endings with OS-specific format.
		/// </summary>
		Text = 16,

		/// <summary>
		/// Used to indicate that custom data is included in the output. Used internally for serialization; not exposed to users.
		/// </summary>
		HasCustomData = 32,

		/// <summary>
		/// File should be materialized as UTF-16 (but is stored as a UTF-8 source)
		/// </summary>
		Utf16 = 64,

		/// <summary>
		/// Whether the file entry includes a modification time
		/// </summary>
		HasModTime = 128,
	}

	/// <summary>
	/// Entry for a file within a directory node
	/// </summary>
	public sealed class FileEntry
	{
		/// <summary>
		/// Name of this file
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Flags for this file
		/// </summary>
		public FileEntryFlags Flags { get; set; }

		/// <summary>
		/// Length of this entry
		/// </summary>
		public long Length { get; }

		/// <summary>
		/// Hash of the file as a contiguous stream. This differs from individual node hashes which hash the Merkle tree of chunks forming it.
		/// </summary>
		public IoHash Hash { get; }

		/// <summary>
		/// Reference to the chunked data for the file
		/// </summary>
		public ChunkedDataNodeRef Target { get; }

		/// <summary>
		/// Last modified time. Only valid if FileEntryFlags.HasModTime is set.
		/// </summary>
		public DateTime ModTime { get; }

		/// <summary>
		/// Custom user data for this file entry
		/// </summary>
		public ReadOnlyMemory<byte> CustomData { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public FileEntry(string name, FileEntryFlags flags, ChunkedData chunkedData, DateTime modTime = default, ReadOnlyMemory<byte> customData = default)
			: this(name, flags, chunkedData.Root.Length, chunkedData.StreamHash, chunkedData.Root, modTime, customData)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FileEntry(string name, FileEntryFlags flags, long length, IoHash hash, ChunkedDataNodeRef target, DateTime modTime = default, ReadOnlyMemory<byte> customData = default)
		{
			flags &= ~(FileEntryFlags.HasModTime | FileEntryFlags.HasCustomData);
			if (modTime != default)
			{
				flags |= FileEntryFlags.HasModTime;
			}
			if (customData.Length > 0)
			{
				flags |= FileEntryFlags.HasCustomData;
			}

			Name = name;
			Flags = flags;
			Length = length;
			Hash = hash;
			Target = target;
			ModTime = modTime;
			CustomData = customData;
		}

		/// <summary>
		/// Creates a stream that returns the contents of this file
		/// </summary>
		/// <returns>The content stream</returns>
		public Stream OpenAsStream()
			=> Target.OpenAsStream();

		/// <summary>
		/// Copies the contents of this node and its children to the given output stream
		/// </summary>
		/// <param name="outputStream">The output stream to receive the data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task CopyToStreamAsync(Stream outputStream, CancellationToken cancellationToken = default)
			=> Target.CopyToStreamAsync(outputStream, cancellationToken);

		/// <summary>
		/// Reads this entire file as a byte array
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<byte[]> ReadAllBytesAsync(CancellationToken cancellationToken = default)
			=> Target.ReadAllBytesAsync(cancellationToken);

		/// <summary>
		/// Extracts the contents of this node to a file
		/// </summary>
		/// <param name="file">File to write with the contents of this node</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task CopyToFileAsync(FileInfo file, CancellationToken cancellationToken)
		{
			try
			{
				await Target.CopyToFileAsync(file, cancellationToken);
				SetPermissions(file, Flags);
			}
			catch (Exception ex) when (ex is not OperationCanceledException)
			{
				throw new Exception($"Unable to extract file {file.FullName}", ex);
			}
		}

		/// <summary>
		/// Get the permission flags from a file on disk
		/// </summary>
		/// <param name="fileInfo">File to check</param>
		/// <returns>Permission flags for the given file</returns>
		public static FileEntryFlags GetPermissions(FileInfo fileInfo)
		{
			FileEntryFlags flags = FileEntryFlags.None;
			if ((fileInfo.Attributes & FileAttributes.ReadOnly) != 0)
			{
				flags |= FileEntryFlags.ReadOnly;
			}

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				int mode = FileUtils.GetFileMode_Linux(fileInfo.FullName);
				if ((mode & ((1 << 0) | (1 << 3) | (1 << 6))) != 0)
				{
					flags |= FileEntryFlags.Executable;
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				int mode = FileUtils.GetFileMode_Mac(fileInfo.FullName);
				if ((mode & ((1 << 0) | (1 << 3) | (1 << 6))) != 0)
				{
					flags |= FileEntryFlags.Executable;
				}
			}
			return flags;
		}

		/// <summary>
		/// Applies the correct permissions to a file for a particular set of file entry flags
		/// </summary>
		/// <param name="fileInfo">File to modify</param>
		/// <param name="flags">Flags for the file</param>
		public static void SetPermissions(FileInfo fileInfo, FileEntryFlags flags)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				int mode = FileUtils.GetFileMode_Linux(fileInfo.FullName);

				int newMode = UpdateFileMode(mode, flags);
				if (mode != newMode)
				{
					FileUtils.SetFileMode_Linux(fileInfo.FullName, (ushort)newMode);
					fileInfo.Refresh();
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				int mode = FileUtils.GetFileMode_Mac(fileInfo.FullName);

				int newMode = UpdateFileMode(mode, flags);
				if (mode != newMode)
				{
					FileUtils.SetFileMode_Mac(fileInfo.FullName, (ushort)newMode);
					fileInfo.Refresh();
				}
			}
			else
			{
				if ((flags & FileEntryFlags.ReadOnly) != 0)
				{
					fileInfo.IsReadOnly = true;
				}
			}
		}

		static int UpdateFileMode(int mode, FileEntryFlags flags)
		{
			if ((flags & FileEntryFlags.ReadOnly) != 0)
			{
				const int WriteMask = 0b_010_010_010;
				mode &= ~WriteMask;
			}
			if ((flags & FileEntryFlags.Executable) != 0)
			{
				const int ExecuteMask = 0b_001_001_001;
				mode |= ExecuteMask;
			}
			return mode;
		}

		/// <inheritdoc/>
		public override string ToString() => Name.ToString();
	}
}
