// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Storage.Nodes
{
	/// <summary>
	/// Shared definitions for <see cref="RedirectNode{T}"/>
	/// </summary>
	public static class RedirectNode
	{
		/// <summary>
		/// Static accessor for the blob type guid
		/// </summary>
		public static Guid BlobTypeGuid { get; } = new Guid("{BE09E54F-47CA-7A6B-2A97-AFBC183B1538}");
	}

	/// <summary>
	/// A node containing arbitrary compact binary data
	/// </summary>
	[BlobConverter(typeof(RedirectNodeConverter<>))]
	public class RedirectNode<T>
	{
		/// <summary>
		/// The target handle
		/// </summary>
		public IHashedBlobRef<T> Target { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="handle">Target node for the redirect</param>
		public RedirectNode(IHashedBlobRef<T> handle) => Target = handle;
	}

	class RedirectNodeConverter<T> : BlobConverter<RedirectNode<T>>
	{
		static readonly BlobType s_blobType = new(RedirectNode.BlobTypeGuid, 1);

		/// <inheritdoc/>
		public override RedirectNode<T> Read(IBlobReader reader, BlobSerializerOptions options)
		{
			IHashedBlobRef<T> handle = reader.ReadBlobRef<T>();
			return new RedirectNode<T>(handle);
		}

		/// <inheritdoc/>
		public override BlobType Write(IBlobWriter writer, RedirectNode<T> value, BlobSerializerOptions options)
		{
			writer.WriteBlobRef(value.Target);
			return s_blobType;
		}
	}
}
