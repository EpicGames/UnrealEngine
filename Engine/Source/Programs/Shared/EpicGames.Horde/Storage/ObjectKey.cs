// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Path to an object within an <see cref="IObjectStore"/>. Object keys are normalized to lowercase, and may consist of the characters [a-z0-9_./].
	/// </summary>
	[JsonSchemaString]
	public readonly struct ObjectKey : IEquatable<ObjectKey>
	{
		/// <summary>
		/// Dummy enum to allow invoking the constructor which takes a sanitized full path
		/// </summary>
		public enum Validate
		{
			/// <summary>
			/// Dummy value
			/// </summary>
			None
		}

		readonly Utf8String _path;

		/// <summary>
		/// Accessor for the internal path string
		/// </summary>
		public Utf8String Path => _path;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Path to the blob. The meaning of this string is implementation defined.</param>
		public ObjectKey(string path)
			: this(new Utf8String(path))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Path to the blob. The meaning of this string is implementation defined.</param>
		public ObjectKey(Utf8String path)
		{
			_path = path;

			if (path.Length > 0)
			{
				if (path[0] == '/' || path[^1] == '/')
				{
					throw new FormatException($"Object locator '{path}' is invalid; locators may not start or end with a slash");
				}
				for (int idx = 0; idx < path.Length; idx++)
				{
					byte character = path[idx];
					if (!StringId.IsValidCharacter(character))
					{
						if (character >= 'A' && character <= 'Z')
						{
							// Allow uppercase characters for now
						}
						else if (character == '+')
						{
							// Allow plus characters for now
						}
						else if (path[idx] == '/' && path[idx - 1] != '/')
						{
							// Non-consecutive path separator; allowed.
						}
						else
						{
							throw new FormatException($"Object locator '{path}' is invalid; character '{(char)path[idx]}' is not allowed");
						}
					}
				}
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path">Path to the blob. The meaning of this string is implementation defined.</param>
		/// <param name="validate"></param>
		public ObjectKey(Utf8String path, Validate validate)
		{
			_path = path;
			_ = validate;
		}

		/// <summary>
		/// Makes an object key from the given path
		/// </summary>
		public static ObjectKey Sanitize(Utf8String path)
		{
			byte[]? data = null;
			for (int idx = 0; idx < path.Length; idx++)
			{
				byte character = path[idx];
				if (!StringId.IsValidCharacter(character) && character != '/')
				{
					data ??= path.Memory.ToArray();
					if (character >= 'A' && character <= 'Z')
					{
						data[idx] = (byte)('a' + (character - 'A'));
					}
					else
					{
						data[idx] = (byte)'_';
					}
				}
			}
			return new ObjectKey((data == null) ? path : new Utf8String(data));
		}

		/// <summary>
		/// Whether the blob locator is valid
		/// </summary>
		public bool IsValid() => !_path.IsEmpty;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is BlobLocator blobId && Equals(blobId);

		/// <inheritdoc/>
		public override int GetHashCode() => _path.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(ObjectKey other) => _path == other._path;

		/// <inheritdoc/>
		public override string ToString() => _path.ToString();

		/// <inheritdoc/>
		public static bool operator ==(ObjectKey left, ObjectKey right) => left.Path == right.Path;

		/// <inheritdoc/>
		public static bool operator !=(ObjectKey left, ObjectKey right) => !(left == right);
	}
}
