// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.ObjectStores
{
	/// <summary>
	/// Storage backend that utilizes the local filesystem
	/// </summary>
	public sealed class FileObjectStore : IObjectStore
	{
		readonly DirectoryReference _baseDir;
		readonly MemoryMappedFileCache _mappedFileCache;

		/// <summary>
		/// Accessor for the base directory
		/// </summary>
		public DirectoryReference BaseDir => _baseDir;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="baseDir">Base directory for the store</param>
		/// <param name="mappedFileCache">Cache for memory mapped files</param>
		public FileObjectStore(DirectoryReference baseDir, MemoryMappedFileCache mappedFileCache)
		{
			_baseDir = baseDir;
			_mappedFileCache = mappedFileCache;

			DirectoryReference.CreateDirectory(_baseDir);
		}

		/// <summary>
		/// Gets the path for storing a file on disk
		/// </summary>
		FileReference GetBlobFile(ObjectKey key) => FileReference.Combine(_baseDir, key.Path.ToString());

		/// <inheritdoc/>
		public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken)
		{
#pragma warning disable CA2000 // Dispose objects before losing scope
			IReadOnlyMemoryOwner<byte> storageObject = await ReadAsync(key, offset, length, cancellationToken);
			return storageObject.AsStream();
#pragma warning restore CA2000 // Dispose objects before losing scope
		}

		/// <summary>
		/// Maps a file into memory for reading, and returns a handle to it
		/// </summary>
		/// <param name="key">Path to the file</param>
		/// <param name="offset">Offset of the data to retrieve</param>
		/// <param name="length">Length of the data</param>
		/// <returns>Handle to the data. Must be disposed by the caller.</returns>
		public IReadOnlyMemoryOwner<byte> Read(ObjectKey key, int offset, int? length)
		{
			FileReference file = GetBlobFile(key);
			return _mappedFileCache.Read(file, offset, length);
		}

		/// <inheritdoc/>
		public Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(Read(key, offset, length));
		}

		/// <inheritdoc/>
		public async Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
		{
			FileReference finalLocation = GetBlobFile(key);
			DirectoryReference.CreateDirectory(finalLocation.Directory);
			FileReference tempLocation = new($"{finalLocation}.tmp");

			using (Stream outputStream = FileReference.Open(tempLocation, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				await stream.CopyToAsync(outputStream, cancellationToken);
			}

			// Move the temp file into place
			try
			{
				FileReference.Move(tempLocation, finalLocation, true);
			}
			catch // Already exists
			{
				if (FileReference.Exists(finalLocation))
				{
					FileReference.Delete(tempLocation);
				}
				else
				{
					throw;
				}
			}
		}

		/// <inheritdoc/>
		public Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			FileReference location = GetBlobFile(key);
			return Task.FromResult(FileReference.Exists(location));
		}

		/// <inheritdoc/>
		public Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			FileInfo info = GetBlobFile(key).ToFileInfo();
			if (info.Exists)
			{
				return Task.FromResult<long>(info.Length);
			}
			else
			{
				return Task.FromResult<long>(-1);
			}
		}

		/// <summary>
		/// Delete a file from the store
		/// </summary>
		/// <param name="key"></param>
		public void Delete(ObjectKey key)
		{
			FileReference location = GetBlobFile(key);
			_mappedFileCache.Delete(location);
		}

		/// <inheritdoc/>
		public Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			Delete(key);
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => default;

		/// <inheritdoc/>
		public void GetStats(StorageStats stats) { }
	}

	/// <summary>
	/// Storage backend that utilizes the local filesystem
	/// </summary>
	public sealed class FileObjectStoreFactory : IDisposable
	{
		readonly MemoryMappedFileCache _mappedFileCache;

		/// <summary>
		/// Constructor
		/// </summary>
		public FileObjectStoreFactory()
		{
			_mappedFileCache = new MemoryMappedFileCache();
		}

		/// <inheritdoc/>
		public void Dispose()
			=> _mappedFileCache.Dispose();

		/// <summary>
		/// Create a new store instance with the given base directory
		/// </summary>
		public FileObjectStore CreateStore(DirectoryReference baseDir)
			=> new(baseDir, _mappedFileCache);
	}
}
