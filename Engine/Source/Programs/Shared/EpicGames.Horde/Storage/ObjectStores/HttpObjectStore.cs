// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.ObjectStores
{
	/// <summary>
	/// Implements an object store utilizing standard HTTP operations
	/// </summary>
	public class HttpObjectStore : IObjectStore
	{
		readonly Uri _baseUrl;
		readonly Func<HttpClient> _createHttpClient;

		/// <inheritdoc/>
		public bool SupportsRedirects => true;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="baseUrl">Base url for the store</param>
		/// <param name="createHttpClient">Method for creating a new http client</param>
		public HttpObjectStore(Uri baseUrl, Func<HttpClient> createHttpClient)
		{
			_baseUrl = baseUrl;
			_createHttpClient = createHttpClient;
		}

		Uri GetUri(ObjectKey key)
			=> new(_baseUrl, key.ToString());

		/// <inheritdoc/>
		public async Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			using HttpClient client = _createHttpClient();

			using HttpResponseMessage response = await client.DeleteAsync(GetUri(key), cancellationToken);
			response.EnsureSuccessStatusCode();
		}

		/// <inheritdoc/>
		public async Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			using HttpClient client = _createHttpClient();

			using HttpRequestMessage request = new(HttpMethod.Head, GetUri(key));
			using HttpResponseMessage response = await client.SendAsync(request, cancellationToken);
			return response.IsSuccessStatusCode;
		}

		/// <inheritdoc/>
		public async Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken = default)
		{
			using HttpClient client = _createHttpClient();

			using HttpRequestMessage request = new(HttpMethod.Head, GetUri(key));
			using HttpResponseMessage response = await client.SendAsync(request, cancellationToken);

			if (response.IsSuccessStatusCode)
			{
				return response.Content.Headers.ContentLength ?? throw new NotSupportedException("Missing Content-Length header");
			}
			else
			{
				return -1;
			}
		}

		/// <inheritdoc/>
		public void GetStats(StorageStats stats)
		{
		}

		/// <inheritdoc/>
		public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			using HttpClient client = _createHttpClient();

			using HttpRequestMessage request = new(HttpMethod.Get, GetUri(key));
			if (offset > 0 || length != null)
			{
				request.Headers.Range = new System.Net.Http.Headers.RangeHeaderValue(offset, length);
			}

			HttpResponseMessage response = await client.SendAsync(request, cancellationToken);
			Stream stream = await response.Content.ReadAsStreamAsync(cancellationToken);
			return stream.WrapOwnership(response);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
		{
			using Stream stream = await OpenAsync(key, offset, length, cancellationToken);

			byte[] data = await stream.ReadAllBytesAsync(cancellationToken);
			return ReadOnlyMemoryOwner.Create(data);
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> new(GetUri(key));

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default)
			=> new(GetUri(key));

		/// <inheritdoc/>
		public async Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
		{
			using HttpClient client = _createHttpClient();

			using HttpRequestMessage request = new(HttpMethod.Put, GetUri(key));
			request.Content = new StreamContent(stream);

			using HttpResponseMessage response = await client.SendAsync(request, cancellationToken);
			response.EnsureSuccessStatusCode();
		}
	}
}
