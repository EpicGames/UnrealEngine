// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.ObjectStores
{
	/// <summary>
	/// In-memory implementation of a storage backend
	/// </summary>
	public sealed class MemoryObjectStore : IObjectStore
	{
		readonly ConcurrentDictionary<ObjectKey, byte[]> _keyToData = new();

		/// <summary>
		/// Read only access to the stored blobs
		/// </summary>
		public IReadOnlyDictionary<ObjectKey, byte[]> Blobs => _keyToData;

		/// <inheritdoc/>
		public bool SupportsRedirects => false;

		/// <inheritdoc/>
		public Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken)
		{
			return Task.FromResult<Stream>(new ReadOnlyMemoryStream(GetData(key, offset, length)));
		}

		/// <inheritdoc/>
		public Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken)
		{
			return Task.FromResult(ReadOnlyMemoryOwner.Create(GetData(key, offset, length)));
		}

		ReadOnlyMemory<byte> GetData(ObjectKey key, int offset, int? length)
		{
			ReadOnlyMemory<byte> data = _keyToData[key].AsMemory(offset);
			if (length != null && length.Value < data.Length)
			{
				data = data.Slice(0, length.Value);
			}
			return data;
		}

		/// <inheritdoc/>
		public async Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default)
		{
			using (MemoryStream buffer = new())
			{
				await stream.CopyToAsync(buffer, cancellationToken);
				_keyToData[key] = buffer.ToArray();
			}
		}

		/// <inheritdoc/>
		public Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			return Task.FromResult(_keyToData.ContainsKey(key));
		}

		/// <inheritdoc/>
		public Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			byte[]? data;
			if (_keyToData.TryGetValue(key, out data))
			{
				return Task.FromResult<long>(data.Length);
			}
			else
			{
				return Task.FromResult<long>(-1);
			}
		}

		/// <inheritdoc/>
		public Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken)
		{
			_keyToData.TryRemove(key, out _);
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => default;

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => default;

		/// <inheritdoc/>
		public void GetStats(StorageStats stats) { }
	}
}
