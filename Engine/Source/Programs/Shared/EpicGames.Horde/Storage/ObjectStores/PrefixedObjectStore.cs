// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Horde.Storage.ObjectStores
{
	/// <summary>
	/// Storage backend wrapper which adds a prefix to the start of each item
	/// </summary>
	public sealed class PrefixedObjectStore : IObjectStore
	{
		readonly Utf8String _prefix;
		readonly IObjectStore _inner;

		/// <inheritdoc/>
		public bool SupportsRedirects => _inner.SupportsRedirects;

		ObjectKey GetKeyWithPrefix(ObjectKey key) => new(_prefix + key.Path);

		/// <summary>
		/// Constructor
		/// </summary>
		public PrefixedObjectStore(string prefix, IObjectStore inner)
		{
#pragma warning disable CA1308 // Use ToUpperInvariant()
			prefix = prefix.TrimEnd('/').ToLowerInvariant();
#pragma warning restore CA1308
			_ = new ObjectKey(prefix); // Validate the syntax
			_prefix = new Utf8String($"{prefix}/");
			_inner = inner;
		}

		/// <inheritdoc/>
		public Task DeleteAsync(ObjectKey locator, CancellationToken cancellationToken = default) => _inner.DeleteAsync(GetKeyWithPrefix(locator), cancellationToken);

		/// <inheritdoc/>
		public Task<bool> ExistsAsync(ObjectKey locator, CancellationToken cancellationToken = default) => _inner.ExistsAsync(GetKeyWithPrefix(locator), cancellationToken);

		/// <inheritdoc/>
		public Task<long> GetSizeAsync(ObjectKey locator, CancellationToken cancellationToken = default) => _inner.GetSizeAsync(GetKeyWithPrefix(locator), cancellationToken);

		/// <inheritdoc/>
		public Task<Stream> OpenAsync(ObjectKey locator, int offset, int? length, CancellationToken cancellationToken = default) => _inner.OpenAsync(GetKeyWithPrefix(locator), offset, length, cancellationToken);

		/// <inheritdoc/>
		public Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey locator, int offset, int? length, CancellationToken cancellationToken = default) => _inner.ReadAsync(GetKeyWithPrefix(locator), offset, length, cancellationToken);

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey locator, CancellationToken cancellationToken = default) => _inner.TryGetReadRedirectAsync(GetKeyWithPrefix(locator), cancellationToken);

		/// <inheritdoc/>
		public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey locator, CancellationToken cancellationToken = default) => _inner.TryGetWriteRedirectAsync(GetKeyWithPrefix(locator), cancellationToken);

		/// <inheritdoc/>
		public Task WriteAsync(ObjectKey locator, Stream stream, CancellationToken cancellationToken = default) => _inner.WriteAsync(GetKeyWithPrefix(locator), stream, cancellationToken);

		/// <inheritdoc/>
		public void GetStats(StorageStats stats) => _inner.GetStats(stats);
	}
}
