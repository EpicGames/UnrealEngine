// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Identifier for a ref in the storage system. Refs serve as GC roots, and are persistent entry points to expanding data structures within the store.
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(RefNameJsonConverter))]
	[TypeConverter(typeof(RefNameTypeConverter))]
	[CbConverter(typeof(RefNameCbConverter))]
	public readonly struct RefName : IEquatable<RefName>, IComparable<RefName>
	{
		/// <summary>
		/// Empty ref name
		/// </summary>
		public static RefName Empty { get; } = default;

		/// <summary>
		/// String for the ref name
		/// </summary>
		public Utf8String Text { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text"></param>
		public RefName(string text)
			: this(new Utf8String(text))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text"></param>
		public RefName(Utf8String text)
		{
			Text = text;
			ValidatePathArgument(text.Span, nameof(text));
		}

		/// <summary>
		/// Validates a given string as a blob id
		/// </summary>
		/// <param name="text">String to validate</param>
		/// <param name="argumentName">Name of the argument</param>
		public static void ValidatePathArgument(ReadOnlySpan<byte> text, string argumentName)
		{
			if (text.Length == 0)
			{
				throw new ArgumentException("Ref names cannot be empty", argumentName);
			}
			if (text[^1] == '/')
			{
				throw new ArgumentException($"{Encoding.UTF8.GetString(text)} is not a valid ref name (cannot start or end with a slash)", argumentName);
			}

			int lastSlashIdx = -1;

			for (int idx = 0; idx < text.Length; idx++)
			{
				if (text[idx] == '/')
				{
					if (lastSlashIdx == idx - 1)
					{
						throw new ArgumentException($"{Encoding.UTF8.GetString(text)} is not a valid ref name (leading and consecutive slashes are not permitted)", argumentName);
					}
					else
					{
						lastSlashIdx = idx;
					}
				}
				else if (text[idx] == '.')
				{
					if (idx == 0 || text[idx - 1] == '/')
					{
						throw new ArgumentException($"{Encoding.UTF8.GetString(text)} is not a valid ref name (path fragment cannot start with a period)", argumentName);
					}
					if (idx + 1 == text.Length || text[idx + 1] == '/')
					{
						throw new ArgumentException($"{Encoding.UTF8.GetString(text)} is not a valid ref name (path fragment cannot end with a period)", argumentName);
					}
				}
				else
				{
					if (!IsValidChar(text[idx]))
					{
						throw new ArgumentException($"{Encoding.UTF8.GetString(text)} is not a valid ref name ('{(char)text[idx]}' is an invalid character)", argumentName);
					}
				}
			}
		}

		static readonly uint[] s_validChars = CreateValidCharsArray();

		static uint[] CreateValidCharsArray()
		{
			const string ValidChars = "0123456789abcdefghijklmnopqrstuvwxyz_/-+.";

			uint[] validChars = new uint[256 / 8];
			for (int idx = 0; idx < ValidChars.Length; idx++)
			{
				int index = ValidChars[idx];
				validChars[index / 32] |= 1U << (index & 31);
			}

			return validChars;
		}

		static bool IsValidChar(byte character)
		{
			return (s_validChars[character / 32] & (1U << (character & 31))) != 0;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is RefName refId && Equals(refId);

		/// <inheritdoc/>
		public override int GetHashCode() => Text.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(RefName refName) => Text == refName.Text;

		/// <inheritdoc/>
		public int CompareTo(RefName other) => Text.CompareTo(other.Text);

		/// <inheritdoc/>
		public override string ToString() => Text.ToString();

		/// <inheritdoc/>
		public static bool operator ==(RefName lhs, RefName rhs) => lhs.Equals(rhs);

		/// <inheritdoc/>
		public static bool operator !=(RefName lhs, RefName rhs) => !lhs.Equals(rhs);

		/// <inheritdoc/>
		public static bool operator <(RefName lhs, RefName rhs) => lhs.CompareTo(rhs) < 0;

		/// <inheritdoc/>
		public static bool operator <=(RefName lhs, RefName rhs) => lhs.CompareTo(rhs) <= 0;

		/// <inheritdoc/>
		public static bool operator >(RefName lhs, RefName rhs) => lhs.CompareTo(rhs) > 0;

		/// <inheritdoc/>
		public static bool operator >=(RefName lhs, RefName rhs) => lhs.CompareTo(rhs) >= 0;

		/// <summary>
		/// Construct a ref from a string
		/// </summary>
		/// <param name="name">Name of the ref</param>
		public static implicit operator RefName(string name) => new(name);
	}

	/// <summary>
	/// Type converter for IoHash to and from JSON
	/// </summary>
	sealed class RefNameJsonConverter : JsonConverter<RefName>
	{
		/// <inheritdoc/>
		public override RefName Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new(new Utf8String(reader.GetUtf8String().ToArray()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, RefName value, JsonSerializerOptions options) => writer.WriteStringValue(value.Text.Span);
	}

	/// <summary>
	/// Type converter from strings to IoHash objects
	/// </summary>
	sealed class RefNameTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object? value)
		{
			return new RefName((string)value!);
		}
	}

	/// <summary>
	/// Type converter to compact binary
	/// </summary>
	sealed class RefNameCbConverter : CbConverter<RefName>
	{
		/// <inheritdoc/>
		public override RefName Read(CbField field) => new(field.AsUtf8String());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, RefName value) => writer.WriteUtf8StringValue(value.Text);

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, RefName value) => writer.WriteUtf8String(name, value.Text);
	}
}
