// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.ObjectStores;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Implementation of a local disk cache which can be shared by multiple backends
	/// </summary>
	public sealed class StorageBackendCache : IDisposable
	{
		class Item
		{
			public ObjectKey Key { get; }
			public long Length { get; }
			public LinkedListNode<Item> ListNode { get; }

			public Item(ObjectKey key, long length)
			{
				Key = key;
				Length = length;
				ListNode = new LinkedListNode<Item>(this);
			}
		}

		class PendingItem
		{
			int _refCount;
			readonly BackgroundTask _readTask;

			public PendingItem(BackgroundTask readTask) => _readTask = readTask;
			public void AddRef() => Interlocked.Increment(ref _refCount);

			public async ValueTask ReleaseAsync()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					await _readTask.DisposeAsync();
				}
			}

			public Task WaitAsync(CancellationToken cancellationToken) => _readTask.WaitAsync(cancellationToken);
		}

		sealed class ObjectStoreWrapper : IObjectStore
		{
			readonly string _keyPrefix;
			readonly StorageBackendCache _cacheStorage;
			readonly IObjectStore _inner;

			public bool SupportsRedirects => _inner.SupportsRedirects;

			public ObjectStoreWrapper(string keyPrefix, StorageBackendCache cacheStorage, IObjectStore inner)
			{
				_keyPrefix = keyPrefix;
				_cacheStorage = cacheStorage;
				_inner = inner;
			}

			public async Task<Stream> OpenAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
			{
				IReadOnlyMemoryOwner<byte> storageObject = await ReadAsync(key, offset, length, cancellationToken);
				return storageObject.AsStream();
			}

			public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, int offset, int? length, CancellationToken cancellationToken = default)
			{
#pragma warning disable CA2000 // Dispose objects before losing scope
				IReadOnlyMemoryOwner<byte> storageObject = await _cacheStorage.ReadAsync(new ObjectKey($"{_keyPrefix}{key}"), ctx => _inner.OpenAsync(key, ctx), cancellationToken);
#pragma warning restore CA2000 // Dispose objects before losing scope
				return storageObject.Slice(offset, length);
			}

			public Task WriteAsync(ObjectKey key, Stream stream, CancellationToken cancellationToken = default) => _inner.WriteAsync(key, stream, cancellationToken);
			public Task DeleteAsync(ObjectKey key, CancellationToken cancellationToken = default) => _inner.DeleteAsync(key, cancellationToken);
			public Task<bool> ExistsAsync(ObjectKey key, CancellationToken cancellationToken = default) => _inner.ExistsAsync(key, cancellationToken);
			public Task<long> GetSizeAsync(ObjectKey key, CancellationToken cancellationToken = default) => _inner.GetSizeAsync(key, cancellationToken);
			public ValueTask<Uri?> TryGetReadRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => _inner.TryGetReadRedirectAsync(key, cancellationToken);
			public ValueTask<Uri?> TryGetWriteRedirectAsync(ObjectKey key, CancellationToken cancellationToken = default) => _inner.TryGetWriteRedirectAsync(key, cancellationToken);

			public void GetStats(StorageStats stats)
			{
				_inner.GetStats(stats);
				_cacheStorage.GetStats(stats);
			}
		}

		sealed class BackendWrapper : IStorageBackend
		{
			public const string BlobExtension = ".blob";

			readonly string _keyPrefix;
			readonly StorageBackendCache _cacheStorage;
			readonly IStorageBackend _inner;

			public bool SupportsRedirects => _inner.SupportsRedirects;

			public BackendWrapper(string keyPrefix, StorageBackendCache cacheStorage, IStorageBackend inner)
			{
				_keyPrefix = keyPrefix.TrimEnd('/');
				if (_keyPrefix.Length > 0)
				{
					_keyPrefix += "/";
				}
				_cacheStorage = cacheStorage;
				_inner = inner;
			}

			public async Task<Stream> OpenBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
			{
				IReadOnlyMemoryOwner<byte> storageObject = await ReadBlobAsync(locator, offset, length, cancellationToken);
				return storageObject.AsStream();
			}

			public async Task<IReadOnlyMemoryOwner<byte>> ReadBlobAsync(BlobLocator locator, int offset, int? length, CancellationToken cancellationToken = default)
			{
#pragma warning disable CA2000 // Dispose objects before losing scope
				IReadOnlyMemoryOwner<byte> storageObject = await _cacheStorage.ReadAsync(new ObjectKey($"{_keyPrefix}{locator}{BlobExtension}"), ctx => _inner.OpenBlobAsync(locator, ctx), cancellationToken);
#pragma warning restore CA2000 // Dispose objects before losing scope
				return storageObject.Slice(offset, length);
			}

			public Task<BlobLocator> WriteBlobAsync(Stream stream, IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default)
				=> _inner.WriteBlobAsync(stream, imports, prefix, cancellationToken);

			public Task WriteBlobAsync(BlobLocator locator, Stream stream, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
				=> _inner.WriteBlobAsync(locator, stream, imports, cancellationToken);

			public ValueTask<Uri?> TryGetBlobReadRedirectAsync(BlobLocator locator, CancellationToken cancellationToken = default)
				=> _inner.TryGetBlobReadRedirectAsync(locator, cancellationToken);

			public ValueTask<Uri?> TryGetBlobWriteRedirectAsync(BlobLocator locator, IReadOnlyCollection<BlobLocator> imports, CancellationToken cancellationToken = default)
				=> _inner.TryGetBlobWriteRedirectAsync(locator, imports, cancellationToken);

			public ValueTask<(BlobLocator, Uri)?> TryGetBlobWriteRedirectAsync(IReadOnlyCollection<BlobLocator> imports, string? prefix = null, CancellationToken cancellationToken = default)
				=> _inner.TryGetBlobWriteRedirectAsync(imports, prefix, cancellationToken);

			public void GetStats(StorageStats stats)
			{
				_inner.GetStats(stats);
				_cacheStorage.GetStats(stats);
			}

			#region Aliases

			public Task<BlobAliasLocator[]> FindAliasesAsync(string name, int? maxResults = null, CancellationToken cancellationToken = default)
				=> _inner.FindAliasesAsync(name, maxResults, cancellationToken);

			#endregion

			#region Refs

			public Task<HashedBlobRefValue?> TryReadRefAsync(RefName name, RefCacheTime cacheTime = default, CancellationToken cancellationToken = default)
				=> _inner.TryReadRefAsync(name, cacheTime, cancellationToken);

			#endregion

			public Task UpdateMetadataAsync(UpdateMetadataRequest request, CancellationToken cancellationToken = default)
				=> _inner.UpdateMetadataAsync(request, cancellationToken);
		}

		private readonly Lock _lockObject = new();

		readonly MemoryMappedFileCache _memoryMappedFileCache;
		readonly FileObjectStore _objectStore;
		readonly long _maxSize;
		readonly ILogger _logger;

		readonly LinkedList<Item> _items = new();
		readonly Dictionary<ObjectKey, Item> _keyToItem = [];
		readonly Dictionary<ObjectKey, PendingItem> _keyToPendingItem = [];

		long _size;
		long _cleanCount;
		long _cleanTimeTicks;
		long _fetchTimeTicks;
		long _writeTimeTicks;
		long _fetchBytes;

		internal IEnumerable<ObjectKey> Items => _items.Select(x => x.Key);

		internal IEnumerable<BlobLocator> GetLocators()
		{
			foreach (Item item in _items)
			{
				ObjectKey key = item.Key;
				if (key.Path.ToString().EndsWith(BackendWrapper.BlobExtension, StringComparison.OrdinalIgnoreCase))
				{
					yield return new BlobLocator(key.Path.Substring(0, key.Path.Length - BackendWrapper.BlobExtension.Length));
				}
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageBackendCache()
			: this(null, null)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageBackendCache(DirectoryReference? cacheDir, long? maxSize)
			: this(cacheDir, maxSize, NullLogger.Instance)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="cacheDir">Directory to storage cache files. Will be cleared on startup. Defaults to a randomly generated directory in the users's temp folder.</param>
		/// <param name="maxSize">Maximum size of the cache. Defaults to 50mb.</param>
		/// <param name="logger">Logger for error/warning messages</param>
		public StorageBackendCache(DirectoryReference? cacheDir, long? maxSize, ILogger logger)
		{
			cacheDir ??= new DirectoryReference(Path.Combine(Path.GetTempPath(), $"horde-{Guid.NewGuid():n}"));
			FileUtils.ForceDeleteDirectoryContents(cacheDir);

			_memoryMappedFileCache = new MemoryMappedFileCache();
			_objectStore = new FileObjectStore(cacheDir, _memoryMappedFileCache);
			_maxSize = maxSize ?? (50 * 1024 * 1024);
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			foreach (Item item in _items)
			{
				_objectStore.Delete(item.Key);
			}
			_memoryMappedFileCache.Dispose();
		}

		/// <summary>
		/// Wraps an onject store in another store that routes requests through the cache
		/// </summary>
		/// <param name="keyPrefix">Prefix for items in this cache</param>
		/// <param name="store">Backend to wrap</param>
		public IObjectStore CreateWrapper(string keyPrefix, IObjectStore store)
		{
			return new ObjectStoreWrapper(keyPrefix, this, store);
		}

		/// <summary>
		/// Wraps a storage backend in another backend that routes requests through the cache
		/// </summary>
		/// <param name="keyPrefix">Prefix for items in this cache</param>
		/// <param name="backend">Backend to wrap</param>
		public IStorageBackend CreateWrapper(string keyPrefix, IStorageBackend backend)
		{
			return new BackendWrapper(keyPrefix, this, backend);
		}

		/// <summary>
		/// Wraps a storage backend in another backend that routes requests through the cache
		/// </summary>
		/// <param name="keyPrefix">Prefix for items in this cache</param>
		/// <param name="backend">Backend to wrap</param>
		/// <param name="cache">The cache instance. May be null.</param>
		public static IStorageBackend CreateWrapper(string keyPrefix, IStorageBackend backend, StorageBackendCache? cache)
		{
			return cache?.CreateWrapper(keyPrefix, backend) ?? backend;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyMemoryOwner<byte>> ReadAsync(ObjectKey key, Func<CancellationToken, Task<Stream>> createStreamAsync, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				PendingItem? pendingItem;
				lock (_lockObject)
				{
					Item? item;
					if (_keyToItem.TryGetValue(key, out item))
					{
						_items.Remove(item.ListNode);
						_items.AddFirst(item.ListNode);
						return _objectStore.Read(item.Key, 0, null);
					}

					if (!_keyToPendingItem.TryGetValue(key, out pendingItem))
					{
						pendingItem = new PendingItem(BackgroundTask.StartNew(x => ReadIntoCacheAsync(key, createStreamAsync, x)));
						_keyToPendingItem.Add(key, pendingItem);
					}

					pendingItem.AddRef();
				}

				try
				{
					await pendingItem.WaitAsync(cancellationToken);
				}
				finally
				{
					await pendingItem.ReleaseAsync();
				}
			}
		}

		async Task ReadIntoCacheAsync(ObjectKey key, Func<CancellationToken, Task<Stream>> createStreamAsync, CancellationToken cancellationToken)
		{
			long openStartTicks = Stopwatch.GetTimestamp();
			using Stream stream = await createStreamAsync(cancellationToken);
			long totalLength = stream.Length;
			long openFinishTicks = Stopwatch.GetTimestamp();
			Interlocked.Add(ref _fetchTimeTicks, openFinishTicks - openStartTicks);

			long cleanStartTicks = openFinishTicks;
			lock (_lockObject)
			{
				_size += totalLength;

				LinkedListNode<Item>? lastNode = _items.Last;
				while (_size > _maxSize && lastNode != null)
				{
					LinkedListNode<Item> node = lastNode;
					lastNode = lastNode.Previous;

					Item item = node.Value;
					try
					{
						_objectStore.Delete(item.Key);

						_keyToItem.Remove(item.Key);
						_items.Remove(node);

						_size -= item.Length;
						_cleanCount++;
					}
					catch (Exception ex)
					{
						_logger.LogDebug(ex, "Unable to delete cache item {Path}: {Message}", item.Key, ex.Message);
					}
				}
			}
			long cleanFinishTicks = Stopwatch.GetTimestamp();
			Interlocked.Add(ref _cleanTimeTicks, cleanFinishTicks - cleanStartTicks);

			long writeStartTicks = cleanFinishTicks;
			await _objectStore.WriteAsync(key, stream, cancellationToken: cancellationToken);
			lock (_lockObject)
			{
				Item item = new(key, totalLength);
				_items.AddFirst(item.ListNode);
				_keyToItem.Add(key, item);

				_keyToPendingItem.Remove(key);
			}
			Interlocked.Add(ref _fetchBytes, totalLength);
			long writeFinishTicks = Stopwatch.GetTimestamp();
			Interlocked.Add(ref _writeTimeTicks, writeFinishTicks - writeStartTicks);
		}

		/// <summary>
		/// Get stats for the cache operation
		/// </summary>
		public void GetStats(StorageStats stats)
		{
			stats.Add("backend.cache.clean_count", _cleanCount);
			stats.Add("backend.cache.fetch_time_ms", (_fetchTimeTicks * 1000) / Stopwatch.Frequency);
			stats.Add("backend.cache.clean_time_ms", (_cleanTimeTicks * 1000) / Stopwatch.Frequency);
			stats.Add("backend.cache.write_time_ms", (_writeTimeTicks * 1000) / Stopwatch.Frequency);
			stats.Add("backend.cache.fetch_bytes", _fetchBytes);
		}

		/// <inheritdoc cref="MemoryMappedFileCache.WriteRefStats(ILogger)"/>
		public void WriteRefStats(ILogger logger)
			=> _memoryMappedFileCache.WriteRefStats(logger);
	}
}
