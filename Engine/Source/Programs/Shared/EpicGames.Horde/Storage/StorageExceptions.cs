// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Base exception for the storage service
	/// </summary>
	public class StorageException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public StorageException(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageException(string message, Exception? innerException)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Exception thrown when an object does not exist
	/// </summary>
	public sealed class ObjectNotFoundException : StorageException
	{
		/// <summary>
		/// Path to the object
		/// </summary>
		public ObjectKey Key { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectNotFoundException(ObjectKey key, Exception? innerException = null)
			: this(key, $"Object '{key}' was not found", innerException)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectNotFoundException(ObjectKey key, string message, Exception? innerException = null)
			: base(message, innerException)
		{
			Key = key;
		}
	}

	/// <summary>
	/// Exception for a ref not existing
	/// </summary>
	public sealed class RefNameNotFoundException : StorageException
	{
		/// <summary>
		/// Name of the missing ref
		/// </summary>
		public RefName Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		public RefNameNotFoundException(RefName name)
			: base($"Ref '{name}' not found")
		{
			Name = name;
		}
	}
}
