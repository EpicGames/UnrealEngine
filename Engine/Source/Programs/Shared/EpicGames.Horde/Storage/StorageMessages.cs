// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

#pragma warning disable CA2227 // Collection properties should be read only

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Response from uploading a bundle
	/// </summary>
	public class WriteBlobResponse
	{
		/// <summary>
		/// Path to the uploaded blob
		/// </summary>
		public string Blob { get; set; } = String.Empty;

		/// <summary>
		/// URL to upload the blob to.
		/// </summary>
		public Uri? UploadUrl { get; set; }

		/// <summary>
		/// Flag for whether the client could use a redirect instead (ie. not post content to the server, and get an upload url back).
		/// </summary>
		public bool? SupportsRedirects { get; set; }
	}

	/// <summary>
	/// Response object for finding an alias
	/// </summary>
	public class FindNodeResponse
	{
		/// <summary>
		/// Locator for the target blob
		/// </summary>
		public BlobLocator Blob { get; set; }

		/// <summary>
		/// Rank of this alias
		/// </summary>
		public int Rank { get; set; }

		/// <summary>
		/// Inline data associated with this alias
		/// </summary>
		public byte[] Data { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public FindNodeResponse()
		{
			Data = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FindNodeResponse(BlobLocator blob, int rank, byte[] data)
		{
			Blob = blob;
			Rank = rank;
			Data = data;
		}
	}

	/// <summary>
	/// Response object for searching for nodes with a given alias
	/// </summary>
	public class FindNodesResponse
	{
		/// <summary>
		/// Hash of the target node
		/// </summary>
		public List<FindNodeResponse> Nodes { get; set; } = [];
	}

	/// <summary>
	/// Request to batch update metadata in the database
	/// </summary>
	public class UpdateMetadataRequest
	{
		/// <summary>
		/// List of aliases to add
		/// </summary>
		public List<AddAliasRequest> AddAliases { get; set; } = [];

		/// <summary>
		/// List of aliases to remove
		/// </summary>
		public List<RemoveAliasRequest> RemoveAliases { get; set; } = [];

		/// <summary>
		/// List of refs to add
		/// </summary>
		public List<AddRefRequest> AddRefs { get; set; } = [];

		/// <summary>
		/// List of refs to remove
		/// </summary>
		public List<RemoveRefRequest> RemoveRefs { get; set; } = [];
	}

	/// <summary>
	/// Request object for adding an alias
	/// </summary>
	public class AddAliasRequest
	{
		/// <summary>
		/// Name of the alias
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Rank for the new alias
		/// </summary>
		public int Rank { get; set; }

		/// <summary>
		/// Data to store with the ref
		/// </summary>
		public byte[]? Data { get; set; }

		/// <summary>
		/// Path to the target blob
		/// </summary>
		public BlobLocator Target { get; set; }
	}

	/// <summary>
	/// Request object for removing an alias
	/// </summary>
	public class RemoveAliasRequest
	{
		/// <summary>
		/// Name of the alias
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Path to the target blob
		/// </summary>
		public BlobLocator Target { get; set; }
	}

	/// <summary>
	/// Request object for writing a ref
	/// </summary>
	public class WriteRefRequest
	{
		/// <summary>
		/// Hash of the target blob
		/// </summary>
		public IoHash Hash { get; set; }

		/// <summary>
		/// Path to the target blob
		/// </summary>
		public BlobLocator Target { get; set; }

		/// <summary>
		/// Locator for the target blob
		/// </summary>
		[Obsolete("Use Target instead")]
		public BlobLocator? Blob { get; set; }

		/// <summary>
		/// Export index for the ref
		/// </summary>
		[Obsolete("Use ExportIdx instead")]
		public int? ExportIdx { get; set; }

		/// <summary>
		/// Inline data associated with the ref
		/// </summary>
		public byte[] Data { get; set; } = [];

		/// <summary>
		/// Options for the ref
		/// </summary>
		public RefOptions? Options { get; set; }
	}

	/// <summary>
	/// Request object for removing a ref
	/// </summary>
	public class AddRefRequest : WriteRefRequest
	{
		/// <summary>
		/// Name of the ref
		/// </summary>
		public RefName RefName { get; set; }
	}

	/// <summary>
	/// Request object for removing a ref
	/// </summary>
	public class RemoveRefRequest
	{
		/// <summary>
		/// Name of the ref
		/// </summary>
		public RefName RefName { get; set; }
	}

	/// <summary>
	/// Response object for reading a ref
	/// </summary>
	public class ReadRefResponse
	{
		/// <summary>
		/// Hash of the target node
		/// </summary>
		public IoHash Hash { get; set; }

		/// <summary>
		/// The target blob
		/// </summary>
		public BlobLocator Target { get; set; }

		/// <summary>
		/// Link to information about the target node
		/// </summary>
		public string Link { get; set; } = String.Empty;

		/// <summary>
		/// Base path for this storage backend
		/// </summary>
		public string BasePath { get; set; } = String.Empty;
	}
}
