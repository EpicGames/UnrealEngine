// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Base exception type for workspace errors
	/// </summary>
	public class WorkspaceException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public WorkspaceException(string message, Exception? innerException = null)
			: base(message, innerException) { }
	}

	/// <summary>
	/// Provides functionality to extract and patch data in a local workspace
	/// </summary>
	public class Workspace
	{
		// Flags for the layers that a file or directory belongs to. Some of these are transient states; others are used
		// to indicate user defined layers.
		[Flags]
		enum LayerFlags : ulong
		{
			// Item is in the cache layer
			Cache = 1,

			// Item has pending actions; state may be inconsistent with disk.
			Pending = 2,

			// The default layer for user operations. Must be the last item in the enum; user layers will be allocated flags beyond this.
			Default = 4,

			// Custom layers
			Custom1 = Default << 1,
			Custom2 = Default << 2,
			Custom3 = Default << 3,
			Custom4 = Default << 4,
			Custom5 = Default << 5,
			Custom6 = Default << 6,
			Custom7 = Default << 7,
			Custom8 = Default << 8,
		}

		// Actions to be performed to an item during a sync
		[Flags]
		enum PendingFlags
		{
			// Item is not modified
			None = 0,

			// Remove this file from the layer
			RemoveFromLayer = 2,

			// Delete this file from disk
			RemoveFromDisk = 4,

			// Move into a new location
			Move = 8,

			// Use this as a source for chunks, Harvest for new files
			Copy = 16,

			// This item overlaps with the location of another output, and must be moved to the cache.
			ForceRenameOrRemove = 32,
		}

		// Maps a user defined layer id to a flag
		[DebuggerDisplay("{Id}")]
		class LayerState
		{
			public WorkspaceLayerId Id { get; }
			public LayerFlags Flag { get; }

			public LayerState(WorkspaceLayerId id, LayerFlags flag)
			{
				Id = id;
				Flag = flag;
			}

			public LayerState(IMemoryReader reader)
			{
				Id = new WorkspaceLayerId(new StringId(reader.ReadUtf8String()));
				Flag = (LayerFlags)reader.ReadUnsignedVarInt();
			}

			public void Write(IMemoryWriter writer)
			{
				writer.WriteUtf8String(Id.Id.Text);
				writer.WriteUnsignedVarInt((ulong)Flag);
			}
		}

		// Tracked state of a directory
		[DebuggerDisplay("{GetPath()}")]
		class DirectoryState
		{
			// Concrete implementation of FileState which can access private members in DirectoryState
			sealed class FileStateImpl : FileState
			{
				public FileStateImpl(DirectoryState parent, string name, IoHash hash, LayerFlags layers)
					: base(parent, name, hash, layers)
				{ }

				public FileStateImpl(DirectoryState parent, IMemoryReader reader)
					: base(parent, reader)
				{ }

				public override void MoveTo(DirectoryState newParent, string newName, LayerFlags newLayers)
				{
					Unlink();

					Parent = newParent;
					Name = newName;
					Layers = newLayers;

					newParent.AddFileInternal(this);
				}

				public override void Unlink()
				{
					if (Parent != null)
					{
						Parent._files.Remove(Name);
						Parent = null!;
					}
				}
			}

			public DirectoryState? Parent { get; private set; }
			public string Name { get; }
			public IReadOnlyCollection<DirectoryState> Directories => _directories.Values;
			public IReadOnlyCollection<FileState> Files => _files.Values;
			public LayerFlags Layers { get; set; }
			public PendingFlags Pending => _pendingFlags;

			PendingFlags _pendingFlags;

			readonly Dictionary<string, DirectoryState> _directories;
			readonly Dictionary<string, FileState> _files;

			public DirectoryState(DirectoryState? parent, string name)
			{
				Parent = parent;
				Name = name;

				_directories = new Dictionary<string, DirectoryState>(FileSystemReference.Comparer);
				_files = new Dictionary<string, FileState>(FileSystemReference.Comparer);
			}

			public DirectoryState(DirectoryState? parent, IMemoryReader reader)
			{
				Parent = parent;
				Name = reader.ReadString();

				int numDirectories = (int)reader.ReadUnsignedVarInt();
				_directories = new Dictionary<string, DirectoryState>(numDirectories, FileSystemReference.Comparer);

				for (int idx = 0; idx < numDirectories; idx++)
				{
					DirectoryState subDirState = new(this, reader);
					_directories.Add(subDirState.Name, subDirState);
				}

				int numFiles = (int)reader.ReadUnsignedVarInt();
				_files = new Dictionary<string, FileState>(numFiles, FileSystemReference.Comparer);

				for (int idx = 0; idx < numFiles; idx++)
				{
					FileState fileState = new FileStateImpl(this, reader);
					_files.Add(fileState.Name, fileState);
				}

				Layers = (LayerFlags)reader.ReadUnsignedVarInt();

				if ((Layers & LayerFlags.Pending) != 0)
				{
					_pendingFlags = (PendingFlags)reader.ReadUnsignedVarInt();
				}
			}

			public void Write(IMemoryWriter writer)
			{
				writer.WriteString(Name);

				writer.WriteUnsignedVarInt(_directories.Count);
				foreach (DirectoryState directory in _directories.Values)
				{
					directory.Write(writer);
				}

				writer.WriteUnsignedVarInt(_files.Count);
				foreach (FileState file in _files.Values)
				{
					file.Write(writer);
				}

				writer.WriteUnsignedVarInt((ulong)Layers);

				if ((Layers & LayerFlags.Pending) != 0)
				{
					writer.WriteUnsignedVarInt((ulong)Pending);
				}
			}

			public void SetPendingAction(PendingFlags pending)
			{
				_pendingFlags |= pending;
				MarkAsDirty();

				foreach (DirectoryState subDirState in Directories)
				{
					subDirState.SetPendingAction(pending);
				}

				foreach (FileState fileState in Files)
				{
					fileState.SetPendingAction(pending);
				}
			}

			public void ClearPendingActions()
			{
				foreach (DirectoryState subDirState in Directories)
				{
					if ((subDirState.Layers & LayerFlags.Pending) != 0)
					{
						subDirState.ClearPendingActions();
					}
				}

				foreach (FileState fileState in Files)
				{
					fileState.Pending = PendingFlags.None;
				}

				_pendingFlags = PendingFlags.None;
				Layers &= ~LayerFlags.Pending;
			}

			public void MarkAsDirty()
			{
				for (DirectoryState? dirState = this; dirState != null; dirState = dirState.Parent)
				{
					if ((dirState.Layers & LayerFlags.Pending) != 0)
					{
						dirState.Layers &= ~LayerFlags.Pending;
					}
					else
					{
						break;
					}
				}
			}

			public FileState AddFile(string name, IoHash hash, LayerFlags layers)
			{
				FileState fileState = new FileStateImpl(this, name, hash, layers);
				AddFileInternal(fileState);
				return fileState;
			}

			void AddFileInternal(FileState fileState)
			{
				if (_directories.TryGetValue(fileState.Name, out DirectoryState? existingDir))
				{
					throw new WorkspaceException($"Directory {existingDir.GetPath()} already exists; cannot replace with file");
				}
				if (!_files.TryAdd(fileState.Name, fileState))
				{
					throw new WorkspaceException($"File {fileState.Name} already exists");
				}
			}

			public bool ContainsFile(string name)
				=> _files.ContainsKey(name);

			public bool TryGetFileCaseSensitive(string name, [NotNullWhen(true)] out FileState? fileState)
				=> _files.TryGetValue(name, out fileState) && name.Equals(fileState.Name, StringComparison.Ordinal);

			public bool TryGetFileCaseInsensitive(string name, [NotNullWhen(true)] out FileState? fileState)
				=> _files.TryGetValue(name, out fileState);

			public DirectoryState? FindDirectory(string name)
			{
				_directories.TryGetValue(name, out DirectoryState? state);
				return state;
			}

			public bool TryGetDirectoryCaseInsensitive(string name, [NotNullWhen(true)] out DirectoryState? directoryState)
				=> _directories.TryGetValue(name, out directoryState);

			public bool TryGetDirectoryCaseSensitive(string name, [NotNullWhen(true)] out DirectoryState? directoryState)
			{
				if (_directories.TryGetValue(name, out DirectoryState? tempDirectoryState) && tempDirectoryState.Name.Equals(name, StringComparison.Ordinal))
				{
					directoryState = tempDirectoryState;
					return true;
				}
				else
				{
					directoryState = null;
					return false;
				}
			}

			public DirectoryState AddDirectory(string name, LayerFlags layer)
			{
				DirectoryState dirState = new(this, name) { Layers = layer };
				_directories.Add(name, dirState);
				return dirState;
			}

			public DirectoryState FindOrAddDirectory(string name, LayerFlags layer)
			{
				DirectoryState? dirState;
				if (TryGetDirectoryCaseSensitive(name, out dirState))
				{
					dirState.Layers |= layer;
					return dirState;
				}
				else
				{
					return AddDirectory(name, layer);
				}
			}

			public void Unlink()
			{
				Parent?._directories.Remove(Name);
				Parent = null;
			}

			public string GetPath()
			{
				StringBuilder builder = new();
				AppendPath(builder);
				return builder.ToString();
			}

			public string GetChildPath(string name)
			{
				StringBuilder builder = new();
				AppendPath(builder);
				AppendChildPath(builder, name);
				return builder.ToString();
			}

			public void AppendPath(StringBuilder builder)
			{
				Parent?.AppendPath(builder);
				AppendChildPath(builder, Name);
			}

			static void AppendChildPath(StringBuilder builder, string name)
			{
				if (name.Length > 0)
				{
					if (builder.Length > 0 && builder[^1] != Path.DirectorySeparatorChar)
					{
						builder.Append(Path.DirectorySeparatorChar);
					}
					builder.Append(name);
				}
			}

			public DirectoryReference GetDirectoryReference(DirectoryReference baseDir)
			{
				StringBuilder builder = new(baseDir.FullName);
				AppendPath(builder);
				return new DirectoryReference(builder.ToString(), DirectoryReference.Sanitize.None);
			}
		}

		// Tracked state of a file
		[DebuggerDisplay("{GetPath()}")]
		abstract class FileState
		{
			public DirectoryState Parent { get; protected set; }
			public string Name { get; protected set; }
			public long Length { get; private set; }
			public long LastModifiedTimeUtc { get; private set; }
			public IoHash Hash { get; }
			public LayerFlags Layers { get; set; }
			public PendingFlags Pending { get; set; }

			protected FileState(DirectoryState parent, string name, IoHash hash, LayerFlags layers)
			{
				Parent = parent;
				Name = name;
				Hash = hash;
				Layers = layers;
			}

			public FileState(DirectoryState parent, IMemoryReader reader)
			{
				Parent = parent;
				Name = reader.ReadString();
				Length = (long)reader.ReadUnsignedVarInt();
				LastModifiedTimeUtc = reader.ReadInt64();
				Hash = reader.ReadIoHash();
				Layers = (LayerFlags)reader.ReadUnsignedVarInt();

				if ((Layers & LayerFlags.Pending) != 0)
				{
					Pending = (PendingFlags)reader.ReadUnsignedVarInt();
				}
			}

			public void Write(IMemoryWriter writer)
			{
				Debug.Assert(Hash != IoHash.Zero);
				writer.WriteString(Name);
				writer.WriteUnsignedVarInt((ulong)Length);
				writer.WriteInt64(LastModifiedTimeUtc);
				writer.WriteIoHash(Hash);
				writer.WriteUnsignedVarInt((ulong)Layers);

				if ((Layers & LayerFlags.Pending) != 0)
				{
					writer.WriteUnsignedVarInt((ulong)Pending);
				}
			}

			public void SetPendingAction(PendingFlags pending)
			{
				Pending |= pending;
				MarkAsDirty();
			}

			public void MarkAsDirty()
			{
				Layers |= LayerFlags.Pending;
				Parent.MarkAsDirty();
			}

			public bool InLayer(LayerFlags layers)
				=> (Layers & layers) != 0;

			public abstract void MoveTo(DirectoryState newParent, string newName, LayerFlags newLayers);

			public abstract void Unlink();

			public bool IsModified(FileInfo fileInfo)
				=> Length != fileInfo.Length || LastModifiedTimeUtc != fileInfo.LastWriteTimeUtc.Ticks;

			public void Update(FileInfo fileInfo)
			{
				Length = fileInfo.Length;
				LastModifiedTimeUtc = fileInfo.LastWriteTimeUtc.Ticks;
			}

			public string GetPath()
			{
				StringBuilder builder = new();
				AppendPath(builder);
				return builder.ToString();
			}

			public void AppendPath(StringBuilder builder)
			{
				Parent.AppendPath(builder);

				if (builder.Length > 0 && builder[^1] != Path.DirectorySeparatorChar)
				{
					builder.Append(Path.DirectorySeparatorChar);
				}

				builder.Append(Name);
			}
		}

		// Collates lists of files and chunks with a particular hash
		[DebuggerDisplay("{Hash}")]
		class HashInfo
		{
			public int Index { get; set; }

			public IoHash Hash { get; }
			public long Length { get; }

			public List<FileState> Files { get; } = [];
			public List<ChunkInfo> Chunks { get; } = [];

			public HashInfo(IoHash hash, long length)
			{
				Hash = hash;
				Length = length;
			}

			public void AddChunk(ChunkInfo chunk)
			{
				if (chunk.Length != chunk.WithinHashInfo.Length && !Chunks.Contains(chunk))
				{
					Chunks.Add(chunk);
				}
			}
		}

		// Hashed chunk within another hashed object
		[DebuggerDisplay("{Offset}+{Length}->{WithinHashInfo}")]
		record class ChunkInfo(HashInfo WithinHashInfo, long Offset, long Length)
		{
			public static ChunkInfo Read(IMemoryReader reader, HashInfo[] hashes)
			{
				HashInfo withinHashInfo = hashes[(int)reader.ReadUnsignedVarInt()];
				long offset = (long)reader.ReadUnsignedVarInt();
				long length = (long)reader.ReadUnsignedVarInt();
				return new ChunkInfo(withinHashInfo, offset, length);
			}

			public void Write(IMemoryWriter writer)
			{
				writer.WriteUnsignedVarInt(WithinHashInfo.Index);
				writer.WriteUnsignedVarInt((ulong)Offset);
				writer.WriteUnsignedVarInt((ulong)Length);
			}
		}

		readonly DirectoryReference _rootDir;
		readonly FileReference _stateFile;
		readonly DirectoryState _rootDirState;
		readonly DirectoryState _hordeDirState;
		readonly DirectoryState _cacheDirState;
		readonly List<LayerState> _customLayers = [];
		readonly Dictionary<IoHash, HashInfo> _hashes = [];
		readonly ILogger _logger;

		/// <summary>
		/// Metadata directory name
		/// </summary>
		public const string HordeDirName = ".horde";

		const string CacheDirName = "cache";
		const string StateFileName = "contents.dat";

		/// <summary>
		/// Root directory for the workspace
		/// </summary>
		public DirectoryReference RootDir => _rootDir;

		/// <summary>
		/// Layers current in this workspace
		/// </summary>
		public IReadOnlyList<WorkspaceLayerId> CustomLayers => [.. _customLayers.Select(x => x.Id)];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="rootDir">Directory for the workspace</param>
		/// <param name="stateFile">Path to the state file for this directory</param>
		/// <param name="reader">Reader for </param>
		/// <param name="logger">Logger for diagnostic output</param>
		private Workspace(DirectoryReference rootDir, FileReference stateFile, IMemoryReader? reader, ILogger logger)
		{
			_rootDir = rootDir;
			_stateFile = stateFile;

			if (reader == null)
			{
				_rootDirState = new DirectoryState(null, String.Empty);
			}
			else
			{
				_rootDirState = new DirectoryState(null, reader);
				Read(reader);
			}

			_hordeDirState = _rootDirState.FindOrAddDirectory(HordeDirName, LayerFlags.Cache);
			_cacheDirState = _hordeDirState.FindOrAddDirectory(CacheDirName, LayerFlags.Cache);
			_logger = logger;
		}

		/// <summary>
		/// Create a new workspace instance in the given location. Opens the existing instance if it already contains workspace data.
		/// </summary>
		/// <param name="rootDir">Root directory for the workspace</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Workspace instance</returns>
		public static async Task<Workspace> CreateAsync(DirectoryReference rootDir, ILogger logger, CancellationToken cancellationToken = default)
		{
			FileReference stateFile = FileReference.Combine(rootDir, HordeDirName, StateFileName);

			using FileStream? stream = FileTransaction.OpenRead(stateFile);
			if (stream != null)
			{
				throw new WorkspaceException($"Workspace already exists in {rootDir}; use Open instead.");
			}

			Workspace workspace = new(rootDir, stateFile, null, logger);
			DirectoryReference.CreateDirectory(workspace.GetDirectoryReference(workspace._cacheDirState));
			await workspace.SaveAsync(cancellationToken);

			return workspace;
		}

		/// <summary>
		/// Create a new workspace instance in the given location. Opens the existing instance if it already contains workspace data.
		/// </summary>
		/// <param name="rootDir">Root directory for the workspace</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Workspace instance</returns>
		public static async Task<Workspace> CreateOrOpenAsync(DirectoryReference rootDir, ILogger logger, CancellationToken cancellationToken = default)
		{
			Workspace? workspace = await TryOpenAsync(rootDir, logger, cancellationToken);
			workspace ??= await CreateAsync(rootDir, logger, cancellationToken);
			return workspace;
		}

		/// <summary>
		/// Attempts to open an existing workspace for the current directory. 
		/// </summary>
		/// <param name="currentDir">Root directory for the workspace</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Workspace instance</returns>
		public static async Task<Workspace?> TryOpenAsync(DirectoryReference currentDir, ILogger logger, CancellationToken cancellationToken = default)
		{
			FileReference stateFile = FileReference.Combine(currentDir, HordeDirName, StateFileName);
			using (FileStream? stream = FileTransaction.OpenRead(stateFile))
			{
				if (stream != null)
				{
					byte[] data = await stream.ReadAllBytesAsync(cancellationToken);

					MemoryReader reader = new(data);
					Workspace workspace = new(currentDir, stateFile, reader, logger);

					return workspace;
				}
			}
			return null;
		}

		/// <summary>
		/// Attempts to open an existing workspace for the current directory. 
		/// </summary>
		/// <param name="currentDir">Root directory for the workspace</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Workspace instance</returns>
		public static async Task<Workspace?> TryFindAndOpenAsync(DirectoryReference currentDir, ILogger logger, CancellationToken cancellationToken = default)
		{
			for (DirectoryReference? testDir = currentDir; testDir != null; testDir = testDir.ParentDirectory)
			{
				Workspace? workspace = await TryOpenAsync(testDir, logger, cancellationToken);
				if (workspace != null)
				{
					return workspace;
				}
			}
			return null;
		}

		/// <summary>
		/// Save the current state of the workspace
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task SaveAsync(CancellationToken cancellationToken)
		{
			DirectoryReference.CreateDirectory(_stateFile.Directory);

			using (FileTransactionStream stream = FileTransaction.OpenWrite(_stateFile))
			{
				using (ChunkedMemoryWriter writer = new(64 * 1024))
				{
					Write(writer);
					await writer.CopyToAsync(stream, cancellationToken);
				}
				stream.CompleteTransaction();
			}
		}

		void Read(IMemoryReader reader)
		{
			reader.ReadList(_customLayers, () => new LayerState(reader));

			// Read the hash lookup
			int numHashes = reader.ReadInt32();
			HashInfo[] hashInfoArray = new HashInfo[numHashes];

			_hashes.Clear();
			_hashes.EnsureCapacity(numHashes);

			for (int idx = 0; idx < numHashes; idx++)
			{
				IoHash hash = reader.ReadIoHash();
				long length = (long)reader.ReadUnsignedVarInt();
				hashInfoArray[idx] = new HashInfo(hash, length);
				_hashes.Add(hash, hashInfoArray[idx]);
			}
			for (int idx = 0; idx < numHashes; idx++)
			{
				HashInfo hashInfo = hashInfoArray[idx];
				reader.ReadList(hashInfoArray[idx].Chunks, () => ChunkInfo.Read(reader, hashInfoArray));
			}

			// Add all the files to the hash lookup
			AddDirToHashLookup(_rootDirState);
		}

		void Write(IMemoryWriter writer)
		{
			RemoveUnusedHashInfoObjects();

			_rootDirState.Write(writer);
			writer.WriteList(_customLayers, x => x.Write(writer));

			// Write the hash lookup
			writer.WriteInt32(_hashes.Count);

			int nextIndex = 0;
			foreach (HashInfo hashInfo in _hashes.Values)
			{
				writer.WriteIoHash(hashInfo.Hash);
				writer.WriteUnsignedVarInt((ulong)hashInfo.Length);
				hashInfo.Index = nextIndex++;
			}
			foreach (HashInfo hashInfo in _hashes.Values)
			{
				writer.WriteList(hashInfo.Chunks, x => x.Write(writer));
			}
		}

		void RemoveUnusedHashInfoObjects()
		{
			const int UnknownIndex = 0;
			const int ReferencedIndex = 1;
			const int UnreferencedIndex = 2;

			static bool IsReferenced(HashInfo hashInfo)
			{
				if (hashInfo.Index == UnknownIndex)
				{
					hashInfo.Chunks.RemoveAll(x => !IsReferenced(x.WithinHashInfo));
					hashInfo.Index = (hashInfo.Chunks.Count > 0) ? ReferencedIndex : UnreferencedIndex;
				}
				return hashInfo.Index == ReferencedIndex;
			}

			List<HashInfo> hashes = [.. _hashes.Values];
			foreach (HashInfo hashInfo in hashes)
			{
				hashInfo.Index = (hashInfo.Files.Count > 0) ? ReferencedIndex : UnknownIndex;
			}

			_hashes.Clear();
			foreach (HashInfo hashInfo in hashes)
			{
				if (IsReferenced(hashInfo))
				{
					_hashes.Add(hashInfo.Hash, hashInfo);
				}
			}
		}

		#region Layers

		/// <summary>
		/// Add or update a layer with the given identifier
		/// </summary>
		/// <param name="id">Identifier for the layer</param>
		public void AddLayer(WorkspaceLayerId id)
		{
			if (id != WorkspaceLayerId.Default && !_customLayers.Any(x => x.Id == id))
			{
				ulong flags = ((ulong)LayerFlags.Default - 1) | (ulong)LayerFlags.Default;
				for (int idx = 0; idx < _customLayers.Count; idx++)
				{
					flags |= (ulong)_customLayers[idx].Flag;
				}
				if (flags == ~0UL)
				{
					throw new WorkspaceException("Maximum number of layers reached");
				}

				LayerFlags nextFlag = (LayerFlags)((flags + 1) & ~flags);
				_customLayers.Add(new LayerState(id, nextFlag));
			}
		}

		/// <summary>
		/// Removes a layer with the given identifier. Does not remove any files in the workspace.
		/// </summary>
		/// <param name="layerId">Layer to update</param>
		public void RemoveLayer(WorkspaceLayerId layerId)
		{
			int layerIdx = _customLayers.FindIndex(x => x.Id == layerId);
			if (layerIdx >= 0)
			{
				LayerState layer = _customLayers[layerIdx];
				if ((_rootDirState.Layers & layer.Flag) != 0)
				{
					throw new WorkspaceException($"Workspace still contains files for layer {layerId}");
				}
				_customLayers.RemoveAt(layerIdx);
			}
		}

		// Gets the names of layers
		List<string> GetLayerNames(LayerFlags layers)
		{
			List<string> names = [];
			while (layers != 0)
			{
				LayerFlags layer = (layers & ~(layers - 1));
				LayerState? layerState = _customLayers.FirstOrDefault(x => x.Flag == layer);
#pragma warning disable CA1308 // Normalize strings to uppercase
				names.Add(layerState?.Id.ToString() ?? $"{layer}".ToLowerInvariant());
#pragma warning restore CA1308 // Normalize strings to uppercase
			}
			return names;
		}

		// Gets the flag for a particular layer
		LayerFlags GetLayerFlag(WorkspaceLayerId id)
		{
			if (id == WorkspaceLayerId.Default)
			{
				return LayerFlags.Default;
			}

			LayerState? customLayer = _customLayers.FirstOrDefault(x => x.Id == id);
			if (customLayer == null)
			{
				throw new WorkspaceException($"Layer '{id}' does not exist");
			}

			return customLayer.Flag;
		}

		#endregion

		#region Add

		/// <summary>
		/// Adds files to the workspace
		/// </summary>
		/// <param name="layerId">Layer to add the files to</param>
		/// <param name="files">Files to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AddAsync(WorkspaceLayerId layerId, List<FileInfo> files, CancellationToken cancellationToken = default)
		{
			LayerFlags layer = GetLayerFlag(layerId);

			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				using SemaphoreSlim semaphore = new(1);
				ParallelBuzHashChunker chunker = new(pipeline, LeafChunkedDataNodeOptions.Default);

				_ = pipeline.AddTask(token => EnumerateFilesToAddAsync(layer, files, semaphore, chunker.SourceWriter, token));
				_ = pipeline.AddTask(chunker.OutputReader, (output, token) => RegisterChunksAsync(output, layer, semaphore, token));

				await pipeline.WaitForCompletionAsync();
			}

			RemoveUnusedHashInfoObjects();
			await SaveAsync(cancellationToken);
		}

		async Task EnumerateFilesToAddAsync(LayerFlags layer, List<FileInfo> files, SemaphoreSlim semaphore, ChannelWriter<ChunkerSource> chunkerSourceWriter, CancellationToken cancellationToken)
		{
			foreach (FileInfo file in files)
			{
				cancellationToken.ThrowIfCancellationRequested();

				string[] fragments = new FileReference(file).MakeRelativeTo(_rootDir).Split(Path.DirectorySeparatorChar);

				DirectoryState dirState = _rootDirState;
				using (IDisposable semaLock = await semaphore.WaitDisposableAsync(cancellationToken))
				{
					for (int idx = 0; idx < fragments.Length - 1; idx++)
					{
						dirState = dirState.FindOrAddDirectory(fragments[idx], layer);
					}

					string fileName = fragments[fragments.Length - 1];
					if (dirState.TryGetFileCaseSensitive(fileName, out FileState? fileState))
					{
						fileState.Layers |= layer;
						continue;
					}
				}

				FileToAdd fileToAdd = new(dirState, file);
				await chunkerSourceWriter.WriteAsync(new FileChunkerSource(file, fileToAdd), cancellationToken);
			}
			chunkerSourceWriter.Complete();
		}

		#endregion

		#region Filter

		/// <summary>
		/// Filters the contents of a layer
		/// </summary>
		/// <param name="layerId">Layer to filter</param>
		/// <param name="filter">Filter to be applied to the tree</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task ForgetAsync(WorkspaceLayerId layerId, FileFilter filter, CancellationToken cancellationToken)
		{
			LayerFlags layer = GetLayerFlag(layerId);
			ForgetMatchingFiles(layer, _rootDirState, "/", filter);
			await SaveAsync(cancellationToken);
		}

		void ForgetMatchingFiles(LayerFlags layer, DirectoryState dirState, string namePrefix, FileFilter filter)
		{
			List<FileState> removeFiles = [];
			foreach (FileState nextFile in dirState.Files)
			{
				if ((nextFile.Layers & layer) != 0)
				{
					string fileName = namePrefix + nextFile.Name;
					if (filter.Matches(fileName))
					{
						nextFile.Layers ^= layer;
						if (nextFile.Layers == 0)
						{
							removeFiles.Add(nextFile);
						}
					}
				}
			}

			List<DirectoryState> removeDirectories = [];
			foreach (DirectoryState nextDirectory in dirState.Directories)
			{
				if ((nextDirectory.Layers & layer) != 0)
				{
					string nextNamePrefix = namePrefix + nextDirectory.Name;
					if (filter.Matches(nextNamePrefix))
					{
						nextDirectory.Layers ^= layer;
						if (nextDirectory.Layers == 0)
						{
							removeDirectories.Add(nextDirectory);
						}
					}
					else if (filter.PossiblyMatches(nextNamePrefix))
					{
						ForgetMatchingFiles(layer, nextDirectory, nextNamePrefix + "/", filter);
					}
				}
			}

			foreach (FileState removeFile in removeFiles)
			{
				_logger.LogInformation("Forgetting {Path}", removeFile.GetPath());
				RemoveFileFromHashLookup(removeFile);
				removeFile.Unlink();
			}

			foreach (DirectoryState removeDirectory in removeDirectories)
			{
				_logger.LogInformation("Forgetting {Path}", removeDirectory.GetPath());
				RemoveDirectoryFromHashLookup(removeDirectory);
				removeDirectory.Unlink();
			}
		}

		#endregion

		#region Clean

		class CleanVisitor(ILogger logger) : Visitor
		{
			public ConcurrentBag<FileState> RemoveFiles { get; } = [];
			public ConcurrentBag<DirectoryState> RemoveDirs { get; } = [];

			public override void FileAdded(string path, DirectoryState parentDir, FileInfo fileInfo)
			{
				logger.LogInformation("Deleting {Path} (untracked)", path);
				FileUtils.ForceDeleteFile(fileInfo);
			}

			public override void FileModified(string path, FileState fileState, FileInfo fileInfo)
			{
				logger.LogInformation("Deleting {Path} (modified)", path);
				FileUtils.ForceDeleteFile(fileInfo);
				RemoveFiles.Add(fileState);
			}

			public override void FileRemoved(string path, FileState fileState)
			{
				logger.LogInformation("Removing {Path} (missing)", path);
				RemoveFiles.Add(fileState);
			}

			public override void DirectoryAdded(string path, DirectoryState parentDir, DirectoryInfo subDirInfo)
			{
				logger.LogInformation("Deleting {Path} (untracked)", path);
				FileUtils.ForceDeleteDirectory(subDirInfo);
			}

			public override void DirectoryRemoved(string path, DirectoryState dirState)
			{
				logger.LogInformation("Removing {Path} (missing)", path);
				RemoveDirs.Add(dirState);
			}
		}

		/// <summary>
		/// Prints the status of files in this workspace based on current filesystem metadata
		/// </summary>
		/// <param name="filter">Filter for files to include</param>
		/// <param name="flags">Flags for changes to show</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task CleanAsync(FileFilter? filter = null, WorkspaceStatusFlags flags = WorkspaceStatusFlags.Default, CancellationToken cancellationToken = default)
		{
			filter ??= new FileFilter(FileFilterType.Include);

			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				ChannelWithRefCount<DirectoryState> queue = Channel.CreateUnbounded<DirectoryState>().WithRefCount();
				queue.Writer.TryWrite(_rootDirState);

				CleanVisitor visitor = new(_logger);
				pipeline.AddTasks(16, ctx => ScanWorkspaceAsync(visitor, filter, flags, queue, ctx));

				await pipeline.WaitForCompletionAsync();

				foreach (DirectoryState removeDir in visitor.RemoveDirs)
				{
					removeDir.Unlink();
				}

				foreach (FileState removeFile in visitor.RemoveFiles)
				{
					removeFile.Unlink();
				}
			}

			RemoveUnusedHashInfoObjects();
			await SaveAsync(cancellationToken);
		}

		#endregion

		#region Syncing

		class DirectoryToSync(string name)
		{
			public string Name { get; } = name;
			public DirectoryState? State { get; set; }
			public Dictionary<string, DirectoryToSync> Directories { get; } = new Dictionary<string, DirectoryToSync>(FileSystemReference.Comparer);
			public Dictionary<string, FileToSync> Files { get; } = new Dictionary<string, FileToSync>(FileSystemReference.Comparer);

			public DirectoryToSync? FindDirectoryExactCase(string name)
				=> (Directories.TryGetValue(name, out DirectoryToSync? dirToSync) && Name.Equals(name, StringComparison.Ordinal)) ? dirToSync : null;

			public FileToSync? FindFileExactCase(string name)
				=> (Files.TryGetValue(name, out FileToSync? fileToSync) && Name.Equals(name, StringComparison.Ordinal)) ? fileToSync : null;
		}

		class FileToSync(FileEntry entry)
		{
			public FileEntry Entry => entry;
			public string Name => Entry.Name;
			public FileState? State { get; set; }

			public HashInfo? CopyFrom { get; set; }
			public FileState? MoveFrom { get; set; }
			public List<LeafChunkedDataNodeRef>? Chunks { get; set; }
			public OutputFile? OutputFile { get; set; }

			public bool Complete { get; set; }
		}

		/// <summary>
		/// Syncs a layer to the given contents at a given base path
		/// </summary>
		/// <param name="layerId">Identifier for the layer</param>
		/// <param name="basePath">Base path within the workspace to sync to.</param>
		/// <param name="contents">New contents for the layer</param>
		/// <param name="options">Options for extraction</param>
		/// <param name="stats">Stats for the sync operation</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task SyncAsync(WorkspaceLayerId layerId, string? basePath, IBlobRef<DirectoryNode>? contents, ExtractOptions? options = null, WorkspaceSyncStats? stats = null, CancellationToken cancellationToken = default)
		{
			basePath = basePath?.Replace('\\', '/').Trim('/') ?? String.Empty;

			await using IBlobWriter writer = new MemoryBlobWriter(BlobSerializerOptions.Default);
			if (basePath.Length > 0 && contents != null)
			{
				using BlobData blobData = await contents.ReadBlobDataAsync(cancellationToken);

				IoHash hash = IoHash.Compute(blobData.Data.Span);
				IHashedBlobRef<DirectoryNode> hashedContents = HashedBlobRef.Create<DirectoryNode>(hash, contents);

				DirectoryNode current = await contents.ReadBlobAsync(cancellationToken);
				foreach (string fragment in basePath.Split('/').Reverse())
				{
					DirectoryNode next = new();

					DirectoryEntry entry = new(fragment, current.Length, hashedContents);
					next.AddDirectory(entry);
					hashedContents = await writer.WriteBlobAsync(next, cancellationToken);

					current = next;
				}

				contents = hashedContents;
			}

			await SyncAsync(layerId, contents, options, stats, cancellationToken);
		}

		/// <summary>
		/// Syncs a layer to the given contents
		/// </summary>
		/// <param name="layerId">Identifier for the layer</param>
		/// <param name="contents">New contents for the layer</param>
		/// <param name="options">Options for the extraction</param>
		/// <param name="stats">Stats for the sync operation</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task SyncAsync(WorkspaceLayerId layerId, IBlobRef<DirectoryNode>? contents, ExtractOptions? options = null, WorkspaceSyncStats? stats = null, CancellationToken cancellationToken = default)
		{
			LayerFlags layer = GetLayerFlag(layerId);

			options ??= new ExtractOptions { Progress = new ExtractStatsLogger(_logger) };
			stats ??= new WorkspaceSyncStats();

			// Find all the files that need to be synced, and create a plan for the existing tree to get there
			_logger.LogInformation("Creating manifest...");
			DirectoryToSync dirToSync = await CreateTreeToSyncAsync("", contents, cancellationToken);
			CreateManifest(_rootDirState, dirToSync, layer);
			ValidateManifest(_rootDirState, layer);
			await SaveAsync(cancellationToken);

			// Remove all the old files and update the workspace state to match the new tree
			RemoveOldState(_rootDirState, layer);
			CreateNewState(dirToSync, layer);
			await SaveAsync(cancellationToken);

			// Materialize the state and remove all the temporary files
			_logger.LogInformation("Updating workspace...");
			CreateDirectories(_rootDir, dirToSync);
			await MaterializeFilesAsync(dirToSync, layer, options, stats, cancellationToken);
			RemoveIntermediates(_rootDirState);
			UpdateMetadata(dirToSync, _rootDirState, layer);
			await SaveAsync(cancellationToken);
		}

		// Figure out all the files and directories that need to be synced, reusing any tracked items to 
		// short-circuit traversal through the tree.
		async Task<DirectoryToSync> CreateTreeToSyncAsync(string name, IBlobRef<DirectoryNode>? dirNodeRef, CancellationToken cancellationToken)
		{
			DirectoryToSync dirToSync = new(name);

			if (dirNodeRef != null)
			{
				DirectoryNode dirNode = await dirNodeRef.ReadBlobAsync(cancellationToken);

				foreach (DirectoryEntry subDirEntry in dirNode.Directories)
				{
					DirectoryToSync subDirToSync = await CreateTreeToSyncAsync(subDirEntry.Name, subDirEntry.Handle, cancellationToken);
					dirToSync.Directories.Add(subDirEntry.Name, subDirToSync);
				}

				foreach (FileEntry fileEntry in dirNode.Files)
				{
					FileToSync fileToSync = new(fileEntry);

					HashInfo? copyFrom;
					if (_hashes.TryGetValue(fileEntry.Hash, out copyFrom))
					{
						fileToSync.CopyFrom = copyFrom;
					}
					else
					{
						fileToSync.Chunks = await fileEntry.Target.EnumerateLeafNodesAsync(cancellationToken).ToListAsync(cancellationToken);
					}

					dirToSync.Files.Add(fileEntry.Name, fileToSync);
				}
			}

			return dirToSync;
		}

		#region Planning

		// Merges the manifest of files to sync with the existing directory state
		static void CreateManifest(DirectoryState dirState, DirectoryToSync dirToSync, LayerFlags layer)
		{
			// Clear out all the pending actions
			dirState.ClearPendingActions();

			// Schedule everything that needs to be removed
			ScheduleRemoveActions(dirState, dirToSync, layer);

			// Reprieve any files that can be moved to new locations
			ScheduleMoveActions(dirToSync, layer);

			// Reprieve any files that can be harvested for chunks in new locations
			ScheduleCopyActions(dirToSync);
		}

		// Schedule all the existing files in the current directory for removal
		static void ScheduleRemoveActions(DirectoryState dirState, DirectoryToSync dirToSync, LayerFlags layer)
		{
			dirToSync.State = dirState;
			dirToSync.State.Layers |= layer;

			foreach (DirectoryState subDirState in dirState.Directories)
			{
				DirectoryToSync? subDirToSync;
				if (dirToSync.Directories.TryGetValue(subDirState.Name, out subDirToSync))
				{
					if (!String.Equals(subDirState.Name, subDirToSync.Name, StringComparison.Ordinal))
					{
						// Remove everything, and set the flag to move through the cache if we need to reuse it.
						subDirState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk | PendingFlags.ForceRenameOrRemove);
					}
					else
					{
						// Run recursively over the contents
						ScheduleRemoveActions(subDirState, subDirToSync, layer);
					}
				}
				else if (dirToSync.Files.ContainsKey(subDirState.Name))
				{
					// Must remove everything and rename if we want to keep it.
					subDirState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk | PendingFlags.ForceRenameOrRemove);
				}
				else
				{
					if (subDirState.Layers == layer)
					{
						// Remove if from the layer and disk
						subDirState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk);
					}
					else if ((subDirState.Layers & layer) != 0)
					{
						// Remove it from the layer
						subDirState.SetPendingAction(PendingFlags.RemoveFromLayer);
					}
				}
			}

			foreach (FileState fileState in dirState.Files)
			{
				FileToSync? fileToSync;
				if (dirToSync.Files.TryGetValue(fileState.Name, out fileToSync))
				{
					if (!String.Equals(fileState.Name, fileToSync.Name, StringComparison.Ordinal))
					{
						// Must remove and set cache flag
						fileState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk | PendingFlags.ForceRenameOrRemove);
					}
					else if (fileState.Hash == fileToSync.Entry.Hash)
					{
						// Can retain it
						fileState.Layers &= ~LayerFlags.Pending;
						fileState.Layers |= layer;
						fileState.Pending = PendingFlags.None;

						fileToSync.State = fileToSync.MoveFrom = fileState;
					}
					else
					{
						// Regular remove
						fileState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk);
					}
				}
				else if ((dirToSync.Directories.ContainsKey(fileState.Name)))
				{
					// Must remove it before we can write the new thing
					fileState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk | PendingFlags.ForceRenameOrRemove);
				}
				else
				{
					if (fileState.Layers == layer)
					{
						// Remove if from the layer and disk
						fileState.SetPendingAction(PendingFlags.RemoveFromLayer | PendingFlags.RemoveFromDisk);
					}
					else if ((fileState.Layers & layer) != 0)
					{
						// Remove it from the layer
						fileState.SetPendingAction(PendingFlags.RemoveFromLayer);
					}
				}
			}
		}

		// Identify files which can be removed from other layers
		static void ScheduleMoveActions(DirectoryToSync dirToSync, LayerFlags layer)
		{
			foreach (DirectoryToSync subDirToSync in dirToSync.Directories.Values)
			{
				ScheduleMoveActions(subDirToSync, layer);
			}

			foreach (FileToSync fileToSync in dirToSync.Files.Values)
			{
				if (fileToSync.CopyFrom != null && fileToSync.MoveFrom == null)
				{
					FileState? sourceFile = FindFileToMove(fileToSync.CopyFrom, layer);
					if (sourceFile != null)
					{
						if ((sourceFile.Pending & PendingFlags.RemoveFromDisk) != 0)
						{
							sourceFile.Pending ^= PendingFlags.RemoveFromDisk | PendingFlags.Move;
						}
						else if (sourceFile.Layers == LayerFlags.Cache)
						{
							sourceFile.SetPendingAction(PendingFlags.Move);
						}
						fileToSync.MoveFrom = sourceFile;
					}
				}
			}
		}

		static FileState? FindFileToMove(HashInfo hashInfo, LayerFlags layer)
		{
			FileState? cacheFile = null;
			foreach (FileState fileState in hashInfo.Files)
			{
				if (fileState.Layers == (layer | LayerFlags.Pending) && (fileState.Pending & PendingFlags.RemoveFromDisk) != 0)
				{
					return fileState;
				}
				else if ((fileState.Layers & LayerFlags.Cache) != 0)
				{
					cacheFile = fileState;
				}
			}
			return cacheFile;
		}

		// Identify files which can be used
		static void ScheduleCopyActions(DirectoryToSync dirToSync)
		{
			foreach (DirectoryToSync subDirToSync in dirToSync.Directories.Values)
			{
				ScheduleCopyActions(subDirToSync);
			}

			foreach (FileToSync fileToSync in dirToSync.Files.Values)
			{
				if (fileToSync.CopyFrom != null)
				{
					FileState? sourceFile = FindFileForCopy(fileToSync.CopyFrom);
					if (sourceFile != null && (sourceFile.Pending & PendingFlags.RemoveFromDisk) != 0)
					{
						sourceFile.Pending ^= PendingFlags.RemoveFromDisk | PendingFlags.Copy;
					}
				}
			}
		}

		// Find a file to copy from, preferring existing files in other layers first
		static FileState FindFileForCopy(HashInfo hashInfo)
			=> FindFileForCopy(hashInfo, PendingFlags.RemoveFromDisk, 0)
			?? FindFileForCopy(hashInfo, PendingFlags.RemoveFromDisk, PendingFlags.RemoveFromDisk)
			?? throw new InvalidDataException($"Unable to find any file matching hash; invalid state?");

		// Try to find any file matching the given layer flags
		static FileState? FindFileForCopy(HashInfo hashInfo, PendingFlags pendingMask, PendingFlags pendingFlag)
		{
			foreach (FileState fileState in hashInfo.Files)
			{
				if ((fileState.Pending & pendingMask) == pendingFlag)
				{
					return fileState;
				}
			}
			foreach (ChunkInfo chunkInfo in hashInfo.Chunks)
			{
				FileState? fileState = FindFileForCopy(chunkInfo.WithinHashInfo, pendingMask, pendingFlag);
				if (fileState != null)
				{
					return fileState;
				}
			}
			return null;
		}

		void ValidateManifest(DirectoryState dirState, LayerFlags layer)
		{
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				if ((subDirState.Layers & LayerFlags.Pending) != 0)
				{
					LayerFlags otherLayers = subDirState.Layers & ~(layer | LayerFlags.Pending);
					if ((subDirState.Pending & (PendingFlags.Move | PendingFlags.RemoveFromDisk)) != 0 && otherLayers != 0)
					{
						throw new WorkspaceException($"Cannot remove {subDirState.GetPath()}: directory is referenced by other layers ({GetLayerNames(otherLayers)})");
					}
					else
					{
						ValidateManifest(subDirState, layer);
					}
				}
			}

			foreach (FileState fileState in dirState.Files)
			{
				LayerFlags otherLayers = fileState.Layers & ~(layer | LayerFlags.Pending);
				if ((fileState.Pending & (PendingFlags.Move | PendingFlags.RemoveFromDisk)) != 0 && otherLayers != 0)
				{
					throw new WorkspaceException($"Cannot remove {fileState.GetPath()}: file is referenced by other layers ({GetLayerNames(otherLayers)})");
				}
			}
		}

		#endregion

		static void CreateNewState(DirectoryToSync dirToSync, LayerFlags layer)
		{
			DirectoryState dirState = dirToSync.State ?? throw new InvalidDataException();
			foreach ((string name, DirectoryToSync subDirToSync) in dirToSync.Directories)
			{
				subDirToSync.State ??= dirState.AddDirectory(name, layer);
				CreateNewState(subDirToSync, layer);
			}
		}

		static void CreateDirectories(DirectoryReference dirRef, DirectoryToSync dirToSync)
		{
			// TODO: only if pending add
			foreach ((string name, DirectoryToSync subDirToSync) in dirToSync.Directories)
			{
				DirectoryReference subDirRef = DirectoryReference.Combine(dirRef, name);
				DirectoryReference.CreateDirectory(subDirRef);

				CreateDirectories(subDirRef, subDirToSync);
			}
		}

		void RemoveIntermediates(DirectoryState dirState)
		{
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				RemoveIntermediates(subDirState);
			}

			foreach (FileState fileState in dirState.Files)
			{
				if ((fileState.Layers & LayerFlags.Cache) != 0 || (fileState.Pending & PendingFlags.Copy) != 0)
				{
					FileInfo fileInfo = GetFileInfo(fileState);
					FileUtils.ForceDeleteFile(fileInfo);
				}
			}
		}

		void UpdateMetadata(DirectoryToSync dirToSync, DirectoryState dirState, LayerFlags layer)
		{
			foreach ((string subDirName, DirectoryToSync subDirToSync) in dirToSync.Directories)
			{
				DirectoryState subDirState = dirState.FindOrAddDirectory(subDirName, layer);
				UpdateMetadata(subDirToSync, subDirState, layer);
			}

			foreach (FileToSync fileToSync in dirToSync.Files.Values)
			{
				OutputFile outputFile = fileToSync.OutputFile!;

				FileState? fileState = fileToSync.State;
				if (fileState == null)
				{
					fileState = dirState.AddFile(outputFile.FileEntry.Name, fileToSync.Entry.Hash, layer);
					fileState.Update(outputFile.FileInfo);

					HashInfo hashInfo = AddFileToHashLookup(fileState);

					if (fileToSync.Chunks != null)
					{
						long offset = 0;
						foreach (LeafChunkedDataNodeRef chunkRef in fileToSync.Chunks!)
						{
							ChunkInfo chunkInfo = new(hashInfo, offset, chunkRef.Length);

							HashInfo chunkHashInfo = FindOrAddHashInfo(chunkRef.Hash, chunkRef.Length);
							chunkHashInfo.AddChunk(chunkInfo);

							offset += chunkRef.Length;
						}
					}
				}
				else
				{
					fileState.Update(outputFile.FileInfo);
				}
			}
		}

		void RemoveOldState(DirectoryState dirState, LayerFlags layer)
		{
			List<DirectoryState> removeDirs = [];
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				RemoveOldState(subDirState, layer);

				if ((subDirState.Pending & PendingFlags.RemoveFromLayer) != 0)
				{
					subDirState.Layers &= ~layer;
				}
				if ((subDirState.Pending & PendingFlags.RemoveFromDisk) != 0)
				{
					removeDirs.Add(subDirState);
				}
			}

			List<FileState> cacheFiles = [];
			List<FileState> removeFiles = [];
			foreach (FileState fileState in dirState.Files)
			{
				if ((fileState.Pending & PendingFlags.RemoveFromLayer) != 0)
				{
					fileState.Layers &= ~layer;
				}

				if ((fileState.Pending & PendingFlags.ForceRenameOrRemove) != 0)
				{
					cacheFiles.Add(fileState);
				}
				else if ((fileState.Pending & PendingFlags.RemoveFromDisk) != 0)
				{
					removeFiles.Add(fileState);
				}
			}

			foreach (FileState cacheFile in cacheFiles)
			{
				MoveFileToCache(cacheFile);
			}

			foreach (FileState removeFile in removeFiles)
			{
				DeleteFile(removeFile);
			}

			foreach (DirectoryState removeDir in removeDirs)
			{
				DeleteDirectory(removeDir);
			}
		}

		async Task MaterializeFilesAsync(DirectoryToSync dirToSync, LayerFlags layer, ExtractOptions options, WorkspaceSyncStats stats, CancellationToken cancellationToken)
		{
			// Split the work into copies within the workspace, and reads from the upstream store
			Channel<ChunkCopyRequest> chunkCopyRequests = Channel.CreateUnbounded<ChunkCopyRequest>();
			Channel<ChunkReadRequest> chunkReadRequests = Channel.CreateUnbounded<ChunkReadRequest>();
			await FindChunkActionsAsync("", dirToSync, _rootDirState, layer, chunkCopyRequests.Writer, chunkReadRequests.Writer, stats, cancellationToken);
			chunkCopyRequests.Writer.Complete();
			chunkReadRequests.Writer.Complete();

			// Run the pipeline to create everything
			int numReadTasks = options.NumReadTasks ?? ExtractOptions.DefaultNumReadTasks;
			int numDecodeTasks = options.NumDecodeTasks ?? ExtractOptions.DefaultNumDecodeTasks;
			int numWriteTasks = 16;

			using BatchOutputWriter outputWriter = new(_logger);
			using BatchChunkReader batchReader = new(outputWriter.RequestWriter) { VerifyHashes = options.VerifyOutput };

			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				Task[] copyTasks = pipeline.AddTasks(8, ct => CopyChunksAsync(chunkCopyRequests.Reader, outputWriter.RequestWriter, ct));
				Task[] readTasks = batchReader.AddToPipeline(pipeline, numReadTasks, numDecodeTasks, chunkReadRequests.Reader);
				_ = Task.WhenAll(Enumerable.Concat(copyTasks, readTasks)).ContinueWith(_ => outputWriter.RequestWriter.TryComplete(), TaskScheduler.Default);

				Task[] writeTasks = outputWriter.AddToPipeline(pipeline, numWriteTasks);

				if (options.Progress != null)
				{
					_ = pipeline.AddTask(ctx => DirectoryNodeExtract.UpdateStatsAsync(batchReader, outputWriter, Task.WhenAll(writeTasks), options.Progress, options.ProgressUpdateFrequency, ctx));
				}

				await pipeline.WaitForCompletionAsync();
			}
		}

		async ValueTask FindChunkActionsAsync(string path, DirectoryToSync dirToSync, DirectoryState dirState, LayerFlags layer, ChannelWriter<ChunkCopyRequest> copyRequests, ChannelWriter<ChunkReadRequest> readRequests, WorkspaceSyncStats stats, CancellationToken cancellationToken)
		{
			foreach ((string subDirName, DirectoryToSync subDirToSync) in dirToSync.Directories)
			{
				DirectoryState subDirState = dirState.FindOrAddDirectory(subDirName, layer);
				await FindChunkActionsAsync($"{path}{subDirName}/", subDirToSync, subDirState, layer, copyRequests, readRequests, stats, cancellationToken);
			}

			foreach ((string fileName, FileToSync fileToSync) in dirToSync.Files)
			{
				if (!fileToSync.Complete)
				{
					FileInfo fileInfo = FileReference.Combine(_rootDir, $"{path}{fileName}").ToFileInfo();
					OutputFile outputFile = new(path, fileInfo, fileToSync.Entry);
					fileToSync.OutputFile = outputFile;

					if (fileToSync.Chunks == null)
					{
						if (fileToSync.MoveFrom != null)
						{
							if (fileToSync.MoveFrom == fileToSync.State)
							{
								stats.NumFilesKept++;
							}
							else
							{
								MoveFile(fileToSync.MoveFrom, dirState, fileToSync.Entry.Name, layer);
								fileToSync.State = fileToSync.MoveFrom;

								stats.NumFilesMoved++;
							}
						}
						else if (fileToSync.CopyFrom != null)
						{
							FileState sourceFile = fileToSync.CopyFrom!.Files[0];
							SourceChunk sourceChunk = new(sourceFile, 0, sourceFile.Length, sourceFile.Hash);

							outputFile.IncrementRemaining();

							ChunkCopyRequest copyRequest = new(outputFile, 0, sourceChunk);
							await copyRequests.WriteAsync(copyRequest, cancellationToken);

							stats.NumBytesReused += sourceFile.Length;
						}
					}
					else
					{
						long offset = 0;
						foreach (LeafChunkedDataNodeRef leafNodeRef in fileToSync.Chunks)
						{
							outputFile.IncrementRemaining();

							SourceChunk? sourceChunk = TryGetSourceChunk(leafNodeRef.Hash);
							if (sourceChunk != null)
							{
								ChunkCopyRequest copyRequest = new(outputFile, offset, sourceChunk);
								await copyRequests.WriteAsync(copyRequest, cancellationToken);

								stats.NumBytesReused += leafNodeRef.Length;
							}
							else
							{
								ChunkReadRequest readRequest = new(outputFile, offset, leafNodeRef);
								await readRequests.WriteAsync(readRequest, cancellationToken);

								stats.NumBytesDownloaded += leafNodeRef.Length;
							}

							offset += leafNodeRef.Length;
						}
					}
				}
			}
		}

		record class ChunkCopyRequest(OutputFile OutputFile, long Offset, SourceChunk SourceChunk);

		async Task CopyChunksAsync(ChannelReader<ChunkCopyRequest> copyRequests, ChannelWriter<OutputChunk[]> outputRequests, CancellationToken cancellationToken)
		{
			await foreach (ChunkCopyRequest copyRequest in copyRequests.ReadAllAsync(cancellationToken))
			{
				FileStreamOptions options = new()
				{
					Mode = FileMode.Open,
					Share = FileShare.Read,
					Access = FileAccess.Read,
					Options = FileOptions.Asynchronous,
					BufferSize = 0,
				};

				SourceChunk sourceChunk = copyRequest.SourceChunk;
				FileInfo sourceFile = GetFileInfo(sourceChunk.File);

				using Blake3.Hasher hasher = Blake3.Hasher.New();
				using (FileStream sourceStream = sourceFile.Open(options))
				{
					sourceStream.Seek(sourceChunk.Offset, SeekOrigin.Begin);

					long remaining = sourceChunk.Length;

#pragma warning disable CA2000
					OutputChunk? outputChunk = null;
					while (outputChunk == null || remaining > 0)
					{
						int length = (int)Math.Min(remaining, 256 * 1024);

						byte[] data = new byte[length];
						await sourceStream.ReadExactlyAsync(data, cancellationToken);

						hasher.Update(data);

						if (outputChunk != null)
						{
							copyRequest.OutputFile.IncrementRemaining();
							await outputRequests.WriteAsync([outputChunk], cancellationToken);
						}

						outputChunk = new OutputChunk(copyRequest.OutputFile, copyRequest.Offset, data, sourceChunk, null);
						remaining -= length;
					}
					await outputRequests.WriteAsync([outputChunk], cancellationToken);
#pragma warning restore CA2000
				}

				IoHash hash = IoHash.FromBlake3(hasher);
				if (hash != sourceChunk.Hash)
				{
					throw new WorkspaceException($"Source data does not have correct hash; expected {sourceChunk.Hash}, actually {hash} ({sourceFile}, offset: {sourceChunk.Offset}, length: {sourceChunk.Length})");
				}
			}
		}

		record class SourceChunk(FileState File, long Offset, long Length, IoHash Hash)
		{
			/// <inheritdoc/>
			public override string ToString()
				=> $"{File.GetPath()}@{Offset}";
		}

		SourceChunk? TryGetSourceChunk(IoHash hash)
		{
			HashInfo? hashInfo;
			if (_hashes.TryGetValue(hash, out hashInfo))
			{
				return TryGetSourceChunk(hashInfo);
			}
			else
			{
				return null;
			}
		}

		static SourceChunk? TryGetSourceChunk(HashInfo hashInfo)
		{
			if (hashInfo.Files.Count > 0)
			{
				return new SourceChunk(hashInfo.Files[0], 0, hashInfo.Length, hashInfo.Hash);
			}

			foreach (ChunkInfo chunkInfo in hashInfo.Chunks)
			{
				SourceChunk? otherChunk = TryGetSourceChunk(chunkInfo.WithinHashInfo);
				if (otherChunk != null)
				{
					return new SourceChunk(otherChunk.File, otherChunk.Offset + chunkInfo.Offset, hashInfo.Length, hashInfo.Hash);
				}
			}

			return null;
		}

		async Task<bool> TryCopyCachedDataAsync(HashInfo hashInfo, long offset, long length, Stream outputStream, CancellationToken cancellationToken)
		{
			foreach (FileState file in hashInfo.Files)
			{
				if (await TryCopyChunkAsync(file, offset, length, outputStream, cancellationToken))
				{
					return true;
				}
			}
			foreach (ChunkInfo chunk in hashInfo.Chunks)
			{
				if (await TryCopyCachedDataAsync(chunk.WithinHashInfo, chunk.Offset, chunk.Length, outputStream, cancellationToken))
				{
					return true;
				}
			}
			return false;
		}

		async Task<bool> TryCopyChunkAsync(FileState file, long offset, long length, Stream outputStream, CancellationToken cancellationToken)
		{
			FileInfo fileInfo = GetFileInfo(file);
			if (fileInfo.Exists)
			{
				long initialPosition = outputStream.Position;

				using FileStream inputStream = fileInfo.OpenRead();
				inputStream.Seek(offset, SeekOrigin.Begin);

				byte[] tempBuffer = new byte[65536];
				while (length > 0)
				{
					int readSize = await inputStream.ReadAsync(tempBuffer.AsMemory(0, (int)Math.Min(length, tempBuffer.Length)), cancellationToken);
					if (readSize == 0)
					{
						outputStream.Seek(initialPosition, SeekOrigin.Begin);
						outputStream.SetLength(initialPosition);
						return false;
					}

					await outputStream.WriteAsync(tempBuffer.AsMemory(0, readSize), cancellationToken);
					length -= readSize;
				}
			}
			return true;
		}

		#endregion

		class Visitor
		{
			public virtual void DirectoryAdded(string path, DirectoryState parentDir, DirectoryInfo subDirInfo)
			{ }

			public virtual void DirectoryRemoved(string path, DirectoryState dirState)
			{ }

			public virtual void FileAdded(string path, DirectoryState parentDir, FileInfo fileInfo)
			{ }

			public virtual void FileModified(string path, FileState fileState, FileInfo fileInfo)
			{ }

			public virtual void FileUnmodified(string path, FileState fileState, FileInfo fileInfo)
			{ }

			public virtual void FileRemoved(string path, FileState fileState)
			{ }
		}

		async Task ScanWorkspaceAsync(Visitor visitor, FileFilter filter, WorkspaceStatusFlags flags, ChannelWithRefCount<DirectoryState> channel, CancellationToken cancellationToken)
		{
			await foreach (DirectoryState dirState in channel.Reader.ReadAllAsync(cancellationToken))
			{
				DirectoryInfo dirInfo = GetDirectoryReference(dirState).ToDirectoryInfo();

				// Enumerate everything in this directory
				Dictionary<string, FileInfo> nameToFileInfo = new(StringComparer.Ordinal);
				Dictionary<string, DirectoryInfo> nameToDirectoryInfo = new(StringComparer.Ordinal);
				foreach (FileSystemInfo fileSystemInfo in dirInfo.EnumerateFileSystemInfos())
				{
					if (fileSystemInfo is FileInfo fileInfo)
					{
						nameToFileInfo.Add(fileInfo.Name, fileInfo);
					}
					else if (fileSystemInfo is DirectoryInfo directoryInfo && !IsRelativeDir(fileSystemInfo.Name))
					{
						nameToDirectoryInfo.Add(directoryInfo.Name, directoryInfo);
					}
				}

				// Add new directories
				foreach (DirectoryInfo subDirInfo in nameToDirectoryInfo.Values)
				{
					if (!dirState.TryGetDirectoryCaseSensitive(subDirInfo.Name, out _))
					{
						if ((flags & WorkspaceStatusFlags.DirectoryAdded) != 0)
						{
							string path = dirState.GetChildPath(subDirInfo.Name);
							if (filter.Matches(path))
							{
								visitor.DirectoryAdded(path, dirState, subDirInfo);
							}
						}
					}
				}

				// Ignore metadata files in the .horde directory
				if (dirState != _hordeDirState)
				{
					// Add new files
					foreach (FileInfo fileInfo in nameToFileInfo.Values)
					{
						if (!dirState.TryGetFileCaseSensitive(fileInfo.Name, out _))
						{
							if ((flags & WorkspaceStatusFlags.FileAdded) != 0)
							{
								string path = dirState.GetChildPath(fileInfo.Name);
								if (filter.Matches(path))
								{
									visitor.FileAdded(path, dirState, fileInfo);
								}
							}
						}
					}

					// Find all the files to remove
					foreach (FileState fileState in dirState.Files)
					{
						if (!nameToFileInfo.TryGetValue(fileState.Name, out FileInfo? fileInfo))
						{
							if ((flags & WorkspaceStatusFlags.FileRemoved) != 0)
							{
								string path = fileState.GetPath();
								if (filter.Matches(path))
								{
									visitor.FileRemoved(path, fileState);
								}
							}
						}
						else if (fileState.IsModified(fileInfo))
						{
							if ((flags & WorkspaceStatusFlags.FileModified) != 0)
							{
								string path = fileState.GetPath();
								if (filter.Matches(path))
								{
									visitor.FileModified(path, fileState, fileInfo);
								}
							}
						}
						else
						{
							if ((flags & WorkspaceStatusFlags.FileUnmodified) != 0)
							{
								string path = fileState.GetPath();
								if (filter.Matches(path))
								{
									visitor.FileUnmodified(path, fileState, fileInfo);
								}
							}
						}
					}
				}

				// Find directories that do not exist any more
				foreach (DirectoryState subDirState in dirState.Directories)
				{
					if (nameToDirectoryInfo.ContainsKey(subDirState.Name))
					{
						string path = subDirState.GetPath();
						if (filter.PossiblyMatches(path))
						{
							channel.AddRef();
							await channel.Writer.WriteAsync(subDirState, cancellationToken);
						}
					}
					else
					{
						if ((flags & WorkspaceStatusFlags.DirectoryRemoved) != 0)
						{
							string path = subDirState.GetPath();
							if (filter.Matches(path))
							{
								visitor.DirectoryRemoved(path, subDirState);
							}
						}
					}
				}

				// Release the completion counter
				channel.Release();
			}
		}

		#region Status

		/// <summary>
		/// Prints the status of files in this workspace based on current filesystem metadata
		/// </summary>
		/// <param name="filter">Filter for files to include</param>
		/// <param name="flags">Flags for changes to show</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task StatusAsync(FileFilter? filter = null, WorkspaceStatusFlags flags = WorkspaceStatusFlags.Default, CancellationToken cancellationToken = default)
		{
			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				ChannelWithRefCount<DirectoryState> directoryQueue = Channel.CreateUnbounded<DirectoryState>().WithRefCount();
				directoryQueue.Writer.TryWrite(_rootDirState);

				filter ??= new FileFilter(FileFilterType.Include);

				StatusVisitor visitor = new();
				pipeline.AddTasks(16, ctx => ScanWorkspaceAsync(visitor, filter, flags, directoryQueue, ctx));

				await pipeline.WaitForCompletionAsync();
			}
		}

		class StatusVisitor : Visitor
		{
			public bool AnyChanges { get; private set; }

			readonly Lock _lockObject = new();

			public override void DirectoryAdded(string path, DirectoryState parentDir, DirectoryInfo subDirInfo)
				=> WriteLine($"{FormatPath(path)}/ (added)", ConsoleColor.Green);

			public override void DirectoryRemoved(string path, DirectoryState dirState)
				=> WriteLine($"{FormatPath(path)}/ (removed)", ConsoleColor.Red);

			public override void FileAdded(string path, DirectoryState parentDir, FileInfo fileInfo)
				=> WriteLine($"{FormatPath(path)} (added)", ConsoleColor.Green);

			public override void FileModified(string path, FileState fileState, FileInfo fileInfo)
				=> WriteLine($"{FormatPath(path)} (modified)", ConsoleColor.Yellow);

			public override void FileUnmodified(string path, FileState fileState, FileInfo fileInfo)
				=> WriteLine(FormatPath(path), null);

			public override void FileRemoved(string path, FileState fileState)
				=> WriteLine($"{FormatPath(path)} (removed)", ConsoleColor.Red);

			static string FormatPath(string path)
				=> path.Replace(Path.DirectorySeparatorChar, '/');

			void WriteLine(string text, ConsoleColor? color)
			{
				lock (_lockObject)
				{
					AnyChanges = true;

					if (color == null)
					{
						Console.WriteLine(text);
					}
					else
					{
						Console.ForegroundColor = color.Value;
						Console.WriteLine(text);
						Console.ResetColor();
					}
				}
			}
		}

		#endregion

		#region Reconcile

		/// <summary>
		/// Updates the status of files in this workspace based on current filesystem metadata
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task ReconcileAsync(CancellationToken cancellationToken = default)
			=> TrackAsync(WorkspaceLayerId.Default, null, cancellationToken);

		/// <summary>
		/// Updates the status of files in this workspace based on current filesystem metadata
		/// </summary>
		/// <param name="layerId">Layer to add new files to</param>
		/// <param name="filter">Filter for the files to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task TrackAsync(WorkspaceLayerId layerId, FileFilter? filter = null, CancellationToken cancellationToken = default)
		{
			LayerFlags layer = GetLayerFlag(layerId);

			filter ??= new FileFilter(FileFilterType.Include);

			byte[] buffer = new byte[2 * 1024 * 1024];
			await using (AsyncPipeline pipeline = new(cancellationToken))
			{
				using SemaphoreSlim semaphore = new(1);
				ParallelBuzHashChunker chunker = new(pipeline, buffer, LeafChunkedDataNodeOptions.Default);

				ChannelWithRefCount<DirectoryState> dirsToScan = Channel.CreateUnbounded<DirectoryState>().WithRefCount();
				await dirsToScan.Writer.WriteAsync(_rootDirState, cancellationToken);
				dirsToScan.OnComplete += () => chunker.SourceWriter.Complete();

				_ = pipeline.AddTasks(16, token => FindFilesToReconcileAsync(filter, dirsToScan, layer, semaphore, chunker, token));
				_ = pipeline.AddTask(chunker.OutputReader, (output, token) => RegisterChunksAsync(output, layer, semaphore, token));

				await pipeline.WaitForCompletionAsync();
			}

			RemoveUnusedHashInfoObjects();
			await SaveAsync(cancellationToken);
		}

		static bool IsRelativeDir(string name)
			=> (name.Length == 1 && name[0] == '.') || (name.Length == 2 && name[0] == '.' && name[1] == '.');

		record class FileToAdd(DirectoryState ParentDir, FileInfo FileInfo);

		async Task FindFilesToReconcileAsync(FileFilter filter, ChannelWithRefCount<DirectoryState> channel, LayerFlags layer, SemaphoreSlim semaphore, ContentChunker chunker, CancellationToken cancellationToken)
		{
			await foreach (DirectoryState dirState in channel.Reader.ReadAllAsync(cancellationToken))
			{
				if (dirState == _hordeDirState)
				{
					channel.Release();
					continue;
				}

				DirectoryInfo dirInfo = GetDirectoryReference(dirState).ToDirectoryInfo();
				FileSystemInfo[] fileSystemInfos = dirInfo.GetFileSystemInfos();

				List<DirectoryState> addedDirectories = [];
				List<FileInfo> addedFiles = [];
				using (IDisposable stateLock = await semaphore.WaitDisposableAsync(cancellationToken))
				{
					// Duplicate the current files and subdirectories before we add anything new
					FileState[] files = [.. dirState.Files];
					DirectoryState[] subDirStates = [.. dirState.Directories];

					// Enumerate everything in this directory
					Dictionary<string, FileInfo> nameToFileInfo = new(StringComparer.Ordinal);
					Dictionary<string, DirectoryInfo> nameToDirectoryInfo = new(StringComparer.Ordinal);
					foreach (FileSystemInfo fileSystemInfo in fileSystemInfos)
					{
						if (fileSystemInfo is FileInfo fileInfo)
						{
							nameToFileInfo.Add(fileInfo.Name, fileInfo);
						}
						else if (fileSystemInfo is DirectoryInfo directoryInfo && !IsRelativeDir(fileSystemInfo.Name))
						{
							nameToDirectoryInfo.Add(directoryInfo.Name, directoryInfo);
						}
					}

					// Add new directories
					foreach (DirectoryInfo subDirInfo in nameToDirectoryInfo.Values)
					{
						if (!dirState.TryGetDirectoryCaseSensitive(subDirInfo.Name, out _))
						{
							string path = dirState.GetChildPath(subDirInfo.Name);
							if (filter.Matches(path))
							{
								DirectoryState subDirState = dirState.AddDirectory(subDirInfo.Name, layer);
								_logger.LogInformation("Adding {Dir}", GetDirectoryReference(subDirState));
								addedDirectories.Add(subDirState);
							}
						}
					}

					// Add new files
					foreach (FileInfo fileInfo in nameToFileInfo.Values)
					{
						if (!dirState.TryGetFileCaseSensitive(fileInfo.Name, out _))
						{
							string path = dirState.GetChildPath(fileInfo.Name);
							if (filter.Matches(path))
							{
								_logger.LogInformation("Adding {File}", new FileReference(fileInfo));
								addedFiles.Add(fileInfo);
							}
						}
					}

					// Find all the files to remove
					foreach (FileState fileState in files)
					{
						string path = fileState.GetPath();
						if (filter.Matches(path))
						{
							if (!nameToFileInfo.TryGetValue(fileState.Name, out FileInfo? fileInfo))
							{
								_logger.LogInformation("Removing {File}", GetFileReference(fileState));
								RemoveFileFromHashLookup(fileState);
								fileState.Unlink();
							}
							else if (fileState.IsModified(fileInfo))
							{
								_logger.LogInformation("Updating {File}", new FileReference(fileInfo));

								RemoveFileFromHashLookup(fileState);
								fileState.Unlink();

								addedFiles.Add(fileInfo);
							}
						}
					}

					// Find directories that do not exist any more
					foreach (DirectoryState subDirState in subDirStates)
					{
						string path = subDirState.GetPath();
						if (!nameToDirectoryInfo.ContainsKey(subDirState.Name))
						{
							if (filter.Matches(path))
							{
								_logger.LogInformation("Removing {File}", GetDirectoryReference(subDirState));

								RemoveDirectoryFromHashLookup(subDirState);
								subDirState.Unlink();
							}
						}
						else
						{
							if (filter.PossiblyMatches(path))
							{
								addedDirectories.Add(subDirState);
							}
						}
					}
				}

				foreach (DirectoryState addedDirectory in addedDirectories)
				{
					channel.AddRef();
					await channel.Writer.WriteAsync(addedDirectory, cancellationToken);
				}

				foreach (FileInfo addedFile in addedFiles)
				{
					FileChunkerSource fileSource = new(addedFile, new FileToAdd(dirState, addedFile));
					await chunker.SourceWriter.WriteAsync(fileSource, cancellationToken);
				}

				channel.Release();
			}
		}

		async ValueTask RegisterChunksAsync(ChunkerOutput output, LayerFlags layer, SemaphoreSlim stateSema, CancellationToken cancellationToken)
		{
			FileToAdd addedFile = (FileToAdd)output.UserData!;

			using Blake3.Hasher hasher = Blake3.Hasher.New();

			List<(IoHash, long)> chunks = [];
			while (await output.MoveNextAsync(cancellationToken))
			{
				IoHash chunkHash = IoHash.Compute(output.Data.Span);
				chunks.Add((chunkHash, output.Data.Length));
				hasher.Update(output.Data.Span);
			}

			IoHash hash = IoHash.FromBlake3(hasher);

			using (IDisposable stateLock = await stateSema.WaitDisposableAsync(cancellationToken))
			{
				FileState fileState = addedFile.ParentDir.AddFile(addedFile.FileInfo.Name, hash, layer);
				fileState.Update(addedFile.FileInfo);

				HashInfo fileHashInfo = FindOrAddHashInfo(hash, addedFile.FileInfo.Length);
				fileHashInfo.Files.Add(fileState);

				long offset = 0;
				foreach ((IoHash chunkHash, long length) in chunks)
				{
					HashInfo chunkHashInfo = FindOrAddHashInfo(chunkHash, length);
					chunkHashInfo.AddChunk(new ChunkInfo(fileHashInfo, offset, length));
					offset += length;
				}
			}
		}

		#endregion

		#region Verify

		/// <summary>
		/// Checks that all files within the workspace have the correct hash
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task VerifyAsync(CancellationToken cancellationToken = default)
		{
			await VerifyAsync(_rootDirState, cancellationToken);
		}

		async Task VerifyAsync(DirectoryState dirState, CancellationToken cancellationToken)
		{
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				await VerifyAsync(subDirState, cancellationToken);
			}
			foreach (FileState fileState in dirState.Files)
			{
				FileInfo fileInfo = GetFileInfo(fileState);
				if (fileState.IsModified(fileInfo))
				{
					throw new WorkspaceException($"File {fileInfo.FullName} has been modified");
				}

				IoHash hash;
				using (FileStream stream = GetFileInfo(fileState).OpenRead())
				{
					hash = await IoHash.ComputeAsync(stream, cancellationToken);
				}

				if (hash != fileState.Hash)
				{
					throw new WorkspaceException($"Hash for {fileInfo.FullName} was {hash}, expected {fileState.Hash}");
				}
			}
		}

		#endregion

		void AddDirToHashLookup(DirectoryState dirState)
		{
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				AddDirToHashLookup(subDirState);
			}
			foreach (FileState fileState in dirState.Files)
			{
				AddFileToHashLookup(fileState);
			}
		}

		HashInfo FindOrAddHashInfo(IoHash hash, long length)
		{
			HashInfo? hashInfo;
			if (!_hashes.TryGetValue(hash, out hashInfo))
			{
				hashInfo = new HashInfo(hash, length);
				_hashes.Add(hash, hashInfo);
			}
			return hashInfo;
		}

		HashInfo AddFileToHashLookup(FileState file)
		{
			HashInfo hashInfo = FindOrAddHashInfo(file.Hash, file.Length);
			hashInfo.Files.Add(file);
			return hashInfo;
		}

		void RemoveFileFromHashLookup(FileState file)
		{
			HashInfo? hashInfo;
			if (_hashes.TryGetValue(file.Hash, out hashInfo))
			{
				hashInfo.Files.Remove(file);
			}
		}

		void RemoveDirectoryFromHashLookup(DirectoryState dirState)
		{
			foreach (DirectoryState subDirState in dirState.Directories)
			{
				RemoveDirectoryFromHashLookup(subDirState);
			}
			foreach (FileState fileState in dirState.Files)
			{
				RemoveFileFromHashLookup(fileState);
			}
		}

		#region Directory operations

		void DeleteDirectory(DirectoryState dirState)
		{
			DirectoryReference dir = GetDirectoryReference(dirState);
			FileUtils.ForceDeleteDirectory(dir);

			dirState.Unlink();
		}

		DirectoryReference GetDirectoryReference(DirectoryState dirState)
		{
			StringBuilder builder = new(_rootDir.FullName);
			dirState.AppendPath(builder);
			return new DirectoryReference(builder.ToString(), DirectoryReference.Sanitize.None);
		}

		#endregion

		#region File operations

		void MoveFileToCache(FileState fileState)
		{
			DirectoryState cacheDirState = _cacheDirState;

			string name = Guid.NewGuid().ToString("N");
			for (int idx = 0; idx < 2; idx++)
			{
				cacheDirState = cacheDirState.FindOrAddDirectory(name.Substring(idx * 2, 2), LayerFlags.Cache);
			}

			FileInfo fileInfo = GetFileInfo(fileState);
			if (fileInfo.Exists)
			{
				MoveFile(fileState, cacheDirState, name, LayerFlags.Cache);
			}
		}

		void MoveFile(FileState fileState, DirectoryState targetDirState, string targetName, LayerFlags targetLayers)
		{
			FileReference sourceFile = GetFileReference(fileState);
			fileState.MoveTo(targetDirState, targetName, targetLayers);

			FileReference targetFile = GetFileReference(fileState);
			DirectoryReference.CreateDirectory(targetFile.Directory);
			FileReference.Move(sourceFile, targetFile, true);

			_logger.LogDebug("Moving file from {Source} to {Target}", sourceFile, targetFile);
		}

		void DeleteFile(FileState fileState)
		{
			RemoveFileFromHashLookup(fileState);

			FileReference file = GetFileReference(fileState);
			FileUtils.ForceDeleteFile(file);

			fileState.Unlink();
		}

		FileInfo GetFileInfo(FileState fileState)
		{
			return GetFileReference(fileState).ToFileInfo();
		}

		FileReference GetFileReference(FileState fileState)
		{
			StringBuilder builder = new(_rootDir.FullName);
			fileState.AppendPath(builder);
			return new FileReference(builder.ToString(), FileReference.Sanitize.None);
		}

		#endregion
	}

	/// <summary>
	/// Flags for showing modified flags in a workspace 
	/// </summary>
	[Flags]
	public enum WorkspaceStatusFlags
	{
		/// <summary>
		/// Include nothing
		/// </summary>
		None = 0,

		/// <summary>
		/// Directories which have been added
		/// </summary>
		DirectoryAdded = 1,

		/// <summary>
		/// Directories which have been removed
		/// </summary>
		DirectoryRemoved = 2,

		/// <summary>
		/// Files which have been added
		/// </summary>
		FileAdded = 4,

		/// <summary>
		/// Files which have been removed
		/// </summary>
		FileRemoved = 8,

		/// <summary>
		/// Files which have been modified
		/// </summary>
		FileModified = 16,

		/// <summary>
		/// Files which have not been modified
		/// </summary>
		FileUnmodified = 32,

		/// <summary>
		/// Default flags for status calls
		/// </summary>
		Default = DirectoryAdded | DirectoryRemoved | FileAdded | FileRemoved | FileModified,
	}

	/// <summary>
	/// Stats for a sync
	/// </summary>
	public class WorkspaceSyncStats
	{
		/// <summary>
		/// Number of identical files that were kept
		/// </summary>
		public int NumFilesKept { get; set; }

		/// <summary>
		/// Number of files that were moved during a sync operation
		/// </summary>
		public int NumFilesMoved { get; set; }

		/// <summary>
		/// Number of bytes that were copied from elsewhere in the tree
		/// </summary>
		public long NumBytesReused { get; set; }

		/// <summary>
		/// Number of bytes that were downloaded
		/// </summary>
		public long NumBytesDownloaded { get; set; }
	}
}
