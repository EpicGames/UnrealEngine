// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;

namespace EpicGames.Horde.Storage
{
	/// <summary>
	/// Identifier for a workspace layer
	/// </summary>
	/// <param name="Id">Identifier for the layer</param>
	[LogValueType]
	[TypeConverter(typeof(StringIdTypeConverter<WorkspaceLayerId, WorkspaceLayerIdConverter>))]
	[BinaryIdConverter(typeof(WorkspaceLayerIdConverter))]
	public readonly record struct WorkspaceLayerId(StringId Id)
	{
		/// <summary>
		/// Name of the default layer
		/// </summary>
		public static WorkspaceLayerId Default { get; } = new WorkspaceLayerId(new StringId("default"));

		/// <summary>
		/// Constructor
		/// </summary>
		public WorkspaceLayerId(string id)
			: this(new StringId(id))
		{ }

		/// <summary>
		/// Parse an identifier
		/// </summary>
		public static bool TryParse(string text, out WorkspaceLayerId id, [NotNullWhen(false)] out string? errorMessage)
		{
			StringId stringId;
			if (StringId.TryParse(text, out stringId, out errorMessage))
			{
				id = new WorkspaceLayerId(stringId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter class to and from ObjectId values
	/// </summary>
	class WorkspaceLayerIdConverter : StringIdConverter<WorkspaceLayerId>
	{
		/// <inheritdoc/>
		public override WorkspaceLayerId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(WorkspaceLayerId value) => value.Id;
	}
}
