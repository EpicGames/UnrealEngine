// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;

namespace EpicGames.Horde.Streams
{
	/// <summary>
	/// Exception thrown when stream validation fails
	/// </summary>
	public class InvalidStreamException : Exception
	{
		/// <inheritdoc/>
		public InvalidStreamException()
		{
		}

		/// <inheritdoc/>
		public InvalidStreamException(string message) : base(message)
		{
		}

		/// <inheritdoc/>
		public InvalidStreamException(string message, Exception innerEx) : base(message, innerEx)
		{
		}
	}

	/// <summary>
	/// Information about a stream
	/// </summary>
	public interface IStream
	{
		/// <summary>
		/// Name of the stream.
		/// </summary>
		StreamId Id { get; }

		/// <summary>
		/// Project that this stream belongs to
		/// </summary>
		ProjectId ProjectId { get; }

		/// <summary>
		/// Name of the stream
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Path to the config file for this stream
		/// </summary>
		string? ConfigPath { get; }

		/// <summary>
		/// Current revision of the config file
		/// </summary>
		string ConfigRevision { get; }

		/// <summary>
		/// Order for this stream on the dashboard
		/// </summary>
		int Order { get; }

		/// <summary>
		/// Notification channel for all jobs in this stream
		/// </summary>
		string? NotificationChannel { get; }

		/// <summary>
		/// Notification channel filter for this template. Can be Success, Failure, or Warnings.
		/// </summary>
		string? NotificationChannelFilter { get; }

		/// <summary>
		/// Channel to post issue triage notifications
		/// </summary>
		string? TriageChannel { get; }

		/// <summary>
		/// Tabs for this stream on the dashboard
		/// </summary>
		IReadOnlyList<IStreamTab> Tabs { get; }

		/// <summary>
		/// Agent types configured for this stream
		/// </summary>
		IReadOnlyDictionary<string, IAgentType> AgentTypes { get; }

		/// <summary>
		/// Workspace types configured for this stream
		/// </summary>
		IReadOnlyDictionary<string, IWorkspaceType> WorkspaceTypes { get; }

		/// <summary>
		/// List of templates available for this stream
		/// </summary>
		IReadOnlyDictionary<TemplateId, ITemplateRef> Templates { get; }

		/// <summary>
		/// Workflows configured for this stream
		/// </summary>
		IReadOnlyList<IWorkflow> Workflows { get; }

		/// <summary>
		/// Stream aliases configured for this stream.
		/// Each alias provides a named subset of the stream's templates.
		/// </summary>
		IReadOnlyList<IStreamAlias> Aliases { get; }

		/// <summary>
		/// Default settings for preflights against this stream
		/// </summary>
		IDefaultPreflight? DefaultPreflight { get; }

		/// <summary>
		/// Stream is paused for builds until specified time
		/// </summary>
		DateTime? PausedUntil { get; }

		/// <summary>
		/// Comment/reason for why the stream was paused
		/// </summary>
		string? PauseComment { get; }

		/// <summary>
		/// Commits for this stream
		/// </summary>
		ICommitCollection Commits { get; }

		/// <summary>
		/// Get the latest stream state
		/// </summary>
		/// <param name="cancellationToken">Cancellation toke for this operation</param>
		/// <returns>Updated stream, or null if it no longer exists</returns>
		Task<IStream?> RefreshAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates user-facing properties for an existing stream
		/// </summary>
		/// <param name="newPausedUntil">The new datetime for pausing builds</param>
		/// <param name="newPauseComment">The reason for pausing</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated stream if successful, null otherwise</returns>
		Task<IStream?> TryUpdatePauseStateAsync(DateTime? newPausedUntil, string? newPauseComment, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to update the last trigger time for a schedule
		/// </summary>
		/// <param name="templateRefId">The template ref id</param>
		/// <param name="lastTriggerTimeUtc">New last trigger time for the schedule</param>
		/// <param name="lastTriggerCommitId">New last trigger commit for the schedule</param>
		/// <param name="newActiveJobs">New list of active jobs</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated stream if successful, null otherwise</returns>
		Task<IStream?> TryUpdateScheduleTriggerAsync(TemplateId templateRefId, DateTime? lastTriggerTimeUtc, CommitIdWithOrder? lastTriggerCommitId, List<JobId> newActiveJobs, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to update a stream template ref
		/// </summary>
		/// <param name="templateRefId">The template ref to update</param>
		/// <param name="stepStates">The stream states to update, pass an empty list to clear all step states, otherwise will be a partial update based on included step updates</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IStream?> TryUpdateTemplateRefAsync(TemplateId templateRefId, List<UpdateStepStateRequest>? stepStates = null, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Style for rendering a tab
	/// </summary>
	public enum TabStyle
	{
		/// <summary>
		/// Regular job list
		/// </summary>
		Normal,

		/// <summary>
		/// Omit job names, show condensed view
		/// </summary>
		Compact,
	}

	/// <summary>
	/// Information about a page to display in the dashboard for a stream
	/// </summary>
	public interface IStreamTab
	{
		/// <summary>
		/// Title of this page
		/// </summary>
		string Title { get; }

		/// <summary>
		/// Type of this tab
		/// </summary>
		string Type { get; }

		/// <summary>
		/// Presentation style for this page
		/// </summary>
		TabStyle Style { get; }

		/// <summary>
		/// Whether to show job names on this page
		/// </summary>
		bool ShowNames { get; }

		/// <summary>
		/// Whether to show all user preflights 
		/// </summary>
		bool? ShowPreflights { get; }

		/// <summary>
		/// Names of jobs to include on this page. If there is only one name specified, the name column does not need to be displayed.
		/// </summary>
		IReadOnlyList<string>? JobNames { get; }

		/// <summary>
		/// List of job template names to show on this page.
		/// </summary>
		IReadOnlyList<TemplateId>? Templates { get; }

		/// <summary>
		/// Columns to display for different types of aggregates
		/// </summary>
		IReadOnlyList<IStreamTabColumn>? Columns { get; }
	}

	/// <summary>
	/// Type of a column in a jobs tab
	/// </summary>
	public enum TabColumnType
	{
		/// <summary>
		/// Contains labels
		/// </summary>
		Labels,

		/// <summary>
		/// Contains parameters
		/// </summary>
		Parameter
	}

	/// <summary>
	/// Describes a column to display on the jobs page
	/// </summary>
	public interface IStreamTabColumn
	{
		/// <summary>
		/// The type of column
		/// </summary>
		TabColumnType Type { get; }

		/// <summary>
		/// Heading for this column
		/// </summary>
		string Heading { get; }

		/// <summary>
		/// Category of aggregates to display in this column. If null, includes any aggregate not matched by another column.
		/// </summary>
		string? Category { get; }

		/// <summary>
		/// Parameter to show in this column
		/// </summary>
		string? Parameter { get; }

		/// <summary>
		/// Relative width of this column.
		/// </summary>
		int? RelativeWidth { get; }
	}

	/// <summary>
	/// Mapping from a BuildGraph agent type to a set of machines on the farm
	/// </summary>
	public interface IAgentType
	{
		/// <summary>
		/// Pool of agents to use for this agent type
		/// </summary>
		PoolId Pool { get; }

		/// <summary>
		/// Name of the workspace to sync
		/// </summary>
		string? Workspace { get; }

		/// <summary>
		/// Path to the temporary storage dir
		/// </summary>
		string? TempStorageDir { get; }

		/// <summary>
		/// Environment variables to be set when executing the job
		/// </summary>
		IReadOnlyDictionary<string, string>? Environment { get; }
	}

	/// <summary>
	/// Information about a workspace type
	/// </summary>
	public interface IWorkspaceType
	{
		/// <summary>
		/// Name of the Perforce server cluster to use
		/// </summary>
		string? Cluster { get; }

		/// <summary>
		/// The Perforce server and port (eg. perforce:1666)
		/// </summary>
		string? ServerAndPort { get; }

		/// <summary>
		/// User to log into Perforce with (defaults to buildmachine)
		/// </summary>
		string? UserName { get; }

		/// <summary>
		/// Identifier to distinguish this workspace from other workspaces. Defaults to the workspace type name.
		/// </summary>
		string? Identifier { get; }

		/// <summary>
		/// Override for the stream to sync
		/// </summary>
		string? Stream { get; }

		/// <summary>
		/// Custom view for the workspace
		/// </summary>
		IReadOnlyList<string>? View { get; }

		/// <summary>
		/// Whether to use an incrementally synced workspace
		/// </summary>
		bool? Incremental { get; }

		/// <summary>
		/// Whether to use the AutoSDK
		/// </summary>
		bool? UseAutoSdk { get; }

		/// <summary>
		/// View for the AutoSDK paths to sync. If null, the whole thing will be synced.
		/// </summary>
		IReadOnlyList<string>? AutoSdkView { get; }

		/// <summary>
		/// Method to use when syncing/materializing data from Perforce
		/// </summary>
		string? Method { get; }

		/// <summary>
		/// Minimum disk space that must be available *after* syncing this workspace (in megabytes)
		/// If not available, the job will be aborted.
		/// </summary>
		long? MinScratchSpace { get; }

		/// <summary>
		/// Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk.
		/// Set to null or 0 to disable.
		/// </summary>
		long? ConformDiskFreeSpace { get; }
	}

	/// <summary>
	/// Specifies defaults for running a preflight
	/// </summary>
	public interface IDefaultPreflight
	{
		/// <summary>
		/// The template id to query
		/// </summary>
		TemplateId? TemplateId { get; }

		/// <summary>
		/// Query for the change to use
		/// </summary>
		IChangeQuery? Change { get; }
	}

	/// <summary>
	/// Represents a named alias for a stream with a subset of its templates
	/// </summary>
	public interface IStreamAlias
	{
		/// <summary>
		/// The unique name/identifier for this alias
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The subset of template IDs available through this alias.
		/// These must be a subset of the enclosing IStream's Templates.
		/// </summary>
		IReadOnlyList<TemplateId> Templates { get; }
	}

	/// <summary>
	/// Job template in a stream
	/// </summary>
	public interface ITemplateRef
	{
		/// <summary>
		/// The template id
		/// </summary>
		TemplateId Id { get; }

		/// <summary>
		/// Whether to show badges in UGS for these jobs
		/// </summary>
		bool ShowUgsBadges { get; }

		/// <summary>
		/// Whether to show alerts in UGS for these jobs
		/// </summary>
		bool ShowUgsAlerts { get; }

		/// <summary>
		/// Notification channel for this template. Overrides the stream channel if set.
		/// </summary>
		string? NotificationChannel { get; }

		/// <summary>
		/// Notification channel filter for this template. Can be a combination of "Success", "Failure" and "Warnings" separated by pipe characters.
		/// </summary>
		string? NotificationChannelFilter { get; }

		/// <summary>
		/// Triage channel for this template. Overrides the stream channel if set.
		/// </summary>
		string? TriageChannel { get; }

		/// <summary>
		/// List of schedules for this template
		/// </summary>
		ISchedule? Schedule { get; }

		/// <summary>
		/// List of chained job triggers
		/// </summary>
		IReadOnlyList<IChainedJobTemplate>? ChainedJobs { get; }

		/// <summary>
		/// List of template step states
		/// </summary>
		IReadOnlyList<ITemplateStep> StepStates { get; }

		/// <summary>
		/// Default change to use for this job
		/// </summary>
		IReadOnlyList<IChangeQuery>? DefaultChange { get; }
	}

	/// <summary>
	/// Trigger for another template
	/// </summary>
	public interface IChainedJobTemplate
	{
		/// <summary>
		/// Name of the target that needs to complete before starting the other template
		/// </summary>
		string Trigger { get; }

		/// <summary>
		/// Id of the template to trigger
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// Whether to use the default change for the template rather than the change for the parent job.
		/// </summary>
		bool UseDefaultChangeForTemplate { get; }
	}

	/// <summary>
	/// Information about a paused template step
	/// </summary>
	public interface ITemplateStep
	{
		/// <summary>
		/// Name of the step
		/// </summary>
		string Name { get; }

		/// <summary>
		/// User who paused the step
		/// </summary>
		UserId PausedByUserId { get; }

		/// <summary>
		/// The UTC time when the step was paused
		/// </summary>
		DateTime PauseTimeUtc { get; }
	}

	/// <summary>
	/// Extension methods for streams
	/// </summary>
	public static class StreamExtensions
	{
		/// <summary>
		/// Updates an existing stream
		/// </summary>
		/// <param name="stream">The stream to update</param>
		/// <param name="newPausedUntil">The new datetime for pausing builds</param>
		/// <param name="newPauseComment">The reason for pausing</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task object</returns>
		public static async Task<IStream?> TryUpdatePauseStateAsync(this IStream? stream, DateTime? newPausedUntil = null, string? newPauseComment = null, CancellationToken cancellationToken = default)
		{
			for (; stream != null; stream = await stream.RefreshAsync(cancellationToken))
			{
				IStream? newStream = await stream.TryUpdatePauseStateAsync(newPausedUntil, newPauseComment, cancellationToken);
				if (newStream != null)
				{
					return newStream;
				}
			}
			return null;
		}

		/// <summary>
		/// Attempts to update the last trigger time for a schedule
		/// </summary>
		/// <param name="stream">The stream to update</param>
		/// <param name="templateRefId">The template ref id</param>
		/// <param name="lastTriggerTimeUtc"></param>
		/// <param name="lastTriggerCommitId"></param>
		/// <param name="addJobs">Jobs to add</param>
		/// <param name="removeJobs">Jobs to remove</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the stream was updated</returns>
		public static async Task<IStream?> TryUpdateScheduleTriggerAsync(this IStream stream, TemplateId templateRefId, DateTime? lastTriggerTimeUtc = null, CommitIdWithOrder? lastTriggerCommitId = null, List<JobId>? addJobs = null, List<JobId>? removeJobs = null, CancellationToken cancellationToken = default)
		{
			IStream? newStream = stream;
			while (newStream != null)
			{
				ITemplateRef? templateRef;
				if (!newStream.Templates.TryGetValue(templateRefId, out templateRef))
				{
					break;
				}
				if (templateRef.Schedule == null)
				{
					break;
				}

				IEnumerable<JobId> newActiveJobs = templateRef.Schedule.ActiveJobs;
				if (removeJobs != null)
				{
					newActiveJobs = newActiveJobs.Except(removeJobs);
				}
				if (addJobs != null)
				{
					newActiveJobs = newActiveJobs.Union(addJobs);
				}

				newStream = await newStream.TryUpdateScheduleTriggerAsync(templateRefId, lastTriggerTimeUtc, lastTriggerCommitId, [.. newActiveJobs], cancellationToken);

				if (newStream != null)
				{
					return newStream;
				}

				newStream = await stream.RefreshAsync(cancellationToken);
			}
			return null;
		}

		/// <summary>
		/// Check if stream is paused for new builds
		/// </summary>
		/// <param name="stream">The stream object</param>
		/// <param name="currentTime">Current time (allow tests to pass in a fake clock)</param>
		/// <returns>If stream is paused</returns>
		public static bool IsPaused(this IStream stream, DateTime currentTime)
		{
			return stream.PausedUntil != null && stream.PausedUntil > currentTime;
		}
	}
}
