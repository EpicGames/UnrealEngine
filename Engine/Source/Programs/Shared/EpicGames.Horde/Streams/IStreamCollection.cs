// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Streams
{
	/// <summary>
	/// Collection of stream documents
	/// </summary>
	public interface IStreamCollection
	{
		/// <summary>
		/// Gets a stream by ID
		/// </summary>
		/// <param name="id">The stream identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The stream document</returns>
		Task<IStream?> GetAsync(StreamId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a stream by ID
		/// </summary>
		/// <param name="ids">The stream identifiers</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The stream document</returns>
		Task<IReadOnlyList<IStream>> FindAsync(IReadOnlyList<StreamId>? ids = null, CancellationToken cancellationToken = default);
	}
}
