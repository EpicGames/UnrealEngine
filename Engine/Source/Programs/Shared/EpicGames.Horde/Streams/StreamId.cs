// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Streams
{
	/// <summary>
	/// Identifier for a stream
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<StreamId, StreamIdConverter>))]
	[StringIdConverter(typeof(StreamIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<StreamId, StreamIdConverter>))]
	public readonly record struct StreamId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public StreamId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	public class StreamIdConverter : StringIdConverter<StreamId>
	{
		/// <inheritdoc/>
		public override StreamId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(StreamId value) => value.Id;
	}
}
