// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using EpicGames.Core;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Streams
{
	/// <summary>
	/// Response describing a stream
	/// </summary>
	public class GetStreamResponse
	{
		/// <summary>
		/// Unique id of the stream
		/// </summary>
		public StreamId Id { get; set; }

		/// <summary>
		/// Unique id of the project containing this stream
		/// </summary>
		public ProjectId ProjectId { get; set; }

		/// <summary>
		/// Name of the stream
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// The config file path on the server
		/// </summary>
		public string ConfigPath { get; set; } = String.Empty;

		/// <summary>
		/// Revision of the config file 
		/// </summary>
		public string ConfigRevision { get; set; } = String.Empty;

		/// <summary>
		/// Order to display in the list
		/// </summary>
		public int Order { get; set; }

		/// <summary>
		/// Notification channel for all jobs in this stream
		/// </summary>
		public string? NotificationChannel { get; set; }

		/// <summary>
		/// Notification channel filter for this template. Can be a combination of "Success", "Failure" and "Warnings" separated by pipe characters.
		/// </summary>
		public string? NotificationChannelFilter { get; set; }

		/// <summary>
		/// Channel to post issue triage notifications
		/// </summary>
		public string? TriageChannel { get; set; }

		/// <summary>
		/// Default template for running preflights
		/// </summary>
		public string? DefaultPreflightTemplate { get; set; }

		/// <summary>
		/// Default template to use for preflights
		/// </summary>
		public DefaultPreflightMessage? DefaultPreflight { get; set; }

		/// <summary>
		/// List of tabs to display for this stream
		/// </summary>
		public List<GetStreamTabResponse> Tabs { get; set; }

		/// <summary>
		/// Map of agent name to type
		/// </summary>
		public Dictionary<string, GetAgentTypeResponse> AgentTypes { get; set; }

		/// <summary>
		/// Map of workspace name to type
		/// </summary>
		public Dictionary<string, GetWorkspaceTypeResponse>? WorkspaceTypes { get; set; }

		/// <summary>
		/// Templates for jobs in this stream
		/// </summary>
		public List<GetTemplateRefResponse> Templates { get; set; }

		/// <summary>
		/// Stream paused for new builds until this date
		/// </summary>
		public DateTime? PausedUntil { get; set; }

		/// <summary>
		/// Reason for stream being paused
		/// </summary>
		public string? PauseComment { get; set; }

		/// <summary>
		/// Workflows for this stream
		/// </summary>
		public List<GetWorkflowResponse> Workflows { get; set; }

		/// <summary>
		/// Stream aliases for this stream
		/// </summary>
		public List<GetStreamAliasResponse> Aliases { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="stream">The stream to construct from</param>
		/// <param name="templates">Templates for this stream</param>
		public GetStreamResponse(IStream stream, List<GetTemplateRefResponse> templates)
		{
			Id = stream.Id;
			ProjectId = stream.ProjectId;
			Name = stream.Name;
			ConfigPath = stream.ConfigPath ?? String.Empty;
			ConfigRevision = stream.ConfigRevision;
			Order = stream.Order;
			NotificationChannel = stream.NotificationChannel;
			NotificationChannelFilter = stream.NotificationChannelFilter;
			TriageChannel = stream.TriageChannel;
			DefaultPreflightTemplate = stream.DefaultPreflight?.TemplateId?.ToString();
			DefaultPreflight = (stream.DefaultPreflight == null) ? null : new DefaultPreflightMessage(stream.DefaultPreflight);
			Tabs = stream.Tabs.ConvertAll(x => new GetStreamTabResponse(x));
			AgentTypes = stream.AgentTypes.ToDictionary(x => x.Key, x => new GetAgentTypeResponse(x.Value));
			WorkspaceTypes = stream.WorkspaceTypes.ToDictionary(x => x.Key, x => new GetWorkspaceTypeResponse(x.Value));
			Templates = templates;
			Workflows = stream.Workflows.ConvertAll(x => new GetWorkflowResponse(x));
			Aliases = [.. stream.Aliases.Select(x => new GetStreamAliasResponse(x))];
			PausedUntil = stream.PausedUntil;
			PauseComment = stream.PauseComment;
		}
	}

	/// <summary>
	/// Response describing a stream alias
	/// </summary>
	public class GetStreamAliasResponse : IStreamAlias
	{
		/// <inheritdoc/>
		public string Name { get; set; }

		/// <inheritdoc cref="IStreamAlias.Templates"/>
		public List<TemplateId> Templates { get; set; }
		IReadOnlyList<TemplateId> IStreamAlias.Templates => Templates;

		/// <summary>
		/// Default constructor for serialization
		/// </summary>
		public GetStreamAliasResponse()
		{
			Name = String.Empty;
			Templates = [];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="alias">The stream alias to construct from</param>
		public GetStreamAliasResponse(IStreamAlias alias)
		{
			Name = alias.Name;
			Templates = [.. alias.Templates];
		}
	}

	/// <summary>
	/// Information about the default preflight to run
	/// </summary>
	public class DefaultPreflightMessage : IDefaultPreflight
	{
		/// <inheritdoc/>
		public TemplateId? TemplateId { get; set; }

		/// <inheritdoc/>
		public ChangeQueryMessage? Change { get; set; }
		IChangeQuery? IDefaultPreflight.Change => Change;

		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultPreflightMessage()
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultPreflightMessage(IDefaultPreflight other)
		{
			TemplateId = other.TemplateId;
			Change = (other.Change == null) ? null : new ChangeQueryMessage(other.Change);
		}
	}

	/// <summary>
	/// Information about a page to display in the dashboard for a stream
	/// </summary>
	public class GetStreamTabResponse : IStreamTab
	{
		/// <inheritdoc/>
		public string Title { get; set; }

		/// <inheritdoc/>
		public string Type { get; set; }

		/// <inheritdoc/>
		public TabStyle Style { get; set; }

		/// <inheritdoc/>
		public bool ShowNames { get; set; }

		/// <inheritdoc/>
		public bool? ShowPreflights { get; set; }

		/// <inheritdoc cref="IStreamTab.JobNames"/>
		public List<string>? JobNames { get; set; }
		IReadOnlyList<string>? IStreamTab.JobNames => JobNames;

		/// <inheritdoc cref="IStreamTab.Templates"/>
		public List<TemplateId>? Templates { get; set; }
		IReadOnlyList<TemplateId>? IStreamTab.Templates => Templates;

		/// <inheritdoc cref="IStreamTab.Columns"/>
		public List<GetStreamTabColumnResponse>? Columns { get; set; }
		IReadOnlyList<IStreamTabColumn>? IStreamTab.Columns => Columns;

		/// <summary>
		/// Constructor
		/// </summary>
		public GetStreamTabResponse()
		{
			Title = String.Empty;
			Type = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetStreamTabResponse(IStreamTab other)
		{
			Title = other.Title;
			Type = other.Type;
			Style = other.Style;
			ShowNames = other.ShowNames;
			ShowPreflights = other.ShowPreflights;
			JobNames = other.JobNames?.ToList();
			Templates = other.Templates?.ToList();
			Columns = other.Columns?.ConvertAll(x => new GetStreamTabColumnResponse(x));
		}
	}

	/// <summary>
	/// Describes a column to display on the jobs page
	/// </summary>
	public class GetStreamTabColumnResponse : IStreamTabColumn
	{
		/// <inheritdoc/>
		public TabColumnType Type { get; set; }

		/// <inheritdoc/>
		public string Heading { get; set; }

		/// <inheritdoc/>
		public string? Category { get; set; }

		/// <inheritdoc/>
		public string? Parameter { get; set; }

		/// <inheritdoc/>
		public int? RelativeWidth { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetStreamTabColumnResponse()
		{
			Heading = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetStreamTabColumnResponse(IStreamTabColumn other)
		{
			Type = other.Type;
			Heading = other.Heading;
			Category = other.Category;
			Parameter = other.Parameter;
			RelativeWidth = other.RelativeWidth;
		}
	}

	/// <summary>
	/// Mapping from a BuildGraph agent type to a set of machines on the farm
	/// </summary>
	public class GetAgentTypeResponse
	{
		/// <summary>
		/// Pool of agents to use for this agent type
		/// </summary>
		public PoolId Pool { get; set; }

		/// <summary>
		/// Name of the workspace to sync
		/// </summary>
		public string? Workspace { get; set; }

		/// <summary>
		/// Path to the temporary storage dir
		/// </summary>
		public string? TempStorageDir { get; set; }

		/// <summary>
		/// Environment variables to be set when executing the job
		/// </summary>
		public Dictionary<string, string>? Environment { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetAgentTypeResponse()
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetAgentTypeResponse(IAgentType other)
		{
			Pool = other.Pool;
			Workspace = other.Workspace;
			TempStorageDir = other.TempStorageDir;
			Environment = other.Environment?.ToDictionary();
		}
	}

	/// <summary>
	/// Information about a workspace type
	/// </summary>
	public class GetWorkspaceTypeResponse : IWorkspaceType
	{
		/// <inheritdoc/>
		public string? Cluster { get; set; }

		/// <inheritdoc/>
		public string? ServerAndPort { get; set; }

		/// <inheritdoc/>
		public string? UserName { get; set; }

		/// <inheritdoc/>
		public string? Identifier { get; set; }

		/// <inheritdoc/>
		public string? Stream { get; set; }

		/// <inheritdoc cref="IWorkspaceType.View"/>
		public List<string>? View { get; set; }
		IReadOnlyList<string>? IWorkspaceType.View => View;

		/// <inheritdoc/>
		public bool? Incremental { get; set; }

		/// <inheritdoc/>
		public bool? UseAutoSdk { get; set; }

		/// <inheritdoc cref="IWorkspaceType.AutoSdkView"/>
		public List<string>? AutoSdkView { get; set; }
		IReadOnlyList<string>? IWorkspaceType.AutoSdkView => AutoSdkView;

		/// <inheritdoc/>
		public string? Method { get; set; } = null;

		/// <inheritdoc/>
		public long? MinScratchSpace { get; set; } = null;

		/// <inheritdoc/>
		public long? ConformDiskFreeSpace { get; set; } = null;

		/// <summary>
		/// Default constructor
		/// </summary>
		public GetWorkspaceTypeResponse()
		{ }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetWorkspaceTypeResponse(IWorkspaceType other)
		{
			Cluster = other.Cluster;
			ServerAndPort = other.ServerAndPort;
			UserName = other.UserName;
			Identifier = other.Identifier;
			Stream = other.Stream;
			View = other.View?.ToList();
			Incremental = other.Incremental;
			UseAutoSdk = other.UseAutoSdk;
			AutoSdkView = other.AutoSdkView?.ToList();
			Method = other.Method;
			MinScratchSpace = other.MinScratchSpace;
			ConformDiskFreeSpace = other.ConformDiskFreeSpace;
		}
	}

	/// <summary>
	/// State information for a step in the stream
	/// </summary>
	public class GetTemplateStepStateResponse
	{
		/// <summary>
		/// Name of the step
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// User who paused the step
		/// </summary>
		public GetThinUserInfoResponse? PausedByUserInfo { get; set; }

		/// <summary>
		/// The UTC time when the step was paused
		/// </summary>
		public DateTime? PauseTimeUtc { get; set; }

		/// <summary>
		/// Default constructor for serialization
		/// </summary>
		private GetTemplateStepStateResponse()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetTemplateStepStateResponse(ITemplateStep state, GetThinUserInfoResponse? pausedByUserInfo)
		{
			Name = state.Name;
			PauseTimeUtc = state.PauseTimeUtc;
			PausedByUserInfo = pausedByUserInfo;
		}
	}

	/// <summary>
	/// Information about a template in this stream
	/// </summary>
	public class GetTemplateRefResponse : GetTemplateResponseBase
	{
		/// <summary>
		/// Id of the template ref
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Hash of the template definition
		/// </summary>
		public string Hash { get; set; }

		/// <summary>
		/// Whether to show badges in UGS for these jobs
		/// </summary>
		public bool ShowUgsBadges { get; set; }

		/// <summary>
		/// Whether to show alerts in UGS for these jobs
		/// </summary>
		public bool ShowUgsAlerts { get; set; }

		/// <summary>
		/// Notification channel for this template. Overrides the stream channel if set.
		/// </summary>
		public string? NotificationChannel { get; set; }

		/// <summary>
		/// Notification channel filter for this template. Can be a combination of "Success", "Failure" and "Warnings" separated by pipe characters.
		/// </summary>
		public string? NotificationChannelFilter { get; set; }

		/// <summary>
		/// Triage channel for this template. Overrides the stream channel if set.
		/// </summary>
		public string? TriageChannel { get; set; }

		/// <summary>
		/// The schedule for this ref
		/// </summary>
		public GetScheduleResponse? Schedule { get; set; }

		/// <summary>
		/// List of templates to trigger
		/// </summary>
		public List<GetChainedJobTemplateResponse>? ChainedJobs { get; set; }

		/// <summary>
		/// List of step states
		/// </summary>
		public List<GetTemplateStepStateResponse>? StepStates { get; set; }

		/// <summary>
		/// List of queries for the default changelist
		/// </summary>
		public List<ChangeQueryMessage>? DefaultChange { get; set; }

		/// <summary>
		/// Whether the user is allowed to create jobs from this template
		/// </summary>
		public bool CanRun { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">The template ref id</param>
		/// <param name="templateRef">The template ref</param>
		/// <param name="template">The actual template</param>
		/// <param name="stepStates">The template step states</param>
		/// <param name="schedulerTimeZone">The scheduler time zone</param>
		/// <param name="canRun">Whether the user can run this template</param>
		public GetTemplateRefResponse(TemplateId id, ITemplateRef templateRef, ITemplate template, List<GetTemplateStepStateResponse>? stepStates, TimeZoneInfo schedulerTimeZone, bool canRun)
			: base(template)
		{
			Id = id.ToString();
			Hash = template.Hash.ToString();
			ShowUgsBadges = templateRef.ShowUgsBadges;
			ShowUgsAlerts = templateRef.ShowUgsAlerts;
			NotificationChannel = templateRef.NotificationChannel;
			NotificationChannelFilter = templateRef.NotificationChannelFilter;
			Schedule = (templateRef.Schedule != null) ? new GetScheduleResponse(templateRef.Schedule, schedulerTimeZone) : null;
			ChainedJobs = (templateRef.ChainedJobs != null && templateRef.ChainedJobs.Count > 0) ? templateRef.ChainedJobs.ConvertAll(x => new GetChainedJobTemplateResponse(x)) : null;
			StepStates = stepStates;
			DefaultChange = templateRef.DefaultChange?.ConvertAll(x => new ChangeQueryMessage(x));
			CanRun = canRun;
		}
	}

	/// <summary>
	/// Trigger for another template
	/// </summary>
	public class GetChainedJobTemplateResponse : IChainedJobTemplate
	{
		/// <inheritdoc/>
		public string Trigger { get; set; }

		/// <inheritdoc/>
		public TemplateId TemplateId { get; set; }

		/// <inheritdoc/>
		public bool UseDefaultChangeForTemplate { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetChainedJobTemplateResponse()
		{
			Trigger = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public GetChainedJobTemplateResponse(IChainedJobTemplate other)
		{
			Trigger = other.Trigger;
			TemplateId = other.TemplateId;
			UseDefaultChangeForTemplate = other.UseDefaultChangeForTemplate;
		}
	}

	/// <summary>
	/// Step state update request
	/// </summary>
	public class UpdateStepStateRequest
	{
		/// <summary>
		/// Name of the step
		/// </summary>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// User who paused the step
		/// </summary>
		public string? PausedByUserId { get; set; }
	}

	/// <summary>
	/// Updates an existing stream template ref
	/// </summary>
	public class UpdateTemplateRefRequest
	{
		/// <summary>
		/// Step states to update
		/// </summary>
		public List<UpdateStepStateRequest>? StepStates { get; set; }
	}
}
