// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Diagnostics;
using System.IO;
using System.IO.MemoryMappedFiles;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Horde.Symbols
{
	/// <summary>
	/// Provides functionality for hashing files to add to a Microsoft Symbol store.
	/// 
	/// * For PE-format files (EXE, DLL), this consists of the 
	/// * For PDB files, this consists of the PDB GUID followed by its age (the number of times it has been written).
	/// </summary>
	public static class SymStore
	{
		/// <summary>
		/// Gets the hash of a file.
		/// </summary>
		/// <param name="file">File to hash</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hash of the file, or null if it cannot be parsed.</returns>
		public static Task<string?> GetHashAsync(FileReference file, ILogger logger, CancellationToken cancellationToken = default)
		{
			try
			{
				if (file.HasExtension(".exe") || file.HasExtension(".dll"))
				{
					return GetExeHashAsync(file, cancellationToken);
				}
				else if (file.HasExtension(".pdb"))
				{
					return Task.FromResult<string?>(GetPdbHash(file));
				}
				else
				{
					return Task.FromResult<string?>(null);
				}
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unable to get hash for {File}: {Message}", file, ex.Message);
				return Task.FromResult<string?>(null);
			}
		}

		#region PE files

		/// <summary>
		/// Gets the hash for a Windows portable executable file, consisting of concatenated hex values for the timestamp and image size fields in the header.
		/// Structures referenced below (IMAGE_DOS_HEADER, IMAGE_NT_HEADERS) are defined in Windows headers, but are parsed via offsets into the executable
		/// to reduce messy marshalling in C#.
		/// </summary>
		static async Task<string?> GetExeHashAsync(FileReference file, CancellationToken cancellationToken)
		{
			using FileStream stream = FileReference.Open(file, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete);

			// Read the IMAGE_DOS_HEADER structure.
			const int SizeOf_IMAGE_DOS_HEADER = 64;
			const int OffsetOf_IMAGE_DOS_HEADER_Magic = 0;
			const int OffsetOf_IMAGE_DOS_HEADER_LfaNew = 0x3c;

			byte[] dosHeader = new byte[SizeOf_IMAGE_DOS_HEADER];
			if (await stream.ReadAsync(dosHeader, 0, dosHeader.Length, cancellationToken) != dosHeader.Length)
			{
				return null;
			}

			ushort magic = BinaryPrimitives.ReadUInt16LittleEndian(dosHeader.AsSpan(OffsetOf_IMAGE_DOS_HEADER_Magic));
			if (magic != (ushort)('M' | ('Z' << 8)))
			{
				return null;
			}

			uint lfaNew = BinaryPrimitives.ReadUInt32LittleEndian(dosHeader.AsSpan(OffsetOf_IMAGE_DOS_HEADER_LfaNew));
			stream.Seek(lfaNew, SeekOrigin.Begin);

			// Read the first part of the IMAGE_NT_HEADERS structure (just including the signature and file header)
			const int SizeOf_IMAGE_NT_HEADERS = 84;
			const int OffsetOf_IMAGE_NT_HEADERS_Signature = 0;
			const int OffsetOf_IMAGE_NT_HEADERS_TimeDateStamp = 8;
			const int OffsetOf_IMAGE_NT_HEADERS_SizeOfImage = 80;

			byte[] ntHeader = new byte[SizeOf_IMAGE_NT_HEADERS];
			if (await stream.ReadAsync(ntHeader, 0, ntHeader.Length, cancellationToken) != ntHeader.Length)
			{
				return null;
			}

			uint signature = BinaryPrimitives.ReadUInt16LittleEndian(ntHeader.AsSpan(OffsetOf_IMAGE_NT_HEADERS_Signature));
			if (signature != (uint)('P' | ('E' << 8)))
			{
				return null;
			}

			uint timeDateStamp = BinaryPrimitives.ReadUInt32LittleEndian(ntHeader.AsSpan(OffsetOf_IMAGE_NT_HEADERS_TimeDateStamp));
			uint sizeOfImage = BinaryPrimitives.ReadUInt32LittleEndian(ntHeader.AsSpan(OffsetOf_IMAGE_NT_HEADERS_SizeOfImage));

			// Return the hash value
			return $"{timeDateStamp:X}{sizeOfImage:X}";
		}

		#endregion
		#region PDB files

		static readonly ReadOnlyMemory<byte> s_portablePdbSignature = new byte[] { 0x42, 0x53, 0x4a, 0x42 };

		/// <summary>
		/// Parse the hash value from a PDB file, consisting of concatenated hex values for the PDB GUID and age (the number of times it has been written).
		/// 
		/// While the fields to obtain are at fairly straightforward offsets in particular data structures, PDBs are internally structured as an MSF file
		/// consisting of multiple data streams with non-contiguous pages. There are a few references for parsing this format:
		/// 
		/// * LLVM documentation: https://llvm.org/docs/PDB/MsfFile.html
		/// * Microsoft's PDB source code: https://github.com/microsoft/microsoft-pdb (particularly PDB/msf/msf.cpp)
		/// 
		/// This function only handles the BigMSF format.
		/// </summary>
		static string? GetPdbHash(FileReference file)
		{
			using FileStream stream = FileReference.Open(file, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete);
			using MemoryMappedFile memoryMappedFile = MemoryMappedFile.CreateFromFile(stream, null, stream.Length, MemoryMappedFileAccess.Read, HandleInheritability.None, false);
			using MemoryMappedView memoryMappedView = new(memoryMappedFile, 0, stream.Length, MemoryMappedFileAccess.Read);

			// Check if this is a portable PDB. Symstore does not support these.
			Span<byte> header = memoryMappedView.GetMemory(0, 4).Span.Slice(0, 4);
			if (header.SequenceEqual(s_portablePdbSignature.Span))
			{
				return null;
			}

			// Create the MSF file
			MsfFile msfFile = new(memoryMappedView);

			// Read the guid from the PDB stream
			const int PdbStream = 1;

			ReadOnlySpan<byte> firstPdbPage = msfFile.GetStreamData(PdbStream, 0);
			Guid guid = new(firstPdbPage.Slice(12, 16));
			string result = guid.ToString("N").ToUpperInvariant();

			// Read the age from the DBI stream
			const int DbiStream = 3;

			ReadOnlySpan<byte> firstDbiPage = msfFile.GetStreamData(DbiStream, 0);
			if (firstDbiPage.Length > 0)
			{
				int age = BinaryPrimitives.ReadInt32LittleEndian(firstDbiPage.Slice(8));
				result += $"{age:X}";
			}

			return result;
		}

		class MsfFile
		{
			static readonly byte[] s_msfSignature = Encoding.ASCII.GetBytes("Microsoft C/C++ MSF 7.00\r\n\x1a\x44\x53\0\0\0");

			record struct MsfStream(int Length, int DirectoryPagesIdx);

			readonly MemoryMappedView _memoryMappedView;
			readonly int _numPages;
			readonly int _pageSize;
			readonly int[] _streamDirectoryPages;
			readonly MsfStream[] _streams;

			public MsfFile(MemoryMappedView memoryMappedView)
			{
				_memoryMappedView = memoryMappedView;

				// Read the file header (the 'superblock' in LLVM docs, BIGMSF_HDR in Microsoft source)
				const int SizeOf_BIGMSF_HDR = 0x20 + (5 * 4);
				const int OffsetOf_BIGMSF_HDR_Magic = 0;
				const int OffsetOf_BIGMSF_HDR_PageSize = 32;
				const int OffsetOf_BIGMSF_HDR_NumPages = 40;
				const int OffsetOf_BIGMSF_HDR_NumDirectoryBytes = 44;

				Span<byte> header = memoryMappedView.GetMemory(0, SizeOf_BIGMSF_HDR).Span.Slice(0, SizeOf_BIGMSF_HDR);
				if (!header.Slice(OffsetOf_BIGMSF_HDR_Magic, s_msfSignature.Length).SequenceEqual(s_msfSignature))
				{
					throw new InvalidDataException($"Invalid signature for PDB file");
				}

				_pageSize = BinaryPrimitives.ReadInt32LittleEndian(header.Slice(OffsetOf_BIGMSF_HDR_PageSize));
				_numPages = BinaryPrimitives.ReadInt32LittleEndian(header.Slice(OffsetOf_BIGMSF_HDR_NumPages));

				// The stream directory is distributed across multiple pages, and the indexes for the pages of the stream directory are also split across multiple pages.
				// We can use the stream directory size to figure out how many pages are in each level of indirection.
				int streamDirectorySize = BinaryPrimitives.ReadInt32LittleEndian(header.Slice(OffsetOf_BIGMSF_HDR_NumDirectoryBytes));

				// The list of stream directory macro pages is immediately after the header
				int numStreamDirectoryPages = GetPageCount(streamDirectorySize);
				int numStreamDirectoryMacroPages = GetPageCount(numStreamDirectoryPages * sizeof(int));
				Span<int> streamDirectoryMacroPages = MemoryMarshal.Cast<byte, int>(memoryMappedView.GetMemory(SizeOf_BIGMSF_HDR, numStreamDirectoryMacroPages * sizeof(int)).Span);

				// Create a flat list of all the stream directory pages to make it easier to seek within it
				_streamDirectoryPages = new int[numStreamDirectoryPages];

				int pageIdx = 0;
				for (int macroPageIdx = 0; macroPageIdx < numStreamDirectoryMacroPages; macroPageIdx++)
				{
					ReadOnlySpan<byte> macroPage = GetPage(streamDirectoryMacroPages[macroPageIdx]);
					for (; macroPage.Length > 0 && pageIdx < _streamDirectoryPages.Length; pageIdx++)
					{
						_streamDirectoryPages[pageIdx] = BinaryPrimitives.ReadInt32LittleEndian(macroPage);
						macroPage = macroPage.Slice(sizeof(int));
					}
				}

				// Parse the header for each stream from the stream directory
				int directoryPage = 0;
				ReadOnlySpan<int> directoryData = GetDirectoryPage(directoryPage);

				int numStreams = directoryData[0];
				directoryData = directoryData.Slice(1);

				_streams = new MsfStream[numStreams];

				int nextDirectoryOffset = 1 + numStreams;
				for (int streamIdx = 0; streamIdx < numStreams; streamIdx++)
				{
					if (directoryData.Length == 0)
					{
						directoryPage++;
						directoryData = GetDirectoryPage(directoryPage);
					}

					int streamSize = directoryData[0];
					_streams[streamIdx] = new MsfStream(streamSize, nextDirectoryOffset);
					nextDirectoryOffset += GetPageCount(streamSize);
					directoryData = directoryData.Slice(1);
				}
			}

			int GetPageCount(int numBytes)
				=> (numBytes + (_pageSize - 1)) / _pageSize;

			public ReadOnlySpan<byte> GetStreamData(int streamIdx, int offset)
			{
				int streamPageIdx = offset / _pageSize;
				int streamPagePos = offset % _pageSize;
				return GetStreamPage(streamIdx, streamPageIdx).Slice(streamPagePos);
			}

			public ReadOnlySpan<int> GetStreamDirectoryData(int offset)
			{
				int pageIdx = (int)(offset / (_pageSize / sizeof(int)));
				int pagePos = (int)(offset % (_pageSize / sizeof(int)));
				return GetDirectoryPage(pageIdx).Slice(pagePos);
			}

			public Span<byte> GetPage(int pageIdx)
			{
				Debug.Assert(pageIdx < _numPages);
				// Cast to long to prevent overflow on offset for large PDBs
				return _memoryMappedView.GetMemory((long)pageIdx * _pageSize, _pageSize).Span;
			}

			ReadOnlySpan<byte> GetStreamPage(int streamIdx, int pageIdx)
			{
				ReadOnlySpan<int> pages = GetStreamDirectoryData(_streams[streamIdx].DirectoryPagesIdx + pageIdx);
				return GetPage(pages[0]);
			}

			ReadOnlySpan<int> GetDirectoryPage(int directoryPageIdx)
			{
				int pageIdx = _streamDirectoryPages[directoryPageIdx];
				return MemoryMarshal.Cast<byte, int>(GetPage(pageIdx));
			}
		}

		#endregion
	}
}
