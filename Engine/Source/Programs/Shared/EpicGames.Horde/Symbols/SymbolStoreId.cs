// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Symbols
{
	/// <summary>
	/// Identifier for a symbol store
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<SymbolStoreId, SymbolStoreIdConverter>))]
	[StringIdConverter(typeof(SymbolStoreIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<SymbolStoreId, SymbolStoreIdConverter>))]
	public record struct SymbolStoreId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public SymbolStoreId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class SymbolStoreIdConverter : StringIdConverter<SymbolStoreId>
	{
		/// <inheritdoc/>
		public override SymbolStoreId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(SymbolStoreId value) => value.Id;
	}
}
