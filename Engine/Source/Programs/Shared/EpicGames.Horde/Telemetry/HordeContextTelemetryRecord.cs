// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Utilities;

namespace EpicGames.Horde.Telemetry
{
	/// <summary>
	/// Record that describes telemetry that can contain common Horde context (and whether it is within a build context or not).
	/// </summary>
	public record HordeContextTelemetryRecord : CorrelatableTelemetryRecord
	{
		/// <summary>
		/// The value for an invalid commit id.
		/// </summary>
		public const int InvalidCommitId = -1;

		#region -- Public Properties --

		/// <summary>
		/// Whether context is build machine, or not.
		/// </summary>
		[Column("is_build_machine")]
		public bool IsBuildMachine { get; init; }

		/// <summary>
		/// The Horde URL.
		/// </summary>
		[Column("horde_url_str")]
#pragma warning disable CA1056 // URI-like properties should not be strings
		public string? HordeUrlStr { get; init; }
#pragma warning restore CA1056 // URI-like properties should not be strings

		/// <summary>
		/// The stream id.
		/// </summary>
		[Column("stream_id")]
		public StreamId? StreamId { get; init; }

		/// <summary>
		/// The template id.
		/// </summary>
		[Column("template_id")]
		public TemplateId? TemplateId { get; init; }

		/// <summary>
		/// The Horde job id.
		/// </summary>
		[Column("job_id")]
		public JobId? JobId { get; init; }

		/// <summary>
		/// The Horde step id.
		/// </summary>
		[Column("step_id")]
		public JobStepId? StepId { get; init; }

		/// <summary>
		/// The ordered commit id.
		/// </summary>
		[Column("commit_id_ordered")]
		public int? CommitIdOrdered { get; init; }

		/// <summary>
		/// The unordered commit id.
		/// </summary>
		[Column("commit_id")]
		public string? CommitId { get; init; }

		#endregion -- Public Properties --

		#region -- Public Api --

		/// <summary>
		/// No arg constructor for ORM construction.
		/// </summary>
		/// <remarks>used for ORM instantiation.</remarks>
		protected HordeContextTelemetryRecord() : base() { }

		/// <summary>
		/// Constructor for standard Horde context.
		/// </summary>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version.</param>
		/// <param name="sessionId">The id used to correlate telemetry emitted in the same session.</param>
		/// <param name="sessionLabel">The label used to correlate telemetry emitted across multiple sessions.</param>
		/// <param name="isBuildMachine">True whether this is a build machine context, false otherwise.</param>
		/// <param name="hordeUrlStr">The horde url.</param>
		/// <param name="streamId">The stream id.</param>
		/// <param name="templateId">The template id.</param>
		/// <param name="jobId">The Horde job id.</param>
		/// <param name="stepId">The Horde step id.</param>
		/// <param name="commitIdOrdered">The ordered commitId.</param>
		/// <param name="commitId">The commitId.</param>
		public HordeContextTelemetryRecord(string eventName, int schemaVersion, string? sessionId, string? sessionLabel, bool isBuildMachine, Uri? hordeUrlStr, StreamId? streamId, TemplateId? templateId, JobId? jobId, JobStepId? stepId, int? commitIdOrdered, string? commitId) : base(eventName, schemaVersion, sessionId, sessionLabel)
		{
			IsBuildMachine = isBuildMachine;
			HordeUrlStr = hordeUrlStr?.ToString();
			StreamId = streamId;
			JobId = jobId;
			TemplateId = templateId;
			StepId = stepId;
			CommitIdOrdered = commitIdOrdered;
			CommitId = commitId;
		}

		/// <summary>
		/// Constructor for standard Horde context.
		/// </summary>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version.</param>
		/// <param name="sessionId">The id used to correlate telemetry emitted in the same session.</param>
		/// <param name="sessionLabel">The label used to correlate telemetry emitted across multiple sessions.</param>
		/// <param name="isBuildMachine">True whether this is a build machine context, false otherwise.</param>
		/// <param name="hordeUrlStr">The horde url.</param>
		/// <param name="streamId">The stream id.</param>
		/// <param name="templateId">The template id.</param>
		/// <param name="jobId">The Horde job id.</param>
		/// <param name="stepId">The Horde step id.</param>
		/// <param name="commitIdOrdered">The ordered commitId.</param>
		/// <param name="commitId">The commitId.</param>
#pragma warning disable CA1054 // URI-like parameters should not be strings
		public HordeContextTelemetryRecord(string eventName, int schemaVersion, string? sessionId, string? sessionLabel, bool isBuildMachine, string? hordeUrlStr, string? streamId, string? templateId, string? jobId, string? stepId, int? commitIdOrdered, string? commitId) : base(eventName, schemaVersion, sessionId, sessionLabel)
#pragma warning restore CA1054 // URI-like parameters should not be strings
		{
			IsBuildMachine = isBuildMachine;
			HordeUrlStr = hordeUrlStr;

			StreamId = TryCreate(streamId, s => new StreamId(s));
			JobId = TryCreate(jobId, j => Jobs.JobId.Parse(j));
			TemplateId = TryCreate(templateId, t => new TemplateId(t));
			StepId = TryCreate(stepId, s => JobStepId.Parse(s));

			CommitIdOrdered = commitIdOrdered;
			CommitId = commitId;
		}

		/// <summary>
		/// Constructs a standard Horde context from environment variables.
		/// </summary>
		/// <param name="eventName">The event name.</param>
		/// <param name="schemaVersion">The schema version.</param>
		/// <param name="sessionId">The id used to correlate telemetry emitted in the same session.</param>
		/// <param name="sessionLabel">The label used to correlate telemetry emitted across multiple sessions.</param>
		/// <returns>A instance of standard context created from environment variables.</returns>
		public HordeContextTelemetryRecord(string eventName, int schemaVersion, string? sessionId, string? sessionLabel) : base(eventName, schemaVersion, sessionId, sessionLabel)
		{
			IsBuildMachine = String.Equals("1", Environment.GetEnvironmentVariable("IsBuildMachine")?.Trim(), StringComparison.Ordinal);
			HordeUrlStr = HordeEnvVars.HordeUrl?.ToString();

			string? streamId = HordeEnvVars.StreamId;
			string? jobId = HordeEnvVars.JobId;
			string? templateId = HordeEnvVars.TemplateId;
			string? stepId = HordeEnvVars.StepId;
			string? commitId = Environment.GetEnvironmentVariable("uebp_CL");

			StreamId = TryCreate(streamId, s => new StreamId(s));
			JobId = TryCreate(jobId, j => Jobs.JobId.Parse(j));
			TemplateId = TryCreate(templateId, t => new TemplateId(t));
			StepId = TryCreate(stepId, s => JobStepId.Parse(s));
			CommitId = commitId;

			if (!String.IsNullOrEmpty(commitId) && Int32.TryParse(commitId, out int commitIdRaw))
			{
				CommitIdOrdered = commitIdRaw;
			}
			else
			{
				CommitIdOrdered = InvalidCommitId;
			}
		}

		#endregion -- Public Api --

		#region -- Protected Api --

		/// <inheritdoc/>
		protected override void WriteProperties(Hashtable hashtable)
		{
			base.WriteProperties(hashtable);

			hashtable[nameof(IsBuildMachine)] = IsBuildMachine;

			if (!String.IsNullOrEmpty(HordeUrlStr))
			{
				hashtable[nameof(HordeUrlStr)] = HordeUrlStr;
			}

			if (!String.IsNullOrEmpty(StreamId?.ToString()))
			{
				hashtable[nameof(StreamId)] = StreamId;
			}

			if (!String.IsNullOrEmpty(JobId?.ToString()))
			{
				hashtable[nameof(JobId)] = JobId;
			}

			if (!String.IsNullOrEmpty(TemplateId?.ToString()))
			{
				hashtable[nameof(TemplateId)] = TemplateId;
			}

			if (!String.IsNullOrEmpty(StepId?.ToString()))
			{
				hashtable[nameof(StepId)] = StepId;
			}

			hashtable[nameof(CommitId)] = CommitId;
			hashtable[nameof(CommitIdOrdered)] = CommitIdOrdered;
		}

		#endregion

		#region -- Private Api --

		private static T? TryCreate<T>(string? value, Func<string, T> creationDelegate) where T : struct
		{
			if (String.IsNullOrEmpty(value))
			{
				return null;
			}

			try
			{
				return creationDelegate(value);
			}
			catch
			{
				return null;
			}
		}

		#endregion -- Private Api --
	}
}