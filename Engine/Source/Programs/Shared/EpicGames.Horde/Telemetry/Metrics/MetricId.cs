// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Telemetry.Metrics
{
	/// <summary>
	/// Identifier for a particular metric
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<MetricId, MetricIdConverter>))]
	[StringIdConverter(typeof(MetricIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<MetricId, MetricIdConverter>))]
	public readonly record struct MetricId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public MetricId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class MetricIdConverter : StringIdConverter<MetricId>
	{
		/// <inheritdoc/>
		public override MetricId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(MetricId value) => value.Id;
	}
}
