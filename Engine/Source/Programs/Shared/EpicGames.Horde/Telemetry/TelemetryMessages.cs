// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Telemetry.Metrics;
using EpicGames.Serialization;

#pragma warning disable CA2227

namespace EpicGames.Horde.Telemetry
{
	/// <summary>
	/// Generic message for a telemetry event
	/// </summary>
	public class PostTelemetryEventStreamRequest
	{
		/// <summary>
		/// List of telemetry events
		/// </summary>
		public List<System.Text.Json.JsonElement> Events { get; set; } = [];
	}

	/// <summary>
	/// Indicates the type of telemetry data being uploaded
	/// </summary>
	public enum TelemetryUploadType
	{
		/// <summary>
		/// A batch of <see cref="PostTelemetryEventStreamRequest"/> objects.
		/// </summary>
		EtEventStream
	}

	/// <summary>
	/// Metrics meta data for query
	/// </summary>
	public class GetTelemetryMetricsMetaResponse
	{
		/// <summary>
		/// The corresponding metric id
		/// </summary>
		public string MetricId { get; set; } = String.Empty;

		/// <summary>
		/// Metric grouping information
		/// </summary>
		public string GroupBy { get; set; } = String.Empty;

		/// <summary>
		/// Metrics groups matching the search terms
		/// </summary>
		public List<string> Groups { get; set; } = [];

		/// <summary>
		/// TopN aggregation
		/// </summary>
		public int TopN { get; set; } = 0;

		/// <summary>
		/// BottomN aggregation
		/// </summary>
		public int BottomN { get; set; } = 0;
	}

	/// <summary>
	/// Metrics matching a particular query
	/// </summary>
	public class GetTelemetryMetricsResponse
	{
		/// <summary>
		/// The corresponding metric id
		/// </summary>
		public string MetricId { get; set; } = String.Empty;

		/// <summary>
		/// Metric grouping information
		/// </summary>
		public string GroupBy { get; set; } = String.Empty;

		/// <summary>
		/// Metrics matching the search terms
		/// </summary>
		public List<GetTelemetryMetricResponse> Metrics { get; set; } = [];
	}

	/// <summary>
	/// Information about a particular metric
	/// </summary>
	public class GetTelemetryMetricResponse
	{
		/// <summary>
		/// Start time for the sample
		/// </summary>
		public DateTime Time { get; set; }

		/// <summary>
		/// Name of the group
		/// </summary>
		public string? Group { get; set; }

		/// <summary>
		/// Value for the metric
		/// </summary>
		public double Value { get; set; }
	}

	/// <summary>
	/// The units used to present the telemetry
	/// </summary>
	public enum TelemetryMetricUnitType
	{
		/// <summary>
		/// Time duration 
		/// </summary>
		Time,
		/// <summary>
		/// Ratio 0-100%
		/// </summary>
		Ratio,
		/// <summary>
		/// Artbitrary numeric value
		/// </summary>
		Value
	}

	/// <summary>
	/// The type of 
	/// </summary>
	public enum TelemetryMetricGraphType
	{
		/// <summary>
		/// A line graph
		/// </summary>
		Line,
		/// <summary>
		/// Key performance indicator (KPI) chart with thrasholds
		/// </summary>
		Indicator
	}

	/// <summary>
	/// Metric attached to a telemetry chart
	/// </summary>
	public class TelemetryChartMetricConfig
	{
		/// <summary>
		/// Associated metric id
		/// </summary>
		[Required]
		public MetricId Id { get; set; }

		/// <summary>
		/// The threshold for KPI values
		/// </summary>
		public int? Threshold { get; set; }

		/// <summary>
		/// The metric alias for display purposes
		/// </summary>
		public string? Alias { get; set; }
	}

	/// <summary>
	/// Telemetry chart configuraton
	/// </summary>
	public class TelemetryChartConfig
	{
		/// <summary>
		/// The name of the chart, will be displayed on the dashboard
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The unit to display
		/// </summary>
		public TelemetryMetricUnitType Display { get; set; } = TelemetryMetricUnitType.Time;

		/// <summary>
		/// The graph type 
		/// </summary>
		public TelemetryMetricGraphType Graph { get; set; } = TelemetryMetricGraphType.Line;

		/// <summary>
		/// List of configured metrics
		/// </summary>
		public List<TelemetryChartMetricConfig> Metrics { get; set; } = [];

		/// <summary>
		/// The min unit value for clamping chart
		/// </summary>
		public int? Min { get; set; }

		/// <summary>
		/// The max unit value for clamping chart
		/// </summary>
		public int? Max { get; set; }
	}

	/// <summary>
	/// A chart categody, will be displayed on the dashbord under an associated pivot
	/// </summary>
	public class TelemetryCategoryConfig
	{
		/// <summary>
		/// The name of the category
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The charts contained within the category
		/// </summary>
		public List<TelemetryChartConfig> Charts { get; set; } = [];
	}

	/// <summary>
	/// A telemetry view variable used for filtering the charting data
	/// </summary>
	public class TelemetryVariableConfig
	{
		/// <summary>
		/// The name of the variable for display purposes
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The associated data group attached to the variable 
		/// </summary>
		[Required]
		public string Group { get; set; } = null!;

		/// <summary>
		/// The default values to select
		/// </summary>
		public List<string> Defaults { get; set; } = [];
	}

	/// <summary>
	/// A telemetry view of related metrics, divided into categofies
	/// </summary>
	public class TelemetryViewConfig
	{
		/// <summary>
		/// Identifier for the view
		/// </summary>
		[Required]
		public TelmetryViewId Id { get; set; }

		/// <summary>
		/// The name of the view
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The telemetry store this view uses
		/// </summary>
		[Required]
		public TelemetryStoreId TelemetryStoreId { get; set; }

		/// <summary>
		///  The variables used to filter the view data
		/// </summary>
		public List<TelemetryVariableConfig> Variables { get; set; } = [];

		/// <summary>
		/// The categories contained within the view
		/// </summary>
		public List<TelemetryCategoryConfig> Categories { get; set; } = [];
	}

	/// <summary>
	/// Identifier for a particular metric view
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<TelmetryViewId, TelmetryViewIdConverter>))]
	[StringIdConverter(typeof(TelmetryViewIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<TelmetryViewId, TelmetryViewIdConverter>))]
	public record struct TelmetryViewId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public TelmetryViewId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class TelmetryViewIdConverter : StringIdConverter<TelmetryViewId>
	{
		/// <inheritdoc/>
		public override TelmetryViewId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(TelmetryViewId value) => value.Id;
	}

	/// <summary>
	/// Metric attached to a telemetry chart
	/// </summary>
	public class GetTelemetryChartMetricResponse
	{
		/// <summary>
		/// Associated metric id
		/// </summary>		
		public string MetricId { get; set; } = null!;

		/// <summary>
		/// The threshold for KPI values
		/// </summary>
		public int? Threshold { get; set; }

		/// <summary>
		/// The metric alias for display purposes
		/// </summary>
		public string? Alias { get; set; }
	}

	/// <summary>
	/// Telemetry chart configuraton
	/// </summary>
	public class GetTelemetryChartResponse
	{
		/// <summary>
		/// The name of the chart, will be displayed on the dashboard
		/// </summary>		
		public string Name { get; set; } = null!;

		/// <summary>
		/// The unit to display
		/// </summary>
		public string Display { get; set; } = null!;

		/// <summary>
		/// The graph type 
		/// </summary>
		public string Graph { get; set; } = null!;

		/// <summary>
		/// List of configured metrics
		/// </summary>
		public List<GetTelemetryChartMetricResponse> Metrics { get; set; } = [];

		/// <summary>
		/// The min unit value for clamping chart
		/// </summary>
		public int? Min { get; set; }

		/// <summary>
		/// The max unit value for clamping chart
		/// </summary>
		public int? Max { get; set; }
	}

	/// <summary>
	/// A chart categody, will be displayed on the dashbord under an associated pivot
	/// </summary>
	public class GetTelemetryCategoryResponse
	{
		/// <summary>
		/// The name of the category
		/// </summary>		
		public string Name { get; set; } = null!;

		/// <summary>
		/// The charts contained within the category
		/// </summary>
		public List<GetTelemetryChartResponse> Charts { get; set; } = [];
	}

	/// <summary>
	/// A telemetry view variable used for filtering the charting data
	/// </summary>
	public class GetTelemetryVariableResponse
	{
		/// <summary>
		/// The name of the variable for display purposes
		/// </summary>
		public string Name { get; set; } = null!;

		/// <summary>
		/// The associated data group attached to the variable 
		/// </summary>
		public string Group { get; set; } = null!;

		/// <summary>
		/// The default values to select
		/// </summary>
		public List<string> Defaults { get; set; } = [];
	}

	/// <summary>
	/// A telemetry view of related metrics, divided into categofies
	/// </summary>
	public class GetTelemetryViewResponse
	{
		/// <summary>
		/// Identifier for the view
		/// </summary>
		public string Id { get; set; } = null!;

		/// <summary>
		/// The name of the view
		/// </summary>
		public string Name { get; set; } = null!;

		/// <summary>
		/// The telemetry store the view uses
		/// </summary>
		public string TelemetryStoreId { get; set; } = null!;

		/// <summary>
		///  The variables used to filter the view data
		/// </summary>
		public List<GetTelemetryVariableResponse> Variables { get; set; } = [];

		/// <summary>
		/// The categories contained within the view
		/// </summary>
		public List<GetTelemetryCategoryResponse> Categories { get; set; } = [];
	}
}
