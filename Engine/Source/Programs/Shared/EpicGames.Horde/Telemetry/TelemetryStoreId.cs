// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Telemetry
{
	/// <summary>
	/// Identifier for a telemetry store
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<TelemetryStoreId, TelemetryStoreIdConverter>))]
	[StringIdConverter(typeof(TelemetryStoreIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<TelemetryStoreId, TelemetryStoreIdConverter>))]
	public readonly record struct TelemetryStoreId(StringId Id)
	{
		/// <summary>
		/// Default telemetry store for Horde internal metrics
		/// </summary>
		public static TelemetryStoreId Default { get; } = new TelemetryStoreId("default");

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetryStoreId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	public class TelemetryStoreIdConverter : StringIdConverter<TelemetryStoreId>
	{
		/// <inheritdoc/>
		public override TelemetryStoreId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(TelemetryStoreId value) => value.Id;
	}
}
