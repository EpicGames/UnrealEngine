// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;

namespace EpicGames.Horde.Tools
{
	class HttpToolCollection : IToolCollection
	{
		[DebuggerDisplay("{Id}")]
		class Tool : ITool
		{
			static readonly IReadOnlyDictionary<string, string> s_defaultMetadata = new Dictionary<string, string>();

			readonly GetToolResponse _response;
			readonly List<ToolDeployment> _deployments;

			public ToolId Id => _response.Id;

			string ITool.Name => _response.Name;
			string ITool.Description => _response.Description;
			string? ITool.Category => _response.Category;
			string? ITool.Group => _response.Group;
			IReadOnlyList<string>? ITool.Platforms => _response.Platforms;
			bool ITool.Public => _response.Public;
			bool ITool.Bundled => _response.Bundled;
			bool ITool.ShowInUgs => _response.ShowInUgs;
			bool ITool.ShowInDashboard => _response.ShowInDashboard;
			bool ITool.ShowInToolbox => _response.ShowInToolbox;
			IReadOnlyDictionary<string, string> ITool.Metadata => _response.Metadata ?? s_defaultMetadata;
			IReadOnlyList<IToolDeployment> ITool.Deployments => _deployments;

			public Tool(HttpToolCollection collection, GetToolResponse response)
			{
				_response = response;
				_deployments = response.Deployments.ConvertAll(x => new ToolDeployment(collection, _response.Id, x));
			}

			public bool Authorize(AclAction action, ClaimsPrincipal principal)
				=> throw new NotImplementedException();

			public Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, Stream stream, CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, HashedBlobRefValue target, CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public IStorageBackend GetStorageBackend()
				=> throw new NotImplementedException();

			public IStorageNamespace GetStorageNamespace()
				=> throw new NotImplementedException();
		}

		[DebuggerDisplay("{Id}")]
		class ToolDeployment : IToolDeployment
		{
			readonly HttpToolCollection _collection;
			readonly ToolId _toolId;
			readonly GetToolDeploymentResponse _response;

			public ToolDeploymentId Id => _response.Id;

			string IToolDeployment.Version => _response.Version;
			ToolDeploymentState IToolDeployment.State => _response.State;
			double IToolDeployment.Progress => _response.Progress;
			DateTime? IToolDeployment.StartedAt => _response.StartedAt;
			TimeSpan IToolDeployment.Duration => _response.Duration;
			NamespaceId IToolDeployment.NamespaceId => throw new NotImplementedException();
			RefName IToolDeployment.RefName => _response.RefName;

			IBlobRef<DirectoryNode> IToolDeployment.Content
				=> _collection.Open(_toolId, _response.Locator);

			public ToolDeployment(HttpToolCollection collection, ToolId toolId, GetToolDeploymentResponse response)
			{
				_collection = collection;
				_toolId = toolId;
				_response = response;
			}

			Task<Stream> IToolDeployment.OpenZipStreamAsync(CancellationToken cancellationToken)
				=> _collection.OpenZipStreamAsync(_toolId, _response.Id, cancellationToken);

			Task<IToolDeployment?> IToolDeployment.UpdateAsync(ToolDeploymentState state, CancellationToken cancellationToken)
				=> throw new NotImplementedException();
		}

		readonly IHordeClient _hordeClient;

		/// <summary>
		/// Constructor
		/// </summary>
		public HttpToolCollection(IHordeClient hordeClient)
			=> _hordeClient = hordeClient;

		/// <inheritdoc/>
		public async Task<ITool?> GetAsync(ToolId id, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetToolResponse response = await hordeHttpClient.GetToolAsync(id, cancellationToken);
			return new Tool(this, response);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITool>> GetAllAsync(CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			GetToolsSummaryResponse responses = await hordeHttpClient.GetToolsAsync(cancellationToken);

			List<ITool> tools = [];
			foreach (GetToolSummaryResponse response in responses.Tools)
			{
				ITool? tool = await GetAsync(response.Id, cancellationToken);
				if (tool != null)
				{
					tools.Add(tool);
				}
			}

			return tools;
		}

		IBlobRef<DirectoryNode> Open(ToolId toolId, BlobLocator locator)
		{
			IStorageNamespace store = _hordeClient.GetStorageNamespace(toolId);
			return store.CreateBlobRef<DirectoryNode>(locator);
		}

		async Task<Stream> OpenZipStreamAsync(ToolId id, ToolDeploymentId deploymentId, CancellationToken cancellationToken = default)
		{
			HordeHttpClient hordeHttpClient = _hordeClient.CreateHttpClient();
			return await hordeHttpClient.GetToolDeploymentZipAsync(id, deploymentId, cancellationToken);
		}
	}
}
