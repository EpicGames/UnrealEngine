// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;

#pragma warning disable CA1716 // Rename virtual/interface member ITool.Public so that it no longer conflicts with the reserved language keyword 'Public'.

namespace EpicGames.Horde.Tools
{
	/// <summary>
	/// Describes a standalone, external tool hosted and deployed by Horde. Provides basic functionality for performing
	/// gradual roll-out, versioning, etc...
	/// </summary>
	public interface ITool
	{
		/// <summary>
		/// Identifier for the tool
		/// </summary>
		ToolId Id { get; }

		/// <summary>
		/// Name of the tool
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Long-form description of the tool
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Category for the tool on the dashboard
		/// </summary>
		string? Category { get; }

		/// <summary>
		/// Grouping key for merging tool versions together on the dashboard
		/// </summary>
		string? Group { get; }

		/// <summary>
		/// Supported platforms, as a NET runtime identifiers
		/// </summary>
		IReadOnlyList<string>? Platforms { get; }

		/// <summary>
		/// Whether the tool is available to authenticated users
		/// </summary>
		bool Public { get; }

		/// <summary>
		/// Whether the tool is bundled with the server
		/// </summary>
		bool Bundled { get; }

		/// <summary>
		/// Whether to show the tool for download in UGS
		/// </summary>
		bool ShowInUgs { get; }

		/// <summary>
		/// Whether to show the tool for download in the dashboard
		/// </summary>
		bool ShowInDashboard { get; }

		/// <summary>
		/// Whether to show the tool for download in the toolbox
		/// </summary>
		bool ShowInToolbox { get; }

		/// <summary>
		/// User-defined metadata for this tool
		/// </summary>
		IReadOnlyDictionary<string, string> Metadata { get; }

		/// <summary>
		/// Current deployments of this tool, sorted by time.
		/// </summary>
		IReadOnlyList<IToolDeployment> Deployments { get; }

		/// <summary>
		/// Authorize a user to perform a particular action
		/// </summary>
		/// <param name="action">Action the user is trying to perform</param>
		/// <param name="principal">Identity of the user trying to perform the action</param>
		bool Authorize(AclAction action, ClaimsPrincipal principal);

		/// <summary>
		/// Adds a new deployment to the given tool. The new deployment will replace the current active deployment.
		/// </summary>
		/// <param name="options">Options for the new deployment</param>
		/// <param name="stream">Stream containing the tool data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated tool document, or null if it does not exist</returns>
		Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, Stream stream, CancellationToken cancellationToken = default);

		/// <summary>
		/// Adds a new deployment to the given tool. The new deployment will replace the current active deployment.
		/// </summary>
		/// <param name="options">Options for the new deployment</param>
		/// <param name="target">Path to the root node containing the tool data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated tool document, or null if it does not exist</returns>
		Task<ITool?> CreateDeploymentAsync(ToolDeploymentConfig options, HashedBlobRefValue target, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the storage backend for this tool
		/// </summary>
		IStorageBackend GetStorageBackend();

		/// <summary>
		/// Gets the storage namespace for this particular tool
		/// </summary>
		IStorageNamespace GetStorageNamespace();
	}

	/// <summary>
	/// Deployment of a tool
	/// </summary>
	public interface IToolDeployment
	{
		/// <summary>
		/// Identifier for this deployment. A new identifier will be assigned to each created instance, so an identifier corresponds to a unique deployment.
		/// </summary>
		ToolDeploymentId Id { get; }

		/// <summary>
		/// Descriptive version string for this tool revision
		/// </summary>
		string Version { get; }

		/// <summary>
		/// Current state of this deployment
		/// </summary>
		ToolDeploymentState State { get; }

		/// <summary>
		/// Current progress of the deployment
		/// </summary>
		double Progress { get; }

		/// <summary>
		/// Last time at which the progress started. Set to null if the deployment was paused.
		/// </summary>
		DateTime? StartedAt { get; }

		/// <summary>
		/// Length of time over which to make the deployment
		/// </summary>
		TimeSpan Duration { get; }

		/// <summary>
		/// Namespace containing the tool
		/// </summary>
		NamespaceId NamespaceId { get; }

		/// <summary>
		/// Reference to this tool in Horde Storage.
		/// </summary>
		RefName RefName { get; }

		/// <summary>
		/// Handle to the tool data
		/// </summary>
		IBlobRef<DirectoryNode> Content { get; }

		/// <summary>
		/// Updates the state of the current deployment
		/// </summary>
		/// <param name="state">New state of the deployment</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IToolDeployment?> UpdateAsync(ToolDeploymentState state, CancellationToken cancellationToken = default);

		/// <summary>
		/// Opens a stream to the data for a particular deployment
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Stream for the data</returns>
		Task<Stream> OpenZipStreamAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Options for a new deployment
	/// </summary>
	public class ToolDeploymentConfig
	{
		/// <inheritdoc cref="IToolDeployment.Version"/>
		public string Version { get; set; } = "Unknown";

		/// <inheritdoc cref="IToolDeployment.Duration"/>
		public TimeSpan Duration { get; set; }

		/// <summary>
		/// Whether to create the deployment in a paused state
		/// </summary>
		public bool CreatePaused { get; set; }
	}

	/// <summary>
	/// Extension methods for tools
	/// </summary>
	public static class ToolExtensions
	{
		/// <summary>
		/// Gets the current deployment
		/// </summary>
		/// <param name="tool">Tool to query</param>
		/// <param name="phase">Adoption phase for the caller. 0 which the </param>
		/// <param name="utcNow">Current time</param>
		/// <returns></returns>
		public static IToolDeployment? GetCurrentDeployment(this ITool tool, double phase, DateTime utcNow)
		{
			int idx = tool.Deployments.Count - 1;
			for (; idx >= 0; idx--)
			{
				if (phase <= tool.Deployments[idx].GetProgressValue(utcNow) || idx == 0)
				{
					return tool.Deployments[idx];
				}
			}
			return null;
		}

		/// <summary>
		/// Get the progress fraction for a particular deployment and time
		/// </summary>
		/// <param name="deployment"></param>
		/// <param name="utcNow"></param>
		/// <returns></returns>
		public static double GetProgressValue(this IToolDeployment deployment, DateTime utcNow)
		{
			if (deployment.StartedAt == null)
			{
				return deployment.Progress;
			}
			else if (deployment.Duration > TimeSpan.Zero)
			{
				return Math.Clamp((utcNow - deployment.StartedAt.Value) / deployment.Duration, 0.0, 1.0);
			}
			else
			{
				return 1.0;
			}
		}
	}
}
