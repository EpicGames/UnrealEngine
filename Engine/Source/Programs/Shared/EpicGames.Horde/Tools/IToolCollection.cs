// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Tools
{
	/// <summary>
	/// Collection of tools
	/// </summary>
	public interface IToolCollection
	{
		/// <summary>
		/// Gets a tool with the given identifier
		/// </summary>
		/// <param name="id">The tool identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The requested tool, or null if it does not exist</returns>
		Task<ITool?> GetAsync(ToolId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets all the available tools
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns>List of the available tools</returns>
		Task<IReadOnlyList<ITool>> GetAllAsync(CancellationToken cancellationToken = default);
	}
}
