// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace EpicGames.Horde.Tools
{
	/// <summary>
	/// Identifier for a tool deployment
	/// </summary>
	/// <param name="Id">Identifier for the artifact</param>
	[LogValueType]
	[JsonSchemaString]
	[JsonConverter(typeof(BinaryIdJsonConverter<ToolDeploymentId, ToolDeploymentIdConverter>))]
	[TypeConverter(typeof(BinaryIdTypeConverter<ToolDeploymentId, ToolDeploymentIdConverter>))]
	[BinaryIdConverter(typeof(ToolDeploymentIdConverter))]
	public readonly record struct ToolDeploymentId(BinaryId Id)
	{
		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static ToolDeploymentId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc/>
		public override readonly string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter class to and from BinaryId values
	/// </summary>
	class ToolDeploymentIdConverter : BinaryIdConverter<ToolDeploymentId>
	{
		/// <inheritdoc/>
		public override ToolDeploymentId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(ToolDeploymentId value) => value.Id;
	}
}
