// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Horde.Tools
{
	/// <summary>
	/// Identifier for a tool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[JsonConverter(typeof(StringIdJsonConverter<ToolId, ToolIdConverter>))]
	[TypeConverter(typeof(StringIdTypeConverter<ToolId, ToolIdConverter>))]
	[StringIdConverter(typeof(ToolIdConverter))]
	[CbConverter(typeof(StringIdCbConverter<ToolId, ToolIdConverter>))]
	public readonly record struct ToolId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ToolId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class ToolIdConverter : StringIdConverter<ToolId>
	{
		/// <inheritdoc/>
		public override ToolId FromStringId(StringId id) => new(id);

		/// <inheritdoc/>
		public override StringId ToStringId(ToolId value) => value.Id;
	}
}
