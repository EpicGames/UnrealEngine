// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Horde.Storage;

namespace EpicGames.Horde.Tools
{
	/// <summary>
	/// Describes a standalone, external tool hosted and deployed by Horde. Provides basic functionality for performing
	/// gradual roll-out, versioning, etc...
	/// </summary>
	/// <param name="Id">Unique identifier for the tool</param>
	/// <param name="Name">Name of the tool</param>
	/// <param name="Description">Description for the tool</param>
	/// <param name="Category">Category to display the tool in on the dashboard</param>
	/// <param name="Group">Grouping key to control how different tools should be merged on the dashboard</param>
	/// <param name="Platforms">List of platforms that this tool supports, as NET runtime identifiers.</param>
	/// <param name="Deployments">Current deployments of this tool, sorted by time.</param>
	/// <param name="Public">Whether this tool should be exposed for download on a public endpoint without authentication</param>
	/// <param name="Bundled">Whether this tool is bundled with the server</param>
	/// <param name="ShowInUgs">Whether to show this tool for download inside UGS</param>
	/// <param name="ShowInDashboard">Whether to show this tool for download on the dashboard</param>
	/// <param name="ShowInToolbox">Whether to show this tool for download in Unreal Toolbox</param>
	/// <param name="Metadata">Metadata for the tool</param>
	public record class GetToolResponse(ToolId Id, string Name, string Description, string? Category, string? Group, List<string>? Platforms, List<GetToolDeploymentResponse> Deployments, bool Public, bool Bundled, bool ShowInUgs, bool ShowInDashboard, bool ShowInToolbox, Dictionary<string, string>? Metadata = null);

	/// <summary>
	/// Summary for a particular tool.
	/// </summary>
	/// <param name="Id">Unique identifier for the tool</param>
	/// <param name="Name">Name of the tool</param>
	/// <param name="Description">Description for the tool</param>
	/// <param name="Category">Category to display the tool in on the dashboard</param>
	/// <param name="Group">Grouping key to control how different tools should be merged on the dashboard</param>
	/// <param name="Platforms">List of platforms that this tool supports, as NET runtime identifiers.</param>
	/// <param name="Version">Version number of the current deployment of this tool</param>
	/// <param name="DeploymentId">Identifier for the current deployment</param>
	/// <param name="DeploymentState">Current state of the deployment</param>
	/// <param name="DeploymentProgress">Current progress of the deployment</param>
	/// <param name="Bundled">Whether this tool is bundled with the server</param>
	/// <param name="ShowInUgs">Whether to show this tool for download inside UGS</param>
	/// <param name="ShowInDashboard">Whether to show this tool for download on the dashboard</param>
	/// <param name="ShowInToolbox">Whether to show this tool for download in the launcher</param>
	/// <param name="Metadata">Metadata for the tool</param>
	public record class GetToolSummaryResponse(ToolId Id, string Name, string Description, string? Category, string? Group, List<string>? Platforms, string? Version, ToolDeploymentId? DeploymentId, ToolDeploymentState? DeploymentState, double? DeploymentProgress, bool Bundled, bool ShowInUgs, bool ShowInDashboard, bool ShowInToolbox, Dictionary<string, string>? Metadata = null);

	/// <summary>
	/// Response when querying all tools
	/// </summary>
	/// <param name="Tools">List of tools currently available</param>
	public record GetToolsSummaryResponse(List<GetToolSummaryResponse> Tools);

	/// <summary>
	/// Response object describing the deployment of a tool
	/// </summary>
	/// <param name="Id">Identifier for this deployment. A new identifier will be assigned to each created instance, so an identifier corresponds to a unique deployment.</param>
	/// <param name="Version">Descriptive version string for this tool revision</param>
	/// <param name="State">Current state of this deployment</param>
	/// <param name="Progress">Current progress of the deployment</param>
	/// <param name="StartedAt">Last time at which the progress started. Set to null if the deployment was paused.</param>
	/// <param name="Duration">Length of time over which to make the deployment</param>
	/// <param name="RefName">Reference to the deployment data</param>
	/// <param name="Locator">Reference to this tool in Horde Storage</param>
	public record GetToolDeploymentResponse(ToolDeploymentId Id, string Version, ToolDeploymentState State, double Progress, DateTime? StartedAt, TimeSpan Duration, RefName RefName, BlobLocator Locator);

	/// <summary>
	/// Request for creating a new deployment
	/// </summary>
	/// <param name="Version">Nominal version string for this deployment</param>
	/// <param name="Duration">Number of minutes over which to do the deployment</param>
	/// <param name="CreatePaused">Whether to create the deployment in a paused state</param>
	/// <param name="Content">Handle to a directory node with the content for the deployment</param>
	public record CreateToolDeploymentRequest(string Version, double? Duration, bool? CreatePaused, HashedBlobRefValue Content);

	/// <summary>
	/// Response from creating a deployment
	/// </summary>
	/// <param name="Id">Identifier for the created deployment</param>
	public record CreateToolDeploymentResponse(ToolDeploymentId Id);

	/// <summary>
	/// Current state of a tool's deployment
	/// </summary>
	public enum ToolDeploymentState
	{
		/// <summary>
		/// The deployment is ongoing
		/// </summary>
		Active,

		/// <summary>
		/// The deployment should be paused at its current state
		/// </summary>
		Paused,

		/// <summary>
		/// Deployment of this version is complete
		/// </summary>
		Complete,

		/// <summary>
		/// The deployment has been cancelled.
		/// </summary>
		Cancelled,
	}

	/// <summary>
	/// Update an existing deployment
	/// </summary>
	public class UpdateDeploymentRequest
	{
		/// <summary>
		/// New state for the deployment
		/// </summary>
		public ToolDeploymentState? State { get; set; }
	}

	/// <summary>
	/// Action for a deployment
	/// </summary>
	public enum GetToolAction
	{
		/// <summary>
		/// Query for information about the deployment 
		/// </summary>
		Info,

		/// <summary>
		/// Download the deployment data
		/// </summary>
		Download,

		/// <summary>
		/// Download the deployment data as a zip file
		/// </summary>
		Zip,
	}
};
