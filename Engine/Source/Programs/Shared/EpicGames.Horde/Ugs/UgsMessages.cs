// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using EpicGames.Horde.Issues;

#pragma warning disable CA2227 //  Change x to be read-only by removing the property setter

namespace EpicGames.Horde.Ugs
{
	/// <summary>
	/// Review by a user of a particular change
	/// </summary>
	public enum UgsUserVote
	{
		/// <summary>
		/// No vote for the current change
		/// </summary>
		None,

		/// <summary>
		/// Succesfully compiled the change
		/// </summary>
		CompileSuccess,

		/// <summary>
		/// Failed to compile the change
		/// </summary>
		CompileFailure,

		/// <summary>
		/// Manually marked the change as good
		/// </summary>
		Good,

		/// <summary>
		/// Manually marked the change as bad
		/// </summary>
		Bad
	}

	/// <summary>
	/// State of a badge
	/// </summary>
	public enum UgsBadgeState
	{
		/// <summary>
		/// Starting work on this badge, outcome currently unknown
		/// </summary>
		Starting,

		/// <summary>
		/// Badge failed
		/// </summary>
		Failure,

		/// <summary>
		/// Badge produced a warning
		/// </summary>
		Warning,

		/// <summary>
		/// Badge succeeded
		/// </summary>
		Success,

		/// <summary>
		/// Badge was skipped
		/// </summary>
		Skipped,
	}

	/// <summary>
	/// Adds a new badge to the change
	/// </summary>
	public class AddUgsBadgeRequest
	{
		/// <summary>
		/// Name of the badge
		/// </summary>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Url for the badge
		/// </summary>
		public Uri? Url { get; set; }

		/// <summary>
		/// Current status of the badge
		/// </summary>
		public UgsBadgeState State { get; set; }
	}

	/// <summary>
	/// Request object for adding new metadata to the server
	/// </summary>
	public class AddUgsMetadataRequest
	{
		/// <summary>
		/// The stream name
		/// </summary>
		[Required]
		public string Stream { get; set; } = null!;

		/// <summary>
		/// The changelist number
		/// </summary>
		[Required]
		public int Change { get; set; }

		/// <summary>
		/// The project name
		/// </summary>
		public string? Project { get; set; }

		/// <summary>
		/// Name of the current user
		/// </summary>
		public string? UserName { get; set; }

		/// <summary>
		/// Whether this changelist has been synced by the user
		/// </summary>
		public bool? Synced { get; set; }

		/// <summary>
		/// State of the user
		/// </summary>
		public UgsUserVote? Vote { get; set; }

		/// <summary>
		/// New starred state for the issue
		/// </summary>
		public bool? Starred { get; set; }

		/// <summary>
		/// Whether the user is investigating
		/// </summary>
		public bool? Investigating { get; set; }

		/// <summary>
		/// Comment for this change
		/// </summary>
		public string? Comment { get; set; }

		/// <summary>
		/// List of badges to add
		/// </summary>
		public List<AddUgsBadgeRequest>? Badges { get; set; }
	}

	/// <summary>
	/// Information about a user synced to a change
	/// </summary>
	public class GetUgsUserResponse
	{
		/// <summary>
		/// Name of the user
		/// </summary>
		public string User { get; set; } = String.Empty;

		/// <summary>
		/// Time that the change was synced
		/// </summary>
		public long? SyncTime { get; set; }

		/// <summary>
		/// State of the user
		/// </summary>
		public UgsUserVote? Vote { get; set; }

		/// <summary>
		/// Comment by this user
		/// </summary>
		public string? Comment { get; set; }

		/// <summary>
		/// Whether the user is investigating this change
		/// </summary>
		public bool? Investigating { get; set; }

		/// <summary>
		/// Whether this changelist is starred
		/// </summary>
		public bool? Starred { get; set; }
	}

	/// <summary>
	/// Information about a badge
	/// </summary>
	public class GetUgsBadgeResponse
	{
		/// <summary>
		/// Name of the badge
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Url for the badge
		/// </summary>
		public Uri? Url { get; set; }

		/// <summary>
		/// Current status of the badge
		/// </summary>
		public UgsBadgeState State { get; set; }
	}

	/// <summary>
	/// Response object for querying metadata
	/// </summary>
	public class GetUgsMetadataResponse
	{
		/// <summary>
		/// Number of the changelist
		/// </summary>
		public int Change { get; set; }

		/// <summary>
		/// The project name
		/// </summary>
		public string? Project { get; set; }

		/// <summary>
		/// Information about a user synced to this change
		/// </summary>
		public List<GetUgsUserResponse>? Users { get; set; }

		/// <summary>
		/// Badges for this change
		/// </summary>
		public List<GetUgsBadgeResponse>? Badges { get; set; }
	}

	/// <summary>
	/// Response object for querying metadata
	/// </summary>
	public class GetUgsMetadataListResponse
	{
		/// <summary>
		/// Last time that the metadata was modified
		/// </summary>
		public long SequenceNumber { get; set; }

		/// <summary>
		/// List of changes matching the requested criteria
		/// </summary>
		public List<GetUgsMetadataResponse> Items { get; set; } = [];
	}

	/// <summary>
	/// Outcome of a particular build
	/// </summary>
	public enum UgsIssueBuildOutcome
	{
		/// <summary>
		/// Unknown outcome
		/// </summary>
		Unknown,

		/// <summary>
		/// Build succeeded
		/// </summary>
		Success,

		/// <summary>
		/// Build failed
		/// </summary>
		Error,

		/// <summary>
		/// Build finished with warnings
		/// </summary>
		Warning,
	}

	/// <summary>
	/// Legacy response describing a build
	/// </summary>
	public class GetUgsIssueBuildResponse
	{
		/// <summary>
		/// Identifier for this build
		/// </summary>
		public long Id { get; set; }

		/// <summary>
		/// Path to the stream containing this build
		/// </summary>
		public string Stream { get; set; }

		/// <summary>
		/// The changelist number for this build
		/// </summary>
		public int Change { get; set; }

		/// <summary>
		/// Name of this job
		/// </summary>
		public string? JobName { get; set; }

		/// <summary>
		/// Link to the job
		/// </summary>
		public Uri? JobUrl { get; set; }

		/// <summary>
		/// Name of the job step
		/// </summary>
		public string? JobStepName { get; set; }

		/// <summary>
		/// Link to the job step
		/// </summary>
		public Uri? JobStepUrl { get; set; }

		/// <summary>
		/// Url for this particular error
		/// </summary>
		public Uri? ErrorUrl { get; set; }

		/// <summary>
		/// Outcome of this build (see <see cref="IssueBuildOutcome"/>)
		/// </summary>
		public int Outcome { get; set; }

		/// <summary>
		/// 
		/// </summary>
		/// <param name="stream"></param>
		/// <param name="change"></param>
		/// <param name="outcome"></param>
		public GetUgsIssueBuildResponse(string stream, int change, IssueBuildOutcome outcome)
		{
			Stream = stream;
			Change = change;
			Outcome = (int)outcome;
		}
	}

	/// <summary>
	/// Information about a diagnostic
	/// </summary>
	public class GetUgsIssueDiagnosticResponse
	{
		/// <summary>
		/// The corresponding build id
		/// </summary>
		public long? BuildId { get; set; }

		/// <summary>
		/// Message for the diagnostic
		/// </summary>
		public string Message { get; set; }

		/// <summary>
		/// Link to the error
		/// </summary>
		public Uri Url { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buildId">The corresponding build id</param>
		/// <param name="message">Message for the diagnostic</param>
		/// <param name="url">Link to the diagnostic</param>
		public GetUgsIssueDiagnosticResponse(long? buildId, string message, Uri url)
		{
			BuildId = buildId;
			Message = message;
			Url = url;
		}
	}

	/// <summary>
	/// Stores information about a build health issue
	/// </summary>
	public class GetUgsIssueResponse
	{
		/// <summary>
		/// Version number for this response
		/// </summary>
		public int Version { get; set; } = 1;

		/// <summary>
		/// The unique object id
		/// </summary>
		public int Id { get; set; }

		/// <summary>
		/// Time at which the issue was created
		/// </summary>
		public DateTime CreatedAt { get; set; }

		/// <summary>
		/// Time at which the issue was retrieved
		/// </summary>
		public DateTime RetrievedAt { get; set; }

		/// <summary>
		/// The associated project for the issue
		/// </summary>
		public string? Project { get; set; }

		/// <summary>
		/// The summary text for this issue
		/// </summary>
		public string Summary { get; set; } = String.Empty;

		/// <summary>
		/// Owner of the issue
		/// </summary>
		public string? Owner { get; set; }

		/// <summary>
		/// User that nominated the current owner
		/// </summary>
		public string? NominatedBy { get; set; }

		/// <summary>
		/// Time that the issue was acknowledged
		/// </summary>
		public DateTime? AcknowledgedAt { get; set; }

		/// <summary>
		/// Changelist that fixed this issue
		/// </summary>
		public int? FixChange { get; set; }

		/// <summary>
		/// Time at which the issue was resolved
		/// </summary>
		public DateTime? ResolvedAt { get; set; }

		/// <summary>
		/// Whether to notify the user about this issue
		/// </summary>
		public bool BNotify { get; set; }

		/// <summary>
		/// Whether this issue just contains warnings
		/// </summary>
		public bool BWarning { get; set; }

		/// <summary>
		/// Link to the last build
		/// </summary>
		public Uri? BuildUrl { get; set; }

		/// <summary>
		/// List of streams affected by this issue
		/// </summary>
		public List<string> Streams { get; set; } = [];
	}

	/// <summary>
	/// Request an issue to be updated
	/// </summary>
	public class UpdateUgsIssueRequest
	{
		/// <summary>
		/// New owner of the issue
		/// </summary>
		public string? Owner { get; set; }

		/// <summary>
		/// User than nominates the new owner
		/// </summary>
		public string? NominatedBy { get; set; }

		/// <summary>
		/// Whether the issue has been acknowledged
		/// </summary>
		public bool? Acknowledged { get; set; }

		/// <summary>
		/// Name of the user that declines the issue
		/// </summary>
		public string? DeclinedBy { get; set; }

		/// <summary>
		/// The change at which the issue is claimed fixed. 0 = not fixed, -1 = systemic issue.
		/// </summary>
		public int? FixChange { get; set; }

		/// <summary>
		/// Whether the issue should be marked as resolved
		/// </summary>
		public bool? Resolved { get; set; }

		/// <summary>
		/// Name of the user that resolved the issue
		/// </summary>
		public string? ResolvedBy { get; set; }
	}
}