// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;

namespace EpicGames.Horde.Users
{
	/// <summary>
	/// Identifier for a user
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(BinaryIdTypeConverter<UserId, UserIdConverter>))]
	[BinaryIdConverter(typeof(UserIdConverter))]
	public readonly record struct UserId(BinaryId Id)
	{
		/// <summary>
		/// Constant value for empty user id
		/// </summary>
		public static UserId Empty { get; } = default;

		/// <summary>
		/// Special user id for an anonymous administrator
		/// </summary>
		public static UserId Anonymous { get; } = Parse("63f7d3525119b9aa4c0f035a");

		/// <inheritdoc cref="BinaryId.Parse(System.String)"/>
		public static UserId Parse(string text) => new(BinaryId.Parse(text));

		/// <inheritdoc cref="BinaryId.TryParse(System.String, out BinaryId)"/>
		public static bool TryParse(string text, out UserId id)
		{
			if (BinaryId.TryParse(text, out BinaryId objectId))
			{
				id = new UserId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override readonly string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="BinaryId"/> instances.
	/// </summary>
	class UserIdConverter : BinaryIdConverter<UserId>
	{
		/// <inheritdoc/>
		public override UserId FromBinaryId(BinaryId id) => new(id);

		/// <inheritdoc/>
		public override BinaryId ToBinaryId(UserId value) => value.Id;
	}
}
