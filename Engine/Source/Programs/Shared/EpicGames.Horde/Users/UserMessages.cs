// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;

#pragma warning disable CA2227

namespace EpicGames.Horde.Users
{
	/// <summary>
	/// Response describing the current user
	/// </summary>
	public class GetUserResponse
	{
		/// <summary>
		/// Id of the user
		/// </summary>
		public UserId Id { get; set; }

		/// <summary>
		/// Name of the user
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Avatar image URL (24px)
		/// </summary>
		public string? Image24 { get; set; }

		/// <summary>
		/// Avatar image URL (32px)
		/// </summary>
		public string? Image32 { get; set; }

		/// <summary>
		/// Avatar image URL (48px)
		/// </summary>
		public string? Image48 { get; set; }

		/// <summary>
		/// Avatar image URL (72px)
		/// </summary>
		public string? Image72 { get; set; }

		/// <summary>
		/// Email of the user
		/// </summary>
		public string? Email { get; set; }

		/// <summary>
		/// Claims for the user
		/// </summary>
		public List<GetUserClaimResponse>? Claims { get; set; }

		/// <summary>
		/// Whether to enable experimental features for this user
		/// </summary>
		public bool? EnableExperimentalFeatures { get; set; }

		/// <summary>
		/// Whether to always tag preflight changelists
		/// </summary>
		public bool? AlwaysTagPreflightCL { get; set; }

		/// <summary>
		/// Settings for the dashboard
		/// </summary>
		public object? DashboardSettings { get; set; }

		/// <summary>
		/// Settings for whether various dashboard features should be shown for the current user
		/// </summary>
		public GetDashboardFeaturesResponse? DashboardFeatures { get; set; }

		/// <summary>
		/// User job template preferences
		/// </summary>
		public List<GetJobTemplateSettingsResponse>? JobTemplateSettings { get; set; }

		/// <summary>
		/// List of pinned job ids
		/// </summary>
		public List<string>? PinnedJobIds { get; set; }

		/// <summary>
		/// List of pinned bisection task ids
		/// </summary>
		public List<string>? PinnedBisectTaskIds { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetUserResponse(UserId id, string name)
		{
			Id = id;
			Name = name;
		}
	}

	/// <summary>
	/// New claim document
	/// </summary>
	public class GetUserClaimResponse
	{
		/// <summary>
		/// Type of the claim
		/// </summary>
		public string Type { get; set; }

		/// <summary>
		/// Value for the claim
		/// </summary>
		public string Value { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetUserClaimResponse(string type, string value)
		{
			Type = type;
			Value = value;
		}
	}

	/// <summary>
	/// Resolved permissions for an action in a given ACL scope
	/// </summary>
	public class UserAclPermission
	{
		/// <summary>
		/// Scope name
		/// </summary>
		public string Scope { get; set; }

		/// <summary>
		/// Action name
		/// </summary>
		public string Action { get; set; }

		/// <summary>
		/// Whether action is authorized in given scope
		/// </summary>
		public bool IsAuthorized { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public UserAclPermission(string scope, string action, bool isAuthorized)
		{
			Scope = scope;
			Action = action;
			IsAuthorized = isAuthorized;
		}
	}

	/// <summary>
	/// Resolved permissions for ACL scopes for a given user
	/// </summary>
	public class GetUserAclPermissionsResponse
	{
		/// <summary>
		/// List of ACL permissions
		/// </summary>
		public List<UserAclPermission> AclPermissions { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetUserAclPermissionsResponse(List<UserAclPermission> aclPermissions)
		{
			AclPermissions = aclPermissions;
		}
	}

	/// <summary>
	/// Job template settings for the current user
	/// </summary>
	public class GetJobTemplateSettingsResponse
	{
		/// <summary>
		/// The stream the job was run in
		/// </summary>
		public string StreamId { get; set; }

		/// <summary>
		/// The template id of the job
		/// </summary>
		public string TemplateId { get; set; }

		/// <summary>
		/// The hash of the template definition
		/// </summary>
		public string TemplateHash { get; set; }

		/// <summary>
		/// The arguments defined when creating the job
		/// </summary>
		public List<string> Arguments { get; set; }

		/// <summary>
		/// The last update time of the job template
		/// </summary>
		public DateTimeOffset UpdateTimeUtc { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetJobTemplateSettingsResponse(string streamId, string templateId, string templateHash, List<string> arguments, DateTime updateTimeUtc)
		{
			StreamId = streamId;
			TemplateId = templateId;
			TemplateHash = templateHash;
			Arguments = arguments;
			UpdateTimeUtc = new DateTimeOffset(updateTimeUtc);
		}
	}

	/// <summary>
	/// Settings for whether various features should be enabled on the dashboard
	/// </summary>
	public class GetDashboardFeaturesResponse
	{
		/// <summary>
		/// Navigate to the landing page by default
		/// </summary>
		public bool ShowLandingPage { get; set; }

		/// <summary>
		/// Custom landing page route to direct users to
		/// </summary>
		public string LandingPageRoute { get; set; } = String.Empty;

		/// <summary>
		/// Enable CI functionality
		/// </summary>
		public bool ShowCI { get; set; }

		/// <summary>
		/// Whether to show functionality related to agents, pools, and utilization on the dashboard.
		/// </summary>
		public bool ShowAgents { get; set; }

		/// <summary>
		/// Whether to show the agent registration page. When using registration tokens from elsewhere this is not needed.
		/// </summary>
		public bool ShowAgentRegistration { get; set; }

		/// <summary>
		/// Show the Perforce server option on the server menu
		/// </summary>
		public bool ShowPerforceServers { get; set; }

		/// <summary>
		/// Show the device manager on the server menu
		/// </summary>
		public bool ShowDeviceManager { get; set; }

		/// <summary>
		/// Show automated tests (legacy) on the server menu
		/// </summary>
		public bool ShowTests { get; set; }

		/// <summary>
		/// Whether the remote desktop button should be shown on the agent modal
		/// </summary>
		public bool ShowAccounts { get; set; }

		/// <summary>
		/// Whether the notice editor should be listed in the server menu
		/// </summary>
		public bool ShowNoticeEditor { get; set; }

		/// <summary>
		/// Whether controls for modifying pools should be shown
		/// </summary>
		public bool ShowPoolEditor { get; set; }

		/// <summary>
		/// Whether the remote desktop button should be shown on the agent modal
		/// </summary>
		public bool ShowRemoteDesktop { get; set; }
	}

	/// <summary>
	/// Basic information about a user. May be embedded in other responses.
	/// </summary>
	public class GetThinUserInfoResponse
	{
		/// <summary>
		/// Id of the user
		/// </summary>
		public UserId Id { get; set; }

		/// <summary>
		/// Name of the user
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// The user's email address
		/// </summary>
		public string? Email { get; set; }

		/// <summary>
		/// The user login [DEPRECATED]
		/// </summary>
		public string? Login { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public GetThinUserInfoResponse(UserId id, string name, string? email, string? login)
		{
			Id = id;
			Name = name;
			Email = email;
			Login = login;
		}
	}

	/// <summary>
	/// Request to update settings for a user
	/// </summary>
	public class UpdateUserRequest
	{
		/// <summary>
		/// Whether to enable experimental features for this user
		/// </summary>
		public bool? EnableExperimentalFeatures { get; set; }

		/// <summary>
		/// Whether to always tag preflight CL
		/// </summary>
		public bool? AlwaysTagPreflightCL { get; set; }

		/// <summary>
		/// New dashboard settings
		/// </summary>
		public JsonElement? DashboardSettings { get; set; }

		/// <summary>
		/// Job ids to add to the pinned list
		/// </summary>
		public List<string>? AddPinnedJobIds { get; set; }

		/// <summary>
		/// Jobs ids to remove from the pinned list
		/// </summary>
		public List<string>? RemovePinnedJobIds { get; set; }

		/// <summary>
		/// Bisection task ids to add to the pinned list
		/// </summary>
		public List<string>? AddPinnedBisectTaskIds { get; set; }

		/// <summary>
		/// Bisection task ids to remove from the pinned list
		/// </summary>
		public List<string>? RemovePinnedBisectTaskIds { get; set; }
	}
}
