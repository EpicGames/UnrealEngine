// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Attribute indicating that an object should generate a schema doc page
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class ConfigDocAttribute : Attribute
	{
		/// <summary>
		/// Page title
		/// </summary>
		public string Title { get; }

		/// <summary>
		/// Rail to show with breadcrumbs at the top of the page
		/// </summary>
		public string LinkRail { get; }

		/// <summary>
		/// Output filename
		/// </summary>
		public string FileName { get; }

		/// <summary>
		/// Optional introductory text on the page
		/// </summary>
		public string? Introduction { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ConfigDocAttribute(string title, string linkRail, string fileName)
		{
			Title = title;
			LinkRail = linkRail;
			FileName = fileName;
		}
	}
}
