// Copyright Epic Games, Inc. All Rights Reserved.

using System;

// The class members contained here are all self-explanatory enough that public
// comments on each of them are unnecessary.
#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member

namespace EpicGames.Horde.Utilities;

/// <summary>
/// Contains the environment variable names for common Horde environment
/// variables.
///
/// These are meant to be used by any and all C# code that is NOT Horde itself.
/// If these keys were used by Horde itself then as soon as a changes to them
/// was committed our tooling would break until Horde was actually deployed. 
/// </summary>
public static class HordeEnvKeys
{
	public const string BatchId = "UE_HORDE_BATCHID";
	public const string CleanupScriptPath = "UE_HORDE_CLEANUP";
	public const string GraphUpdatePath = "UE_HORDE_GRAPH_UPDATE";
	public const string HordeToken = "UE_HORDE_TOKEN";
	public const string HordeUrl = "UE_HORDE_URL";
	public const string JobId = "UE_HORDE_JOBID";
	public const string LeaseCleanupScriptPath = "UE_HORDE_LEASE_CLEANUP";
	public const string SharedDirPath = "UE_HORDE_SHARED_DIR";
	public const string StepId = "UE_HORDE_STEPID";
	public const string StepName = "UE_HORDE_STEPNAME";
	public const string StreamId = "UE_HORDE_STREAMID";
	public const string TemplateId = "UE_HORDE_TEMPLATEID";
	public const string TemplateName = "UE_HORDE_TEMPLATENAME";
}

/// <summary>
/// Contains static references to common Horde environment variables.
///
/// Note that for each of these we use expression bodied members so that the
/// value of the environment variable is read again each time. Ideally no one
/// would be changing any of these environment variables and this wouldn't be
/// required, however we have no way to guarantee that doesn't happen (and
/// in fact have seen cases where it does) and so we do this to be safe.
/// </summary>
public static class HordeEnvVars
{
	public static string? BatchId => Environment.GetEnvironmentVariable(HordeEnvKeys.BatchId);
	public static string? CleanupScriptPath => Environment.GetEnvironmentVariable(HordeEnvKeys.CleanupScriptPath);
	public static string? GraphUpdatePath => Environment.GetEnvironmentVariable(HordeEnvKeys.GraphUpdatePath);
	public static string? HordeToken => Environment.GetEnvironmentVariable(HordeEnvKeys.HordeToken);
	public static Uri? HordeUrl
	{
		get
		{
			string? value = Environment.GetEnvironmentVariable(HordeEnvKeys.HordeUrl);
			if (String.IsNullOrWhiteSpace(value))
			{
				return null;
			}

			return Uri.TryCreate(value, UriKind.Absolute, out Uri? uri) ? uri : null;
		}
	}
	public static string? JobId => Environment.GetEnvironmentVariable(HordeEnvKeys.JobId);
	public static string? LeaseCleanupScriptPath => Environment.GetEnvironmentVariable(HordeEnvKeys.LeaseCleanupScriptPath);
	public static string? SharedDirPath => Environment.GetEnvironmentVariable(HordeEnvKeys.SharedDirPath);
	public static string? StepId => Environment.GetEnvironmentVariable(HordeEnvKeys.StepId);
	public static string? StepName => Environment.GetEnvironmentVariable(HordeEnvKeys.StepName);
	public static string? StreamId => Environment.GetEnvironmentVariable(HordeEnvKeys.StreamId);
	public static string? TemplateId => Environment.GetEnvironmentVariable(HordeEnvKeys.TemplateId);
	public static string? TemplateName => Environment.GetEnvironmentVariable(HordeEnvKeys.TemplateName);
}