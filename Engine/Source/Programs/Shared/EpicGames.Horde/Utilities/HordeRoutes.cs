// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace EpicGames.Horde.Utilities;

/// <summary>
/// Provides simple access to different Horde dashboard routes. The primary use
/// case is when adding a link to Horde that is inserted into a log or report.
/// </summary>
public static class HordeRoutes
{
	/// <summary>
	/// Returns a <see cref="System.Uri" /> for a given Horde job.
	///
	/// Example: https://horde.example.com/job/690b24e0c619fc760780db81
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when jobId is empty or just whitespace.
	/// </exception>
	public static Uri JobPath(Uri hordeUrl, string jobId)
	{
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);

		return new Uri(hordeUrl, $"job/{jobId}");
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde job with details
	/// taken from environment variables.
	///
	/// Uses the following environment variables:
	/// - <see cref="HordeEnvVars.HordeUrl" />
	/// - <see cref="HordeEnvVars.JobId" />
	///
	/// Example: https://horde.example.com/job/690b24e0c619fc760780db81
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when required environment variables are not set.
	/// </exception>
	public static Uri JobPathFromEnv()
	{
		Uri? url = HordeEnvVars.HordeUrl;
		string? jobId = HordeEnvVars.JobId;

		ArgumentNullException.ThrowIfNull(url);
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);

		return JobPath(url, jobId);
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde step.
	///
	/// Example: https://horde.example.com/job/690b24e0c619fc760780db81?step=ab17
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when jobId or stepId are empty or just whitespace.
	/// </exception>
	public static Uri StepPath(Uri hordeUrl, string jobId, string stepId)
	{
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);
		ArgumentException.ThrowIfNullOrWhiteSpace(stepId);

		QueryStringBuilder query = new();
		query.Add("step", stepId);

		return new Uri(hordeUrl, $"job/{jobId}?{query}");
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde step with details
	/// taken from environment variables.
	///
	/// Uses the following environment variables:
	/// - <see cref="HordeEnvVars.HordeUrl" />
	/// - <see cref="HordeEnvVars.JobId" />
	/// - <see cref="HordeEnvVars.StepId" />
	///
	/// Example: https://horde.example.com/job/690b24e0c619fc760780db81?step=ab17
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when required environment variables are not set.
	/// </exception>
	public static Uri StepPathFromEnv()
	{
		Uri? url = HordeEnvVars.HordeUrl;
		string? jobId = HordeEnvVars.JobId;
		string? stepId = HordeEnvVars.StepId;

		ArgumentNullException.ThrowIfNull(url);
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);
		ArgumentException.ThrowIfNullOrWhiteSpace(stepId);

		return StepPath(url, jobId, stepId);
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde test automation.
	///
	/// Example: https://horde.example.com/test-automation?job-690b24e0c619fc760780db81&amp;step=ab17
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when jobId or stepId are empty or just whitespace.
	/// </exception>
	public static Uri TestAutomationPath(Uri hordeUrl, string jobId, string stepId)
	{
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);
		ArgumentException.ThrowIfNullOrWhiteSpace(stepId);

		QueryStringBuilder query = new();
		query.Add("job", jobId);
		query.Add("step", stepId);

		return new Uri(hordeUrl, $"test-automation?{query}");
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde test automation
	/// with details taken from environment variables.
	///
	/// Uses the following environment variables:
	/// - <see cref="HordeEnvVars.HordeUrl" />
	/// - <see cref="HordeEnvVars.JobId" />
	/// - <see cref="HordeEnvVars.StepId" />
	///
	///
	/// Example: https://horde.example.com/test-automation?job-690b24e0c619fc760780db81&amp;step=ab17
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when required environment variables are not set.
	/// </exception>
	public static Uri TestAutomationPathFromEnv()
	{
		Uri? url = HordeEnvVars.HordeUrl;
		string? jobId = HordeEnvVars.JobId;
		string? stepId = HordeEnvVars.StepId;

		ArgumentNullException.ThrowIfNull(url);
		ArgumentException.ThrowIfNullOrWhiteSpace(jobId);
		ArgumentException.ThrowIfNullOrWhiteSpace(stepId);

		return TestAutomationPath(url, jobId, stepId);
	}

	/// <summary>
	/// Returns a <see cref="System.Uri"/> for a given Horde log.
	///
	/// Example: https://horde.example.com/log/690b24e0c619fc760780db81?step=ab17
	/// </summary>
	/// <exception cref="ArgumentException">
	/// Thrown when logId is empty or just whitespace.
	/// </exception>
	public static Uri LogPath(Uri hordeUrl, string logId)
	{
		ArgumentException.ThrowIfNullOrWhiteSpace(logId);

		return new Uri(hordeUrl, $"log/{logId}");
	}
}