// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Horde.Utilities
{
	/// <summary>
	/// Converts TimeSpan intervals from formats like "30m", "1h30m", etc...
	/// </summary>
	public class IntervalJsonConverter : JsonConverter<TimeSpan>
	{
		record struct TimeSuffix(string Text, long Ticks);

		static readonly TimeSuffix[] s_timeSuffixes =
		[
			new TimeSuffix("w", TimeSpan.TicksPerDay * 7),
			new TimeSuffix("d", TimeSpan.TicksPerDay),
			new TimeSuffix("h", TimeSpan.TicksPerHour),
			new TimeSuffix("m", TimeSpan.TicksPerMinute),
			new TimeSuffix("s", TimeSpan.TicksPerSecond)
		];

		/// <inheritdoc/>
		public override TimeSpan Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return Parse(reader.GetString());
		}

		/// <summary>
		/// Parse a string as a time interval
		/// </summary>
		public static TimeSpan Parse(string? time)
		{
			if (String.IsNullOrEmpty(time))
			{
				throw new FormatException("TimeSpan may not be empty");
			}

			if (TimeSpan.TryParse(time, out TimeSpan result))
			{
				return result;
			}

			long ticks = 0;
			int suffixIdx = 0;
			for (ReadOnlySpan<char> span = time.AsSpan(); span.Length > 0;)
			{
				long value = 0;

				// Parse the value
				int length = 0;
				for (; length < span.Length && span[length] >= '0' && span[length] <= '9'; length++)
				{
					value = (value * 10) + (span[length] - '0');
				}
				if (length == 0)
				{
					throw new FormatException($"Unable to parse '{time}' as a TimeSpan. Invalid value.");
				}
				span = span.Slice(length);

				// Parse the suffix
				length = 0;
				while (length < span.Length && !(span[length] >= '0' && span[length] <= '9'))
				{
					length++;
				}
				if (length != 0) // Allow '3m30' to use the next suffix in order.
				{
					while (suffixIdx < s_timeSuffixes.Length && !s_timeSuffixes[suffixIdx].Text.AsSpan().SequenceEqual(span[0..length]))
					{
						suffixIdx++;
					}
					if (suffixIdx == s_timeSuffixes.Length)
					{
						throw new FormatException($"Unable to parse '{time}' as a TimeSpan. Invalid suffix.");
					}
					span = span.Slice(length);
				}

				ticks += s_timeSuffixes[suffixIdx++].Ticks * value;
			}

			return new TimeSpan(ticks);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TimeSpan value, JsonSerializerOptions options)
		{
			long ticks = value.Ticks;

			StringBuilder builder = new();
			for (int suffixIdx = 0; suffixIdx < s_timeSuffixes.Length; suffixIdx++)
			{
				if (ticks >= s_timeSuffixes[suffixIdx].Ticks)
				{
					long timeValue = ticks / s_timeSuffixes[suffixIdx].Ticks;
					builder.Append(timeValue);
					builder.Append(s_timeSuffixes[suffixIdx].Text);
					ticks -= timeValue * s_timeSuffixes[suffixIdx].Ticks;
				}
			}

			writer.WriteStringValue(builder.ToString());
		}
	}
}
