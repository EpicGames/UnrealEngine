// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.Json;

namespace EpicGames.Horde.Utilities;

#pragma warning disable CA2227 // Collection properties should be read only

/// <summary>
/// OpenTelemetry configuration for collection and sending of traces and metrics.
/// </summary>
public class OpenTelemetrySettings
{
	/// <summary>
	/// Whether OpenTelemetry exporting is enabled
	/// </summary>
	public bool Enabled { get; set; } = false;

	/// <summary>
	/// Service name
	/// </summary>
	public string ServiceName { get; set; } = "HordeServer";

	/// <summary>
	/// Service namespace
	/// </summary>
	public string ServiceNamespace { get; set; } = "Horde";

	/// <summary>
	/// Service version
	/// </summary>
	public string? ServiceVersion { get; set; }

	/// <summary>
	/// Whether to enrich and format telemetry to fit presentation in Datadog
	/// </summary>
	public bool EnableDatadogCompatibility { get; set; } = false;

	/// <summary>
	/// Extra attributes to set
	/// </summary>
	public Dictionary<string, string> Attributes { get; set; } = [];

	/// <summary>
	/// Whether to enable the console exporter (for debugging purposes)
	/// </summary>
	public bool EnableConsoleExporter { get; set; } = false;

	/// <summary>
	/// Protocol exporters (key is a unique and arbitrary name) 
	/// </summary>
	public Dictionary<string, OpenTelemetryProtocolExporterSettings> ProtocolExporters { get; set; } = [];
}

/// <summary>
/// Configuration for an OpenTelemetry exporter
/// </summary>
public class OpenTelemetryProtocolExporterSettings
{
	/// <summary>
	/// Endpoint URL. Usually differs depending on protocol used.
	/// </summary>
	public Uri? Endpoint { get; set; }

	/// <summary>
	/// Protocol for the exporter ('grpc' or 'httpprotobuf')
	/// </summary>
	public string Protocol { get; set; } = "grpc";
}

/// <summary>
/// Provides extension methods for serializing and deserializing OpenTelemetrySettings
/// </summary>
public static class OpenTelemetrySettingsExtensions
{
	/// <summary>
	/// Serializes OpenTelemetrySettings to a JSON string, with an option to encode as base64
	/// </summary>
	/// <param name="settings">OpenTelemetrySettings to serialize.</param>
	/// <param name="asBase64">If true, the resulting JSON string is encoded as base64</param>
	public static string Serialize(OpenTelemetrySettings settings, bool asBase64 = false)
	{
		string data = JsonSerializer.Serialize(settings);
		return asBase64 ? Convert.ToBase64String(Encoding.UTF8.GetBytes(data)) : data;
	}

	/// <summary>
	/// Deserializes a JSON string of OpenTelemetrySettings
	/// </summary>
	/// <param name="data">The string to deserialize, which can be either a JSON string or a Base64 encoded JSON string.</param>
	/// <param name="asBase64">If true, the input string is treated as base64 encoded</param>
	/// <returns>The deserialized OpenTelemetrySettings</returns>
	/// <exception cref="JsonException">Thrown when deserialization fails or results in a null object.</exception>
	public static OpenTelemetrySettings Deserialize(string data, bool asBase64 = false)
	{
		byte[] temp = asBase64 ? Convert.FromBase64String(data) : Encoding.UTF8.GetBytes(data);
		OpenTelemetrySettings? settings = JsonSerializer.Deserialize<OpenTelemetrySettings>(temp);
		if (settings == null)
		{
			throw new JsonException($"Unable to deserialize {nameof(OpenTelemetrySettings)} from JSON");
		}

		return settings;
	}
}