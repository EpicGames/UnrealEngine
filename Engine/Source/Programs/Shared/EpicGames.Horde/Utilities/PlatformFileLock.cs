// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Horde.Utilities
{
	/// <summary>
	/// Wraps a semaphore controlling the number of open files at any time. Used to prevent exceeding handle limit on MacOS.
	/// </summary>
	class PlatformFileLock : IDisposable
	{
		static readonly SemaphoreSlim? s_semaphore = CreateSemaphore();

		bool _locked;

		private PlatformFileLock()
			=> _locked = true;

		/// <summary>
		/// Acquire the platform file lock
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async ValueTask<PlatformFileLock?> CreateAsync(CancellationToken cancellationToken)
		{
			if (s_semaphore != null)
			{
				await s_semaphore.WaitAsync(cancellationToken);
				return new PlatformFileLock();
			}
			return null;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_locked)
			{
				s_semaphore?.Release();
				_locked = true;
			}
		}

		static SemaphoreSlim? CreateSemaphore()
		{
			if (OperatingSystem.IsMacOS())
			{
				return new SemaphoreSlim(32);
			}
			else
			{
				return null;
			}
		}
	}
}
