// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;

namespace EpicGames.Horde.Utilities
{
	/// <summary>
	/// Parses a time of day into a number of minutes since midnight
	/// </summary>
	public class TimeOfDayJsonConverter : JsonConverter<TimeSpan>
	{
		/// <inheritdoc/>
		public override TimeSpan Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return Parse(reader.GetString());
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TimeSpan value, JsonSerializerOptions options)
		{
			writer.WriteStringValue($"{value.Hours}:{value.Minutes}");
		}

		/// <summary>
		/// Parse a string as a number of minutes since midnight
		/// </summary>
		public static TimeSpan Parse(string? text)
		{
			if (String.IsNullOrEmpty(text))
			{
				ThrowInvalidTimeException(text);
			}

			Match match = Regex.Match(text, @"(?<hours>[0-9]+)(?:[.:](?<minutes>[0-9]+))?\s*(?<morning>am|pm)?$", RegexOptions.IgnoreCase);
			if (!match.Success)
			{
				ThrowInvalidTimeException(text);
			}

			int hours = Int32.Parse(match.Groups["hours"].Value);

			// Parse the number of minutes
			int minutes = 0;

			Group minutesGroup = match.Groups["minutes"];
			if (minutesGroup.Success)
			{
				minutes = Int32.Parse(minutesGroup.Value);
				if (minutes >= 60)
				{
					ThrowInvalidTimeException(text);
				}
			}

			// Handle 12h/24h suffix
			Group morningGroup = match.Groups["morning"];
			if (morningGroup.Success)
			{
				if (hours == 12)
				{
					hours = 0;
				}
				else if (hours == 0 || hours > 12)
				{
					ThrowInvalidTimeException(text);
				}
				if (String.Equals(morningGroup.Value, "pm", StringComparison.OrdinalIgnoreCase))
				{
					hours += 12;
				}
			}
			else
			{
				if (hours >= 24)
				{
					ThrowInvalidTimeException(text);
				}
			}

			return new TimeSpan(hours, minutes, 0);
		}

		[DoesNotReturn]
		static void ThrowInvalidTimeException(string? text)
		{
			throw new FormatException($"Unable to parse schedule time value '{text ?? "(null)"}'");
		}
	}
}
