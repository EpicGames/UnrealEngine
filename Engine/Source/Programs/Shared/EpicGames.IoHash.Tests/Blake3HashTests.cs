// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Text;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.IoHash.Tests
{
	[TestClass]
	public class Blake3HashTests
	{
		static readonly ReadOnlyMemory<byte> s_testData = Encoding.ASCII.GetBytes("The quick brown fox jumped over the lazy dog.");
		static readonly Core.Blake3Hash s_testHash = Core.Blake3Hash.Parse("7b0142eafb4d0bed8265978281a1398099b1d727acf4c4b8158b407b1e8385b3");

		[TestMethod]
		public void ComputeSpan()
		{
			Assert.AreEqual(Core.Blake3Hash.Compute(s_testData.Span), s_testHash);
		}

		[TestMethod]
		public void ComputeSequence()
		{
			Assert.AreEqual(Core.Blake3Hash.Compute(new ReadOnlySequence<byte>(s_testData)), s_testHash);
		}

		[TestMethod]
		public void ComputeStream()
		{
			using MemoryStream stream = new(s_testData.Length);
			stream.Write(s_testData.Span);
			stream.Position = 0;
			Assert.AreEqual(Core.Blake3Hash.Compute(stream), s_testHash);
		}

		[TestMethod]
		public async Task ComputeStreamAsync()
		{
			using MemoryStream stream = new(s_testData.Length);
			stream.Write(s_testData.Span);
			stream.Position = 0;
			Assert.AreEqual(await Core.Blake3Hash.ComputeAsync(stream), s_testHash);
		}

		[TestMethod]
		public void Parse()
		{
			Assert.AreEqual(Core.Blake3Hash.Parse(s_testHash.ToString()), s_testHash);
		}
	}
}
