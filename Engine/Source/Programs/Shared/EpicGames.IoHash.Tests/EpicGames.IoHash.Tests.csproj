<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
  </ItemGroup>
</Project>
