// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.IO;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.IoHash.Tests
{
	[TestClass]
	public class IoHashTests
	{
		static readonly ReadOnlyMemory<byte> s_testData = Encoding.ASCII.GetBytes("The quick brown fox jumped over the lazy dog.");
		static readonly Core.IoHash s_testHash = Core.IoHash.Parse("7b0142eafb4d0bed8265978281a1398099b1d727");

		[TestMethod]
		public void ComputeSpan()
		{
			Assert.AreEqual(Core.IoHash.Compute(s_testData.Span), s_testHash);
		}

		[TestMethod]
		public void ComputeSequence()
		{
			Assert.AreEqual(Core.IoHash.Compute(new ReadOnlySequence<byte>(s_testData)), s_testHash);
		}

		[TestMethod]
		public void ComputeStream()
		{
			using MemoryStream stream = new(s_testData.Length);
			stream.Write(s_testData.Span);
			stream.Position = 0;
			Assert.AreEqual(Core.IoHash.Compute(stream), s_testHash);
		}

		[TestMethod]
		public async Task ComputeStreamAsync()
		{
			using MemoryStream stream = new(s_testData.Length);
			stream.Write(s_testData.Span);
			stream.Position = 0;
			Assert.AreEqual(await Core.IoHash.ComputeAsync(stream), s_testHash);
		}

		[TestMethod]
		public void FromBlake3Hash()
		{
			Assert.AreEqual(new Core.IoHash(Core.Blake3Hash.Compute(s_testData.Span).Span), s_testHash);
		}

		[TestMethod]
		public void Parse()
		{
			Assert.AreEqual(Core.IoHash.Parse(s_testHash.ToString()), s_testHash);
			Assert.AreEqual(Core.IoHash.Parse(Encoding.ASCII.GetBytes(s_testHash.ToString())), s_testHash);
		}

		[TestMethod]
		public void TryParse()
		{
			Core.IoHash result;
			Assert.IsTrue(Core.IoHash.TryParse(s_testHash.ToString(), out result));
			Assert.AreEqual(result, s_testHash);

			Assert.IsTrue(Core.IoHash.TryParse(Encoding.ASCII.GetBytes(s_testHash.ToString()), out result));
			Assert.AreEqual(result, s_testHash);

			Assert.IsFalse(Core.IoHash.TryParse(String.Empty, out _));
			Assert.IsFalse(Core.IoHash.TryParse(Encoding.ASCII.GetBytes(String.Empty), out _));
		}

		[TestMethod]
		[DataRow(5)]
		[DataRow(22)]
		[DataRow(30)]
		public void JsonConverterMultiSegment(int splitPoint)
		{
			string json = $"\"{s_testHash}\"";
			byte[] bytes = Encoding.UTF8.GetBytes(json);
			ReadOnlySequence<byte> sequence = Core.ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			Core.IoHash result = JsonSerializer.Deserialize<Core.IoHash>(ref reader);
			Assert.AreEqual(s_testHash, result);
		}
	}
}
