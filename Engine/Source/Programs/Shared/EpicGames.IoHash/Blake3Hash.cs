// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Struct representing a strongly typed Blake3 hash value (a 32-byte Blake3 hash).
	/// </summary>
	public readonly struct Blake3Hash : IEquatable<Blake3Hash>, IComparable<Blake3Hash>
	{
		/// <summary>
		/// Length of an Blake3Hash
		/// </summary>
		public const int NumBytes = 32;

		/// <summary>
		/// Length of the hash in bits
		/// </summary>
		public const int NumBits = NumBytes * 8;

		/// <summary>
		/// Memory storing the digest data
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Span for the underlying memory
		/// </summary>
		public ReadOnlySpan<byte> Span => Memory.Span;

		/// <summary>
		/// Hash consisting of zeroes
		/// </summary>
		public static Blake3Hash Zero { get; } = new Blake3Hash(new byte[NumBytes]);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="memory">Memory to construct from</param>
		public Blake3Hash(ReadOnlyMemory<byte> memory)
		{
			if (memory.Length != NumBytes)
			{
				throw new ArgumentException($"Blake3Hash must be {NumBytes} bytes long");
			}

			Memory = memory;
		}

		/// <summary>
		/// Creates a content hash for a block of data.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static Blake3Hash Compute(ReadOnlySpan<byte> data)
		{
			byte[] output = new byte[32];
			Blake3.Hasher.Hash(data, output);
			return new Blake3Hash(output);
		}

		/// <summary>
		/// Creates a content hash for a block of data.
		/// </summary>
		/// <param name="sequence">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static IoHash Compute(ReadOnlySequence<byte> sequence)
		{
			if (sequence.IsSingleSegment)
			{
				return Compute(sequence.FirstSpan);
			}

			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				foreach (ReadOnlyMemory<byte> segment in sequence)
				{
					hasher.Update(segment.Span);
				}

				byte[] output = new byte[32];
				hasher.Finalize(output);
				return new Blake3Hash(output);
			}
		}

		/// <summary>
		/// Creates a content hash for a stream.
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static IoHash Compute(Stream stream)
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				Span<byte> buffer = new byte[16384];
				int length;
				while ((length = stream.Read(buffer)) > 0)
				{
					hasher.Update(buffer.Slice(0, length));
				}

				byte[] output = new byte[32];
				hasher.Finalize(output);
				return new Blake3Hash(output);
			}
		}

		/// <summary>
		/// Creates a content hash for a stream asynchronously. 
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static async Task<IoHash> ComputeAsync(Stream stream, CancellationToken cancellationToken = default)
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				Memory<byte> buffer = new byte[16384];
				int length;
				while ((length = await stream.ReadAsync(buffer, cancellationToken)) > 0)
				{
					hasher.Update(buffer.Slice(0, length).Span);
				}

				byte[] output = new byte[32];
				hasher.Finalize(output);
				return new Blake3Hash(output);
			}
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static Blake3Hash Parse(string text)
		{
			return new Blake3Hash(StringUtils.ParseHexString(text));
		}

		/// <inheritdoc cref="IComparable{T}.CompareTo(T)"/>
		public int CompareTo(Blake3Hash other)
		{
			ReadOnlySpan<byte> a = Span;
			ReadOnlySpan<byte> b = other.Span;

			for (int idx = 0; idx < a.Length && idx < b.Length; idx++)
			{
				int compare = a[idx] - b[idx];
				if (compare != 0)
				{
					return compare;
				}
			}
			return a.Length - b.Length;
		}

		/// <inheritdoc/>
		public bool Equals(Blake3Hash other) => Span.SequenceEqual(other.Span);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is Blake3Hash hash) && hash.Span.SequenceEqual(Span);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32LittleEndian(Span);

		/// <inheritdoc/>
		public override string ToString() => StringUtils.FormatHexString(Memory.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator ==(Blake3Hash a, Blake3Hash b) => a.Span.SequenceEqual(b.Span);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator !=(Blake3Hash a, Blake3Hash b) => !(a == b);

		/// <summary>
		/// Tests whether A > B
		/// </summary>
		public static bool operator >(Blake3Hash a, Blake3Hash b) => a.CompareTo(b) > 0;

		/// <summary>
		/// Tests whether A is less than B
		/// </summary>
		public static bool operator <(Blake3Hash a, Blake3Hash b) => a.CompareTo(b) < 0;

		/// <summary>
		/// Tests whether A is greater than or equal to B
		/// </summary>
		public static bool operator >=(Blake3Hash a, Blake3Hash b) => a.CompareTo(b) >= 0;

		/// <summary>
		/// Tests whether A is less than or equal to B
		/// </summary>
		public static bool operator <=(Blake3Hash a, Blake3Hash b) => a.CompareTo(b) <= 0;
	}

	/// <summary>
	/// Extension methods for dealing with Blake3Hash values
	/// </summary>
	public static class Blake3HashExtensions
	{
		/// <summary>
		/// Read an <see cref="Blake3Hash"/> from a memory reader
		/// </summary>
		/// <param name="reader"></param>
		/// <returns></returns>
		public static Blake3Hash ReadBlake3Hash(this MemoryReader reader)
		{
			return new Blake3Hash(reader.ReadFixedLengthBytes(Blake3Hash.NumBytes));
		}

		/// <summary>
		/// Write an <see cref="Blake3Hash"/> to a memory writer
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="hash"></param>
		public static void WriteBlake3Hash(this MemoryWriter writer, Blake3Hash hash)
		{
			writer.WriteFixedLengthBytes(hash.Span);
		}
	}
}
