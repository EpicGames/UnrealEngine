<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <RootNamespace>EpicGames.Core</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Blake3" Version="2.2.1" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>
</Project>
