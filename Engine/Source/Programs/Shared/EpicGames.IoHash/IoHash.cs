// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.Core
{
	/// <summary>
	/// Struct representing a strongly typed IoHash value (a 20-byte Blake3 hash).
	/// </summary>
	[JsonConverter(typeof(IoHashJsonConverter))]
	[TypeConverter(typeof(IoHashTypeConverter))]
	public readonly struct IoHash(ulong a, ulong b, uint c) : IEquatable<IoHash>, IComparable<IoHash>
	{
		/// <summary>
		/// Length of an IoHash
		/// </summary>
		public const int NumBytes = 20;

		/// <summary>
		/// Length of the hash in bits
		/// </summary>
		public const int NumBits = NumBytes * 8;

		/// <summary>
		/// Threshold size at which to use multiple threads for hashing
		/// </summary>
		/// <remarks>
		/// Performance testing has shown that setting this lower than 1MB results in a significant performance degredation.
		/// For some reason, smaller concurrent requests are not only slower than their non-concurrent counterparts, they
		/// are also slower than larger concurrent requests (i.e. concurrent 256KB is slower than concurrent 1MB).
		/// </remarks>
		const int MultiThreadedSize = 1024 * 1024;

		readonly ulong _a = a;
		readonly ulong _b = b;
		readonly uint _c = c;

		/// <summary>
		/// Hash consisting of zeroes
		/// </summary>
		public static IoHash Zero { get; } = new(0, 0, 0);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="span">Memory to construct from</param>
		public IoHash(ReadOnlySpan<byte> span)
			: this(BinaryPrimitives.ReadUInt64BigEndian(span), BinaryPrimitives.ReadUInt64BigEndian(span.Slice(8)), BinaryPrimitives.ReadUInt32BigEndian(span.Slice(16)))
		{
		}

		/// <summary>
		/// Construct 
		/// </summary>
		/// <param name="hasher">The hasher to construct from</param>
		public static IoHash FromBlake3(Blake3.Hasher hasher)
		{
			Span<byte> output = stackalloc byte[Blake3Hash.NumBytes];
			hasher.Finalize(output);
			return new(output);
		}

		/// <summary>
		/// Creates the IoHash for a block of data.
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute(ReadOnlySpan<byte> data) => Compute<byte>(data);

		/// <summary>
		/// Creates the IoHash for a block of data.
		/// </summary>
		/// <typeparam name="T">Type of the data</typeparam>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute<T>(ReadOnlySpan<T> data) where T : unmanaged
		{
			using Blake3.Hasher hasher = Blake3.Hasher.New();
			if (data.Length * Marshal.SizeOf<T>() < MultiThreadedSize)
			{
				hasher.Update(data);
			}
			else
			{
				hasher.UpdateWithJoin(data);
			}
			return FromBlake3(hasher);
		}

		/// <summary>
		/// Creates the IoHash for a sequence of data.
		/// </summary>
		/// <param name="sequence">Data to compute the hash for</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute(ReadOnlySequence<byte> sequence) => Compute<byte>(sequence);

		/// <summary>
		/// Creates the IoHash for a sequence of data.
		/// </summary>
		/// <typeparam name="T">Type of the data</typeparam>
		/// <param name="sequence">Data to compute the hash for</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute<T>(ReadOnlySequence<T> sequence) where T : unmanaged
		{
			if (sequence.IsSingleSegment)
			{
				return Compute(sequence.FirstSpan);
			}

			using Blake3.Hasher hasher = Blake3.Hasher.New();
			int size = Marshal.SizeOf<T>();
			foreach (ReadOnlyMemory<T> segment in sequence)
			{
				if (segment.Length * size < MultiThreadedSize)
				{
					hasher.Update(segment.Span);
				}
				else
				{
					hasher.UpdateWithJoin(segment.Span);
				}
			}
			return FromBlake3(hasher);
		}

		/// <summary>
		/// Creates the IoHash for a string with a given encoding
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <param name="encoding">The character encoding of the data</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute(string data, Encoding encoding) => Compute(encoding.GetBytes(data));

		/// <summary>
		/// Creates the IoHash for a string with the default encoding
		/// </summary>
		/// <param name="data">Data to compute the hash for</param>
		/// <returns>New hash instance containing the hash of the data</returns>
		public static IoHash Compute(string data) => Compute(data, Encoding.Default);

		/// <summary>
		/// Creates the IoHash for a stream.
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static IoHash Compute(Stream stream)
		{
			byte[]? buffer = null;
			try
			{
				using Blake3.Hasher hasher = Blake3.Hasher.New();
				buffer = ArrayPool<byte>.Shared.Rent(16 * 1024);
				Span<byte> span = buffer.AsSpan();

				int length;
				while ((length = stream.Read(span)) > 0)
				{
					hasher.Update(span.Slice(0, length));
				}

				ArrayPool<byte>.Shared.Return(buffer);
				return FromBlake3(hasher);
			}
			finally
			{
				if (buffer != null)
				{
					ArrayPool<byte>.Shared.Return(buffer);
				}
			}
		}

		/// <summary>
		/// Creates the IoHash for a stream asynchronously. 
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static async Task<IoHash> ComputeAsync(Stream stream, CancellationToken cancellationToken = default) => await ComputeAsync(stream, -1, cancellationToken);

		/// <summary>
		/// Creates the IoHash for a stream asynchronously. 
		/// </summary>
		/// <param name="stream">Data to compute the hash for</param>
		/// <param name="fileSizeHint">If available, the file size so an appropriate buffer size can be used</param>
		/// <param name="cancellationToken">Cancellation token used to terminate processing</param>
		/// <returns>New content hash instance containing the hash of the data</returns>
		public static async Task<IoHash> ComputeAsync(Stream stream, long fileSizeHint, CancellationToken cancellationToken = default)
		{
			const int MaxBufferSize = 1 * 1024 * 1024;
			const int MinBufferSize = 16 * 1024;
			using Blake3.Hasher hasher = Blake3.Hasher.New();
			Task Callback(ReadOnlyMemory<byte> data)
			{
				if (data.Length < MultiThreadedSize)
				{
					hasher.Update(data.Span);
				}
				else
				{
					hasher.UpdateWithJoin(data.Span);
				}
				return Task.CompletedTask;
			}

			await stream.ReadAllBytesAsync(fileSizeHint, MinBufferSize, MaxBufferSize, Callback, cancellationToken);
			return FromBlake3(hasher);
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static IoHash Parse(string text) => new(StringUtils.ParseHexString(text));

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static IoHash Parse(ReadOnlySpan<char> text) => new(StringUtils.ParseHexString(text));

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static IoHash Parse(ReadOnlySpan<byte> text) => new(StringUtils.ParseHexString(text));

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <param name="hash">Receives the hash on success</param>
		/// <returns></returns>
		public static bool TryParse(ReadOnlySpan<char> text, out IoHash hash)
		{
			byte[]? bytes;
			if (StringUtils.TryParseHexString(text, out bytes) && bytes.Length == NumBytes)
			{
				hash = new(bytes);
				return true;
			}
			else
			{
				hash = default;
				return false;
			}
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <param name="hash">Receives the hash on success</param>
		/// <returns></returns>
		public static bool TryParse(ReadOnlySpan<byte> text, out IoHash hash)
		{
			byte[]? bytes;
			if (StringUtils.TryParseHexString(text, out bytes) && bytes.Length == NumBytes)
			{
				hash = new(bytes);
				return true;
			}
			else
			{
				hash = default;
				return false;
			}
		}

		/// <inheritdoc cref="IComparable{T}.CompareTo(T)"/>
		public int CompareTo(IoHash other)
		{
			if (_a != other._a)
			{
				return (_a < other._a) ? -1 : +1;
			}
			else if (_b != other._b)
			{
				return (_b < other._b) ? -1 : +1;
			}
			else
			{
				return (_c < other._c) ? -1 : +1;
			}
		}

		/// <inheritdoc/>
		public bool Equals(IoHash other) => _a == other._a && _b == other._b && _c == other._c;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is IoHash hash) && Equals(hash);

		/// <inheritdoc/>
		public override int GetHashCode() => (int)_a;

		/// <summary>
		/// Format the hash as a utf8 string
		/// </summary>
		public Utf8String ToUtf8String()
		{
			Span<byte> buffer = stackalloc byte[NumBytes];
			CopyTo(buffer);
			return StringUtils.FormatUtf8HexString(buffer);
		}

		/// <summary>
		/// Formats the hash as a utf8 string
		/// </summary>
		/// <param name="chars">Output buffer for the converted string</param>
		public void ToUtf8String(Span<byte> chars)
		{
			Span<byte> buffer = stackalloc byte[NumBytes];
			CopyTo(buffer);
			StringUtils.FormatUtf8HexString(buffer, chars);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			Span<byte> buffer = stackalloc byte[NumBytes];
			CopyTo(buffer);
			return StringUtils.FormatHexString(buffer);
		}

		/// <summary>
		/// Convert this hash to a byte array
		/// </summary>
		/// <returns>Data for the hash</returns>
		public byte[] ToByteArray()
		{
			byte[] data = new byte[NumBytes];
			CopyTo(data);
			return data;
		}

		/// <summary>
		/// Copies this hash into a span
		/// </summary>
		/// <param name="span">Span to copy to</param>
		public void CopyTo(Span<byte> span)
		{
			BinaryPrimitives.WriteUInt64BigEndian(span, _a);
			BinaryPrimitives.WriteUInt64BigEndian(span[8..], _b);
			BinaryPrimitives.WriteUInt32BigEndian(span[16..], _c);
		}

		/// <summary>
		/// Combine two hashes into a new hash
		/// </summary>
		/// <param name="first">The first hash</param>
		/// <param name="second">The second hash</param>
		/// <remarks>This function is not commutative, nor will it result the same hash as if the original data was hashed.</remarks>
		/// <returns>A combined hash</returns>
		public static IoHash Combine(IoHash first, IoHash second)
		{
			Span<byte> buffer = stackalloc byte[NumBytes];
			using Blake3.Hasher hasher = Blake3.Hasher.New();
			first.CopyTo(buffer);
			hasher.Update(buffer);
			second.CopyTo(buffer);
			hasher.Update(buffer);
			return FromBlake3(hasher);
		}

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator ==(IoHash a, IoHash b) => a.Equals(b);

		/// <summary>
		/// Test two hash values for equality
		/// </summary>
		public static bool operator !=(IoHash a, IoHash b) => !(a == b);

		/// <summary>
		/// Tests whether A > B
		/// </summary>
		public static bool operator >(IoHash a, IoHash b) => a.CompareTo(b) > 0;

		/// <summary>
		/// Tests whether A is less than B
		/// </summary>
		public static bool operator <(IoHash a, IoHash b) => a.CompareTo(b) < 0;

		/// <summary>
		/// Tests whether A is greater than or equal to B
		/// </summary>
		public static bool operator >=(IoHash a, IoHash b) => a.CompareTo(b) >= 0;

		/// <summary>
		/// Tests whether A is less than or equal to B
		/// </summary>
		public static bool operator <=(IoHash a, IoHash b) => a.CompareTo(b) <= 0;

		/// <summary>
		/// Convert a Blake3Hash to an IoHash
		/// </summary>
		/// <param name="hash"></param>
		public static implicit operator IoHash(Blake3Hash hash) => new(hash.Span.Slice(0, NumBytes));

		/// <summary>
		/// Convert an IoHash to a uint by returning the first 32 bits of the hashed value
		/// </summary>
		/// <param name="hash"></param>
		public static explicit operator uint(IoHash hash) => (uint)(hash._a >> 32);
	}

	/// <summary>
	/// Extension methods for dealing with IoHash values
	/// </summary>
	public static class IoHashExtensions
	{
		/// <summary>
		/// Read an <see cref="IoHash"/> from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New IoHash instance</returns>
		public static IoHash? ReadIoHash(this BinaryArchiveReader reader)
		{
			byte[]? data = reader.ReadByteArray();
			return data == null ? null : new(data);
		}

		/// <summary>
		/// Read an <see cref="IoHash"/> from a memory reader
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>New IoHash instance</returns>
		public static IoHash ReadIoHash(this IMemoryReader reader) => new(reader.ReadFixedLengthBytes(IoHash.NumBytes).Span);

		/// <summary>
		/// Write an <see cref="IoHash"/> to a binary archive
		/// </summary>
		/// <param name="writer">The writer to output data to</param>
		/// <param name="hash">The IoHash to write</param>
		public static void WriteIoHash(this BinaryArchiveWriter writer, IoHash? hash) => writer.WriteByteArray(hash?.ToByteArray());

		/// <summary>
		/// Write an <see cref="IoHash"/> to a memory writer
		/// </summary>
		/// <param name="writer">The writer to output data to</param>
		/// <param name="hash">The IoHash to write</param>
		public static void WriteIoHash(this IMemoryWriter writer, IoHash hash)
		{
			hash.CopyTo(writer.GetSpan(IoHash.NumBytes));
			writer.Advance(IoHash.NumBytes);
		}
	}

	/// <summary>
	/// Type converter for IoHash to and from JSON
	/// </summary>
	sealed class IoHashJsonConverter : JsonConverter<IoHash>
	{
		/// <inheritdoc/>
		public override IoHash Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => IoHash.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, IoHash value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToUtf8String().Span);
	}

	/// <summary>
	/// Type converter from strings to IoHash objects
	/// </summary>
	sealed class IoHashTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => IoHash.Parse((string)value);
	}
}
