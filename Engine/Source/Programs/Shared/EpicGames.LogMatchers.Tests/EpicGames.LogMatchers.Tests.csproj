<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.LogMatchers\EpicGames.LogMatchers.csproj" />
    <ProjectReference Include="..\EpicGames.Perforce\EpicGames.Perforce.csproj" />
  </ItemGroup>
</Project>
