// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches compile errors and annotates with the source file path and revision
	/// </summary>
	class CompileEventMatcher : ILogEventMatcher
	{
		const string FilePattern =
			@"(?:(?<file>" +
				// optional drive letter
				@"(?:[a-zA-Z]:)?" +
				// any non-colon character
				@"[^:(\s]+" +
				// any path character
				@"[^:<>*?""]+" +
				// valid source file extension (or extensionless file)
				@"(?:\.(?i)(?:h|hpp|hxx|c|cc|cpp|cxx|inc|inl|cs|targets|verse)|[/\\][a-zA-Z0-9_]+)" +
			// or the string "<scratch space>"
			@")|<scratch space>)";

		const string VisualCppLocationPattern =
			@"\(" +
				@"(?:" +
					@"(?<line>\d+)" + // (line)
					@"|" +
					@"(?:(?<line>\d+)-(?<maxline>\d+))" + // (line-line)
					@"|" +
					@"(?:(?<line>\d+),(?<column>\d+))" + // (line,col)
					@"|" +
					@"(?:(?<line>\d+),(?<column>\d+)-(?<maxcolumn>\d+))" + // (line,col-col)
					@"|" +
					@"(?:(?<line>\d+),(?<column>\d+),\s*(?<maxline>\d+),(?<maxcolumn>\d+))" + // (line,col,line,col)
				@")" +
			@"\)";

		const string VisualCppSeverity =
			@"(?:Verse compiler |Script )?(?<severity>fatal error|[Ee]rror|[Ww]arning)(?: (?<code>[A-Z]*[0-9]+))?"; // "E"rror/"W"arning are from UHT

		const string ClangLocationPattern =
			@":" +
				@"(?<line>\d+)" + // line number
				@"(?::(?<column>\d+))?" + // optional column number
			@"";

		const string ClangSeverity =
			@"(?<severity>error|warning|fatal error)";

		static readonly Regex s_baseFilePattern = new(@"^\s*(?:\[[\d/]+\] Compile |[^/\ :]+\\.cpp\s*(?:\([^\)]*\))?$)");
		static readonly Regex s_preludePattern = new(@"^\s*(?:In (member )?function|In file included from)");
		static readonly Regex s_preludeFilePattern = new($"^\\s*In file included from {FilePattern}:");
		static readonly Regex s_blankLinePattern = new(@"^\s*$");
		static readonly Regex s_errorWarningPattern = new("[Ee]rror|[Ww]arning");
		static readonly Regex s_clangDiagnosticPattern = new($"^\\s*{FilePattern}\\s*{ClangLocationPattern}:\\s*{ClangSeverity}\\s*:");
		static readonly Regex s_clangNotePattern = new($"^\\s*{FilePattern}\\s*{ClangLocationPattern}:\\s*note:");
		static readonly Regex s_clangMarkerPattern = new(@"^(\s*)[\^~][\s\^~]*$");
		static readonly Regex s_llvmDiagnosticPattern = new($"^\\s*{ClangSeverity}:\\s*{FilePattern}\\s*{ClangLocationPattern}:\\s*");
		static readonly Regex s_llvmNotePattern = new($"^\\s*note:\\s*(?:{FilePattern}\\s*{ClangLocationPattern}:)?");
		static readonly Regex s_xcodeIDEWatchExtensionPattern = new(@"xcodebuild.*Requested but did not find extension point with identifier.*for extension.*\.watchOS of plug-in com\.apple\.dt\.IDEWatchSupportCore");
		static readonly Regex s_scriptCompilePattern = new(@"^\s*[A-Za-z0-9_\.]+ ERROR:.* [A-Za-z_]+ failed to compile\.");
		static readonly Regex s_cscSummaryPattern = new(@"^\s+\d+ (?:Warning|Error)\(s\)");
		static readonly Regex s_cscOutputPattern = new(@"^  [^ ]+ -> ");
		static readonly Regex s_tempLibpvxMacLoadCmdPattern = new(@"no platform load command found in");
		static readonly Regex s_tempXCodeDupLibrariesPattern = new(@"ignoring duplicate libraries");

		static readonly string[] s_invalidExtensions =
		[
			".obj",
			".dll",
			".exe"
		];

		const string DefaultSourceFileBaseDir = "Engine/Source";

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			// Match the prelude to any error
			int maxOffset = 0;
			if (input.IsMatch(maxOffset, s_baseFilePattern))
			{
				maxOffset++;
			}
			while (input.IsMatch(maxOffset, s_preludePattern))
			{
				maxOffset++;
			}

			// Do the match in two phases so we can early out if the strings "error" or "warning" are not present. The patterns before these strings can
			// produce many false positives, making them very slow to execute.
			if (input.IsMatch(maxOffset, s_errorWarningPattern))
			{
				// Tag any files in the prelude with their source files
				LogEventBuilder builder = new(input);
				for (int idx = 0; idx < maxOffset; idx++)
				{
					Match? fileMatch;
					if (builder.Current.TryMatch(s_preludeFilePattern, out fileMatch))
					{
						builder.AnnotateSourceFile(fileMatch.Groups["file"], DefaultSourceFileBaseDir);
					}
					builder.MoveNext();
				}

				// Try to match a Visual C++ diagnostic
				LogEventMatch? eventMatch;
				if (TryMatchVisualCppEvent(builder, out eventMatch))
				{
					LogEvent newEvent = eventMatch!.Events[eventMatch.Events.Count - 1];

					// If warnings as errors is enabled, upgrade any following warnings to errors.
					LogValue? code;
					if (newEvent.Properties != null && newEvent.TryGetProperty("code", out code) && code.Text.Equals("C2220", StringComparison.Ordinal))
					{
						ILogCursor nextCursor = builder.Next;
						while (nextCursor.CurrentLine != null)
						{
							LogEventBuilder nextBuilder = new(nextCursor);

							LogEventMatch? nextMatch;
							if (!TryMatchVisualCppEvent(nextBuilder, out nextMatch))
							{
								break;
							}
							foreach (LogEvent matchEvent in nextMatch.Events)
							{
								matchEvent.Level = LogLevel.Error;
							}
							eventMatch.Events.AddRange(nextMatch.Events);

							nextCursor = nextBuilder.Next;
						}
					}
					return eventMatch;
				}

				// Try to match a Clang or LLVM diagnostic
				Match? match;
				if ((builder.Current.TryMatch(s_clangDiagnosticPattern, out match) ||
					 builder.Current.TryMatch(s_llvmDiagnosticPattern, out match)) && IsSourceFile(match))
				{
					LogLevel level = GetLogLevelFromSeverity(match);

					builder.AnnotateSourceFile(match.Groups["file"], DefaultSourceFileBaseDir);
					builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
					builder.TryAnnotate(match.Groups["line"], LogEventMarkup.LineNumber);
					builder.TryAnnotate(match.Groups["column"], LogEventMarkup.ColumnNumber);

					for (; ; )
					{
						SkipClangMarker(builder);

						if (!(builder.Next.TryMatch(s_clangNotePattern, out match) ||
							  builder.Next.TryMatch(s_llvmNotePattern, out match)))
						{
							break;
						}

						builder.MoveNext();

						Group fileGroup = match.Groups["file"];
						if (fileGroup.Success)
						{
							builder.AnnotateSourceFile(fileGroup, DefaultSourceFileBaseDir);
							builder.TryAnnotate(match.Groups["line"], LogEventMarkup.LineNumber);
						}
					}

					return builder.ToMatch(LogEventPriority.High, level, KnownLogEvents.Compiler);
				}

				// Try to match an Xcode diagnostic.
				if (TryMatchXcodeCppEvent(builder, out eventMatch))
				{
					return eventMatch;
				}
			}
			else if (input.IsMatch(s_xcodeIDEWatchExtensionPattern))
			{
				LogEventBuilder builder = new(input);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_XCode);
			}
			else if (input.IsMatch(s_scriptCompilePattern))
			{
				LogEventBuilder builder = new(input);
				return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Compiler_Summary);
			}
			return null;
		}

		static readonly Regex s_xcodeProjectErrorPattern = new(@"^\s*(?<file>.+?\.xcodeproj):\s*(?<severity>[Ee]rror|[Ww]arning):\s*(?<message>.+)$");

		static readonly Regex s_xcodeRebuildPCHPattern = new(@"(?<severity>(?:[Ff]atal )?[Ee]rror): file '(?<file>.*)' has been modified since the precompiled header '(?<pch>.*)' was built");
		
		static bool TryMatchXcodeCppEvent(LogEventBuilder builder, [NotNullWhen(true)] out LogEventMatch? outEvent)
		{
			Match? match;
			
			if (builder.Current.TryMatch(s_xcodeProjectErrorPattern, out match))
			{
				LogLevel level = GetLogLevelFromSeverity(match);
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
				builder.AnnotateSourceFile(match.Groups["file"], null);
				outEvent = builder.ToMatch(LogEventPriority.High, level, KnownLogEvents.Systemic_XCode);
				return true;
			}

			if (builder.Current.TryMatch(s_xcodeRebuildPCHPattern, out match))
			{
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
				builder.AnnotateSourceFile(match.Groups["pch"], null);
				outEvent = builder.ToMatch(LogEventPriority.Highest, LogLevel.Error, KnownLogEvents.Compiler);
				return true;
			}

			if (builder.Current.Contains("was built for newer macOS version"))
			{
				outEvent = builder.ToMatch(LogEventPriority.Highest, LogLevel.Information, KnownLogEvents.Systemic_XCode);
				return true;
			}

			// Temporary XCode silencing when linker makes assumption for macos.
			if (builder.Current.IsMatch(s_tempLibpvxMacLoadCmdPattern))
			{
				outEvent = builder.ToMatch(LogEventPriority.Highest, LogLevel.Information, KnownLogEvents.Systemic_XCode);
				return true;
			}
			if (builder.Current.IsMatch(s_tempXCodeDupLibrariesPattern))
			{
				outEvent = builder.ToMatch(LogEventPriority.Highest, LogLevel.Information, KnownLogEvents.Systemic_XCode);
				return true;
			}

			outEvent = null;
			return false;
		}

		static readonly Regex s_msvcPattern = new($"^\\s*(?:ERROR: |WARNING: |Log[A-Za-z_]+: [A-Z][a-z]+: )?{FilePattern}(?:{VisualCppLocationPattern})? ?:\\s+{VisualCppSeverity}:");
		static readonly Regex s_msvcNotePattern = new($"^\\s*{FilePattern}(?:{VisualCppLocationPattern})?\\s*: note:");
		static readonly Regex s_projectPattern = new(@"\[(?<project>[^[\]]+)]\s*$");

		static bool TryMatchVisualCppEvent(LogEventBuilder builder, [NotNullWhen(true)] out LogEventMatch? outEvent)
		{
			Match? match;
			if (!builder.Current.TryMatch(s_msvcPattern, out match) || !IsSourceFile(match))
			{
				outEvent = null;
				return false;
			}

			LogLevel level = GetLogLevelFromSeverity(match);
			builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);

			string sourceFileBaseDir = DefaultSourceFileBaseDir;

			Group codeGroup = match.Groups["code"];
			if (codeGroup.Success)
			{
				builder.Annotate(codeGroup, LogEventMarkup.ErrorCode);

				if (codeGroup.Value.StartsWith("CS", StringComparison.Ordinal))
				{
					Match? projectMatch;
					if (builder.Current.TryMatch(s_projectPattern, out projectMatch))
					{
						builder.AnnotateSourceFile(projectMatch.Groups[1], "");
						sourceFileBaseDir = GetPlatformAgnosticDirectoryName(projectMatch.Groups[1].Value) ?? sourceFileBaseDir;
					}
				}
				else if (codeGroup.Value.StartsWith("MSB", StringComparison.Ordinal))
				{
					if (codeGroup.Value.Equals("MSB3026", StringComparison.Ordinal))
					{
						outEvent = builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_MSBuild);
						return true;
					}

					Match? projectMatch;
					if (builder.Current.TryMatch(s_projectPattern, out projectMatch))
					{
						builder.AnnotateSourceFile(projectMatch.Groups[1], "");
						outEvent = builder.ToMatch(LogEventPriority.High, level, KnownLogEvents.MSBuild);
						return true;
					}
				}
				else if (codeGroup.Value.StartsWith('C'))
				{
					if (codeGroup.Value.Equals("C1060", StringComparison.Ordinal))
					{
						outEvent = builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Systemic_MSBuild);
						return true;
					}
				}
			}

			builder.AnnotateSourceFile(match.Groups["file"], sourceFileBaseDir);
			builder.TryAnnotate(match.Groups["line"], LogEventMarkup.LineNumber);
			builder.TryAnnotate(match.Groups["column"], LogEventMarkup.ColumnNumber);
			builder.TryAnnotate(match.Groups["maxline"], LogEventMarkup.LineNumber);
			builder.TryAnnotate(match.Groups["maxcolumn"], LogEventMarkup.ColumnNumber);

			string indent = ExtractIndent(builder.Current.CurrentLine ?? String.Empty);
			string nextIndent = indent + " ";

			for (; ; )
			{
				while (builder.Current.StartsWith(1, nextIndent) && !builder.Current.IsMatch(1, s_cscSummaryPattern) && !builder.Current.IsMatch(1, s_cscOutputPattern))
				{
					builder.MoveNext();
				}

				// Clang-as-MSVC outputs warnings using its own marker syntax
				SkipClangMarker(builder);

				int offset = 1;
				while (builder.Current.IsMatch(offset, s_blankLinePattern))
				{
					offset++;
				}
				if (!builder.Current.TryMatch(offset, s_msvcNotePattern, out match))
				{
					break;
				}
				builder.MoveNext(offset);

				Group group = match.Groups["file"];
				if (group.Success)
				{
					builder.AnnotateSourceFile(group, DefaultSourceFileBaseDir);
					builder.TryAnnotate(match.Groups["line"], LogEventMarkup.LineNumber);
					builder.AddProperty("note", true);
				}
			}

			outEvent = builder.ToMatch(LogEventPriority.High, level, KnownLogEvents.Compiler);
			return true;
		}

		static void SkipClangMarker(LogEventBuilder builder)
		{
			Match? match;
			if (builder.Current.TryMatch(2, s_clangMarkerPattern, out match))
			{
				string indent = match.Groups[1].Value;

				int length = 2;
				if (indent.Length > 0 && builder.Current.TryGetLine(3, out string? suggestLine) && suggestLine.Length > indent.Length && suggestLine.StartsWith(indent, StringComparison.Ordinal) && !Char.IsWhiteSpace(suggestLine[indent.Length]))
				{
					length++;
				}
				builder.MoveNext(length);
			}
		}

		static string? GetPlatformAgnosticDirectoryName(string fileName)
		{
			int index = fileName.LastIndexOfAny(['/', '\\']);
			if (index == -1)
			{
				return null;
			}
			else
			{
				return fileName[..index];
			}
		}

		static bool IsSourceFile(Match match)
		{
			Group group = match.Groups["file"];
			if (!group.Success)
			{
				return false;
			}

			string text = group.Value;
			if (s_invalidExtensions.Any(x => text.EndsWith(x, StringComparison.OrdinalIgnoreCase)))
			{
				return false;
			}

			return true;
		}

		static LogLevel GetLogLevelFromSeverity(Match match)
		{
			string severity = match.Groups["severity"].Value;
			if (severity.Equals("warning", StringComparison.OrdinalIgnoreCase))
			{
				return LogLevel.Warning;
			}
			else
			{
				return LogLevel.Error;
			}
		}

		static string ExtractIndent(string line)
		{
			int length = 0;
			while (length < line.Length && line[length] == ' ')
			{
				length++;
			}
			return new string(' ', length);
		}
	}
}
