// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches compile errors and annotates with the source file path and revision
	/// </summary>
	class ContentEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_errorWarningPattern = new("Error:|Warning:");

		static readonly Regex s_pattern = new(
			@"^\s*" +
			@"(?:\[[\d\.\-: ]+\])*" +
			@"(?<channel>[a-zA-Z0-9]+):\s+" +
			@"(?<severity>Error:|Warning:)\s+" +
			@"(?:\[AssetLog\] )?" +
			@"(?<asset>(?:[a-zA-Z]:)?[^:]+(?:.uasset|.umap)):\s*" +
			@"(?<message>.*)");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			// Do the match in two phases so we can early out if the strings "error" or "warning" are not present. The patterns before these strings can
			// produce many false positives, making them very slow to execute.
			if (input.IsMatch(s_errorWarningPattern))
			{
				Match? match = s_pattern.Match(input.CurrentLine!);
				if (match.Success)
				{
					LogEventBuilder builder = new(input);
					builder.Annotate(match.Groups["channel"], LogEventMarkup.Channel);
					builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
					builder.AnnotateAsset(match.Groups["asset"]);
					builder.Annotate(match.Groups["message"], LogEventMarkup.Message);

					return builder.ToMatchWithHangingLines(LogEventPriority.AboveNormal, GetLogLevelFromSeverity(match), KnownLogEvents.Engine_AssetLog);
				}
			}
			return null;
		}

		static LogLevel GetLogLevelFromSeverity(Match match)
		{
			string severity = match.Groups["severity"].Value;
			if (severity.Equals("Warning:", StringComparison.Ordinal))
			{
				return LogLevel.Warning;
			}
			else
			{
				return LogLevel.Error;
			}
		}
	}
}
