// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for engine crashes
	/// </summary>
	class CrashEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_exitCodePattern = new(@"ExitCode=(3|139|255)(?!\d)");

		static readonly Regex s_appErrorPattern = new(@"^\s*[A-Za-z]+: Error: appError called: ");

		static readonly Regex s_assertionFailedPattern = new(@"^Assertion failed: ");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.Contains("begin: stack for UAT"))
			{
				LogEventBuilder builder = new(cursor);
				for (int count = 0; count < 200; count++)
				{
					if (builder.Next.CurrentLine == null)
					{
						break;
					}
					builder.MoveNext();
					if (builder.Current.Contains("end: stack for UAT"))
					{
						return builder.ToMatch(LogEventPriority.BelowNormal, GetLogLevel(cursor), KnownLogEvents.Engine_Crash);
					}
				}
				// Found "begin: stack for UAT" but not the closing marker yet; return a partial
				// match so the parser waits for more data rather than falling through line-by-line.
				return builder.ToMatch(LogEventPriority.BelowNormal, GetLogLevel(cursor), KnownLogEvents.Engine_Crash, isComplete: false);
			}
			if (cursor.IsMatch(s_appErrorPattern))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Error, KnownLogEvents.Engine_AppError);
			}
			if (cursor.IsMatch(s_assertionFailedPattern))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Error, KnownLogEvents.Engine_AssertionFailed);
			}
			if (cursor.Contains("AutomationTool: Stack:"))
			{
				LogEventBuilder builder = new(cursor);
				while (builder.Current.Contains(1, "AutomationTool: Stack:"))
				{
					builder.MoveNext();
				}
				return builder.ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.AutomationTool_Crash);
			}

			Match? match;
			if (cursor.TryMatch(s_exitCodePattern, out match))
			{
				LogEventBuilder builder = new(cursor);
				builder.Annotate("exitCode", match.Groups[1]);
				return builder.ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.AutomationTool_CrashExitCode);
			}
			return null;
		}

		static readonly Regex s_errorPattern = new("[Ee]rror:");
		static readonly Regex s_warningPattern = new("[Ww]arning:");

		static LogLevel GetLogLevel(ILogCursor cursor)
		{
			if (cursor.IsMatch(0, s_errorPattern))
			{
				return LogLevel.Error;
			}
			else if (cursor.IsMatch(0, s_warningPattern))
			{
				return LogLevel.Warning;
			}
			else
			{
				return LogLevel.Information;
			}
		}
	}
}
