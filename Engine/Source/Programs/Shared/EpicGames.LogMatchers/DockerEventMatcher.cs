// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for docker errors
	/// </summary>
	class DockerEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_noSpacePattern = new(": no space left on device$");
		static readonly Regex s_updateAlternativesPattern = new(@"^update-alternatives: warning:");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_noSpacePattern))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Error, KnownLogEvents.Systemic_OutOfDiskSpace);
			}
			if (cursor.IsMatch(s_updateAlternativesPattern))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_Docker);
			}
			return null;
		}
	}
}
