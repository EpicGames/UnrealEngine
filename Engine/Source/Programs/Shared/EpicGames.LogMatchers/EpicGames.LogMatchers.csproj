<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Logging" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>
</Project>
