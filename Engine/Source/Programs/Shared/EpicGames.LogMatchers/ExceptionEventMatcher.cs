// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches a generic C# exception
	/// </summary>
	class ExceptionEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_pattern = new(@"^\s*Unhandled Exception: ");
		static readonly Regex s_atPattern = new(@"^\s*at ");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_pattern))
			{
				LogEventBuilder builder = new(cursor);
				while (builder.Current.IsMatch(1, s_atPattern))
				{
					builder.MoveNext();
				}
				return builder.ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.Exception);
			}
			return null;
		}
	}
}
