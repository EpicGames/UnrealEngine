// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for editor/UAT instances exiting with an error
	/// </summary>
	class ExitCodeEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_pattern = new(
			@"Editor terminated with exit code -?[1-9]" +
			@"|(?:Error executing.+)(?:tool returned code)(.+)" +
			@"|\s*RunU[AB]T ERROR:" +
			@"|\s*ERROR:.*did not complete successfully: exit code: [0-9]+");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			int numLines = 0;
			for (; ; )
			{
				if (cursor.IsMatch(numLines, s_pattern))
				{
					numLines++;
				}
				else
				{
					break;
				}
			}

			if (numLines > 0)
			{
				LogEventBuilder builder = new(cursor);
				builder.MoveNext(numLines - 1);
				return builder.ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.ExitCode);
			}
			return null;
		}
	}
}

