// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Low-priority matcher for generic error strings like "warning:" and "error:"
	/// </summary>
	class GenericEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_fatalPattern = new(
			@"^\s*(FATAL|fatal error):");

		static readonly Regex s_warningErrorPattern = new(
			@"(?<!\w)(?i)(WARNING|ERROR) ?(\([^)]+\)|\[[^\]]+\])?:(?: |$)");

		static readonly Regex s_errorPattern = new(
			@"[Ee]rror [A-Z]\d+\s:");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor cursor)
		{
			Match? match;
			if (cursor.TryMatch(s_fatalPattern, out _))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.Generic);
			}
			if (cursor.TryMatch(s_warningErrorPattern, out match))
			{
				// Careful to match the first WARNING or ERROR in the line here.
				LogLevel level = LogLevel.Error;
				if (match.Groups[1].Value.Equals("WARNING", StringComparison.OrdinalIgnoreCase))
				{
					level = LogLevel.Warning;
				}

				return new LogEventBuilder(cursor).ToMatchWithHangingLines(LogEventPriority.Lowest, level, KnownLogEvents.Generic);
			}
			if (cursor.IsMatch(s_errorPattern))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Lowest, LogLevel.Error, KnownLogEvents.Generic);
			}
			return null;
		}
	}
}
