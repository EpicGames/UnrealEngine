// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for Gradle errors
	/// </summary>
	class GradleEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_failurePattern = new(@"^\s*FAILURE:");
		static readonly Regex s_blankPattern = new(@"^\s*$");
		static readonly Regex s_indentPattern = new(@"^(\s*)\*");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_failurePattern))
			{
				int maxOffset = 1;
				for (; ; )
				{
					int newMaxOffset = maxOffset;
					while (cursor.IsMatch(newMaxOffset, s_blankPattern))
					{
						newMaxOffset++;
					}

					Match? match;
					if (!cursor.TryMatch(newMaxOffset, s_indentPattern, out match))
					{
						break;
					}
					maxOffset = newMaxOffset + 1;

					while (cursor.IsMatch(maxOffset, new Regex($"^{match.Groups[1].Value}")))
					{
						maxOffset++;
					}
				}

				LogEventBuilder builder = new(cursor, maxOffset);
				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Error, KnownLogEvents.AutomationTool);
			}

			return null;
		}
	}
}

