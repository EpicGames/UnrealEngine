// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for linker errors
	/// </summary>
	class LinkEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_undefinedReferencePattern = new(
			@": undefined reference to |undefined symbol|^\s*(ld|ld.lld|lld-link|(.*clang)):|^[^:]*[^a-zA-Z][a-z]?ld: |^\s*>>>");

		static readonly Regex s_undefinedSymbolsForArchPattern = new(
			@"^(\s*)Undefined symbols for architecture");

		static readonly Regex s_ldClangPattern = new(
			@"^\s*(ld|ld.lld|lld-link|clang|clang\+\+):");

		static readonly Regex s_ldClangErrorPattern = new(
			@"^\s*(ld|ld.lld|lld-link|clang|clang\+\+):\s+error:\s+(?<error>.+)$");

		static readonly Regex s_ldMultiplyDefinedPattern = new(
			@"(?<severity>error): L\d+: symbol `(?<symbol>[^`]+)' multiply defined$");

		static readonly Regex s_ldMultiplyDefinedInfoPattern = new(
			@"error:  \.\.\.");

		static readonly Regex s_microsoftErrorPattern = new(
			@"error (?<code>LNK\d+):");

		static readonly Regex s_ignoredLinkWarningPattern = new(
			@"^\s*ld: warning: Linker asked to preserve internal global:");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_ignoredLinkWarningPattern))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Linker);
			}

			int lineCount = 0;
			bool isError = false;
			bool isWarning = false;
			while (cursor.IsMatch(lineCount, s_undefinedReferencePattern))
			{
				isError |= cursor.Contains(lineCount, "error:");
				isWarning |= cursor.Contains(lineCount, "warning:");
				lineCount++;
			}
			if (lineCount > 0)
			{
				LogEventBuilder builder = new(cursor);

				bool hasSymbol = AddSymbolMarkupForLine(builder);
				for (int idx = 1; idx < lineCount; idx++)
				{
					builder.MoveNext();
					hasSymbol |= AddSymbolMarkupForLine(builder);
				}
				for (; ; )
				{
					if (builder.Next.Contains("ld:"))
					{
						break;
					}
					else if (builder.Next.Contains("error:"))
					{
						isError = true;
					}
					else if (builder.Next.Contains("warning:"))
					{
						isWarning = true;
					}
					else
					{
						break;
					}

					hasSymbol |= AddSymbolMarkupForLine(builder);
					builder.MoveNext();
				}

				LogLevel level = (isError || !isWarning) ? LogLevel.Error : LogLevel.Warning;
				EventId eventId = hasSymbol ? KnownLogEvents.Linker_UndefinedSymbol : KnownLogEvents.Linker;
				return builder.ToMatch(LogEventPriority.Normal, level, eventId);
			}

			Match? match;
			if (cursor.TryMatch(s_undefinedSymbolsForArchPattern, out match))
			{
				LogEventBuilder builder = new(cursor);
				AddSymbolMarkupForLine(builder);

				string prefix = $"^(?<prefix>{match.Groups[1].Value}\\s+)";
				while (builder.Next.TryMatch(new Regex(prefix + @"""(?<symbol>[^""]+)"""), out match))
				{
					string nextPrefix = $"^{match.Groups["prefix"].Value}\\s+";

					builder.MoveNext();
					builder.AnnotateSymbol(match.Groups["symbol"]);

					while (builder.Next.TryMatch(new Regex(nextPrefix + "(?<symbol>[^ ].*) in "), out match))
					{
						builder.MoveNext();
						builder.AnnotateSymbol(match.Groups["symbol"]);
					}
				}

				while (builder.Next.IsMatch(s_ldClangPattern))
				{
					builder.MoveNext();
				}

				return builder.ToMatch(LogEventPriority.Normal, LogLevel.Error, KnownLogEvents.Linker_UndefinedSymbol);
			}
			if (cursor.TryMatch(s_microsoftErrorPattern, out match))
			{
				LogEventBuilder builder = new(cursor);
				AddSymbolMarkupForLine(builder);

				Group codeGroup = match.Groups["code"];
				builder.Annotate(codeGroup, LogEventMarkup.ErrorCode);

				if (codeGroup.Value.Equals("LNK2001", StringComparison.Ordinal) || codeGroup.Value.Equals("LNK2019", StringComparison.Ordinal))
				{
					return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Linker_UndefinedSymbol);
				}
				else if (codeGroup.Value.Equals("LNK2005", StringComparison.Ordinal) || codeGroup.Value.Equals("LNK4022", StringComparison.Ordinal))
				{
					return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Linker_DuplicateSymbol);
				}
				else
				{
					return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Linker);
				}
			}
			if (cursor.TryMatch(s_ldMultiplyDefinedPattern, out match))
			{
				LogEventBuilder builder = new(cursor);
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
				builder.AnnotateSymbol(match.Groups["symbol"]);

				while (builder.Current.IsMatch(1, s_ldMultiplyDefinedInfoPattern))
				{
					builder.MoveNext();
				}

				return builder.ToMatch(LogEventPriority.Highest, LogLevel.Error, KnownLogEvents.Linker_DuplicateSymbol);
			}
			if (cursor.TryMatch(s_ldClangErrorPattern, out match))
			{
				LogEventBuilder builder = new(cursor);
				builder.Annotate(match.Groups["error"], LogEventMarkup.Message);

				return builder.ToMatch(LogEventPriority.Highest, LogLevel.Error, KnownLogEvents.Linker);
			}

			return null;
		}

		static bool AddSymbolMarkupForLine(LogEventBuilder builder)
		{
			bool hasSymbols = false;

			string? message = builder.Current.CurrentLine;
			if (message == null)
			{
				return false;
			}

			// Mac link error:
			//   Undefined symbols for architecture arm64:
			//     "Foo::Bar() const", referenced from:
			Match symbolMatch = Regex.Match(message, "^  \"(?<symbol>.+)\"");
			if (symbolMatch.Success)
			{
				builder.AnnotateSymbol(symbolMatch.Groups[1]);
				hasSymbols = true;
			}

			// Android link error:
			//   Foo.o:(.data.rel.ro + 0x5d88): undefined reference to `Foo::Bar()'
			Match undefinedReference = Regex.Match(message, ": undefined reference to [`'](?<symbol>[^`']+)");
			if (undefinedReference.Success)
			{
				builder.AnnotateSymbol(undefinedReference.Groups[1]);
				hasSymbols = true;
			}

			// LLD link error:
			//   ld.lld.exe: error: undefined symbol: Foo::Bar() const
			Match lldMatch = Regex.Match(message, "error: undefined symbol:\\s*(?<symbol>.+)");
			if (lldMatch.Success)
			{
				builder.AnnotateSymbol(lldMatch.Groups[1]);
				hasSymbols = true;
			}

			// Link error:
			//   Link: error: L0039: reference to undefined symbol `Foo::Bar() const' in file
			Match linkMatch = Regex.Match(message, ": reference to undefined symbol [`'](?<symbol>[^`']+)");
			if (linkMatch.Success)
			{
				builder.AnnotateSymbol(linkMatch.Groups[1]);
				hasSymbols = true;
			}
			Match linkMultipleMatch = Regex.Match(message, @": (?<symbol>[^\s]+) already defined in");
			if (linkMultipleMatch.Success)
			{
				builder.AnnotateSymbol(linkMultipleMatch.Groups[1]);
				hasSymbols = true;
			}

			// Microsoft linker error:
			//   Foo.cpp.obj : error LNK2001: unresolved external symbol \"private: virtual void __cdecl UAssetManager::InitializeAssetBundlesFromMetadata_Recursive(class UStruct const *,void const *,struct FAssetBundleData &,class FName,class TSet<void const *,struct DefaultKeyFuncs<void const *,0>,class FDefaultSetAllocator> &)const \" (?InitializeAssetBundlesFromMetadata_Recursive@UAssetManager@@EEBAXPEBVUStruct@@PEBXAEAUFAssetBundleData@@VFName@@AEAV?$TSet@PEBXU?$DefaultKeyFuncs@PEBX$0A@@@VFDefaultSetAllocator@@@@@Z)",
			Match microsoftMatch = Regex.Match(message, " symbol \"(?<symbol>[^\"]*)\"");
			if (microsoftMatch.Success)
			{
				builder.AnnotateSymbol(microsoftMatch.Groups[1]);
				hasSymbols = true;
			}

			return hasSymbols;
		}
	}
}
