// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches events formatted as UE log channel output by the localization commandlet
	/// </summary>
	class LocalizationEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_pattern = new(
			@"^(\s*)" +
			@"(?:\[[\d\.\-: ]+\])*" +
			@"(?<channel>LogLocTextHelper|LogGatherTextFromSourceCommandlet):\s*" +
			@"(?<severity>Error|Warning|Display):\s+" +
			@"(?<file>([a-zA-Z]:)?[^:/\\]*[/\\][^:]+[^\)])" +
			@"(?:\((?<line>\d+)\))?" +
			@":");

		static readonly Regex s_trailingPattern = new(@": See conflicting location\.\s*$");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			Match? match;
			if (input.TryMatch(s_pattern, out match))
			{
				LogLevel level = match.Groups["severity"].Value switch
				{
					"Error" => LogLevel.Error,
					"Warning" => LogLevel.Warning,
					_ => LogLevel.Information,
				};

				LogEventBuilder builder = new(input);
				Annotate(builder, match);

				if (builder.IsNextLineAligned() && builder.Next.TryMatch(s_pattern, out match) && builder.Next.IsMatch(s_trailingPattern))
				{
					builder.MoveNext();
					Annotate(builder, match);
				}

				return builder.ToMatch(LogEventPriority.High, level, KnownLogEvents.Engine_Localization);
			}
			return null;
		}

		static void Annotate(LogEventBuilder builder, Match match)
		{
			builder.Annotate(match.Groups["channel"], LogEventMarkup.Channel);
			builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
			builder.AnnotateSourceFile(match.Groups["file"], "Engine");
			builder.Annotate(match.Groups["line"], LogEventMarkup.LineNumber);
		}
	}
}
