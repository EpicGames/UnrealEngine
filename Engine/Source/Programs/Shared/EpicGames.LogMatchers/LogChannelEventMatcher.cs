// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches events formatted as UE log channel output
	/// </summary>
	class LogChannelEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_pattern = new(
			@"^(\s*)" +
			@"(?:\[[\d\.\-: ]+\])*" +
			@"(?<channel>[a-zA-Z_][a-zA-Z0-9_]*):\s*" +
			@"(?<severity>Error|Warning|Display): "
		);

		static readonly Regex s_callstackLine = new(@"\[Callstack\]");
		static readonly Regex s_crashHeader = new(@"=== Critical error: ===|=== Handled ensure: ===");

		// Extracts the assertion/ensure condition and filename for platform-agnostic hashing
		static readonly Regex s_assertPattern = new(
			@"(?<type>Assertion failed|Ensure condition failed): (?<condition>.+?)\s+\[File:(?:.*[/\\])?(?<file>[^\]]+)\]");

		// Matches blank log channel lines like "LogWindows: Error:" that UE appends as padding after crash output
		static readonly Regex s_blankChannelLine = new(
			@"^(\s*)" +
			@"(?:\[[\d\.\-: ]+\])*" +
			@"[a-zA-Z_][a-zA-Z0-9_]*:\s*" +
			@"(?:Error|Warning|Display):\s*$"
		);

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			if (input.TryMatch(s_pattern, out Match? match))
			{
				if (input.Contains("There is not enough space on the disk."))
				{
					return new LogEventBuilder(input).ToMatch(LogEventPriority.Low, LogLevel.Error, KnownLogEvents.Systemic_OutOfDiskSpace);
				}

				LogEventBuilder builder = new(input);
				builder.Annotate(match.Groups["channel"], LogEventMarkup.Channel);
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);

				LogLevel level = match.Groups["severity"].Value switch
				{
					"Error" => LogLevel.Error,
					"Warning" => LogLevel.Warning,
					_ => LogLevel.Information,
				};

				// Crash header with callstack: consume header + context + all [Callstack] frames + trailing blank lines
				if (s_crashHeader.IsMatch(input.CurrentLine ?? ""))
				{
					int callstackOffset = -1;
					for (int i = 1; i <= 20; i++)
					{
						if (!input.TryGetLine(i, out string? line))
						{
							break;
						}
						if (line.Contains("[Callstack]", System.StringComparison.Ordinal))
						{
							callstackOffset = i;
							break;
						}
					}

					if (callstackOffset > 0)
					{
						builder.AddProperty("Callstack", true);

						// Scan intermediate lines for an assertion/ensure message to use as
						// a platform-agnostic summary for cross-platform issue matching
						for (int j = 1; j < callstackOffset; j++)
						{
							if (input.TryGetLine(j, out string? intermediateLine))
							{
								Match assertMatch = s_assertPattern.Match(intermediateLine);
								if (assertMatch.Success)
								{
									string summary = $"{assertMatch.Groups["type"].Value}: {assertMatch.Groups["condition"].Value.Trim()} [{assertMatch.Groups["file"].Value}]";
									builder.AddProperty("CrashSummary", summary);
									break;
								}
							}
						}

						// Skip directly to the first [Callstack] line using the known offset
						builder.MoveNext(callstackOffset - 1);
						while (builder.Next.IsMatch(s_callstackLine))
						{
							builder.MoveNext();
						}
						while (builder.Next.IsMatch(s_blankChannelLine))
						{
							builder.MoveNext();
						}
						return builder.ToMatch(LogEventPriority.Low, level, KnownLogEvents.Engine_LogChannel);
					}
				}

				// [Callstack] line without preceding header: group consecutive frames + trailing blank lines
				if (s_callstackLine.IsMatch(input.CurrentLine ?? ""))
				{
					builder.AddProperty("Callstack", true);
					while (builder.Next.IsMatch(s_callstackLine))
					{
						builder.MoveNext();
					}
					while (builder.Next.IsMatch(s_blankChannelLine))
					{
						builder.MoveNext();
					}
					return builder.ToMatch(LogEventPriority.Low, level, KnownLogEvents.Engine_LogChannel);
				}

				return builder.ToMatchWithHangingLines(LogEventPriority.Low, level, KnownLogEvents.Engine_LogChannel);

			}
			return null;
		}
	}
}
