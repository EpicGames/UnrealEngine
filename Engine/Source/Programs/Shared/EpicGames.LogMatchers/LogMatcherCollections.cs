// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.LogMatchers;

/// <summary>
/// Helper class that specifies collections of matchers to use for log parsing.
/// </summary>
public static class LogMatcherCollections
{
	/// <summary>
	/// All matchers originally from the AutomationUtils project, for legacy compatibility.
	/// Not recommended for new usage - choose what matchers you need instead.
	/// </summary>
	public static IEnumerable<ILogEventMatcher> AutomationUtilsMatchers => [
		new ContentEventMatcher(),
		new CrashEventMatcher(),
		new DockerEventMatcher(),
		new ExceptionEventMatcher(),
		new ExitCodeEventMatcher(),
		new GenericEventMatcher(),
		new GradleEventMatcher(),
		new LocalizationEventMatcher(),
		new LogChannelEventMatcher(),
		new MsTestEventMatcher(),
		new SanitizerEventMatcher(),
		new ShaderEventMatcher(),
		new SourceFileLineEventMatcher(),
		new SystemicEventMatcher(),
		new UnsecureHttpMatcher(),
		new WorldLeakEventMatcher(),
		new WriteDummyPipeMatcher(),
		new ZenCliEventMatcher(),
	];

	/// <summary>
	/// All matchers originally from the UBT project, for legacy compatibility.
	/// Not recommended for new usage - choose what matchers you need instead.
	/// </summary>
	public static IEnumerable<ILogEventMatcher> UBTMatchers => [
		new CompileEventMatcher(),
		new LinkEventMatcher(),
		new MicrosoftEventMatcher(),
		new XoreaxEventMatcher(),
	];
}
