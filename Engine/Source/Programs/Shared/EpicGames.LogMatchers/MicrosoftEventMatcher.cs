// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for generic Microsoft errors (see https://docs.microsoft.com/en-us/cpp/build/formatting-the-output-of-a-custom-build-step-or-build-event?view=msvc-160)
	/// </summary>
	class MicrosoftEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_warningCodePattern = new(
			@"(?<severity>(?:error|warning)) " +
			@"(?<code>[a-zA-Z]+[0-9]+)\s*:");

		static readonly Regex s_fileOrToolPattern = new(@"^\s*(.*[^\s])\s*:");

		static readonly Regex s_fileLinePattern = new(@"^\s*(?<file>.*)\((?<line>\d+)(?:, (?<column>\d+))?\)\s*:$");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			// filename(line# [, column#]) | toolname} : [ any text ] {error | warning} code+number:localizable string [ any text ]

			Match? match;
			if (cursor.TryMatch(s_warningCodePattern, out match))
			{
				string prefix = cursor.CurrentLine!.Substring(0, match.Index);

				Match fileOrToolMatch = s_fileOrToolPattern.Match(prefix);
				if (fileOrToolMatch.Success)
				{
					LogEventBuilder builder = new(cursor);

					Match fileMatch = s_fileLinePattern.Match(fileOrToolMatch.Value);
					if (fileMatch.Success)
					{
						builder.AnnotateSourceFile(fileMatch.Groups["file"], null);
						builder.Annotate(fileMatch.Groups["line"], LogEventMarkup.LineNumber);
						builder.TryAnnotate(fileMatch.Groups["column"]);
					}
					else
					{
						builder.Annotate("tool", fileOrToolMatch.Groups[1], LogEventMarkup.ToolName);
					}

					Group severity = match.Groups["severity"];
					builder.Annotate(severity);
					builder.Annotate(match.Groups["code"]);
					return builder.ToMatch(LogEventPriority.Normal, severity.Value.Equals("error", StringComparison.OrdinalIgnoreCase) ? LogLevel.Error : LogLevel.Warning, KnownLogEvents.Microsoft);
				}
			}
			return null;
		}
	}
}
