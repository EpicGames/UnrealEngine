// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace EpicGames.LogMatchers
{
	class MsTestEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_failedPattern = new(@"^  \s*Failed [A-Za-z0-9_]+ (?:\(.*\) )?\[\d+ (?:s|ms)\]$");
		static readonly Regex s_detailPattern = new(@"^  \s*(?:Error Message|Stack Trace):$");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_failedPattern))
			{
				int lineCount = 1;
				while (cursor.IsMatch(lineCount, s_detailPattern) || cursor.IsHanging(lineCount, cursor.CurrentLine))
				{
					lineCount++;
				}
				while (lineCount > 0 && cursor.IsBlank(lineCount - 1))
				{
					lineCount--;
				}

				if (lineCount > 1)
				{
					LogEventBuilder builder = new(cursor, lineCount);
					return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.MSTest);
				}
			}
			return null;
		}
	}
}
