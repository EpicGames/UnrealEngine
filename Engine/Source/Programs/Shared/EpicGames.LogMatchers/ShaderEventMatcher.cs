// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches shader compile errors and annotates with the source file path and revision.
	/// Also detects "Failed to compile Material" warnings and extracts the material path
	/// to enable grouping by material in the issue handler.
	/// </summary>
	class ShaderEventMatcher : ILogEventMatcher
	{
		const string Prefix = @"^\s*LogShaderCompilers: (?<severity>Error|Warning): ";

		const string FilePattern =
			@"(?<file>" +
				// optional drive letter
				@"(?:[a-zA-Z]:)?" +
				// any non-colon character
				@"[^:(\s]+" +
				// any filename character (not whitespace or slash)
				@"[^:(\s\\/]" +
			@")";

		const string LinePattern =
			@"(?<line>\d+)(?::(?<column>\d+))?";

		/// <summary>
		/// Pattern to extract material path from "Failed to compile Material {path}" messages.
		/// The material path is a forward-slash separated path like "/Game/Materials/MyMaterial.MyMaterial".
		/// This property enables ShaderIssueHandler to group all compilation failures for the same
		/// material together, rather than creating separate issues for each shader/platform combination.
		/// </summary>
		const string MaterialPattern = @"Failed to compile Material (?<material>/[^\s]+)";

		static readonly Regex s_pattern = new(Prefix);
		static readonly Regex s_patternWithFile = new($"{Prefix}{FilePattern}(?:\\({LinePattern}\\))?:");
		static readonly Regex s_materialPattern = new(MaterialPattern);

		// Matches continuation lines that are part of the same shader error:
		// - Lines starting with whitespace (indented content, error details)
		// - "Validation failed" messages
		// - Windows file paths like "D:\build\..." (shader source file references)
		// - "Internal Error!" messages from shader compiler
		static readonly Regex s_content = new(@"^(?:\s+|Validation failed|[A-Za-z]:[/\\]|Internal Error!)");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			Match? match;
			if (input.TryMatch(s_pattern, out match))
			{
				LogLevel level = GetLogLevelFromSeverity(match);

				LogEventBuilder builder = new(input);
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);

				Match? fileMatch;
				if (input.TryMatch(s_patternWithFile, out fileMatch))
				{
					builder.AnnotateSourceFile(fileMatch.Groups["file"], "");
					builder.TryAnnotate(fileMatch.Groups["line"], LogEventMarkup.LineNumber);
					builder.TryAnnotate(fileMatch.Groups["column"], LogEventMarkup.ColumnNumber);
				}

				// Check if this is a "Failed to compile Material" warning and extract the material path.
				// This allows ShaderIssueHandler to group issues by material rather than by source file.
				string? currentLine = input.CurrentLine;
				if (currentLine != null)
				{
					Match materialMatch = s_materialPattern.Match(currentLine);
					if (materialMatch.Success)
					{
						builder.AddProperty("material", materialMatch.Groups["material"].Value);
					}
				}

				while (builder.Next.IsMatch(s_content))
				{
					builder.MoveNext();
				}

				return builder.ToMatch(LogEventPriority.AboveNormal, level, KnownLogEvents.Engine_ShaderCompiler);
			}
			return null;
		}

		static LogLevel GetLogLevelFromSeverity(Match match)
		{
			string severity = match.Groups["severity"].Value;
			if (severity.Equals("Warning", StringComparison.Ordinal))
			{
				return LogLevel.Warning;
			}
			else
			{
				return LogLevel.Error;
			}
		}
	}
}
