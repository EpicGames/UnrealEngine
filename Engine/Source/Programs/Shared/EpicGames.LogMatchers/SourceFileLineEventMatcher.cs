// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matches compile errors and annotates with the source file path and revision
	/// </summary>
	class SourceFileLineEventMatcher : ILogEventMatcher
	{
		const string SeverityPattern =
			"(?<severity>ERROR|WARNING)";

		const string FilePattern =
			@"(?<file>" +
				// optional drive letter
				@"(?:[a-zA-Z]:)?" +
				// any non-colon character
				@"[^:(\s]+" +
				// any filename character (not whitespace or slash)
				@"[^:(\s\\/]" +
			@")";

		const string LinePattern =
			@"(?<line>\d+)";

		static readonly Regex s_pattern = new($"^\\s*{SeverityPattern}: {FilePattern}(?:\\({LinePattern}\\))?: ");
		static readonly Regex s_copyright = new("copyright");

		/// <inheritdoc/>
		public LogEventMatch? Match(ILogCursor input)
		{
			Match? match;
			if (input.TryMatch(s_pattern, out match))
			{
				LogLevel level = GetLogLevelFromSeverity(match);

				LogEventBuilder builder = new(input);

				builder.AnnotateSourceFile(match.Groups["file"], "");
				builder.Annotate(match.Groups["severity"], LogEventMarkup.Severity);
				builder.TryAnnotate(match.Groups["line"], LogEventMarkup.LineNumber);

				EventId eventId;
				if (input.IsMatch(s_copyright))
				{
					eventId = KnownLogEvents.AutomationTool_MissingCopyright;
				}
				else
				{
					eventId = KnownLogEvents.AutomationTool_SourceFileLine;
				}

				return builder.ToMatch(LogEventPriority.AboveNormal, level, eventId);
			}
			return null;
		}

		static LogLevel GetLogLevelFromSeverity(Match match)
		{
			string severity = match.Groups["severity"].Value;
			if (severity.Equals("WARNING", StringComparison.Ordinal))
			{
				return LogLevel.Warning;
			}
			else
			{
				return LogLevel.Error;
			}
		}
	}
}
