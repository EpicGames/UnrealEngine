// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace EpicGames.LogMatchers
{
	class SystemicEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_ddc = new(@"^\s*LogDerivedDataCache: Warning:");

		static readonly Regex s_pdbUtil = new(@"^\s*ERROR: Error: EC_OK");
		static readonly Regex s_pdbUtilSuffix = new(@"^\s*ERROR:\s*$");

		static readonly Regex s_roboMerge = new(@"RoboMerge\/gates.*already locked on Commit Server by buildmachine");

		static readonly Regex s_hostDown = new(@"ERROR: System\.IO\.IOException: Host is down");

		static readonly Regex s_missingFileList = new(@"^\s*ERROR: Missing local or shared file list.*\.xml");

		static readonly Regex s_ubaCacheResponseTimeout = new(@"UbaCache (.*) - Timed out after .* waiting for message response from server");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_ddc))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_SlowDDC);
			}
			if (cursor.IsMatch(s_pdbUtil))
			{
				LogEventBuilder builder = new(cursor);
				while (builder.Next.IsMatch(s_pdbUtilSuffix))
				{
					builder.MoveNext();
				}
				return builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_PdbUtil);
			}
			if (cursor.Contains("failed to submit"))
			{
				if (cursor.CurrentLineNumber != cursor.MatchForwardsLimited(cursor.CurrentLineNumber, s_roboMerge, 5))
				{
					return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Low, LogLevel.Information, KnownLogEvents.Systemic_RoboMergeGateLocked);
				}
			}
			if (cursor.IsMatch(s_hostDown))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Low, LogLevel.Information, KnownLogEvents.Systemic_HostDownIOException);
			}
			if (cursor.Contains("LogXGEController: Warning: XGEControlWorker.exe does not exist"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_MissingXgeControlWorker);
			}

			if (cursor.IsMatch(s_missingFileList))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Systemic_MissingFileList);
			}

			if (cursor.IsMatch(s_ubaCacheResponseTimeout))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.High, LogLevel.Warning, KnownLogEvents.Systemic_UbaCacheResponseTimeout);
			}

			if (cursor.Contains("SignTool Error: The specified timestamp server either could not be reached or"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_SignToolTimeStampServer);
			}
			else if (cursor.Contains("SignTool Error: An error occurred while attempting to sign"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("SignTool Error: SignedCode::Sign returned error: 0x80090027"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}

			// Common signing service issues, drop to a warning so they can still be tracked but tagg as systemic
			if (cursor.Contains("Error: SignerSign() failed"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("SignTool Error: An unexpected internal error has occurred"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("SignTool Error: No certificates were found that met all the given criteria."))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("SignTool Error: An error occurred while attempting to load the signing"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("Attempting to fix RTService..."))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("Attempted fix to signing service failed, retry attempt"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("RTService service in an unhealthy state, attempting remediation."))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("Unknown failure condition from signtool, assuming service is healthy."))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}
			if (cursor.Contains("RTService Service is not running, starting..."))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Warning, KnownLogEvents.Systemic_SignTool);
			}

			return null;
		}
	}
}
