// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	class UnsecureHttpMatcher : ILogEventMatcher
	{
		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.Contains("WARNING: Insecure HTTP request"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_SignToolTimeStampServer);
			}

			return null;
		}
	}
}
