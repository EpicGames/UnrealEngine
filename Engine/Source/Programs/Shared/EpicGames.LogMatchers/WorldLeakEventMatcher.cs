// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Issues.Handlers;
using Microsoft.Extensions.Logging;
using System;
using System.Text.RegularExpressions;

#pragma warning disable MA0016

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher for World Leak Errors
	/// 
	/// </summary>
	public class WorldLeakEventMatcher : ILogEventMatcher
	{
		/// <summary>
		/// Regex pattern that matches Report Start
		/// </summary>
		public static readonly Regex ReportStartPattern = new(@"^====Fatal World Leaks====$", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex pattern that matches Report End
		/// </summary>
		public static readonly Regex ReportEndPattern = new(@"^====End Fatal World Leaks====$", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex pattern that matches callstacks reported during leak reports
		/// </summary>
		public static readonly Regex ReportCallstackPattern = new(
			@"^[\s]*\^\s" +
			@"(?<Symbol>.+?)\s" +
			@"\[(?<SourceFile>(\/|\w:).+?)" +
			@"(:(?<Line>\d+?))?\]", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex pattern that matches the underlining of the reference preventing the object from being GC'd
		/// </summary>
		public static readonly Regex ReportLingeringReferenceUnderlinePattern = new(@"^\s*\^+\s*$", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex pattern that matches leaking reference chain text
		/// </summary>
		public static readonly Regex ReportLingeringReference = new(@"^\s*\->\s*(?<ObjectText>.+)$", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex pattern that matches reference chain text
		/// </summary>
		public static readonly Regex ReportBodyPattern = new(@"^\s*(?<ObjectFlags>((\([\w\s]+\)\s))+)?\s*(?<ObjectText>(.+)\s/(.)+/(.)+)$", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <inheritdoc/> 
		public LogEventMatch? Match(ILogCursor cursor)
		{
			Match? match;
			bool bFoundLingeringReference = false;
			bool bFoundFirstObject = false;
			if (cursor.TryMatch(ReportStartPattern, out _))
			{
				EventId eventId = KnownLogEvents.WorldLeak;
				LogLevel reportLevel = LogLevel.Error;

				LogEventBuilder builder = new(cursor);

				builder.MoveNext();

				while (builder.Current.CurrentLine != null)
				{
					if (builder.Current.TryMatch(ReportCallstackPattern, out match))
					{
						do
						{
							builder.AnnotateSymbol(match!.Groups["Symbol"]);
							builder.AnnotateSourceFile(match.Groups["SourceFile"], "");
							builder.TryAnnotate(match.Groups["Line"], LogEventMarkup.LineNumber);

							builder.MoveNext();
						} while (builder.Current.TryMatch(ReportCallstackPattern, out match));

						// Minor speed up to avoid checking k conditions below when we know they will all fail
						if (builder.Current.CurrentLine == null)
						{
							return null;
						}
					}

					// Don't use else if as its possible the callstack walking above ended with
					// a non-callstack line that we need to try matching for the cases below
					if (!bFoundLingeringReference && builder.Next != null && builder.Next.TryMatch(ReportLingeringReferenceUnderlinePattern, out _))
					{
						// If we found the underline pattern, the current line should be our lingering reference
						if (builder.Current.TryMatch(ReportLingeringReference, out match))
						{
							if (match.Groups.TryGetValue("ObjectText", out Group? group) && !String.IsNullOrEmpty(group.Value))
							{
								builder.Annotate(group, WorldLeakIssueHandler.LingeringReference);
								bFoundLingeringReference = true;
							}
						}
					}
					else if (builder.Current.IsMatch(ReportEndPattern))
					{
						return builder.ToMatch(LogEventPriority.High, reportLevel, eventId);
					}
					else if (!bFoundFirstObject && builder.Current.TryMatch(ReportBodyPattern, out match))
					{
						// optional
						if (match.Groups.TryGetValue("ObjectFlags", out Group? objectFlagGroup) && !String.IsNullOrEmpty(objectFlagGroup.Value))
						{
							builder.TryAnnotate(objectFlagGroup, WorldLeakIssueHandler.ObjectFlags);
						}
						if (match.Groups.TryGetValue("ObjectText", out Group? objectTextGroup) && !String.IsNullOrEmpty(objectTextGroup.Value))
						{
							builder.TryAnnotate(objectTextGroup, WorldLeakIssueHandler.ObjectText);
							bFoundFirstObject = true;
						}
					}
					builder.MoveNext();
				}
			}

			return null;
		}
	}
}
