// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	class WriteDummyPipeMatcher : ILogEventMatcher
	{
		public LogEventMatch? Match(ILogCursor cursor)
		{
			// libWebsockets can output that warning when lws_cancel_service get called to Shutdown from FLwsWebSocketsManager
			// The api write() could fail for various reasons, it doesn't mean it's a failure we need to handle
			if (cursor.Contains("Cannot write to dummy pipe"))
			{
				return new LogEventBuilder(cursor).ToMatch(LogEventPriority.Normal, LogLevel.Information, KnownLogEvents.Systemic_SignToolTimeStampServer);
			}

			return null;
		}
	}
}
