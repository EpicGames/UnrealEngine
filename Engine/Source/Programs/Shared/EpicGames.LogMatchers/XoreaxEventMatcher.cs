// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.LogMatchers
{
	class XoreaxEventMatcher : ILogEventMatcher
	{
		public const string PropertyName = "name";
		public const string PropertyAgent = "agent";
		public const string PropertyDuration = "duration";
		public const string PropertyStartTime = "startTime";

		static readonly Regex s_buildService = new(@"\(BuildService.exe\) is not running");

		static readonly Regex s_xgConsole = new(@"BUILD FAILED: (.*)xgConsole\.exe(.*)");

		static readonly Regex s_buildSystemError = new(@"^\s*--------------------Build System Error");
		static readonly Regex s_buildSystemErrorEnd = new(@"^(\s*)--------------------");

		static readonly Regex s_buildSystemWarning = new(@"^\s*--------------------Build System Warning[- ]");
		static readonly Regex s_buildSystemWarningNext = new(@"^(\s*)([^ ].*):");

		static readonly Regex s_buildSystemWarningSummary = new(@"^\d+ build system warning\(s\):");
		static readonly Regex s_buildSystemWarningSummaryNext = new(@"^(\s*-\s*)([^ ].*)");

		static readonly Regex s_cacheWarning = new(@"^\s*WARNING: \d+ items \([^\)]*\) removed from the cache due to reaching the cache size limit");
		static readonly Regex s_cacheWarning2 = new(@"^\s*WARNING: Several items removed from the cache due to reaching the cache size limit");
		static readonly Regex s_cacheWarning3 = new(@"^\s*WARNING: The Build Cache is close to full");

		public LogEventMatch? Match(ILogCursor cursor)
		{
			if (cursor.IsMatch(s_cacheWarning) || cursor.IsMatch(s_cacheWarning2) || cursor.IsMatch(s_cacheWarning3))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_Xge_CacheLimit);
			}

			if (cursor.IsMatch(s_buildService))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.Systemic_Xge_ServiceNotRunning);
			}

			if (cursor.IsMatch(s_xgConsole))
			{
				LogEventBuilder builder = new(cursor);
				return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Systemic_Xge_BuildFailed);
			}

			if (cursor.IsMatch(s_buildSystemError))
			{
				LogEventBuilder builder = new(cursor);
				builder.MoveNext();

				for (int idx = 0; idx < 100; idx++)
				{
					if (builder.Current.IsMatch(idx, s_buildSystemErrorEnd))
					{
						builder.MoveNext(idx);
						break;
					}
				}

				return builder.ToMatch(LogEventPriority.High, LogLevel.Error, KnownLogEvents.Systemic_Xge);
			}

			if (cursor.IsMatch(s_buildSystemWarning))
			{
				LogEventBuilder builder = new(cursor);
				if (builder.Next.TryMatch(s_buildSystemWarningNext, out Match? prefix))
				{
					builder.MoveNext();

					string message = prefix.Groups[2].Value;

					EventId eventId = KnownLogEvents.Systemic_Xge;
					if (Regex.IsMatch(message, "Failed to connect to Coordinator") || Regex.IsMatch(message, "Failed to connect to Primary Coordinator"))
					{
						eventId = KnownLogEvents.Systemic_Xge_Standalone;
					}

					Regex prefixPattern = new($"^{Regex.Escape(prefix.Groups[1].Value)}\\s");
					while (builder.Next.IsMatch(prefixPattern))
					{
						builder.MoveNext();
					}

					return builder.ToMatch(LogEventPriority.High, LogLevel.Information, eventId);
				}
			}

			if (cursor.IsMatch(s_buildSystemWarningSummary))
			{
				LogEventBuilder builder = new(cursor);
				if (builder.Next.TryMatch(s_buildSystemWarningSummaryNext, out Match? prefix))
				{
					builder.MoveNext();

					string message = prefix.Groups[2].Value;

					EventId eventId = KnownLogEvents.Systemic_Xge;
					if (Regex.IsMatch(message, "Failed to connect to Primary Coordinator"))
					{
						eventId = KnownLogEvents.Systemic_Xge_Standalone;
					}

					Regex prefixPattern = new($"^{Regex.Escape(prefix.Groups[1].Value)}\\s");
					while (builder.Next.IsMatch(prefixPattern))
					{
						builder.MoveNext();
					}

					return builder.ToMatch(LogEventPriority.High, LogLevel.Information, eventId);
				}
			}

			return null;
		}
	}
}
