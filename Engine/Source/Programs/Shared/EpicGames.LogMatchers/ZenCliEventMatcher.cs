// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Text.RegularExpressions;

namespace EpicGames.LogMatchers
{
	/// <summary>
	/// Matcher log output from zen cli in plain progress
	/// </summary>
	class ZenCliEventMatcher : ILogEventMatcher
	{
		static readonly Regex s_httpOperationErrorPattern = new(
			@"Error: Operation failed due to a http error: (?<message>.*)\((?<http_error>\d+)\)");

		static readonly Regex s_httpRequestErrorPattern = new(
			@"HttpClient request failed.*?""error_code"": (?<http_error>\d+)");

		static readonly Regex s_httpRequestJsonErrorPattern = new(
			@"HttpClient request failed.*? Status: (?<http_error>\d+), Error: \'");

		public LogEventMatch? Match(ILogCursor input)
		{
			// Error: Operation failed due to a http error: Failed putting build part: JupiterSession::PutBuildBlob: HTTP error 504 Gateway Timeout ({"error_code": 504}) (504)
			// HttpClient request failed (session: guid): Url: url, Status: 504, Error: '' (0), Bytes: 0/19 (Up/Down), Elapsed: 1m00s, Response: '{\"error_code\": 504}', Reason: 'Gateway Time-out'
			Match? httpMatch;
			if (input.TryMatch(s_httpOperationErrorPattern, out httpMatch) || input.TryMatch(s_httpRequestErrorPattern, out httpMatch) || input.TryMatch(s_httpRequestJsonErrorPattern, out httpMatch))
			{
				LogEventBuilder builder = new(input);
				string httpError = httpMatch.Groups["http_error"].Value;

				// http error codes that we will retry
				if (httpError is "502" or "504")
				{
					return builder.ToMatch(LogEventPriority.High, LogLevel.Information, KnownLogEvents.ZenCli_Event);
				}
			}

			return null;
		}
	}
}

