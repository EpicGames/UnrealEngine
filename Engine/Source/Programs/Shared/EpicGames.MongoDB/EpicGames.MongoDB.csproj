<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <PackageReference Include="MongoDB.Driver" Version="2.19.2" />
    <PackageReference Include="SharpCompress" Version="0.48.1" />
    <PackageReference Include="Snappier" Version="1.3.1" />
  </ItemGroup>
</Project>
