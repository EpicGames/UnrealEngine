// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Driver;
using System;
using System.Collections.Generic;
using System.Linq.Expressions;
using System.Threading.Tasks;

namespace EpicGames.MongoDB
{
	/// <summary>
	/// Extension methods for MongoDB
	/// </summary>
	public static class MongoExtensions
	{
		/// <summary>
		/// Filters the documents returned from a search
		/// </summary>
		/// <param name="query">The query to filter</param>
		/// <param name="index">Index of the first document to return</param>
		/// <param name="count">Number of documents to return</param>
		/// <returns>New query</returns>
		public static IFindFluent<TDocument, TProjection> Range<TDocument, TProjection>(this IFindFluent<TDocument, TProjection> query, int? index, int? count)
		{
			if (index != null)
			{
				query = query.Skip(index.Value);
			}
			if (count != null)
			{
				query = query.Limit(count.Value);
			}
			return query;
		}

		/// <summary>
		/// Filters the documents returned from a search
		/// </summary>
		/// <param name="query">The query to filter</param>
		/// <returns>New query</returns>
		public static async Task<List<TResult>> ToListAsync<TDocument, TResult>(this IAsyncCursorSource<TDocument> query) where TDocument : TResult
		{
			List<TResult> results = [];
			using (IAsyncCursor<TDocument> cursor = await query.ToCursorAsync())
			{
				while (await cursor.MoveNextAsync())
				{
					foreach (TDocument document in cursor.Current)
					{
						results.Add(document);
					}
				}
			}
			return results;
		}

		/// <summary>
		/// Attempts to insert a document into a collection, handling the error case that a document with the given key already exists
		/// </summary>
		/// <typeparam name="TDocument"></typeparam>
		/// <param name="collection">Collection to insert into</param>
		/// <param name="newDocument">The document to insert</param>
		/// <returns>True if the document was inserted, false if it already exists</returns>
		public static async Task<bool> InsertOneIgnoreDuplicatesAsync<TDocument>(this IMongoCollection<TDocument> collection, TDocument newDocument)
		{
			try
			{
				await collection.InsertOneAsync(newDocument);
				return true;
			}
			catch (MongoWriteException ex)
			{
				if (ex.WriteError.Category == ServerErrorCategory.DuplicateKey)
				{
					return false;
				}
				else
				{
					throw;
				}
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <typeparam name="TField">Type of the field to set</typeparam>
		/// <param name="updateBuilder">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetNull<TDocument, TField>(this UpdateDefinitionBuilder<TDocument> updateBuilder, Expression<Func<TDocument, TField?>> field, TField? value) where TField : struct
		{
			if (value.HasValue)
			{
				return updateBuilder.Set(field, value.Value);
			}
			else
			{
				return updateBuilder.Unset(new ExpressionFieldDefinition<TDocument, TField?>(field));
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <typeparam name="TField">Type of the field to set</typeparam>
		/// <param name="updateBuilder">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetNullRef<TDocument, TField>(this UpdateDefinitionBuilder<TDocument> updateBuilder, Expression<Func<TDocument, TField?>> field, TField? value) where TField : class
		{
			if (value != null)
			{
				return updateBuilder.Set(field, value);
			}
			else
			{
				return updateBuilder.Unset(new ExpressionFieldDefinition<TDocument, TField?>(field));
			}
		}

		/// <summary>
		/// Creates a filter definition from a linq expression. This is not generally explicitly castable, so expose it as a FilterDefinitionBuilder method.
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <param name="filter">The filter builder</param>
		/// <param name="expression">Expression to parse</param>
		/// <returns>New filter definition</returns>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public static FilterDefinition<TDocument> Expr<TDocument>(this FilterDefinitionBuilder<TDocument> filter, Expression<Func<TDocument, bool>> expression)
		{
			return expression;
		}
	}
}
