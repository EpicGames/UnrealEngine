// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Bson;

namespace EpicGames.MongoDB
{
	static class ObjectIdHelpers
	{
		public static ObjectId ToObjectId(this string text)
		{
			if (text.Length == 0)
			{
				return ObjectId.Empty;
			}
			else
			{
				return ObjectId.Parse(text);
			}
		}
	}
}
