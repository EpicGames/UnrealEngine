// Copyright Epic Games, Inc. All Rights Reserved.

using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using System;
using System.Reflection;
using System.Threading.Tasks;

namespace EpicGames.MongoDB
{
	/// <summary>
	/// Attribute specifying the unique id for a singleton document
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class SingletonDocumentAttribute : Attribute
	{
		/// <summary>
		/// Unique id for the singleton document
		/// </summary>
		public string Id { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id for the singleton document</param>
		public SingletonDocumentAttribute(string id)
		{
			Id = id;
		}
	}

	/// <summary>
	/// Base class for singletons
	/// </summary>
	public class SingletonBase
	{
		/// <summary>
		/// Unique id of the 
		/// </summary>
		public ObjectId Id { get; set; }

		/// <summary>
		/// Current revision number
		/// </summary>
		public int Revision { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		[BsonConstructor]
		protected SingletonBase()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id for the singleton</param>
		public SingletonBase(ObjectId id)
		{
			Id = id;
		}

		/// <summary>
		/// Base class for singleton documents
		/// </summary>
		class CachedId<T>
		{
			public static ObjectId Value { get; } = GetSingletonId();

			static ObjectId GetSingletonId()
			{
				SingletonDocumentAttribute? attribute = typeof(T).GetCustomAttribute<SingletonDocumentAttribute>() ?? throw new Exception($"Type {typeof(T).Name} is missing a {nameof(SingletonDocumentAttribute)} annotation");
				return ObjectId.Parse(attribute.Id);
			}
		}

		/// <summary>
		/// Gets the id for a singleton type
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <returns></returns>
		public static ObjectId GetId<T>() where T : SingletonBase
		{
			return CachedId<T>.Value;
		}
	}

	/// <summary>
	/// Interface for the getting and setting the singleton
	/// </summary>
	/// <typeparam name="T">Type of document</typeparam>
	public interface ISingletonDocument<T>
	{
		/// <summary>
		/// Gets the current document
		/// </summary>
		/// <returns>The current document</returns>
		Task<T> GetAsync();

		/// <summary>
		/// Attempts to update the document
		/// </summary>
		/// <param name="value">New state of the document</param>
		/// <returns>True if the document was updated, false otherwise</returns>
		Task<bool> TryUpdateAsync(T value);
	}

	/// <summary>
	/// Concrete implementation of <see cref="ISingletonDocument{T}"/>
	/// </summary>
	/// <typeparam name="T">The document type</typeparam>
	public class SingletonDocument<T> : ISingletonDocument<T> where T : SingletonBase, new()
	{
		/// <summary>
		/// The database service instance
		/// </summary>
		readonly IMongoCollection<T> _collection;

		/// <summary>
		/// Unique id for the singleton document
		/// </summary>
		readonly ObjectId _objectId;

		/// <summary>
		/// Static constructor. Registers the document using the automapper.
		/// </summary>
		static SingletonDocument()
		{
			BsonClassMap.RegisterClassMap<T>();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="collection">The database service instance</param>
		public SingletonDocument(IMongoCollection<SingletonBase> collection)
		{
			_collection = collection.OfType<T>();

			SingletonDocumentAttribute? attribute = typeof(T).GetCustomAttribute<SingletonDocumentAttribute>() ?? throw new Exception($"Type {typeof(T).Name} is missing a {nameof(SingletonDocumentAttribute)} annotation");
			_objectId = new ObjectId(attribute.Id);
		}

		/// <inheritdoc/>
		public async Task<T> GetAsync()
		{
			for (; ; )
			{
				T? @object = await _collection.Find<T>(x => x.Id == _objectId).FirstOrDefaultAsync();
				if (@object != null)
				{
					return @object;
				}

				T newItem = new();
				newItem.Id = _objectId;
				await _collection.InsertOneAsync(newItem);
			}
		}

		/// <inheritdoc/>
		public async Task<bool> TryUpdateAsync(T value)
		{
			int prevRevision = value.Revision++;
			try
			{
				ReplaceOneResult result = await _collection.ReplaceOneAsync(x => x.Id == _objectId && x.Revision == prevRevision, value, new ReplaceOptions { IsUpsert = true });
				return result.MatchedCount > 0;
			}
			catch (MongoWriteException ex)
			{
				// Duplicate key error occurs if filter fails to match because revision is not the same.
				if (ex.WriteError != null && ex.WriteError.Category == ServerErrorCategory.DuplicateKey)
				{
					return false;
				}
				else
				{
					throw;
				}
			}
		}
	}
}
