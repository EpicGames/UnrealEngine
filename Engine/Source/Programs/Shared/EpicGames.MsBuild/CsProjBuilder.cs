// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Build.Evaluation;
using Microsoft.Extensions.Logging;

namespace EpicGames.MsBuild
{
	/// <summary>
	/// Builds .csproj files
	/// </summary>
	public static class CsProjBuilder
	{
		static FileReference ConstructBuildRecordPath(CsProjBuildHook hook, FileReference projectPath, IEnumerable<DirectoryReference> baseDirectories)
		{
			DirectoryReference? basePath = null;

			foreach (DirectoryReference scriptFolder in baseDirectories)
			{
				if (projectPath.IsUnderDirectory(scriptFolder))
				{
					basePath = scriptFolder;
					break;
				}
			}

			if (basePath == null)
			{
				throw new Exception($"Unable to map csproj {projectPath} to Engine, game, or an additional script folder. Candidates were:{Environment.NewLine} {String.Join(Environment.NewLine, baseDirectories)}");
			}

			DirectoryReference buildRecordDirectory = hook.GetBuildRecordDirectory(basePath);
			DirectoryReference.CreateDirectory(buildRecordDirectory);

			return FileReference.Combine(buildRecordDirectory, projectPath.GetFileName()).ChangeExtension(".json");
		}

		/// <summary>
		/// Builds multiple projects
		/// </summary>
		/// <param name="foundProjects">Collection of project to be built</param>
		/// <param name="bForceCompile">If true, force the compilation of the projects</param>
		/// <param name="bBuildSuccess">Set to true/false depending on if all projects compiled or are up-to-date</param>
		/// <param name="hook">Interface to fetch data about the building environment</param>
		/// <param name="baseDirectories">Base directories of the engine and project</param>
		/// <param name="customGlobalProperties">Additional MSBuild global properties to set while building projects. Each value is joined into a semicolon-delimited property value.</param>
		/// <param name="logger">Destination logger</param>
		public static Dictionary<FileReference, CsProjBuildRecordEntry> Build(HashSet<FileReference> foundProjects,
			bool bForceCompile, out bool bBuildSuccess, CsProjBuildHook hook, IEnumerable<DirectoryReference> baseDirectories,
			IReadOnlyDictionary<string, IEnumerable<string>> customGlobalProperties, ILogger logger)
		{
			// Register the MS build path prior to invoking the internal routine.  By not having the internal routine
			// inline, we avoid having the issue of the Microsoft.Build libraries being resolved prior to the build path
			// being set.
			MsBuildSupport.RegisterMsBuildPath(hook);
			return BuildInternal(foundProjects, bForceCompile, out bBuildSuccess, hook, baseDirectories, customGlobalProperties, logger);
		}

		/// <summary>
		/// Builds multiple projects.  This is the internal implementation invoked after the MS build path is set
		/// </summary>
		/// <param name="foundProjects">Collection of project to be built</param>
		/// <param name="bForceCompile">If true, force the compilation of the projects</param>
		/// <param name="bBuildSuccess">Set to true/false depending on if all projects compiled or are up-to-date</param>
		/// <param name="hook">Interface to fetch data about the building environment</param>
		/// <param name="baseDirectories">Base directories of the engine and project</param>
		/// <param name="customGlobalProperties">Additional MSBuild global properties to set while building projects. Each value is joined into a semicolon-delimited property value.</param>
		/// <param name="logger">Destination logger</param>
		private static Dictionary<FileReference, CsProjBuildRecordEntry> BuildInternal(HashSet<FileReference> foundProjects,
			bool bForceCompile, out bool bBuildSuccess, CsProjBuildHook hook, IEnumerable<DirectoryReference> baseDirectories,
			IReadOnlyDictionary<string, IEnumerable<string>> customGlobalProperties, ILogger logger)
		{
			Stopwatch stopwatch = Stopwatch.StartNew();
			FileReference csharpTargetsFile = FileReference.Combine(hook.EngineDirectory, "Source", "Programs", "Shared", "UnrealEngine.CSharp.targets");
			Dictionary<string, string> globalProperties = new()
			{
				{ "EngineDir", hook.EngineDirectory.FullName },
				{ "EnginePath", hook.EngineDirectory.FullName },
				{ "EpicGamesMsBuild", "true" },
				{ "CustomAfterMicrosoftCommonProps", csharpTargetsFile.FullName },
				{ "UnrealEngineCSharpTargetsImported", "true" },
				{ "NoWarn", "$(NoWarn);MSB3026;NETSDK1206".Replace(";", "%3B", StringComparison.Ordinal) },
#if DEBUG
				{ "Configuration", "Debug" },
#else
				{ "Configuration", "Development" },
#endif
			};

			foreach (KeyValuePair<string, IEnumerable<string>> customProperty in customGlobalProperties)
			{
				globalProperties.Add(customProperty.Key, String.Join("%3B", customProperty.Value));
			}

			ConcurrentDictionary<FileReference, CsProjBuildRecordEntry> buildRecords = new(hook.ValidBuildRecords);
			IEnumerable<FileReference> outOfDateProjects = foundProjects.Except(buildRecords.Keys).Distinct();

			using ProjectCollection projectCollection = new(globalProperties);
			ConcurrentDictionary<FileReference, Project> projects = new();
			ConcurrentBag<FileReference> platformSpecificProjects = [];
			
			// Projects whose TargetFramework we don't build directly (e.g. netstandard2.0
			// Roslyn analyzers, or platform-restricted frameworks on the wrong host). We
			// still load and generate build records for these so that consumers' build
			// records can validate their ProjectReferences against a real entry, but we
			// can skip passing them to MSBuild as they will be built when required as
			// dependencies automatically.
			// @todo Should we really be skipping netstandard projects in all cases?
			ConcurrentBag<FileReference> skippedProjects = [];
			logger.LogDebug("Setup {Count} projects time: {TimeSeconds:0.00} s", foundProjects.Count, stopwatch.Elapsed.TotalSeconds);

			stopwatch.Restart();
			// Load all found projects
			IEnumerable<FileReference> toProcess = outOfDateProjects;
			while (toProcess.Any())
			{
				ConcurrentBag<FileReference> referencedProjects = [];
				Parallel.ForEach(toProcess, (foundProject) =>
				{
					string projectPath = Path.GetFullPath(foundProject.FullName);

					Project project;
					// Microsoft.Build.Evaluation.Project doesn't give a lot of useful information if this fails,
					// so make sure to print our own diagnostic info if something goes wrong
					try
					{
						project = new Project(projectPath, globalProperties, toolsVersion: null, projectCollection: projectCollection);

						HashSet<string> targetFrameworks = [.. (project.GetProperty("TargetFrameworks")?.EvaluatedValue ?? String.Empty)
							.Split(';', StringSplitOptions.TrimEntries | StringSplitOptions.RemoveEmptyEntries)];
						if (targetFrameworks.Count > 0)
						{
							Dictionary<string, string> innerProps = project.GlobalProperties.ToDictionary(kvp => kvp.Key, kvp => kvp.Value);
#if NET10_0_OR_GREATER
							if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net10.0"))
							{
								innerProps["TargetFramework"] = "net10.0";
							}
#endif
#if NET9_0_OR_GREATER
							if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net9.0"))
							{
								innerProps["TargetFramework"] = "net9.0";
							}
#endif
#if NET8_0_OR_GREATER
							if (!innerProps.ContainsKey("TargetFramework") && targetFrameworks.Contains("net8.0"))
							{
								innerProps["TargetFramework"] = "net8.0";
							}
#endif
							if (!innerProps.ContainsKey("TargetFramework"))
							{
								innerProps["TargetFramework"] = targetFrameworks.Order().Last();
							}
							project = new Project(project.FullPath, innerProps, project.ToolsVersion, project.ProjectCollection);
						}
					}
					catch (Microsoft.Build.Exceptions.InvalidProjectFileException iPFEx)
					{
						logger.LogWarning(iPFEx, "Could not load project file {ProjectPath}: {Message}", projectPath, iPFEx.BaseMessage);
						return;
					}

					// check the TargetFramework for unsupported versions. see https://learn.microsoft.com/en-us/dotnet/standard/frameworks
					string targetFramework = project.GetProperty("TargetFramework")?.EvaluatedValue ?? String.Empty;
					bool skipFramework = String.IsNullOrEmpty(targetFramework)
						| targetFramework.Contains("netstandard", StringComparison.Ordinal)
						| targetFramework.Contains("netcoreapp", StringComparison.Ordinal)
						| targetFramework.Contains("uap", StringComparison.Ordinal)
						| targetFramework.Contains("netnano", StringComparison.Ordinal)
						| !targetFramework.Contains('.', StringComparison.Ordinal) // only .NET Core versions are supported and will always contain a period
#if NET8_0_OR_GREATER
						| targetFramework.Contains("net5.0", StringComparison.Ordinal)
						| targetFramework.Contains("net6.0", StringComparison.Ordinal)
						| targetFramework.Contains("net7.0", StringComparison.Ordinal)
#endif
#if NET9_0_OR_GREATER
						| targetFramework.Contains("net8.0", StringComparison.Ordinal)
#endif
#if NET10_0_OR_GREATER
						| targetFramework.Contains("net9.0", StringComparison.Ordinal)
#endif
					;

					// check the TargetFramework of the project: we can't build Windows-only projects on non-Windows platforms etc.
					if (targetFramework.Contains('-', StringComparison.Ordinal))
					{
						skipFramework |= (OperatingSystem.IsWindows() && !targetFramework.Contains("windows", StringComparison.Ordinal))
							|| (OperatingSystem.IsMacOS() && !targetFramework.Contains("macos", StringComparison.Ordinal))
							|| (OperatingSystem.IsLinux());
						platformSpecificProjects.Add(foundProject);
					}

					if (skipFramework)
					{
						logger.LogInformation("Skipping {Framework} project {ProjectPath}", targetFramework, projectPath);

						projects.TryAdd(foundProject, project);
						skippedProjects.Add(foundProject);

						return;
					}

					projects.TryAdd(foundProject, project);
					foreach (ProjectItem reference in project.GetItems("ProjectReference"))
					{
						referencedProjects.Add(new(Path.GetFullPath(reference.EvaluatedInclude, foundProject.Directory.FullName)));
					}
				});

				toProcess = referencedProjects
					.Except(projects.Keys)
					.Except(buildRecords.Keys)
					.Distinct();
			}
			logger.LogDebug("Load {Count} projects time: {TimeSeconds:0.00} s", projects.Count, stopwatch.Elapsed.TotalSeconds);

			// generate a BuildRecord for each loaded project - the gathered information will be used to determine if the project is
			// out of date, and if building this project can be skipped. It is also used to populate Intermediate/ScriptModules after the
			// build completes
			stopwatch.Restart();
			Parallel.ForEach(projects.Values, (project) =>
			{
				if (String.IsNullOrEmpty(project.FullPath))
				{
					return;
				}
				string targetPath = Path.GetRelativePath(project.DirectoryPath, project.ExpandString(project.GetPropertyValue("TargetPath")));

				FileReference projectPath = new(project.FullPath);
				FileReference buildRecordPath = ConstructBuildRecordPath(hook, projectPath, baseDirectories);

				CsProjBuildRecord buildRecord = new()
				{
					Version = CsProjBuildRecord.CurrentVersion,
					TargetPath = targetPath,
					TargetBuildTime = hook.GetLastWriteTime(project.DirectoryPath, targetPath),
					ProjectPath = Path.GetRelativePath(buildRecordPath.Directory.FullName, project.FullPath)
				};

				// the .csproj
				buildRecord.Dependencies.Add(Path.GetRelativePath(project.DirectoryPath, project.FullPath));

				// Imports: files included in the xml (typically props, targets, etc)
				foreach (ResolvedImport import in project.Imports)
				{
					string importPath = Path.GetRelativePath(project.DirectoryPath, import.ImportedProject.FullPath);

					// nuget.g.props and nuget.g.targets are generated by Restore, and are frequently re-written;
					// it should be safe to ignore these files - changes to references from a .csproj file will
					// show up as that file being out of date.
					if (importPath.Contains("nuget.g.", StringComparison.Ordinal) || importPath.Contains(".nuget", StringComparison.Ordinal))
					{
						continue;
					}
					buildRecord.Dependencies.Add(importPath);
				}

				// Project references by dll
				buildRecord.Dependencies.UnionWith(project.GetItems("Reference")
					.Where(x => x.HasMetadata("HintPath"))
					.Select(x => x.GetMetadataValue("HintPath"))
				);

				// Project references
				buildRecord.ProjectReferencesAndTimes.AddRange(project.GetItems("ProjectReference")
					.Select(x => new CsProjBuildRecordRef { ProjectPath = Path.IsPathRooted(x.EvaluatedInclude) ? Path.GetRelativePath(project.DirectoryPath, x.EvaluatedInclude) : x.EvaluatedInclude })
				);

				// Dependency files
				IEnumerable<ProjectItem> dependencyItems = [
					.. project.GetItems("Compile"),
					.. project.GetItems("Content"),
					.. project.GetItems("EmbeddedResource"),
				];
				buildRecord.Dependencies.UnionWith(dependencyItems.Select(x => Path.IsPathRooted(x.EvaluatedInclude) ? Path.GetRelativePath(project.DirectoryPath, x.EvaluatedInclude) : x.EvaluatedInclude));

				// Track any file globs, if they exist
				IEnumerable<GlobResult> globs = [
					.. project.GetAllGlobs("Compile"),
					.. project.GetAllGlobs("Content"),
					.. project.GetAllGlobs("EmbeddedResource")
				];
				buildRecord.Globs.AddRange(globs.Select(glob => new CsProjBuildRecord.Glob()
				{
					ItemType = glob.ItemElement.ItemType,
					Include = [.. glob.IncludeGlobs.Select(CleanGlobString).Order()],
					Exclude = [.. glob.Excludes.Select(CleanGlobString).Order()],
					Remove = [.. glob.Removes.Select(CleanGlobString).Order()]
				}));

				CsProjBuildRecordEntry entry = new(projectPath, buildRecordPath, buildRecord);
				buildRecords.AddOrUpdate(entry.ProjectFile, entry, (key, oldValue) => entry);
			});

			logger.LogDebug("Generate {Count} projects build records time: {TimeSeconds:0.00} s", projects.Count, stopwatch.Elapsed.TotalSeconds);

			// Ensure no projects are set to the same OutputPath, as this will cause build issues with multiprocess compiling
			{
				Dictionary<DirectoryReference, HashSet<FileReference>> outputPaths = [];
				foreach (CsProjBuildRecordEntry entry in buildRecords.Values)
				{
					DirectoryReference outputDirectory = FileReference.Combine(entry.ProjectFile.Directory, entry.BuildRecord.TargetPath!).Directory;
					FileReference projectPath = FileReference.Combine(entry.ProjectFile.Directory, entry.BuildRecord.ProjectPath!);
					if (!outputPaths.TryAdd(outputDirectory, [projectPath]))
					{
						outputPaths[outputDirectory].Add(projectPath);
					}
				}

				foreach (KeyValuePair<DirectoryReference, HashSet<FileReference>> item in outputPaths.Where(x => x.Value.Count > 1).Order())
				{
					logger.LogWarning("Multiple projects share the same output directory '{OutputPath}'. Please update <OutputPath> in the following projects to avoid build issues:{NewLine}{Projects}", item.Key, Environment.NewLine, String.Join(Environment.NewLine, item.Value.Order()));
				}
			}

			if (bForceCompile)
			{
				logger.LogDebug("Script modules will build: '-Compile' on command line");
				outOfDateProjects = projects.Keys;
			}
			else
			{
				stopwatch.Restart();
				foreach (FileReference project in projects.Keys)
				{
					hook.ValidateRecursively(buildRecords.ToDictionary(), project);
				}
				logger.LogDebug("Validate {Count} projects time: {TimeSeconds:0.00} s", projects.Count, stopwatch.Elapsed.TotalSeconds);

				// Select the projects that have been found to be out of date
				outOfDateProjects = buildRecords.Where(x => x.Value.Status == CsProjBuildRecordStatus.Invalid).Select(x => x.Key);
			}

			// Remove skipped projects from the out of date list. If a project was skipped and
			// we try to rebuild it the result is that skipped project failing to build, and
			// projects that depend on the skipped project will be recompiled when they didn't
			// need to be.
			outOfDateProjects = outOfDateProjects.Except(skippedProjects);

			if (outOfDateProjects.Any())
			{
				logger.LogInformation("Building {Count} projects...", outOfDateProjects.Count());

				IEnumerable<FileReference> singleBuildProjects = outOfDateProjects.Intersect(platformSpecificProjects);
				bBuildSuccess = RunDotnetBuildAsync(outOfDateProjects.Except(singleBuildProjects), globalProperties, hook, logger, default).Result;

				// Projects that have a platform specific TargetFramework can't be built using EpicGames.ScriptBuild due to the mismatch
				foreach (FileReference project in singleBuildProjects)
				{
					bBuildSuccess = RunDotnetBuildAsync([project], globalProperties, hook, logger, default).Result && bBuildSuccess;
				}
			}
			else
			{
				bBuildSuccess = true;
			}

			Parallel.ForEach(buildRecords.Values, entry =>
			{
				// Update the target times
				FileReference fullPath = FileReference.Combine(entry.ProjectFile.Directory, entry.BuildRecord.TargetPath!);
				entry.BuildRecord.TargetBuildTime = FileReference.GetLastWriteTime(fullPath);
			});

			// Update the project reference target times
			Parallel.ForEach(buildRecords.Values, entry =>
			{
				foreach (CsProjBuildRecordRef referencedProject in entry.BuildRecord.ProjectReferencesAndTimes)
				{
					FileReference refProjectPath = new(Path.GetFullPath(referencedProject.ProjectPath!, entry.ProjectFile.Directory.FullName));
					if (buildRecords.TryGetValue(refProjectPath, out CsProjBuildRecordEntry? refEntry))
					{
						referencedProject.TargetBuildTime = refEntry.BuildRecord.TargetBuildTime;
					}
				}
			});

			// write all build records
			JsonSerializerOptions jsonOptions = new() { WriteIndented = true };
			Parallel.ForEach(buildRecords.Values, entry =>
			{
				// write all build records
				if (FileReference.WriteAllTextIfDifferent(entry.BuildRecordFile, JsonSerializer.Serialize(entry.BuildRecord, jsonOptions)))
				{
					logger.LogTrace("Wrote script module build record to {BuildRecordPath}", entry.BuildRecordFile);
				}
			});

			// todo: re-verify build records after a build to verify that everything is actually up to date

			// even if only a subset was built, this function returns the full list of target assembly paths
			return buildRecords.Where(x => foundProjects.Contains(x.Key)).ToDictionary();
		}

		// FileMatcher.IsMatch() requires directory separators in glob strings to match the
		// local flavor. There's probably a better way.
		private static string CleanGlobString(string globString)
		{
			char sep = Path.DirectorySeparatorChar;
			char notSep = sep == '/' ? '\\' : '/'; // AltDirectorySeparatorChar isn't always what we need (it's '/' on Mac)

			char[] chars = globString.ToCharArray();
			int p = 0;
			for (int i = 0; i < globString.Length; ++i, ++p)
			{
				// Flip a non-native separator
				if (chars[i] == notSep)
				{
					chars[p] = sep;
				}
				else
				{
					chars[p] = chars[i];
				}

				// Collapse adjacent separators
				if (i > 0 && chars[p] == sep && chars[p - 1] == sep)
				{
					p -= 1;
				}
			}

			return new string(chars, 0, p);
		}

		private static async Task<FileReference> WriteCsPropertiesAsync(IEnumerable<FileReference> projects, CsProjBuildHook hook, CancellationToken cancellationToken)
		{
			FileReference projectPath = FileReference.Combine(hook.EngineDirectory, "Intermediate", "Build", "EpicGames.ScriptBuild.props");

			XNamespace ns = XNamespace.Get("http://schemas.microsoft.com/developer/msbuild/2003");
			XDocument document = new(
				new XElement(ns + "Project",
					new XAttribute("ToolsVersion", "Current"),
					new XElement(ns + "ItemGroup",
						projects.Order().Select(project =>
							new XElement(ns + "ProjectReference",
								new XAttribute("Include", project),
								new XAttribute("PrivateAssets", "All"),
								new XElement(ns + "Private", "false")
							)
						)
					)
				)
			);

			StringBuilder output = new();
			output.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");

			XmlWriterSettings xmlSettings = new();
			xmlSettings.Async = true;
			xmlSettings.Encoding = new UTF8Encoding(false);
			xmlSettings.Indent = true;
			xmlSettings.OmitXmlDeclaration = true;

			using (XmlWriter writer = XmlWriter.Create(output, xmlSettings))
			{
				await document.SaveAsync(writer, cancellationToken);
			}
			await FileReference.WriteAllTextIfDifferentAsync(projectPath, output.ToString(), cancellationToken);

			return projectPath;
		}

		private static async Task<bool> RunDotnetAsync(IEnumerable<string> arguments, CsProjBuildHook hook, ILogger logger, CancellationToken cancellationToken)
		{
			ProcessStartInfo startInfo = new()
			{
				FileName = hook.DotnetPath.FullName,
				Arguments = String.Join(' ', arguments),
				WorkingDirectory = DirectoryReference.Combine(hook.EngineDirectory, "Source").FullName,
				UseShellExecute = false,
				CreateNoWindow = true,
				RedirectStandardOutput = true,
				RedirectStandardError = true,
			};
			startInfo.EnvironmentVariables["DOTNET_MULTILEVEL_LOOKUP"] = "0"; // use only the bundled dotnet installation - ignore any other/system dotnet install

			using LogEventParser parser = new(logger);
			parser.AddMatchers(Log.EventParser.EnabledMatchers);

			logger.LogDebug("Running: {Application} {Arguments}", startInfo.FileName, startInfo.Arguments);
			using Process dotnetProcess = new();
			dotnetProcess.StartInfo = startInfo;
			dotnetProcess.OutputDataReceived += (sender, args) =>
			{
				string output = args.Data?.TrimEnd() ?? String.Empty;
				if (!String.IsNullOrEmpty(output))
				{
					parser.WriteLine(output);
				}
			};
			dotnetProcess.ErrorDataReceived += (sender, args) =>
			{
				string output = args.Data?.TrimEnd() ?? String.Empty;
				if (!String.IsNullOrEmpty(output))
				{
					parser.WriteLine(output);
				}
			};
			dotnetProcess.Start();
			dotnetProcess.BeginOutputReadLine();
			dotnetProcess.BeginErrorReadLine();
			await dotnetProcess.WaitForExitAsync(cancellationToken);
			return dotnetProcess.ExitCode == 0;
		}

		private static async Task<bool> RunDotnetBuildAsync(IEnumerable<FileReference> projects, Dictionary<string, string> globalProperties, CsProjBuildHook hook, ILogger logger, CancellationToken cancellationToken)
		{
			if (!projects.Any())
			{
				return true;
			}

			// Acquire a mutex to prevent running builds from different processes concurrently
			Task<IDisposable> mutexTask = SingleInstanceMutex.AcquireAsync($"Global\\CsProj_{Sha1Hash.Compute(Encoding.Default.GetBytes(hook.EngineDirectory.FullName))}", cancellationToken);

			Task delayTask = Task.Delay(TimeSpan.FromSeconds(1.0), cancellationToken);
			if (Task.WhenAny(mutexTask, delayTask) == delayTask)
			{
				logger.LogInformation("dotnet build is already running. Waiting for it to terminate...");
			}

			using IDisposable mutex = await mutexTask;

			Stopwatch stopwatch = Stopwatch.StartNew();

			FileReference buildProject;
			if (projects.Count() > 1)
			{
				buildProject = FileReference.Combine(hook.EngineDirectory, "Source", "Programs", "Shared", "EpicGames.ScriptBuild");
				await WriteCsPropertiesAsync(projects, hook, cancellationToken);
			}
			else
			{
				buildProject = projects.First();
			}

			IEnumerable<string> arguments = [
				"build",
				$"\"{buildProject}\"",
				"-nologo",
				"-v:quiet",
				.. globalProperties.Select(x => $"\"/p:{x.Key}={x.Value}\""),
			];

			bool result = false;
			CaptureLogger captureLogger = new();

			// MacOS has been experiencing intermittent SIGBUS errors when building, so retry a few times on failure
			if (OperatingSystem.IsMacOS())
			{
				int maxRetries = 3;
				TimeSpan retryDelay = new(0, 0, 1);
				int count = 0;

				while (count < maxRetries && !cancellationToken.IsCancellationRequested)
				{
					captureLogger = new();
					result = await RunDotnetAsync(arguments, hook, captureLogger, cancellationToken);
					if (result)
					{
						break;
					}
					count++;
					if (count < maxRetries)
					{
						logger.LogInformation("dotnet command failed. Retrying in {Seconds} seconds... ({Count}/{MaxRetries})", retryDelay.Seconds, count, maxRetries);
						await Task.Delay(retryDelay, cancellationToken);
					}
					else
					{
						logger.LogInformation("dotnet command failed after {MaxRetries} attempts", maxRetries);
					}
				}
			}
			else
			{
				result = await RunDotnetAsync(arguments, hook, captureLogger, cancellationToken);
			}

			// If successful downgrade logs to Debug
			if (result)
			{
				captureLogger.Events.ForEach(x => x.Level = LogLevel.Debug);
			}
			captureLogger.RenderTo(logger);

			logger.LogInformation("Build {Count} projects time: {TimeSeconds:0.00} s", projects.Count(), stopwatch.Elapsed.TotalSeconds);

			return result;
		}
	}
}
