<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <PackageReference Include="Microsoft.Build" Version="18.4.0" ExcludeAssets="runtime" />
    <PackageReference Include="Microsoft.Build.Locator" Version="1.11.2" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>
</Project>
