// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Threading;
using EpicGames.Core;
using Microsoft.Build.Locator;

namespace EpicGames.MsBuild;

/// <summary>
/// Support for loading MSBuild assemblies from a dotnet install.
/// </summary>
public static class MsBuildSupport
{
	private static bool s_hasRegisteredMsBuildPath = false;
	private static readonly Lock s_lock = new();

	/// <summary>
	/// Register our bundled dotnet installation to be used by Microsoft.Build
	/// This needs to happen in a function called before the first use of any Microsoft.Build types
	/// </summary>
	public static void RegisterMsBuildPath(IMsBuildRegistration hook)
	{
		if (s_hasRegisteredMsBuildPath)
		{
			return;
		}

		lock (s_lock)
		{
			if (s_hasRegisteredMsBuildPath)
			{
				return;
			}

			// Find our bundled dotnet SDK
			List<string> listOfSdks = [];
			{
				ProcessStartInfo startInfo = new()
				{
					FileName = hook.DotnetPath.FullName,
					RedirectStandardOutput = true,
					UseShellExecute = false,
					ArgumentList = { "--list-sdks" }
				};
				startInfo.EnvironmentVariables["DOTNET_MULTILEVEL_LOOKUP"] = "0"; // use only the bundled dotnet installation - ignore any other/system dotnet install

				using Process? dotnetProcess = Process.Start(startInfo) ?? throw new Exception("Failed to start dotnet process");
				string? line;
				while ((line = dotnetProcess.StandardOutput.ReadLine()) != null)
				{
					listOfSdks.Add(line);
				}
				dotnetProcess.WaitForExit();
			}

			if (listOfSdks.Count != 1)
			{
				throw new Exception("Expected only one sdk installed for bundled dotnet");
			}

			// Expected output has this form:
			// 3.1.403 [D:\UE5_Main\engine\binaries\ThirdParty\DotNet\Windows\sdk]
			string sdkVersion = listOfSdks[0].Split(' ')[0];

			DirectoryReference dotnetSdkDirectory = DirectoryReference.Combine(hook.DotnetDirectory, "sdk", sdkVersion);
			if (!DirectoryReference.Exists(dotnetSdkDirectory))
			{
				throw new Exception($"Failed to find .NET SDK directory: {dotnetSdkDirectory.FullName}");
			}

			MSBuildLocator.RegisterMSBuildPath(dotnetSdkDirectory.FullName);

			s_hasRegisteredMsBuildPath = true;
		}
	}
}
