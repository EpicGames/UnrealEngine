<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <EnableConfigurationBindingGenerator>false</EnableConfigurationBindingGenerator>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="IdentityModel.OidcClient" Version="6.0.0" />
    <PackageReference Include="Microsoft.Extensions.Configuration.Binder" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Configuration.Json" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Options" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>

</Project>
