// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.OIDC;

#pragma warning disable CA1721 // Property names should not match get methods

/// <summary>
/// A fake implementation of OidcTokenManager for testing
/// Assumes just a single provider.
/// </summary>
public class FakeOidcTokenManager : IOidcTokenManager
{
	private const string AccessTokenPrefix = "fakeAccessToken";

	/// <summary>
	/// Refresh token in use
	/// </summary>
	public string? RefreshToken { get; set; }

	/// <summary>
	/// Latest access token
	/// </summary>
	public string? AccessToken { get; set; }

	/// <summary>
	/// Time to live for newly minted access token
	/// </summary>
	public TimeSpan AccessTokenTtl { get; set; } = TimeSpan.FromMinutes(15);

	/// <summary>
	/// Expiry time for access token
	/// </summary>
	public DateTimeOffset AccessTokenExpiry { get; set; } = DateTimeOffset.UnixEpoch;

	private int _refreshCounter = 1;
	private int _accessCounter = 1;
	private readonly Func<DateTimeOffset> _utcNow;

	/// <summary>
	/// Constructor
	/// </summary>
	public FakeOidcTokenManager(Func<DateTimeOffset> utcNow)
	{
		_utcNow = utcNow;
	}

	/// <inheritdoc/>
	public Task<IOidcTokenInfo> LoginAsync(string providerIdentifier, CancellationToken cancellationToken = default)
	{
		RefreshToken = "fakeRefreshToken-" + _refreshCounter++;
		RefreshAccessToken();
		return Task.FromResult(GetOidcTokenInfo());
	}

	/// <inheritdoc/>
	public Task<IOidcTokenInfo> GetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException("Method not in use");
	}

	/// <inheritdoc/>
	public Task<IOidcTokenInfo?> TryGetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default)
	{
		if (String.IsNullOrEmpty(RefreshToken))
		{
			throw new NotLoggedInException();
		}

		IOidcTokenInfo tokenInfo = GetOidcTokenInfo();

		// Ensure token is valid at least for another two minutes
		if (tokenInfo.IsValid(_utcNow().AddMinutes(-2)))
		{
			return Task.FromResult<IOidcTokenInfo?>(tokenInfo);
		}

		// Access token not valid, simulate a refresh with provider
		RefreshAccessToken();

		// Assume the updated access token is valid
		return Task.FromResult<IOidcTokenInfo?>(GetOidcTokenInfo());
	}

	/// <inheritdoc/>
	public Task<OidcStatus> GetStatusForProvider(string providerIdentifier, CancellationToken cancellationToken)
	{
		return Task.FromResult(OidcTokenClient.GetStatus(RefreshToken, AccessToken, AccessTokenExpiry));
	}

	/// <summary>
	/// Validate a fake access token minted by this class
	/// </summary>
	/// <param name="token">Token to check</param>
	/// <exception cref="Exception">If token is malformed or expired</exception>
	public void ValidateAccessToken(string? token)
	{
		if (String.IsNullOrEmpty(token))
		{
			throw new Exception($"Empty or null token: {token}");
		}

		string[] parts = token.Split('-');
		string prefix = parts[0];
		int id = Convert.ToInt32(parts[1]);
		long expireUnixTime = Convert.ToInt64(parts[2]);
		DateTimeOffset expireTime = DateTimeOffset.FromUnixTimeMilliseconds(expireUnixTime);
		DateTimeOffset utcNow = _utcNow();

		if (prefix != AccessTokenPrefix)
		{
			throw new Exception($"Bad prefix for token: {token}");
		}

		if (utcNow > expireTime)
		{
			Console.WriteLine($"_clock.UtcNow: {utcNow}");
			Console.WriteLine($"   expireTime: {expireTime}");
			int totalSeconds = (int)(_utcNow() - expireTime).TotalSeconds;
			throw new Exception($"Access token expired ({totalSeconds} seconds ago). Token: {token}");
		}
	}

	/// <summary>
	/// Perform fake refresh of the access token
	/// </summary>
	private void RefreshAccessToken()
	{
		AccessTokenExpiry = _utcNow() + AccessTokenTtl;
		AccessToken = $"{AccessTokenPrefix}-{_accessCounter++}-{AccessTokenExpiry.ToUnixTimeMilliseconds()}-{AccessTokenExpiry}";
	}

	private IOidcTokenInfo GetOidcTokenInfo()
	{
		return new OidcTokenInfo { RefreshToken = RefreshToken, AccessToken = AccessToken, TokenExpiry = AccessTokenExpiry };
	}
}
