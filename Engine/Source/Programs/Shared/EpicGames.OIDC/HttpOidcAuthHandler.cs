// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace EpicGames.OIDC;

/// <summary>
/// HTTP message handler which automatically refreshes access tokens as required.
/// </summary>
public sealed class HttpOidcAuthHandler : DelegatingHandler
{
	/// <summary>
	/// Constructor
	/// </summary>
	public HttpOidcAuthHandler(HttpMessageHandler innerHandler, IOidcAuthState authState, HttpOidcAuthHandlerConfig config, ILogger<HttpOidcAuthHandler>? logger = null)
		: base(innerHandler)
	{
		_authState = authState;
		_config = config;
		_logger = logger;
		_allowInteractiveLogin = new(_config.AllowInteractiveLoginHeaderKey);
	}

	private readonly IOidcAuthState _authState;
	private readonly HttpOidcAuthHandlerConfig _config;
	private readonly ILogger<HttpOidcAuthHandler>? _logger;
	private readonly HttpRequestOptionsKey<bool> _allowInteractiveLogin;

	/// <inheritdoc/>
	protected override async Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
	{
		// If the request already has a custom auth header, send the request as it is.
		if (request.Headers.Authorization != null)
		{
			return await base.SendAsync(request, cancellationToken);
		}

		// Check whether the request specifically allows interactive auth, otherwise fall back to the default.
		bool allowInteractiveLogin;
		if (!request.Options.TryGetValue(_allowInteractiveLogin, out allowInteractiveLogin))
		{
			allowInteractiveLogin = _config.AllowInteractiveLogin;
		}

		// Get the current access token and send the request with that.
		string? accessToken = await _authState.GetAccessTokenAsync(allowInteractiveLogin, cancellationToken);
		for (int attempt = 0; ; attempt++)
		{
			// Attempt to perform the request with this access token.
			request.Headers.Authorization = (accessToken == null) ? null : new AuthenticationHeaderValue(_config.AuthenticationScheme, accessToken);
			HttpResponseMessage response = await base.SendAsync(request, cancellationToken);

			if (response.StatusCode != HttpStatusCode.Unauthorized || attempt >= _config.MaxAttempts)
			{
				if (response.StatusCode == HttpStatusCode.Unauthorized && attempt >= _config.MaxAttempts)
				{
					_logger?.LogWarning("Received HTTP 401 from {Uri} after {Attempts} attempt(s). Auth failed.", request.RequestUri, attempt + 1);
				}
				return response;
			}

			_logger?.LogWarning("Received HTTP 401 from {Uri}. Invalidating token and retrying (attempt {Attempt}/{MaxAttempts}).", request.RequestUri, attempt + 1, _config.MaxAttempts);

			// Mark this access token as invalid.
			if (accessToken != null)
			{
				_authState.Invalidate(accessToken);
			}

			// Get the next token, and quit out if it's the same.
			string? nextAccessToken = await _authState.GetAccessTokenAsync(allowInteractiveLogin, cancellationToken);
			if (String.Equals(accessToken, nextAccessToken, StringComparison.Ordinal))
			{
				_logger?.LogWarning("New access token is identical to the invalidated one. Cannot recover auth for {Uri}.", request.RequestUri);
				return response;
			}

			// Otherwise update the token and try again.
			accessToken = nextAccessToken;
		}
	}
}
