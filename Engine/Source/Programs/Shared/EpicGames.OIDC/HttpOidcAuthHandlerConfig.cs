// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.OIDC;

/// <summary>
/// Configuration for a <see cref="HttpOidcAuthHandler"/>.
/// </summary>
public sealed record HttpOidcAuthHandlerConfig(
	string AllowInteractiveLoginHeaderKey,
	bool AllowInteractiveLogin = true,
	string AuthenticationScheme = "Bearer",
	int MaxAttempts = 3
);
