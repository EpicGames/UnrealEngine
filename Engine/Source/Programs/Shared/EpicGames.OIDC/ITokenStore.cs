// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using System.Security.Cryptography;
using System.Text;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#pragma warning disable CS1591 // Missing XML documentation on public types

namespace EpicGames.OIDC
{
	public interface ITokenStore : IDisposable
	{
		/// <summary>Always reads fresh from the backing store (disk/keychain).</summary>
		public bool TryGetRefreshToken(string oidcProvider, [MaybeNullWhen(false)] out string refreshToken);

		/// <summary>Persists immediately to the backing store.</summary>
		public void AddRefreshToken(string oidcProvider, string refreshToken);
	}

	/// <summary>
	/// Shared utilities for safe token logging.
	/// </summary>
	public static class TokenUtils
	{
		/// <summary>First 8 hex chars of SHA-256 - enough for log correlation without leaking the token.</summary>
		public static string Fingerprint(string token) => Convert.ToHexString(SHA256.HashData(Encoding.UTF8.GetBytes(token)))[..8];
	}

	public static class TokenStoreFactory
	{
		public static ITokenStore CreateTokenStore(IServiceProvider provider)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return ActivatorUtilities.CreateInstance<WindowsTokenStore>(provider);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				return ActivatorUtilities.CreateInstance<MacOSTokenStore>(provider);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				return ActivatorUtilities.CreateInstance<FilesystemTokenStore>(provider);
			}

			throw new NotSupportedException("Unknown platform when attempting to create ITokenStore");
		}

		public static ITokenStore CreateTokenStore(ILoggerFactory? loggerFactory = null)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return loggerFactory != null
					? new WindowsTokenStore(loggerFactory.CreateLogger<WindowsTokenStore>())
					: new WindowsTokenStore();
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				return new MacOSTokenStore();
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				return new FilesystemTokenStore();
			}

			throw new NotSupportedException("Unknown platform when attempting to create ITokenStore");
		}
	}
}