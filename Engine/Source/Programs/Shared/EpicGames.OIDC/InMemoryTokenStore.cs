// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.OIDC;

/// <summary>
/// An in-memory token store without encryption, intended for testing
/// </summary>
internal sealed class InMemoryTokenStore : ITokenStore
{
	private readonly Dictionary<string, string> _providerToRefreshToken = [];

	/// <inheritdoc/>
	public bool TryGetRefreshToken(string oidcProvider, out string refreshToken)
	{
		if (!_providerToRefreshToken.TryGetValue(oidcProvider, out string? result))
		{
			refreshToken = "";
			return false;
		}

		refreshToken = result;
		return true;
	}

	/// <inheritdoc/>
	public void AddRefreshToken(string oidcProvider, string refreshToken)
	{
		_providerToRefreshToken[oidcProvider] = refreshToken;
	}

	/// <inheritdoc/>
	public void Dispose()
	{
		// No unmanaged resources; required by ITokenStore (IDisposable)
	}
}
