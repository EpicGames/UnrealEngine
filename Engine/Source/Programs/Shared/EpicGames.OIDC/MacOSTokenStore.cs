// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Text;

#pragma warning disable CS1591 // Missing XML documentation on public types

namespace EpicGames.OIDC
{
	internal sealed class MacOSTokenStore : ITokenStore
	{
		public MacOSTokenStore()
		{
		}

		public bool TryGetRefreshToken(string oidcProvider, out string refreshToken)
		{
			string? token = KeychainHelper.GetKeychainItem(oidcProvider);

			if (token == null)
			{
				refreshToken = "";
				return false;
			}

			refreshToken = token;
			return true;
		}

		public void AddRefreshToken(string oidcProvider, string refreshToken)
		{
			KeychainHelper.AddKeychainItem(refreshToken, oidcProvider);
		}

		public void Dispose()
		{
		}
	}

	static class KeychainHelper
	{
		private const string OidcTokenIdentifier = "EpicGames OidcToken";

		private static string? RunKeychainHandler(string verb, List<string> args)
		{
			ProcessStartInfo startInfo = new()
			{
				FileName = "security",
				RedirectStandardOutput = true,
				RedirectStandardError = true
			};
			startInfo.ArgumentList.Add(verb);
			foreach (string s in args)
			{
				startInfo.ArgumentList.Add(s);
			}

			using Process p = Process.Start(startInfo) ?? throw new Exception("Failed to run \"security\" to interact with keychain");

			StringBuilder sb = new();
			p.OutputDataReceived += (o, e) => { sb.Append(e.Data); };
			p.BeginOutputReadLine();
			p.WaitForExit();

			if (p.ExitCode == 44)
			{
				return null; // not found
			}

			if (p.ExitCode == 36)
			{
				return null; // not found
			}

			if (p.ExitCode != 0)
			{
				throw new Exception($"Unhandled exitcode {p.ExitCode} when interacting with keychain");
			}

			return sb.ToString();
		}

		public static void AddKeychainItem(string dataToEncrypt, string itemIdentifier)
		{
			RunKeychainHandler("add-generic-password", [
				"-a",
				itemIdentifier,
				"-s",
				OidcTokenIdentifier,
				"-U",
				"-w",
				dataToEncrypt
			]);
		}

		public static string? GetKeychainItem(string itemIdentifier)
		{
			string? output = RunKeychainHandler("find-generic-password", [
				"-a",
				itemIdentifier,
				"-s",
				OidcTokenIdentifier,
				"-w"
			]);

			return output;
		}
	}
}