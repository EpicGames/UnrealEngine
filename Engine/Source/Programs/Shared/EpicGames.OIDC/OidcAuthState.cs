// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.OIDC;

/// <summary>
/// Interface to manage the state of OIDC authentication.
/// </summary>
public interface IOidcAuthState
{
	/// <summary>
	/// Event handler for the auth state changes.
	/// </summary>
	event Action? OnStateChanged;

	/// <summary>
	/// Is there a valid auth header available?
	/// </summary>
	bool IsLoggedIn { get; }

	/// <summary>
	/// Get the current access token, if one exists.
	/// </summary>
	string? CurrentAccessToken { get; }

	/// <summary>
	/// Reset the current auth state.
	/// </summary>
	void Reset();

	/// <summary>
	/// Marks the given access token as invalid, typically after having attempted to use it and got an unauthorized response.
	/// </summary>
	/// <param name="accessToken">The access token to invalidate.</param>
	void Invalidate(string? accessToken);

	/// <summary>
	/// Get a valid access token, if possible.
	/// </summary>
	Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken);
}

/// <summary>
/// Class used to track the last obtained token for a given OIDC provider.
/// </summary>
public sealed class OidcAuthState(
	IOidcTokenManager oidcTokenManager,
	IStaticAccessTokenProvider staticAccessTokenProvider,
	IClock clock,
	IOidcAuthStateConfig config,
	ILogger<OidcAuthState> logger
) : IOidcAuthState, IAsyncDisposable
{
	/// <inheritdoc/>
	public event Action? OnStateChanged;

	/// <inheritdoc/>
	public async ValueTask DisposeAsync()
	{
		if (_currentAuthWorker != null)
		{
			await _currentAuthWorker.DisposeAsync();
			_currentAuthWorker = null;
		}
	}

	/// <inheritdoc/>
	public bool IsLoggedIn
	{
		get
		{
			if (_staticAccessTokenProvider.AccessToken != null)
			{
				return true;
			}

			if (TryGetCurrentAuthState(out AuthState? authState) && authState.IsAuthorized())
			{
				return true;
			}

			return false;
		}
	}

	/// <inheritdoc/>
	public string? CurrentAccessToken
	{
		get
		{
			string? accessToken = _staticAccessTokenProvider.AccessToken;
			if (accessToken != null)
			{
				return accessToken;
			}

			if (TryGetCurrentAuthState(out AuthState? authState) && authState.IsAuthorized())
			{
				return authState.TokenInfo?.AccessToken;
			}

			return null;
		}
	}

	/// <inheritdoc/>
	public void Reset()
	{
		if (_staticAccessTokenProvider.AccessToken != null)
		{
			return;
		}

		lock (_lockObject)
		{
			_currentAuthResult = null;
		}
	}

	/// <summary>
	/// Marks the given access token as invalid, having attempted to use it and got an unauthorized response
	/// </summary>
	/// <param name="accessToken">The access  header to invalidate</param>
	public void Invalidate(string? accessToken)
	{
		if (_staticAccessTokenProvider.AccessToken != null)
		{
			return;
		}

		lock (_lockObject)
		{
			if (TryGetCurrentAuthState(out AuthState? authState) && Equals(authState.TokenInfo?.AccessToken, accessToken))
			{
				_logger.LogInformation("Access token invalidated (likely received HTTP 401). Will attempt to acquire a new token.");
				_currentAuthResult = null;
			}
		}
	}

	/// <summary>
	/// Gets the current access token
	/// </summary>
	public async Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken)
	{
		// If there's a static access token configured, use that
		string? accessToken = _staticAccessTokenProvider.AccessToken;
		if (accessToken != null)
		{
			return accessToken;
		}

		// Otherwise check if we need to start a background task to figure it out
		Task<AuthState> authStateTask;
		lock (_lockObject)
		{
			if (_currentAuthResult == null || (interactive && !_currentAuthInteractive))
			{
				int authTaskId = ++_currentAuthTaskId;

				if (_currentAuthResult == null || _currentAuthResult.Task.IsCompleted)
				{
					_currentAuthResult = new TaskCompletionSource<AuthState>(TaskCreationOptions.RunContinuationsAsynchronously);
				}

				BackgroundTask? prevAuthWorker = _currentAuthWorker;
				_currentAuthWorker = BackgroundTask.StartNew(ctx => GetAuthStateHandlerAsync(authTaskId, interactive, prevAuthWorker, ctx));

				_currentAuthInteractive = interactive;
			}
			authStateTask = _currentAuthResult.Task;
		}

		// Wait for the task to complete
		AuthState authState = await authStateTask.WaitAsync(cancellationToken);
		return authState?.TokenInfo?.AccessToken;
	}

	private readonly Lock _lockObject = new();
	private readonly IOidcTokenManager _oidcTokenManager = oidcTokenManager;
	private readonly IStaticAccessTokenProvider _staticAccessTokenProvider = staticAccessTokenProvider;
	private readonly IClock _clock = clock;
	private readonly IOidcAuthStateConfig _config = config;
	private readonly ILogger _logger = logger;

	// Background auth process
	private int _currentAuthTaskId;
	private bool _currentAuthInteractive;
	private BackgroundTask? _currentAuthWorker;
	private TaskCompletionSource<AuthState>? _currentAuthResult;

	/// <summary>
	/// Gets the current auth state instance. Fails if the current auth task has not finished.
	/// </summary>
	private bool TryGetCurrentAuthState([NotNullWhen(true)] out AuthState? authState)
	{
		AuthState? state;
		if (_currentAuthResult != null && _currentAuthResult.Task.IsCompletedSuccessfully && _currentAuthResult.Task.TryGetResult(out state))
		{
			authState = state;
			return true;
		}
		else
		{
			authState = null;
			return false;
		}
	}

	private async Task GetAuthStateHandlerAsync(int authTaskId, bool interactive, BackgroundTask? prevAuthTask, CancellationToken cancellationToken)
	{
		Task? disposeTask = null;
		try
		{
			// Start disposing of the previous auth task asynchronously
			if (prevAuthTask != null)
			{
				disposeTask = Task.Run(() => prevAuthTask.DisposeAsync().AsTask(), CancellationToken.None);
			}

			// Get the new auth state
			bool stateHasChanged = false;
			try
			{
				AuthState authState = await GetAuthStateInternalAsync(interactive, cancellationToken);
				lock (_lockObject)
				{
					if (_currentAuthTaskId == authTaskId)
					{
						_logger.LogDebug("Auth task complete (interactive: {Interactive}, authorized: {Authorized})", authState.Interactive, authState.IsAuthorized());
						stateHasChanged = _currentAuthResult?.TrySetResult(authState) ?? false;
					}
				}
			}
			catch (Exception ex)
			{
				lock (_lockObject)
				{
					if (_currentAuthTaskId == authTaskId)
					{
						_logger.LogDebug(ex, "Exception while attempting auth: {Message}", ex.Message);
						stateHasChanged = _currentAuthResult?.TrySetException(ex) ?? false;
					}
				}
			}

			// Send notifications about the updated state
			if (stateHasChanged)
			{
				try
				{
					OnStateChanged?.Invoke();
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while sending state change notifications: {Message}", ex.Message);
				}
			}
		}
		finally
		{
			// Wait for the child task to finish being disposed
			if (disposeTask != null)
			{
				await disposeTask;
			}
		}
	}

	private async Task<AuthState> GetAuthStateInternalAsync(bool interactive, CancellationToken cancellationToken)
	{
		IOidcTokenInfo? result = null;
		string oidcProvider = await _config.GetOidcProviderAsync(cancellationToken);

		OidcStatus providerStatus = await _oidcTokenManager.GetStatusForProvider(oidcProvider, cancellationToken);
		if (providerStatus != OidcStatus.NotLoggedIn)
		{
			try
			{
				result = await _oidcTokenManager.TryGetAccessToken(oidcProvider, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to get access token for provider '{Provider}': {Message}", oidcProvider, ex.Message);
			}
		}
		if (result == null && interactive)
		{
			_logger.LogWarning("No valid access token obtained for provider '{Provider}'. Starting interactive login (browser).", oidcProvider);
			result = await _oidcTokenManager.LoginAsync(oidcProvider, cancellationToken);
		}
		else if (result == null)
		{
			_logger.LogWarning("No valid access token obtained for provider '{Provider}' and interactive login is disabled.", oidcProvider);
		}

		return new AuthState(_clock, result, interactive);
	}

	private record AuthState(IClock Clock, IOidcTokenInfo? TokenInfo, bool Interactive)
	{
		public bool IsAuthorized()
		{
			return TokenInfo != null && TokenInfo.IsValid(Clock.UtcNow);
		}
	}
}
