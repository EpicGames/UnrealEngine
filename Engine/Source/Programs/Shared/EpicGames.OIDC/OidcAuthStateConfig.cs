// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

namespace EpicGames.OIDC;

/// <summary>
/// Interface defining configuration for an <see cref="OidcAuthState"/>.
/// </summary>
public interface IOidcAuthStateConfig
{
	/// <summary>
	/// The OIDC provider associated with this auth state.
	/// </summary>
	Task<string> GetOidcProviderAsync(CancellationToken cancellationToken);
}

/// <summary>
/// Configuration for an <see cref="OidcAuthState"/>.
/// </summary>
public sealed class OidcAuthStateConfig(string oidcProvider) : IOidcAuthStateConfig
{
	/// <inheritdoc/>
	public Task<string> GetOidcProviderAsync(CancellationToken cancellationToken) => Task.FromResult(oidcProvider);
}
