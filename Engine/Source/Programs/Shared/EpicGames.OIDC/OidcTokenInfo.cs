// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.OIDC;

/// <summary>
/// Interface describing an OIDC token.
/// </summary>
public interface IOidcTokenInfo
{
	/// <summary>
	/// The refresh token.
	/// </summary>
	public string? RefreshToken { get; }
	/// <summary>
	/// The access token.
	/// </summary>
	public string? AccessToken { get; }

	/// <summary>
	/// When the token expires.
	/// </summary>
	public DateTimeOffset TokenExpiry { get; }

	/// <summary>
	/// Determine if the token is valid at the given time.
	/// </summary>
	public bool IsValid(DateTimeOffset currentTime);
}

/// <summary>
/// An OIDC access token, refresh token, and expiry.
/// </summary>
public class OidcTokenInfo : IOidcTokenInfo
{
	/// <inheritdoc/>
	public string? RefreshToken { get; set; }

	/// <inheritdoc/>
	public string? AccessToken { get; set; }

	/// <inheritdoc/>
	public DateTimeOffset TokenExpiry { get; set; }

	/// <inheritdoc/>
	public bool IsValid(DateTimeOffset currentTime)
	{
		if (String.IsNullOrEmpty(RefreshToken) || String.IsNullOrEmpty(AccessToken))
		{
			return false;
		}

		// An expiry of MinValue means it has no expiration time
		if (TokenExpiry == DateTimeOffset.MinValue)
		{
			return true;
		}

		return currentTime <= TokenExpiry;
	}
}

/// <summary>
/// A fake OIDC token info representing calls that allow anonymous callers.
/// </summary>
public sealed class AnonymousOidcTokenInfo : IOidcTokenInfo
{
	/// <inheritdoc/>
	public string? RefreshToken => null;

	/// <inheritdoc/>
	public string? AccessToken => null;

	/// <inheritdoc/>
	public DateTimeOffset TokenExpiry => DateTimeOffset.MaxValue;

	/// <inheritdoc/>
	public bool IsValid(DateTimeOffset currentTime) => true;
}
