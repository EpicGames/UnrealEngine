// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using IdentityModel.Client;
using IdentityModel.OidcClient;
using IdentityModel.OidcClient.Results;
using EpicGames.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

#pragma warning disable CS1591 // Missing XML documentation on public types
#pragma warning disable CA2227 // Remote property setters on collection types

namespace EpicGames.OIDC
{
	/// <summary>
	/// Result of an OIDC token refresh operation.
	/// </summary>
	public record TokenRefreshResult(
		bool IsError, string? Error, string? ErrorDescription,
		string? AccessToken, string? RefreshToken, DateTimeOffset AccessTokenExpiration);

	public interface IOidcTokenManager
	{
		public Task<IOidcTokenInfo> LoginAsync(string providerIdentifier, CancellationToken cancellationToken = default);

		public Task<IOidcTokenInfo> GetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default);

		public Task<IOidcTokenInfo?> TryGetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default);

		public Task<OidcStatus> GetStatusForProvider(string providerIdentifier, CancellationToken cancellationToken = default);
	}

	public class OidcTokenManager : IOidcTokenManager
	{
		private readonly Lock _lockObject = new();
		private readonly Dictionary<string, OidcTokenClient> _tokenClients = new(StringComparer.OrdinalIgnoreCase);
		private readonly ITokenStore _tokenStore;

		public OidcTokenManager(ITokenStore tokenStore)
		{
			_tokenStore = tokenStore;
		}

		public OidcTokenClient FindOrAddClient(string name, ProviderInfo providerInfo)
		{
			lock (_lockObject)
			{
				OidcTokenClient? client;
				if (!_tokenClients.TryGetValue(name, out client))
				{
					client = new OidcTokenClient(name, providerInfo, TimeSpan.FromMinutes(20), _tokenStore);
					_tokenClients.Add(name, client);
				}
				return client;
			}
		}

		public OidcTokenManager(IOptionsMonitor<OidcTokenOptions> settings, ITokenStore tokenStore, IOidcTokenClientFactory tokenClientFactory, List<string>? allowedProviders = null)
			: this(tokenStore)
		{
			List<string> providerKeys = [.. settings.CurrentValue.Providers.Keys.Where(k => allowedProviders == null || allowedProviders.Contains(k))];

			foreach ((string key, ProviderInfo providerInfo) in settings.CurrentValue.Providers)
			{
				if (providerKeys != null && !providerKeys.Contains(key))
				{
					continue;
				}

				OidcTokenClient tokenClient = tokenClientFactory.CreateTokenClient(key, providerInfo, settings.CurrentValue.LoginTimeout);
				_tokenClients.Add(key, tokenClient);
			}
		}

		public static OidcTokenManager CreateTokenManager(IConfiguration providerConfiguration, ITokenStore tokenStore, List<string>? allowedProviders = null, ILoggerFactory? loggerFactory = null)
		{
			return new OidcTokenManager(providerConfiguration, tokenStore, allowedProviders, loggerFactory);
		}

		private OidcTokenManager(IConfiguration providerConfiguration, ITokenStore tokenStore, List<string>? allowedProviders = null, ILoggerFactory? loggerFactory = null)
			: this(tokenStore)
		{
			OidcTokenOptions options = new();
			providerConfiguration.Bind(options);

			List<string> providerKeys = [.. options.Providers.Keys.Where(k => allowedProviders == null || allowedProviders.Contains(k))];

			foreach ((string key, ProviderInfo providerInfo) in options.Providers)
			{
				if (providerKeys != null && !providerKeys.Contains(key))
				{
					continue;
				}

				OidcTokenClient tokenClient = new(key, providerInfo, options.LoginTimeout, _tokenStore, loggerFactory?.CreateLogger<OidcTokenClient>());
				_tokenClients.Add(key, tokenClient);
			}
		}

		public bool HasUnfinishedLogin()
		{
			return _tokenClients.Any(pair => pair.Value.GetStatus() == OidcStatus.NotLoggedIn);
		}

		/// <inheritdoc/>
		public Task<IOidcTokenInfo> LoginAsync(string providerIdentifier, CancellationToken cancellationToken = default)
		{
			return _tokenClients[providerIdentifier].LoginAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<IOidcTokenInfo> GetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default)
		{
			return _tokenClients[providerIdentifier].GetAccessTokenAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<IOidcTokenInfo?> TryGetAccessToken(string providerIdentifier, CancellationToken cancellationToken = default)
		{
			return _tokenClients[providerIdentifier].TryGetAccessTokenAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<OidcStatus> GetStatusForProvider(string providerIdentifier, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(_tokenClients[providerIdentifier].GetStatus());
		}
	}

	public interface IOidcTokenClientFactory
	{
		OidcTokenClient CreateTokenClient(string key, ProviderInfo providerInfo, TimeSpan loginTimeout);
	}

	public class OidcTokenClientFactory(IServiceProvider provider) : IOidcTokenClientFactory
	{
		public OidcTokenClient CreateTokenClient(string key, ProviderInfo providerInfo, TimeSpan loginTimeout)
		{
			return ActivatorUtilities.CreateInstance<OidcTokenClient>(provider, key, providerInfo, loginTimeout);
		}
	}

	public enum OidcStatus
	{
		/// <summary>
		/// Both access and refresh token are valid
		/// </summary>
		Connected,

		/// <summary>
		/// No refresh token is set and requires login
		/// </summary>
		NotLoggedIn,

		/// <summary>
		/// Access token has not been generated or has expired, and needs refreshing 
		/// </summary>
		TokenRefreshRequired
	}

	/// <summary>
	/// The data read from a http request into the oidc client, mostly a wrapper around the http.listener context
	/// </summary>
	public class OidcHttpRequest
	{
		public OidcHttpRequest(HttpListenerContext context)
		{
			HttpListenerRequest request = context.Request;
			HasBody = request.HasEntityBody;
			HttpMethod = request.HttpMethod;
			RawUrl = request.RawUrl;
			ContentType = request.ContentType;
			Body = request.InputStream;
			BodyEncoding = request.ContentEncoding;
		}

#pragma warning disable CA1054
		public OidcHttpRequest(string rawUrl, string contentType, string httpMethod, Stream body, bool hasBody, Encoding bodyEncoding)
#pragma warning restore CA1054
		{
			RawUrl = rawUrl;
			ContentType = contentType;
			HttpMethod = httpMethod;
			Body = body;
			HasBody = hasBody;
			BodyEncoding = bodyEncoding;
		}

		public bool HasBody { get; init; }
		public string HttpMethod { get; init; }
#pragma warning disable CA1056 // This does not need to be a Uri
		public string? RawUrl { get; init; }
#pragma warning restore CA1056
		public string? ContentType { get; init; }
		public Stream Body { get; init; }
		public Encoding BodyEncoding { get; init; }
	}

	/// <summary>
	/// Interface for the response writing that happens from the local http server, usually wraps a http.listener response
	/// </summary>
	public interface IOidcHttpResponse : IDisposable
	{
		/// <summary>
		/// The content length header value
		/// </summary>
		public long ContentLength64 { get; set; }
		/// <summary>
		/// The status code header value
		/// </summary>
		public int StatusCode { get; set; }

		/// <summary>
		/// The body of the response
		/// </summary>
		public Stream OutputStream { get; }

		/// <summary>
		/// Used to explicitly close the response when finished
		/// </summary>
		public void Close()
		{
			Dispose();
		}
	}

	/// <summary>
	/// Implementation of a response writer for http listener
	/// </summary>
	/// <param name="context"></param>
	public sealed class OidcHttpResponse(HttpListenerContext context) : IOidcHttpResponse
	{
		private readonly HttpListenerResponse _response = context.Response;

		/// <inheritdoc/>
		public long ContentLength64 { get => _response.ContentLength64; set => _response.ContentLength64 = value; }
		/// <inheritdoc/>
		public int StatusCode { get => _response.StatusCode; set => _response.StatusCode = value; }
		/// <inheritdoc/>
		public Stream OutputStream => _response.OutputStream;

		/// <inheritdoc/>
		public void Close()
		{
			_response.Close();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			((IDisposable)_response).Dispose();
		}
	}

	// a response writer that does not actually use the response in any way, just buffering it internally
	public class NullOidcHttpResponse(Stream outputStream) : IOidcHttpResponse
	{
		/// <summary>
		/// Dispose method
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				// no resources to release
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public long ContentLength64 { get; set; }
		/// <inheritdoc/>
		public int StatusCode { get; set; }
		/// <inheritdoc/>
		public Stream OutputStream { get; } = outputStream;
	}

	/// <summary>
	/// Wrapper around HttpListener to make it possible to inject a mock and thus test the Oidc flow without starting a http server
	/// </summary>
	// ReSharper disable once ClassWithVirtualMembersNeverInherited.Global - overriden in test mocks
	public class OidcHttpServer : IDisposable
	{
		private readonly HttpListener _http = new();

		public virtual void Start()
		{
			_http.Start();
		}

		/// <summary>
		/// Dispose method
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				((IDisposable)_http).Dispose();
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

#pragma warning disable CA1716 // we want to call this Stop to maintain an interface that is very similar to http.listener
		public virtual void Stop()
#pragma warning restore CA1716
		{
			_http.Stop();
		}

		public virtual void AddPrefixes(string prefix)
		{
			_http.Prefixes.Add(prefix);
		}

		public virtual async Task<(OidcHttpRequest, IOidcHttpResponse)> ProcessNextRequestAsync()
		{
			HttpListenerContext context = await _http.GetContextAsync();
			return (new OidcHttpRequest(context), new OidcHttpResponse(context));
		}
	}

	public class OidcTokenClient
	{
		/// <summary>Max seconds the refresh HTTP call is allowed before cancellation.</summary>
		private const int RefreshHttpTimeoutSeconds = 30;

		/// <summary>Max seconds to wait for another process to release the refresh lock.</summary>
		private const int LockAcquireTimeoutSeconds = 60;

		public string Name => _name;
		public string DisplayName => _providerInfo.DisplayName;

		private readonly string _name;
		private readonly ProviderInfo _providerInfo;
		private readonly TimeSpan _loginTimeout;
		private readonly ITokenStore _tokenStore;
		private readonly ILogger<OidcTokenClient>? _logger;
		private readonly Uri _authorityUri;
		private readonly string _clientId;
		private readonly string _scopes;

		private string? _accessToken;
		private DateTimeOffset _tokenExpiry;

		private readonly List<Uri> _redirectUris;
		private readonly string? _genericErrorInformation;
		private readonly bool _useDiscoveryDocument;

		public OidcTokenClient(string name, ProviderInfo providerInfo, TimeSpan loginTimeout, ITokenStore tokenStore, ILogger<OidcTokenClient>? logger = null)
		{
			_name = name;
			_providerInfo = providerInfo;
			_loginTimeout = loginTimeout;
			_tokenStore = tokenStore;
			_logger = logger;

			_authorityUri = providerInfo.ServerUri;
			_clientId = providerInfo.ClientId;

			List<Uri> possibleRedirectUris = [];
			if (providerInfo.RedirectUri != null)
			{
				possibleRedirectUris.Add(providerInfo.RedirectUri);
			}
			if (providerInfo.PossibleRedirectUri != null)
			{
				possibleRedirectUris.AddRange(providerInfo.PossibleRedirectUri);
			}
			_redirectUris = possibleRedirectUris;

			_scopes = providerInfo.Scopes;
			_useDiscoveryDocument = providerInfo.UseDiscoveryDocument;

			_genericErrorInformation = providerInfo.GenericErrorInformation;
			if (_logger?.IsEnabled(LogLevel.Debug) == true)
			{
				_tokenStore.TryGetRefreshToken(name, out string? existingToken);
				_logger.LogDebug("OidcTokenClient created for provider '{Name}'. Refresh token from store: {HasToken}", name, !String.IsNullOrEmpty(existingToken));
			}
		}

		private async Task<OidcClientOptions> BuildClientOptionsAsync(Uri redirectUri, CancellationToken cancellationToken)
		{
			OidcClientOptions options = new()
			{
				Authority = _authorityUri.ToString(),
				Policy = new Policy { Discovery = new DiscoveryPolicy { Authority = _authorityUri.ToString() } },
				ClientId = _clientId,
				Scope = _scopes,
				FilterClaims = false,
				RedirectUri = redirectUri.ToString(),
				LoadProfile = _providerInfo.LoadClaimsFromUserProfile,
			};

			// we need to fetch the discovery document ourselves to support OIDC Authorities which have a subresource for it
			// which Okta has for authorization servers for instance.

			if (_useDiscoveryDocument)
			{
				DiscoveryDocumentResponse discoveryDocument = await GetDiscoveryDocumentAsync(cancellationToken);
				options.ProviderInformation = new ProviderInformation
				{
					IssuerName = discoveryDocument.Issuer,
					KeySet = discoveryDocument.KeySet,

					AuthorizeEndpoint = discoveryDocument.AuthorizeEndpoint,
					TokenEndpoint = discoveryDocument.TokenEndpoint,
					EndSessionEndpoint = discoveryDocument.EndSessionEndpoint,
					UserInfoEndpoint = discoveryDocument.UserInfoEndpoint,
					TokenEndPointAuthenticationMethods = discoveryDocument.TokenEndpointAuthenticationMethodsSupported
				};
			}

			return options;
		}

		public async Task<IOidcTokenInfo> LoginAsync(CancellationToken cancellationToken)
		{
			_logger?.LogInformation("Starting interactive login for provider '{Name}'.", _name);
			IOidcTokenInfo tokenInfo = await DoLoginAsync(cancellationToken);

			if (String.IsNullOrEmpty(tokenInfo.RefreshToken))
			{
				throw new Exception("No refresh token was provided in response.");
			}

			_accessToken = tokenInfo.AccessToken;
			_tokenExpiry = tokenInfo.TokenExpiry;

			_logger?.LogDebug("Saving refresh token to store after interactive login for provider '{Name}'.", _name);
			// After browser login, token MUST be persisted regardless of external cancellation.
			using CancellationTokenSource loginPersistCts = new(TimeSpan.FromSeconds(LockAcquireTimeoutSeconds));
			using IDisposable _ = await AcquireRefreshLockAsync(loginPersistCts.Token);
			_tokenStore.AddRefreshToken(_name, tokenInfo.RefreshToken);

			_logger?.LogInformation("Interactive login completed for provider '{Name}'. Token expiry: {Expiry}", _name, tokenInfo.TokenExpiry);
			return tokenInfo;
		}

		private async Task<IOidcTokenInfo> DoLoginAsync(CancellationToken cancellationToken)
		{
			// setup a local http server to listen for the result of the login
			LoginResult? loginResult = null;
			foreach (Uri uri in _redirectUris)
			{
				try
				{
#pragma warning disable CA2000 // Dispose objects before losing scope <-- FALSE POSITIVE?
					using OidcHttpServer http = CreateHttpServer();
#pragma warning restore CA2000 // Dispose objects before losing scope

					// build the url the server should be hosted at
					string prefix = $"{uri.Scheme}{Uri.SchemeDelimiter}{uri.Authority}/";
					http.AddPrefixes(prefix);
					http.Start();

					OidcClientOptions options = await BuildClientOptionsAsync(uri, cancellationToken);
					OidcClient oidcClient = DoCreateOidcClient(options);
					// generate the appropriate codes we need to login
					AuthorizeState loginState = await oidcClient!.PrepareLoginAsync(cancellationToken: cancellationToken);

					// start the user browser
#pragma warning disable CA2000 // Dispose objects before losing scope <-- FALSE POSITIVE?
					using (Process? process = OpenBrowser(loginState.StartUrl))
					{
						using (IDisposable _ = cancellationToken.Register(() => http.Stop()))
						{
							CancellationTokenSource tokenSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);

							try
							{
								Task<LoginResult> processHttpTask = ProcessHttpRequestAsync(http, loginState, oidcClient, tokenSource.Token);
								Task finishedTask = await Task.WhenAny(Task.Delay(_loginTimeout, cancellationToken), processHttpTask);
								if (finishedTask == processHttpTask)
								{
									loginResult = await processHttpTask;
								}
								else
								{
									// timed out
									loginResult = new LoginResult($"Login timed out after: {_loginTimeout.TotalMinutes} minutes");
									await tokenSource.CancelAsync();
									// we do not await the processHttpTask as the GetContext method does not provide any way to cancel, as such this task will be stuck until httplistener is disposed
								}
							}
							catch when (cancellationToken.IsCancellationRequested)
							{
								loginResult = new LoginResult("Operation cancelled");
							}

							// wait a few seconds before shutting down the http server to give the browser time to actually load everything it needs
							await Task.Delay(2000, cancellationToken);

							http.Stop();
						}
					}
#pragma warning restore CA2000 // Dispose objects before losing scope

					break;
				}
				catch (HttpListenerException)
				{
					continue;
				}
			}

			if (loginResult == null)
			{
				throw new HttpServerException("Unable to login as none of the possible redirect uris were successful. Uris used: " + String.Join(' ', _redirectUris));
			}
			if (loginResult.IsError)
			{
				throw new LoginFailedException("Failed to login due to error: " + loginResult.Error, loginResult.ErrorDescription);
			}

			return new OidcTokenInfo
			{
				RefreshToken = loginResult.RefreshToken,
				AccessToken = loginResult.AccessToken,
				TokenExpiry = loginResult.AccessTokenExpiration
			};
		}

		public virtual OidcHttpServer CreateHttpServer()
		{
			return new OidcHttpServer();
		}

		public virtual OidcClient DoCreateOidcClient(OidcClientOptions options)
		{
			return new OidcClient(options);
		}

		private async Task<LoginResult> ProcessHttpRequestAsync(OidcHttpServer http, AuthorizeState loginState, OidcClient oidcClient, CancellationToken cancellationToken)
		{
			const int MaxAttempts = 5;
			for (int i = 0; i < MaxAttempts; i++)
			{
				if (cancellationToken.IsCancellationRequested)
				{
					return new LoginResult("Login cancelled");
				}

				(OidcHttpRequest request, IOidcHttpResponse response) = await http.ProcessNextRequestAsync();

				if (cancellationToken.IsCancellationRequested)
				{
					return new LoginResult("Login cancelled");
				}

				LoginResult loginResult;
				string? responseData;
				switch (request.HttpMethod)
				{
					case "GET":
						responseData = request.RawUrl;

						// parse the returned url for the tokens needed to complete the login
						loginResult = await oidcClient!.ProcessResponseAsync(responseData, loginState, cancellationToken: cancellationToken);
						break;
					case "POST":
						{
							if (request.ContentType != null && !request.ContentType.Equals("application/x-www-form-urlencoded", StringComparison.OrdinalIgnoreCase))
							{
								// we do not support url encoded return types
								response.StatusCode = 415;
								throw new Exception("URL encoded responses not support");
							}

							// attempt to parse the body

							// if there is no body we can not handle the post
							if (!request.HasBody)
							{
								response.StatusCode = 415;
								throw new Exception("Empty body not supported");
							}

							await using Stream body = request.Body;
							using StreamReader reader = new(body, request.BodyEncoding);
							responseData = await reader.ReadToEndAsync(cancellationToken);

							loginResult = await oidcClient!.ProcessResponseAsync(responseData, loginState, cancellationToken: cancellationToken);
							break;
						}
					case "OPTIONS":
						response.StatusCode = 200;
						response.Close();
						continue;
					default:
						// if we receive any other http method something is very odd. Tell them to use a different method.
						response.StatusCode = 415;
						throw new Exception("Unsupported method used: " + request.HttpMethod);
				}

				if (cancellationToken.IsCancellationRequested)
				{
					return new LoginResult("Login cancelled");
				}

				// generate a simple http page to show the user
				string httpPage = loginResult.IsError ? GetFailurePage(loginResult, _genericErrorInformation) : GetSuccessPage();
				byte[] buffer = Encoding.UTF8.GetBytes(httpPage);
				response.ContentLength64 = buffer.Length;
				Stream responseOutput = response.OutputStream;
				await responseOutput.WriteAsync(buffer, 0, buffer.Length, cancellationToken);
				responseOutput.Close();

				return loginResult;
			}

			throw new Exception("Failed to process login after multiple attempts");
		}

		private static string GetSuccessPage()
		{
			string httpPageSuccess = String.Format("<html><head></head><body style=\"background-color: black; color: white; \"><div style=\"text-align: center; \"><img style=\"object-fit:contain; width: 40 %; height: 30 %; opacity: 0.5; margin-top: 64px;\" alt=\"Unreal Engine Logo\" src=\"{0}\" /></div><br /><br /><h1 style=\"margin:1%; text-align: center;font-family: sans-serif;font-style: normal;font-weight: 400;font-size: 25.236px;line-height: 36px;color: #FFFFFF;\"><img style=\"object-fit:contain; vertical-align:middle;\" alt=\"\" src=\"{1}\" />You have been successfully logged in.<h1></body><p style=\"text-align: center; font-family: sans-serif;font-style: normal;font-weight: 400;font-size: 16px;line-height: 30px;letter-spacing: 0.371118px;color: #FFFFFF;opacity: 0.5; \">You may safely close this page to continue.</p><script>setTimeout(function() {{window.close()}}, 1000);</script></html>"
				, "data:image/png; base64,iVBORw0KGgoAAAANSUhEUgAAAqUAAAIhCAMAAABAJzPOAAAAM1BMVEUAAAD///////////////////////////////////////////////////////////////+3leKCAAAAEHRSTlMAQMCAEPCgYNAw4CCQcFCwJwL4owAAHPRJREFUeNrs3dl62jAQBeDRvlnL+z9t268UbIxpAsjRcv77AI4PkmZkGwIAAAAAAAAAAAAA+KjAdgigAY4ZsXBuyxHFuRYCgYWfEJjQXJVv4FkYhBVOwqLm5VU+CyYJoJ5gFlve53PEqAo1uJhV+SAukFT4JGm0LxXw6AjgA1y05X8853wRdzjnX/hLnQjgLW7x5ZjlQiTm6BnJWBT5WT9AaYOCCipElC+RBfoWZgQ/fL2MERVeEKI/6NOLFOhVkol88Loa1RR8j+GPF5HG0ftkWh6/vAgE8DVhUQ8T+tEMpcVi5odXJX5Wy0garco9H1FKwXNyvxpVuWYJzpb9G2pM/HAsCLVLTKIDNVsJGZUUPBb0LixnNTLdbiXMkVPYC/pnF4gp3+fUEMCzjCrt6GwheoyncEjqRurslJFTeEje1Uw80dmOe7Uc9T78Zrax+PE2kBS+rGn0T6fHbIOtSrPJqRIEMwt5G4dmhq1tTj2WpxMTqs2M7nOamxjj4XzMNrUefZ5TFQnmI5fmx6pt88HiHqnpMN9DW3L7VUIVNZfN2fctb0UGjuF0UpuBtK2iaS95DKczEn3t7cjOPi98QLDlSrU82d843t1Hho9tiObGJ/ubqLBlOo/11U+qpzvhAkcRNYtgexxI/4oKs/4UkupzIN1/xRaCQYneS+XVAdjOZgL4Gpn7bzuuGr0Ki9MBreZL1eqG6P/JjMXpwJy6zfZdT5ax/xkBDhg1zLllt0PRBAMxPdf296QdZFqADT1YQ1yj1B+PHm7siYN97UDqAddxSaEjNZLVKm6kW4ccYjoQaQftL0qLmI5i4HM58KFNZuwzqUc+uHlIO3YpjJgOQNrR24qIaffGDyli2j89fkiJFsS0a1OEdHVltBr6MAc1SUgnOtABmXnOHWLaq4lCSqTHu05hCtddbtXlXXiI6QyCmqzwzeVirGsVhibtZCFdtYb7vfFwNhOOLPN9MXsn5gspkVMTlYsDSHOWEq5cZILmOTXp2TIj3pIwKGmnnfkWVFC90OdWESxq3szz7zJ29PtgykWi2oLRf8Zt3k5RfZ1HOEHDnDrnMTsyad/g83ycGuQpQ0OT9ozKKcTc6o0qBkvT9v2rH7ykWmRs+qHn1/9Ac58MLli5cFRJyq3/4qKdslncEanqNgyD8OWPpre2rhfa9P9swTHlqhUu06W0H1Iig3ZUy1LN08N46SOkRHnSvbcuyIpTHePl50IaksiRvk56zPnN0quyoV5GT94rdzH7b78bw5zfKlarBRNy2dNUn4tZvTZuL6jz2yR9nXa2FOUBK6mypP3td39a+WfAm0SdXz5MquzUv+UvabV6M/fGxELQkFBlKRZ4eShSRW65RfTla7uWhvcd5sVrlLVRlYc4VSONLeX9kJJUM93o3YlUIT+BlwOBKgmLKm+F9CahafqLvXtdchOGoQAs38EGovd/2nZIOr0kgRALo8L5/u/uDDkrLBkTdbJ85UiB3xhoHyXyE0PfsmiglHHiqzB/4yf7TiFL5GepfqXeEajgg3RH22fmilIqlFFOVMFp38m9mEG6dUqB3yskzjt+JVZOkPGkqSJFunWKvOBG4lLYZZSQMI1SJMoWOd/xkomE9R3zPvtbFtv5ahTZTes+85JAsvzAT6ROahsUUzWs6BSqD8wNb/gmL9VsFNOzkC0YiVeMJGnYd5JQ+E7Nay0uS7ReJF5j9liRPutkl+wopsfqJUtp4lVEu7b2wq8PKliZqhAFS2niVZmk+MjvjRovD2goFonXWRJSOn7Pqrw+oKBWJG6YUhP4vVBUXiD4kpcrFYkbpjTxEkdyCnbzD+fEKkUfGqZ04CWZJEWcLTla4NnQKqRsZYKzyJCkHofzD5b4rkjv3VfMMav/1o1kWbx491id1A6+5U/tHlIuJMvwHU5AHcNIvQdy4I+VvUMaSVrGKySOFIXuZYk/N+0W0t1KXsIw6kCe71KbzmlLo1aMGd1oTL85pJHE+YBh1HFGnoU2ndND3vhAXheT3/KnCsmLOKd3nE5mDDXwJmb7GdRb+jikkXZQ+E7ZN1VcQi9TfSbexn4T/jzntNvwP6Dx/xm+roG2etG20bhtQfKQDcWKcaxM/0TQmkxPcOPNEj2UTVtM3YZfLcwH7D8dYxKZr0z8hVjmjLpAyyxv5WkfESPTj+i88D7zVzprO2aWXk1E2onByPQYIjexgaush2ObifaSMTI9wiTxQFrPdWhN5E1oNwPeE3mEKDFcsVxHbIKwe4Z6viNoKQgMqieuRKtcxYxLVEaX/wmFN/zMlYTHsYX2M6DLb28QuOGPXItEi2mgHfUY7H9G2Q3MhxYpLSqWpT9lvMB8nb7K4LiabIfm6AVd7SZ8cbOOx5ZSll1XGNrThMf3WrP1Q2rHbVJaVDRPP+H4U2sC296hUUopHD/Tn92w/dSWqb97JW6VUltxCkDZKgm2cPWdQGYBoksLS/vq8ZKTtmx1u2H4cimlgIVpU/ULuZu+lA60s4iFaUumuvgU1pdSRztLWJi25Ko/VnfFlPaYmLZ0q94ezVdMKQV8FcR7+q624Uum9Iat/HZK9Z0r8t9yHI0xYwznTqnDO/bbmXgWpfadcqJfUj5zSg2OlbTjap9snxbSMZw4pR5z/RWaZvqRfwuG/pbOm1LKaJ+aCbUz/bz4JXXpvClF+9SMr22e+sWQEt1Om1K0T4tU9QBu5cFjH86aUoPdp1bG2g/Vrh0tHs6a0oKvK2llqNx58qvH4cpZU0p4d0Qrlmd99Rwqe3ojnzWlFk1+I7myHrj103C3s6Y0oslvpHY0bdef53RnTanDI6Zt9LUdAN8Ff8GUThhFtWF4NtROSxNdMKUYRS1RNIhKPLN0xZR6jKLacDybKgdZ5pIpJTxvskBRm2of1eSaKe0wMG3C8qzUPatSLppSi5S+ou06+0f7oC+lkd7CU1H/m8Czui63KEyppQYcUtpE5Uea7hv4F08pXq//D2UpdfyTuWxKR4z139Oz9TTMP3zZlBqkdImWrSfLzAkpJfiDustsmTNdPqXYIv2HspQG5vHCKcXT+q8pSykz+wunlJDSRUqOQTJHQkqR0pfUjKWZDVKKlP5LV0oNZ0JKkdJ/aUvpiJQipU+0pbQgpUjpE2UptYSUIqVPlKU0IaVI6TNdKe09UoqUPtOVUiKkFCl9ASlFSq8GKUVK9UNKkVL9Xh+JQEqRUk3eHIlASpFSRQxSipT+YO9esNyEYSiACix/sPlo/6vtOXDaSUNmQmsBgry7hORhW7LB5iGlVTqk9DWk1FBK8d7Tty734g5SCva/bYiUgv3l/31T6mQ2EOzq9c15SKmhTRH45ooSpBQptaRFSjUuy4JHl/ydT0mp0GuXe8bhmzkLKd3II6U/utLFWvdNKbaefnSpi7Vum1J8JuqdC20+3Taljcx6grWrbT7dNqX4FPRRZObpL0ipnTPk8N3FWp+UUh7of7Vo6v/sSlcWmU4pJ2f5CYfvWlGflFJO0uAaUvPiiwrgc1LKSQQn9e1r1j/156SUk8hk+QGH74v8T0kpJxEZLC+W4O8SgOnLh6R0DqkUlPgX8KLI/4yULiHNOGtyBetrSj4jpZwqtzeLzBLB/hqZTfTHR6S0S7Una0dcnHccXpdPH5DSzlc3O3u8mnegJPJURdw/pZ2XxWD7NQf4LawaKrdPaZRZ3bMpKJ4OFFfrq7un1Mlvk+ntEPjSrVZo904pB/ljtNwbgUf+efK7dUqX4r7+yWxxuPRQ0/PC9M4pHb3MqgdCWTDBIYbnhel9U8pBHviKiDXo6b91qYWpakon2YhprcmyqH8ue7yZd7D81Pmzm1JuZavmzUBauRJPOGpysFDR4VZMKdM7XZbN0lMG2XkRtaG0oFt6tFixxlJMqduygN7OD/SlOC/PSv1PNhEchSvqVcWUeq6a7dfyUJaIxknWAlWYsIl/uFTRi1JJ6YaRafTyH3zbtvJaUXiuC8FhhlVKjk7pu6GptKKspwoj+lDHK7tO+U42i/QSOy/KPFOFgD7UCfKeU76TypjGLOoGquFxau8E/Z5TvqubiGMWfZlqjPhexBm6iilfN6WSIz0ovZc9NFRjwoS/2VWmfCf/JvdLhrjpk+xjohosmPBP0e94preV/9A+BdRQ6UQRE/45yo4NwCTWRKrSYsI/Sdrt6HkRa1q7TzT8aNhtFnNijGeq0qOlfxaWxUjKihdjRqqTsYd/mmmflimbW5UGqjPiXZLzjEqrrWEaOvojmhtJE1OdFl/eOVFWqp9aEd/2rmmaIWSxxnfVxSBO6Z/IKX0wiu1F80GkSgHN0jMVrf+xMzfLf+mpEnvUTqcKWh2WKFa1Wh07j9rpJI0smvu1SDUqp0VG7XSypLaZH8QiX6hWxL7T2aIsujvu3c/lfbWML+2dLsss3LGZL9JYWhRBbWkg5Y4xjVSvxRb++dh/DaZ3a5tGlfoSF5EZ4B4G03u1TSMpaNHRt0BzMKXO0GgaCUPpfThZdBc4D+UPDillDKU2sJdZa//YXuB0VAtqETGUWuFk0ViPaSDq/NaQYii9F/aPzRa7u1Bhc4GWCqkYMJTa4WQRTb/2FGg2ylsT23x8QaPPmZl0RC/qwubjVz0p6bHtZEl8/DSzzY5U2DpU+0hKimAH35Skfe6HJ1EVti58U0daWnx1x5ZGf9gYRFHYerdOz6RlFJwrNWbSfze/S6IlbOx2+ZHUcMYRfWuKLDKTGu71QropphOTnl5wN645TrdAXjRZP6SLzqsOpGudoKFvD+c9agV2KiHdEtPApCmhC2VRI7u0sEurHtJX3f3caE8tuIPMpGmnhdiYlUO67u77gXR1gtLJJvZ79QedVw7pwq3aT2qS4EMRRg0Kr7ArX94UNhxrCYW0Oew62dU+1PkmchroZ9Mqo7rzPV7BN6j4HQtbdlk7pMSukD7OmO8tG/YtGmKqCOlxAuZ729qd/6AuePMhjZjvjWO/92THsbUd0s7jgL51o+y/51KGZDeknNDPt68/pJ9dhslmSCnIIqOfbxino2qHsU/2QhoF+/dX0Hk57DZDblzrLYW0E5zX+8XOnSC5DQJRGDarAG3v/qdNUklsVeyZeMaLBPzfAabGpWfR3WDqMOi97cNoXLThECH1gSFULaL2+MFPNoNzk/1pOu3EJ4rSeqROJ4ZR/B6vHued0tTVO2USk9KaGHVYnw27l8Xgif2HkbhvpzLxcp1NH3Kgc6qP7atIO4c00DlVxKeeYuoDe05V8qGfmPb1lWxKPsd0ObXNJzZGq5XVR63mU38TjYYMXcSUkFauh5j61OMWRlMGtd5W+KQut4Ob0npMCWkTXNMxzYWQNmGSmr1EIQdC2ohh/zP0L2IIaTs2MW3qYQ4ipA1p83E6/WUb+lQdu8S0NDM4jc0WMt1agv4IbWzq+0RI25NDU61+DuKASYNyaaiHGtT0FLhjmzUyVV6cRjVWv+DMbx5uzW+gMTXYC+LM6WyqdtVfQpNzNZwNofZV308HuI4KL7XtoUKNvX5OampWgZv8qoq3bJzUwblunE5OtXbIo635G4YvMUFna0UPew5ilN8Pb1XfTCpv/2muhujBrAtbwyWn3onVvjs56cId/qmborpnE3h47qhy7C5qXKXa57z4HlPqWPa9C6Jt6pafJB3+oNRQeJH2zSRdhCOWp6aIirR7LujAUyljVUdVgtcarTbKkXJqrOrp8PBaSzlkTo2VqpqW4bVc0EY5QiD+yejKYg8ftRXczqEYirYsG6K4Lk+luF8wvCs6ZA2C/RkrbaXBn3Zgosgo7s9piPn0Xn4uko4+w8WernOqNL8xJcsq6XB9HI7nKqda37Py5xjEWo87jVFvD2qeiv6RyCg+M7pwHdR5PL3IEoOkw4wYUI0h6Uqanh+dPK/65VjjWtQix6Br1j0vqeMQi65Zlnrczw9Wt1i3+NODzLwG3VB4jeKrxrnoprK65Xt58maOSTcFqlE82n5fs9Et5v6/ZNxkgz4QIifz8IDskj5j7eRmY8wH2TSDc/Z3PIkoXmicV90l2Iuiu6SJHzPhWcyU9GwlDrRLeC6/TJaEogJmXoseE6xbOEiCF/NmjlbfkVa38ArF+3gzu9UG3SXZyRn6JOxlNMY5t1prk7aK/WlybjGM6wEAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAIAf7NFBCsMwEANAe1NSTB3w/1/bcyCUddnjzAMkkAAAAAAAAAAAAAAAAACAf8x4Vhg12l0kzLZjJJtHVDiP4+j9lRuk3mybPvHLVf90vVjPCqN6u1sZ77ahJ5v7qhTnNRKDVIu26ctO3a04EMJgGK6j8bd2vvu/2t0934x2QiGheU8FkfiQjKtofOCnv0UpNfVK/6L80q504LrsSm8rRU0WlAKgmFQrzVg0XOltpchGlAJUFCsN6wtd6X2lmFaUAmdSq/TAsuBK7ytFM6MUNSlVGnZudKUCpT2ZUYqadCqt2OjlSu8rxWlHKbJKpQU7dVcqUIppRymCRqUdWxVXKlCKYEdpV6h0Yq+eXKlAKSUzShHUKU2EzaIrFSjFYUdpVqc0YjdKrlSgFNGMUtKmNBG2i65UohQvK0rRlCnNeKPhSiVKaVhROnUpHXin7EolSlGtKH3qUprxVs2VSpTiaUTpoUppw/+d3L2uVKQU5cNKc1hWYoznQbiq7w5EVJN9R+ef5UpFSql9Vml87NZmB59gtsL2Z1LYk8OVrpV28NWkROlv04ZSdpXy7yqudKk0ZvBlPUovmDY9SgsrkR9Dd6VrpamCr+hR+qhgCnqUdhbi1TJ1pUulj0Fgo6ZH6dSvlF2ll3Og5Ep/2LeXJDthIIiipT9IfHL/q3V45klCyUZW2aG7AqJ1XvbrBl6VygVejWaUOvNK2b3R+nJlYSl9VyoBvNuM0mheaXj5lb7TMV1K35WKB++0olSsK6VT+npf6lhKFUpjAm9fSnUdr38dNZDaUvquVHbwUlxKFVGCVXGTPy+lCqVygueXUppCYFFIhltKFUolgxeW0j+YUhVlv5RqlMYNPGda6finTdofnENRWYZbShVKZU+gpfYfKFXXf6Xq+598TLelVKNUCnibAaWRfYRMKPXKN3MaSGUp1SiVA7xjvlL66JsFpRe/OOWY1qVUpVQ28K7pSk/2AbKgtKoFNZDOpVSlND5/NTX6TFQxoLTwKVWPaYpLqUapOPC2OFdpAanNVxprB6AGUlhKVUrlBC/PVMqRVpmvNPAp7RrTpVR3ADd4ZZ7Si9M75yuNqctPAykvpboDiBW0tI97h5QXQvbgpThfad+U8jFFW0p1B7An0LZo7338INOVNpBc7/bmpVR5AAW825zSGucr7b83H6jrpVR5ABm805jStMt0pYopVY+pX0q1B7CBt9tSWmS+0lshTj+mS6nyAFoCrUZDStMu85U6BTj9mNZ/QWm0oPSRlbej9I5iQKnXTKl+TMs/oNSZUCoBvGBE6e1EDChVTql+TJdS9QF48JwBpf5s/eP2+xWhVd2U6ocgLKX8h9Lxz/04XekWrn6lMqIC0h2eO/iP939UWkcolR08P13pz+rhDCit+LxgXmmR7jBEqRTwDgtKAdQyW2nA96VmXWkwo1QyeJcNpYCPU5XGhAFl60qz9OZGKY0baKkZUYot9im1P6UAmnGlVXo7RymVlkDbrCiFn6i0JQzpNq4UTTq7hymVC7xsRSmueUozBuWMKw3SV8M4pXKAV6wo9dOUNozKG1eaopB6ITW1Up4HLe1GlKJ1KP03phS4TCjluLL0tIMmHyiNj49E/6nS6l/aoKlMUuowrmpCaQDt7EGaxiqVHbz8N957iu4KHo/lSUo9BlYsKD3Bu5soO5+mrldp/4X6j5XyLg+eVyv9Z6YUqNGAUoen8qXZ0VDxkP9GqdzgDVDKuhJYaY7SDUMLBpRGvORDcTvz6Uq4E547PlIaNxNKn77caO/r/WGN3EEeUxr3QpfedoKy5H+tQlvpVcp5mFAqF/qVDjzWCt4/8XKsRumNwe1fKZViQyk/pDZB6YnhtflKT4wtyWdK5bChtIDk/r7SmDC8PF9pw9jyh0plM6E0GlIaQKq+tw2sNl2peAzNfak0JgtKpZpRGhPGP8YOP1/phZFV+VKpOBNKvRmlmcPqr4Dlpiv9wd6d7TYSQlEUvcxQ4/3/r+10px9MYjBUgQzyWY8ZkdlCiHIIee7Ita2U1AiV2lEqXXNf03IxfX+lgvvx1LhSOgeoVI1SaW4pbbqYHm+vlDbuJjSvVC+olP4LuS9pupgu769Ue+7EUfNKKRhU+uoXSKLGi6l7e6XdMt2oQ6XkUOmrrZro8Hbjt1faKdOdulRKFpX2WErz4av3V9pjb2oO6lQpeVRacHDUejEdoFISnpuymrpVuhpUmj836rCYbiNUSuR8w0ZXSmqQyoFKeyyl+UGvQ1RKFLaFG/C7powWqaiPr1SbzFLaZTG1g1T6JSjJdyzWrZTXJBX56ZWqdrNeOuowTKV/Baek51pGWiU0RfpVqpfPrjS7lHZaTOVQlX7TQjilNvmX4ae8/HIqpQ4RCAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAA+mPimCT4d15K5e0eWdFIFl5xooZSUMvmPL61ygX5z6lH+7hBTPUAt6uBejy9jV8rn5UoPu3CJ0+nshUv5SlnWDlDUv0AweKW8X6pUK8PFzKavV8oKlc6ndaUsLlR6GK5iwvVK+UCl02leqdHVlTquZcL1Ss2KSmfTvFKWtZU6rrdcr5Q9Kp1N+0p5q6t0NXyBu14pW1Q6mQ6V8lFVqeUr/I1K2aHSufSo1KwVlWq+Zr1RqQmodCo9KmVfUanja8SNStlrVDoTfrSo11xBpWzLKz2fNCQ3Fdnkwj+pO5WyRaUzufwCK85xxZUajp0HPaOdaVkp79cqFQQlxqxUccyEwgg0xw5KWT0/snWV7vxDQKXzaFSpUBzz+koEvFFaHLSsq1TsHFs0Kp1Gq0rp5Jgti8DF6VCOvFMpWY6dqHQazSrVC8f2oghU/PGOlWrPMYVKZ9GsUgqGY6G+0oNynHrgSirND1Cg0km0q5RcYueXr/RyBNWV0sExo1HpHLiOzOVlOSYLIjj5ke5aKW0c89WVWvWCIyAauVLK7fzKDjv7VkqSY1tBpTjTf7+mla6GY+JOpYqzxIVKteHYgUpn0LRSEhwzeqxKKXDMrKh0Aq0qTZTlB6uUdo55jUrH17DS5zu/wSolyzGLSsfXulK9cOwYrFLtOeZQ6fAaVpra+Y1V6a/DfRNQ6eiaV0qOY16PVemTAaLSwfEjI1/ZMpUmd36DVUobx05UOriGT0jzO784gkTSa8dKH3iO7cWV7rglqsj4ldL6bOdX/xx/FT+1qlQbjgU8xx9aj0rp4JjXhZU6ympQaerpAyodWZdKSXHsLKzUdq80cbgvUenI+lRKkmN7KoKDI4EyQrtK6eSYQqUD61SpXjgWiiJgrynNxjneqlR7jglUOq7GlaYP9wv/hvRcKWXjBpWmD/dR6biqb41QoqhSchxLREAL/3Aq8btUcWwLR5b6SvMDrDiJylsJGuNqqqxSsmWVWk6Q//nEp+9WSlthpbUUwTSVal80kQdfo25XSh6VTqJfpbSaook0fEm4X6k2qHQOHSulo2giVc/7S/MDFKh0Dj0rpa1kIvXCF4gWldKOSqfQtVKSJRMZDFez1KRSOlHpDPpWqk3qZ9zL1FKjSvWCSifQt1IKRRO5ytohtKqUgkGl4+tcKe1lEylOLmW2ldpVSg6Vjq93pWQLJ1I76/kluR0UuV0pWVQ6PFFtffYeZU0puuYhYhC7Uqf8Z+Fvi/xHqV2ExDc9SvxiXfcSrKnx4wkpAAAAAAAAAMCf9uCABAAAAEDQ/9ftCFQAAICjAKAG6mKYnP9DAAAAAElFTkSuQmCC"
				, "data:image/png; base64,iVBORw0KGgoAAAANSUhEUgAAADEAAAAwCAYAAAC4wJK5AAAACXBIWXMAAAsTAAALEwEAmpwYAAAAAXNSR0IArs4c6QAAAARnQU1BAACxjwv8YQUAAAV7SURBVHgB7VlNcts2FH6AJGfqdKbaplUd+gR1ThB5l8SdqXICOyewfALbJ7BzAts3UGdSuzurJ7BzgsCuO96yi6iNKOHlARRJgCREipKcLPKtSIIA3gfg/QLgG74OMFgwvA8dD2qfNoCzJo3+NG5AuAHkAkb1a7He82GBmJsECd2ER6NtQNlG5G2StlncC68ZsGtAdiZ+fteHOVGZhHdPKz4OdhHYTjnBnRAkxIFo/XEGFTEzCb3y9dEBMtx1/OJrwRgTgGgeGw8BFNkNR7/KZGYi4f39axs5ntD59lJNPpE65hz74snFX1PH+NBuwsrqcwnQYah2MSPQKQSNQ9IbASVRmgQRUKu/b3+ls13DbpHgzjHvX3gg+TbpUjd1JAULGptliZQikUPARy4Pbn+8eAsLgCIjJd9P7YzPADdF6/y6qH8hiSwBpFWStEp/ClgwvLutLunNkfGpFBE226B0fIIBEegv1M7bc3Y2EEaXxvEqPFrcORiZ0IcmoCBaPfIh9U1a32geDxvBybQ+ThIog0vjTR2h18smECEkIveMT23v7mXX9X/ucfLuXu2QE4vZs2C8vgwdKMLa3dYRCRgJ77Pg43reQubuBDK2nzzjwZcgoMDJXwBDMXltypXVXAebIaF2IXFmKPhQVg4HyoCs3/HTuy0kI3KUblOBIkM4jN7JBHe1s0whQ8LeBThd5i5MzLdeXUyOjQUyr6fmbsCj77bT/1gklHkzQ4pl7oK2fob/QcBT1786FIn+k7yTbrd3go+eG2/9Ze2CCiLT1o8Hgz1nh+GKGRm000fKJoHQjh4Zlz1YFlaCI3PHwwjAbb51EpUcKVABpNlukUBEL35hUBizVIGOAhB24jlLWj9KovrGqxXOpxU7afz/v/ewYIRRQGI4aHV7tz+dH5bqzFiyqMg8s4knE7ywGpbhnUM9iGIiigKGcq90Zyn/jccBO5+p5/cwzt+CoPyBeVwpD9kRrRkMhyoykNPIQx1mFeb2ZQd5jUIS9MsmLmEYk6SzSg9uKiZSeXAEgMxz9qjVtidHQkWXl7pEMwWhP2BmQtUvrQeWSNJzNcUkxBN7a/Pcu8YYTAdYSETrgRHGUDD5BqqA8x/iZ2YVIFI7Ydri+ve5VQmx9q5H2ZYpiJOICissfyBxr7IDRYzlIQ9umX+LBAOeNNbGv4ADKp4pIqJ1xwwrlD9Yu6jsQMm3mIvqJkEs+nGnnBjFxDQiWg9q3MwKRSU9iAa+14uTkBhKy4fZJGTdtBhtp15M4CJClcETWw/kJsyD8bBtvGViOm4LpdLCZDdcSYjdJ4cIGDEYsLmTKtO6mRFthKyJ5Sw+t64kJI0cIuHkII/nqbEq2EkaIRhn/EuWxKf6mVFpaELj8T6UQJYI6UHronxYkQMdsptJGuUcebuaIaFTQsDDpCN0VQ0WSsAg4s+tBwpUuDZ2weeBzDUOzuIZ5b1XkFgEUs6Pz8oGhSqYTDvPWZGuuCgT7bJwzD1IuhL3MMUz19w3rfNnrv+dxbPQUqHBnG1gY/WyjKLPgxwCunA3rQ+f1kiW5ZjRNiZfFJHHV0VBX1WEtd/gyqjDkn7C6yITXa60/8+rfUR2YHyiweWhaF0cwwKgb59WRieUb3TsORZU2o8nyhJRmOu+bXJ11qX61q59yTLb9UFpEnpSfd0lT9I5LoRkevT996LbUC14Y9ihcspvqKsrqUtLyrvZcPBmFgMyEwkthLrVGdd2maNiF4IsGQNBpHxDuCYqk51dgKiP0KF6hUh3ZhIRHFdUFYBClUv5cPC2qvmuTCKCNrmN1cnxwDaE17wFs5LgiD1eg55YQK49N4k0yNNuQA0plUR1M+qFs3Cq4I18GPP3EAzEQ13WfMND4zMnXf+pw+Ue3wAAAABJRU5ErkJggg=="
				);

			return httpPageSuccess;
		}

		private static string GetFailurePage(LoginResult loginResult, string? generalErrorMessage = null)
		{
			generalErrorMessage ??= "";
			string httpPageFailure = String.Format("<html><head></head><body style=\"background-color: black; color: white; \"><div style=\"text-align: center; \"><img style=\"object-fit:contain; width: 40 %; height: 30 %; opacity: 0.5; margin-top: 64px; \" alt=\"Unreal Engine Logo\" src=\"{0}\" /></div><br /><br /><h1 style=\"margin:1%; text-align: center;font-family: sans-serif;font-style: normal;font-weight: 400;font-size: 25.236px;line-height: 36px;color: #EF3535;\"><img style=\"object-fit:contain; vertical-align:middle; padding-right: 10px;\" alt=\"Error:&nbsp;\" src=\"{1}\" />An error occurred while logging you in: {2}<h1><h3 style=\"margin:1%; text-align: center;font-family: sans-serif;font-style: normal;font-weight: 400;font-size: 16px;line-height: 24px;color: #F3F6F8;\">{3}</h3></body></html>"
				, "data:image/png; base64,iVBORw0KGgoAAAANSUhEUgAAAqUAAAIhCAMAAABAJzPOAAAAM1BMVEUAAAD///////////////////////////////////////////////////////////////+3leKCAAAAEHRSTlMAQMCAEPCgYNAw4CCQcFCwJwL4owAAHPRJREFUeNrs3dl62jAQBeDRvlnL+z9t268UbIxpAsjRcv77AI4PkmZkGwIAAAAAAAAAAAAA+KjAdgigAY4ZsXBuyxHFuRYCgYWfEJjQXJVv4FkYhBVOwqLm5VU+CyYJoJ5gFlve53PEqAo1uJhV+SAukFT4JGm0LxXw6AjgA1y05X8853wRdzjnX/hLnQjgLW7x5ZjlQiTm6BnJWBT5WT9AaYOCCipElC+RBfoWZgQ/fL2MERVeEKI/6NOLFOhVkol88Loa1RR8j+GPF5HG0ftkWh6/vAgE8DVhUQ8T+tEMpcVi5odXJX5Wy0garco9H1FKwXNyvxpVuWYJzpb9G2pM/HAsCLVLTKIDNVsJGZUUPBb0LixnNTLdbiXMkVPYC/pnF4gp3+fUEMCzjCrt6GwheoyncEjqRurslJFTeEje1Uw80dmOe7Uc9T78Zrax+PE2kBS+rGn0T6fHbIOtSrPJqRIEMwt5G4dmhq1tTj2WpxMTqs2M7nOamxjj4XzMNrUefZ5TFQnmI5fmx6pt88HiHqnpMN9DW3L7VUIVNZfN2fctb0UGjuF0UpuBtK2iaS95DKczEn3t7cjOPi98QLDlSrU82d843t1Hho9tiObGJ/ubqLBlOo/11U+qpzvhAkcRNYtgexxI/4oKs/4UkupzIN1/xRaCQYneS+XVAdjOZgL4Gpn7bzuuGr0Ki9MBreZL1eqG6P/JjMXpwJy6zfZdT5ax/xkBDhg1zLllt0PRBAMxPdf296QdZFqADT1YQ1yj1B+PHm7siYN97UDqAddxSaEjNZLVKm6kW4ccYjoQaQftL0qLmI5i4HM58KFNZuwzqUc+uHlIO3YpjJgOQNrR24qIaffGDyli2j89fkiJFsS0a1OEdHVltBr6MAc1SUgnOtABmXnOHWLaq4lCSqTHu05hCtddbtXlXXiI6QyCmqzwzeVirGsVhibtZCFdtYb7vfFwNhOOLPN9MXsn5gspkVMTlYsDSHOWEq5cZILmOTXp2TIj3pIwKGmnnfkWVFC90OdWESxq3szz7zJ29PtgykWi2oLRf8Zt3k5RfZ1HOEHDnDrnMTsyad/g83ycGuQpQ0OT9ozKKcTc6o0qBkvT9v2rH7ykWmRs+qHn1/9Ac58MLli5cFRJyq3/4qKdslncEanqNgyD8OWPpre2rhfa9P9swTHlqhUu06W0H1Iig3ZUy1LN08N46SOkRHnSvbcuyIpTHePl50IaksiRvk56zPnN0quyoV5GT94rdzH7b78bw5zfKlarBRNy2dNUn4tZvTZuL6jz2yR9nXa2FOUBK6mypP3td39a+WfAm0SdXz5MquzUv+UvabV6M/fGxELQkFBlKRZ4eShSRW65RfTla7uWhvcd5sVrlLVRlYc4VSONLeX9kJJUM93o3YlUIT+BlwOBKgmLKm+F9CahafqLvXtdchOGoQAs38EGovd/2nZIOr0kgRALo8L5/u/uDDkrLBkTdbJ85UiB3xhoHyXyE0PfsmiglHHiqzB/4yf7TiFL5GepfqXeEajgg3RH22fmilIqlFFOVMFp38m9mEG6dUqB3yskzjt+JVZOkPGkqSJFunWKvOBG4lLYZZSQMI1SJMoWOd/xkomE9R3zPvtbFtv5ahTZTes+85JAsvzAT6ROahsUUzWs6BSqD8wNb/gmL9VsFNOzkC0YiVeMJGnYd5JQ+E7Nay0uS7ReJF5j9liRPutkl+wopsfqJUtp4lVEu7b2wq8PKliZqhAFS2niVZmk+MjvjRovD2goFonXWRJSOn7Pqrw+oKBWJG6YUhP4vVBUXiD4kpcrFYkbpjTxEkdyCnbzD+fEKkUfGqZ04CWZJEWcLTla4NnQKqRsZYKzyJCkHofzD5b4rkjv3VfMMav/1o1kWbx491id1A6+5U/tHlIuJMvwHU5AHcNIvQdy4I+VvUMaSVrGKySOFIXuZYk/N+0W0t1KXsIw6kCe71KbzmlLo1aMGd1oTL85pJHE+YBh1HFGnoU2ndND3vhAXheT3/KnCsmLOKd3nE5mDDXwJmb7GdRb+jikkXZQ+E7ZN1VcQi9TfSbexn4T/jzntNvwP6Dx/xm+roG2etG20bhtQfKQDcWKcaxM/0TQmkxPcOPNEj2UTVtM3YZfLcwH7D8dYxKZr0z8hVjmjLpAyyxv5WkfESPTj+i88D7zVzprO2aWXk1E2onByPQYIjexgaush2ObifaSMTI9wiTxQFrPdWhN5E1oNwPeE3mEKDFcsVxHbIKwe4Z6viNoKQgMqieuRKtcxYxLVEaX/wmFN/zMlYTHsYX2M6DLb28QuOGPXItEi2mgHfUY7H9G2Q3MhxYpLSqWpT9lvMB8nb7K4LiabIfm6AVd7SZ8cbOOx5ZSll1XGNrThMf3WrP1Q2rHbVJaVDRPP+H4U2sC296hUUopHD/Tn92w/dSWqb97JW6VUltxCkDZKgm2cPWdQGYBoksLS/vq8ZKTtmx1u2H4cimlgIVpU/ULuZu+lA60s4iFaUumuvgU1pdSRztLWJi25Ko/VnfFlPaYmLZ0q94ezVdMKQV8FcR7+q624Uum9Iat/HZK9Z0r8t9yHI0xYwznTqnDO/bbmXgWpfadcqJfUj5zSg2OlbTjap9snxbSMZw4pR5z/RWaZvqRfwuG/pbOm1LKaJ+aCbUz/bz4JXXpvClF+9SMr22e+sWQEt1Om1K0T4tU9QBu5cFjH86aUoPdp1bG2g/Vrh0tHs6a0oKvK2llqNx58qvH4cpZU0p4d0Qrlmd99Rwqe3ojnzWlFk1+I7myHrj103C3s6Y0oslvpHY0bdef53RnTanDI6Zt9LUdAN8Ff8GUThhFtWF4NtROSxNdMKUYRS1RNIhKPLN0xZR6jKLacDybKgdZ5pIpJTxvskBRm2of1eSaKe0wMG3C8qzUPatSLppSi5S+ou06+0f7oC+lkd7CU1H/m8Czui63KEyppQYcUtpE5Uea7hv4F08pXq//D2UpdfyTuWxKR4z139Oz9TTMP3zZlBqkdImWrSfLzAkpJfiDustsmTNdPqXYIv2HspQG5vHCKcXT+q8pSykz+wunlJDSRUqOQTJHQkqR0pfUjKWZDVKKlP5LV0oNZ0JKkdJ/aUvpiJQipU+0pbQgpUjpE2UptYSUIqVPlKU0IaVI6TNdKe09UoqUPtOVUiKkFCl9ASlFSq8GKUVK9UNKkVL9Xh+JQEqRUk3eHIlASpFSRQxSipT+YO9esNyEYSiACix/sPlo/6vtOXDaSUNmQmsBgry7hORhW7LB5iGlVTqk9DWk1FBK8d7Tty734g5SCva/bYiUgv3l/31T6mQ2EOzq9c15SKmhTRH45ooSpBQptaRFSjUuy4JHl/ydT0mp0GuXe8bhmzkLKd3II6U/utLFWvdNKbaefnSpi7Vum1J8JuqdC20+3Taljcx6grWrbT7dNqX4FPRRZObpL0ipnTPk8N3FWp+UUh7of7Vo6v/sSlcWmU4pJ2f5CYfvWlGflFJO0uAaUvPiiwrgc1LKSQQn9e1r1j/156SUk8hk+QGH74v8T0kpJxEZLC+W4O8SgOnLh6R0DqkUlPgX8KLI/4yULiHNOGtyBetrSj4jpZwqtzeLzBLB/hqZTfTHR6S0S7Una0dcnHccXpdPH5DSzlc3O3u8mnegJPJURdw/pZ2XxWD7NQf4LawaKrdPaZRZ3bMpKJ4OFFfrq7un1Mlvk+ntEPjSrVZo904pB/ljtNwbgUf+efK7dUqX4r7+yWxxuPRQ0/PC9M4pHb3MqgdCWTDBIYbnhel9U8pBHviKiDXo6b91qYWpakon2YhprcmyqH8ue7yZd7D81Pmzm1JuZavmzUBauRJPOGpysFDR4VZMKdM7XZbN0lMG2XkRtaG0oFt6tFixxlJMqduygN7OD/SlOC/PSv1PNhEchSvqVcWUeq6a7dfyUJaIxknWAlWYsIl/uFTRi1JJ6YaRafTyH3zbtvJaUXiuC8FhhlVKjk7pu6GptKKspwoj+lDHK7tO+U42i/QSOy/KPFOFgD7UCfKeU76TypjGLOoGquFxau8E/Z5TvqubiGMWfZlqjPhexBm6iilfN6WSIz0ovZc9NFRjwoS/2VWmfCf/JvdLhrjpk+xjohosmPBP0e94preV/9A+BdRQ6UQRE/45yo4NwCTWRKrSYsI/Sdrt6HkRa1q7TzT8aNhtFnNijGeq0qOlfxaWxUjKihdjRqqTsYd/mmmflimbW5UGqjPiXZLzjEqrrWEaOvojmhtJE1OdFl/eOVFWqp9aEd/2rmmaIWSxxnfVxSBO6Z/IKX0wiu1F80GkSgHN0jMVrf+xMzfLf+mpEnvUTqcKWh2WKFa1Wh07j9rpJI0smvu1SDUqp0VG7XSypLaZH8QiX6hWxL7T2aIsujvu3c/lfbWML+2dLsss3LGZL9JYWhRBbWkg5Y4xjVSvxRb++dh/DaZ3a5tGlfoSF5EZ4B4G03u1TSMpaNHRt0BzMKXO0GgaCUPpfThZdBc4D+UPDillDKU2sJdZa//YXuB0VAtqETGUWuFk0ViPaSDq/NaQYii9F/aPzRa7u1Bhc4GWCqkYMJTa4WQRTb/2FGg2ylsT23x8QaPPmZl0RC/qwubjVz0p6bHtZEl8/DSzzY5U2DpU+0hKimAH35Skfe6HJ1EVti58U0daWnx1x5ZGf9gYRFHYerdOz6RlFJwrNWbSfze/S6IlbOx2+ZHUcMYRfWuKLDKTGu71QropphOTnl5wN645TrdAXjRZP6SLzqsOpGudoKFvD+c9agV2KiHdEtPApCmhC2VRI7u0sEurHtJX3f3caE8tuIPMpGmnhdiYlUO67u77gXR1gtLJJvZ79QedVw7pwq3aT2qS4EMRRg0Kr7ArX94UNhxrCYW0Oew62dU+1PkmchroZ9Mqo7rzPV7BN6j4HQtbdlk7pMSukD7OmO8tG/YtGmKqCOlxAuZ729qd/6AuePMhjZjvjWO/92THsbUd0s7jgL51o+y/51KGZDeknNDPt68/pJ9dhslmSCnIIqOfbxino2qHsU/2QhoF+/dX0Hk57DZDblzrLYW0E5zX+8XOnSC5DQJRGDarAG3v/qdNUklsVeyZeMaLBPzfAabGpWfR3WDqMOi97cNoXLThECH1gSFULaL2+MFPNoNzk/1pOu3EJ4rSeqROJ4ZR/B6vHued0tTVO2USk9KaGHVYnw27l8Xgif2HkbhvpzLxcp1NH3Kgc6qP7atIO4c00DlVxKeeYuoDe05V8qGfmPb1lWxKPsd0ObXNJzZGq5XVR63mU38TjYYMXcSUkFauh5j61OMWRlMGtd5W+KQut4Ob0npMCWkTXNMxzYWQNmGSmr1EIQdC2ohh/zP0L2IIaTs2MW3qYQ4ipA1p83E6/WUb+lQdu8S0NDM4jc0WMt1agv4IbWzq+0RI25NDU61+DuKASYNyaaiHGtT0FLhjmzUyVV6cRjVWv+DMbx5uzW+gMTXYC+LM6WyqdtVfQpNzNZwNofZV308HuI4KL7XtoUKNvX5OampWgZv8qoq3bJzUwblunE5OtXbIo635G4YvMUFna0UPew5ilN8Pb1XfTCpv/2muhujBrAtbwyWn3onVvjs56cId/qmborpnE3h47qhy7C5qXKXa57z4HlPqWPa9C6Jt6pafJB3+oNRQeJH2zSRdhCOWp6aIirR7LujAUyljVUdVgtcarTbKkXJqrOrp8PBaSzlkTo2VqpqW4bVc0EY5QiD+yejKYg8ftRXczqEYirYsG6K4Lk+luF8wvCs6ZA2C/RkrbaXBn3Zgosgo7s9piPn0Xn4uko4+w8WernOqNL8xJcsq6XB9HI7nKqda37Py5xjEWo87jVFvD2qeiv6RyCg+M7pwHdR5PL3IEoOkw4wYUI0h6Uqanh+dPK/65VjjWtQix6Br1j0vqeMQi65Zlnrczw9Wt1i3+NODzLwG3VB4jeKrxrnoprK65Xt58maOSTcFqlE82n5fs9Et5v6/ZNxkgz4QIifz8IDskj5j7eRmY8wH2TSDc/Z3PIkoXmicV90l2Iuiu6SJHzPhWcyU9GwlDrRLeC6/TJaEogJmXoseE6xbOEiCF/NmjlbfkVa38ArF+3gzu9UG3SXZyRn6JOxlNMY5t1prk7aK/WlybjGM6wEAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAIAf7NFBCsMwEANAe1NSTB3w/1/bcyCUddnjzAMkkAAAAAAAAAAAAAAAAACAf8x4Vhg12l0kzLZjJJtHVDiP4+j9lRuk3mybPvHLVf90vVjPCqN6u1sZ77ahJ5v7qhTnNRKDVIu26ctO3a04EMJgGK6j8bd2vvu/2t0934x2QiGheU8FkfiQjKtofOCnv0UpNfVK/6L80q504LrsSm8rRU0WlAKgmFQrzVg0XOltpchGlAJUFCsN6wtd6X2lmFaUAmdSq/TAsuBK7ytFM6MUNSlVGnZudKUCpT2ZUYqadCqt2OjlSu8rxWlHKbJKpQU7dVcqUIppRymCRqUdWxVXKlCKYEdpV6h0Yq+eXKlAKSUzShHUKU2EzaIrFSjFYUdpVqc0YjdKrlSgFNGMUtKmNBG2i65UohQvK0rRlCnNeKPhSiVKaVhROnUpHXin7EolSlGtKH3qUprxVs2VSpTiaUTpoUppw/+d3L2uVKQU5cNKc1hWYoznQbiq7w5EVJN9R+ef5UpFSql9Vml87NZmB59gtsL2Z1LYk8OVrpV28NWkROlv04ZSdpXy7yqudKk0ZvBlPUovmDY9SgsrkR9Dd6VrpamCr+hR+qhgCnqUdhbi1TJ1pUulj0Fgo6ZH6dSvlF2ll3Og5Ep/2LeXJDthIIiipT9IfHL/q3V45klCyUZW2aG7AqJ1XvbrBl6VygVejWaUOvNK2b3R+nJlYSl9VyoBvNuM0mheaXj5lb7TMV1K35WKB++0olSsK6VT+npf6lhKFUpjAm9fSnUdr38dNZDaUvquVHbwUlxKFVGCVXGTPy+lCqVygueXUppCYFFIhltKFUolgxeW0j+YUhVlv5RqlMYNPGda6finTdofnENRWYZbShVKZU+gpfYfKFXXf6Xq+598TLelVKNUCnibAaWRfYRMKPXKN3MaSGUp1SiVA7xjvlL66JsFpRe/OOWY1qVUpVQ28K7pSk/2AbKgtKoFNZDOpVSlND5/NTX6TFQxoLTwKVWPaYpLqUapOPC2OFdpAanNVxprB6AGUlhKVUrlBC/PVMqRVpmvNPAp7RrTpVR3ADd4ZZ7Si9M75yuNqctPAykvpboDiBW0tI97h5QXQvbgpThfad+U8jFFW0p1B7An0LZo7338INOVNpBc7/bmpVR5AAW825zSGucr7b83H6jrpVR5ABm805jStMt0pYopVY+pX0q1B7CBt9tSWmS+0lshTj+mS6nyAFoCrUZDStMu85U6BTj9mNZ/QWm0oPSRlbej9I5iQKnXTKl+TMs/oNSZUCoBvGBE6e1EDChVTql+TJdS9QF48JwBpf5s/eP2+xWhVd2U6ocgLKX8h9Lxz/04XekWrn6lMqIC0h2eO/iP939UWkcolR08P13pz+rhDCit+LxgXmmR7jBEqRTwDgtKAdQyW2nA96VmXWkwo1QyeJcNpYCPU5XGhAFl60qz9OZGKY0baKkZUYot9im1P6UAmnGlVXo7RymVlkDbrCiFn6i0JQzpNq4UTTq7hymVC7xsRSmueUozBuWMKw3SV8M4pXKAV6wo9dOUNozKG1eaopB6ITW1Up4HLe1GlKJ1KP03phS4TCjluLL0tIMmHyiNj49E/6nS6l/aoKlMUuowrmpCaQDt7EGaxiqVHbz8N957iu4KHo/lSUo9BlYsKD3Bu5soO5+mrldp/4X6j5XyLg+eVyv9Z6YUqNGAUoen8qXZ0VDxkP9GqdzgDVDKuhJYaY7SDUMLBpRGvORDcTvz6Uq4E547PlIaNxNKn77caO/r/WGN3EEeUxr3QpfedoKy5H+tQlvpVcp5mFAqF/qVDjzWCt4/8XKsRumNwe1fKZViQyk/pDZB6YnhtflKT4wtyWdK5bChtIDk/r7SmDC8PF9pw9jyh0plM6E0GlIaQKq+tw2sNl2peAzNfak0JgtKpZpRGhPGP8YOP1/phZFV+VKpOBNKvRmlmcPqr4Dlpiv9wd6d7TYSQlEUvcxQ4/3/r+10px9MYjBUgQzyWY8ZkdlCiHIIee7Ita2U1AiV2lEqXXNf03IxfX+lgvvx1LhSOgeoVI1SaW4pbbqYHm+vlDbuJjSvVC+olP4LuS9pupgu769Ue+7EUfNKKRhU+uoXSKLGi6l7e6XdMt2oQ6XkUOmrrZro8Hbjt1faKdOdulRKFpX2WErz4av3V9pjb2oO6lQpeVRacHDUejEdoFISnpuymrpVuhpUmj836rCYbiNUSuR8w0ZXSmqQyoFKeyyl+UGvQ1RKFLaFG/C7powWqaiPr1SbzFLaZTG1g1T6JSjJdyzWrZTXJBX56ZWqdrNeOuowTKV/Baek51pGWiU0RfpVqpfPrjS7lHZaTOVQlX7TQjilNvmX4ae8/HIqpQ4RCAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAA+mPimCT4d15K5e0eWdFIFl5xooZSUMvmPL61ygX5z6lH+7hBTPUAt6uBejy9jV8rn5UoPu3CJ0+nshUv5SlnWDlDUv0AweKW8X6pUK8PFzKavV8oKlc6ndaUsLlR6GK5iwvVK+UCl02leqdHVlTquZcL1Ss2KSmfTvFKWtZU6rrdcr5Q9Kp1N+0p5q6t0NXyBu14pW1Q6mQ6V8lFVqeUr/I1K2aHSufSo1KwVlWq+Zr1RqQmodCo9KmVfUanja8SNStlrVDoTfrSo11xBpWzLKz2fNCQ3Fdnkwj+pO5WyRaUzufwCK85xxZUajp0HPaOdaVkp79cqFQQlxqxUccyEwgg0xw5KWT0/snWV7vxDQKXzaFSpUBzz+koEvFFaHLSsq1TsHFs0Kp1Gq0rp5Jgti8DF6VCOvFMpWY6dqHQazSrVC8f2oghU/PGOlWrPMYVKZ9GsUgqGY6G+0oNynHrgSirND1Cg0km0q5RcYueXr/RyBNWV0sExo1HpHLiOzOVlOSYLIjj5ke5aKW0c89WVWvWCIyAauVLK7fzKDjv7VkqSY1tBpTjTf7+mla6GY+JOpYqzxIVKteHYgUpn0LRSEhwzeqxKKXDMrKh0Aq0qTZTlB6uUdo55jUrH17DS5zu/wSolyzGLSsfXulK9cOwYrFLtOeZQ6fAaVpra+Y1V6a/DfRNQ6eiaV0qOY16PVemTAaLSwfEjI1/ZMpUmd36DVUobx05UOriGT0jzO784gkTSa8dKH3iO7cWV7rglqsj4ldL6bOdX/xx/FT+1qlQbjgU8xx9aj0rp4JjXhZU6ympQaerpAyodWZdKSXHsLKzUdq80cbgvUenI+lRKkmN7KoKDI4EyQrtK6eSYQqUD61SpXjgWiiJgrynNxjneqlR7jglUOq7GlaYP9wv/hvRcKWXjBpWmD/dR6biqb41QoqhSchxLREAL/3Aq8btUcWwLR5b6SvMDrDiJylsJGuNqqqxSsmWVWk6Q//nEp+9WSlthpbUUwTSVal80kQdfo25XSh6VTqJfpbSaook0fEm4X6k2qHQOHSulo2giVc/7S/MDFKh0Dj0rpa1kIvXCF4gWldKOSqfQtVKSJRMZDFez1KRSOlHpDPpWqk3qZ9zL1FKjSvWCSifQt1IKRRO5ytohtKqUgkGl4+tcKe1lEylOLmW2ldpVSg6Vjq93pWQLJ1I76/kluR0UuV0pWVQ6PFFtffYeZU0puuYhYhC7Uqf8Z+Fvi/xHqV2ExDc9SvxiXfcSrKnx4wkpAAAAAAAAAMCf9uCABAAAAEDQ/9ftCFQAAICjAKAG6mKYnP9DAAAAAElFTkSuQmCC"
				, "data:image/png; base64,iVBORw0KGgoAAAANSUhEUgAAADAAAAAwCAYAAABXAvmHAAAACXBIWXMAAAsTAAALEwEAmpwYAAAAAXNSR0IArs4c6QAAAARnQU1BAACxjwv8YQUAAAPfSURBVHgB1VpdctowEF5BeC7M9L3kBmSGMHmrc4JOThBygqQnKD1BkxOUnqC5Qdy3zJAfblDfwE4feMAYdyVkW/4DyVrS9JvRxF7LRPt9QrtawYAYvuN0YbG4gjg+x9u+NM+xudBu3/Tu7z0gBANC+MfHA2DsJw6+X9mBMQ+fnfVmszkQgcwB/+SkD+v1Xe3gMwTYTqmcaAEV1uuxMngPmwNh2OMtiqILaePoYr9vQAQSBUrsh+Fhbz73cn349AJ4Tg1xfNp7eHDBEjQK5NmfFgfPIaeMq5i+AAGsHRDsA5ynhjD8Wts5jrNnjDn+cOiAJewViCJnF/sJ5JRxFZO1CvYOMJYNYhv7CYhVsHIAv5jZ3I9jdxv7CahVsFNAZZ+xie5rlCo0dqDE/mz2S/ddShWaK9CU/QREKjRyIMc+RlgT9hNQqdBMAZX99XoCTVFUYRNTjGDsQIn9x8cflf1Gown29UUbDitzn5IKUWSsgrkCGuzjoK/RSd6vKxpjV+hE5eAw0VMJGJuqYOSALvugphYJ0Imqju+fnqaQZarGKpgpoD/3u5o2AVRBjeBGKmg7YMC+MaQKQWqIokvdd/UVoFp56j//Wrkb+4NBV+c1LQf2yX6K5fIGMhW6cHCgpYKeAvtmH4GJYJBTga9cGirsdOBV2E/QQIXdCqjsx/EU9ogmKmx1oMg+rFb7Yz+BoQrbFSiwr7NhsYWpCrUOFNiHV2E/gYEK9Qqo7O/YrFPDRIVKB8TmQmVfZ7Oeh1uyxLFZKVFThWoF7Nn/DGpqEMcBTsELg/e1VSg5ILd2TmowZ39ThQvDI8xprkRbrXip0byYu1yq37tKFUq1Ufzy3kHmwBQHY8QcNXA83/HPWN4Gsu6aqptTgIJ9cuTHUFIhP4X+4cpTBzEGxm5TQ6s1Vp+nDlCzz4+aeEmdooCLCeRNeo2r45/RKN3xZQoQsi/OAhYLfhbwjJ97hxv83/J8oBGKm/8IYJJcCwfkFs5J37Cd+8VzMn6NNt1NSiXUEoyiwkYBdSOtWaStQykIKv8UOp0+NESdCi3J/jjt2aRMqIKxYMvTd2CDggqcrBZOl4FidJuUCXMIQw/UMkkGz/azS4Uwxj61cFn6oPRxwRIyyJxB3gkPHTsDCiDJynX/AGsyL+12e2NgrA8EkAd6h7jyfAwBXjqoiho9raCOEacrEytDp+NLUwDt9hH1zwGoUDrOxdPRlmQmyfq6vIMaKN4CRFDkq1v+lwCiwCCSOakCDzx9+B+wSc9PeYYr4oBQgae/GIHh7WOeDJ7flNPpwYAHnEsZPZtHTkpwxvkvXQBui0vxX27MR88d8RHHAAAAAElFTkSuQmCC"
				, loginResult.ErrorDescription, generalErrorMessage
			);

			return httpPageFailure;
		}

#pragma warning disable CA1054
		public virtual Process? OpenBrowser(string url)
#pragma warning restore CA1054
		{
			_logger?.LogDebug("Opening browser.");
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return Process.Start(new ProcessStartInfo(url) { UseShellExecute = true });
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				List<string> browsers = ["xdg-open"];
				bool isWsl = IsRunningWsl();
				if (isWsl)
				{
					browsers.Add("wslview");
				}

				foreach (string browser in browsers)
				{
					try
					{
						return Process.Start(browser, url);
					}
					catch (Exception e)
					{
						_logger?.LogDebug("Failed to launch {Browser}", e);
					}
				}

				if (isWsl)
				{
					_logger?.LogInformation("Did not find a browser, use 'sudo apt install xdg-utils' or 'sudo apt install wslu'.");
				}
				else
				{
					_logger?.LogInformation("Did not find a browser, use 'sudo apt install xdg-utils'.");
				}
				return null;
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				return Process.Start("open", url);
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		private static bool IsRunningWsl()
		{
			string versionFile = "/proc/version";
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Linux) || !File.Exists(versionFile))
			{
				return false;
			}

			string version = File.ReadAllText(versionFile);
			return version.Contains("microsoft", StringComparison.InvariantCultureIgnoreCase) || version.Contains("wsl2", StringComparison.InvariantCultureIgnoreCase);
		}

		/// <summary>
		/// Performs the OIDC token refresh via HTTP. Virtual for testability.
		/// </summary>
		protected virtual async Task<TokenRefreshResult> DoRefreshAsync(string refreshToken, CancellationToken cancellationToken)
		{
			OidcClientOptions options = await BuildClientOptionsAsync(_redirectUris.First(), cancellationToken);
			OidcClient oidcClient = new(options);
			RefreshTokenResult r = await oidcClient.RefreshTokenAsync(refreshToken, cancellationToken: cancellationToken);
			return new TokenRefreshResult(r.IsError, r.Error, r.ErrorDescription, r.AccessToken, r.RefreshToken, r.AccessTokenExpiration);
		}

		/// <summary>
		/// Acquire machine-wide lock for the refresh operation. Virtual for testability.
		/// </summary>
		protected virtual async Task<IDisposable> AcquireRefreshLockAsync(CancellationToken cancellationToken)
		{
			using CancellationTokenSource timeoutCts = new(TimeSpan.FromSeconds(LockAcquireTimeoutSeconds));
			using CancellationTokenSource linkedCts = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken, timeoutCts.Token);

			Stopwatch sw = Stopwatch.StartNew();
			IDisposable handle;
			try
			{
				handle = await SingleInstanceMutex.AcquireAsync(@"Global\OidcRefreshLock", linkedCts.Token);
			}
			catch (OperationCanceledException) when (timeoutCts.IsCancellationRequested && !cancellationToken.IsCancellationRequested)
			{
				throw new TimeoutException($"Timed out after {LockAcquireTimeoutSeconds}s waiting to acquire OIDC refresh lock.");
			}

			_logger?.LogInformation("Acquired refresh lock for '{Name}' (waited {Elapsed}ms).", _name, sw.ElapsedMilliseconds);
			return handle;
		}

		private async Task<OidcTokenInfo?> TryDoRefreshTokenAsync(string refreshToken, CancellationToken cancellationToken)
		{
			try
			{
				_logger?.LogDebug("Starting OIDC refresh HTTP call for '{Name}' (token: {Fingerprint}).",
					_name, TokenUtils.Fingerprint(refreshToken));

				TokenRefreshResult result = await DoRefreshAsync(refreshToken, cancellationToken);
				_logger?.LogDebug("OIDC refresh HTTP call completed for '{Name}'. IsError={IsError}.", _name, result.IsError);

				// Fallback for non-cooperating processes (old binaries without new lock)
				if (result is { IsError: true, Error: "invalid_grant" })
				{
					_logger?.LogWarning("Refresh token for provider '{Name}' returned invalid_grant. May have been rotated by another process.", _name);

					if (_tokenStore.TryGetRefreshToken(_name, out string? freshToken) &&
					    !String.Equals(freshToken, refreshToken, StringComparison.Ordinal))
					{
						_logger?.LogInformation("Found rotated refresh token for provider '{Name}'. Retrying.", _name);
						result = await DoRefreshAsync(freshToken, cancellationToken);
					}
				}

				if (result.IsError)
				{
					if (result.Error == "invalid_grant")
					{
						_logger?.LogWarning("Refresh token for provider '{Name}' still invalid. User must re-authenticate.", _name);
						return null;
					}
					_logger?.LogWarning("Refresh token error for provider '{Name}': {Error} - {Description}", _name, result.Error, result.ErrorDescription);
					throw new Exception($"Error using the refresh token: {result.Error} , details: {result.ErrorDescription}");
				}

				// Persist new token to disk immediately
				if (String.IsNullOrEmpty(result.RefreshToken))
				{
					throw new Exception("No refresh token was provided in response to refresh.");
				}
				_logger?.LogDebug("Persisting new refresh token to store for '{Name}' (token: {Fingerprint}).",
					_name, TokenUtils.Fingerprint(result.RefreshToken));
				_tokenStore.AddRefreshToken(_name, result.RefreshToken);
				_logger?.LogDebug("Refresh token persisted successfully for '{Name}'.", _name);

				// Cache access token in memory (short-lived)
				_accessToken = result.AccessToken;
				_tokenExpiry = result.AccessTokenExpiration;

				_logger?.LogInformation("Successfully refreshed token for provider '{Name}'. New expiry: {Expiry}", _name, result.AccessTokenExpiration);

				return new OidcTokenInfo
				{
					RefreshToken = result.RefreshToken,
					AccessToken = result.AccessToken,
					TokenExpiry = result.AccessTokenExpiration
				};
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				_logger?.LogWarning(
					"OIDC refresh for '{Name}' timed out after {TimeoutSeconds}s (token: {Fingerprint}). "
					+ "If Okta already rotated the token, the old token on disk may now be invalid.",
					_name, RefreshHttpTimeoutSeconds, TokenUtils.Fingerprint(refreshToken));
				throw;
			}
		}

		private async Task<DiscoveryDocumentResponse> GetDiscoveryDocumentAsync(CancellationToken cancellationToken)
		{
			string baseUrl = _authorityUri.ToString().TrimEnd('/');
			string discoUrl = $"{baseUrl}/.well-known/openid-configuration";

			using HttpClient client = new();
			using DiscoveryDocumentRequest doc = new()
			{
				Address = discoUrl,
				Policy =
				{
					ValidateEndpoints = false,
					RequireHttps = _authorityUri.Scheme == "https"
				}
			};
			DiscoveryDocumentResponse disco = await client.GetDiscoveryDocumentAsync(doc, cancellationToken);

			if (disco.IsError)
			{
				throw new Exception(disco.Error);
			}

			return disco;
		}

		public async Task<IOidcTokenInfo> GetAccessTokenAsync(CancellationToken cancellationToken)
		{
			return await TryGetAccessTokenAsync(cancellationToken) ?? throw new NotLoggedInException();
		}

		public async Task<IOidcTokenInfo?> TryGetAccessTokenAsync(CancellationToken cancellationToken)
		{
			// Unlocked fast-path check; authoritative read happens inside lock below
			if (!_tokenStore.TryGetRefreshToken(_name, out string? refreshToken) || String.IsNullOrEmpty(refreshToken))
			{
				_logger?.LogDebug("No refresh token for provider '{Name}'.", _name);
				throw new NotLoggedInException();
			}

			// Check cached access token (short-lived, OK to cache)
			OidcTokenInfo tokenInfo = new()
			{
				RefreshToken = refreshToken,
				AccessToken = _accessToken,
				TokenExpiry = _tokenExpiry
			};

			if (tokenInfo.IsValid(DateTime.Now.AddMinutes(-2)))
			{
				return tokenInfo;
			}

			_logger?.LogInformation("Access token for provider '{Name}' expired or expiring. Attempting refresh.", _name);

			// Machine-wide lock - only one process refreshes at a time
			using IDisposable refreshLock = await AcquireRefreshLockAsync(cancellationToken);

			// Re-read from disk inside lock — another process/thread may have refreshed while we waited
			if (!_tokenStore.TryGetRefreshToken(_name, out refreshToken) || String.IsNullOrEmpty(refreshToken))
			{
				throw new NotLoggedInException();
			}

			// If another in-process caller refreshed while we waited for the lock, skip the redundant rotation.
			OidcTokenInfo freshCheck = new()
			{
				RefreshToken = refreshToken,
				AccessToken = _accessToken,
				TokenExpiry = _tokenExpiry
			};
			if (freshCheck.IsValid(DateTime.Now.AddMinutes(-2)))
			{
				_logger?.LogInformation("Access token for '{Name}' became valid while waiting for lock. Skipping refresh.", _name);
				return freshCheck;
			}

			// Once committed to refresh: only a timeout can cancel, NOT external cancellation.
			// Prevents losing a rotated refresh token if the caller cancels mid-HTTP-call (e.g., process shutdown via BackgroundTask disposal).
			if (cancellationToken.IsCancellationRequested)
			{
				_logger?.LogInformation("Cancellation requested before starting refresh for '{Name}'. Releasing lock without refreshing.", _name);
				cancellationToken.ThrowIfCancellationRequested();
			}

			_logger?.LogInformation("Starting refresh for '{Name}' with timeout-only cancellation ({Timeout}s)", _name, RefreshHttpTimeoutSeconds);

			using CancellationTokenSource refreshCts = new();
			refreshCts.CancelAfter(TimeSpan.FromSeconds(RefreshHttpTimeoutSeconds));

			return await TryDoRefreshTokenAsync(refreshToken, refreshCts.Token);
		}

		public OidcStatus GetStatus()
		{
			_tokenStore.TryGetRefreshToken(_name, out string? rt);
			return GetStatus(rt, _accessToken, _tokenExpiry);
		}

		public static OidcStatus GetStatus(string? refreshToken, string? accessToken, DateTimeOffset accessTokenExpiry)
		{
			if (String.IsNullOrEmpty(refreshToken))
			{
				return OidcStatus.NotLoggedIn;
			}

			if (String.IsNullOrEmpty(accessToken))
			{
				return OidcStatus.TokenRefreshRequired;
			}

			if (accessTokenExpiry < DateTime.Now)
			{
				return OidcStatus.TokenRefreshRequired;
			}

			return OidcStatus.Connected;
		}
	}

	public class LoginFailedException : Exception
	{
		public LoginFailedException(string message, string errorDescription) : base(message)
		{
			ErrorDescription = errorDescription;
		}
		public string ErrorDescription { get; }
	}

	public class HttpServerException : Exception
	{
		public HttpServerException(string message) : base(message)
		{
		}
	}

	public class NotLoggedInException : Exception
	{
	}

	public class OidcTokenOptions
	{
		public Dictionary<string, ProviderInfo> Providers { get; set; } = [];

		public TimeSpan LoginTimeout { get; set; } = TimeSpan.FromMinutes(20);

		public static OidcTokenOptions Bind(IConfiguration config)
		{
			OidcTokenOptions options = new();
			config.GetSection("OidcToken").Bind(options);
			return options;
		}
	}

	public class ProviderInfo
	{
		[Required] public Uri ServerUri { get; set; } = null!;

		[Required] public string ClientId { get; set; } = null!;

		[Required] public string DisplayName { get; set; } = null!;

		public Uri? RedirectUri { get; set; } = null;

		public List<Uri>? PossibleRedirectUri { get; set; } = null!;
		[Required] public bool LoadClaimsFromUserProfile { get; set; } = false;

		public string Scopes { get; set; } = "openid profile offline_access email";
		public string? GenericErrorInformation { get; set; }
		public bool UseDiscoveryDocument { get; set; } = true;
		public string? ClientSecret { get; set; } = null;
	}
}
