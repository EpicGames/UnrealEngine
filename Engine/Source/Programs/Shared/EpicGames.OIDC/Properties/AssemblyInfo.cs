// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("EpicGames.Horde.Tests")]
