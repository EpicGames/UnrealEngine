// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net.Http;
using System.Net;
using System.Security.Cryptography;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using Microsoft.Extensions.Configuration;

#pragma warning disable CS1591 // Missing XML documentation on public types

namespace EpicGames.OIDC
{
	public static class ProviderConfigurationFactory
	{
		const string ConfigFileName = "oidc-configuration.json";

		public static IReadOnlyList<string> ConfigPaths { get; } =
		[
			$"Programs/OidcToken/{ConfigFileName}",
			$"Restricted/NoRedist/Programs/OidcToken/{ConfigFileName}",
			$"Restricted/NotForLicensees/Programs/OidcToken/{ConfigFileName}"
		];

		public static IConfiguration ReadConfiguration(DirectoryInfo engineDir, DirectoryInfo? gameDir)
		{

			if (!engineDir.Exists)
			{
				throw new Exception($"Failed to locate engine dir at {engineDir}");
			}

			ConfigurationBuilder configBuilder = new();
			configBuilder
				.AddJsonFile($"{engineDir}/Programs/OidcToken/{ConfigFileName}", true, false)
				.AddJsonFile($"{engineDir}/Restricted/NoRedist/Programs/OidcToken/{ConfigFileName}", true, false)
				.AddJsonFile($"{engineDir}/Restricted/NotForLicensees/Programs/OidcToken/{ConfigFileName}", true, false);

			if (gameDir?.Exists ?? false)
			{
				configBuilder.AddJsonFile($"{gameDir}/Programs/OidcToken/{ConfigFileName}", true, false)
					.AddJsonFile($"{gameDir}/Restricted/NoRedist/Programs/OidcToken/{ConfigFileName}", true, false)
					.AddJsonFile($"{gameDir}/Restricted/NotForLicensees/Programs/OidcToken/{ConfigFileName}", true, false);
			}

			IConfiguration config = configBuilder.Build();
			return config.GetSection("OidcToken");
		}

		public static IConfiguration MergeConfiguration(IEnumerable<(DirectoryInfo, DirectoryInfo?)> configurationPaths)
		{
			ConfigurationBuilder builder = new();
			foreach ((DirectoryInfo engineDir, DirectoryInfo? gameDir) in configurationPaths)
			{
				IConfiguration newConfiguration = ReadConfiguration(engineDir, gameDir);
				builder.AddConfiguration(newConfiguration);
			}

			return builder.Build();
		}

		public static async Task<ClientAuthConfigurationV1?> ReadRemoteAuthConfigurationAsync(Uri remoteUrl, string encryptionKey)
		{
			using HttpClient httpClient = new();
			const int MaxAttempts = 3;
			byte[]? b = null;
			for (int i = 0; i < MaxAttempts; i++)
			{
				using HttpRequestMessage request = new(HttpMethod.Get, new Uri(remoteUrl, "api/v1/auth/oidc-configuration"));
				using HttpResponseMessage response = await httpClient.SendAsync(request);
				if (response.StatusCode == HttpStatusCode.NotFound)
				{
					return null;
				}

				if (response.IsSuccessStatusCode)
				{
					b = await response.Content.ReadAsByteArrayAsync();
				}
			}

			if (b == null)
			{
				throw new Exception($"Failed to read config after {MaxAttempts} attempts. Unable to read configuration");
			}

			using Aes aes = Aes.Create();
			int ivLength = aes.IV.Length;
			byte[] iv = new byte[ivLength];

			Array.Copy(b, iv, ivLength);

			byte[] key = Convert.FromHexString(encryptionKey);
			MemoryStream ms = new(b, ivLength, b.Length - ivLength);
			await using CryptoStream cryptoStream = new(ms, aes.CreateDecryptor(key, iv), CryptoStreamMode.Read);
			using StreamReader decryptReader = new(cryptoStream);

			return await JsonSerializer.DeserializeAsync<ClientAuthConfigurationV1>(cryptoStream, ProviderConfigurationStateContext.Default.ClientAuthConfigurationV1);
		}

		public const string DefaultEncryptionKey = "892a27ef5cbf4894af2e6bd53a54aa48";

		public static IConfiguration BindOptions(ClientAuthConfigurationV1 remoteAuthConfig)
		{
			using MemoryStream ms = new();
			{
				using Utf8JsonWriter jsonWriter = new(ms);
				JsonSerializer.Serialize(jsonWriter, remoteAuthConfig, ProviderConfigurationStateContext.Default.ClientAuthConfigurationV1);
				ms.Position = 0;
			}
			IConfigurationRoot config = new ConfigurationBuilder().AddJsonStream(ms).Build();
			return config;
		}
	}

	public class ClientAuthConfigurationV1 : OidcTokenOptions
	{
		/// <summary>
		/// The provider in Providers to use by default
		/// </summary>
		public string? DefaultProvider { get; set; } = null;

		/// <summary>
		/// Can be set to "Anonymous" to indicate that auth is disabled. This is a horde specific convention.
		/// </summary>
		public string Method { get; set; } = "";
	}

	[JsonSerializable(typeof(ClientAuthConfigurationV1))]
	internal partial class ProviderConfigurationStateContext : JsonSerializerContext
	{
	}
}