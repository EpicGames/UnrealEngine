// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.OIDC;

/// <summary>
/// Interface to support getting a static access token.
/// </summary>
public interface IStaticAccessTokenProvider
{
	/// <summary>
	/// The static access token, if one exists, or <see langword="null"/>.
	/// </summary>
	string? AccessToken { get; }
}

/// <summary>
/// Static access token provider that never provides a static access token.
/// </summary>
public sealed class NullStaticAccessTokenProvider : IStaticAccessTokenProvider
{
	/// <inheritdoc/>
	public string? AccessToken => null;
}
