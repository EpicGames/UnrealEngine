// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Security.Cryptography;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;

#pragma warning disable CS1591 // Missing XML documentation on public types

namespace EpicGames.OIDC
{
	[JsonSerializable(typeof(TokenStoreState))]
	internal partial class TokenStoreStateContext : JsonSerializerContext
	{
	}

	internal class TokenStoreState
	{
		public Dictionary<string, string> Providers { get; set; } = [];
		public string Key { get; set; }

		[JsonConstructor]
		public TokenStoreState(string key, Dictionary<string, string> providers)
		{
			Key = key;
			Providers = providers;
		}

		public TokenStoreState(byte[] key, Dictionary<string, byte[]> providers)
		{
			Key = Convert.ToBase64String(key);

			foreach (KeyValuePair<string, byte[]> pair in providers)
			{
				Providers[pair.Key] = Convert.ToBase64String(pair.Value);
			}
		}
	}

	/// <summary>
	/// A generic token store that saves the offline token in a file on disk, this works on any platform
	/// </summary>
	internal sealed class FilesystemTokenStore : ITokenStore
	{
		private readonly SymmetricAlgorithm _crypt = Aes.Create();

		public FilesystemTokenStore()
		{
		}

		private static FileInfo GetStorePath()
		{
			return new FileInfo(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), "Epic", "UnrealEngine", "Common", "OidcToken", "tokenStore-v2.dat"));
		}

		private static void ReadStoreFromDisk(out Dictionary<string, byte[]> providers, out byte[]? iv)
		{
			FileInfo fi = GetStorePath();
			if (!fi.Exists)
			{
				providers = [];
				iv = null;
				return;
			}

			using FileStream fs = fi.OpenRead();
			TokenStoreState? state;
			try
			{
				state = JsonSerializer.Deserialize(fs, TokenStoreStateContext.Default.TokenStoreState);
			}
			catch (JsonException)
			{
				state = null;
			}

			if (state == null)
			{
				providers = [];
				iv = null;
				return;
			}

			providers = [];
			foreach ((string key, string value) in state.Providers)
			{
				providers[key] = Convert.FromBase64String(value);
			}

			iv = Convert.FromBase64String(state.Key);
		}

		private static void WriteStoreToDisk(byte[] iv, Dictionary<string, byte[]> providers)
		{
			FileInfo fi = GetStorePath();

			if (!fi.Directory?.Exists ?? false)
			{
				Directory.CreateDirectory(fi.Directory!.FullName);
			}

			string tempFile = Path.GetTempFileName();
			{
				using FileStream fs = new(tempFile, FileMode.Create, FileAccess.Write);
				using Utf8JsonWriter writer = new(fs);
				JsonSerializer.Serialize<TokenStoreState>(writer, new TokenStoreState(iv, providers), TokenStoreStateContext.Default.TokenStoreState);
			}

			File.Move(tempFile, fi.FullName, true);
		}

		public bool TryGetRefreshToken(string oidcProvider, out string refreshToken)
		{
			ReadStoreFromDisk(out Dictionary<string, byte[]> providers, out byte[]? iv);

			if (!providers.TryGetValue(oidcProvider, out byte[]? encryptedToken))
			{
				refreshToken = "";
				return false;
			}

			ICryptoTransform decryptor = _crypt.CreateDecryptor(GetSeed(), iv ?? _crypt.IV);
			byte[] bytes = decryptor.TransformFinalBlock(encryptedToken, 0, encryptedToken.Length);
			refreshToken = Encoding.Unicode.GetString(bytes);
			return true;
		}

		public void AddRefreshToken(string oidcProvider, string refreshToken)
		{
			ReadStoreFromDisk(out Dictionary<string, byte[]> providers, out byte[]? iv);
			byte[] encryptIv = iv ?? _crypt.IV;

			byte[] bytes = Encoding.Unicode.GetBytes(refreshToken);
#pragma warning disable CA5401 // Do not use CreateEncryptor with non-default IV
			ICryptoTransform encryptor = _crypt.CreateEncryptor(GetSeed(), encryptIv);
#pragma warning restore CA5401 // Do not use CreateEncryptor with non-default IV

			byte[] encryptedToken = encryptor.TransformFinalBlock(bytes, 0, bytes.Length);

			providers[oidcProvider] = encryptedToken;
			WriteStoreToDisk(encryptIv, providers);
		}

		static byte[] GetSeed()
		{
			return
			[
				0x1e, 0x72, 0x5e, 0xe7, 0x08, 0x9e, 0x29, 0x5e, 0xcb, 0xbe, 0x1b, 0xdf, 0x0e, 0xf9, 0x4a, 0x30, 0xd1,
				0xa9, 0x9b, 0xa2, 0xee, 0x58, 0xc4, 0x8e
			];
		}

		public void Dispose()
		{
			_crypt.Dispose();
		}
	}
}