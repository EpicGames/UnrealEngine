// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#pragma warning disable CS1591 // Missing XML documentation on public types

namespace EpicGames.OIDC
{
	[JsonSerializable(typeof(WindowsTokenStoreState))]
	internal partial class WindowsTokenStoreStateContext : JsonSerializerContext
	{
	}

	internal class WindowsTokenStoreState
	{
		public Dictionary<string, string> Providers { get; set; } = [];

		[JsonConstructor]
		public WindowsTokenStoreState(Dictionary<string, string> providers)
		{
			Providers = providers;
		}

		public WindowsTokenStoreState(Dictionary<string, byte[]> providers)
		{
			foreach (KeyValuePair<string, byte[]> pair in providers)
			{
				Providers[pair.Key] = Convert.ToBase64String(pair.Value);
			}
		}
	}

	internal sealed class WindowsTokenStore : ITokenStore
	{
		private readonly ILogger<WindowsTokenStore>? _logger = null;
		private readonly string? _storePathOverride;

		public WindowsTokenStore()
		{
		}

		[ActivatorUtilitiesConstructor]
		public WindowsTokenStore(ILogger<WindowsTokenStore> logger)
		{
			_logger = logger;
		}

		/// <summary>Test-only constructor that redirects the store to a custom path.</summary>
		internal WindowsTokenStore(ILogger<WindowsTokenStore>? logger, string storePath)
		{
			_logger = logger;
			_storePathOverride = storePath;
		}

		private FileInfo GetStorePath()
		{
			// Always return a fresh FileInfo to avoid stale cached Exists state.
			return new FileInfo(_storePathOverride ?? Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "UnrealEngine", "Common", "OidcToken", "oidcTokenStore-v2.dat"));
		}

		private Dictionary<string, byte[]> ReadStoreFromDisk()
		{
			FileInfo fi = GetStorePath();
			if (!fi.Exists)
			{
				_logger?.LogDebug("No existing token store found at {Path}. Assuming empty store.", fi.FullName);
				return [];
			}

			string json;
			try
			{
				using FileStream fs = fi.Open(FileMode.Open, FileAccess.Read, FileShare.Read);
				using TextReader tr = new StreamReader(fs);
				json = tr.ReadToEnd();
			}
			catch (IOException ex)
			{
				_logger?.LogWarning(ex, "Unable to read token store file at {Path}. Assuming empty store.", fi.FullName);
				return [];
			}
			catch (UnauthorizedAccessException ex)
			{
				_logger?.LogWarning(ex, "Access denied reading token store file at {Path}. Assuming empty store.", fi.FullName);
				return [];
			}

			WindowsTokenStoreState? state;
			try
			{
				state = JsonSerializer.Deserialize(json, WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
			}
			catch (JsonException ex)
			{
				_logger?.LogWarning(ex, "Failed to deserialize token store JSON. Dropping the existing state.");
				state = null;
			}

			if (state == null)
			{
				_logger?.LogDebug("Token store deserialized to null. Dropping the existing state.");
				return [];
			}

			Dictionary<string, byte[]> providers = [];
			foreach ((string key, string value) in state.Providers)
			{
				try
				{
					providers[key] = Convert.FromBase64String(value);
				}
				catch (FormatException ex)
				{
					_logger?.LogWarning(ex, "Corrupt Base64 data for provider '{Provider}'. Skipping entry.", key);
				}
			}

			_logger?.LogDebug("Loaded token store with {Count} provider(s).", providers.Count);
			return providers;
		}

		private void WriteStoreToDisk(Dictionary<string, byte[]> providers, ILogger? logger)
		{
			FileInfo fi = GetStorePath();

			if (!fi.Directory?.Exists ?? false)
			{
				Directory.CreateDirectory(fi.Directory!.FullName);
			}

			string tempFile = Path.GetTempFileName();
			try
			{
				{
					using FileStream fs = new(tempFile, FileMode.Create, FileAccess.Write);
					using Utf8JsonWriter writer = new(fs);
					JsonSerializer.Serialize(writer, new WindowsTokenStoreState(providers), WindowsTokenStoreStateContext.Default.WindowsTokenStoreState);
				}

				File.Move(tempFile, fi.FullName, true);
			}
			catch
			{
				try
				{
					File.Delete(tempFile);
				}
				catch (Exception deleteEx)
				{
					logger?.LogDebug(deleteEx, "Failed to clean up temp file {TempFile}.", tempFile);
				}

				throw;
			}
		}

		public bool TryGetRefreshToken(string oidcProvider, out string refreshToken)
		{
			try
			{
				Dictionary<string, byte[]> providers = ReadStoreFromDisk();
				if (!providers.TryGetValue(oidcProvider, out byte[]? encryptedToken))
				{
					_logger?.LogDebug("No refresh token found in store for provider '{Provider}'.", oidcProvider);
					refreshToken = "";
					return false;
				}

				try
				{
					byte[] bytes = CryptProtectDataHelper.DoCryptUnprotectData(encryptedToken, $"OidcToken-{oidcProvider}", GetEntropy(oidcProvider));
					refreshToken = Encoding.Unicode.GetString(bytes);
					_logger?.LogInformation("Read refresh token for '{Provider}': {Fingerprint}", oidcProvider, TokenUtils.Fingerprint(refreshToken));
					return true;
				}
				catch (Win32Exception e)
				{
					_logger?.LogWarning(e, "Unable to decrypt refresh token for provider '{Provider}' (Win32 error 0x{ErrorCode:X}). Discarding stored token.", oidcProvider, e.NativeErrorCode);
					refreshToken = "";
					return false;
				}
			}
			catch (Exception ex)
			{
				_logger?.LogError(ex, "Failed to read refresh token for provider '{Provider}'. Treating as not logged in.", oidcProvider);
				refreshToken = "";
				return false;
			}
		}

		private static byte[] GetEntropy(string oidcProvider)
		{
			return Encoding.UTF8.GetBytes(oidcProvider);
		}

		public void AddRefreshToken(string oidcProvider, string refreshToken)
		{
			try
			{
				byte[] bytes = Encoding.Unicode.GetBytes(refreshToken);
				byte[] encryptedToken = CryptProtectDataHelper.DoCryptProtectData(bytes, $"OidcToken-{oidcProvider}", GetEntropy(oidcProvider));

				// Acquire the same named Mutex that older binaries use for store writes,
				// so a new binary and an old binary cannot write concurrently.
				using Mutex mutex = new(false, "oidcTokenStoreDat");
				try
				{
					mutex.WaitOne();
				}
				catch (AbandonedMutexException)
				{
					// Previous holder crashed — we now own the mutex, safe to proceed.
				}

				try
				{
					Dictionary<string, byte[]> providers = ReadStoreFromDisk();
					providers[oidcProvider] = encryptedToken;
					WriteStoreToDisk(providers, _logger);
				}
				finally
				{
					mutex.ReleaseMutex();
				}

				_logger?.LogInformation("Saved refresh token for '{Provider}': {Fingerprint}", oidcProvider, TokenUtils.Fingerprint(refreshToken));
			}
			catch (Exception ex)
			{
				_logger?.LogError(ex, "Failed to persist refresh token for provider '{Provider}'", oidcProvider);
			}
		}

		public void Dispose()
		{
		}
	}

#pragma warning disable IDE1006 // Pinvoke code doesnt use the same naming conventions as C#
	static class CryptProtectDataHelper
	{
		[DllImport("kernel32.dll", SetLastError = true)]
		private static extern IntPtr LocalFree(IntPtr hMem);

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		private struct DataBlob
		{
			public int cbData;
			public IntPtr pbData;
		}

		[Flags]
		private enum CryptProtectFlags
		{
			// for remote-access situations where ui is not an option
			// if UI was specified on protect or unprotect operation, the call
			// will fail and GetLastError() will indicate ERROR_PASSWORD_RESTRICTION
			CryptprotectUiForbidden = 0x1,

			// per machine protected data -- any user on machine where CryptProtectData
			// took place may CryptUnprotectData
			CryptprotectLocalMachine = 0x4,

			// force credential synchronize during CryptProtectData()
			// Synchronize is only operation that occurs during this operation
			CryptprotectCredSync = 0x8,

			// Generate an Audit on protect and unprotect operations
			CryptprotectAudit = 0x10,

			// Protect data with a non-recoverable key
			CryptprotectNoRecovery = 0x20,

			// Verify the protection of a protected blob
			CryptprotectVerifyProtection = 0x40
		}

		[Flags]
		private enum CryptProtectPromptFlags
		{
			// prompt on unprotect
			CryptprotectPromptOnUnprotect = 0x1,

			// prompt on protect
			CryptprotectPromptOnProtect = 0x2
		}

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		private struct CryptprotectPromptstruct
		{
			public int cbSize;
			public CryptProtectPromptFlags dwPromptFlags;
			public IntPtr hwndApp;
			public string szPrompt;
		}

		[
			DllImport("Crypt32.dll",
				SetLastError = true,
				CharSet = CharSet.Auto)
		]
		[return: MarshalAs(UnmanagedType.Bool)]
		private static extern bool CryptProtectData(
			ref DataBlob pDataIn,
			string szDataDescr,
			ref DataBlob pOptionalEntropy,
			IntPtr pvReserved,
			IntPtr pPromptStruct,
			CryptProtectFlags dwFlags,
			ref DataBlob pDataOut
		);

		[
			DllImport("Crypt32.dll",
				SetLastError = true,
				CharSet = CharSet.Auto)
		]
		[return: MarshalAs(UnmanagedType.Bool)]
		private static extern bool CryptUnprotectData(
			ref DataBlob pDataIn,
			string szDataDescr,
			ref DataBlob pOptionalEntropy,
			IntPtr pvReserved,
			IntPtr pPromptStruct,
			CryptProtectFlags dwFlags,
			ref DataBlob pDataOut
		);

		public static byte[] DoCryptProtectData(byte[] dataToProtect, string description, byte[] entropy)
		{
			DataBlob dataOut = new();

			GCHandle dataHandle = GCHandle.Alloc(dataToProtect, GCHandleType.Pinned);
			GCHandle entropyHandle = GCHandle.Alloc(entropy, GCHandleType.Pinned);
			try
			{
				Marshal.Copy(dataToProtect, 0, dataHandle.AddrOfPinnedObject(), dataToProtect.Length);
				Marshal.Copy(entropy, 0, entropyHandle.AddrOfPinnedObject(), entropy.Length);

				DataBlob data = new()
				{
					cbData = dataToProtect.Length,
					pbData = dataHandle.AddrOfPinnedObject()
				};

				DataBlob entropyBlob = new()
				{
					cbData = entropy.Length,
					pbData = entropyHandle.AddrOfPinnedObject()
				};

				CryptProtectFlags flags = 0;

				if (!CryptProtectData(ref data, description, ref entropyBlob, IntPtr.Zero, IntPtr.Zero, flags, ref dataOut))
				{
					throw new Win32Exception();
				}
			}
			finally
			{
				dataHandle.Free();
				entropyHandle.Free();
			}

			try
			{
				byte[] buf = new byte[dataOut.cbData];
				Marshal.Copy(dataOut.pbData, buf, 0, dataOut.cbData);
				return buf;
			}
			finally
			{
				if (dataOut.pbData != IntPtr.Zero)
				{
					LocalFree(dataOut.pbData);
				}
			}
		}

		public static byte[] DoCryptUnprotectData(byte[] dataToDecrypt, string description, byte[] entropy)
		{
			DataBlob dataOut = new();

			GCHandle dataHandle = GCHandle.Alloc(dataToDecrypt, GCHandleType.Pinned);
			GCHandle entropyHandle = GCHandle.Alloc(entropy, GCHandleType.Pinned);
			try
			{
				Marshal.Copy(dataToDecrypt, 0, dataHandle.AddrOfPinnedObject(), dataToDecrypt.Length);
				Marshal.Copy(entropy, 0, entropyHandle.AddrOfPinnedObject(), entropy.Length);

				DataBlob data = new()
				{
					cbData = dataToDecrypt.Length,
					pbData = dataHandle.AddrOfPinnedObject()
				};

				DataBlob entropyBlob = new()
				{
					cbData = entropy.Length,
					pbData = entropyHandle.AddrOfPinnedObject()
				};

				CryptProtectFlags flags = 0;

				if (!CryptUnprotectData(ref data, description, ref entropyBlob, IntPtr.Zero, IntPtr.Zero, flags, ref dataOut))
				{
					throw new Win32Exception();
				}
			}
			finally
			{
				dataHandle.Free();
				entropyHandle.Free();
			}

			try
			{
				byte[] buf = new byte[dataOut.cbData];
				Marshal.Copy(dataOut.pbData, buf, 0, dataOut.cbData);
				return buf;
			}
			finally
			{
				if (dataOut.pbData != IntPtr.Zero)
				{
					LocalFree(dataOut.pbData);
				}
			}
		}
	}
#pragma warning restore IDE1006 // Naming Styles
}