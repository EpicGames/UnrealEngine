// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Fixture;

/// <summary>
/// Base test class for creating tests that use the Perforce fixture server.
/// Every test invocation checks for the <see cref="EnvVar" />.
/// If present, test continues and a PerforceConnection using that connection string is created.
/// If not present, the test is marked as inconclusive, which is safe to ignore and doesn't fail the test suite.
/// </summary>
[TestClass]
public abstract class BasePerforceFixtureTest : IDisposable
{
	internal const string EnvVar = "EPICGAMES_P4_FIXTURE_SERVER_URL";
	internal const string EdgeEnvVar = "EPICGAMES_P4_FIXTURE_EDGE_SERVER_URL";
	private const string UseDefaultMarker = "default";
	private const string DefaultUrl = "p4://test.user@localhost:1666";
#pragma warning disable CA1823, IDE0051 // Unused field - kept for future use
	private const string DefaultEdgeUrl = "p4://test.user@localhost:1667";
#pragma warning restore CA1823, IDE0051

	/// <summary>
	/// Console-based logger factory
	/// </summary>
	protected ILoggerFactory LoggerFactory { get; }

	/// <summary>
	/// Temporary scratch dir, deleted after every test.
	/// Provides a location for (managed) workspace root or cache files. 
	/// </summary>
	protected DirectoryReference TempDir { get; set; }

	/// <summary>
	/// A Perforce connection to the fixture test server
	/// </summary>
	protected PerforceConnection PerforceConnection { get; private set; }

	/// <summary>
	/// Fixture data mirroring how the stream/depot looks
	/// </summary>
	protected PerforceFixture Fixture { get; } = new();

	/// <summary>
	/// Unique ID to avoid tests clashing with each other
	/// </summary>
	protected string TestGuid { get; }

	private readonly string _clientName;

	protected BasePerforceFixtureTest()
	{
		LoggerFactory = Microsoft.Extensions.Logging.LoggerFactory.Create(builder => builder.AddConsole());

		// PerforceConnection is set in a [TestInitialize] method as Assert.Inconclusive cannot be raised in the constructor.
		// That initialize method will get called after the constructor ensuring the connection is instantiated.
		PerforceConnection = null!;

		TestGuid = Guid.NewGuid().ToString()[..8];
		_clientName = "test-fixture-" + TestGuid;
		TempDir = CreateTempDir();
	}

	[TestInitialize]
	public void SetUpPerforceConnection()
	{
		string? serverUrl = Environment.GetEnvironmentVariable(EnvVar);
		if (serverUrl == null)
		{
			// Only the first ~120 chars of the message is printed by MSTest runner. Try to keep it below that.
			Assert.Inconclusive($"P4 fixture server not configured. Set env var {EnvVar} to enable. See {nameof(BasePerforceFixtureTest)}.");
		}

		PerforceConnection = GetPerforceConnection(serverUrl!);
	}

	protected DirectoryReference CreateTempDir()
	{
		DirectoryReference dir = new(Path.Join(Path.GetTempPath(), "p4-" + _clientName));
		DirectoryReference.CreateDirectory(dir);
		TempDir = dir;
		return dir;
	}

	[TestCleanup]
	public void RemoveTempDir()
	{
		DeletePerforceDir(TempDir.FullName);
	}

	public static void DeletePerforceDir(string dirPath)
	{
		if (Directory.Exists(dirPath))
		{
			// Remove the read-only flags set by the P4 client
			foreach (string filePath in Directory.EnumerateFiles(dirPath, "*", SearchOption.AllDirectories))
			{
				FileInfo fileInfo = new(filePath);
				fileInfo.IsReadOnly = false;
			}

			Directory.Delete(dirPath, true);
		}
	}

	private PerforceConnection GetPerforceConnection(string connectionString)
	{
		return GetPerforceConnection(connectionString, _clientName, LoggerFactory.CreateLogger<PerforceConnection>());
	}

	internal static PerforceConnection GetPerforceConnection(string connectionString, string clientName, ILogger logger)
	{
		connectionString = connectionString == UseDefaultMarker ? DefaultUrl : connectionString;
		Uri uri = new(connectionString);

		Assert.AreEqual("p4", uri.Scheme, "P4 fixture server URL must start with p4://");

		string serverAndPort = $"{uri.Host}:{uri.Port}";
		(string username, string? password) = GetCredentialsFromUri(uri);

		PerforceSettings perforceSettings = new(serverAndPort, username)
		{
			AppName = "Perforce Test Fixture",
			ClientName = clientName,
			PreferNativeClient = true
		};

		if (password != null)
		{
			perforceSettings.Password = password;
		}

		return new PerforceConnection(perforceSettings, logger);
	}

	private static (string username, string? password) GetCredentialsFromUri(Uri uri)
	{
		if (!uri.UserInfo.Contains(':', StringComparison.Ordinal))
		{
			return (uri.UserInfo, null);
		}

		string[] parts = uri.UserInfo.Split(":");
		return (parts[0], parts[1]);
	}

	public void Dispose()
	{
		Dispose(true);
		GC.SuppressFinalize(this);
	}

	/// <summary>
	/// Standard Dispose pattern method
	/// </summary>
	/// <param name="disposing"></param>
	protected virtual void Dispose(bool disposing)
	{
		if (disposing)
		{
			LoggerFactory.Dispose();
		}
	}
}