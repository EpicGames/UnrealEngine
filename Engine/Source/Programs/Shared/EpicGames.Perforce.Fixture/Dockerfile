# Latest Ubuntu LTS release as of November 2022
FROM docker.io/library/ubuntu:22.04 AS builder
WORKDIR /tmp

RUN apt-get update
RUN apt-get -qq install -y curl netcat dos2unix

RUN curl https://cdist2.perforce.com/perforce/r24.2/bin.linux26x86_64/helix-core-server.tgz --location --silent --output helix-core-server.tgz
RUN tar zxf helix-core-server.tgz && \
    mv ./p4d /usr/local/bin/p4d && \
    mv ./p4 /usr/local/bin/p4 && \
    mkdir -p /app/data

# Start the server and generate changelists + edge server
WORKDIR /app
ADD setup.sh /app/setup.sh
RUN dos2unix setup.sh && bash setup.sh

ADD entrypoint.sh /app/entrypoint.sh
RUN dos2unix /app/entrypoint.sh && chmod +x /app/entrypoint.sh

# Copy everything to separate image to minimize image size and number of layers
FROM docker.io/library/ubuntu:22.04
COPY --from=builder /usr/local/bin/p4 /usr/local/bin/p4d /usr/local/bin/
COPY --from=builder /app/data /app/data
COPY --from=builder /app/edge-data /app/edge-data
COPY --from=builder /app/entrypoint.sh /app/entrypoint.sh
COPY --from=builder /trigger-log.sh /trigger-log.sh
ENTRYPOINT ["/app/entrypoint.sh"]
