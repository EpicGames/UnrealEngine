<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <StartupObject>EpicGames.Perforce.Fixture.Program</StartupObject>
    <IsPackable>false</IsPackable>
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <ItemGroup>
    <EmbeddedResource Include="snapshot.json" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Logging.Console" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
  </ItemGroup>
</Project>
