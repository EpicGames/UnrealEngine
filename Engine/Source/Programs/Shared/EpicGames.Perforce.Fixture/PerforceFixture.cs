// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Fixture;

using FileSet = HashSet<(string clientFile, long size, string digest)>;

public class DepotFileFixture
{
	public string DepotFile { get; }
	public string ClientFile { get; }
	public long Size { get; }
	public int Revision { get; }
	public string Digest { get; }
	public string Content { get; }

	public DepotFileFixture(string depotFile, string clientFile, string content, int revision)
	{
		DepotFile = depotFile;
		ClientFile = clientFile;
		Size = content.Length;
		Revision = revision;
		Digest = PerforceFixture.CalcMd5(content).ToUpperInvariant();
		Content = content;
	}

	public override string ToString()
	{
		return $"DepotFile={DepotFile}, Size={Size}, Revision={Revision}, Digest={Digest}, Content={Content}";
	}
}

public class ChangelistFixture
{
	public const string Placeholder = "<placeholder>";

	public int Number { get; }
	public string Description { get; }
	public bool IsShelved { get; }

	/// <summary>
	/// List of files in stream as how they would appear locally on disk, when synced to this changelist
	/// (after any view maps have been applied)
	/// </summary>
	public IReadOnlyList<DepotFileFixture> StreamFiles { get; }

	public ChangelistFixture(int number, string description, List<DepotFileFixture> streamFiles, bool isShelved = false)
	{
		Number = number;
		Description = description;
		StreamFiles = streamFiles;
		IsShelved = isShelved;
	}

	/// <summary>
	/// Assert directory contains exactly the files described by stream
	/// </summary>
	/// <param name="clientRoot">Client/workspace root directory</param>
	public void AssertDepotFiles(string clientRoot)
	{
		(FileSet actual, FileSet expected) = GetFileSets(clientRoot);
		AssertFileSets(actual, expected);
	}

	/// <summary>
	/// Assert directory contains exactly the specified file set
	/// </summary>
	/// <param name="actual">Actual file set on disk</param>
	/// <param name="expected">Expected file set to exist on disk</param>
	public static void AssertFileSets(FileSet actual, FileSet expected)
	{
		if (!expected.SetEquals(actual))
		{
			List<(string clientFile, long size, string digest)> expectedList = [.. expected];
			List<(string clientFile, long size, string digest)> actualList = [.. actual];

			expectedList.Sort();
			actualList.Sort();

			Console.WriteLine("Expected ------------------------------------------------------");
			foreach ((string clientFile, long size, string digest) in expectedList)
			{
				Console.WriteLine($"{clientFile,-20} | {size,5} | {digest}");
			}

			Console.WriteLine("");
			Console.WriteLine("Actual --------------------------------------------------------");
			foreach ((string clientFile, long size, string digest) in actualList)
			{
				Console.WriteLine($"{clientFile,-20} | {size,5} | {digest}");
			}

			Assert.Fail("Files on disk does not match files in stream at given CL");
		}
	}

	/// <summary>
	/// Assert files in stream for this changelist matches the client's have table
	/// </summary>
	/// <param name="perforce">Perforce connection</param>
	/// <param name="useHaveTable">When set to false, have table is expected to be empty</param>
	public async Task AssertHaveTableAsync(IPerforceConnection perforce, bool useHaveTable = true)
	{
		List<HaveRecord> haveRecords = await perforce.HaveAsync(new FileSpecList(), CancellationToken.None).ToListAsync();
		HashSet<(string depotFile, int rev)> actual = [.. haveRecords.Select((x) => (x.DepotFile, x.HaveRev))];
		HashSet<(string depotFile, int rev)> expected = [.. StreamFiles.Select((x) => (x.DepotFile, x.Revision))];

		if (!useHaveTable)
		{
			expected.Clear();
		}

		if (!expected.SetEquals(actual))
		{
			List<(string depotFile, int rev)> expectedList = [.. expected];
			List<(string depotFile, int rev)> actualList = [.. actual];

			expectedList.Sort();
			actualList.Sort();

			Console.WriteLine("Expected ------------------------------------------------------");
			foreach ((string depotFile, int rev) in expectedList)
			{
				Console.WriteLine($"{depotFile,-30} | {rev,5}");
			}

			Console.WriteLine("");
			Console.WriteLine("Actual --------------------------------------------------------");
			foreach ((string depotFile, int rev) in actualList)
			{
				Console.WriteLine($"{depotFile,-30} | {rev,5}");
			}

			Assert.Fail("Files in stream does not match files in client's have table");
		}
	}

	public (FileSet localFiles, FileSet streamFiles) GetFileSets(string clientRoot)
	{
		FileSet streamFiles = [.. StreamFiles.Select(x => (x.ClientFile, x.Size, x.Digest))];
		FileSet localFiles = GetLocalFileSet(clientRoot);
		return (localFiles, streamFiles);
	}

	public static FileSet GetLocalFileSet(string clientRoot)
	{
		EnumerationOptions options = new() { RecurseSubdirectories = true };

		FileSet localFiles = [.. Directory.EnumerateFiles(clientRoot, "*", options)
			.Select(x => Path.GetRelativePath(clientRoot, x))
			.Select(x => x.Replace("\\", "/", StringComparison.Ordinal))
			.Select(clientFile =>
			{
				string absPath = Path.Join(clientRoot, clientFile);
				string content = File.ReadAllText(absPath);

				// Since content of files in fixture are only single lines, the trick below works to workaround
				// differences in line endings after sync (client vs server).
				content = content.Replace("\r\n", "\n", StringComparison.Ordinal);
				long size = content.Length;
				return (clientFile, size, PerforceFixture.CalcMd5(content).ToUpperInvariant());
			})];

		return localFiles;
	}

	public (List<string> localFiles, List<string> streamFiles) GetFiles(string clientRoot)
	{
		EnumerationOptions options = new() { RecurseSubdirectories = true };
		List<string> localFiles = [.. Directory.EnumerateFiles(clientRoot, "*", options)
			.Select(x => Path.GetRelativePath(clientRoot, x))
			.Select(x => x.Replace("\\", "/", StringComparison.Ordinal))];
		List<string> streamFiles = [.. StreamFiles.Select(x => x.ClientFile)];

		localFiles.Sort();
		streamFiles.Sort();

		return (localFiles, streamFiles);
	}

	public DepotFileFixture? GetFile(string depotPath)
	{
		return new List<DepotFileFixture>(StreamFiles).Find(x => x.DepotFile == depotPath);
	}
}

public class StreamFixture
{
	public string Root { get; }
	public IEnumerable<ChangelistFixture> Changelists => _changelists.Where(x => !x.IsShelved);
	public ChangelistFixture Head { get; }
	private readonly IReadOnlyList<ChangelistFixture> _changelists;

	public StreamFixture(string root, int headCl, IReadOnlyList<ChangelistFixture> changelists)
	{
		Root = root;
		_changelists = changelists;
		Head = changelists.FirstOrDefault(x => x.Number == headCl) ?? throw new Exception($"Head CL {headCl} not found");
	}

	public ChangelistFixture LatestChangelist => Changelists.Last();

	public ChangelistFixture GetChangelist(int changeNum)
	{
		foreach (ChangelistFixture changelist in _changelists)
		{
			if (changelist.Number == changeNum)
			{
				return changelist;
			}
		}

		throw new Exception($"Change {changeNum} not found in {Root}");
	}
}

public class PerforceFixture
{
	public StreamFixture StreamFooMain { get; } = new("//Foo/Main", 9,
		[
			new(0, ChangelistFixture.Placeholder, []),
			new(1, ChangelistFixture.Placeholder, []),
			new(2, "Initial import",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #1\n", 1),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/common.h", "common.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/unused.cpp", "unused.cpp", "This is change unused.cpp #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
				]),
			new(3, "Improvement to main.cpp",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #2\n", 2),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/common.h", "common.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/unused.cpp", "unused.cpp", "This is change unused.cpp #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
				]),
			new(4, "Delete an unused file",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #2\n", 2),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/common.h", "common.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
				]),
			new(5, "Rename common.h to shared.h",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #2\n", 2),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
				]),
			new(6, "Some updates to main",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #3\n", 3),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #2\n", 2),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
				]),
			new(7, "Add more data",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #3\n", 3),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #2\n", 2),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
					new("//Foo/Main/Data/moredata.txt", "Data/moredata.txt", "This is change moredata.txt #1\n", 1),
				]),
			new(8, "A shelved CL", // Assumes base CL is 7
				[
					new("//Foo/Main/main.h", "main.h", "This is change main.h #3\n", 3),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/shelved.cpp", "shelved.cpp", "This is change shelved.cpp #1\n", 1),
					new("//Foo/Main/unused.cpp", "unused.cpp", "This is change unused.cpp #2\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
					new("//Foo/Main/Data/moredata.txt", "Data/moredata.txt", "This is change moredata.txt #1\n", 1),
				], true),
			new(9, "Did more changes to main",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #4\n", 3),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #4\n", 2),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
					new("//Foo/Main/Data/moredata.txt", "Data/moredata.txt", "This is change moredata.txt #1\n", 1),
				])
		]);

	public StreamFixture StreamFooRelease { get; } = new("//Foo/Release", 11,
		[
			new(11, "Initial import",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #2\n", 1),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					// data.txt is excluded in view
				]),
			new(12, "Initial import",
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #2 in //Foo/Release\n", 1),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #1\n", 1),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/release.cpp", "release.cpp", "This is change release.cpp #1 in //Foo/Release\n", 1),
					// data.txt is excluded in view
				]),
		]);

	public StreamFixture StreamFooSparseDev { get; } = new("//Foo/Main-SparseDev", 13,
		[
			new(13, "Stream //Foo/Main-SparseDev saved.", // TODO: add action
				[
					new("//Foo/Main/main.cpp", "main.cpp", "This is change main.cpp #4\n", 3),
					new("//Foo/Main/main.h", "main.h", "This is change main.h #4\n", 2),
					new("//Foo/Main/shared.h", "shared.h", "This is change common.h #1\n", 1),
					new("//Foo/Main/Data/data.txt", "Data/data.txt", "This is change data.txt #1\n", 1),
					new("//Foo/Main/Data/moredata.txt", "Data/moredata.txt", "This is change moredata.txt #1\n", 1),
				])
		]);

	public static string CalcMd5(string content)
	{
		byte[] data = Encoding.ASCII.GetBytes(content);
		return Md5Hash.Compute(data).ToString();
	}

	/// <summary>
	/// Assert cache contains exactly the provided digests
	/// </summary>
	/// <param name="cacheDir">Directory path to cache</param>
	/// <param name="digests">Digest expected to be in the cache</param>
	public static void AssertCacheEquals(string cacheDir, params string[] digests)
	{
		HashSet<(string cacheId, string digest, long fileSize)> cachedFiles = GetCachedFiles(cacheDir);

		HashSet<string> actual = [.. cachedFiles.Select(x => x.digest)];
		HashSet<string> expected = [.. digests];

		if (!expected.SetEquals(actual))
		{
			List<string> expectedList = [.. expected];
			List<string> actualList = [.. actual];

			expectedList.Sort();
			actualList.Sort();

			Console.WriteLine("Expected ------------------------------------------------------");
			foreach (string digest in expectedList)
			{
				Console.WriteLine($"{digest}");
			}

			Console.WriteLine("");
			Console.WriteLine("Actual --------------------------------------------------------");
			foreach (string digest in actual)
			{
				Console.WriteLine($"{digest}");
			}

			Assert.Fail("Digests in cache does not match given list");
		}
	}

	private static HashSet<(string cacheId, string digest, long fileSize)> GetCachedFiles(string cacheDir)
	{
		EnumerationOptions options = new() { RecurseSubdirectories = true };

		return [.. Directory.EnumerateFiles(cacheDir, "*", options)
			.Select(x => Path.GetRelativePath(cacheDir, x))
			.Select(clientFile =>
			{
				string absPath = Path.Join(cacheDir, clientFile);
				string content = File.ReadAllText(absPath);

				// Since content of files in fixture are only single lines, the trick below works to workaround
				// differences in line endings after sync (client vs server).
				content = content.Replace("\r\n", "\n", StringComparison.Ordinal);
				long size = content.Length;
				string fileName = Path.GetFileName(clientFile);
				string digest = CalcMd5(content).ToUpperInvariant();

				// This is a simplification. The fileName/cache ID can theoretically differ if they collide.
				// See cache ID implementation in ManagedWorkspace
				Assert.AreEqual(fileName, digest[..16]);
				return (fileName, digest, size);
			})];
	}
}