// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA1307 // Specify StringComparison for clarity

namespace EpicGames.Perforce.Fixture;

public record ServerSnapshot(Dictionary<string, StreamSnapshot> StreamSnapshots)
{
	public StreamSnapshot GetStream(string stream)
	{
		return StreamSnapshots.GetValueOrDefault(stream) ?? throw new Exception($"Unable to find stream {stream}");
	}
}

public record StreamSnapshot(string Name, List<Changelist> Changelists, List<WorkspaceSnapshot> WorkspaceSnapshots)
{
	public WorkspaceSnapshot GetWorkspace(int cl, int shelvedCl = -1)
	{
		WorkspaceSnapshot? ws = WorkspaceSnapshots.Find(x => x.Change == cl && x.ShelvedChange == shelvedCl);
		return ws ?? throw new Exception($"Unable to find CL {cl} with shelve {shelvedCl} in {Name}");
	}
}

public record Changelist(int Change, string Description); // TODO: Files in changelist

public record WorkspaceFile(
	string Path,
	long Size,
	string Content,
	string Digest,
	int Revision
)
{
	public static WorkspaceFile Create(string path, string content)
	{
		// Since content of files in fixture are only single lines,
		// the trick below is to workaround differences in line endings after sync (client vs server).
		content = content.Replace("\r\n", "\n", StringComparison.Ordinal);
		long size = content.Length;

		return new WorkspaceFile(path, size, content, PerforceFixture.CalcMd5(content).ToUpperInvariant(), -1);
	}
}

public record WorkspaceSnapshot(int Change, int ShelvedChange, HashSet<WorkspaceFile> Files)
{
	/// <summary>
	/// Assert local directory contains exactly the files described by stream
	/// </summary>
	/// <param name="clientRoot">Client/workspace root directory in the local file system</param>
	public void AssertDepotFiles(string clientRoot)
	{
		//FileSet snapshotFiles = new (Files.Select(x => (x.Path, x.Size, x.Digest)));
		HashSet<WorkspaceFile> localFiles = PerforceSnapshot.GetLocalFileSet(clientRoot);
		AssertFileSets(localFiles, Files);
	}

	/// <summary>
	/// Assert directory contains exactly the specified file set
	/// </summary>
	/// <param name="actual">Actual file set on disk</param>
	/// <param name="expected">Expected file set to exist on disk</param>
	public static void AssertFileSets(HashSet<WorkspaceFile> actual, HashSet<WorkspaceFile> expected)
	{
		if (!expected.SetEquals(actual))
		{
			List<WorkspaceFile> expectedList = [.. expected];
			List<WorkspaceFile> actualList = [.. actual];

			expectedList.Sort((a, b) => String.Compare(a.Path, b.Path, StringComparison.Ordinal));
			actualList.Sort((a, b) => String.Compare(a.Path, b.Path, StringComparison.Ordinal));

			static string ReplaceLineEndings(string s) => s.Replace("\n", "\\n").Replace("\r", "\\r");

			Console.WriteLine("Expected " + new string('-', 90));
			foreach (WorkspaceFile wf in expectedList)
			{
				Console.WriteLine($"{wf.Path,-20} | {wf.Size,5} | {wf.Digest} | {ReplaceLineEndings(wf.Content)}");
			}

			Console.WriteLine("");
			Console.WriteLine("Actual " + new string('-', 92));
			foreach (WorkspaceFile wf in actualList)
			{
				Console.WriteLine($"{wf.Path,-20} | {wf.Size,5} | {wf.Digest} | {ReplaceLineEndings(wf.Content)}");
			}

			Assert.Fail("Files on disk does not match files in stream at given CL");
		}
	}
}

public record WorkspaceState(string Stream, int Cl, int ShelvedCl = 0, bool RemoveUntracked = true, Action? OnPreSync = null, Action? OnPostSync = null);

public class PerforceSnapshot(PerforceConnection perforce)
{
	public const string SnapshotJsonFile = "snapshot.json"; // Also referenced in .csproj for inclusion

	public const string FooMain = "//Foo/Main";
	public const string FooRelease = "//Foo/Release";
	public const string FooMainSparseDev = "//Foo/Main-SparseDev";
	public const string FooMainSparseRel = "//Foo/Main-SparseRel";

	public static readonly IReadOnlyList<WorkspaceState> FooMainStates = [
		// Normal changelists
		Main(2), Main(3), Main(4), Main(5), Main(6), Main(7), Main(9),
			
		// Shelved changelists
		Main(2, 8), Main(3, 8), Main(7, 8), Main(9, 8)
	];

	public static readonly IReadOnlyList<WorkspaceState> FooReleaseStates = [Release(11), Release(12)];
	public static readonly IReadOnlyList<WorkspaceState> FooMainSparseDevStates = [SparseDev(9), SparseDev(13)];

	private static readonly JsonSerializerOptions s_jsonSerializerOptions = new() { WriteIndented = true };
	private static readonly Lazy<ServerSnapshot> s_serverSnapshot = new(LoadSnapshot);

	public static ServerSnapshot GetSnapshot() => s_serverSnapshot.Value;
	public static WorkspaceState Main(int cl, int shelvedCl = -1) => new(FooMain, cl, shelvedCl);
	public static WorkspaceState Release(int cl, int shelvedCl = -1) => new(FooRelease, cl, shelvedCl);
	public static WorkspaceState SparseDev(int cl, int shelvedCl = -1) => new(FooMainSparseDev, cl, shelvedCl);

	private static ServerSnapshot LoadSnapshot()
	{
		Assembly assembly = typeof(PerforceSnapshot).Assembly;
		const string ResourceName = "EpicGames.Perforce.Fixture." + SnapshotJsonFile;
		using Stream? stream = assembly.GetManifestResourceStream(ResourceName);
		if (stream == null)
		{
			throw new FileNotFoundException($"Embedded resource not found: {ResourceName}");
		}

		ServerSnapshot? snapshot = JsonSerializer.Deserialize<ServerSnapshot>(stream, s_jsonSerializerOptions);
		return snapshot ?? throw new Exception("Unable to JSON deserialize snapshot");
	}

	public async Task TakeSnapshotsAsync(string snapshotJsonFile)
	{
		Dictionary<string, StreamSnapshot> streamSnapshots = [];
		async Task AddStream(IReadOnlyList<WorkspaceState> states)
		{
			string streamName = states[0].Stream;
			streamSnapshots[streamName] = await TakeStreamSnapshotAsync(streamName, states);
		}

		await AddStream(FooMainStates);
		await AddStream(FooReleaseStates);
		await AddStream(FooMainSparseDevStates);
		string data = JsonSerializer.Serialize(new ServerSnapshot(streamSnapshots), s_jsonSerializerOptions);
		await File.WriteAllTextAsync(snapshotJsonFile, data);
	}

	private async Task<StreamSnapshot> TakeStreamSnapshotAsync(string stream, IReadOnlyList<WorkspaceState> streamStates)
	{
		List<WorkspaceSnapshot> workspaceSnapshots = [];
		foreach (WorkspaceState ss in streamStates)
		{
			workspaceSnapshots.Add(await TakeWorkspaceSnapshotAsync(ss));
		}
		return new StreamSnapshot(stream, [], workspaceSnapshots);
	}

	private async Task<WorkspaceSnapshot> TakeWorkspaceSnapshotAsync(WorkspaceState ss)
	{
		Console.WriteLine($"Syncing {ss.Stream} @ {ss.Cl} with shelve {ss.ShelvedCl} ...\n");
		string guid = Guid.NewGuid().ToString()[..8];
		string pathFriendlyName = $"{ss.Stream.Replace("/", "", StringComparison.Ordinal)}-{ss.Cl}-{ss.ShelvedCl}";
		DirectoryReference dir = new(Path.Join(Path.GetTempPath(), $"p4-snapshot-{guid}-{pathFriendlyName}"));
		DirectoryReference.CreateDirectory(dir);

		HashSet<WorkspaceFile> workspaceFiles = [];
		(IPerforceConnection perforce, string clientRoot) = await CreateClientAsync(ss.Stream, CancellationToken.None);
		await perforce.SyncAsync($"//...@{ss.Cl}").ToListAsync();

		if (ss.ShelvedCl > 0)
		{
			await perforce.UnshelveAsync(
				ss.ShelvedCl, -1, null, null, null,
				UnshelveOptions.ForceOverwrite, Array.Empty<string>(), CancellationToken.None);
		}

		List<string> localFiles = [.. Directory.EnumerateFiles(clientRoot, "*", SearchOption.AllDirectories)];
		foreach (string absPath in localFiles)
		{
			string localFile = Path.GetRelativePath(clientRoot, absPath).Replace('\\', '/');
			string content = await File.ReadAllTextAsync(absPath);
			workspaceFiles.Add(WorkspaceFile.Create(localFile, content));
		}

		return new WorkspaceSnapshot(ss.Cl, ss.ShelvedCl, workspaceFiles);
	}

	public static HashSet<WorkspaceFile> GetLocalFileSet(string clientRoot)
	{
		EnumerationOptions options = new() { RecurseSubdirectories = true };

		HashSet<WorkspaceFile> localFiles = [.. Directory.EnumerateFiles(clientRoot, "*", options)
			.Select(x => Path.GetRelativePath(clientRoot, x))
			.Select(x => x.Replace("\\", "/", StringComparison.Ordinal))
			.Select(clientFile =>
			{
				string absPath = Path.Join(clientRoot, clientFile);
				string content = File.ReadAllText(absPath);

				// Since content of files in fixture are only single lines, the trick below works to workaround
				// differences in line endings after sync (client vs server).
				content = content.Replace("\r\n", "\n", StringComparison.Ordinal);
				long size = content.Length;
				// return WorkspaceFile.C(clientFile, size, PerforceFixture.CalcMd5(content).ToUpperInvariant());
				return WorkspaceFile.Create(clientFile, content);
			})];

		return localFiles;
	}

	private async Task<(IPerforceConnection perforce, string clientRoot)> CreateClientAsync(string stream, CancellationToken ct)
	{
		string guid = Guid.NewGuid().ToString()[..8];
		string clientName = $"p4-snapshot-{guid}";
		string clientRoot = Path.Join(Path.GetTempPath(), clientName);
		Directory.CreateDirectory(clientRoot);

		ClientRecord newClient = new(clientName, perforce.Settings.UserName, clientRoot)
		{
			Host = Environment.MachineName,
			Type = "partitioned",
			Stream = stream,
			Options = ClientOptions.Clobber
		};
		Directory.CreateDirectory(newClient.Root);
		await perforce.TryDeleteClientAsync(DeleteClientOptions.None, clientName, ct);
		await perforce.CreateClientAsync(newClient, ct);

		PerforceSettings perforceSettings = new(perforce.Settings) { ClientName = clientName, PreferNativeClient = true };
		return (await PerforceConnection.CreateAsync(perforceSettings, NullLogger.Instance), clientRoot);
	}
}