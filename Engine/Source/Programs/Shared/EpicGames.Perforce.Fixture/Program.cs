// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Perforce.Fixture;

public static class Program
{
	public static async Task<int> Main()
	{
		Console.WriteLine("Generate snapshot data from Perforce server for use in tests");

		string? serverUrl = Environment.GetEnvironmentVariable(BasePerforceFixtureTest.EnvVar);
		if (serverUrl == null)
		{
			Console.WriteLine($"Set env var {BasePerforceFixtureTest.EnvVar} to configure Perforce connection settings");
			return 1;
		}

		string guid = Guid.NewGuid().ToString()[..8];
		string clientName = "p4-snapshot-generator-" + guid;
		using PerforceConnection perforce = BasePerforceFixtureTest.GetPerforceConnection(serverUrl, clientName, new DefaultConsoleLogger());
		PerforceSnapshot snapshot = new(perforce);
		await snapshot.TakeSnapshotsAsync(PerforceSnapshot.SnapshotJsonFile);
		return 0;
	}
}