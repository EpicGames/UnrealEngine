#!/bin/bash

# Forward signals to child processes for clean Docker shutdown
trap 'kill $(jobs -p) 2>/dev/null; wait' SIGTERM SIGINT

# Start the commit server
p4d -r /app/data -p 1666 &

# Wait for the commit server to accept connections before starting the edge
until p4 -p localhost:1666 info > /dev/null 2>&1; do sleep 0.5; done

# Start the edge server (connects to commit server via configured P4TARGET)
p4d -r /app/edge-data -p 1667 &

wait
