#!/bin/bash

set -euo pipefail # Exit on any error

export PORT=1666
export P4PORT=localhost:$PORT

# Start P4D
p4d -r /app/data -p $PORT &

# The time needed for P4D to start listening on the TCP port varies greatly from 2 to +10 secs
# Waiting for the port to be available instead is more reliable than an arbitrary sleep
echo "Waiting for server to listen on port $PORT..."
while ! nc -z localhost $PORT </dev/null; do sleep 1; done

# Set client.readonly.dir for partitioned client types
mkdir -p /app/part-db-have
p4 configure set client.readonly.dir=/app/part-db-have

# Create a dummy user
p4 user -f -i << EOF
User:	test.user
Type:   standard
Email:	test.user@localhost
FullName:	Test User
AuthMethod:	perforce
EOF

# Set up triggers
cat > /trigger-log.sh <<'EOF'
#!/bin/bash
echo "Trigger: $@" >> /tmp/trigger.log
EOF
chmod +x /trigger-log.sh

cat > /tmp/triggers.txt <<EOF
Triggers:
        FormSaveAll form-save change "/trigger-log.sh form-save<SEP>%client%<SEP>%clienthost%<SEP>%clientip%<SEP>%clientprog%<SEP>%clientversion%<SEP>%peerhost%<SEP>%peerip%<SEP>%serverhost%<SEP>%serverid%<SEP>%serverip%<SEP>%servername%<SEP>%serverport%<SEP>%submitserverid%<SEP>%user%<SEP>%formname%<SEP>%formtype%<SEP>%change%<SEP>%changeroot%"
        ChangeCommitAll change-commit //... "/trigger-log.sh change-commit<SEP>%client%<SEP>%clienthost%<SEP>%clientip%<SEP>%clientprog%<SEP>%clientversion%<SEP>%peerhost%<SEP>%peerip%<SEP>%serverhost%<SEP>%serverid%<SEP>%serverip%<SEP>%servername%<SEP>%serverport%<SEP>%submitserverid%<SEP>%user%<SEP>%formname%<SEP>%formtype%<SEP>%change%<SEP>%changeroot%"
        ChangeCommitAll shelve-commit //... "/trigger-log.sh shelve-commit<SEP>%client%<SEP>%clienthost%<SEP>%clientip%<SEP>%clientprog%<SEP>%clientversion%<SEP>%peerhost%<SEP>%peerip%<SEP>%serverhost%<SEP>%serverid%<SEP>%serverip%<SEP>%servername%<SEP>%serverport%<SEP>%submitserverid%<SEP>%user%<SEP>%formname%<SEP>%formtype%<SEP>%change%<SEP>%changeroot%"
EOF
p4 triggers -i < /tmp/triggers.txt

# Create a stream depot
p4 depot -i << EOF
Depot:          Foo
Owner:          test.user
Description:
                Created by test.user.

Type:           stream
StreamDepth:    //Foo/2
Map:            Foo/...
EOF

# Create an actual stream
p4 stream -i << EOF
Stream: //Foo/Main
Owner:  test.user
Name:   Main
Parent: none
Type:   mainline
Description:
        Main stream for Foo depot

Options:        allsubmit unlocked notoparent nofromparent mergedown
ParentView:     inherit
Paths:
        share ...
EOF

export P4CLIENT=generate-fixture-data-main
export CLIENT_ROOT=/tmp/${P4CLIENT}
p4 client -i << EOF
Client: ${P4CLIENT}
Root: ${CLIENT_ROOT}
Stream: //Foo/Main
EOF

# Create the workspace root dir and sync the empty stream
mkdir -p $CLIENT_ROOT
cd $CLIENT_ROOT
p4 sync


# Changelist #2
mkdir $CLIENT_ROOT/Data
echo "This is change main.cpp #1" > $CLIENT_ROOT/main.cpp && p4 add $CLIENT_ROOT/main.cpp
echo "This is change main.h #1" > $CLIENT_ROOT/main.h && p4 add $CLIENT_ROOT/main.h
echo "This is change common.h #1" > $CLIENT_ROOT/common.h && p4 add $CLIENT_ROOT/common.h
echo "This is change unused.cpp #1" > $CLIENT_ROOT/unused.cpp && p4 add $CLIENT_ROOT/unused.cpp
echo "This is change data.txt #1" > $CLIENT_ROOT/Data/data.txt && p4 add $CLIENT_ROOT/Data/data.txt
p4 submit -d "Initial import"

# Changelist #3
p4 edit main.cpp
echo "This is change main.cpp #2" > $CLIENT_ROOT/main.cpp
p4 submit -d "Improvement to main.cpp"

# Changelist #4
p4 delete unused.cpp
p4 submit -d "Delete an unused file"

# Changelist #5
p4 edit common.h
p4 move common.h shared.h
p4 submit -d "Rename common.h to shared.h"

# Changelist #6
p4 edit main.cpp
p4 edit main.h
echo "This is change main.cpp #3" > $CLIENT_ROOT/main.cpp
echo "This is change main.h #2" > $CLIENT_ROOT/main.h
p4 submit -d "Some updates to main"

# Changelist #7
echo "This is change moredata.txt #1" > $CLIENT_ROOT/Data/moredata.txt && p4 add $CLIENT_ROOT/Data/moredata.txt
p4 submit -d "Add more data"

# Changelist #8
p4 change -i << EOF
Change: new
Description: A shelved CL
EOF
# Re-add a file that's been deleted. This is known to cause tracking issues.
echo "This is change unused.cpp #2" > $CLIENT_ROOT/unused.cpp && p4 add -c 8 $CLIENT_ROOT/unused.cpp
echo "This is change shelved.cpp #1" > $CLIENT_ROOT/shelved.cpp && p4 add -c 8 $CLIENT_ROOT/shelved.cpp
p4 edit -c 8 main.h && echo "This is change main.h #3" > $CLIENT_ROOT/main.h
p4 delete -c 8 main.cpp
p4 shelve -c 8
p4 revert -c 8 //...

# Changelist #9
p4 edit main.cpp
p4 edit main.h
echo "This is change main.cpp #4" > $CLIENT_ROOT/main.cpp
echo "This is change main.h #4" > $CLIENT_ROOT/main.h
p4 submit -d "Did more changes to main"


# Create alternative release stream
p4 stream -i << EOF
Stream: //Foo/Release
Owner:  test.user
Name:   Release
Parent: //Foo/Main
Type:   release
Description:
        Release stream for Foo depot

Options:        allsubmit unlocked notoparent nofromparent mergedown
ParentView:     inherit
Paths:
        share ...
        exclude Data/...
EOF

export P4CLIENT=generate-fixture-data-release
export CLIENT_ROOT=/tmp/${P4CLIENT}
p4 client -i << EOF
Client: ${P4CLIENT}
Root: ${CLIENT_ROOT}
Stream: //Foo/Release
EOF

# Create the workspace root dir and sync the empty stream
mkdir -p $CLIENT_ROOT
cd $CLIENT_ROOT

# Changelist #11
p4 populate -d "Populate from CL 5" //Foo/Main/...@5 //Foo/Release/...
p4 sync

# Changelist #12
p4 edit main.cpp
echo "This is change main.cpp #2 in //Foo/Release" > $CLIENT_ROOT/main.cpp
echo "This is change release.cpp #1 in //Foo/Release" > $CLIENT_ROOT/release.cpp && p4 add $CLIENT_ROOT/release.cpp
p4 submit -d "Improvements to //Foo/Release"

# Create a sparse dev stream
p4 stream -i << EOF
Stream: //Foo/Main-SparseDev
Owner:  test.user
Name:   Main-SparseDev
Parent: //Foo/Main
Type:   sparsedev
Description:
        Sparse dev stream for //Foo/Main

Options:        allsubmit unlocked notoparent nofromparent mergedown
ParentView:     inherit
Paths:
        share ...
EOF

# Create a sparse release stream
p4 stream -i << EOF
Stream: //Foo/Main-SparseRel
Owner:  test.user
Name:   Main-SparseRel
Parent: //Foo/Release
Type:   sparserel
Description:
        Sparse release stream for //Foo/Main

Options:        allsubmit unlocked notoparent nofromparent mergedown
ParentView:     inherit
Paths:
        share ...
EOF

# Configure Perforce edge server
EDGE_PORT=1667

# Use test.user for edge setup commands (needs super access)
export P4USER=test.user

echo "Configuring edge server..."

# Set the commit server's identity
p4 serverid commit

# Create a service user for the edge server
p4 user -f -i << EOF
User:	svc-edge
Type:	service
Email:	svc-edge@localhost
FullName:	Edge Service User
AuthMethod:	perforce
EOF

# Grant the service user super access needed for replication
p4 protect -i << EOF
Protections:
	write user * * //...
	super user test.user * //...
	super user svc-edge * //...
EOF

# Create the edge server spec on the commit server
p4 server -i << EOF
ServerID:	edge
Type:	server
Name:	edge
Address:	localhost:$EDGE_PORT
Services:	edge-server
User:	svc-edge
Description:
	Edge server for testing
EOF

# Configure replication settings for the edge (stored on the commit server)
p4 configure set "edge#P4TARGET=localhost:$PORT"
p4 configure set "edge#db.replication=readonly"
p4 configure set "edge#lbr.replication=readonly"
p4 configure set "edge#serviceUser=svc-edge"
p4 configure set "edge#startup.1=pull -i 1"
p4 configure set "edge#startup.2=pull -u -i 1"
p4 configure set "edge#db.peeking=2"
p4 configure set "edge#client.readonly.dir=/app/edge-part-db-have"

# Stop the commit server cleanly
p4 admin stop
wait || true

# Create a journal dump of the commit server database
p4d -r /app/data -jd /tmp/commit.ckp

# Seed the edge server database from the commit server checkpoint
mkdir -p /app/edge-data
p4d -r /app/edge-data -jr /tmp/commit.ckp

# Write the edge server identity file (overrides the commit server ID from the checkpoint)
echo edge > /app/edge-data/server.id