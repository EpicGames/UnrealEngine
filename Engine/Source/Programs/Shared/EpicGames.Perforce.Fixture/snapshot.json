{
  "StreamSnapshots": {
    "//Foo/Main": {
      "Name": "//Foo/Main",
      "Changelists": [],
      "WorkspaceSnapshots": [
        {
          "Change": 2,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "common.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #1\n",
              "Digest": "4FE1DE51507BF699F8BCF087AFB27DBB",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #1\n",
              "Digest": "B84DF012D6393DD890000B144F876F40",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 3,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "common.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #2\n",
              "Digest": "9364BBCA53127F5049DB72A33EDD5CB2",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #1\n",
              "Digest": "B84DF012D6393DD890000B144F876F40",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 4,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "common.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #2\n",
              "Digest": "9364BBCA53127F5049DB72A33EDD5CB2",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 5,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #2\n",
              "Digest": "9364BBCA53127F5049DB72A33EDD5CB2",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 6,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #3\n",
              "Digest": "FCB0EE3699D298EBFDBDD5646E076E59",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #2\n",
              "Digest": "6BBB5178B88ED28C895300AF0F362209",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 7,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #3\n",
              "Digest": "FCB0EE3699D298EBFDBDD5646E076E59",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #2\n",
              "Digest": "6BBB5178B88ED28C895300AF0F362209",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 9,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #4\n",
              "Digest": "6D8B09596E7B45C923FCE5A45662E20E",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #4\n",
              "Digest": "ADDF62E18A7E93D7F643FE4ACCCE2990",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 2,
          "ShelvedChange": 8,
          "Files": [
            {
              "Path": "common.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #3\n",
              "Digest": "B5423FE256B087FF11045710904FD235",
              "Revision": -1
            },
            {
              "Path": "shelved.cpp",
              "Size": 30,
              "Content": "This is change shelved.cpp #1\n",
              "Digest": "0B0894F8B928D830559151063B72DCD7",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #2\n",
              "Digest": "9C381D85D336FB6B45842E808E43DCA4",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 3,
          "ShelvedChange": 8,
          "Files": [
            {
              "Path": "common.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #3\n",
              "Digest": "B5423FE256B087FF11045710904FD235",
              "Revision": -1
            },
            {
              "Path": "shelved.cpp",
              "Size": 30,
              "Content": "This is change shelved.cpp #1\n",
              "Digest": "0B0894F8B928D830559151063B72DCD7",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #2\n",
              "Digest": "9C381D85D336FB6B45842E808E43DCA4",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 7,
          "ShelvedChange": 8,
          "Files": [
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #3\n",
              "Digest": "B5423FE256B087FF11045710904FD235",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "shelved.cpp",
              "Size": 30,
              "Content": "This is change shelved.cpp #1\n",
              "Digest": "0B0894F8B928D830559151063B72DCD7",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #2\n",
              "Digest": "9C381D85D336FB6B45842E808E43DCA4",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 9,
          "ShelvedChange": 8,
          "Files": [
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #3\n",
              "Digest": "B5423FE256B087FF11045710904FD235",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "shelved.cpp",
              "Size": 30,
              "Content": "This is change shelved.cpp #1\n",
              "Digest": "0B0894F8B928D830559151063B72DCD7",
              "Revision": -1
            },
            {
              "Path": "unused.cpp",
              "Size": 29,
              "Content": "This is change unused.cpp #2\n",
              "Digest": "9C381D85D336FB6B45842E808E43DCA4",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        }
      ]
    },
    "//Foo/Release": {
      "Name": "//Foo/Release",
      "Changelists": [],
      "WorkspaceSnapshots": [
        {
          "Change": 11,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #2\n",
              "Digest": "9364BBCA53127F5049DB72A33EDD5CB2",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 12,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 44,
              "Content": "This is change main.cpp #2 in //Foo/Release\n",
              "Digest": "16009C33B215A393F42B04E3A11EA0D2",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #1\n",
              "Digest": "16A5187ED86AD1425C7108F7869477BA",
              "Revision": -1
            },
            {
              "Path": "release.cpp",
              "Size": 47,
              "Content": "This is change release.cpp #1 in //Foo/Release\n",
              "Digest": "B0AF06334A16CEEF8CCFF655BEA862FF",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            }
          ]
        }
      ]
    },
    "//Foo/Main-SparseDev": {
      "Name": "//Foo/Main-SparseDev",
      "Changelists": [],
      "WorkspaceSnapshots": [
        {
          "Change": 9,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #4\n",
              "Digest": "6D8B09596E7B45C923FCE5A45662E20E",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #4\n",
              "Digest": "ADDF62E18A7E93D7F643FE4ACCCE2990",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        },
        {
          "Change": 13,
          "ShelvedChange": -1,
          "Files": [
            {
              "Path": "main.cpp",
              "Size": 27,
              "Content": "This is change main.cpp #4\n",
              "Digest": "6D8B09596E7B45C923FCE5A45662E20E",
              "Revision": -1
            },
            {
              "Path": "main.h",
              "Size": 25,
              "Content": "This is change main.h #4\n",
              "Digest": "ADDF62E18A7E93D7F643FE4ACCCE2990",
              "Revision": -1
            },
            {
              "Path": "shared.h",
              "Size": 27,
              "Content": "This is change common.h #1\n",
              "Digest": "66521760C323E061BC3E9C9A2C67CD13",
              "Revision": -1
            },
            {
              "Path": "Data/data.txt",
              "Size": 27,
              "Content": "This is change data.txt #1\n",
              "Digest": "6EB6D4624CA0D84C8A1AB46A42756D35",
              "Revision": -1
            },
            {
              "Path": "Data/moredata.txt",
              "Size": 31,
              "Content": "This is change moredata.txt #1\n",
              "Digest": "C8C9EE282B2B5BCB3F1F9DE0C4DF024A",
              "Revision": -1
            }
          ]
        }
      ]
    }
  }
}