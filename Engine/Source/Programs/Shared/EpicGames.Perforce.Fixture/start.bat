@echo off
IF NOT DEFINED DOCKER_EXECUTABLE (SET DOCKER_EXECUTABLE=docker)

set TAG=perforce-fixture
%DOCKER_EXECUTABLE% build --tag %TAG% . && %DOCKER_EXECUTABLE% run -it --rm --publish 1666:1666 --ulimit nofile=65536:65536 --ulimit nproc=32768:32768 %TAG%