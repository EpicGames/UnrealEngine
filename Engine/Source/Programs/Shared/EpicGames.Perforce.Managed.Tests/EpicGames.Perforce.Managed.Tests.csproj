<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
    <ProjectReference Include="..\EpicGames.Perforce.Fixture\EpicGames.Perforce.Fixture.csproj" />
  </ItemGroup>
</Project>
