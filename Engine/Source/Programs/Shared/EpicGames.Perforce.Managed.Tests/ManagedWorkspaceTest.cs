// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce.Fixture;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;

namespace EpicGames.Perforce.Managed.Tests;

[TestClass]
[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores")]
public class ManagedWorkspaceTest : BasePerforceFixtureTest
{
	private string SyncDir => Path.Join(TempDir.FullName, "Sync");
	private string CacheDir => Path.Join(TempDir.FullName, "Cache");
	private string StreamName => Fixture.StreamFooMain.Root;
	private StreamFixture Stream => Fixture.StreamFooMain;

	private readonly ILogger<ManagedWorkspace> _mwLogger;

	public ManagedWorkspaceTest()
	{
		_mwLogger = LoggerFactory.CreateLogger<ManagedWorkspace>();
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task SyncSingleChangelistAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		await AssertHaveTableFileCountAsync(0);

		await SyncAsync(ws, 6);
		Stream.GetChangelist(6).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(6).AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	private class CleanCounter
	{
		public int NumCleans { get; private set; } = 0;
		public int NumFilesDeleted { get; private set; } = 0;
		public int NumDirsDeleted { get; private set; } = 0;

		public void OnClean(int numFiles, int numDirs)
		{
			NumCleans += 1;
			NumFilesDeleted += numFiles;
			NumDirsDeleted += numDirs;
			Console.WriteLine($"Clean performed: numFiles={numFiles} numDirs={numDirs}");
		}

		public CleanCounter Reset()
		{
			NumCleans = 0;
			NumFilesDeleted = 0;
			NumDirsDeleted = 0;
			return this;
		}

		public CleanCounter Assert(int expectedCount, int expectedNumFilesCleaned, int expectedNumDirsCleaned)
		{
			Microsoft.VisualStudio.TestTools.UnitTesting.Assert.AreEqual(expectedCount, NumCleans);
			Microsoft.VisualStudio.TestTools.UnitTesting.Assert.AreEqual(expectedNumFilesCleaned, NumFilesDeleted);
			Microsoft.VisualStudio.TestTools.UnitTesting.Assert.AreEqual(expectedNumDirsCleaned, NumDirsDeleted);
			return this;
		}
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	public async Task SyncSkipCleanIfPossibleAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		await AssertHaveTableFileCountAsync(0);

		CleanCounter cc = new();
		ws.OnClean += cc.OnClean;

		// Sync first time, clean is expected
		await SyncAsync(ws, 6, assertFiles: true, useHaveTable: useHaveTable);
		cc.Assert(1, 0, 0).Reset();

		// Simulate some build output inside workspace
		await File.WriteAllTextAsync(Path.Join(SyncDir, "build-output.obj"), "somecontent");

		// Finalize and clean up is called when job finishes
		await ws.CleanAsync(PerforceConnection, true, CancellationToken.None);
		cc.Assert(1, 1, 0).Reset();

		// Syncing again should not result in a clean up, as it was called during finalize and we're using the same P4 client
		await SyncAsync(ws, 7, assertFiles: true, useHaveTable: useHaveTable);
		cc.Assert(0, 0, 0).Reset();
	}

	[TestMethod]
	public async Task SyncBackwardsToOlderChangelistRemoveUntrackedAsync()
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(true);

		await SyncAsync(ws, 6, removeUntracked: false);
		Stream.GetChangelist(6).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(6).AssertHaveTableAsync(PerforceConnection);

		await SyncAsync(ws, 7, removeUntracked: false);
		Stream.GetChangelist(7).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(7).AssertHaveTableAsync(PerforceConnection);

		await SyncAsync(ws, 6, removeUntracked: false);
		Stream.GetChangelist(6).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(6).AssertHaveTableAsync(PerforceConnection);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task SyncBackwardsToOlderChangelistAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);

		await SyncAsync(ws, 6);
		Stream.GetChangelist(6).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(6).AssertHaveTableAsync(PerforceConnection, useHaveTable);

		await SyncAsync(ws, 7);
		Stream.GetChangelist(7).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(7).AssertHaveTableAsync(PerforceConnection, useHaveTable);

		// Go back one changelist
		await SyncAsync(ws, 6);
		Stream.GetChangelist(6).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(6).AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task SyncUsingCacheFilesAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);

		FileReference GetCacheFilePath(int changeNumber)
		{
			return new FileReference(Path.Join(TempDir.FullName, $"CacheFile-{changeNumber}.bin"));
		}

		// Sync and create a new cache file per change number
		foreach (ChangelistFixture cl in Stream.Changelists)
		{
			await SyncAsync(ws, cl.Number, cacheFile: GetCacheFilePath(cl.Number));
			cl.AssertDepotFiles(SyncDir);
			await cl.AssertHaveTableAsync(PerforceConnection, useHaveTable);
		}

		// Sync again but using the cache files created above
		foreach (ChangelistFixture cl in Stream.Changelists.Reverse())
		{
			await SyncAsync(ws, cl.Number, cacheFile: GetCacheFilePath(cl.Number));
			cl.AssertDepotFiles(SyncDir);
			await cl.AssertHaveTableAsync(PerforceConnection, useHaveTable);
		}
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task SyncWithViewExclusivePathFirstAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		ChangelistFixture cl = Stream.GetChangelist(6);

		List<string> view = ["-/Data/...", "-/shared.h",];
		await ws.SyncAsync(PerforceConnection, StreamName, cl.Number, view, true, false, null, CancellationToken.None);

		List<DepotFileFixture> filtered = [.. cl.StreamFiles
			.Where(x => !x.DepotFile.Contains("shared.h", StringComparison.Ordinal))
			.Where(x => !x.DepotFile.Contains("Data/", StringComparison.Ordinal))];

		ChangelistFixture clViewApplied = new(cl.Number, cl.Description, filtered, cl.IsShelved);
		clViewApplied.AssertDepotFiles(SyncDir);
		await clViewApplied.AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task SyncWithViewInclusivePathFirstAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		ChangelistFixture cl = Stream.GetChangelist(6);

		List<string> view = ["/...", "-/Data/..."];
		await ws.SyncAsync(PerforceConnection, StreamName, cl.Number, view, true, false, null, CancellationToken.None);

		List<DepotFileFixture> filtered = [.. cl.StreamFiles.Where(x => !x.DepotFile.Contains("Data/", StringComparison.Ordinal))];

		ChangelistFixture clViewApplied = new(cl.Number, cl.Description, filtered, cl.IsShelved);
		clViewApplied.AssertDepotFiles(SyncDir);
		await clViewApplied.AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task PopulateAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		List<PopulateRequest> populateRequests = [new PopulateRequest(PerforceConnection, StreamName, [])];
		await ws.PopulateAsync(populateRequests, false, CancellationToken.None);
		Stream.LatestChangelist.AssertDepotFiles(SyncDir);
		await Stream.LatestChangelist.AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task PopulateWithViewAsync(bool useHaveTable)
	{
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		List<string> view = ["-/Data/...", "-/shared.h"];
		ChangelistFixture cl = Stream.LatestChangelist;
		List<DepotFileFixture> filtered = [.. cl.StreamFiles
			.Where(x => !x.DepotFile.Contains("shared.h", StringComparison.Ordinal))
			.Where(x => !x.DepotFile.Contains("Data/", StringComparison.Ordinal))];

		List<PopulateRequest> populateRequests = [new PopulateRequest(PerforceConnection, StreamName, view)];
		await ws.PopulateAsync(populateRequests, false, CancellationToken.None);

		ChangelistFixture clViewApplied = new(cl.Number, cl.Description, filtered, cl.IsShelved);
		clViewApplied.AssertDepotFiles(SyncDir);
		await clViewApplied.AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task UnshelveAsync(bool useHaveTable)
	{
		ManagedWorkspace mw = await CreateManagedWorkspaceAsync(useHaveTable);
		await SyncAsync(mw, 7);
		await mw.UnshelveAsync(PerforceConnection, 8, CancellationToken.None);
		Stream.GetChangelist(8).AssertDepotFiles(SyncDir);

		// Have-table still correspond to CL 7 as CL 8 is shelved, only p4 printed to workspace
		await Stream.GetChangelist(7).AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task Unshelve_WithDeletedFiles_NextSync_RestoresDeletedFilesAsync(bool useHaveTable)
	{
		ManagedWorkspace mw = await CreateManagedWorkspaceAsync(useHaveTable);

		// Sync CL 7 (has main.cpp)
		await SyncAsync(mw, 7);
		Stream.GetChangelist(7).AssertDepotFiles(SyncDir);

		// Unshelve CL 8 (deletes main.cpp)
		await mw.UnshelveAsync(PerforceConnection, 8, CancellationToken.None);
		Stream.GetChangelist(8).AssertDepotFiles(SyncDir);

		// Sync CL 7 again - main.cpp should be restored
		await SyncAsync(mw, 7);
		Stream.GetChangelist(7).AssertDepotFiles(SyncDir);
		await Stream.GetChangelist(7).AssertHaveTableAsync(PerforceConnection, useHaveTable);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task Caching_SyncingNewChangelist_UnusedFilesMovedToCacheAsync(bool useHaveTable)
	{
		// Arrange
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		await SyncAsync(ws, 7);

		// Act
		await SyncAsync(ws, 1);

		// Assert - all files from CL 7 are moved to the cache
		PerforceFixture.AssertCacheEquals(CacheDir, [.. Stream.GetChangelist(7).StreamFiles.Select(x => x.Digest)]);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task Caching_SyncingWithCachedData_FilesPopulatedFromCacheAsync(bool useHaveTable)
	{
		// Arrange
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable);
		await SyncAsync(ws, 7);
		await SyncAsync(ws, 1);

		// Act
		await SyncAsync(ws, 7);

		// Assert - cache becomes empty
		PerforceFixture.AssertCacheEquals(CacheDir, []);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task ModTime_NormalSync_PreservesDepotTimestamp_Async(bool useHaveTable)
	{
		// Arrange
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable, true);

		// Act
		await SyncAsync(ws, 7);

		// Assert
		ChangelistFixture cl = Stream.GetChangelist(7);
		DepotFileFixture file = cl.GetFile("//Foo/Main/main.cpp")!;
		Assert.IsNotNull(file);
		Assert.AreEqual(await GetDepotFileModTimeAsync(file), GetLocalFileModTime(file));
	}

	[TestMethod]
	[DataRow(true, DisplayName = "With have-table")]
	[DataRow(false, DisplayName = "Without have-table")]
	public async Task ModTime_CachedSync_PreservesDepotTimestamp_Async(bool useHaveTable)
	{
		// Arrange
		ManagedWorkspace ws = await CreateManagedWorkspaceAsync(useHaveTable, true);
		await SyncAsync(ws, 7);
		await SyncAsync(ws, 1);

		// Act
		await SyncAsync(ws, 7);

		// Assert
		ChangelistFixture cl = Stream.GetChangelist(7);
		DepotFileFixture file = cl.GetFile("//Foo/Main/main.cpp")!;
		Assert.IsNotNull(file);
		Assert.AreEqual(await GetDepotFileModTimeAsync(file), GetLocalFileModTime(file));
	}

	private DateTime GetLocalFileModTime(DepotFileFixture file)
	{
		FileInfo fi = new(Path.Join(SyncDir, file.ClientFile));
		return fi.LastWriteTimeUtc;
	}

	private async Task<DateTime> GetDepotFileModTimeAsync(DepotFileFixture file)
	{
		List<FStatRecord> fstatRecords = await PerforceConnection.FStatAsync($"{file.DepotFile}#{file.Revision}").ToListAsync();
		Assert.AreEqual(1, fstatRecords.Count);
		return fstatRecords[0].HeadModTime;
	}

	[TestMethod]
	public async Task FixtureMetadataMatchesServerMetadataAsync()
	{
		FStatOptions options = FStatOptions.IncludeFileSizes;

		string fileSpec = "//Foo/Main/...";
		List<ChangesRecord> submittedChanges = await PerforceConnection.GetChangesAsync(ChangesOptions.None, -1, ChangeStatus.Submitted, fileSpec);
		List<ChangesRecord> shelvedChanges = await PerforceConnection.GetChangesAsync(ChangesOptions.None, -1, ChangeStatus.Shelved, fileSpec);
		foreach (ChangesRecord cr in submittedChanges.Concat(shelvedChanges).OrderBy(x => x.Number))
		{
			HashSet<(string clientFile, int rev, long size, string digest)> depotFiles = [];
			List<FStatRecord> fstatRecords = await PerforceConnection.FStatAsync(options, fileSpec + "@" + cr.Number).ToListAsync();
			foreach (FStatRecord fsr in fstatRecords)
			{
				if (fsr.HeadAction is FileAction.Add or FileAction.MoveAdd or FileAction.Edit)
				{
					depotFiles.Add((fsr.DepotFile!, fsr.HeadRevision, fsr.FileSize, fsr.Digest!));
				}
			}

			HashSet<(string clientFile, int rev, long size, string digest)> fixtureFiles = [];
			ChangelistFixture changelist = Stream.GetChangelist(cr.Number);
			if (changelist.IsShelved)
			{
				continue;
			}

			foreach (DepotFileFixture depotFile in changelist.StreamFiles)
			{
				fixtureFiles.Add((depotFile.DepotFile, depotFile.Revision, depotFile.Size, depotFile.Digest));
			}

			Assert.AreEqual(depotFiles.Count, fixtureFiles.Count);
			foreach ((string clientFile, int rev, long size, string digest) tuple in fixtureFiles)
			{
				if (!depotFiles.Contains(tuple))
				{
					Assert.Fail("File in fixtures does not exist in depot: " + tuple);
				}
			}
		}
	}

	[TestMethod]
	public async Task ReusePerforceClientWithoutHaveTableAsync()
	{
		// Sync with have-table as normal
		ManagedWorkspace wsWithHave = await CreateManagedWorkspaceAsync(true);
		await wsWithHave.SetupAsync(PerforceConnection, StreamName, CancellationToken.None);
		ChangelistFixture cl = Stream.GetChangelist(6);
		await SyncAsync(wsWithHave, cl.Number);
		cl.AssertDepotFiles(SyncDir);
		await cl.AssertHaveTableAsync(PerforceConnection);

		// Create a new ManagedWorkspace without have-table but re-use same Perforce connection
		// The have-table remnants from above should not interfere with this workspace
		ManagedWorkspace wsWithoutHave = await CreateManagedWorkspaceAsync(false);
		await wsWithoutHave.SetupAsync(PerforceConnection, StreamName, CancellationToken.None);
		await SyncAsync(wsWithoutHave, cl.Number);
		await AssertHaveTableFileCountAsync(0);
		await cl.AssertHaveTableAsync(PerforceConnection, useHaveTable: false);
		cl.AssertDepotFiles(SyncDir);
	}

	private async Task<ManagedWorkspace> CreateManagedWorkspaceAsync(bool useHaveTable, bool useModTime = false)
	{
		ManagedWorkspaceOptions options = new() { UseHaveTable = useHaveTable, UseModTime = useModTime };
		ManagedWorkspace ws = await ManagedWorkspace.CreateAsync(Environment.MachineName, TempDir, options, TracerProvider.Default.GetTracer("Test"), _mwLogger, CancellationToken.None);
		await ws.SetupAsync(PerforceConnection, StreamName, CancellationToken.None);
		return ws;
	}

	private async Task SyncAsync(ManagedWorkspace managedWorkspace, int changeNumber, FileReference? cacheFile = null, bool removeUntracked = true, bool assertFiles = false, bool useHaveTable = true)
	{
		await managedWorkspace.SyncAsync(PerforceConnection, StreamName, changeNumber, [], removeUntracked, false, cacheFile, CancellationToken.None);
		if (assertFiles)
		{
			Stream.GetChangelist(changeNumber).AssertDepotFiles(SyncDir);
			await Stream.GetChangelist(changeNumber).AssertHaveTableAsync(PerforceConnection, useHaveTable);
		}
	}

	private async Task AssertHaveTableFileCountAsync(int expected)
	{
		List<HaveRecord> haveRecords = await PerforceConnection.HaveAsync(new FileSpecList(), CancellationToken.None).ToListAsync();

		if (haveRecords.Count != expected)
		{
			Console.WriteLine("Have table contains:");
			foreach (HaveRecord haveRecord in haveRecords)
			{
				Console.WriteLine(haveRecord.DepotFile + "#" + haveRecord.HaveRev);
			}
			Assert.Fail($"Actual have table file count does not match expected count. Actual={haveRecords.Count} Expected={expected}");
		}
	}
}