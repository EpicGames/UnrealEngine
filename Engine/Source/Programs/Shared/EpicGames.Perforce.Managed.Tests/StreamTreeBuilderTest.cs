// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Managed.Tests;

[TestClass]
public class DepotStreamTreeBuilderTest
{
	private readonly StreamFile _file1 = new("//UE5/Main/main.cpp", 100, new FileContentId(Md5Hash.Zero, new Utf8String("text")), 10);
	private readonly StreamFile _file2 = new("//UE5/Main/Data/data2.bin", 200, new FileContentId(Md5Hash.Zero, new Utf8String("binary")), 20);
	private readonly StreamFile _file3 = new("//UE5/Main/Data/data3.bin", 300, new FileContentId(Md5Hash.Zero, new Utf8String("binary")), 30);
	private readonly StreamFile _file4 = new("//UE5/Main/Data/Audio/Samples/data4.bin", 400, new FileContentId(Md5Hash.Zero, new Utf8String("binary")), 40);

	[TestMethod]
	public void Basic()
	{
		DepotStreamTreeBuilder builder = new();
		builder.AddFile("/main.cpp", _file1);
		builder.AddFile("Data/data2.bin", _file2);
		builder.AddFile("Data/data3.bin", _file3);
		builder.AddFile("Data/Audio/Samples/data4.bin", _file4);
		StreamSnapshotFromMemory snapshot = new(builder);

		StreamTree root = snapshot.Lookup(snapshot.Root);
		{
			Assert.AreEqual(1, root.NameToFile.Count);
			Assert.AreEqual(100, root.NameToFile[new Utf8String("main.cpp")].Length);

			Assert.AreEqual(1, root.NameToTree.Count);
		}

		StreamTree data = snapshot.Lookup(root.NameToTree[new Utf8String("Data")]);
		{
			Assert.AreEqual(2, data.NameToFile.Count);
			Assert.AreEqual(1, data.NameToTree.Count);

			Assert.AreEqual(_file2.Length, data.NameToFile[new Utf8String("data2.bin")].Length);
			Assert.AreEqual(_file3.Length, data.NameToFile[new Utf8String("data3.bin")].Length);
		}

		StreamTree audio = snapshot.Lookup(data.NameToTree[new Utf8String("Audio")]);
		{
			Assert.AreEqual(0, audio.NameToFile.Count);
			Assert.AreEqual(1, audio.NameToTree.Count);
		}

		StreamTree samples = snapshot.Lookup(audio.NameToTree[new Utf8String("Samples")]);
		{
			Assert.AreEqual(1, samples.NameToFile.Count);
			Assert.AreEqual(0, samples.NameToTree.Count);

			Assert.AreEqual(_file4.Length, samples.NameToFile[new Utf8String("data4.bin")].Length);
		}
	}
}