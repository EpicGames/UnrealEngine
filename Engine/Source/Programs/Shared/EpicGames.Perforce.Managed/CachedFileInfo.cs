// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Stores information about where a particular FileContentId has been staged, or its location in the cache
	/// </summary>
	[DebuggerDisplay("{ContentId}")]
	class CachedFileInfo
	{
		public readonly DirectoryReference CacheDir;
		public readonly FileContentId ContentId;
		public readonly ulong CacheId;
		public readonly long Length;
		public readonly long LastModifiedTicks;
		public readonly bool ReadOnly;
		public readonly uint SequenceNumber;

		public CachedFileInfo(DirectoryReference cacheDir, FileContentId contentId, ulong cacheId, long length, long lastModifiedTicks, bool readOnly, uint sequenceNumber)
		{
			CacheDir = cacheDir;
			ContentId = contentId;
			CacheId = cacheId;
			Length = length;
			LastModifiedTicks = lastModifiedTicks;
			ReadOnly = readOnly;
			SequenceNumber = sequenceNumber;
		}

		public bool CheckIntegrity(ILogger logger)
		{
			FileReference location = GetLocation();

			FileInfo info = new(location.FullName);
			if (!info.Exists)
			{
				logger.LogWarning("warning: {File} was missing from cache.", location);
				return false;
			}
			if (info.Length != Length)
			{
				logger.LogWarning("warning: {File} was {Size:n} bytes; expected {ExpectedSize:n} bytes", location, info.Length, Length);
				return false;
			}
			if (info.LastWriteTimeUtc.Ticks != LastModifiedTicks)
			{
				logger.LogWarning("warning: {File} was last modified at {Time}; expected {ExpectedTime}", location, info.LastWriteTimeUtc, new DateTime(LastModifiedTicks, DateTimeKind.Utc));
				return false;
			}
			if (info.Attributes.HasFlag(FileAttributes.ReadOnly) != ReadOnly)
			{
				logger.LogWarning("warning: {File} readonly flag is {Flag}; expected {ExpectedFlag}", location, info.Attributes.HasFlag(FileAttributes.ReadOnly), ReadOnly);
				return false;
			}

			return true;
		}

		public FileReference GetLocation()
		{
			StringBuilder fullName = new(CacheDir.FullName);
			fullName.Append(Path.DirectorySeparatorChar);
			fullName.AppendFormat("{0:X}", (CacheId >> 60) & 0xf);
			fullName.Append(Path.DirectorySeparatorChar);
			fullName.AppendFormat("{0:X}", (CacheId >> 56) & 0xf);
			fullName.Append(Path.DirectorySeparatorChar);
			fullName.AppendFormat("{0:X}", (CacheId >> 62) & 0xf);
			fullName.Append(Path.DirectorySeparatorChar);
			fullName.AppendFormat("{0:X}", CacheId);
			return new FileReference(fullName.ToString());
		}
	}

	static class CachedFileInfoExtensions
	{
		public static CachedFileInfo ReadCachedFileInfo(this MemoryReader reader, DirectoryReference cacheDir)
		{
			FileContentId contentId = reader.ReadFileContentId();
			ulong cacheId = reader.ReadUInt64();
			long length = reader.ReadInt64();
			long lastModifiedTicks = reader.ReadInt64();
			bool readOnly = reader.ReadBoolean();
			uint sequenceNumber = reader.ReadUInt32();
			return new CachedFileInfo(cacheDir, contentId, cacheId, length, lastModifiedTicks, readOnly, sequenceNumber);
		}

		public static void WriteCachedFileInfo(this MemoryWriter writer, CachedFileInfo fileInfo)
		{
			writer.WriteFileContentId(fileInfo.ContentId);
			writer.WriteUInt64(fileInfo.CacheId);
			writer.WriteInt64(fileInfo.Length);
			writer.WriteInt64(fileInfo.LastModifiedTicks);
			writer.WriteBoolean(fileInfo.ReadOnly);
			writer.WriteUInt32(fileInfo.SequenceNumber);
		}

		public static int GetSerializedSize(this CachedFileInfo fileInfo)
		{
			return fileInfo.ContentId.GetSerializedSize() + sizeof(ulong) + sizeof(long) + sizeof(long) + sizeof(byte) + sizeof(uint);
		}
	}
}

