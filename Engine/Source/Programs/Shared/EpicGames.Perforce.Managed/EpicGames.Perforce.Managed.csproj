<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <PackageReference Include="OpenTelemetry" Version="1.8.1" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>
</Project>
