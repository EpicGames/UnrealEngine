// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Unique identifier for the content of a file
	/// </summary>
	[DebuggerDisplay("{Digest} ({Type})")]
	public class FileContentId
	{
		/// <summary>
		/// MD5 digest of the file
		/// </summary>
		public Md5Hash Digest
		{
			get;
		}

		/// <summary>
		/// Type of the file
		/// </summary>
		public Utf8String Type
		{
			get;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public FileContentId(Md5Hash digest, Utf8String type)
		{
			Digest = digest;
			Type = type;
		}

		/// <inheritdoc/>
		public override bool Equals(object? other)
		{
			return (other is FileContentId otherFile) && Digest == otherFile.Digest && Type == otherFile.Type;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return HashCode.Combine(Digest.GetHashCode(), Type.GetHashCode());
		}
	}

	static class FileContentIdExtensions
	{
		public static FileContentId ReadFileContentId(this MemoryReader reader)
		{
			Md5Hash digest = reader.ReadMd5Hash();
			Utf8String type = reader.ReadNullTerminatedUtf8String();
			return new FileContentId(digest, type);
		}

		public static void WriteFileContentId(this MemoryWriter writer, FileContentId fileContentId)
		{
			writer.WriteMd5Hash(fileContentId.Digest);
			writer.WriteNullTerminatedUtf8String(fileContentId.Type);
		}

		public static int GetSerializedSize(this FileContentId fileContentId)
		{
			return Digest<Md5>.Length + fileContentId.Type.GetNullTerminatedSize();
		}
	}
}
