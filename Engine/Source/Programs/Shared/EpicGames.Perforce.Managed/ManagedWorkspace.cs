// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using OpenTelemetry.Trace;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Exception relating to managed workspace
	/// </summary>
	public class ManagedWorkspaceException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ManagedWorkspaceException(string message) : base(message)
		{
		}
	}

	/// <summary>
	/// Exception thrown when there is not enough free space on the drive
	/// </summary>
	public class InsufficientSpaceException : ManagedWorkspaceException
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Error message</param>
		public InsufficientSpaceException(string message)
			: base(message)
		{
		}
	}

	/// <summary>
	/// Information about a populate request
	/// </summary>
	public class PopulateRequest
	{
		/// <summary>
		/// The Perforce connection
		/// </summary>
		public IPerforceConnection PerforceClient { get; }

		/// <summary>
		/// Stream to sync it to
		/// </summary>
		public string StreamName { get; }

		/// <summary>
		/// View for this client
		/// </summary>
		public IReadOnlyList<string> View { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="perforceClient">The perforce connection</param>
		/// <param name="streamName">Stream to be synced</param>
		/// <param name="view">List of filters for the stream</param>
		public PopulateRequest(IPerforceConnection perforceClient, string streamName, IReadOnlyList<string> view)
		{
			PerforceClient = perforceClient;
			StreamName = streamName;
			View = view;
		}
	}

	/// <summary>
	/// Extra options for configuring ManagedWorkspace
	/// </summary>
	/// <param name="NumParallelSyncThreads">Maximum number of threads to sync in parallel</param>
	/// <param name="MaxFileConcurrency">Maximum number of concurrent file system operations (copying, moving, deleting)</param>
	/// <param name="MaxFileScanConcurrency">Maximum number of current file system scaning operations (file system discovery, traversing)</param>
	/// <param name="MinScratchSpace">Minimum amount of disk space that must be available after a sync (in megabytes)</param>
	/// <param name="UseHaveTable">
	///		Use the client's have table when syncing.
	///		
	///		When set to false, updates to the have table will be prevented through use of "sync -p".
	///		Actual files to sync will be gathered through "fstat". This puts less strain on the Perforce server and can improve sync performance.
	///	</param>
	/// <param name="Partitioned">Whether to allow using partitioned workspaces</param>
	/// <param name="PreferNativeClient">Whether to prefer the native p4 client</param>
	/// <param name="UseModTime">Whether to set a synced file's timestamp to match P4 stream/depot timestamp ("modtime" client option)</param>
	public record ManagedWorkspaceOptions
	(
		int NumParallelSyncThreads,
		int MaxFileConcurrency,
		int MaxFileScanConcurrency,
		long MinScratchSpace,
		bool UseHaveTable,
		bool Partitioned,
		bool PreferNativeClient,
		bool UseModTime
	)
	{
		/// <inheritdoc cref="ManagedWorkspaceOptions(Int32, Int32, Int32, Int64, Boolean, Boolean, Boolean, Boolean)"/>
		public ManagedWorkspaceOptions(int? numParallelSyncThreads = null, int? maxFileConcurrency = null, int? maxFileScanConcurrency = null, long minScratchSpace = 50L * 1024, bool useHaveTable = true, bool partitioned = false, bool preferNativeClient = false, bool useModTime = false)
			: this(
				  numParallelSyncThreads ?? GetDefaultThreadCount(4),
				  maxFileConcurrency ?? GetDefaultThreadCount(4),
				  maxFileScanConcurrency ?? GetDefaultThreadCount(4),
				  minScratchSpace,
				  useHaveTable,
				  partitioned,
				  preferNativeClient,
				  useModTime)
		{ }

		static int GetDefaultThreadCount(int defaultValue)
			=> Math.Min(defaultValue, Math.Max(Environment.ProcessorCount - 1, 1));
	}

	/// <summary>
	/// Version number for managed workspace cache files
	/// </summary>
	enum ManagedWorkspaceVersion
	{
		/// <summary>
		/// Initial version number
		/// </summary>
		Initial = 2,

		/// <summary>
		/// Including stream directory digests in workspace directories
		/// </summary>
		AddDigest = 3,

		/// <summary>
		/// Changing hash algorithm from SHA1 to IoHash
		/// </summary>
		AddDigestIoHash = 4,
	}

	/// <summary>
	/// Represents a repository of streams and cached data
	/// </summary>
	public class ManagedWorkspace
	{
		/// <summary>
		/// The current transaction state. Used to determine whether a repository needs to be cleaned on startup.
		/// </summary>
		enum TransactionState
		{
			Dirty,
			Clean,
		}

		/// <summary>
		/// The file signature and version. Update this to introduce breaking changes and ignore old repositories.
		/// </summary>
		const int CurrentSignature = ('W' << 24) | ('T' << 16) | 2;

		/// <summary>
		/// The current revision number for cache archives.
		/// </summary>
		static int CurrentVersion { get; } = Enum.GetValues(typeof(ManagedWorkspaceVersion)).Cast<int>().Max();

		/// <summary>
		/// File format version for untracked state file
		/// </summary>
		private const int UntrackedFileVersion = 1;

		/// <summary>
		/// Externally configurable options
		/// </summary>
		private readonly ManagedWorkspaceOptions _options;

		/// <summary>
		/// Constant for syncing the latest change number
		/// </summary>
		public const int LatestChangeNumber = -1;

		/// <summary>
		/// Name of the signature file for a repository. This 
		/// </summary>
		const string SignatureFileName = "Repository.sig";

		/// <summary>
		/// Name of the main data file for a repository
		/// </summary>
		const string DataFileName = "Repository.dat";

		/// <summary>
		/// Name of the untracked state file for a repository
		/// </summary>
		const string UntrackedStateFileName = "Untracked.dat";

		/// <summary>
		/// Name of the host
		/// </summary>
		readonly string _hostName;

		/// <summary>
		/// Incrementing number assigned to sequential operations that modify files. Used to age out files in the cache.
		/// </summary>
		uint _nextSequenceNumber;

		/// <summary>
		/// Whether a repair operation should be run on this workspace. Set whenever the state may be inconsistent.
		/// </summary>
		bool _requiresRepair;

		/// <summary>
		/// Tracer
		/// </summary>
		readonly Tracer _tracer;

		/// <summary>
		/// The log output device
		/// </summary>
		readonly ILogger _logger;

		/// <summary>
		/// The root directory for the stash
		/// </summary>
		readonly DirectoryReference _baseDir;

		/// <summary>
		/// Root directory for storing cache files
		/// </summary>
		readonly DirectoryReference _cacheDir;

		/// <summary>
		/// Root directory for storing workspace files
		/// </summary>
		readonly DirectoryReference _workspaceDir;

		/// <summary>
		/// Set of clients that we're created. Used to avoid updating multiple times during one run.
		/// </summary>
		readonly Dictionary<string, ClientRecord> _createdClients = [];

		/// <summary>
		/// Set of unique cache entries. We use this to ensure new names in the cache are unique.
		/// </summary>
		readonly HashSet<ulong> _cacheEntries = [];

		/// <summary>
		/// List of all the staged files
		/// </summary>
		WorkspaceDirectoryInfo _workspace;

		/// <summary>
		/// All the files which are currently being tracked
		/// </summary>
		Dictionary<FileContentId, CachedFileInfo> _contentIdToTrackedFile = [];

		/// <summary>
		/// Notification that a clean has been performed
		/// </summary>
		public delegate void OnCleanDelegate(int numFilesDeleted, int numDirsDeleted);

		/// <summary>
		/// Occurs when a clean operation has been performed.
		/// </summary>
		public event OnCleanDelegate? OnClean;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="hostName">Name of the current host</param>
		/// <param name="nextSequenceNumber">The next sequence number for operations</param>
		/// <param name="baseDir">The root directory for the stash</param>
		/// <param name="options">Extra options</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">The log output device</param>
		private ManagedWorkspace(string hostName, uint nextSequenceNumber, DirectoryReference baseDir, ManagedWorkspaceOptions options, Tracer tracer, ILogger logger)
		{
			// Save the Perforce settings
			_hostName = hostName;
			_nextSequenceNumber = nextSequenceNumber;
			_tracer = tracer;
			_logger = logger;
			_options = options;

			// Get all the directories
			_baseDir = baseDir;
			DirectoryReference.CreateDirectory(baseDir);

			_cacheDir = DirectoryReference.Combine(baseDir, "Cache");
			DirectoryReference.CreateDirectory(_cacheDir);

			_workspaceDir = DirectoryReference.Combine(baseDir, "Sync");
			DirectoryReference.CreateDirectory(_workspaceDir);

			// Create the workspace
			_workspace = new WorkspaceDirectoryInfo(_workspaceDir);
		}

		/// <summary>
		/// Loads a repository from the given directory, or create it if it doesn't exist
		/// </summary>
		/// <param name="hostName">Name of the current machine. Will be automatically detected from the host settings if not present.</param>
		/// <param name="baseDir">The base directory for the repository</param>
		/// <param name="overwrite">Whether to allow overwriting a repository that's not up to date</param>
		/// <param name="options">Extra options</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">The logging interface</param>
		/// <param name="cancellationToken">Cancellation token for this operation</param>
		/// <returns></returns>
		public static async Task<ManagedWorkspace> LoadOrCreateAsync(string hostName, DirectoryReference baseDir, bool overwrite, ManagedWorkspaceOptions options, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			if (Exists(baseDir))
			{
				try
				{
					return await LoadAsync(hostName, baseDir, options, tracer, logger, cancellationToken);
				}
				catch (Exception ex)
				{
					if (overwrite)
					{
						logger.LogWarning(ex, "Unable to load existing repository.");
					}
					else
					{
						throw;
					}
				}
			}

			return await CreateAsync(hostName, baseDir, options, tracer, logger, cancellationToken);
		}
		/*
				public static PerforceConnection GetPerforceConnection(PerforceConnection Perforce)
				{
					if (Perforce.UserName == null || HostName == null)
					{
						InfoRecord ServerInfo = await Perforce.GetInfoAsync(InfoOptions.ShortOutput, CancellationToken);
						if (Perforce.UserName == null)
						{
							Perforce = new PerforceConnection(Perforce) { UserName = ServerInfo.UserName };
						}
						if (HostName == null)
						{
							if (ServerInfo.ClientHost == null)
							{
								throw new Exception("Unable to determine host name");
							}
							else
							{
								HostName = ServerInfo.ClientHost;
							}
						}
					}
					return Perforce;
				}
		*/

		/// <summary>
		/// Creates a repository at the given location
		/// </summary>
		/// <param name="hostName">Name of the current machine.</param>
		/// <param name="baseDir">The base directory for the repository</param>
		/// <param name="options">Extra options</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">The log output device</param>
		/// <param name="cancellationToken">Cancellation token for this operation</param>
		/// <returns>New repository instance</returns>
		public static async Task<ManagedWorkspace> CreateAsync(string hostName, DirectoryReference baseDir, ManagedWorkspaceOptions options, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(CreateAsync)}");
			logger.LogInformation("Creating repository at {Location}...", baseDir);

			// Make sure all the fields are valid
			DirectoryReference.CreateDirectory(baseDir);
			FileUtils.ForceDeleteDirectoryContents(baseDir);

			ManagedWorkspace repo = new(hostName, 1, baseDir, options, tracer, logger);
			await repo.SaveAsync(TransactionState.Clean, cancellationToken);
			repo.CreateCacheHierarchy();

			FileReference signatureFile = FileReference.Combine(baseDir, SignatureFileName);
			using (BinaryWriter writer = new(File.Open(signatureFile.FullName, FileMode.Create, FileAccess.Write, FileShare.Read)))
			{
				writer.Write(CurrentSignature);
			}

			return repo;
		}

		/// <summary>
		/// Tests whether a repository exists in the given directory
		/// </summary>
		/// <param name="baseDir"></param>
		/// <returns></returns>
		public static bool Exists(DirectoryReference baseDir)
		{
			FileReference signatureFile = FileReference.Combine(baseDir, SignatureFileName);
			if (FileReference.Exists(signatureFile))
			{
				using (BinaryReader reader = new(File.Open(signatureFile.FullName, FileMode.Open, FileAccess.Read, FileShare.Read)))
				{
					int signature = reader.ReadInt32();
					if (signature == CurrentSignature)
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Loads a repository from disk
		/// </summary>
		/// <param name="hostName">Name of the current host. Will be obtained from a 'p4 info' call if not specified</param>
		/// <param name="baseDir">The base directory for the repository</param>
		/// <param name="options">Extra options</param> 
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">The log output device</param>
		/// <param name="cancellationToken">Cancellation token for this command</param>
		public static async Task<ManagedWorkspace> LoadAsync(string hostName, DirectoryReference baseDir, ManagedWorkspaceOptions options, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(LoadAsync)}");

			if (!Exists(baseDir))
			{
				throw new FatalErrorException("No valid repository found at {0}", baseDir);
			}

			FileReference dataFile = FileReference.Combine(baseDir, DataFileName);
			RestoreBackup(dataFile);

			byte[] data = await FileReference.ReadAllBytesAsync(dataFile, cancellationToken);
			MemoryReader reader = new(data.AsMemory());

			int version = reader.ReadInt32();
			if (version > CurrentVersion)
			{
				throw new FatalErrorException("Unsupported data format (version {0}, current {1})", version, CurrentVersion);
			}

			bool requiresRepair = reader.ReadBoolean();
			uint nextSequenceNumber = reader.ReadUInt32();

			ManagedWorkspace repo = new(hostName, nextSequenceNumber, baseDir, options, tracer, logger);
			repo._requiresRepair = requiresRepair;

			int numTrackedFiles = reader.ReadInt32();
			for (int idx = 0; idx < numTrackedFiles; idx++)
			{
				CachedFileInfo trackedFile = reader.ReadCachedFileInfo(repo._cacheDir);
				repo._contentIdToTrackedFile.Add(trackedFile.ContentId, trackedFile);
				repo._cacheEntries.Add(trackedFile.CacheId);
			}

			reader.ReadWorkspaceDirectoryInfo(repo._workspace, (ManagedWorkspaceVersion)version);

			await repo.RunOptionalCacheRepairAsync(cancellationToken);
			return repo;
		}

		/// <summary>
		/// Save the state of the repository
		/// </summary>
		private async Task SaveAsync(TransactionState state, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(SaveAsync)}");

			// Allocate the buffer for writing
			int serializedSize = sizeof(int) + sizeof(byte) + sizeof(int) + sizeof(int) + _contentIdToTrackedFile.Values.Sum(x => x.GetSerializedSize()) + _workspace.GetSerializedSize();
			byte[] buffer = new byte[serializedSize];

			// Write the data to memory
			MemoryWriter writer = new(buffer.AsMemory());
			writer.WriteInt32(CurrentVersion);
			writer.WriteBoolean(_requiresRepair || (state != TransactionState.Clean));
			writer.WriteUInt32(_nextSequenceNumber);
			writer.WriteInt32(_contentIdToTrackedFile.Count);
			foreach (CachedFileInfo trackedFile in _contentIdToTrackedFile.Values)
			{
				writer.WriteCachedFileInfo(trackedFile);
			}
			writer.WriteWorkspaceDirectoryInfo(_workspace);
			writer.CheckEmpty();

			// Write it to disk
			FileReference dataFile = FileReference.Combine(_baseDir, DataFileName);
			BeginTransaction(dataFile);
			await FileReference.WriteAllBytesAsync(dataFile, buffer, cancellationToken);
			CompleteTransaction(dataFile);
		}

		/// <summary>
		/// Mark whether current repository in combination with P4 client contains untracked files
		/// Stores data in a file separate to the repository metadata as that can grow large (hundreds of megabytes)
		/// </summary>
		private async Task SaveUntrackedStateAsync(IPerforceConnection connection, bool containsUntrackedFiles, CancellationToken cancellationToken)
		{
			if (connection.Settings.ClientName == null)
			{
				throw new ArgumentException("Unable to save untracked state as P4 client name is not set");
			}

			using MemoryStream ms = new(300);
			await using BinaryWriter bw = new(ms);
			bw.Write(UntrackedFileVersion);
			bw.Write(connection.Settings.ClientName);
			bw.Write(containsUntrackedFiles);

			// Write it to disk
			FileReference dataFile = FileReference.Combine(_baseDir, UntrackedStateFileName);
			BeginTransaction(dataFile);
			await FileReference.WriteAllBytesAsync(dataFile, ms.ToArray(), cancellationToken);
			CompleteTransaction(dataFile);
		}

		/// <summary>
		/// Check if current repository in combination with P4 client contains untracked files
		/// If deserialization is not possible, assume the repo contains untracked files 
		/// </summary>
		/// <returns>True if repository contains untracked files</returns>
		private async Task<bool> LoadUntrackedStateAsync(IPerforceConnection connection, CancellationToken cancellationToken)
		{
			try
			{
				FileReference dataFile = FileReference.Combine(_baseDir, UntrackedStateFileName);
				if (!File.Exists(dataFile.FullName))
				{
					return true;
				}

				byte[] data = await File.ReadAllBytesAsync(dataFile.FullName, cancellationToken);
				using MemoryStream ms = new(data);
				using BinaryReader br = new(ms);

				int version = br.ReadInt32();
				if (version != UntrackedFileVersion)
				{
					return true;
				}

				string clientName = br.ReadString();
				bool containsUntrackedFiles = br.ReadBoolean();
				return clientName != connection.Settings.ClientName || containsUntrackedFiles;
			}
			catch (Exception)
			{
				return true;
			}
		}

		#region Commands

		/// <summary>
		/// Cleans the current workspace
		/// </summary>
		/// <param name="perforce">Optional Perforce client, if not set untracked state file cannot be updated</param>
		/// <param name="removeUntracked">Whether to remove untracked files</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task CleanAsync(IPerforceConnection? perforce, bool removeUntracked, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();

			_logger.LogInformation("Cleaning workspace...");
			using (_logger.BeginIndentScope("  "))
			{
				await CleanInternalAsync(removeUntracked, cancellationToken);
			}

			if (perforce != null && removeUntracked)
			{
				await SaveUntrackedStateAsync(perforce, false, cancellationToken);
			}

			_logger.LogInformation("Completed in {ElapsedTime}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Cleans the current workspace
		/// </summary>
		/// <param name="removeUntracked">Whether to remove untracked files</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task CleanInternalAsync(bool removeUntracked, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(CleanInternalAsync)}");

			FileInfo[] filesToDelete;
			DirectoryInfo[] directoriesToDelete;
			using (Trace("FindFilesToClean"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Finding files to clean..."))
			{
				using TelemetrySpan findFilesSpan = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.FindFilesToClean");
				Stopwatch timer = Stopwatch.StartNew();

				(FileInfo[] filesToDelete, DirectoryInfo[] directoriesToDelete) refreshResult = await _workspace.RefreshAsync(removeUntracked, _options.MaxFileScanConcurrency);
				filesToDelete = refreshResult.filesToDelete;
				directoriesToDelete = refreshResult.directoriesToDelete;
				findFilesSpan.SetAttribute("horde.mw.num_delete_files", filesToDelete.Length);
				findFilesSpan.SetAttribute("horde.mw.num_delete_dirs", directoriesToDelete.Length);

				status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}

			if (filesToDelete.Length > 0 || directoriesToDelete.Length > 0)
			{
				List<string> paths =
				[
					.. directoriesToDelete.Select(x => String.Format("/{0}/...", new DirectoryReference(x).MakeRelativeTo(_workspaceDir).Replace(Path.DirectorySeparatorChar, '/'))),
					.. filesToDelete.Select(x => String.Format("/{0}", new FileReference(x).MakeRelativeTo(_workspaceDir).Replace(Path.DirectorySeparatorChar, '/'))),
				];

				const int MaxDisplay = 1000;
				foreach (string path in paths.OrderBy(x => x).Take(MaxDisplay))
				{
					_logger.LogInformation("  {Path}", path);
				}
				if (paths.Count > MaxDisplay)
				{
					_logger.LogInformation("  +{NumPaths:n0} more", paths.Count - MaxDisplay);
				}

				using (Trace("CleanFiles"))
				using (ILoggerProgress scope = _logger.BeginProgressScope("Cleaning files..."))
				{
					using TelemetrySpan cleanSpan = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.CleaningFilesAndDirs");
					Stopwatch timer = Stopwatch.StartNew();

					ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency, CancellationToken = cancellationToken };
					{
						using TelemetrySpan cleanFilesSpan = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.CleaningFiles");
						await Parallel.ForEachAsync(filesToDelete, options, (fileToDelete, ct) =>
						{
							FileUtils.ForceDeleteFile(fileToDelete);
							return ValueTask.CompletedTask;
						});
					}

					{
						using TelemetrySpan cleanDirsSpan = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.CleaningDirs");
						await Parallel.ForEachAsync(directoriesToDelete, options, (directoryToDelete, ct) =>
						{
							FileUtils.ForceDeleteDirectory(directoryToDelete);
							return ValueTask.CompletedTask;
						});
					}

					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}

				await SaveAsync(TransactionState.Clean, cancellationToken);
			}

			OnClean?.Invoke(filesToDelete.Length, directoriesToDelete.Length);
		}

		/// <summary>
		/// Empties the staging directory of any staged files
		/// </summary>
		public async Task ClearAsync(CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();

			_logger.LogInformation("Clearing workspace...");
			using (Trace("Clear"))
			using (_logger.BeginIndentScope("  "))
			{
				await CleanInternalAsync(true, cancellationToken);
				await RemoveFilesFromWorkspaceAsync(StreamSnapshot.Empty, cancellationToken);
				await SaveAsync(TransactionState.Clean, cancellationToken);
			}

			_logger.LogInformation("Completed in {ElapsedTime}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Dumps the contents of the repository to the log for analysis
		/// </summary>
		public void Dump()
		{
			Stopwatch timer = Stopwatch.StartNew();

			_logger.LogInformation("Dumping repository to log...");

			WorkspaceFileInfo[] workspaceFiles = [.. _workspace.GetFiles().OrderBy(x => x.GetLocation().FullName)];
			if (workspaceFiles.Length > 0)
			{
				_logger.LogDebug("  Workspace:");
				foreach (WorkspaceFileInfo file in workspaceFiles)
				{
					_logger.LogDebug("    {File,-128} [{ContentId,-48}] [{Length,20:n0}] [{LastModified,20}]{Writable}", file.GetClientPath(), file.ContentId, file._length, file._lastModifiedTicks, file._readOnly ? "" : " [ writable ]");
				}
			}

			if (_contentIdToTrackedFile.Count > 0)
			{
				_logger.LogDebug("  Cache:");
				foreach (KeyValuePair<FileContentId, CachedFileInfo> pair in _contentIdToTrackedFile)
				{
					_logger.LogDebug("    {File,-128} [{ContentId,-48}] [{Length,20:n0}] [{LastModified,20}]{Writable}", pair.Value.GetLocation(), pair.Key, pair.Value.Length, pair.Value.LastModifiedTicks, pair.Value.ReadOnly ? "" : "[ writable ]");
				}
			}

			_logger.LogInformation("Completed in {ElapsedTime}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Checks the integrity of the cache
		/// </summary>
		public async Task RepairCacheAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(RepairCacheAsync)}");

			using (Trace("Repair"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Checking cache..."))
			{
				// Make sure all the folders exist in the cache
				CreateCacheHierarchy();

				List<CachedFileInfo> trackedFiles = [.. _contentIdToTrackedFile.Values];

				// Check that all the files in the cache appear as we expect them to
				const int MaxLoggedMissingFiles = 250;
				int numMissingFiles = 0;
				foreach (CachedFileInfo trackedFile in trackedFiles)
				{
					if (!trackedFile.CheckIntegrity((numMissingFiles < MaxLoggedMissingFiles) ? _logger : NullLogger.Instance))
					{
						RemoveTrackedFile(trackedFile);
						numMissingFiles++;
					}
				}
				if (numMissingFiles > MaxLoggedMissingFiles)
				{
					_logger.LogWarning("+ {Count} more", numMissingFiles - MaxLoggedMissingFiles);
				}

				// Clear the repair flag
				_requiresRepair = false;

				await SaveAsync(TransactionState.Clean, cancellationToken);

				status.Progress = "Done";
			}
		}

		/// <summary>
		/// Cleans the current workspace
		/// </summary>
		public async Task RevertAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();

			await RevertInternalAsync(perforceClient, cancellationToken);

			_logger.LogInformation("Completed in {ElapsedTime}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Checks the <see cref="_requiresRepair"/> flag, and repairs/resets it if set.
		/// </summary>
		private async Task RunOptionalCacheRepairAsync(CancellationToken cancellationToken)
		{
			if (_requiresRepair)
			{
				await RepairCacheAsync(cancellationToken);
			}
		}

		/// <summary>
		/// Shrink the size of the cache to the given size
		/// </summary>
		/// <param name="maxSize">The maximum cache size, in bytes</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task PurgeAsync(long maxSize, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(PurgeAsync)}");
			_logger.LogInformation("Purging cache (limit {MaxSize:n0} bytes)...", maxSize);
			using (Trace("Purge"))
			using (_logger.BeginIndentScope("  "))
			{
				List<CachedFileInfo> cachedFiles = [.. _contentIdToTrackedFile.Values.OrderBy(x => x.SequenceNumber)];

				int numRemovedFiles = 0;
				long totalSize = cachedFiles.Sum(x => x.Length);

				while (maxSize < totalSize && numRemovedFiles < cachedFiles.Count)
				{
					CachedFileInfo file = cachedFiles[numRemovedFiles];

					RemoveTrackedFile(file);
					totalSize -= file.Length;

					numRemovedFiles++;
				}

				await SaveAsync(TransactionState.Clean, cancellationToken);

				_logger.LogInformation("{NumFilesRemoved} files removed, {NumFilesRemaining} files remaining, new size {NewSize:n0} bytes.", numRemovedFiles, cachedFiles.Count - numRemovedFiles, totalSize);
			}
		}

		/// <summary>
		/// Configures the client for the given stream
		/// </summary>
		/// <param name="perforceClient">The Perforce connection</param>
		/// <param name="streamName">Name of the stream</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task SetupAsync(IPerforceConnection perforceClient, string streamName, CancellationToken cancellationToken)
		{
			await UpdateClientAsync(perforceClient, streamName, cancellationToken);
		}

		/// <summary>
		/// Prints stats showing coherence between different streams
		/// </summary>
		public async Task StatsAsync(IPerforceConnection perforceClient, List<string> streamNames, List<string> view, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(StatsAsync)}");

			_logger.LogInformation("Finding stats for {NumStreams} streams", streamNames.Count);
			using (_logger.BeginIndentScope("  "))
			{
				// Update the list of files in each stream
				Tuple<int, StreamSnapshot>[] streamState = new Tuple<int, StreamSnapshot>[streamNames.Count];
				for (int idx = 0; idx < streamNames.Count; idx++)
				{
					string streamName = streamNames[idx];
					_logger.LogInformation("Finding contents of {StreamName}:", streamName);

					using (_logger.BeginIndentScope("  "))
					{
						_createdClients.Remove(perforceClient.Settings.ClientName!); // Force the client to be updated

						await UpdateClientAsync(perforceClient, streamName, cancellationToken);

						int changeNumber = await GetLatestClientChangeAsync(perforceClient, cancellationToken);
						_logger.LogInformation("Latest change is CL {ChangeNumber}", changeNumber);

						await RevertInternalAsync(perforceClient, cancellationToken);
						await ClearClientHaveTableAsync(perforceClient, cancellationToken);
						await UpdateClientHaveTableAsync(perforceClient, changeNumber, view, cancellationToken);

						StreamSnapshot contents = await FindClientContentsAsync(perforceClient, changeNumber, cancellationToken);
						streamState[idx] = Tuple.Create(changeNumber, contents);

						GC.Collect();
					}
				}

				// Find stats for 
				using (Trace("Stats"))
				using (ILoggerProgress scope = _logger.BeginProgressScope("Finding usage stats..."))
				{
					Stopwatch timer = Stopwatch.StartNew();

					// Find the set of files in each stream
					HashSet<FileContentId>[] filesInStream = new HashSet<FileContentId>[streamNames.Count];
					for (int idx = 0; idx < streamNames.Count; idx++)
					{
						List<StreamFile> files = streamState[idx].Item2.GetFiles();
						filesInStream[idx] = [.. files.Select(x => x.ContentId)];
					}

					// Build a table showing amount of unique content in each stream
					string[][] cells = new string[streamNames.Count + 1][];
					for (int idx = 0; idx < cells.Length; idx++)
					{
						cells[idx] = new string[streamNames.Count + 1];
					}
					cells[0][0] = "";
					for (int idx = 0; idx < streamNames.Count; idx++)
					{
						cells[idx + 1][0] = streamNames[idx];
						cells[0][idx + 1] = streamNames[idx];
					}

					// Populate the table
					for (int rowIdx = 0; rowIdx < streamNames.Count; rowIdx++)
					{
						List<StreamFile> files = streamState[rowIdx].Item2.GetFiles();
						for (int colIdx = 0; colIdx < streamNames.Count; colIdx++)
						{
							long diffSize = files.Where(x => !filesInStream[colIdx].Contains(x.ContentId)).Sum(x => x.Length);
							cells[rowIdx + 1][colIdx + 1] = String.Format("{0:0.0}mb", diffSize / (1024.0 * 1024.0));
						}
					}

					// Find the width of each row
					int[] colWidths = new int[streamNames.Count + 1];
					for (int colIdx = 0; colIdx < streamNames.Count + 1; colIdx++)
					{
						for (int rowIdx = 0; rowIdx < streamNames.Count + 1; rowIdx++)
						{
							colWidths[colIdx] = Math.Max(colWidths[colIdx], cells[rowIdx][colIdx].Length);
						}
					}

					// Print the table
					_logger.LogInformation("");
					_logger.LogInformation("Each row shows the size of files in a stream which are unique to that stream compared to each column:");
					_logger.LogInformation("");
					for (int rowIdx = 0; rowIdx < streamNames.Count + 1; rowIdx++)
					{
						StringBuilder row = new();
						for (int colIdx = 0; colIdx < streamNames.Count + 1; colIdx++)
						{
							string cell = cells[rowIdx][colIdx];
							row.Append(' ', colWidths[colIdx] - cell.Length);
							row.Append(cell);
							row.Append(" | ");
						}
						_logger.LogInformation("{Row}", row.ToString());
					}
					_logger.LogInformation("");

					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}
		}

		/// <summary>
		/// Prints information about the repository state
		/// </summary>
		public async Task StatusAsync()
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(StatusAsync)}");

			// Print size stats
			_logger.LogInformation("Cache contains {NumFiles:n0} files, {TotalSize:n1}mb", _contentIdToTrackedFile.Count, _contentIdToTrackedFile.Values.Sum(x => x.Length) / (1024.0 * 1024.0));
			_logger.LogInformation("Stage contains {NumFiles:n0} files, {TotalSize:n1}mb", _workspace.GetFiles().Count, _workspace.GetFiles().Sum(x => x._length) / (1024.0 * 1024.0));

			// Print the contents of the workspace
			string[] differences = await _workspace.FindDifferencesAsync(_options.MaxFileScanConcurrency);
			if (differences.Length > 0)
			{
				_logger.LogInformation("Local changes:");
				foreach (string difference in differences)
				{
					if (difference.StartsWith('+'))
					{
						Console.ForegroundColor = ConsoleColor.Green;
					}
					else if (difference.StartsWith('-'))
					{
						Console.ForegroundColor = ConsoleColor.Red;
					}
					else if (difference.StartsWith('!'))
					{
						Console.ForegroundColor = ConsoleColor.Yellow;
					}
					else
					{
						Console.ResetColor();
					}
					_logger.LogInformation("  {Line}", difference);
				}
				Console.ResetColor();
			}
		}

		/// <summary>
		/// Switches to the given stream
		/// </summary>
		/// <param name="perforce">The perforce connection</param>
		/// <param name="streamName">Name of the stream to sync</param>
		/// <param name="changeNumber">Changelist number to sync. -1 to sync to latest.</param>
		/// <param name="view">View of the workspace</param>
		/// <param name="removeUntracked">Whether to remove untracked files from the workspace</param>
		/// <param name="fakeSync">Whether to simulate the syncing operation rather than actually getting files from the server</param>
		/// <param name="cacheFile">If set, uses the given file to cache the contents of the workspace. This can improve sync times when multiple machines sync the same workspace.</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task SyncAsync(IPerforceConnection perforce, string streamName, int changeNumber, IReadOnlyList<string> view, bool removeUntracked, bool fakeSync, FileReference? cacheFile, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(SyncAsync)}");
			span.SetAttribute("horde.mw.use_have_table", _options.UseHaveTable);

			Stopwatch timer = Stopwatch.StartNew();
			if (changeNumber == -1)
			{
				_logger.LogInformation("Syncing to {StreamName} at latest (removeUntracked={RemoveUntracked})", streamName, removeUntracked);
			}
			else
			{
				_logger.LogInformation("Syncing to {StreamName} at CL {CL} (removeUntracked={RemoveUntracked})", streamName, changeNumber, removeUntracked);
			}

			using (_logger.WithProperty("useHaveTable", _options.UseHaveTable).BeginScope())
			using (_logger.BeginIndentScope("  "))
			{
				// Update the client to the current stream
				await UpdateClientAsync(perforce, streamName, cancellationToken);

				// Get the latest change number
				if (changeNumber == -1)
				{
					changeNumber = await GetLatestClientChangeAsync(perforce, cancellationToken);
				}

				if (_options.UseHaveTable)
				{
					// Revert any open files
					await RevertInternalAsync(perforce, cancellationToken);

					// Force the P4 metadata to match up
					Task updateHaveTableTask = Task.Run(() => UpdateClientHaveTableAsync(perforce, changeNumber, view, cancellationToken), cancellationToken);

					// Check if clean up can be skipped
					bool containsUntrackedFiles = await LoadUntrackedStateAsync(perforce, cancellationToken);
					if (containsUntrackedFiles)
					{
						await CleanInternalAsync(removeUntracked, cancellationToken);
					}

					// Immediately after a potential clean, mark repository having untracked files.
					// This ensure a clean is run next time even if the sync below is aborted or fails
					await SaveUntrackedStateAsync(perforce, true, cancellationToken);

					// Wait for the have table update to finish
					await updateHaveTableTask;
				}

				// Update the state of the current stream, if necessary
				StreamSnapshot? contents;
				if (cacheFile == null)
				{
					if (_options.UseHaveTable)
					{
						contents = await FindClientContentsAsync(perforce, changeNumber, cancellationToken);
					}
					else
					{
						contents = await FindClientContentsWithoutHaveTableAsync(perforce, streamName, view, changeNumber, cancellationToken);
					}
				}
				else
				{
					contents = await TryLoadClientContentsAsync(cacheFile, new Utf8String(streamName), cancellationToken);
					contents ??= await FindAndSaveClientContentsAsync(perforce, new Utf8String(streamName), view, changeNumber, cacheFile, cancellationToken);
				}

				// Sync all the appropriate files
				await RemoveFilesFromWorkspaceAsync(contents, cancellationToken);
				await AddFilesToWorkspaceAsync(perforce, contents, fakeSync, cancellationToken);
			}

			_logger.LogInformation("Completed in {ElapsedTime}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Replays the effects of unshelving a changelist, but clobbering files in the workspace rather than actually unshelving them (to prevent problems with multiple machines locking them)
		/// </summary>
		/// <returns>Async task</returns>
		public async Task UnshelveAsync(IPerforceConnection perforce, int unshelveChangelist, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(UnshelveAsync)}");

			// Need to mark those files as dirty - update the workspace with those files
			// Both written and deleted files must be flagged so the next sync re-syncs them

			Stopwatch timer = Stopwatch.StartNew();
			_logger.LogInformation("Unshelving changelist {Change}...", unshelveChangelist);

			// query the contents of the shelved changelist
			List<DescribeRecord> records = await perforce.DescribeAsync(DescribeOptions.Shelved, -1, [unshelveChangelist], cancellationToken);
			if (records.Count != 1)
			{
				throw new PerforceException($"Changelist {unshelveChangelist} is not shelved");
			}
			DescribeRecord lastRecord = records[0];
			if (lastRecord.Files.Count == 0)
			{
				throw new PerforceException($"Changelist {unshelveChangelist} does not contain any shelved files");
			}

			// query the location of each file
			List<PerforceResponse<WhereRecord>> whereResponseList = await perforce.TryWhereAsync(lastRecord.Files.Select(x => x.DepotFile).ToArray(), cancellationToken).ToListAsync(cancellationToken);
			Dictionary<string, WhereRecord> whereRecords = whereResponseList.Where(x => x.Succeeded).Select(x => x.Data).ToDictionary(x => x.DepotFile, x => x, StringComparer.OrdinalIgnoreCase);

			// parse out all the list of deleted and modified files
			List<WhereRecord> deleteFiles = [];
			List<WhereRecord> writeFiles = [];
			foreach (DescribeFileRecord fileRecord in lastRecord.Files)
			{
				WhereRecord? whereRecord = null;
				if (whereRecords.TryGetValue(fileRecord.DepotFile, out whereRecord) == false)
				{
					_logger.LogInformation("Unable to get location of {File} in current workspace; ignoring.", fileRecord.DepotFile);
					continue;
				}

				switch (fileRecord.Action)
				{
					case FileAction.Delete:
					case FileAction.MoveDelete:
						deleteFiles.Add(whereRecord);
						break;
					case FileAction.Add:
					case FileAction.Edit:
					case FileAction.MoveAdd:
					case FileAction.Branch:
					case FileAction.Integrate:
						writeFiles.Add(whereRecord);
						break;
					default:
						throw new Exception($"Unknown action '{fileRecord.Action}' for shelved file {fileRecord.DepotFile}");
				}
			}

			if (!_createdClients.TryGetValue(perforce.Settings.ClientName!, out ClientRecord? perforceClient))
			{
				throw new Exception($"Unknown client {perforce.Settings.ClientName}");
			}

			// Add all modified and deleted files to the workspace with invalid metadata. This will ensure they're re-synced next time.
			List<WhereRecord> invalidateFiles = [..writeFiles, ..deleteFiles];
			if (invalidateFiles.Count > 0)
			{
				_logger.LogInformation("Invalidating {NumFiles} files in tracked workspace", invalidateFiles.Count);
				foreach (WhereRecord file in invalidateFiles)
				{
					string path = Regex.Replace(file.ClientFile, "^//[^/]+/", "");
					_workspace.AddFile(new Utf8String(path), 0, 0, false, new FileContentId(Md5Hash.Zero, default));
				}
				await SaveAsync(TransactionState.Clean, CancellationToken.None);
			}

			// Delete all the files
			foreach (WhereRecord deleteFile in deleteFiles)
			{
				string localPath = deleteFile.Path;
				if (File.Exists(localPath))
				{
					_logger.LogInformation("  Deleting {LocalPath}", localPath);
					FileUtils.ForceDeleteFile(localPath);
				}
			}

			// Use common paths with wild cards speed up the print operation with one call instead of many calls to print.
			_logger.LogInformation("Writing files from shelved changelist {Change}", unshelveChangelist);
			PerforceResponseList<PrintRecord> printResponse = await perforce.TryPrintAsync($"{perforceClient.Root}{Path.DirectorySeparatorChar}...", $"//{perforceClient.Name}/...@={unshelveChangelist}", cancellationToken);
			if (!printResponse.Succeeded)
			{
				_logger.LogWarning("Unable to print shelved changelist: {Error}", printResponse.FirstError?.ToString());
			}

			_logger.LogInformation("Completed in {TimeSeconds}s", $"{timer.Elapsed.TotalSeconds:0.0}");
		}

		/// <summary>
		/// Populates the cache with the head revision of the given streams.
		/// </summary>
		public async Task PopulateAsync(List<PopulateRequest> requests, bool fakeSync, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Populating with {NumStreams} streams", requests.Count);
			using (_logger.BeginIndentScope("  "))
			{
				Tuple<int, StreamSnapshot>[] streamState = await PopulateCleanAsync(requests, cancellationToken);
				await PopulateSyncAsync(requests, streamState, fakeSync, cancellationToken);
			}
		}

		/// <summary>
		/// Perform the clean part of a populate command
		/// </summary>
		public async Task<Tuple<int, StreamSnapshot>[]> PopulateCleanAsync(List<PopulateRequest> requests, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(PopulateCleanAsync)}");

			// Revert all changes in each of the unique clients
			foreach (PopulateRequest request in requests)
			{
				using IPerforceConnection perforce = await request.PerforceClient.WithoutClientAsync();

				PerforceResponse<ClientRecord> response = await perforce.TryGetClientAsync(request.PerforceClient.Settings.ClientName!, cancellationToken);
				if (response.Succeeded)
				{
					await RevertInternalAsync(request.PerforceClient, cancellationToken);
				}
			}

			// Clean the current workspace
			await CleanAsync(null, true, cancellationToken);

			// Update the list of files in each stream
			Tuple<int, StreamSnapshot>[] streamState = new Tuple<int, StreamSnapshot>[requests.Count];
			for (int idx = 0; idx < requests.Count; idx++)
			{
				PopulateRequest request = requests[idx];
				string streamName = request.StreamName;
				_logger.LogInformation("Finding contents of {StreamName}:", streamName);

				using (_logger.BeginIndentScope("  "))
				{
					await DeleteClientAsync(request.PerforceClient, cancellationToken);
					await UpdateClientAsync(request.PerforceClient, streamName, cancellationToken);

					int changeNumber = await GetLatestClientChangeAsync(request.PerforceClient, cancellationToken);
					_logger.LogInformation("Latest change is CL {CL}", changeNumber);

					if (_options.UseHaveTable)
					{
						await UpdateClientHaveTableAsync(request.PerforceClient, changeNumber, request.View, cancellationToken);

						StreamSnapshot contents = await FindClientContentsAsync(request.PerforceClient, changeNumber, cancellationToken);
						streamState[idx] = Tuple.Create(changeNumber, contents);
					}
					else
					{
						StreamSnapshot contents = await FindClientContentsWithoutHaveTableAsync(request.PerforceClient, streamName, request.View, changeNumber, cancellationToken);
						streamState[idx] = Tuple.Create(changeNumber, contents);
					}

					GC.Collect();
				}
			}

			// Remove any files from the workspace not referenced by the first stream. This ensures we can purge things from the cache that we no longer need.
			if (requests.Count > 0)
			{
				await RemoveFilesFromWorkspaceAsync(streamState[0].Item2, cancellationToken);
			}

			// Shrink the contents of the cache
			using (Trace("UpdateCache"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Updating cache..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				HashSet<FileContentId> commonContentIds = [];
				Dictionary<FileContentId, long> contentIdToLength = [];
				for (int idx = 0; idx < requests.Count; idx++)
				{
					List<StreamFile> files = streamState[idx].Item2.GetFiles();
					foreach (StreamFile file in files)
					{
						contentIdToLength[file.ContentId] = file.Length;
					}

					if (idx == 0)
					{
						commonContentIds.UnionWith(files.Select(x => x.ContentId));
					}
					else
					{
						commonContentIds.IntersectWith(files.Select(x => x.ContentId));
					}
				}

				List<CachedFileInfo> trackedFiles = [.. _contentIdToTrackedFile.Values];
				foreach (CachedFileInfo trackedFile in trackedFiles)
				{
					if (!contentIdToLength.ContainsKey(trackedFile.ContentId))
					{
						RemoveTrackedFile(trackedFile);
					}
				}

				GC.Collect();

				double totalSize = contentIdToLength.Sum(x => x.Value) / (1024.0 * 1024.0);
				status.Progress = String.Format("{0:n1}mb total, {1:n1}mb differences ({2:0.0}s)", totalSize, totalSize - commonContentIds.Sum(x => contentIdToLength[x]) / (1024.0 * 1024.0), timer.Elapsed.TotalSeconds);
			}

			return streamState;
		}

		/// <summary>
		/// Perform the sync part of a populate command
		/// </summary>
		public async Task PopulateSyncAsync(List<PopulateRequest> requests, Tuple<int, StreamSnapshot>[] streamState, bool fakeSync, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(PopulateSyncAsync)}");

			// Sync all the new files
			for (int idx = 0; idx < requests.Count; idx++)
			{
				PopulateRequest request = requests[idx];
				string streamName = request.StreamName;
				_logger.LogInformation("Syncing files for {StreamName}:", streamName);

				using (_logger.BeginIndentScope("  "))
				{
					if (_options.UseHaveTable)
					{
						await DeleteClientAsync(request.PerforceClient, cancellationToken);
						await UpdateClientAsync(request.PerforceClient, streamName, cancellationToken);

						int changeNumber = streamState[idx].Item1;
						await UpdateClientHaveTableAsync(request.PerforceClient, changeNumber, requests[idx].View, cancellationToken);
					}

					StreamSnapshot contents = streamState[idx].Item2;
					await RemoveFilesFromWorkspaceAsync(contents, cancellationToken);
					await AddFilesToWorkspaceAsync(request.PerforceClient, contents, fakeSync, cancellationToken);
				}
			}

			// Save the new repo state
			await SaveAsync(TransactionState.Clean, cancellationToken);
		}

		#endregion

		#region Core operations

		/// <summary>
		/// Deletes a client
		/// </summary>
		/// <param name="perforceClient">The Perforce connection</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		public async Task DeleteClientAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(DeleteClientAsync)}");
			async Task DeleteAsync(string clientName)
			{
				PerforceResponse response = await perforceClient.TryDeleteClientAsync(DeleteClientOptions.None, clientName, cancellationToken);
				if (response.Error != null && response.Error.Generic != PerforceGenericCode.Unknown)
				{
					if (response.Error.Generic == PerforceGenericCode.NotYet)
					{
						await RevertInternalAsync(perforceClient, cancellationToken);
						response = await perforceClient.TryDeleteClientAsync(DeleteClientOptions.None, clientName, cancellationToken);
					}
					response.EnsureSuccess();
				}
				_createdClients.Remove(clientName);
			}

			await DeleteAsync(perforceClient.Settings.ClientName!);
		}

		private async Task<ClientRecord> GetOrCreateClientAsync(IPerforceConnection perforceClient, string streamName, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(GetOrCreateClientAsync)}");
			string clientName = perforceClient.Settings.ClientName!;//GetClientName(perforceClient, useHaveTable);
			if (_createdClients.TryGetValue(clientName, out ClientRecord? client) && client.Stream == streamName)
			{
				return client;
			}

			client = new ClientRecord(clientName, perforceClient.Settings.UserName!, _workspaceDir.FullName);
			client.Host = _hostName;
			client.Stream = streamName;

			if (_options.UseModTime)
			{
				client.Options = ClientOptions.ModTime;
			}

			if (_options.Partitioned)
			{
				// Partitioned and read-only types store their have table separately on the server, compared to normal (writeable) clients
				// Clients that sync without updating the have table cannot submit so they're marked as read-only. 
				client.Type = _options.UseHaveTable ? "partitioned" : "readonly";
			}

			_logger.LogInformation("Using client {ClientName} (Host: {HostName}, Stream: {StreamName}, Type: {Type}, Root: {Path})", client.Name, client.Host, client.Stream, client.Type ?? "full", client.Root);

			using (Trace("UpdateClient"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Updating client..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				using IPerforceConnection perforce = await perforceClient.WithoutClientAsync();

				PerforceResponse response = await perforce.TryCreateClientAsync(client, cancellationToken);
				if (!response.Succeeded)
				{
					await perforceClient.TryDeleteClientAsync(DeleteClientOptions.None, clientName, cancellationToken);
					await perforceClient.CreateClientAsync(client, cancellationToken);
				}

				if (!_options.UseHaveTable)
				{
					// If have table is not used, make client is fully reset as it may have been re-used
					await UpdateHaveTablePathAsync(perforceClient, $"//{clientName}/...#0", _tracer, cancellationToken);
				}

				status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}
			_createdClients[clientName] = client;

			return client;
		}

		/// <summary>
		/// Sets the stream for the current client
		/// </summary>
		/// <param name="perforceClient">The Perforce connection</param>
		/// <param name="streamName">New stream for the client</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task UpdateClientAsync(IPerforceConnection perforceClient, string streamName, CancellationToken cancellationToken)
		{
			await GetOrCreateClientAsync(perforceClient, streamName, cancellationToken);

			// Update the config file with the name of the client
			FileReference configFile = FileReference.Combine(_baseDir, "p4.ini");
			using (StreamWriter writer = new(configFile.FullName))
			{
				await writer.WriteLineAsync($"P4PORT={perforceClient.Settings.ServerAndPort}");
				await writer.WriteLineAsync($"P4CLIENT={perforceClient.Settings.ClientName}");
			}
		}

		/// <summary>
		/// Gets the latest change submitted for the given stream
		/// </summary>
		/// <param name="perforceClient">The Perforce connection</param>
		/// <param name="streamName">The stream to sync</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The latest changelist number</returns>
		public async Task<int> GetLatestChangeAsync(IPerforceConnection perforceClient, string streamName, CancellationToken cancellationToken)
		{
			// Update the client to the current stream
			await UpdateClientAsync(perforceClient, streamName, cancellationToken);

			// Get the latest change number
			return await GetLatestClientChangeAsync(perforceClient, cancellationToken);
		}

		/// <summary>
		/// Get the latest change number in the current client
		/// </summary>
		/// <param name="perforceClient">The perforce client connection</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The latest submitted change number</returns>
		private async Task<int> GetLatestClientChangeAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			int changeNumber;
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(GetLatestClientChangeAsync)}");
			using (Trace("FindChange"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Finding latest change..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				List<ChangesRecord> changes = await perforceClient.GetChangesAsync(ChangesOptions.None, 1, ChangeStatus.Submitted, new[] { String.Format("//{0}/...", perforceClient.Settings.ClientName) }, cancellationToken);
				if (changes.Count == 0)
				{
					throw new ManagedWorkspaceException($"Unable to find latest change; no changes in view for {perforceClient.Settings.ClientName}.");
				}
				changeNumber = changes[0].Number;

				status.Progress = String.Format("CL {0} ({1:0.0}s)", changeNumber, timer.Elapsed.TotalSeconds);
			}
			return changeNumber;
		}

		/// <summary>
		/// Revert all files that are open in the current workspace. Does not replace them with valid revisions.
		/// </summary>
		/// <param name="perforceClient">The current client connection</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task RevertInternalAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(RevertInternalAsync)}");
			using (Trace("Revert"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Reverting changes..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				// Get a list of open files
				List<OpenedRecord> openedFilesResponse = await perforceClient.OpenedAsync(OpenedOptions.ShortOutput, -1, perforceClient.Settings.ClientName!, null, 1, FileSpecList.Any, cancellationToken).ToListAsync(cancellationToken);

				// If there are any files, revert them
				if (openedFilesResponse.Count != 0)
				{
					await perforceClient.RevertAsync(-1, null, RevertOptions.KeepWorkspaceFiles, new[] { "//..." }, cancellationToken);
				}

				// Find all the open changes
				List<ChangesRecord> changes = await perforceClient.GetChangesAsync(ChangesOptions.None, perforceClient.Settings.ClientName!, -1, ChangeStatus.Pending, null, Array.Empty<string>(), cancellationToken);

				// Delete the changelist
				foreach (ChangesRecord change in changes)
				{
					// Delete the shelved files
					List<DescribeRecord> describeResponse = await perforceClient.DescribeAsync(DescribeOptions.Shelved, -1, [change.Number], cancellationToken);
					foreach (DescribeRecord record in describeResponse)
					{
						if (record.Files.Count > 0)
						{
							await perforceClient.DeleteShelvedFilesAsync(record.Number, Array.Empty<string>(), cancellationToken);
						}
					}

					// Delete the changelist
					await perforceClient.DeleteChangeAsync(DeleteChangeOptions.None, change.Number, cancellationToken);
				}

				status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}
		}

		/// <summary>
		/// Clears the have table. This ensures that we'll always fetch the names of files at head revision, which aren't updated otherwise.
		/// </summary>
		/// <param name="perforceClient">The client connection</param>
		/// <param name="cancellationToken">The cancellation token</param>
		private async Task ClearClientHaveTableAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(ClearClientHaveTableAsync)}");
			using (Trace("ClearHaveTable"))
			using (ILoggerProgress scope = _logger.BeginProgressScope("Clearing have table..."))
			{
				Stopwatch timer = Stopwatch.StartNew();
				await perforceClient.SyncQuietAsync(SyncOptions.KeepWorkspaceFiles, -1, new[] { String.Format("//{0}/...#0", perforceClient.Settings.ClientName!) }, cancellationToken);
				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}
		}

		/// <summary>
		/// Updates the have table to reflect the given stream
		/// </summary>
		/// <param name="perforceClient">The client connection</param>
		/// <param name="changeNumber">The change number to sync. May be -1, for latest.</param>
		/// <param name="view">View of the stream. Each entry should be a path relative to the stream root, with an optional '-'prefix.</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task UpdateClientHaveTableAsync(IPerforceConnection perforceClient, int changeNumber, IReadOnlyList<string> view, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(UpdateClientHaveTableAsync)}");
			using (Trace("UpdateHaveTable"))
			using (ILoggerProgress scope = _logger.BeginProgressScope("Updating have table..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				// Sync an initial set of files. Either start with a full workspace and remove files, or start with nothing and add files.
				if (view.Count == 0 || view[0].StartsWith('-'))
				{
					await UpdateHaveTablePathAsync(perforceClient, $"//{perforceClient.Settings.ClientName}/...@{changeNumber}", _tracer, cancellationToken);
				}
				else
				{
					await UpdateHaveTablePathAsync(perforceClient, $"//{perforceClient.Settings.ClientName}/...#0", _tracer, cancellationToken);
				}

				// Update with the contents of each filter
				foreach (string filter in view)
				{
					string syncPath;
					if (filter.StartsWith('-'))
					{
						syncPath = String.Format("//{0}/{1}#0", perforceClient.Settings.ClientName, RemoveLeadingSlash(filter.Substring(1)));
					}
					else
					{
						syncPath = String.Format("//{0}/{1}@{2}", perforceClient.Settings.ClientName, RemoveLeadingSlash(filter), changeNumber);
					}
					await UpdateHaveTablePathAsync(perforceClient, syncPath, _tracer, cancellationToken);
				}

				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}
		}

		/// <summary>
		/// Update a path in the have table
		/// </summary>
		/// <param name="perforceClient">The Perforce client</param>
		/// <param name="syncPath">Path to sync</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		private static async Task UpdateHaveTablePathAsync(IPerforceConnection perforceClient, string syncPath, Tracer tracer, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(UpdateHaveTablePathAsync)}");
			span.SetAttribute("horde.sync_path", syncPath);

			PerforceResponseList<SyncSummaryRecord> responseList = await perforceClient.TrySyncQuietAsync(SyncOptions.KeepWorkspaceFiles, -1, new[] { syncPath }, cancellationToken);
			foreach (PerforceResponse<SyncSummaryRecord> response in responseList)
			{
				PerforceError? error = response.Error;
				if (error != null && error.Generic != PerforceGenericCode.Empty)
				{
					throw new PerforceException(error);
				}
			}
		}

		/// <summary>
		/// Optimized record definition for fstat calls when populating a workspace. Since there are so many files in a typical branch,
		/// the speed of serializing these records is crucial for performance. Rather than deseralizing everything, we filter to just
		/// the fields we need, and avoid any unnecessary conversions from their primitive data types.
		/// </summary>
		class FStatIndexedRecord
		{
			// Note: This enum is used for indexing an array of fields, and member names much match P4 field names (including case).
			enum Field
			{
				code,
				depotFile,
				clientFile,
				headType,
				haveRev,
				fileSize,
				digest
			}

			public static readonly string[] FieldNames = Enum.GetNames(typeof(Field));
			public static readonly Utf8String[] Utf8FieldNames = Array.ConvertAll(FieldNames, x => new Utf8String(x));

			public PerforceValue[] Values { get; } = new PerforceValue[FieldNames.Length];

			public Utf8String DepotFile => Values[(int)Field.depotFile].GetString();

			public Utf8String ClientFile => Values[(int)Field.clientFile].GetString();

			public Utf8String HeadType => Values[(int)Field.headType].GetString();

			public Utf8String HaveRev => Values[(int)Field.haveRev].GetString();

			public long FileSize => Values[(int)Field.fileSize].AsLong();

			public Utf8String Digest => Values[(int)Field.digest].GetString();
		}

		/// <summary>
		/// Get the contents of the client, as synced.
		/// </summary>
		/// <param name="perforceClient">The client connection</param>
		/// <param name="changeNumber">The change number being synced. This must be specified in order to get the digest at the correct revision.</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task<StreamSnapshotFromMemory> FindClientContentsAsync(IPerforceConnection perforceClient, int changeNumber, CancellationToken cancellationToken)
		{
			StreamTreeBuilder builder = new();

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(FindClientContentsAsync)}");
			using (Trace("FetchMetadata"))
			using (ILoggerProgress scope = _logger.BeginProgressScope("Fetching metadata..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				// Get the expected prefix for any paths in client syntax
				Utf8String clientPrefix = new($"//{perforceClient.Settings.ClientName}/");

				// List of the last path fragments. Since file records that are returned are typically sorted by their position in the tree, we can save quite a lot of processing by
				// reusing as many fragemnts as possible.
				List<(Utf8String, StreamTreeBuilder)> fragments = [];

				int numFiles = 0;

				// Handler for each returned record
				FStatIndexedRecord record = new();
				void HandleRecord(PerforceRecord rawRecord)
				{
					// Copy into the values array
					rawRecord.CopyInto(FStatIndexedRecord.Utf8FieldNames, record.Values);

					// Make sure it has all the fields we're interested in
					if (record.Digest.IsEmpty)
					{
						return;
					}
					if (record.ClientFile.IsEmpty)
					{
						throw new InvalidDataException("Record returned by Peforce does not have ClientFile set");
					}
					if (!record.ClientFile.StartsWith(clientPrefix))
					{
						throw new InvalidDataException($"Client path returned by Perforce ('{record.ClientFile}') does not begin with client name ('{clientPrefix}')");
					}

					numFiles++;

					// Duplicate the client path. If we reference into the raw record, we'll prevent all the raw P4 output from being garbage collected.
					Utf8String clientFile = record.ClientFile.Clone();

					// Get the client path after the initial client prefix
					ReadOnlySpan<byte> pathSpan = clientFile.Span;

					// Parse out the data
					StreamTreeBuilder lastStreamDirectory = builder;

					// Try to match up as many fragments from the last file.
					int fragmentMinIdx = clientPrefix.Length;
					for (int fragmentIdx = 0; ; fragmentIdx++)
					{
						// Find the next directory separator
						int fragmentMaxIdx = fragmentMinIdx;
						while (fragmentMaxIdx < pathSpan.Length && pathSpan[fragmentMaxIdx] != '/')
						{
							fragmentMaxIdx++;
						}
						if (fragmentMaxIdx == pathSpan.Length)
						{
							fragments.RemoveRange(fragmentIdx, fragments.Count - fragmentIdx);
							break;
						}

						// Get the fragment text
						Utf8String fragment = new(clientFile.Memory.Slice(fragmentMinIdx, fragmentMaxIdx - fragmentMinIdx));

						// If this fragment matches the same fragment from the previous iteration, take the last stream directory straight away
						if (fragmentIdx < fragments.Count)
						{
							if (fragments[fragmentIdx].Item1 == fragment)
							{
								lastStreamDirectory = fragments[fragmentIdx].Item2;
							}
							else
							{
								fragments.RemoveRange(fragmentIdx, fragments.Count - fragmentIdx);
							}
						}

						// Otherwise, find or add a directory for this fragment into the last directory
						if (fragmentIdx >= fragments.Count)
						{
							Utf8String unescapedFragment = PerforceUtils.UnescapePath(fragment);

							StreamTreeBuilder? nextStreamDirectory;
							if (!lastStreamDirectory.NameToTreeBuilder.TryGetValue(unescapedFragment, out nextStreamDirectory))
							{
								nextStreamDirectory = new StreamTreeBuilder();
								lastStreamDirectory.NameToTreeBuilder.Add(unescapedFragment, nextStreamDirectory);
							}
							lastStreamDirectory = nextStreamDirectory;

							fragments.Add((fragment, lastStreamDirectory));
						}

						// Move to the next fragment
						fragmentMinIdx = fragmentMaxIdx + 1;
					}

					Md5Hash digest = Md5Hash.Parse(record.Digest);
					FileContentId contentId = new(digest, record.HeadType.Clone());
					int revision = (int)Utf8String.ParseUnsignedInt(record.HaveRev);

					// Add a new StreamFileInfo to the last directory object
					Utf8String fileName = PerforceUtils.UnescapePath(clientFile.Slice(fragmentMinIdx));
					lastStreamDirectory.NameToFile.Add(fileName, new StreamFile(record.DepotFile.Clone(), record.FileSize, contentId, revision));
				}

				// Create the workspace, and add records for all the files. Exclude deleted files with digest = null.
				List<string> arguments =
				[
					"-Ol", // Output fileSize and digest field
					"-Op", // Output clientFile field in both server and local path syntax
					"-Os", // Shorten output by excluding client workspace data (for instance, the clientFile field).
					"-Rh", // Limit output to files on your have list;
					"-T", // Include only the fields listed below
					String.Join(",", FStatIndexedRecord.FieldNames),
					$"//{perforceClient.Settings.ClientName}/...@{changeNumber}",
				];
				await perforceClient.RecordCommandAsync("fstat", arguments, null, HandleRecord, cancellationToken);

				// Output the elapsed time
				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s {numFiles} files)";
			}

			return new StreamSnapshotFromMemory(builder);
		}

		class FStatRecordWithoutHaveTable
		{
			// Note: This enum is used for indexing an array of fields, and member names much match P4 field names (including case).
			enum Field
			{
				code,
				depotFile,
				headType,
				headRev,
				fileSize,
				digest
			}

			public static readonly string[] FieldNames = Enum.GetNames(typeof(Field));
			public static readonly Utf8String[] Utf8FieldNames = Array.ConvertAll(FieldNames, x => new Utf8String(x));

			public PerforceValue[] Values { get; } = new PerforceValue[FieldNames.Length];

			public Utf8String DepotFile => Values[(int)Field.depotFile].GetString();

			public Utf8String HeadType => Values[(int)Field.headType].GetString();

			public int HeadRev => Values[(int)Field.headRev].AsInteger();

			public long FileSize => Values[(int)Field.fileSize].AsLong();

			public Utf8String Digest => Values[(int)Field.digest].GetString();
		}

		/// <summary>
		/// Get the contents of the client without using the have table
		/// </summary>
		/// <param name="perforceClient">The client connection</param>
		/// <param name="streamName">Name of stream</param>
		/// <param name="view">View of the workspace</param>
		/// <param name="changeNumber">The change number being synced. This must be specified in order to get the digest at the correct revision.</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task<StreamSnapshotFromMemory> FindClientContentsWithoutHaveTableAsync(IPerforceConnection perforceClient, string streamName, IReadOnlyList<string> view, int changeNumber, CancellationToken cancellationToken)
		{
			DepotStreamTreeBuilder builder = new();

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(FindClientContentsWithoutHaveTableAsync)}");
			using (Trace("FetchMetadata"))
			using (ILoggerProgress scope = _logger.BeginProgressScope("Fetching metadata (without have table)..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				StreamRecord streamRecord = await perforceClient.GetStreamAsync(streamName, true, cancellationToken);
				PerforceViewMap viewMap = PerforceViewMap.Parse(streamRecord.View);

				int numFiles = 0;

				// Use Horde's additional filtering taking place after stream view mapping
				PerforceViewFilter viewFilter = PerforceViewFilter.Parse(view);

				// Re-use a single class instance as there can be millions of records
				FStatRecordWithoutHaveTable record = new();

				void HandleRecord(PerforceRecord rawRecord)
				{
					// Copy into the values array
					rawRecord.CopyInto(FStatRecordWithoutHaveTable.Utf8FieldNames, record.Values);
					if (record.Digest.IsEmpty)
					{
						return;
					}

					if (viewMap.TryMapFile(record.DepotFile.ToString(), StringComparison.OrdinalIgnoreCase, out string clientFile))
					{
						if (!viewFilter.IncludeFile(clientFile, StringComparison.OrdinalIgnoreCase))
						{
							return;
						}

						Md5Hash md5Hash = Md5Hash.Parse(record.Digest);
						FileContentId fileContentId = new(md5Hash, record.HeadType);
						builder.AddFile(clientFile, new StreamFile(record.DepotFile, record.FileSize, fileContentId, record.HeadRev));
						numFiles++;
					}
					else
					{
						_logger.LogError("Failed to view map depot file {DepotFile}", record.DepotFile.ToString());
					}
				}

				string fileSpec = $"//{perforceClient.Settings.ClientName}/...@{changeNumber}";
				List<string> arguments =
				[
					"-Ol", // Output fileSize and digest field
					"-Os", // Shorten output by excluding client workspace data (for instance, the clientFile field).
					"-F", // Filter any files not existing at current revision (filter below)
					"^headAction=delete&^headAction=move/delete&^headAction=purge",
					"-T", // Include only the fields listed below
					String.Join(",", FStatRecordWithoutHaveTable.FieldNames),
					fileSpec,
				];

				await perforceClient.RecordCommandAsync("fstat", arguments, null, HandleRecord, cancellationToken);

				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s {numFiles} files)";
			}

			return new StreamSnapshotFromMemory(builder);
		}

		/// <summary>
		/// Loads the contents of a client from disk
		/// </summary>
		/// <param name="cacheFile">The cache file to read from</param>
		/// <param name="basePath">Default path for the stream</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Contents of the workspace</returns>
		async Task<StreamSnapshot?> TryLoadClientContentsAsync(FileReference cacheFile, Utf8String basePath, CancellationToken cancellationToken)
		{
			StreamSnapshot? contents = null;
			if (FileReference.Exists(cacheFile))
			{
				using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(TryLoadClientContentsAsync)}");
				using (Trace("ReadMetadata"))
				using (ILoggerProgress scope = _logger.BeginProgressScope($"Reading cached metadata from {cacheFile}..."))
				{
					Stopwatch timer = Stopwatch.StartNew();
					contents = await StreamSnapshotFromMemory.TryLoadAsync(cacheFile, basePath, cancellationToken);
					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}
			return contents;
		}

		/// <summary>
		/// Finds the contents of a workspace, and saves it to disk
		/// </summary>
		/// <param name="perforceClient">The client connection</param>
		/// <param name="basePath">Base path for the stream</param>
		/// <param name="view">View of the workspace</param>
		/// <param name="changeNumber">The change number being synced. This must be specified in order to get the digest at the correct revision.</param>
		/// <param name="cacheFile">Location of the file to save the cached contents</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Contents of the workspace</returns>
		private async Task<StreamSnapshotFromMemory> FindAndSaveClientContentsAsync(IPerforceConnection perforceClient, Utf8String basePath, IReadOnlyList<string> view, int changeNumber, FileReference cacheFile, CancellationToken cancellationToken)
		{
			StreamSnapshotFromMemory contents = _options.UseHaveTable
				? await FindClientContentsAsync(perforceClient, changeNumber, cancellationToken)
				: await FindClientContentsWithoutHaveTableAsync(perforceClient, basePath.ToString(), view, changeNumber, cancellationToken);

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(FindAndSaveClientContentsAsync)}");
			using (Trace("WriteMetadata"))
			using (ILoggerProgress scope = _logger.BeginProgressScope($"Saving metadata to {cacheFile}..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				// Handle the case where two machines may try to write to the cache file at once by writing to a temporary file
				FileReference tempCacheFile = new(String.Format("{0}.{1}", cacheFile, Guid.NewGuid()));
				await contents.SaveAsync(tempCacheFile, basePath);

				// Try to move it into place
				try
				{
					FileReference.Move(tempCacheFile, cacheFile);
				}
				catch (IOException)
				{
					if (!FileReference.Exists(cacheFile))
					{
						throw;
					}
					FileReference.Delete(tempCacheFile);
				}

				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}
			return contents;
		}

		/// <summary>
		/// Remove files from the workspace
		/// </summary>
		/// <param name="contents">Contents of the target stream</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task RemoveFilesFromWorkspaceAsync(StreamSnapshot contents, CancellationToken cancellationToken)
		{
			// Make sure the repair flag is clear before we start
			await RunOptionalCacheRepairAsync(cancellationToken);

			// Figure out what to remove
			RemoveTransaction transaction;
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(RemoveFilesFromWorkspaceAsync)}");
			using (Trace("GatherFilesToRemove"))
			using (ILoggerProgress scope = _logger.BeginProgressScope("Gathering files to remove..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				transaction = await RemoveTransaction.CreateAsync(_workspace, contents, _contentIdToTrackedFile, _options.MaxFileScanConcurrency);

				scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}

			// Move files into the cache
			KeyValuePair<FileContentId, WorkspaceFileInfo>[] filesToMove = [.. transaction._filesToMove];
			if (filesToMove.Length > 0)
			{
				using (Trace("MoveToCache"))
				using (ILoggerProgress scope = _logger.BeginProgressScope(String.Format("Moving {0} {1} to cache...", filesToMove.Length, (filesToMove.Length == 1) ? "file" : "files")))
				{
					Stopwatch timer = Stopwatch.StartNew();

					// Add any new files to the cache
					List<(FileContentId ContentId, FileReference Source, FileReference Target)> files = [];
					foreach (KeyValuePair<FileContentId, WorkspaceFileInfo> fileToMove in filesToMove)
					{
						ulong cacheId = GetUniqueCacheId(fileToMove.Key);
						CachedFileInfo newTrackingInfo = new(_cacheDir, fileToMove.Key, cacheId, fileToMove.Value._length, fileToMove.Value._lastModifiedTicks, fileToMove.Value._readOnly, _nextSequenceNumber);
						_contentIdToTrackedFile.Add(fileToMove.Key, newTrackingInfo);
						files.Add((fileToMove.Key, fileToMove.Value.GetLocation(), newTrackingInfo.GetLocation()));
					}
					_nextSequenceNumber++;

					// Save the current state of the repository as dirty. If we're interrupted, we will have two places to check for each file (the cache and workspace).
					await SaveAsync(TransactionState.Dirty, cancellationToken);

					// Execute all the moves and deletes
					ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency, CancellationToken = cancellationToken };
					await Parallel.ForEachAsync(files, options, (file, ctx) => MoveFileToCache(file.ContentId, file.Source, file.Target, _contentIdToTrackedFile, ctx));

					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}

			// Remove files which are no longer needed
			WorkspaceFileInfo[] filesToDelete = [.. transaction._filesToDelete];
			if (filesToDelete.Length > 0)
			{
				using (Trace("DeleteFiles"))
				using (ILoggerProgress scope = _logger.BeginProgressScope(String.Format("Deleting {0} {1}...", filesToDelete.Length, (filesToDelete.Length == 1) ? "file" : "files")))
				{
					Stopwatch timer = Stopwatch.StartNew();

					ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency, CancellationToken = cancellationToken };
					await Parallel.ForEachAsync(filesToDelete, options, (fileToDelete, ct) =>
					{
						RemoveFile(fileToDelete);
						return ValueTask.CompletedTask;
					});

					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}

			// Remove directories which are no longer needed
			WorkspaceDirectoryInfo[] directoriesToDelete = [.. transaction._directoriesToDelete];
			if (directoriesToDelete.Length > 0)
			{
				using (Trace("DeleteDirectories"))
				using (ILoggerProgress scope = _logger.BeginProgressScope(String.Format("Deleting {0} {1}...", directoriesToDelete.Length, (directoriesToDelete.Length == 1) ? "directory" : "directories")))
				{
					Stopwatch timer = Stopwatch.StartNew();

					foreach (string directoryToDelete in directoriesToDelete.Select(x => x.GetFullName()).OrderByDescending(x => x.Length))
					{
						RemoveDirectory(directoryToDelete);
					}

					scope.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}

			// Update the workspace and save the new state
			_workspace = transaction._newWorkspaceRootDir;
			await SaveAsync(TransactionState.Clean, cancellationToken);
		}

		ValueTask MoveFileToCache(FileContentId contentId, FileReference sourceFile, FileReference targetFile, Dictionary<FileContentId, CachedFileInfo> contentIdToTrackedFile, CancellationToken cancellationToken)
		{
			cancellationToken.ThrowIfCancellationRequested();
			try
			{
				FileUtils.ForceMoveFile(sourceFile, targetFile);
			}
			catch (Exception ex)
			{
				Exception innerException = (ex as WrappedFileOrDirectoryException)?.InnerException ?? ex;
				_logger.LogWarning(KnownLogEvents.Systemic_ManagedWorkspace, innerException, "Unable to move {SourceFile} to {TargetFile}: {Error}", sourceFile, targetFile, innerException.Message);

				lock (contentIdToTrackedFile)
				{
					contentIdToTrackedFile.Remove(contentId);
				}
			}
			return default;
		}

		/// <summary>
		/// Helper function to delete a file from the workspace, and output any failure as a warning.
		/// </summary>
		/// <param name="fileToDelete">The file to be deleted</param>
		void RemoveFile(WorkspaceFileInfo fileToDelete)
		{
			try
			{
				FileUtils.ForceDeleteFile(fileToDelete.GetLocation());
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "warning: Unable to delete file {FileName}.", fileToDelete.GetFullName());
				_requiresRepair = true;
			}
		}

		/// <summary>
		/// Helper function to delete a directory from the workspace, and output any failure as a warning.
		/// </summary>
		/// <param name="directoryToDelete">The directory to be deleted</param>
		void RemoveDirectory(string directoryToDelete)
		{
			try
			{
				Directory.Delete(directoryToDelete, false);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "warning: Unable to delete directory {Directory}", directoryToDelete);
				_requiresRepair = true;
			}
		}

		[DllImport("kernel32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool GetDiskFreeSpaceEx(
			string lpDirectoryName,
			out ulong lpFreeBytesAvailable,
			out ulong lpTotalNumberOfBytes,
			out ulong lpTotalNumberOfFreeBytes
		);

		/// <summary>
		/// Update the workspace to match the given stream, syncing files and moving to/from the cache as necessary.
		/// </summary>
		/// <param name="client">The client connection</param>
		/// <param name="stream">Contents of the stream</param>
		/// <param name="fakeSync">Whether to simulate the sync operation, rather than actually syncing files</param>
		/// <param name="cancellationToken">Cancellation token</param>
		private async Task AddFilesToWorkspaceAsync(IPerforceConnection client, StreamSnapshot stream, bool fakeSync, CancellationToken cancellationToken)
		{
			// Make sure the repair flag is reset
			await RunOptionalCacheRepairAsync(cancellationToken);

			// Figure out what we need to do
			AddTransaction transaction;
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(AddFilesToWorkspaceAsync)}");
			using (Trace("GatherFilesToAdd"))
			using (ILoggerProgress status = _logger.BeginProgressScope("Gathering files to add..."))
			{
				Stopwatch timer = Stopwatch.StartNew();

				transaction = await AddTransaction.CreateAsync(_workspace, stream, _contentIdToTrackedFile, _options.MaxFileScanConcurrency);
				_workspace = transaction._newWorkspaceRootDir;
				await SaveAsync(TransactionState.Dirty, cancellationToken);

				status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
			}

			// Swap files in and out of the cache
			WorkspaceFileToMove[] filesToMove = [.. transaction._filesToMove.Values];
			if (filesToMove.Length > 0)
			{
				using (Trace("MoveFromCache"))
				using (ILoggerProgress status = _logger.BeginProgressScope(String.Format("Moving {0} {1} from cache...", filesToMove.Length, (filesToMove.Length == 1) ? "file" : "files")))
				{
					Stopwatch timer = Stopwatch.StartNew();
					ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency, CancellationToken = cancellationToken };
					await Parallel.ForEachAsync(filesToMove, options, (fileToMove, ct) =>
					{
						MoveFileFromCache(fileToMove, transaction._filesToSync);
						return ValueTask.CompletedTask;
					});

					_contentIdToTrackedFile = _contentIdToTrackedFile.Where(x => !transaction._filesToMove.ContainsKey(x.Value)).ToDictionary(x => x.Key, x => x.Value);
					status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}

			// Swap files in and out of the cache
			WorkspaceFileToCopy[] filesToCopy = [.. transaction._filesToCopy];
			if (filesToCopy.Length > 0)
			{
				using (Trace("CopyFiles"))
				using (ILoggerProgress status = _logger.BeginProgressScope(String.Format("Copying {0} {1} within workspace...", filesToCopy.Length, (filesToCopy.Length == 1) ? "file" : "files")))
				{
					Stopwatch timer = Stopwatch.StartNew();
					ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency, CancellationToken = cancellationToken };
					await Parallel.ForEachAsync(filesToCopy, options, (fileToCopy, ct) =>
					{
						CopyFileWithinWorkspace(fileToCopy, transaction._filesToSync);
						return ValueTask.CompletedTask;
					});
					status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
				}
			}

			// Find all the files we want to sync
			WorkspaceFileToSync[] filesToSync = [.. transaction._filesToSync];
			if (filesToSync.Length > 0)
			{
				long syncSize = filesToSync.Sum(x => x._streamFile.Length);

				// Make sure there's enough space on this drive
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					ulong freeSpace;
					bool success = GetDiskFreeSpaceEx(_baseDir.FullName, out _, out _, out freeSpace);
					if (!success)
					{
						throw new InsufficientSpaceException($"Can't get free space for {_baseDir.FullName}");
					}
					long minScratchSpaceBytes = _options.MinScratchSpace * 1024 * 1024;
					if ((long)freeSpace - syncSize < minScratchSpaceBytes)
					{
						throw new InsufficientSpaceException($"Not enough space to sync new files (free space: {freeSpace / (1024.0 * 1024.0):n1}mb, sync size: {syncSize / (1024.0 * 1024.0):n1}mb, min scratch space: {minScratchSpaceBytes / (1024.0 * 1024.0):n1}mb)");
					}
				}

				// Sync all the files
				using (Trace("SyncFiles"))
				using (ILoggerProgress status = _logger.BeginProgressScope(String.Format("Syncing {0} {1} using {2} threads...", filesToSync.Length, (filesToSync.Length == 1) ? "file" : "files", _options.NumParallelSyncThreads)))
				{
					Stopwatch timer = Stopwatch.StartNew();

					// Remove all the previous response files
					foreach (FileReference file in DirectoryReference.EnumerateFiles(_baseDir, "SyncList-*.txt"))
					{
						FileUtils.ForceDeleteFile(file);
					}

					// Create a list of all the batches that we want to sync
					List<(int, int)> batches = [];
					for (int endIdx = 0; endIdx < filesToSync.Length;)
					{
						int beginIdx = endIdx;

						// Figure out the next batch of files to sync
						long batchSize = 0;
						for (; endIdx < filesToSync.Length && batchSize < 256 * 1024 * 1024; endIdx++)
						{
							batchSize += filesToSync[endIdx]._streamFile.Length;
						}

						// Add this batch to the list
						batches.Add((beginIdx, endIdx));
					}

					// The next batch to be synced
					int nextBatchIdx = 0;

					// Total size of synced files
					long syncedSize = 0;

					// Spawn some background threads to sync them
					Dictionary<Task, int> tasks = [];
					Stack<IPerforceConnection> connectionPool = new();
					try
					{
						while (tasks.Count > 0 || nextBatchIdx < batches.Count)
						{
							// Create new tasks
							while (tasks.Count < _options.NumParallelSyncThreads && nextBatchIdx < batches.Count)
							{
								(int batchBeginIdx, int batchEndIdx) = batches[nextBatchIdx];

								Task task = Task.Run(() => SyncBatchAsync(client, filesToSync, batchBeginIdx, batchEndIdx, fakeSync, connectionPool, cancellationToken), cancellationToken);
								tasks[task] = nextBatchIdx++;
							}

							// Wait for anything to complete
							Task completeTask = await Task.WhenAny(tasks.Keys);
							await completeTask; // Make sure we re-throw any exceptions from the task that completed

							int batchIdx = tasks[completeTask];
							tasks.Remove(completeTask);

							// Update metadata for the complete batch
							(int beginIdx, int endIdx) = batches[batchIdx];

							int[] indexesToUpdate = [.. Enumerable.Range(beginIdx, endIdx - beginIdx)];
							ParallelOptions options = new() { MaxDegreeOfParallelism = _options.MaxFileScanConcurrency, CancellationToken = cancellationToken };
							await Parallel.ForEachAsync(indexesToUpdate, options, (idx, ct) =>
							{
								filesToSync[idx]._workspaceFile.UpdateMetadata();
								return ValueTask.CompletedTask;
							});

							// Save the current state every minute
							TimeSpan elapsed = timer.Elapsed;
							if (elapsed > TimeSpan.FromMinutes(5.0))
							{
								await SaveAsync(TransactionState.Dirty, cancellationToken);
								_logger.LogInformation("Saved workspace state ({Elapsed:0.0}s)", (timer.Elapsed - elapsed).TotalSeconds);
								timer.Restart();
							}

							// Update the status
							for (int idx = beginIdx; idx < endIdx; idx++)
							{
								syncedSize += filesToSync[idx]._streamFile.Length;
							}
							status.Progress = String.Format("{0:n1}% ({1:n1}mb/{2:n1}mb)", syncedSize * 100.0 / syncSize, syncedSize / (1024.0 * 1024.0), syncSize / (1024.0 * 1024.0));
						}
					}
					finally
					{
						await Task.WhenAll(tasks.Keys);

						foreach (IPerforceConnection connection in connectionPool)
						{
							connection.Dispose();
						}
					}
				}
			}

			// Save the clean state
			_workspace = transaction._newWorkspaceRootDir;
			await SaveAsync(TransactionState.Clean, cancellationToken);
		}

		/// <summary>
		/// Syncs a batch of files
		/// </summary>
		/// <param name="client">The client to sync</param>
		/// <param name="filesToSync">List of files to sync</param>
		/// <param name="beginIdx">First file to sync</param>
		/// <param name="endIdx">Index of the last file to sync (exclusive)</param>
		/// <param name="fakeSync">Whether to fake a sync</param>
		/// <param name="connectionPool">Pool of connection instances</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Async task</returns>
		async Task SyncBatchAsync(IPerforceConnection client, WorkspaceFileToSync[] filesToSync, int beginIdx, int endIdx, bool fakeSync, Stack<IPerforceConnection> connectionPool, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ManagedWorkspace)}.{nameof(SyncBatchAsync)}");
			if (fakeSync)
			{
				for (int idx = beginIdx; idx < endIdx; idx++)
				{
					FileReference localFile = filesToSync[idx]._workspaceFile.GetLocation();
					DirectoryReference.CreateDirectory(localFile.Directory);
					await FileReference.WriteAllBytesAsync(localFile, [], cancellationToken);
				}
			}
			else if (client is NativePerforceConnection)
			{
				List<string> files = [];
				for (int idx = beginIdx; idx < endIdx; idx++)
				{
					files.Add($"{filesToSync[idx]._streamFile.Path}#{filesToSync[idx]._streamFile.Revision}");
				}

				SyncOptions options = SyncOptions.FullDepotSyntax;
				if (_options.UseHaveTable)
				{
					options |= SyncOptions.Force;
				}
				else
				{
					options |= SyncOptions.DoNotUpdateHaveList;
				}

				// Allocate a connection to use for syncing
#pragma warning disable CA2000
				IPerforceConnection? connection = null;
				try
				{
					lock (connectionPool)
					{
						connectionPool.TryPop(out connection);
					}
					connection ??= await PerforceConnection.CreateAsync(client.Settings, client.Logger);

					// Note: Explicitly disable parallel syncing here; the P4 API attempts to shell out to p4.exe, which may not be installed.
					await connection.SyncAsync(options, -1, 0, -1, -1, -1, -1, files, cancellationToken).ToListAsync(cancellationToken);
				}
				finally
				{
					if (connection != null)
					{
						lock (connectionPool)
						{
							connectionPool.Push(connection);
						}
					}
				}
#pragma warning restore CA2000
			}
			else
			{
				FileReference syncFileName = FileReference.Combine(_baseDir, $"SyncList-{beginIdx}.txt");
				using (StreamWriter writer = new(syncFileName.FullName))
				{
					for (int idx = beginIdx; idx < endIdx; idx++)
					{
						await writer.WriteLineAsync($"{filesToSync[idx]._streamFile.Path}#{filesToSync[idx]._streamFile.Revision}");
					}
				}

				if (_options.UseHaveTable)
				{
					using PerforceConnection clientWithFileList = new(client.Settings, client.Logger);
					clientWithFileList.GlobalOptions.Add($"-x\"{syncFileName}\"");
					await clientWithFileList.SyncAsync(SyncOptions.Force | SyncOptions.FullDepotSyntax, -1, Array.Empty<string>(), cancellationToken).ToListAsync(cancellationToken);
				}
				else
				{
					// Ensure a client with an empty have table is used to not interfere with the DoNotUpdateHaveList option.
					using PerforceConnection clientWithFileList = new(client.Settings, client.Logger);
					clientWithFileList.ClientName = client.Settings.ClientName!;
					clientWithFileList.GlobalOptions.Add($"-x\"{syncFileName}\"");
					await clientWithFileList.SyncAsync(SyncOptions.DoNotUpdateHaveList | SyncOptions.FullDepotSyntax, -1, Array.Empty<string>(), cancellationToken).ToListAsync(cancellationToken);
				}
			}
		}

		/// <summary>
		/// Helper function to move a file from the cache into the workspace. If it fails, adds the file to a list to be synced.
		/// </summary>
		/// <param name="fileToMove">Information about the file to move</param>
		/// <param name="filesToSync">List of files to be synced. If the move fails, the file will be added to this list of files to sync.</param>
		void MoveFileFromCache(WorkspaceFileToMove fileToMove, ConcurrentQueue<WorkspaceFileToSync> filesToSync)
		{
			try
			{
				FileReference targetFile = fileToMove._workspaceFile.GetLocation();
				FileReference.Move(fileToMove._trackedFile.GetLocation(), targetFile);
				try
				{
					DateTime lastWriteTime = _options.UseModTime
						? new DateTime(fileToMove._workspaceFile._lastModifiedTicks, DateTimeKind.Utc)
						: DateTime.UtcNow;

					FileReference.SetLastWriteTimeUtc(targetFile, lastWriteTime);
					fileToMove._workspaceFile.UpdateMetadata();
				}
				catch
				{
					_logger.LogWarning("Unable to update timestamp on {TargetFile}", targetFile);
				}
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "warning: Unable to move {CacheFile} from cache to {WorkspaceFile}. Syncing instead.", fileToMove._trackedFile.GetLocation(), fileToMove._workspaceFile.GetLocation());
				filesToSync.Enqueue(new WorkspaceFileToSync(fileToMove._streamFile, fileToMove._workspaceFile));
				_requiresRepair = true;
			}
		}

		/// <summary>
		/// Helper function to copy a file within the workspace. If it fails, adds the file to a list to be synced.
		/// </summary>
		/// <param name="fileToCopy">Information about the file to move</param>
		/// <param name="filesToSync">List of files to be synced. If the move fails, the file will be added to this list of files to sync.</param>
		void CopyFileWithinWorkspace(WorkspaceFileToCopy fileToCopy, ConcurrentQueue<WorkspaceFileToSync> filesToSync)
		{
			try
			{
				FileReference.Copy(fileToCopy._sourceWorkspaceFile.GetLocation(), fileToCopy._targetWorkspaceFile.GetLocation());
				fileToCopy._targetWorkspaceFile.UpdateMetadata();
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "warning: Unable to copy {SourceFile} to {TargetFile}. Syncing instead.", fileToCopy._sourceWorkspaceFile.GetLocation(), fileToCopy._targetWorkspaceFile.GetLocation());
				filesToSync.Enqueue(new WorkspaceFileToSync(fileToCopy._streamFile, fileToCopy._targetWorkspaceFile));
				_requiresRepair = true;
			}
		}

		void RemoveTrackedFile(CachedFileInfo trackedFile)
		{
			_contentIdToTrackedFile.Remove(trackedFile.ContentId);
			_cacheEntries.Remove(trackedFile.CacheId);
			FileUtils.ForceDeleteFile(trackedFile.GetLocation());
		}

		void CreateCacheHierarchy()
		{
			for (int idxA = 0; idxA < 16; idxA++)
			{
				DirectoryReference dirA = DirectoryReference.Combine(_cacheDir, String.Format("{0:X}", idxA));
				DirectoryReference.CreateDirectory(dirA);
				for (int idxB = 0; idxB < 16; idxB++)
				{
					DirectoryReference dirB = DirectoryReference.Combine(dirA, String.Format("{0:X}", idxB));
					DirectoryReference.CreateDirectory(dirB);
					for (int idxC = 0; idxC < 16; idxC++)
					{
						DirectoryReference dirC = DirectoryReference.Combine(dirB, String.Format("{0:X}", idxC));
						DirectoryReference.CreateDirectory(dirC);
					}
				}
			}
		}

		/// <summary>
		/// Determines a unique cache id for a file content id
		/// </summary>
		/// <param name="contentId">File content id to get a unique id for</param>
		/// <returns>The unique cache id</returns>
		ulong GetUniqueCacheId(FileContentId contentId)
		{
			// Initialize the cache id to the top 16 bytes of the digest, then increment it until we find a unique id
			ulong cacheId = 0;
			for (int idx = 0; idx < 8; idx++)
			{
				cacheId = (cacheId << 8) | contentId.Digest.Span[idx];
			}
			while (!_cacheEntries.Add(cacheId))
			{
				cacheId++;
			}
			return cacheId;
		}

		/// <summary>
		/// Removes the leading slash from a path
		/// </summary>
		/// <param name="path">The path to remove a slash from</param>
		/// <returns>The path without a leading slash</returns>
		static string RemoveLeadingSlash(string path)
		{
			if (path.Length > 0 && path[0] == '/')
			{
				return path.Substring(1);
			}
			else
			{
				return path;
			}
		}

		/// <summary>
		/// Gets the path to a backup file used while a new file is being written out
		/// </summary>
		/// <param name="targetFile">The file being written to</param>
		/// <returns>The path to a backup file</returns>
		private static FileReference GetBackupFile(FileReference targetFile)
		{
			return new FileReference(targetFile.FullName + ".transaction");
		}

		/// <summary>
		/// Begins a write transaction on the given file. Assumes only one process will be reading/writing at a time, but the operation can be interrupted.
		/// </summary>
		/// <param name="targetFile">The file being written to</param>
		public static void BeginTransaction(FileReference targetFile)
		{
			FileReference transactionFile = GetBackupFile(targetFile);
			if (FileReference.Exists(targetFile))
			{
				FileUtils.ForceMoveFile(targetFile, transactionFile);
			}
			else if (FileReference.Exists(transactionFile))
			{
				FileUtils.ForceDeleteFile(transactionFile);
			}
		}

		/// <summary>
		/// Mark a transaction on the given file as complete, and removes the backup file.
		/// </summary>
		/// <param name="targetFile">The file being written to</param>
		public static void CompleteTransaction(FileReference targetFile)
		{
			FileReference transactionFile = GetBackupFile(targetFile);
			FileUtils.ForceDeleteFile(transactionFile);
		}

		/// <summary>
		/// Restores the backup for a target file, if it exists. This allows recovery from an incomplete transaction.
		/// </summary>
		/// <param name="targetFile">The file being written to</param>
		public static void RestoreBackup(FileReference targetFile)
		{
			FileReference transactionFile = GetBackupFile(targetFile);
			if (FileReference.Exists(transactionFile))
			{
				FileUtils.ForceMoveFile(transactionFile, targetFile);
			}
		}

		/// <summary>
		/// Creates a scoped trace object
		/// </summary>
		/// <param name="operation">Name of the operation</param>
		/// <returns>Disposable object for the trace</returns>
		private static IDisposable Trace(string operation)
		{
			return TraceSpan.Create(operation, service: "hordeagent_repository");
		}

		#endregion
	}
}
