// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Interface for a stream snapshot
	/// </summary>
	public abstract class StreamSnapshot
	{
		/// <summary>
		/// Empty snapshot instance
		/// </summary>
		public static StreamSnapshot Empty => new StreamSnapshotFromMemory(new StreamTreeBuilder());

		/// <summary>
		/// The root digest
		/// </summary>
		public abstract StreamTreeRef Root { get; }

		/// <summary>
		/// Lookup a directory by reference
		/// </summary>
		/// <param name="treeRef">The reference</param>
		/// <returns></returns>
		public abstract StreamTree Lookup(StreamTreeRef treeRef);
	}

	/// <summary>
	/// Extension methods for IStreamSnapshot
	/// </summary>
	static class StreamSnapshotExtensions
	{
		/// <summary>
		/// Get all the files in this directory
		/// </summary>
		/// <returns>List of files</returns>
		public static List<StreamFile> GetFiles(this StreamSnapshot snapshot)
		{
			List<StreamFile> files = [];
			AppendFiles(snapshot, snapshot.Root, files);
			return files;
		}

		/// <summary>
		/// Append the contents of this directory and subdirectories to a list
		/// </summary>
		/// <param name="snapshot"></param>
		/// <param name="treeRef"></param>
		/// <param name="files">List to append to</param>
		static void AppendFiles(StreamSnapshot snapshot, StreamTreeRef treeRef, List<StreamFile> files)
		{
			StreamTree directoryInfo = snapshot.Lookup(treeRef);
			foreach (StreamTreeRef subDirRef in directoryInfo.NameToTree.Values)
			{
				AppendFiles(snapshot, subDirRef, files);
			}
			files.AddRange(directoryInfo.NameToFile.Values);
		}
	}
}
