// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Stores the contents of a stream in memory
	/// </summary>
	public class StreamSnapshotFromMemory : StreamSnapshot
	{
		/// <summary>
		/// The current signature for saved directory objects
		/// </summary>
		static readonly byte[] s_currentSignature = [(byte)'W', (byte)'S', (byte)'D', 5];

		/// <summary>
		/// The root digest
		/// </summary>
		public override StreamTreeRef Root { get; }

		/// <summary>
		/// Map of digest to directory
		/// </summary>
		public IReadOnlyDictionary<IoHash, CbObject> HashToTree { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="root"></param>
		/// <param name="hashToTree"></param>
		public StreamSnapshotFromMemory(StreamTreeRef root, Dictionary<IoHash, CbObject> hashToTree)
		{
			Root = root;
			HashToTree = hashToTree;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="builder"></param>
		public StreamSnapshotFromMemory(StreamTreeBuilder builder)
		{
			Dictionary<IoHash, CbObject> hashToTree = [];
			Root = builder.EncodeRef(tree => EncodeObject(tree, hashToTree));
			HashToTree = hashToTree;
		}

		/// <summary>
		/// Serialize to a compact binary object
		/// </summary>
		/// <param name="tree"></param>
		/// <param name="hashToTree"></param>
		/// <returns></returns>
		static IoHash EncodeObject(StreamTree tree, Dictionary<IoHash, CbObject> hashToTree)
		{
			CbObject @object = tree.ToCbObject();

			IoHash hash = @object.GetHash();
			hashToTree[hash] = @object;

			return hash;
		}

		/// <inheritdoc/>
		public override StreamTree Lookup(StreamTreeRef treeRef)
		{
			return new StreamTree(treeRef.Path, HashToTree[treeRef.Hash]);
		}

		/// <summary>
		/// Load a stream directory from a file on disk
		/// </summary>
		/// <param name="inputFile">File to read from</param>
		/// <param name="defaultBasePath">Base path to use if missing inside loaded file</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>New StreamDirectoryInfo object</returns>
		public static async Task<StreamSnapshotFromMemory?> TryLoadAsync(FileReference inputFile, Utf8String defaultBasePath, CancellationToken cancellationToken)
		{
			byte[] data = await FileReference.ReadAllBytesAsync(inputFile, cancellationToken);
			if (!data.AsSpan().StartsWith(s_currentSignature))
			{
				return null;
			}

			CbObject rootObj = new(data.AsMemory(s_currentSignature.Length));

			CbObject rootObj2 = rootObj["root"].AsObject();
			Utf8String rootPath = rootObj2["path"].AsUtf8String(defaultBasePath);
			StreamTreeRef root = new(rootPath, rootObj2);

			CbArray array = rootObj["items"].AsArray();

			Dictionary<IoHash, CbObject> hashToTree = new(array.Count);
			foreach (CbField element in array)
			{
				CbObject objectElement = element.AsObject();
				IoHash hash = objectElement["hash"].AsHash();
				CbObject tree = objectElement["tree"].AsObject();
				hashToTree[hash] = tree;
			}

			return new StreamSnapshotFromMemory(root, hashToTree);
		}

		/// <summary>
		/// Saves the contents of this object to disk
		/// </summary>
		/// <param name="outputFile">The output file to write to</param>
		/// <param name="basePath"></param>
		public async Task SaveAsync(FileReference outputFile, Utf8String basePath)
		{
			CbWriter writer = new();
			writer.BeginObject();

			writer.BeginObject("root");
			if (Root.Path != basePath)
			{
				writer.WriteUtf8String("path", Root.Path);
			}
			Root.Write(writer);
			writer.EndObject();

			writer.BeginArray("items");
			foreach ((IoHash hash, CbObject tree) in HashToTree)
			{
				writer.BeginObject();
				writer.WriteHash("hash", hash);
				writer.WriteObject("tree", tree);
				writer.EndObject();
			}
			writer.EndArray();

			writer.EndObject();

			byte[] data = writer.ToByteArray();
			using (FileStream outputStream = FileReference.Open(outputFile, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				await outputStream.WriteAsync(s_currentSignature, 0, s_currentSignature.Length);
				await outputStream.WriteAsync(data, 0, data.Length);
			}
		}
	}
}
