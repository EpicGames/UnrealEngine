// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Metadata for a Perforce file
	/// </summary>
	[DebuggerDisplay("{Path}")]
	public class StreamFile
	{
		/// <summary>
		/// Depot path for this file
		/// </summary>
		public Utf8String Path { get; }

		/// <summary>
		/// Length of the file, as reported by the server (actual size on disk may be different due to workspace options).
		/// </summary>
		public long Length { get; }

		/// <summary>
		/// Unique identifier for the file content
		/// </summary>
		public FileContentId ContentId { get; }

		/// <summary>
		/// Revision number of the file
		/// </summary>
		public int Revision { get; }

		#region Field names
		static readonly Utf8String s_lengthField = new("len");
		static readonly Utf8String s_digestField = new("dig");
		static readonly Utf8String s_typeField = new("type");
		static readonly Utf8String s_revisionField = new("rev");
		#endregion

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamFile(string path, long length, FileContentId contentId, int revision)
			: this(new Utf8String(path), length, contentId, revision)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamFile(Utf8String path, long length, FileContentId contentId, int revision)
		{
			Path = path;
			Length = length;
			ContentId = contentId;
			Revision = revision;
		}

		/// <summary>
		/// Parse from a compact binary object
		/// </summary>
		/// <param name="path">Path to the file</param>
		/// <param name="field"></param>
		/// <returns></returns>
		public StreamFile(Utf8String path, CbObject field)
		{
			Path = path;
			Length = field[s_lengthField].AsInt64();

			Md5Hash digest = new(field[s_digestField].AsBinary());
			Utf8String type = field[s_typeField].AsUtf8String();
			ContentId = new FileContentId(digest, type);

			Revision = field[s_revisionField].AsInt32();
		}

		/// <summary>
		/// Write this object to compact binary
		/// </summary>
		/// <param name="writer"></param>
		public void Write(CbWriter writer)
		{
			writer.WriteInteger(s_lengthField, Length);
			writer.WriteBinarySpan(s_digestField, ContentId.Digest.Span);
			writer.WriteUtf8String(s_typeField, ContentId.Type);
			writer.WriteInteger(s_revisionField, Revision);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"StreamFile({Path}#{Revision} Len={Length} Digest={ContentId.Digest} Type={ContentId.Type})";
		}
	}

	/// <summary>
	/// Stores a reference to another tree
	/// </summary>
	public class StreamTreeRef
	{
		/// <summary>
		/// Base depot path for the directory
		/// </summary>
		public Utf8String Path { get; set; }

		/// <summary>
		/// Hash of the tree
		/// </summary>
		public IoHash Hash { get; set; }

		#region Field names
		static readonly Utf8String s_hashField = new("hash");
		#endregion

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path"></param>
		/// <param name="hash"></param>
		public StreamTreeRef(Utf8String path, IoHash hash)
		{
			Path = path;
			Hash = hash;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="path"></param>
		/// <param name="field"></param>
		public StreamTreeRef(Utf8String path, CbObject field)
		{
			Path = path;
			Hash = field[s_hashField].AsObjectAttachment();
		}

		/// <summary>
		/// Gets the hash of this reference
		/// </summary>
		/// <returns></returns>
		public IoHash ComputeHash()
		{
			CbWriter writer = new();
			writer.BeginObject();
			Write(writer);
			writer.EndObject();
			return writer.ToObject().GetHash();
		}

		/// <summary>
		/// Serialize to a compact binary object
		/// </summary>
		/// <param name="writer"></param>
		public void Write(CbWriter writer)
		{
			writer.WriteObjectAttachment(s_hashField, Hash);
		}
	}

	/// <summary>
	/// Information about a directory within a stream
	/// </summary>
	public class StreamTree
	{
		/// <summary>
		/// The path to this tree
		/// </summary>
		public Utf8String Path { get; }

		/// <summary>
		/// Map of name to file within the directory
		/// </summary>
		public Dictionary<Utf8String, StreamFile> NameToFile { get; } = [];

		/// <summary>
		/// Map of name to subdirectory
		/// </summary>
		public Dictionary<Utf8String, StreamTreeRef> NameToTree { get; } = new Dictionary<Utf8String, StreamTreeRef>(FileUtils.PlatformPathComparerUtf8);

		#region Field names
		static readonly Utf8String s_nameField = new("name");
		static readonly Utf8String s_pathField = new("path");
		static readonly Utf8String s_filesField = new("files");
		static readonly Utf8String s_treesField = new("trees");
		#endregion

		/// <summary>
		/// Default constructor
		/// </summary>
		public StreamTree()
		{
		}

		/// <summary>
		/// Default constructor
		/// </summary>
		public StreamTree(Utf8String path, Dictionary<Utf8String, StreamFile> nameToFile, Dictionary<Utf8String, StreamTreeRef> nameToTree)
		{
			CheckPath(path);
			Path = path;

			NameToFile = nameToFile;
			NameToTree = nameToTree;
		}

		/// <summary>
		/// Deserialize a tree from a compact binary object
		/// </summary>
		public StreamTree(Utf8String path, CbObject @object)
		{
			CheckPath(path);
			Path = path;

			CbArray fileArray = @object[s_filesField].AsArray();
			foreach (CbField fileField in fileArray)
			{
				CbObject fileObject = fileField.AsObject();

				Utf8String name = fileObject[s_nameField].AsUtf8String();
				Utf8String filePath = ReadPath(fileObject, path, name);
				StreamFile file = new(filePath, fileObject);

				NameToFile.Add(name, file);
			}

			CbArray treeArray = @object[s_treesField].AsArray();
			foreach (CbField treeField in treeArray)
			{
				CbObject treeObject = treeField.AsObject();

				Utf8String name = treeObject[s_nameField].AsUtf8String();
				Utf8String treePath = ReadPath(treeObject, path, name);
				StreamTreeRef tree = new(treePath, treeObject);

				NameToTree.Add(name, tree);
			}
		}

		/// <summary>
		/// Serialize to a compact binary object
		/// </summary>
		/// <param name="writer"></param>
		public void Write(CbWriter writer)
		{
			if (NameToFile.Count > 0)
			{
				writer.BeginArray(s_filesField);
				foreach ((Utf8String name, StreamFile file) in NameToFile.OrderBy(x => x.Key))
				{
					writer.BeginObject();
					writer.WriteUtf8String(s_nameField, name);
					WritePath(writer, file.Path, Path, name);
					file.Write(writer);
					writer.EndObject();
				}
				writer.EndArray();
			}

			if (NameToTree.Count > 0)
			{
				writer.BeginArray(s_treesField);
				foreach ((Utf8String name, StreamTreeRef tree) in NameToTree.OrderBy(x => x.Key))
				{
					writer.BeginObject();
					writer.WriteUtf8String(s_nameField, name);
					WritePath(writer, tree.Path, Path, name);
					tree.Write(writer);
					writer.EndObject();
				}
				writer.EndArray();
			}
		}

		/// <summary>
		/// Reads a path from an object, defaulting it to the parent path plus the child name
		/// </summary>
		/// <param name="object"></param>
		/// <param name="basePath"></param>
		/// <param name="name"></param>
		/// <returns></returns>
		static Utf8String ReadPath(CbObject @object, Utf8String basePath, Utf8String name)
		{
			Utf8String path = @object[s_pathField].AsUtf8String();
			if (path.IsEmpty)
			{
				byte[] data = new byte[basePath.Length + 1 + name.Length];
				basePath.Memory.CopyTo(data);
				data[basePath.Length] = (byte)'/';
				name.Memory.CopyTo(data.AsMemory(basePath.Length + 1));
				path = new Utf8String(data);
			}
			return path;
		}

		/// <summary>
		/// Writes a path if it's not the default (the parent path, a slash, followed by the child name)
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="path"></param>
		/// <param name="parentPath"></param>
		/// <param name="name"></param>
		static void WritePath(CbWriter writer, Utf8String path, Utf8String parentPath, Utf8String name)
		{
			if (path.Length != parentPath.Length + name.Length + 1 || !path.StartsWith(parentPath) || path[parentPath.Length] != '/' || !path.EndsWith(name))
			{
				writer.WriteUtf8String(s_pathField, path);
			}
		}

		/// <summary>
		/// Checks that a base path does not have a trailing slash
		/// </summary>
		/// <param name="path"></param>
		/// <exception cref="ArgumentException"></exception>
		static void CheckPath(Utf8String path)
		{
			if (path.Length > 0 && path[^1] == '/')
			{
				throw new ArgumentException("BasePath must not end in a slash", nameof(path));
			}
		}

		/// <summary>
		/// Convert to a compact binary object
		/// </summary>
		/// <returns></returns>
		public CbObject ToCbObject()
		{
			CbWriter writer = new();
			writer.BeginObject();
			Write(writer);
			writer.EndObject();
			return writer.ToObject();
		}
	}
}
