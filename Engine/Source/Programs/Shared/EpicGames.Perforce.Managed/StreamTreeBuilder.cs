// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Utility class to efficiently track changes to a StreamTree object in memory
	/// </summary>
	public class StreamTreeBuilder
	{
		/// <summary>
		/// Map from name to mutable tree
		/// </summary>
		public Dictionary<Utf8String, StreamFile> NameToFile { get; }

		/// <summary>
		/// Map from name to mutable tree
		/// </summary>
		public Dictionary<Utf8String, StreamTreeRef> NameToTree { get; }

		/// <summary>
		/// Map from name to mutable tree
		/// </summary>
		public Dictionary<Utf8String, StreamTreeBuilder> NameToTreeBuilder { get; } = new Dictionary<Utf8String, StreamTreeBuilder>(FileUtils.PlatformPathComparerUtf8);

		/// <summary>
		/// Tests whether the tree is empty
		/// </summary>
		public bool IsEmpty => NameToFile.Count == 0 && NameToTree.Count == 0 && NameToTreeBuilder.Count == 0;

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamTreeBuilder()
		{
			NameToFile = new Dictionary<Utf8String, StreamFile>(FileUtils.PlatformPathComparerUtf8);
			NameToTree = new Dictionary<Utf8String, StreamTreeRef>(FileUtils.PlatformPathComparerUtf8);
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="tree"></param>
		public StreamTreeBuilder(StreamTree tree)
		{
			NameToFile = new Dictionary<Utf8String, StreamFile>(tree.NameToFile, FileUtils.PlatformPathComparerUtf8);
			NameToTree = new Dictionary<Utf8String, StreamTreeRef>(tree.NameToTree, FileUtils.PlatformPathComparerUtf8);
		}

		/// <summary>
		/// Encodes the current tree state and returns a reference to it
		/// </summary>
		/// <param name="writeTree">Dictionary of encoded objects</param>
		/// <returns></returns>
		public StreamTree Encode(Func<StreamTree, IoHash> writeTree)
		{
			// Recursively serialize all the child items
			EncodeChildren(writeTree);

			// Find the common base path for all items in this tree.
			Dictionary<Utf8String, int> basePathToCount = [];
			foreach ((Utf8String name, StreamFile file) in NameToFile)
			{
				AddBasePath(basePathToCount, file.Path, name);
			}
			foreach ((Utf8String name, StreamTreeRef tree) in NameToTree)
			{
				AddBasePath(basePathToCount, tree.Path, name);
			}

			// Create the new tree
			Utf8String basePath = (basePathToCount.Count == 0) ? Utf8String.Empty : basePathToCount.MaxBy(x => x.Value).Key;
			return new StreamTree(basePath, NameToFile, NameToTree);
		}

		/// <summary>
		/// Encodes a StreamTreeRef from this tree
		/// </summary>
		/// <param name="writeTree"></param>
		/// <returns>The new tree ref</returns>
		public StreamTreeRef EncodeRef(Func<StreamTree, IoHash> writeTree)
		{
			StreamTree tree = Encode(writeTree);
			return new StreamTreeRef(tree.Path, writeTree(tree));
		}

		/// <summary>
		/// Collapses all of the builders underneath this node
		/// </summary>
		/// <param name="writeTree"></param>
		public void EncodeChildren(Func<StreamTree, IoHash> writeTree)
		{
			foreach ((Utf8String subTreeName, StreamTreeBuilder subTreeBuilder) in NameToTreeBuilder)
			{
				StreamTree subTree = subTreeBuilder.Encode(writeTree);
				if (subTree.NameToFile.Count > 0 || subTree.NameToTree.Count > 0)
				{
					IoHash hash = writeTree(subTree);
					NameToTree[subTreeName] = new StreamTreeRef(subTree.Path, hash);
				}
			}
			NameToTreeBuilder.Clear();
		}

		/// <summary>
		/// Adds the base path of the given item to the count of similar items
		/// </summary>
		/// <param name="basePathToCount"></param>
		/// <param name="path"></param>
		/// <param name="name"></param>
		static void AddBasePath(Dictionary<Utf8String, int> basePathToCount, Utf8String path, Utf8String name)
		{
			if (path.EndsWith(name) && path[^(name.Length + 1)] == '/')
			{
				Utf8String basePath = path[..^(name.Length + 1)];
				basePathToCount.TryGetValue(basePath, out int count);
				basePathToCount[basePath] = count + 1;
			}
		}
	}

	/// <summary>
	/// Helper variant of StreamTreeBuilder capable of adding files to tree using both client and depot file paths
	/// </summary>
	public class DepotStreamTreeBuilder : StreamTreeBuilder
	{
		/// <summary>
		/// Adds a file to the tree
		/// </summary>
		public void AddFile(string clientFile, StreamFile depotFile)
		{
			if (clientFile[0] == '/')
			{
				// Strip any slash at the start
				clientFile = clientFile[1..];
			}

			StreamTreeBuilder currentStreamDirectory = this;
			string[] pathFragments = clientFile.Split('/');

			// Find stream tree builder for deepest path fragment
			// and skip last fragment as that is the filename
			for (int i = 0; i < pathFragments.Length - 1; i++)
			{
				string pathFragment = pathFragments[i];
				Utf8String unescapedFragment = new(PerforceUtils.UnescapePath(pathFragment));

				if (!currentStreamDirectory.NameToTreeBuilder.TryGetValue(unescapedFragment, out StreamTreeBuilder? nextStreamDirectory))
				{
					nextStreamDirectory = new StreamTreeBuilder();
					currentStreamDirectory.NameToTreeBuilder.Add(unescapedFragment, nextStreamDirectory);
				}
				currentStreamDirectory = nextStreamDirectory;
			}

			Utf8String filename = new(PerforceUtils.UnescapePath(pathFragments[^1])); // Last fragment is filename
			currentStreamDirectory.NameToFile[filename] = depotFile;
		}
	}
}
