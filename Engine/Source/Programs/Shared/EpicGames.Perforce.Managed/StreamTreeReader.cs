// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Interface for 
	/// </summary>
	public abstract class StreamTreeReader
	{
		/// <summary>
		/// Reads a node of the tree
		/// </summary>
		/// <param name="treeRef"></param>
		/// <returns></returns>
		public abstract Task<StreamTree> ReadAsync(StreamTreeRef treeRef);
	}

	/// <summary>
	/// Implements a <see cref="StreamTreeReader"/> using a contiguous block of memory
	/// </summary>
	public class StreamTreeMemoryReader : StreamTreeReader
	{
		/// <summary>
		/// Map from hash to encoded CB tree object
		/// </summary>
		readonly Dictionary<IoHash, CbObject> _hashToTree;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="hashToTree"></param>
		public StreamTreeMemoryReader(Dictionary<IoHash, CbObject> hashToTree)
		{
			_hashToTree = hashToTree;
		}

		/// <inheritdoc/>
		public override Task<StreamTree> ReadAsync(StreamTreeRef treeRef)
		{
			return Task.FromResult(new StreamTree(treeRef.Path, _hashToTree[treeRef.Hash]));
		}
	}
}
