// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	class WorkspaceFileToMove
	{
		public StreamFile _streamFile;
		public CachedFileInfo _trackedFile;
		public WorkspaceFileInfo _workspaceFile;

		public WorkspaceFileToMove(StreamFile streamFile, CachedFileInfo trackedFile, WorkspaceFileInfo workspaceFile)
		{
			_streamFile = streamFile;
			_trackedFile = trackedFile;
			_workspaceFile = workspaceFile;
		}
	}

	class WorkspaceFileToCopy
	{
		public StreamFile _streamFile;
		public WorkspaceFileInfo _sourceWorkspaceFile;
		public WorkspaceFileInfo _targetWorkspaceFile;

		public WorkspaceFileToCopy(StreamFile streamFile, WorkspaceFileInfo sourceWorkspaceFile, WorkspaceFileInfo targetWorkspaceFile)
		{
			_streamFile = streamFile;
			_sourceWorkspaceFile = sourceWorkspaceFile;
			_targetWorkspaceFile = targetWorkspaceFile;
		}
	}

	class WorkspaceFileToSync
	{
		public StreamFile _streamFile;
		public WorkspaceFileInfo _workspaceFile;

		public WorkspaceFileToSync(StreamFile streamFile, WorkspaceFileInfo workspaceFile)
		{
			_streamFile = streamFile;
			_workspaceFile = workspaceFile;
		}
	}

	class AddTransaction
	{
		public WorkspaceDirectoryInfo _newWorkspaceRootDir;
		public StreamSnapshot _streamSnapshot;
		public ConcurrentDictionary<CachedFileInfo, WorkspaceFileToMove> _filesToMove = new();
		public ConcurrentQueue<WorkspaceFileToCopy> _filesToCopy = new();
		public ConcurrentQueue<WorkspaceFileToSync> _filesToSync = new();
		readonly Dictionary<FileContentId, CachedFileInfo> _contentIdToTrackedFile;
		readonly Dictionary<FileContentId, WorkspaceFileInfo> _contentIdToWorkspaceFile = [];

		private AddTransaction(WorkspaceDirectoryInfo workspaceRootDir, StreamSnapshot streamSnapshot, Dictionary<FileContentId, CachedFileInfo> contentIdToTrackedFile)
		{
			_streamSnapshot = streamSnapshot;
			_newWorkspaceRootDir = new WorkspaceDirectoryInfo(workspaceRootDir.GetLocation());

			_contentIdToTrackedFile = contentIdToTrackedFile;

			List<WorkspaceFileInfo> workspaceFiles = workspaceRootDir.GetFiles();
			foreach (WorkspaceFileInfo workspaceFile in workspaceFiles)
			{
				_contentIdToWorkspaceFile[workspaceFile.ContentId] = workspaceFile;
			}
		}

		public static async Task<AddTransaction> CreateAsync(WorkspaceDirectoryInfo workspaceRootDir, StreamSnapshot streamSnapshot, Dictionary<FileContentId, CachedFileInfo> contentIdToTrackedFile, int numWorkers)
		{
			AddTransaction transaction = new(workspaceRootDir, streamSnapshot, contentIdToTrackedFile);
			using AsyncThreadPoolWorkQueue queue = new(numWorkers);
			await queue.EnqueueAsync(_ => transaction.MergeDirectoryAsync(workspaceRootDir, transaction._newWorkspaceRootDir, streamSnapshot.Root, queue));
			await queue.ExecuteAsync();
			return transaction;
		}

		async Task MergeDirectoryAsync(WorkspaceDirectoryInfo workspaceDir, WorkspaceDirectoryInfo newWorkspaceDir, StreamTreeRef streamTreeRef, AsyncThreadPoolWorkQueue queue)
		{
			// Make sure the directory exists
			Directory.CreateDirectory(workspaceDir.GetFullName());

			// Update all the subdirectories
			StreamTree streamDir = _streamSnapshot.Lookup(streamTreeRef);
			foreach ((Utf8String subDirName, StreamTreeRef subDirRef) in streamDir.NameToTree)
			{
				WorkspaceDirectoryInfo? workspaceSubDir;
				if (workspaceDir.NameToSubDirectory.TryGetValue(subDirName, out workspaceSubDir))
				{
					await MergeSubDirectoryAsync(subDirName, workspaceSubDir, subDirRef, newWorkspaceDir, queue);
				}
				else
				{
					await AddSubDirectoryAsync(subDirName, newWorkspaceDir, subDirRef, queue);
				}
			}

			// Move files into this folder
			foreach ((Utf8String name, StreamFile streamFile) in streamDir.NameToFile)
			{
				WorkspaceFileInfo? workspaceFile;
				if (workspaceDir.NameToFile.TryGetValue(name, out workspaceFile))
				{
					newWorkspaceDir.NameToFile.Add(workspaceFile.Name, workspaceFile);
				}
				else
				{
					AddFile(newWorkspaceDir, name, streamFile);
				}
			}
		}

		async Task AddDirectoryAsync(WorkspaceDirectoryInfo newWorkspaceDir, StreamTreeRef streamTreeRef, AsyncThreadPoolWorkQueue queue)
		{
			StreamTree streamDir = _streamSnapshot.Lookup(streamTreeRef);

			// Make sure the directory exists
			Directory.CreateDirectory(newWorkspaceDir.GetFullName());

			// Add all the sub directories and files
			foreach ((Utf8String subDirName, StreamTreeRef subDirRef) in streamDir.NameToTree)
			{
				await AddSubDirectoryAsync(subDirName, newWorkspaceDir, subDirRef, queue);
			}
			foreach ((Utf8String name, StreamFile streamFile) in streamDir.NameToFile)
			{
				AddFile(newWorkspaceDir, name, streamFile);
			}
		}

		async Task MergeSubDirectoryAsync(Utf8String name, WorkspaceDirectoryInfo workspaceSubDir, StreamTreeRef streamSubTreeRef, WorkspaceDirectoryInfo newWorkspaceDir, AsyncThreadPoolWorkQueue queue)
		{
			WorkspaceDirectoryInfo newWorkspaceSubDir = new(newWorkspaceDir, name, streamSubTreeRef);
			newWorkspaceDir.NameToSubDirectory.Add(name, newWorkspaceSubDir);
			await queue.EnqueueAsync(_ => MergeDirectoryAsync(workspaceSubDir, newWorkspaceSubDir, streamSubTreeRef, queue));
		}

		async Task AddSubDirectoryAsync(Utf8String name, WorkspaceDirectoryInfo newWorkspaceDir, StreamTreeRef streamSubTreeRef, AsyncThreadPoolWorkQueue queue)
		{
			WorkspaceDirectoryInfo newWorkspaceSubDir = new(newWorkspaceDir, name, streamSubTreeRef);
			newWorkspaceDir.NameToSubDirectory.Add(name, newWorkspaceSubDir);
			await queue.EnqueueAsync(_ => AddDirectoryAsync(newWorkspaceSubDir, streamSubTreeRef, queue));
		}

		void AddFile(WorkspaceDirectoryInfo newWorkspaceDir, Utf8String name, StreamFile streamFile)
		{
			// Create a new file for this workspace
			WorkspaceFileInfo workspaceFile = new(newWorkspaceDir, name, streamFile.ContentId);
			newWorkspaceDir.NameToFile.Add(name, workspaceFile);

			// Try to add it to the cache
			CachedFileInfo? trackedFile;
			if (_contentIdToTrackedFile.TryGetValue(streamFile.ContentId, out trackedFile))
			{
				if (_filesToMove.TryAdd(trackedFile, new WorkspaceFileToMove(streamFile, trackedFile, workspaceFile)))
				{
					workspaceFile.SetMetadata(trackedFile.Length, trackedFile.LastModifiedTicks, trackedFile.ReadOnly);
				}
				else
				{
					_filesToCopy.Enqueue(new WorkspaceFileToCopy(streamFile, _filesToMove[trackedFile]._workspaceFile, workspaceFile));
				}
			}
			else
			{
				WorkspaceFileInfo? sourceWorkspaceFile;
				if (_contentIdToWorkspaceFile.TryGetValue(streamFile.ContentId, out sourceWorkspaceFile))
				{
					_filesToCopy.Enqueue(new WorkspaceFileToCopy(streamFile, sourceWorkspaceFile, workspaceFile));
				}
				else
				{
					_filesToSync.Enqueue(new WorkspaceFileToSync(streamFile, workspaceFile));
				}
			}
		}
	}
}
