// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	class RemoveTransaction
	{
		readonly Dictionary<FileContentId, CachedFileInfo> _contentIdToTrackedFile;

		public WorkspaceDirectoryInfo _newWorkspaceRootDir;
		public StreamSnapshot _streamSnapshot;
		public ConcurrentDictionary<FileContentId, WorkspaceFileInfo> _filesToMove = new();
		public ConcurrentQueue<WorkspaceFileInfo> _filesToDelete = new();
		public ConcurrentQueue<WorkspaceDirectoryInfo> _directoriesToDelete = new();

		public RemoveTransaction(WorkspaceDirectoryInfo workspaceRootDir, StreamSnapshot streamSnapshot, Dictionary<FileContentId, CachedFileInfo> contentIdToTrackedFile)
		{
			_newWorkspaceRootDir = new WorkspaceDirectoryInfo(workspaceRootDir.GetLocation());
			_streamSnapshot = streamSnapshot;
			_contentIdToTrackedFile = contentIdToTrackedFile;
		}

		public static async Task<RemoveTransaction> CreateAsync(WorkspaceDirectoryInfo workspaceRootDir, StreamSnapshot streamSnapshot, Dictionary<FileContentId, CachedFileInfo> contentIdToTrackedFile, int numWorkers)
		{
			RemoveTransaction transaction = new(workspaceRootDir, streamSnapshot, contentIdToTrackedFile);

			using AsyncThreadPoolWorkQueue queue = new(numWorkers);
			await queue.EnqueueAsync(_ => transaction.MergeAsync(workspaceRootDir, transaction._newWorkspaceRootDir, streamSnapshot.Root, queue));
			await queue.ExecuteAsync();
			return transaction;
		}

		async Task MergeAsync(WorkspaceDirectoryInfo workspaceDir, WorkspaceDirectoryInfo newWorkspaceDir, StreamTreeRef streamTreeRef, AsyncThreadPoolWorkQueue queue)
		{
			StreamTree streamDir = _streamSnapshot.Lookup(streamTreeRef);

			// Update all the subdirectories
			foreach (WorkspaceDirectoryInfo workspaceSubDir in workspaceDir.NameToSubDirectory.Values)
			{
				StreamTreeRef? streamSubTreeRef;
				if (streamDir.NameToTree.TryGetValue(workspaceSubDir.Name, out streamSubTreeRef))
				{
					await MergeSubDirectoryAsync(workspaceSubDir.Name, newWorkspaceDir, workspaceSubDir, streamSubTreeRef, queue);
				}
				else
				{
					await RemoveDirectoryAsync(workspaceSubDir, queue);
				}
			}

			// Update the staged files
			foreach (WorkspaceFileInfo workspaceFile in workspaceDir.NameToFile.Values)
			{
				StreamFile? streamFile;
				if (streamDir != null && streamDir.NameToFile.TryGetValue(workspaceFile.Name, out streamFile) && streamFile.ContentId.Equals(workspaceFile.ContentId))
				{
					newWorkspaceDir.NameToFile.Add(workspaceFile.Name, workspaceFile);
				}
				else
				{
					RemoveFile(workspaceFile);
				}
			}
		}

		async Task MergeSubDirectoryAsync(Utf8String name, WorkspaceDirectoryInfo newWorkspaceDir, WorkspaceDirectoryInfo workspaceSubDir, StreamTreeRef streamSubTreeRef, AsyncThreadPoolWorkQueue queue)
		{
			WorkspaceDirectoryInfo newWorkspaceSubDir = new(newWorkspaceDir, name, streamSubTreeRef);
			newWorkspaceDir.NameToSubDirectory.Add(newWorkspaceSubDir.Name, newWorkspaceSubDir);
			await queue.EnqueueAsync(_ => MergeAsync(workspaceSubDir, newWorkspaceSubDir, streamSubTreeRef, queue));
		}

		async Task RemoveDirectoryAsync(WorkspaceDirectoryInfo workspaceDir, AsyncThreadPoolWorkQueue queue)
		{
			_directoriesToDelete.Enqueue(workspaceDir);

			foreach (WorkspaceDirectoryInfo workspaceSubDir in workspaceDir.NameToSubDirectory.Values)
			{
				await queue.EnqueueAsync(_ => RemoveDirectoryAsync(workspaceSubDir, queue));
			}
			foreach (WorkspaceFileInfo workspaceFile in workspaceDir.NameToFile.Values)
			{
				RemoveFile(workspaceFile);
			}
		}

		void RemoveFile(WorkspaceFileInfo workspaceFile)
		{
			if (_contentIdToTrackedFile.ContainsKey(workspaceFile.ContentId) || !_filesToMove.TryAdd(workspaceFile.ContentId, workspaceFile))
			{
				_filesToDelete.Enqueue(workspaceFile);
			}
		}
	}
}
