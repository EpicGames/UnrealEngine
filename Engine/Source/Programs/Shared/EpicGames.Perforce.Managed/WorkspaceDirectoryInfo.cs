// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Stores the state of a directory in the workspace
	/// </summary>
	class WorkspaceDirectoryInfo
	{
		/// <summary>
		/// The parent directory
		/// </summary>
		public WorkspaceDirectoryInfo? ParentDirectory { get; }

		/// <summary>
		/// Name of this directory
		/// </summary>
		public Utf8String Name { get; }

		/// <summary>
		/// Digest of the matching stream directory info with the base path. This should be set to zero if the workspace is modified.
		/// </summary>
		public IoHash StreamDirectoryDigest { get; set; }

		/// <summary>
		/// Map of name to file
		/// </summary>
		public Dictionary<Utf8String, WorkspaceFileInfo> NameToFile { get; set; }

		/// <summary>
		/// Map of name to subdirectory
		/// </summary>
		public Dictionary<Utf8String, WorkspaceDirectoryInfo> NameToSubDirectory { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="rootDir"></param>
		public WorkspaceDirectoryInfo(DirectoryReference rootDir)
			: this(null, new Utf8String(rootDir.FullName), null)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parentDirectory">The parent directory</param>
		/// <param name="name">Name of this directory</param>
		/// <param name="treeRef">The corresponding stream digest</param>
		public WorkspaceDirectoryInfo(WorkspaceDirectoryInfo? parentDirectory, Utf8String name, StreamTreeRef? treeRef)
		{
			ParentDirectory = parentDirectory;
			Name = name;
			StreamDirectoryDigest = (treeRef == null) ? IoHash.Zero : treeRef.ComputeHash();
			NameToFile = new Dictionary<Utf8String, WorkspaceFileInfo>(Utf8StringComparer.Ordinal);
			NameToSubDirectory = new Dictionary<Utf8String, WorkspaceDirectoryInfo>(FileUtils.PlatformPathComparerUtf8);
		}

		/// <summary>
		/// Adds a file to the workspace
		/// </summary>
		/// <param name="path">Relative path to the file, using forward slashes, and without a leading slash</param>
		/// <param name="length">Length of the file on disk</param>
		/// <param name="lastModifiedTicks">Last modified time of the file</param>
		/// <param name="readOnly">Whether the file is read only</param>
		/// <param name="contentId">Unique identifier for the server content</param>
		public void AddFile(Utf8String path, long length, long lastModifiedTicks, bool readOnly, FileContentId contentId)
		{
			StreamDirectoryDigest = IoHash.Zero;

			int idx = path.Span.IndexOf((byte)'/');
			if (idx == -1)
			{
				NameToFile[path] = new WorkspaceFileInfo(this, path, length, lastModifiedTicks, readOnly, contentId);
			}
			else
			{
				Utf8String name = path.Slice(0, idx);

				WorkspaceDirectoryInfo? subDirectory;
				if (!NameToSubDirectory.TryGetValue(name, out subDirectory))
				{
					subDirectory = new WorkspaceDirectoryInfo(this, name, null);
					NameToSubDirectory[name] = subDirectory;
				}

				subDirectory.AddFile(path.Slice(idx + 1), length, lastModifiedTicks, readOnly, contentId);
			}
		}

		/// <summary>
		/// Create a flat list of files in this workspace
		/// </summary>
		/// <returns>List of files</returns>
		public List<WorkspaceFileInfo> GetFiles()
		{
			List<WorkspaceFileInfo> files = [];
			GetFilesInternal(files);
			return files;
		}

		/// <summary>
		/// Internal helper method for recursing through the tree to build a file list
		/// </summary>
		/// <param name="files"></param>
		private void GetFilesInternal(List<WorkspaceFileInfo> files)
		{
			files.AddRange(NameToFile.Values);

			foreach (KeyValuePair<Utf8String, WorkspaceDirectoryInfo> pair in NameToSubDirectory)
			{
				pair.Value.GetFilesInternal(files);
			}
		}

		/// <summary>
		/// Refresh the state of the workspace on disk
		/// </summary>
		/// <param name="removeUntracked">Whether to remove files that are not part of the stream</param>
		/// <param name="numWorkers">Number of concurrent workers to use when refreshing</param>
		public async Task<(FileInfo[] filesToDelete, DirectoryInfo[] directoriesToDelete)> RefreshAsync(bool removeUntracked, int numWorkers)
		{
			ConcurrentQueue<FileInfo> concurrentFilesToDelete = new();
			ConcurrentQueue<DirectoryInfo> concurrentDirectoriesToDelete = new();
			using AsyncThreadPoolWorkQueue queue = new(numWorkers);
			await queue.EnqueueAsync(_ => RefreshAsync(new DirectoryInfo(GetFullName()), removeUntracked, concurrentFilesToDelete, concurrentDirectoriesToDelete, queue));
			await queue.ExecuteAsync();
			return (concurrentFilesToDelete.ToArray(), concurrentDirectoriesToDelete.ToArray());
		}

		/// <summary>
		/// Recursive method for querying the workspace state
		/// </summary>
		/// <param name="info"></param>
		/// <param name="removeUntracked"></param>
		/// <param name="filesToDelete"></param>
		/// <param name="directoriesToDelete"></param>
		/// <param name="queue"></param>
		async Task RefreshAsync(DirectoryInfo info, bool removeUntracked, ConcurrentQueue<FileInfo> filesToDelete, ConcurrentQueue<DirectoryInfo> directoriesToDelete, AsyncThreadPoolWorkQueue queue)
		{
			// Recurse through subdirectories
			Dictionary<Utf8String, WorkspaceDirectoryInfo> newNameToSubDirectory = new(NameToSubDirectory.Count, NameToSubDirectory.Comparer);

			// Figure out which files have changed.
			Dictionary<Utf8String, WorkspaceFileInfo> newNameToFile = new(NameToFile.Count, NameToFile.Comparer);

			foreach (FileSystemInfo fileSystemInfo in info.EnumerateFileSystemInfos())
			{
				if (fileSystemInfo is DirectoryInfo subDirectoryInfo)
				{
					WorkspaceDirectoryInfo? subDirectory;
					if (NameToSubDirectory.TryGetValue(new Utf8String(subDirectoryInfo.Name), out subDirectory))
					{
						newNameToSubDirectory.Add(subDirectory.Name, subDirectory);
						await queue.EnqueueAsync(_ => subDirectory.RefreshAsync(subDirectoryInfo, removeUntracked, filesToDelete, directoriesToDelete, queue));
					}
					else if (removeUntracked)
					{
						directoriesToDelete.Enqueue(subDirectoryInfo);
					}
				}
				else
				{
					FileInfo file = (FileInfo)fileSystemInfo;
					WorkspaceFileInfo? stagedFile;
					if (NameToFile.TryGetValue(new Utf8String(file.Name), out stagedFile))
					{
						if (stagedFile.MatchesAttributes(file))
						{
							newNameToFile.Add(stagedFile.Name, stagedFile);
						}
						else
						{
							filesToDelete.Enqueue(file);
						}
					}
					else
					{
						if (removeUntracked)
						{
							filesToDelete.Enqueue(file);
						}
					}
				}
			}
			NameToSubDirectory = newNameToSubDirectory;

			// If the file state has changed, clear the directory hashes
			if (NameToFile.Count != newNameToFile.Count)
			{
				for (WorkspaceDirectoryInfo? directory = this; directory != null && directory.StreamDirectoryDigest != IoHash.Zero; directory = directory.ParentDirectory)
				{
					directory.StreamDirectoryDigest = IoHash.Zero;
				}
			}

			// Update the new file list
			NameToFile = newNameToFile;
		}

		/// <summary>
		/// Builds a list of differences from the working directory
		/// </summary>
		/// <returns></returns>
		public async Task<string[]> FindDifferencesAsync(int numWorkers)
		{
			ConcurrentQueue<string> paths = new();
			using AsyncThreadPoolWorkQueue queue = new(numWorkers);
			await queue.EnqueueAsync(_ => FindDifferencesAsync(new DirectoryInfo(GetFullName()), "/", paths, queue));
			await queue.ExecuteAsync();
			return [.. paths.OrderBy(x => x)];
		}

		/// <summary>
		/// Helper method for finding differences from the working directory
		/// </summary>
		/// <param name="directory"></param>
		/// <param name="path"></param>
		/// <param name="paths"></param>
		/// <param name="queue"></param>
		async Task FindDifferencesAsync(DirectoryInfo directory, string path, ConcurrentQueue<string> paths, AsyncThreadPoolWorkQueue queue)
		{
			// Recurse through subdirectories
			HashSet<Utf8String> remainingSubDirectoryNames = [.. NameToSubDirectory.Keys];
			foreach (DirectoryInfo subDirectory in directory.EnumerateDirectories())
			{
				WorkspaceDirectoryInfo? stagedSubDirectory;
				if (NameToSubDirectory.TryGetValue(new Utf8String(subDirectory.Name), out stagedSubDirectory))
				{
					remainingSubDirectoryNames.Remove(new Utf8String(subDirectory.Name));
					await queue.EnqueueAsync(_ => stagedSubDirectory.FindDifferencesAsync(subDirectory, String.Format("{0}{1}/", path, subDirectory.Name), paths, queue));
					continue;
				}
				paths.Enqueue(String.Format("+{0}{1}/...", path, subDirectory.Name));
			}
			foreach (Utf8String remainingSubDirectoryName in remainingSubDirectoryNames)
			{
				paths.Enqueue(String.Format("-{0}{1}/...", path, remainingSubDirectoryName));
			}

			// Search through files
			HashSet<Utf8String> remainingFileNames = [.. NameToFile.Keys];
			foreach (FileInfo file in directory.EnumerateFiles())
			{
				WorkspaceFileInfo? stagedFile;
				if (!NameToFile.TryGetValue(new Utf8String(file.Name), out stagedFile))
				{
					paths.Enqueue(String.Format("+{0}{1}", path, file.Name));
				}
				else if (!stagedFile.MatchesAttributes(file))
				{
					paths.Enqueue(String.Format("!{0}{1}", path, file.Name));
					remainingFileNames.Remove(new Utf8String(file.Name));
				}
				else
				{
					remainingFileNames.Remove(new Utf8String(file.Name));
				}
			}
			foreach (Utf8String remainingFileName in remainingFileNames)
			{
				paths.Enqueue(String.Format("-{0}{1}", path, remainingFileName));
			}
		}

		/// <summary>
		/// Get the full path to this directory
		/// </summary>
		/// <returns></returns>
		public string GetFullName()
		{
			StringBuilder builder = new();
			AppendFullPath(builder);
			return builder.ToString();
		}

		/// <summary>
		/// Get the path to this directory
		/// </summary>
		/// <returns></returns>
		public DirectoryReference GetLocation()
		{
			return new DirectoryReference(GetFullName());
		}

		/// <summary>
		/// Append the client path, using native directory separators, to the given string builder
		/// </summary>
		/// <param name="builder"></param>
		public void AppendClientPath(StringBuilder builder)
		{
			if (ParentDirectory != null)
			{
				ParentDirectory.AppendClientPath(builder);
				builder.Append(Name);
				builder.Append(Path.DirectorySeparatorChar);
			}
		}

		/// <summary>
		/// Append the path for this directory to the given string builder
		/// </summary>
		/// <param name="builder"></param>
		public void AppendFullPath(StringBuilder builder)
		{
			if (ParentDirectory != null)
			{
				ParentDirectory.AppendFullPath(builder);
				builder.Append(Path.DirectorySeparatorChar);
			}
			builder.Append(Name);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return GetFullName();
		}
	}

	/// <summary>
	/// Extension methods for WorkspaceDirectoryInfo
	/// </summary>
	static class WorkspaceDirectoryInfoExtensions
	{
		public static void ReadWorkspaceDirectoryInfo(this MemoryReader reader, WorkspaceDirectoryInfo directoryInfo, ManagedWorkspaceVersion version)
		{
			if (version < ManagedWorkspaceVersion.AddDigest)
			{
				directoryInfo.StreamDirectoryDigest = IoHash.Zero;
			}
			else if (version < ManagedWorkspaceVersion.AddDigestIoHash)
			{
				reader.ReadFixedLengthBytes(Sha1.Length);
				directoryInfo.StreamDirectoryDigest = IoHash.Zero;
			}
			else
			{
				directoryInfo.StreamDirectoryDigest = reader.ReadIoHash();
			}

			int numFiles = reader.ReadInt32();
			for (int idx = 0; idx < numFiles; idx++)
			{
				WorkspaceFileInfo fileInfo = reader.ReadWorkspaceFileInfo(directoryInfo);
				directoryInfo.NameToFile.Add(fileInfo.Name, fileInfo);
			}

			int numSubDirectories = reader.ReadInt32();
			for (int idx = 0; idx < numSubDirectories; idx++)
			{
				Utf8String name = reader.ReadNullTerminatedUtf8String();

				WorkspaceDirectoryInfo subDirectory = new(directoryInfo, name, null);
				reader.ReadWorkspaceDirectoryInfo(subDirectory, version);
				directoryInfo.NameToSubDirectory[subDirectory.Name] = subDirectory;
			}
		}

		public static void WriteWorkspaceDirectoryInfo(this MemoryWriter writer, WorkspaceDirectoryInfo directoryInfo)
		{
			writer.WriteIoHash(directoryInfo.StreamDirectoryDigest);

			writer.WriteInt32(directoryInfo.NameToFile.Count);
			foreach (WorkspaceFileInfo file in directoryInfo.NameToFile.Values)
			{
				writer.WriteWorkspaceFileInfo(file);
			}

			writer.WriteInt32(directoryInfo.NameToSubDirectory.Count);
			foreach (WorkspaceDirectoryInfo subDirectory in directoryInfo.NameToSubDirectory.Values)
			{
				writer.WriteNullTerminatedUtf8String(subDirectory.Name);
				writer.WriteWorkspaceDirectoryInfo(subDirectory);
			}
		}

		public static int GetSerializedSize(this WorkspaceDirectoryInfo directoryInfo)
		{
			return Digest<Sha1>.Length + sizeof(int) + directoryInfo.NameToFile.Values.Sum(x => x.GetSerializedSize()) + sizeof(int) + directoryInfo.NameToSubDirectory.Values.Sum(x => x.Name.GetNullTerminatedSize() + x.GetSerializedSize());
		}
	}
}
