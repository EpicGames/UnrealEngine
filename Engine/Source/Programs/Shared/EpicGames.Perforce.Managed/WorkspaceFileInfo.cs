// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Text;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Stores information about a file that has been staged into a workspace
	/// </summary>
	class WorkspaceFileInfo
	{
		public readonly WorkspaceDirectoryInfo Directory;
		public readonly Utf8String Name;
		public long _length;
		public long _lastModifiedTicks;
		public bool _readOnly;
		public readonly FileContentId ContentId;

		public WorkspaceFileInfo(WorkspaceDirectoryInfo directory, Utf8String name, FileContentId contentId)
		{
			Directory = directory;
			Name = name;
			ContentId = contentId;
		}

		public WorkspaceFileInfo(WorkspaceDirectoryInfo directory, Utf8String name, FileInfo info, FileContentId contentId)
			: this(directory, name, info.Length, info.LastWriteTimeUtc.Ticks, info.Attributes.HasFlag(FileAttributes.ReadOnly), contentId)
		{
		}

		public WorkspaceFileInfo(WorkspaceDirectoryInfo directory, Utf8String name, long length, long lastModifiedTicks, bool readOnly, FileContentId contentId)
		{
			Directory = directory;
			Name = name;
			_length = length;
			_lastModifiedTicks = lastModifiedTicks;
			_readOnly = readOnly;
			ContentId = contentId;
		}

		public void SetMetadata(long length, long lastModifiedTicks, bool readOnly)
		{
			_length = length;
			_lastModifiedTicks = lastModifiedTicks;
			_readOnly = readOnly;
		}

		public void UpdateMetadata()
		{
			FileInfo info = new(GetFullName());
			if (info.Exists)
			{
				_length = info.Length;
				_lastModifiedTicks = info.LastWriteTimeUtc.Ticks;
				_readOnly = info.Attributes.HasFlag(FileAttributes.ReadOnly);
			}
		}

		public bool MatchesAttributes(FileInfo info)
		{
			return _length == info.Length && _lastModifiedTicks == info.LastWriteTimeUtc.Ticks && (info.Attributes.HasFlag(FileAttributes.ReadOnly) == _readOnly);
		}

		public string GetClientPath()
		{
			StringBuilder builder = new();
			Directory.AppendClientPath(builder);
			builder.Append(Name);
			return builder.ToString();
		}

		public string GetFullName()
		{
			StringBuilder builder = new();
			Directory.AppendFullPath(builder);
			builder.Append(Path.DirectorySeparatorChar);
			builder.Append(Name);
			return builder.ToString();
		}

		public FileReference GetLocation()
		{
			return new FileReference(GetFullName());
		}

		public override string ToString()
		{
			return GetFullName();
		}
	}

	static class WorkspaceFileInfoExtensions
	{
		public static WorkspaceFileInfo ReadWorkspaceFileInfo(this MemoryReader reader, WorkspaceDirectoryInfo directory)
		{
			Utf8String name = reader.ReadNullTerminatedUtf8String();
			long length = reader.ReadInt64();
			long lastModifiedTicks = reader.ReadInt64();
			bool readOnly = reader.ReadBoolean();
			FileContentId contentId = reader.ReadFileContentId();
			return new WorkspaceFileInfo(directory, name, length, lastModifiedTicks, readOnly, contentId);
		}

		public static void WriteWorkspaceFileInfo(this MemoryWriter writer, WorkspaceFileInfo fileInfo)
		{
			writer.WriteNullTerminatedUtf8String(fileInfo.Name);
			writer.WriteInt64(fileInfo._length);
			writer.WriteInt64(fileInfo._lastModifiedTicks);
			writer.WriteBoolean(fileInfo._readOnly);
			writer.WriteFileContentId(fileInfo.ContentId);
		}

		public static int GetSerializedSize(this WorkspaceFileInfo fileInfo)
		{
			return fileInfo.Name.GetNullTerminatedSize() + sizeof(long) + sizeof(long) + sizeof(byte) + fileInfo.ContentId.GetSerializedSize();
		}
	}
}
