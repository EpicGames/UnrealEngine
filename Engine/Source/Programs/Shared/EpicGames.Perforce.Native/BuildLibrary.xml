<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../Engine/Build/Graph/Schema.xsd" >

	<Agent Name="DotNetPerforceLib Win64" Type="Win64">
		<Node Name="Compile DotNetPerforceLib Win64" Produces="#BinariesWin64">
			<SetVersion Change="$(Change)" Branch="$(EscapedBranch)" If="$(IsBuildMachine)"/>
			<Delete Files="$(RootDir)/Engine/Binaries/DotNET/EpicGames.Perforce.Native/win-x64/..."/>
			<Compile Target="DotNetPerforceLib" Configuration="Debug" Platform="Win64" Tag="#FilesToSign"/>
			<Compile Target="DotNetPerforceLib" Configuration="Shipping" Platform="Win64" Tag="#FilesToSign"/>
			<Sign Files="#FilesToSign" Tag="#BinariesWin64"/>
			<Log Message="Files:" Files="#BinariesWin64"/>
		</Node>
		<Artifact Name="Binaries (Win64)" Type="p4-binaries" Tag="#BinariesWin64"/>
	</Agent>

	<Agent Name="DotNetPerforceLib Mac" Type="Mac">
		<Node Name="Compile DotNetPerforceLib Mac" Produces="#BinariesMac">
			<SetVersion Change="$(Change)" Branch="$(EscapedBranch)" If="$(IsBuildMachine)"/>
			<Delete Files="$(RootDir)/Engine/Binaries/DotNET/EpicGames.Perforce.Native/mac-x64/..."/>
			<Compile Target="DotNetPerforceLib" Configuration="Debug" Platform="Mac" Tag="#BinariesMac"/>
			<Compile Target="DotNetPerforceLib" Configuration="Shipping" Platform="Mac" Tag="#BinariesMac"/>
			<Log Message="Files:" Files="#BinariesMac"/>
		</Node>
		<Artifact Name="Binaries (Mac)" Type="p4-binaries" Tag="#BinariesMac"/>
	</Agent>

	<Agent Name="DotNetPerforceLib Linux" Type="Linux">
		<Node Name="Compile DotNetPerforceLib Linux" Produces="#BinariesLinux">
			<SetVersion Change="$(Change)" Branch="$(EscapedBranch)" If="$(IsBuildMachine)"/>
			<Delete Files="$(RootDir)/Engine/Binaries/DotNET/EpicGames.Perforce.Native/linux-x64/..."/>
			<Compile Target="DotNetPerforceLib" Configuration="Debug" Platform="Linux" Tag="#BinariesLinux"/>
			<Compile Target="DotNetPerforceLib" Configuration="Shipping" Platform="Linux" Tag="#BinariesLinux"/>
			<Log Message="Files:" Files="#BinariesLinux"/>
		</Node>
		<Artifact Name="Binaries (Linux)" Type="p4-binaries" Tag="#BinariesLinux"/>
	</Agent>

	<Aggregate Name="DotNetPerforceLib" Requires="#BinariesWin64;#BinariesMac;#BinariesLinux"/>

</BuildGraph>
