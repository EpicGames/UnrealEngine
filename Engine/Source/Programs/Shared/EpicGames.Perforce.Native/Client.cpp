// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable: 4244)
#pragma warning(disable: 4458)
#include "p4/clientapi.h"
#include "p4/strtable.h"
#include "p4/stdhdrs.h"
#include "p4/datetime.h"
#include "p4/i18napi.h"
#include "p4/charset.h"
#include "p4/md5.h"
#include "p4/p4libs.h"
#include "p4/signaler.h"
#include "ThirdParty/gzip.h"
#pragma warning(pop)
#include <signal.h>
#include <assert.h>
#include <atomic>
THIRD_PARTY_INCLUDES_END

#ifdef _MSC_VER
	#define NATIVE_API __declspec(dllexport)
	#define strcasecmp _stricmp
#else
	#define NATIVE_API __attribute__((visibility("default")))
#endif

#pragma warning(disable:4100)
#pragma warning(disable:4458)

struct FSettings
{
	const char* ServerAndPort;
	const char* User;
	const char* Password;
	const char* Host;
	const char* Client;
	const char* AppName;
	const char* AppVersion;
};

struct FReadBuffer
{
	unsigned char* Data;
	unsigned int Length;
	unsigned int Count;
	unsigned int MaxLength;
	unsigned int MaxCount;
};

struct FWriteBuffer
{
	unsigned char* Data;
	unsigned int MaxLength;
	unsigned int MaxCount;
};

typedef void (FOnBufferReadyFn)(const FReadBuffer* ReadBuffer, FWriteBuffer* WriteBuffer);

class FClientUser : public ClientUser
{
private:
	unsigned char* Data;
	unsigned int Count;
	unsigned int Length;
	unsigned int MaxCount;
	unsigned int MaxLength;
	FOnBufferReadyFn* OnBufferReady;

	const char* InputBuffer = nullptr;
	int InputLength = 0;

public:
	const char* Func = nullptr;
	const char* PromptResponse = nullptr;
	bool InterceptIo = false;

	FClientUser(FWriteBuffer* WriteBuffer, FOnBufferReadyFn* OnBufferReady)
	{
		SetWriteBuffer(WriteBuffer);
		this->OnBufferReady = OnBufferReady;
	}

	virtual FileSys* File(FileSysType type) override;

	virtual void InputData(StrBuf* strbuf, Error* e) override
	{
		if (InputBuffer != nullptr)
		{
			strbuf->Set(InputBuffer, InputLength);
		}
		else
		{
			ClientUser::InputData(strbuf, e);
		}
	}

	virtual void Prompt(Error* err, StrBuf& rsp, int noEcho, Error* e) override
	{
		rsp.Set(PromptResponse);
	}

	virtual void Prompt(Error* err, StrBuf& rsp, int noEcho, int noOutput, Error* e) override
	{
		rsp.Set(PromptResponse);
	}

	virtual void Prompt(const StrPtr& msg, StrBuf& rsp, int noEcho, Error* e) override
	{
		rsp.Set(PromptResponse);
	}

	virtual void Prompt(const StrPtr& msg, StrBuf& rsp, int noEcho, int noOutput, Error* e) override
	{
		rsp.Set(PromptResponse);
	}

	void SetInputBuffer(const char* InInputBuffer, int InInputLength)
	{
		InputBuffer = InInputBuffer;
		InputLength = InInputLength;
	}

	void SetWriteBuffer(FWriteBuffer* WriteBuffer)
	{
		this->Data = WriteBuffer->Data;
		this->Length = 0;
		this->Count = 0;
		this->MaxLength = WriteBuffer->MaxLength;
		this->MaxCount = (size_t)WriteBuffer->MaxCount;
	}

	void Flush()
	{
		// Writing an empty buffer indicates to the owning process that we need more data, so avoid that during a normal flush.
		if (Length > 0)
		{
			FlushOrExpandBuffer();
		}
	}

	void FlushOrExpandBuffer()
	{
		FReadBuffer ReadBuffer;
		ReadBuffer.Data = Data;
		ReadBuffer.Length = Length;
		ReadBuffer.Count = Count;
		ReadBuffer.MaxLength = MaxLength;
		ReadBuffer.MaxCount = MaxCount;

		FWriteBuffer WriteBuffer;
		memset(&WriteBuffer, 0, sizeof(WriteBuffer));

		(*OnBufferReady)(&ReadBuffer, &WriteBuffer);

		SetWriteBuffer(&WriteBuffer);
	}

	virtual void HandleError(Error* err) override
	{
		while (!TryOutputError(err))
		{
			FlushOrExpandBuffer();
		}
	}

	virtual void Message(Error* Err) override
	{
		while (!TryOutputError(Err))
		{
			FlushOrExpandBuffer();
		}
	}

	bool TryOutputError(Error* Err)
	{
		static const char CodeKey[] = "code";
		static const char SeverityKey[] = "severity";
		static const char GenericKey[] = "generic";
		static const char DataKey[] = "data";

		StrBuf Message;
		Err->Fmt(&Message, 0);
		unsigned int MessageLen = Message.Length();

		unsigned int RecordLen;
		if (Err->GetSeverity() == E_INFO)
		{
			static const char InfoCode[] = "info";

			RecordLen = 1 + MeasureStringField(CodeKey, InfoCode) + MeasureStringField(DataKey, MessageLen) + 1;
			if (Length + RecordLen > MaxLength)
			{
				return false;
			}

			unsigned char* Pos = Data + Length;
			*(Pos++) = '{';
			Pos = WriteStringField(Pos, CodeKey, InfoCode);
			Pos = WriteStringField(Pos, DataKey, Message.Text(), MessageLen);
			*(Pos++) = '0';
			assert(Pos == Data + Length + RecordLen);
		}
		else
		{
			static const char ErrorCode[] = "error";

			RecordLen = 1 + MeasureStringField(CodeKey, ErrorCode) + MeasureIntField(SeverityKey) + MeasureIntField(GenericKey) + MeasureStringField(DataKey, MessageLen) + 1;
			if (Length + RecordLen > MaxLength)
			{
				return false;
			}

			unsigned char* Pos = Data + Length;
			*(Pos++) = '{';
			Pos = WriteStringField(Pos, CodeKey, ErrorCode);
			Pos = WriteIntField(Pos, SeverityKey, Err->GetSeverity());
			Pos = WriteIntField(Pos, GenericKey, Err->GetGeneric());
			Pos = WriteStringField(Pos, DataKey, Message.Text(), MessageLen);
			*(Pos++) = '0';
			assert(Pos == Data + Length + RecordLen);
		}

		Length += RecordLen;
		return true;
	}

	virtual void OutputError(const char* errBuf) override
	{
		assert(false);
	}

	void OutputIo(int FileId, const char* Command, const void* Payload, int PayloadLen)
	{
		while (!TryOutputIo(FileId, Command, Payload, PayloadLen))
		{
			FlushOrExpandBuffer();
		}
	}

	bool TryOutputIo(int FileId, const char* Command, const void* Payload, int PayloadLen)
	{
		static const char CodeKey[] = "code";
		static const char FileKey[] = "file";
		static const char CommandKey[] = "command";
		static const char PayloadKey[] = "payload";

		static const char IoCode[] = "io";

		unsigned int CommandLen = (unsigned int)strlen(Command);
		unsigned int RecordLen = (unsigned int)(1 + MeasureStringField(CodeKey, IoCode) + MeasureIntField(FileKey) + MeasureStringField(CommandKey, CommandLen) + MeasureStringField(PayloadKey, PayloadLen) + 1);
		if (Length + RecordLen > MaxLength)
		{
			return false;
		}

		unsigned char* Pos = Data + Length;
		*(Pos++) = '{';
		Pos = WriteStringField(Pos, CodeKey, IoCode);
		Pos = WriteIntField(Pos, FileKey, FileId);
		Pos = WriteStringField(Pos, CommandKey, Command, CommandLen);
		Pos = WriteStringField(Pos, PayloadKey, (const char*)Payload, PayloadLen);
		*(Pos++) = '0';
		assert(Pos == Data + Length + RecordLen);

		Length += RecordLen;
		return true;
	}

	virtual void OutputInfo(char InLevel, const char* InData) override
	{
		while (!TryOutputInfo(InLevel, InData))
		{
			FlushOrExpandBuffer();
		}
	}

	bool TryOutputInfo(char InLevel, const char* InInfo)
	{
		static const char CodeKey[] = "code";
		static const char LevelKey[] = "level";
		static const char DataKey[] = "data";

		static const char InfoCode[] = "info";

		unsigned int InfoLen = (unsigned int)strlen(InInfo);
		unsigned int RecordLen = 1 + MeasureStringField(CodeKey, InfoCode) + MeasureIntField(LevelKey) + MeasureStringField(DataKey, InfoLen) + 1;
		if (Length + RecordLen > MaxLength)
		{
			return false;
		}

		unsigned char* Pos = Data + Length;
		*(Pos++) = '{';
		Pos = WriteStringField(Pos, CodeKey, InfoCode);
		Pos = WriteIntField(Pos, LevelKey, InLevel);
		Pos = WriteStringField(Pos, DataKey, InInfo, InfoLen);
		*(Pos++) = '0';
		assert(Pos == Data + Length + RecordLen);

		Length += RecordLen;
		return true;
	}

	virtual void OutputBinary(const char* InData, int InLength) override
	{
		assert(false);
	}

	virtual void OutputText(const char* InData, int InLength) override
	{
		assert(false);
	}

	virtual void OutputStat(StrDict* VarList) override
	{
		while (!TryWriteRecord(VarList))
		{
			FlushOrExpandBuffer();
		}
		if (++Count > MaxCount)
		{
			Flush();
		}
	}

	template<int KeyLen>
	static int MeasureStringField(const char(&Key)[KeyLen], int ValueLen)
	{
		return 10 + (KeyLen - 1) + ValueLen;
	}

	template<int KeyLen, int ValueLen>
	static int MeasureStringField(const char(&Key)[KeyLen], const char(&Value)[ValueLen])
	{
		return MeasureStringField(Key, ValueLen - 1);
	}

	template<int KeyLen>
	static int MeasureIntField(const char(&Key)[KeyLen])
	{
		return 10 + (KeyLen - 1);
	}

	template<int KeyLen>
	static unsigned char* WriteStringField(unsigned char* Data, const char(&Key)[KeyLen], const char* Value, int ValueLen)
	{
		Data = WriteString(Data, Key);
		return WriteString(Data, Value, ValueLen);
	}

	template<int KeyLen, int ValueLen>
	static unsigned char* WriteStringField(unsigned char* Data, const char(&Key)[KeyLen], const char(&Value)[ValueLen])
	{
		Data = WriteString(Data, Key);
		return WriteString(Data, Value);
	}

	template<int KeyLen>
	static unsigned char* WriteIntField(unsigned char* Data, const char(&Key)[KeyLen], int Value)
	{
		Data = WriteString(Data, Key);
		return WriteInt(Data, Value);
	}

	static unsigned char* WriteInt(unsigned char* Data, int Value)
	{
		*(Data++) = 'i';
		return WriteIntValue(Data, Value);
	}

	static unsigned char* WriteIntValue(unsigned char* Data, int Value)
	{
		*(Data++) = Value & 0xff;
		*(Data++) = (Value >> 8) & 0xff;
		*(Data++) = (Value >> 16) & 0xff;
		*(Data++) = (Value >> 24) & 0xff;
		return Data;
	}

	static unsigned char* WriteString(unsigned char* Data, const char* Text, int Length)
	{
		*(Data++) = 's';
		Data = WriteIntValue(Data, Length);
		memcpy(Data, Text, Length);
		return Data + Length;
	}

	template<int Length>
	static unsigned char* WriteString(unsigned char* Data, const char(&Text)[Length])
	{
		return WriteString(Data, Text, Length - 1);
	}

	bool TryWriteRecord(StrDict* VarList)
	{
		return TryWriteRecord("stat", 4, VarList);
	}

	bool TryWriteRecord(const char* Code, unsigned int CodeLen, StrDict* VarList)
	{
		unsigned int Pos = Length;

		// Record prefix: '{' (open record), string, 4 bytes, 'code', string ...
		unsigned char Prefix[] = { '{', 's', 4, 0, 0, 0, 'c', 'o', 'd', 'e', 's' };
		unsigned int PrefixLen = sizeof(Prefix) / sizeof(Prefix[0]);

		if (Pos + PrefixLen + (unsigned int)sizeof(unsigned int) + CodeLen > MaxLength - 1)
		{
			return false;
		}

		memcpy(Data + Pos, Prefix, PrefixLen);
		Pos += PrefixLen;

		memcpy(Data + Pos, &CodeLen, sizeof(CodeLen));
		Pos += sizeof(unsigned int);

		memcpy(Data + Pos, Code, CodeLen);
		Pos += CodeLen;

		for (int Idx = 0;; Idx++)
		{
			StrRef Field;
			StrRef Value;
			if (VarList->GetVar(Idx, Field, Value) == 0)
			{
				break;
			}

			unsigned int FieldLength = Field.Length();
			unsigned int ValueLength = Value.Length();

			if (Pos + 1 + 4 + FieldLength + 1 + 4 + ValueLength > MaxLength - 1)
			{
				return false;
			}

			Data[Pos] = 's';
			Pos++;

			memcpy(Data + Pos, &FieldLength, sizeof(FieldLength));
			Pos += sizeof(FieldLength);

			memcpy(Data + Pos, Field.Text(), FieldLength);
			Pos += FieldLength;

			Data[Pos] = 's';
			Pos++;

			memcpy(Data + Pos, &ValueLength, sizeof(ValueLength));
			Pos += sizeof(ValueLength);

			memcpy(Data + Pos, Value.Text(), ValueLength);
			Pos += ValueLength;
		}

		Data[Pos++] = '0';

		Length = Pos;
		return true;
	}
};

class FFileSys : public FileSys
{
public:
	static int NextFileId;
	int FileId = -1;

	const FileSysType Type;
	class FClientUser& User;

	FFileSys(FileSysType InType, FClientUser& InUser)
		: Type(InType)
		, User(InUser)
	{
		this->type = InType;
	}

	virtual void	ChmodTimeHP(const DateTimeHighPrecision& /* modTime */, Error* /* e */) override {};
	virtual void	SetAttribute(FileSysAttr, Error*) override { };

	virtual bool	HasOnlyPerm(FilePerm perms) override { return false; }
	virtual FD_PTR	GetFd() override { return FD_ERR; }
	virtual int     GetOwner() override { return 0; }
	virtual offL_t	GetSize() override { return 0; }
	virtual void	Seek(offL_t offset, Error*) override { assert(false); }
	virtual offL_t	Tell() override { assert(false); return 0; }

	virtual int StatAccessTime() override { assert(false); return 0; }

	virtual void	MakeLocalTemp(char* file) override { assert(false); }
	virtual void	SetDeleteOnClose() override { }
	virtual void	ClearDeleteOnClose() override {}
	virtual StrArray* ScanDir(Error* e) override { return nullptr; }

	virtual void	MkDir(const StrPtr& p, Error* e) override { }
	virtual void	PurgeDir(const char* p, Error* e) override { }
	virtual void	RmDir(const StrPtr& p, Error* e) override { }
	virtual int	ReadLine(StrBuf* buf, Error* e) override { return 0; }

	virtual void Open(FileOpenMode mode, Error* e) override
	{
		this->mode = mode;

		int PathLen = path.Length() + 1;

		int BufferLen = PathLen + (sizeof(int) * 3);

		char* Buffer = new char[BufferLen];

		char* BufferEnd = Buffer;
		memcpy(BufferEnd, path.Text(), (size_t)PathLen);
		BufferEnd += PathLen;

		int TypeInt = (int)Type;
		memcpy(BufferEnd, &TypeInt, sizeof(int));
		BufferEnd += sizeof(int);

		int ModeInt = (int)mode;
		memcpy(BufferEnd, &ModeInt, sizeof(int));
		BufferEnd += sizeof(int);

		int PermsInt = (int)perms;
		memcpy(BufferEnd, &PermsInt, sizeof(int));
		BufferEnd += sizeof(int);

		FileId = ++NextFileId;
		User.OutputIo(FileId, "open", Buffer, BufferLen);

		delete[] Buffer;
	}

	virtual void Write(const char* buf, int len, Error* e) override
	{
		User.OutputIo(FileId, "write", buf, len);

		if (checksum && len > 0)
			checksum->Update(StrRef(buf, len));
	}

	virtual int Read(char* buf, int len, Error* e) override
	{
		return 0;
	}

	virtual void Close(Error* e) override
	{
		if (FileId != -1)
		{
			User.OutputIo(FileId, "close", nullptr, 0);
			FileId = -1;
		}
	}

	virtual int Stat() override
	{
		return 0;
	}

	virtual int StatModTime() override
	{
		return 0;
	}

	virtual void StatModTimeHP(DateTimeHighPrecision* modTime) override
	{
	}

	virtual void Truncate(Error* e) override
	{
		assert(false);
	}

	virtual void Truncate(offL_t offset, Error* e) override
	{
		assert(false);
	}

	virtual void Unlink(Error* e = 0) override
	{
		User.OutputIo(FileId, "unlink", path.Text(), path.Length());
	}

	virtual void Rename(FileSys* target, Error* e) override
	{
		assert(false);
	}

	virtual void Chmod(FilePerm perms, Error* e) override
	{
		assert(false);
	}

	virtual void ChmodTime(Error* e) override
	{
		assert(false);
	}
};

class FUnzipFileSys : public FFileSys
{
public:
	Gzip* GzipInst;

	char* Buffer;
	const size_t BufferSize;

	FUnzipFileSys(FileSysType InType, FClientUser& InUser)
		: FFileSys(InType, InUser)
		, BufferSize(FFileSys::BufferSize())
	{
		GzipInst = nullptr;
		Buffer = nullptr;
	}

	~FUnzipFileSys()
	{
		Cleanup();
		delete GzipInst;
		delete[] Buffer;
	}

	virtual void Open(FileOpenMode Mode, Error* Err) override
	{
		Buffer = new char[BufferSize];

		GzipInst = new Gzip();
		GzipInst->is = Buffer;
		GzipInst->ie = Buffer;
		GzipInst->os = Buffer;
		GzipInst->oe = Buffer + BufferSize;

		FFileSys::Open(Mode, Err);
	}

	virtual void Write(const char* WriteBuf, int WriteLen, Error* e) override
	{
		GzipInst->is = WriteBuf;
		GzipInst->ie = WriteBuf + WriteLen;

		for (;;)
		{
			if (GzipInst->OutputFull())
			{
				FFileSys::Write(Buffer, (int)(GzipInst->os - Buffer), e);
				GzipInst->os = Buffer;
			}
			if (e->Test() || !GzipInst->Uncompress(e) || GzipInst->InputEmpty())
			{
				break;
			}
		}
	}

	virtual void Close(Error* e) override
	{
		if (GzipInst && mode == FOM_WRITE && GzipInst->os > Buffer)
		{
			FFileSys::Write(Buffer, (int)(GzipInst->os - Buffer), e);
			GzipInst->os = Buffer;
		}

		FFileSys::Close(e);
	}
};

int FFileSys::NextFileId = 100;

FileSys* FClientUser::File(FileSysType type)
{
	if (InterceptIo)
	{
		if ((type & FST_C_MASK) == FST_C_GUNZIP)
		{
			return new FUnzipFileSys(type, *this);
		}
		else
		{
			return new FFileSys(type, *this);
		}
	}
	else
	{
		return ClientUser::File(type);
	}
}

class FKeepAlive : public KeepAlive
{
public:
	std::atomic<bool> Cancelled{ false };

	virtual int IsAlive() override
	{
		return Cancelled.load() ? 0 : 1;
	}
};

class FClient
{
public:
	ClientApi ClientApi;
	FClientUser User;
	FKeepAlive KeepAlive;
	char* AppVersion;

	FClient(FWriteBuffer* WriteBuffer, FOnBufferReadyFn* OnBufferReady)
		: User(WriteBuffer, OnBufferReady)
	{
		AppVersion = nullptr;
	}

	~FClient()
	{
		if (AppVersion != nullptr)
		{
			free(AppVersion);
		}
	}
};

struct FClientGlobalInit
{
	FClientGlobalInit()
	{
		Error e;
		P4Libraries::Initialize(P4LIBRARIES_INIT_ALL, &e);
		signal(SIGINT, SIG_DFL); // unset the default set by global signaler in C++ so it does not exit 
		signaler.Disable(); // disable the global signaler memory tracking at runtime
	}
};

extern "C" NATIVE_API FClient* Client_Create(const FSettings* Settings, FWriteBuffer* WriteBuffer, FOnBufferReadyFn* OnBufferReady)
{
	static FClientGlobalInit GlobalInit;

	FClient* Client = new FClient(WriteBuffer, OnBufferReady);

	if (Settings != nullptr)
	{
		if (Settings->ServerAndPort != nullptr)
		{
			Client->ClientApi.SetPort(Settings->ServerAndPort);
		}
		if (Settings->User != nullptr)
		{
			Client->ClientApi.SetUser(Settings->User);
		}
		if (Settings->Password != nullptr)
		{
			Client->ClientApi.SetPassword(Settings->Password);
		}
		if (Settings->Host != nullptr)
		{
			Client->ClientApi.SetHost(Settings->Host);
		}
		if (Settings->Client != nullptr)
		{
			Client->ClientApi.SetClient(Settings->Client);
		}
		if (Settings->AppName != nullptr)
		{
			Client->ClientApi.SetProg(Settings->AppName);
		}
		if (Settings->AppVersion != nullptr)
		{
			Client->AppVersion = strdup(Settings->AppVersion);
		}
	}
	Client->ClientApi.SetProtocol("tag", "");
	Client->ClientApi.SetProtocol("unicode", "");

	Error Err;
	Client->ClientApi.Init(&Err);
	Client->ClientApi.SetBreak(&Client->KeepAlive);
	Client->User.HandleError(&Err);
	Client->User.Flush();

	GlobalCharSet::Set(CharSetApi::UTF_8);

	return Client;
}

extern "C" NATIVE_API void Client_Command(FClient* Client, const char* Func, int ArgCount, const char** Args, const char* InputData, int InputLength, const char* promptResponse, bool InterceptIo)
{
	if (Client->AppVersion != nullptr)
	{
		Client->ClientApi.SetVersion(Client->AppVersion);
	}

	Client->KeepAlive.Cancelled.store(false);
	Client->User.InterceptIo = InterceptIo;
	Client->User.Func = Func;
	Client->User.SetInputBuffer(InputData, InputLength);
	Client->User.PromptResponse = promptResponse;
	Client->ClientApi.SetArgv(ArgCount, (char* const*)Args);
	Client->ClientApi.Run(Func, &Client->User);
	Client->User.Flush();
	Client->User.PromptResponse = nullptr;
	Client->User.SetInputBuffer(nullptr, 0);
	Client->User.Func = nullptr;
	Client->User.InterceptIo = false;
}

extern "C" NATIVE_API void Client_CancelCommand(FClient* Client)
{
	Client->KeepAlive.Cancelled.store(true);
}

extern "C" NATIVE_API void Client_Destroy(FClient* Client)
{
	Error Err;
	Client->ClientApi.Final(&Err);
	Client->User.HandleError(&Err);
	delete Client;
}
