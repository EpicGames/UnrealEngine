// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class DotNetPerforceLib : ModuleRules
{
	public DotNetPerforceLib(ReadOnlyTargetRules target) : base(target)
	{
		if (target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDefinitions.Add("OS_NT");
			PrivateDefinitions.Add("_CRT_NONSTDC_NO_DEPRECATE");
		}

		PublicIncludePathModuleNames.Add("Core");

		PrivateDependencyModuleNames.Add("OpenSSL");

		string apiPath = Target.UEThirdPartySourceDirectory + "Perforce/p4api-2023.2/";
		if (target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicSystemIncludePaths.Add(apiPath + "Linux/include");
			PublicAdditionalLibraries.Add(apiPath + "Linux/lib/libclient.a");
			PublicAdditionalLibraries.Add(apiPath + "Linux/lib/librpc.a");
			PublicAdditionalLibraries.Add(apiPath + "Linux/lib/libsupp.a");
			PublicAdditionalLibraries.Add(apiPath + "Linux/lib/libp4script_cstub.a");

			PublicSystemLibraries.Add("dl");
		}
		else if (target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicIncludePaths.Add(apiPath + "Mac/include");
			PublicAdditionalLibraries.Add(apiPath + "Mac/lib/libclient.a");
			PublicAdditionalLibraries.Add(apiPath + "Mac/lib/librpc.a");
			PublicAdditionalLibraries.Add(apiPath + "Mac/lib/libsupp.a");
			PublicAdditionalLibraries.Add(apiPath + "Mac/lib/libp4script_cstub.a");

			PublicFrameworks.Add("Foundation");
			PublicFrameworks.Add("CoreFoundation");
			PublicFrameworks.Add("CoreGraphics");
			PublicFrameworks.Add("CoreServices");
			PublicFrameworks.Add("Security");
		}
		else if (target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicIncludePaths.Add(apiPath + "Win64/include");
			PublicAdditionalLibraries.Add(apiPath + "Win64/lib/libclient.lib");
			PublicAdditionalLibraries.Add(apiPath + "Win64/lib/librpc.lib");
			PublicAdditionalLibraries.Add(apiPath + "Win64/lib/libsupp.lib");
			PublicAdditionalLibraries.Add(apiPath + "Win64/lib/libp4script_cstub.lib");
		}
	}
}