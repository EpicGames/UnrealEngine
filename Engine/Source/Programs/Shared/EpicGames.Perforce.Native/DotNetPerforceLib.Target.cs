// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1050:Declare types in namespaces", Justification = "<Pending>")]
public class DotNetPerforceLibTarget : TargetRules
{
	public DotNetPerforceLibTarget(TargetInfo target) : base(target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "DotNetPerforceLib";

		bShouldCompileAsDLL = true;

		bBuildDeveloperTools = false;
		bCompileICU = false;
		bBuildWithEditorOnlyData = false;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		string configFolder = (target.Configuration == UnrealTargetConfiguration.Shipping) ? "Release" : "Debug";
		if (target.Platform == UnrealTargetPlatform.Win64)
		{
			OutputFile = $"Binaries/DotNET/EpicGames.Perforce.Native/win-x64/{configFolder}/EpicGames.Perforce.Native.dll";
		}
		else if (target.Platform == UnrealTargetPlatform.Mac)
		{
			OutputFile = $"Binaries/DotNET/EpicGames.Perforce.Native/mac-x64/{configFolder}/EpicGames.Perforce.Native.dylib";
		}
		else if (target.Platform == UnrealTargetPlatform.Linux)
		{
			OutputFile = $"Binaries/DotNET/EpicGames.Perforce.Native/linux-x64/{configFolder}/EpicGames.Perforce.Native.so";
		}
	}
}
