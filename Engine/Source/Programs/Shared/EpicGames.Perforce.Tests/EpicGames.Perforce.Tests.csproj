<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
  </ItemGroup>
</Project>
