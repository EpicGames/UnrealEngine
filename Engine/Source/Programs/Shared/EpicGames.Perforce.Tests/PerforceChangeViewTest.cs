// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Tests
{
	[TestClass]
	public class PerforceChangeViewTest
	{
		[TestMethod]
		public void TestImportAtChange()
		{
			string[] viewLines =
			[
				"//UE5/Main/Testing/...@123",
				"//UE5/Main/Testing/Foo/...@456",
				"//UE5/Main/Testing/Foo/Bar/...@100",
			];

			PerforceChangeView changeView = PerforceChangeView.Parse(viewLines, true);

			Assert.IsTrue(changeView.IsVisible("//UE5/Main/File.txt", 1));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Other/File.txt", 1));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/File.txt", 1));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/File.txt", 1));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/Bar/File.txt", 1));

			Assert.IsTrue(changeView.IsVisible("//UE5/Main/File.txt", 100));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Other/File.txt", 100));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/File.txt", 100));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/File.txt", 100));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/Bar/File.txt", 100));

			Assert.IsTrue(changeView.IsVisible("//UE5/Main/File.txt", 101));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Other/File.txt", 101));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/File.txt", 101));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/File.txt", 101));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/Foo/Bar/File.txt", 101));

			Assert.IsTrue(changeView.IsVisible("//UE5/Main/File.txt", 124));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Other/File.txt", 124));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/File.txt", 124));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Testing/Foo/File.txt", 124));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/Foo/Bar/File.txt", 124));

			Assert.IsTrue(changeView.IsVisible("//UE5/Main/File.txt", 457));
			Assert.IsTrue(changeView.IsVisible("//UE5/Main/Other/File.txt", 457));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/File.txt", 457));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/Foo/File.txt", 457));
			Assert.IsFalse(changeView.IsVisible("//UE5/Main/Testing/Foo/Bar/File.txt", 457));
		}
	}
}
