// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Tests;

[TestClass]
public class PerforceViewFilterTest
{
	[TestMethod]
	public void FileWildcard()
	{
		AssertInclude(true, "main.h", "*");
		AssertInclude(true, "/main.h", "*");
		AssertInclude(false, "/Engine/main.h", "*");
		AssertInclude(true, "/Engine/main.h", "-*");
		AssertInclude(false, "/Engine/Foo/main.h", "*");
		AssertInclude(true, "/Engine/main.h", "-*");
		AssertInclude(true, "/setup.bat", "-/...", "/*");
		AssertInclude(true, "/setup.bat", "-/...", "*");
		AssertInclude(false, "/Engine/main.cpp", "-/...", "*");
	}

	[TestMethod]
	public void DirectoryWildcard()
	{
		AssertInclude(true, "/Engine/main.h", "/Engine/...");
		AssertInclude(true, "/Engine/Foo/foo.h", "/Engine/...");
		AssertInclude(true, "/foo.h", "/...");
		AssertInclude(true, "/foo.h", "...");
		AssertInclude(true, "foo.h", "...h");
		AssertInclude(true, "/Engine/foo.h", "/...");
		AssertInclude(true, "/Engine/main.cpp", "-/...", "/Engine/...");
		AssertInclude(false, "/Engine/readme.txt", "-...", "/Docs/...");
		AssertInclude(false, "/Engine/readme.txt", "/Docs/...", "-...");
		AssertInclude(false, "/Docs/readme.txt", "-/...", "/Engine/...");
		AssertInclude(false, "/Docs/readme.txt", "-...", "/Engine/...");
		AssertInclude(false, "/Engine/main.cpp", "/...", "-/Engine/...");

		AssertInclude(false, "/Engine/main.cpp", "-...");
		AssertInclude(false, "Engine/main.cpp", "-...");
		AssertInclude(true, "/main.cpp", "-...", "*");
		AssertInclude(true, "main.cpp", "-...", "*");
		AssertInclude(true, "Engine/main.cpp", "-...", "Engine/...");
		AssertInclude(true, "/Engine/main.cpp", "-...", "/Engine/...");
		AssertInclude(true, "/Engine/main.cpp", "-...", "Engine/...");
		AssertInclude(true, "Engine/main.cpp", "-...", "/Engine/...");
	}

	[TestMethod]
	public void DirectoryWildcardFileTypes()
	{
		AssertInclude(true, "/Engine/main.h", "/Engine/...h");
		AssertInclude(true, "/Engine/Foo/foo.h", "/Engine/...h");
		AssertInclude(true, "/foo.h", "/...h");
		AssertInclude(true, "/foo.h", "...h");
		AssertInclude(true, "foo.h", "...h");
		AssertInclude(true, "/Engine/foo.h", "/...h");
		AssertInclude(false, "/Engine/foo.h", "/...c");
		AssertInclude(true, "/Engine/readme.txt", "-...h", "-...cpp");
		AssertInclude(false, "/Engine/readme.txt", "/Docs/...", "-...h", "-...cpp");
	}

	[TestMethod]
	public void Exact()
	{
		// Leading slash is handled no matter if path or filter has one
		AssertInclude(true, "/main.cpp", "/main.cpp");
		AssertInclude(true, "/main.cpp", "main.cpp");
		AssertInclude(true, "main.cpp", "main.cpp");
		AssertInclude(true, "main.cpp", "/main.cpp");

		AssertInclude(true, "/Engine/main.cpp", "/Engine/main.cpp");
		AssertInclude(false, "/Engine/main.cpp", "/Engine/main.c");
		AssertInclude(false, "/Engine/main.cpp", "-/Engine/main.cpp");
		AssertInclude(true, "/Engine/main.cpp", "-/Engine/main.c", "/Engine/main.cpp");
	}

	[TestMethod]
	public void EntryParse()
	{
		AssertEntry("/Engine/...", true, "...", "/Engine/", "");
		AssertEntry("/Engine/....cpp", true, "...", "/Engine/", ".cpp");
		AssertEntry("/Engine/*", true, "*", "/Engine/", "");
		AssertEntry("/Engine/*.h", true, "*", "/Engine/", ".h");
		AssertEntry("/Engine/main.cpp", true, "", "/Engine/main.cpp", "");
		AssertEntry("-/Engine/...", false, "...", "/Engine/", "");
		AssertEntry("-/*", false, "*", "/", "");
	}

	private static void AssertInclude(bool expectIncluded, string path, params string[] filters)
	{
		Assert.AreEqual(expectIncluded, PerforceViewFilter.Parse(filters).IncludeFile(path, StringComparison.OrdinalIgnoreCase));
	}

	private static void AssertEntry(string text, bool include, string wildcard, string prefix, string suffix)
	{
		PerforceViewFilterEntry e = PerforceViewFilterEntry.Parse(text);
		Assert.AreEqual(include, e.Include);
		Assert.AreEqual(wildcard, e.Wildcard);
		Assert.AreEqual(prefix, e.Prefix);
		Assert.AreEqual(suffix, e.Suffix);
	}
}