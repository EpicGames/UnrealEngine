// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Perforce.Tests;

[TestClass]
public class PerforceViewMapTests
{
	[TestMethod]
	public void Mappings()
	{
		// ... wildcard with file extension (four dots)
		AssertMapSpec("//UE5/Main/Engine/Plugins/....xml Engine/Plugins/....xml")
			.Maps("//UE5/Main/Engine/Plugins/SomeFile.xml", "Engine/Plugins/SomeFile.xml")
			.Maps("//UE5/Main/Engine/Plugins/Sub/Dir/SomeFile.xml", "Engine/Plugins/Sub/Dir/SomeFile.xml")
			.NoMap("//UE5/Main/Engine/Plugins/SomeFile.cpp");

		// ... wildcard
		AssertMapSpec("//UE5/Engine/... Engine/...")
			.Maps("//UE5/Engine/SomeFile.xml", "Engine/SomeFile.xml")
			.Maps("//UE5/Engine/Some/Dir/SomeFile.xml", "Engine/Some/Dir/SomeFile.xml");

		// ... wildcard rewrite
		AssertMapSpec("//UE5/Engine/... NewDir/...")
			.Maps("//UE5/Engine/SomeFile.xml", "NewDir/SomeFile.xml")
			.Maps("//UE5/Engine/Some/Dir/SomeFile.xml", "NewDir/Some/Dir/SomeFile.xml");

		// * wildcard
		AssertMapSpec("//UE5/Engine/* Engine/*")
			.Maps("//UE5/Engine/SomeFile.xml", "Engine/SomeFile.xml")
			.NoMap("//UE5/Engine/Some/Dir/SomeFile.xml");

		// * wildcard with file extension
		AssertMapSpec("//UE5/Engine/*.cpp Engine/*.cpp")
			.Maps("//UE5/Engine/SomeFile.cpp", "Engine/SomeFile.cpp")
			.NoMap("//UE5/Engine/MyHeader.h")
			.NoMap("//UE5/Engine/Some/Dir/SomeFile.cpp");

		// %%1 numbered wildcard
		AssertMapSpec("//UE5/Engine/%%1 %%1")
			.Maps("//UE5/Engine/SomeFile.cpp", "SomeFile.cpp");
	}

	private class MapAssertion(PerforceViewMap map)
	{
		public MapAssertion Maps(string depotPath, string mappedPath)
		{
			Assert.IsTrue(map.TryMapFile(depotPath, StringComparison.Ordinal, out string target));
			Assert.AreEqual(mappedPath, target);
			return this;
		}

		public MapAssertion NoMap(string depotPath)
		{
			Assert.IsFalse(map.TryMapFile(depotPath, StringComparison.Ordinal, out string _));
			return this;
		}
	}

	private static MapAssertion AssertMapSpec(string mapping) => new(PerforceViewMap.Parse([mapping]));
}