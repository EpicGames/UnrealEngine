<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <None Remove="EpicGames.Perforce.uatbuildrecord" />
    <None Remove="EpicGames.Perforce.xml" />
  </ItemGroup>

  <PropertyGroup>
    <WindowsNativeConfig>Release</WindowsNativeConfig>
    <WindowsNativeConfig Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">Debug</WindowsNativeConfig>
    <EnableNETAnalyzers>True</EnableNETAnalyzers>
  </PropertyGroup>

  <ItemGroup>
    <None Include="..\..\..\..\Binaries\DotNET\EpicGames.Perforce.Native\win-x64\$(WindowsNativeConfig)\EpicGames.Perforce.Native.dll" Visible="false">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\DotNET\EpicGames.Perforce.Native\win-x64\$(WindowsNativeConfig)\EpicGames.Perforce.Native.pdb" Visible="false" Condition="Exists('..\..\..\..\Binaries\DotNET\EpicGames.Perforce.Native\win-x64\$(WindowsNativeConfig)\EpicGames.Perforce.Native.pdb')">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\MSVCP140.dll" Visible="false" Condition="Exists('..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\MSVCP140.dll')">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\VCRUNTIME140.dll" Visible="false" Condition="Exists('..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\VCRUNTIME140.dll')">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\VCRUNTIME140_1.dll" Visible="false" Condition="Exists('..\..\..\..\Binaries\ThirdParty\AppLocalDependencies\Win64\Microsoft.VC.CRT\VCRUNTIME140_1.dll')">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\DotNET\EpicGames.Perforce.Native\linux-x64\Release\EpicGames.Perforce.Native.so" Visible="false">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Include="..\..\..\..\Binaries\DotNET\EpicGames.Perforce.Native\mac-x64\Release\EpicGames.Perforce.Native.dylib" Visible="false">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="OpenTelemetry.Api" Version="1.15.3" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>
</Project>
