// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Wrapper for a list of filespecs. Allows implicit conversion from string (a single entry) or list.
	/// </summary>
	public readonly struct FileSpecList : IEquatable<FileSpecList>
	{
		/// <summary>
		/// Empty filespec list
		/// </summary>
		public static FileSpecList Empty { get; } = new FileSpecList([]);

		/// <summary>
		/// Matches any files in the depot
		/// </summary>
		public static FileSpecList Any { get; } = new FileSpecList(["//..."]);

		/// <summary>
		/// The list of filespecs
		/// </summary>
		public IReadOnlyList<string> List { get; }

		/// <summary>
		/// Private constructor. Use implicit conversion operators below instead.
		/// </summary>
		/// <param name="fileSpecList">List of filespecs</param>
		private FileSpecList(IReadOnlyList<string> fileSpecList)
		{
			List = fileSpecList;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is FileSpecList other && Equals(other);

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			HashCode code = new();
			foreach (string item in List)
			{
				code.Add(item);
			}
			return code.ToHashCode();
		}

		/// <inheritdoc/>
		public bool Equals(FileSpecList other) => List.SequenceEqual(other.List, StringComparer.Ordinal);

		/// <summary>
		/// Test two filespecs for equality
		/// </summary>
		public static bool operator ==(FileSpecList a, FileSpecList b) => a.Equals(b);

		/// <summary>
		/// Test two filespecs for inequality
		/// </summary>
		public static bool operator !=(FileSpecList a, FileSpecList b) => !a.Equals(b);

		/// <summary>
		/// Implicit conversion operator from a list of filespecs
		/// </summary>
		/// <param name="list">The list to construct from</param>
		public static implicit operator FileSpecList(List<string> list)
		{
			return new FileSpecList(list);
		}

		/// <summary>
		/// Implicit conversion operator from an array of filespecs
		/// </summary>
		/// <param name="array">The array to construct from</param>
		public static implicit operator FileSpecList(string[] array)
		{
			return new FileSpecList(array);
		}

		/// <summary>
		/// Implicit conversion operator from a single filespec
		/// </summary>
		/// <param name="fileSpec">The single filespec to construct from</param>
		public static implicit operator FileSpecList(string fileSpec)
		{
			return new FileSpecList([fileSpec]);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			if (List.Count == 0)
			{
				return "(none)";
			}
			else
			{
				return String.Join(", ", List);
			}
		}
	}
}
