// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Runs a background task that logs warnings if the Tick method isn't called within a certain time period
	/// </summary>
	public sealed class HangMonitor : IDisposable
	{
		class Scope : IDisposable
		{
			readonly HangMonitor _hangMonitor;
			readonly string _activity;

			public string Activity => _activity;

			public Scope(HangMonitor hangMonitor, string activity)
			{
				_hangMonitor = hangMonitor;
				_activity = activity;
			}

			public void Dispose()
			{
				Interlocked.CompareExchange(ref _hangMonitor._scope, null, this);
			}
		}

		readonly AsyncEvent _activeEvent = new();
		readonly TimeSpan _interval;
		readonly string _context;
		readonly ILogger _logger;
		long _lastUpdateTicks;
		Scope? _scope;
		BackgroundTask? _hangTask;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="interval">Interval after which to log a message</param>
		/// <param name="context">Context for hang messages</param>
		/// <param name="logger">Logger to write to </param>
		public HangMonitor(TimeSpan interval, string context, ILogger logger)
		{
			_interval = interval;
			_context = context;
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_hangTask != null)
			{
				Task.Run(async () => await _hangTask.DisposeAsync()).Wait();
				_hangTask = null;
			}
		}

		/// <summary>
		/// Start monitoring for hangs.
		/// </summary>
		/// <param name="activity">Activity to log if a hang is detected</param>
		public IDisposable Start(string activity)
		{
			Scope scope = new(this, activity);
			if (Interlocked.CompareExchange(ref _scope, scope, null) != null)
			{
				throw new InvalidOperationException();
			}

			_lastUpdateTicks = Stopwatch.GetTimestamp();
			_hangTask ??= BackgroundTask.StartNew(CheckStatusAsync);
			_activeEvent.Pulse();

			return scope;
		}

		/// <summary>
		/// Marks the operation as ongoing
		/// </summary>
		public void Tick()
		{
			Interlocked.Exchange(ref _lastUpdateTicks, Stopwatch.GetTimestamp());
		}

		async Task CheckStatusAsync(CancellationToken cancellationToken)
		{
			TimeSpan nextInterval = TimeSpan.Zero;
			for (; ; )
			{
				await Task.Delay(nextInterval, cancellationToken);
				nextInterval = _interval;

				Task activeTask = _activeEvent.Task;

				Scope? scope = Interlocked.CompareExchange(ref _scope, null, null);
				if (scope == null)
				{
					await activeTask.WaitAsync(cancellationToken);
					continue;
				}

				long currentTicks = Stopwatch.GetTimestamp();
				long lastUpdateTicks = Interlocked.CompareExchange(ref _lastUpdateTicks, 0, 0);
				if (currentTicks > lastUpdateTicks)
				{
					TimeSpan time = TimeSpan.FromSeconds((double)(currentTicks - lastUpdateTicks) / Stopwatch.Frequency);
					if (time >= _interval)
					{
						_logger.LogWarning("Hang detected ({Context}): {Activity} ({Time}s)", _context, scope.Activity, (int)time.TotalSeconds);
					}
					else
					{
						nextInterval = _interval - time;
					}
				}
			}
		}
	}
}
