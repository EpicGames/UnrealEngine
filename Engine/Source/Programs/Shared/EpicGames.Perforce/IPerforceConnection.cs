// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Base interface for Perforce clients
	/// </summary>
	public interface IPerforceConnection : IDisposable
	{
		/// <summary>
		/// Connection settings
		/// </summary>
		IPerforceSettings Settings { get; }

		/// <summary>
		/// Logger for this connection
		/// </summary>
		ILogger Logger { get; }

		/// <summary>
		/// Tracer for this connection
		/// </summary>
		Tracer Tracer { get; }

		/// <summary>
		/// Queues a Perforce command to be executed
		/// </summary>
		/// <param name="command">The command name</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="fileArguments">File arguments (may be put into a response file)</param>
		/// <param name="inputData">Input data to be passed to the command</param>
		/// <param name="promptResponse">Response for login prompts</param>
		/// <param name="interceptIo">Whether to intercept file I/O and return it in the reponse stream. Only supported by the native client.</param>
		/// <returns>Response object</returns>
		IPerforceOutput Command(string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, string? promptResponse, bool interceptIo);

		/// <summary>
		/// Creates a record from a set of input fields
		/// </summary>
		/// <param name="fields">Fields for the record</param>
		/// <returns>Serialized record data</returns>
		PerforceRecord CreateRecord(List<KeyValuePair<string, object>> fields);
	}
}
