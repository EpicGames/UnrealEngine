// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Interface for the result of a Perforce operation
	/// </summary>
	public interface IPerforceOutput : IAsyncDisposable
	{
		/// <summary>
		/// Data containing the result
		/// </summary>
		ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Waits until more data has been read into the buffer. 
		/// </summary>
		/// <returns>True if more data was read, false otherwise</returns>
		Task<bool> ReadAsync(CancellationToken token);

		/// <summary>
		/// Discard bytes from the start of the result buffer
		/// </summary>
		/// <param name="numBytes">Number of bytes to discard</param>
		void Discard(int numBytes);
	}

	/// <summary>
	/// Utility methods for IPerforceOutput
	/// </summary>
	public static class PerforceOutput
	{
		/// <summary>
		/// String constants for records
		/// </summary>
		static class ReadOnlyUtf8StringConstants
		{
			public static readonly Utf8String Code = new("code");
			public static readonly Utf8String Stat = new("stat");
			public static readonly Utf8String Info = new("info");
			public static readonly Utf8String Error = new("error");
			public static readonly Utf8String Io = new("io");
			public static readonly Utf8String Func = new("func");
			public static readonly Utf8String IsSparse = new("isSparse");
		}

		/// <summary>
		/// Standard prefix for a returned record: record indicator, string, 4 bytes, 'code', string, [value]
		/// </summary>
		static readonly byte[] s_recordPrefix = [(byte)'{', (byte)'s', 4, 0, 0, 0, (byte)'c', (byte)'o', (byte)'d', (byte)'e', (byte)'s'];

		class BufferedPerforceOutput : IPerforceOutput
		{
			public ReadOnlyMemory<byte> Data { get; private set; }

			public BufferedPerforceOutput(ReadOnlyMemory<byte> data)
			{
				Data = data;
			}

			public Task<bool> ReadAsync(CancellationToken cancellationToken) => Task.FromResult(Data.Length > 0);
			public void Discard(int numBytes) => Data = Data.Slice(numBytes);
			public ValueTask DisposeAsync() => new();
		}

		/// <summary>
		/// Constructs an <see cref="IPerforceOutput"/> object from a block of data
		/// </summary>
		/// <param name="data">Data to construct from</param>
		/// <returns>Output object</returns>
		public static IPerforceOutput FromData(ReadOnlyMemory<byte> data) => new BufferedPerforceOutput(data);

		/// <summary>
		/// Constructs an <see cref="IPerforceOutput"/> object from a response
		/// </summary>
		/// <param name="response">Response to construct from</param>
		/// <returns>Output object</returns>
		public static IPerforceOutput FromResponse(PerforceResponse response) => FromResponses([response]);

		/// <summary>
		/// Constructs an <see cref="IPerforceOutput"/> object from a sequence of responses
		/// </summary>
		/// <param name="responses">Responses to construct from</param>
		/// <returns>Output object</returns>
		public static IPerforceOutput FromResponses(IEnumerable<PerforceResponse> responses)
		{
			using ChunkedMemoryWriter writer = new();
			foreach (PerforceResponse response in responses)
			{
				writer.WriteFixedLengthBytes(s_recordPrefix);

				Utf8String code = ReadOnlyUtf8StringConstants.Stat;

				Span<byte> span = writer.GetSpanAndAdvance(code.Length + 4);
				BinaryPrimitives.WriteInt32LittleEndian(span.Slice(0, 4), code.Length);
				code.Span.CopyTo(span.Slice(4));

				PerforceReflection.Serialize(response.Data, writer);
				writer.WriteUInt8((byte)'0');
			}
			return FromData(writer.ToByteArray());
		}

		/// <summary>
		/// Formats the current contents of the buffer to a string
		/// </summary>
		/// <param name="data">The next byte that was read</param>
		/// <returns>String representation of the buffer</returns>
		private static string FormatDataAsString(ReadOnlySpan<byte> data)
		{
			StringBuilder result = new();
			if (data.Length > 0)
			{
				for (int idx = 0; idx < data.Length && idx < 1024;)
				{
					result.Append("\n   ");

					// Output to the end of the line
					for (; idx < data.Length && idx < 1024 && data[idx] != '\r' && data[idx] != '\n'; idx++)
					{
						if (data[idx] == '\t')
						{
							result.Append('\t');
						}
						else if (data[idx] == '\\')
						{
							result.Append('\\');
						}
						else if (data[idx] >= 0x20 && data[idx] <= 0x7f)
						{
							result.Append((char)data[idx]);
						}
						else
						{
							result.AppendFormat("\\x{0:x2}", data[idx]);
						}
					}

					// Skip the newline characters
					if (idx < data.Length && data[idx] == '\r')
					{
						idx++;
					}
					if (idx < data.Length && data[idx] == '\n')
					{
						idx++;
					}
				}
			}
			return result.ToString();
		}

		/// <summary>
		/// Formats the current contents of the buffer to a string
		/// </summary>
		/// <param name="data">The data to format</param>
		/// <param name="maxLength"></param>
		/// <returns>String representation of the buffer</returns>
		private static string FormatDataAsHexDump(ReadOnlySpan<byte> data, int maxLength = 1024)
		{
			// Format the result
			StringBuilder result = new();

			const int RowLength = 16;
			for (int baseIdx = 0; baseIdx < data.Length && baseIdx < maxLength; baseIdx += RowLength)
			{
				result.Append("\n    ");
				for (int offset = 0; offset < RowLength; offset++)
				{
					int idx = baseIdx + offset;
					if (idx >= data.Length)
					{
						result.Append("   ");
					}
					else
					{
						result.AppendFormat("{0:x2} ", data[idx]);
					}
				}
				result.Append("   ");
				for (int offset = 0; offset < RowLength; offset++)
				{
					int idx = baseIdx + offset;
					if (idx >= data.Length)
					{
						break;
					}
					else if (data[idx] < 0x20 || data[idx] >= 0x7f)
					{
						result.Append('.');
					}
					else
					{
						result.Append((char)data[idx]);
					}
				}
			}
			return result.ToString();
		}

		static readonly byte[] s_connectionErrorPrefix = Encoding.UTF8.GetBytes("Perforce client error:");

		/// <summary>
		/// Read a list of responses from the child process
		/// </summary>
		/// <param name="perforce">The response to read from</param>
		/// <param name="statRecordType">The type of stat record to parse</param>
		/// <param name="cancellationToken">Cancellation token for the read</param>
		/// <returns>Async task</returns>
		public static async IAsyncEnumerable<PerforceResponse> ReadStreamingResponsesAsync(this IPerforceOutput perforce, Type? statRecordType, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			CachedRecordInfo? statRecordInfo = (statRecordType == null) ? null : PerforceReflection.GetCachedRecordInfo(statRecordType);

			List<PerforceResponse> responses = [];

			// Read all the records into a list
			long parsedLen = 0;
			long maxParsedLen = 0;
			while (await perforce.ReadAsync(cancellationToken))
			{
				// Check for the whole message not being a marshalled python object, and produce a better response in that scenario
				ReadOnlyMemory<byte> data = perforce.Data;
				if (data.Length > 0 && responses.Count == 0 && data.Span[0] != '{')
				{
					if (data.Span.StartsWith(s_connectionErrorPrefix))
					{
						throw new PerforceException("Unable to connect to Perforce server:{0}", FormatDataAsString(data.Span.Slice(s_connectionErrorPrefix.Length)).TrimStart('\n', '\r'));
					}
					else
					{
						throw new PerforceException("Unexpected response from server (expected '{{'):{0}", FormatDataAsString(data.Span));
					}
				}

				// Parse the responses from the current buffer
				int bufferPos = 0;
				for (; ; )
				{
					int newBufferPos = bufferPos;
					if (!TryReadResponse(data, ref newBufferPos, statRecordInfo, out PerforceResponse? response))
					{
						maxParsedLen = parsedLen + newBufferPos;
						break;
					}

					yield return response;
					bufferPos = newBufferPos;
				}

				// Discard all the data that we've processed
				perforce.Discard(bufferPos);
				parsedLen += bufferPos;
			}

			// If the stream is complete but we couldn't parse a response from the server, treat it as an error
			if (perforce.Data.Length > 0)
			{
				long dumpOffset = Math.Max(maxParsedLen - 32, parsedLen);
				int sliceOffset = (int)(dumpOffset - parsedLen);
				string strDump = FormatDataAsString(perforce.Data.Span.Slice(sliceOffset));
				string hexDump = FormatDataAsHexDump(perforce.Data.Span.Slice(sliceOffset, Math.Min(1024, perforce.Data.Length - sliceOffset)));
				throw new PerforceException("Unparsable data at offset {0}+{1}/{2}.\nString data from offset {3}:{4}\nHex data from offset {3}:{5}", parsedLen, maxParsedLen - parsedLen, parsedLen + perforce.Data.Length, dumpOffset, strDump, hexDump);
			}
		}

		/// <summary>
		/// Read a list of responses from the child process
		/// </summary>
		/// <param name="perforce">The response to read from</param>
		/// <param name="statRecordType">The type of stat record to parse</param>
		/// <param name="cancellationToken">Cancellation token for the read</param>
		/// <returns>Async task</returns>
		public static async Task<List<PerforceResponse>> ReadResponsesAsync(this IPerforceOutput perforce, Type? statRecordType, CancellationToken cancellationToken)
		{
			CachedRecordInfo? statRecordInfo = (statRecordType == null) ? null : PerforceReflection.GetCachedRecordInfo(statRecordType);

			List<PerforceResponse> responses = [];

			// Read all the records into a list
			long maxParsedLen;
			for (; ; )
			{
				// Check for the whole message not being a marshalled python object, and produce a better response in that scenario
				ReadOnlyMemory<byte> data = perforce.Data;
				if (data.Length > 0 && responses.Count == 0 && data.Span[0] != '{')
				{
					throw new PerforceException("Unexpected response from server (expected '{{'):{0}", FormatDataAsString(data.Span));
				}

				// Parse the responses from the current buffer
				int bufferPos = 0;
				for (; ; )
				{
					int newBufferPos = bufferPos;
					if (!TryReadResponse(data, ref newBufferPos, statRecordInfo, out PerforceResponse? response))
					{
						maxParsedLen = newBufferPos;
						break;
					}

					responses.Add(response);
					bufferPos = newBufferPos;
				}

				// Discard all the data that we've processed
				perforce.Discard(bufferPos);
				maxParsedLen -= bufferPos;

				// Try to read more data into the buffer
				if (!await perforce.ReadAsync(cancellationToken))
				{
					break;
				}
			}

			// If the stream is complete but we couldn't parse a response from the server, treat it as an error
			if (perforce.Data.Length > 0)
			{
				int sliceOffset = (int)Math.Max(maxParsedLen - 32, 0);
				string strDump = FormatDataAsString(perforce.Data.Span.Slice(sliceOffset));
				string hexDump = FormatDataAsHexDump(perforce.Data.Span.Slice(sliceOffset, Math.Min(1024, perforce.Data.Length - sliceOffset)));
				throw new PerforceException("Unparsable data at offset {0}.\nString data from offset {1}:{2}\nHex data from offset {1}:{3}", maxParsedLen, sliceOffset, strDump, hexDump);
			}

			return responses;
		}

		/// <summary>
		/// Read a list of responses from the child process
		/// </summary>
		/// <param name="perforce">The Perforce response</param>
		/// <param name="handleRecord">Delegate to invoke for each record read</param>
		/// <param name="cancellationToken">Cancellation token for the read</param>
		/// <returns>Async task</returns>
		public static async Task ReadRecordsAsync(this IPerforceOutput perforce, Action<PerforceRecord> handleRecord, CancellationToken cancellationToken)
		{
			PerforceRecord record = new();
			while (await perforce.ReadAsync(cancellationToken))
			{
				// Start a read to add more data
				ReadOnlyMemory<byte> data = perforce.Data;

				// Parse the responses from the current buffer
				int bufferPos = 0;
				for (; ; )
				{
					int initialBufferPos = bufferPos;
					if (!ParseRecord(data, ref bufferPos, record.Rows))
					{
						bufferPos = initialBufferPos;
						break;
					}
					handleRecord(record);
				}
				perforce.Discard(bufferPos);
			}

			// If the stream is complete but we couldn't parse a response from the server, treat it as an error
			if (perforce.Data.Length > 0)
			{
				throw new PerforceException("Unexpected trailing response data from server:{0}", FormatDataAsString(perforce.Data.Span));
			}
		}

		/// <summary>
		/// Reads from the buffer into a record object
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="rows">List of rows to read into</param>
		/// <returns>True if a record could be read; false if more data is required</returns>
		[SuppressMessage("Design", "CA1045:Do not pass types by reference", Justification = "<Pending>")]
		public static bool ParseRecord(ReadOnlyMemory<byte> buffer, ref int bufferPos, List<KeyValuePair<Utf8String, PerforceValue>> rows)
		{
			rows.Clear();
			ReadOnlySpan<byte> bufferSpan = buffer.Span;

			// Check we can read the initial record marker
			if (bufferPos >= buffer.Length)
			{
				return false;
			}
			if (bufferSpan[bufferPos] != '{')
			{
				throw new PerforceException("Invalid record start");
			}
			bufferPos++;

			// Capture the start of the string
			for (; ; )
			{
				// Check that we've got a string field
				if (bufferPos >= buffer.Length)
				{
					return false;
				}

				// If this is the end of the record, break out
				byte keyType = buffer.Span[bufferPos++];
				if (keyType == '0')
				{
					break;
				}
				else if (keyType != 's')
				{
					throw new PerforceException("Unexpected key field type while parsing marshalled output ({0}) - expected 's', got: {1}", (int)keyType, FormatDataAsHexDump(buffer.Slice(bufferPos - 1).Span));
				}

				// Read the tag
				Utf8String key;
				if (!TryReadString(buffer, ref bufferPos, out key))
				{
					return false;
				}

				// Remember the start of the value
				int valueOffset = bufferPos;

				// Read the value type
				byte valueType;
				if (!TryReadByte(bufferSpan, ref bufferPos, out valueType))
				{
					return false;
				}

				// Parse the appropriate value
				PerforceValue value;
				if (valueType == 's')
				{
					if (!TryReadString(buffer, ref bufferPos, out _))
					{
						return false;
					}
					value = new PerforceValue(buffer.Slice(valueOffset, bufferPos - valueOffset).ToArray());
				}
				else if (valueType == 'i')
				{
					if (!TryReadInt(bufferSpan, ref bufferPos, out _))
					{
						return false;
					}
					value = new PerforceValue(buffer.Slice(valueOffset, bufferPos - valueOffset).ToArray());
				}
				else
				{
					throw new PerforceException("Unrecognized value type {0}", valueType);
				}

				// Construct the response object with the record
				rows.Add(KeyValuePair.Create(key.Clone(), value));
			}
			return true;
		}

		/// <summary>
		/// Reads a response object from the buffer
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="statRecordInfo">The type of record expected to parse from the response</param>
		/// <param name="response">Receives the response object on success</param>
		/// <returns>True if a response was read, false if the buffer needs more data</returns>
		static bool TryReadResponse(ReadOnlyMemory<byte> buffer, ref int bufferPos, CachedRecordInfo? statRecordInfo, [NotNullWhen(true)] out PerforceResponse? response)
		{
			if (bufferPos + s_recordPrefix.Length + 4 > buffer.Length)
			{
				response = null;
				return false;
			}

			ReadOnlyMemory<byte> prefix = buffer.Slice(bufferPos, s_recordPrefix.Length);
			if (!prefix.Span.SequenceEqual(s_recordPrefix))
			{
				throw new PerforceException("Expected 'code' field at the start of record");
			}
			bufferPos += prefix.Length;

			Utf8String code;
			if (!TryReadString(buffer, ref bufferPos, out code))
			{
				response = null;
				return false;
			}

			// Dispatch it to the appropriate handler
			object? record;
			if (code == ReadOnlyUtf8StringConstants.Stat && statRecordInfo != null)
			{
				if (!TryReadTypedRecord(buffer, ref bufferPos, statRecordInfo, out record))
				{
					response = null;
					return false;
				}
			}
			else if (code == ReadOnlyUtf8StringConstants.Info)
			{
				if (!TryReadTypedRecord(buffer, ref bufferPos, PerforceReflection.InfoRecordInfo, out record))
				{
					response = null;
					return false;
				}
			}
			else if (code == ReadOnlyUtf8StringConstants.Error)
			{
				if (!TryReadTypedRecord(buffer, ref bufferPos, PerforceReflection.ErrorRecordInfo, out record))
				{
					response = null;
					return false;
				}
			}
			else if (code == ReadOnlyUtf8StringConstants.Io)
			{
				if (!TryReadTypedRecord(buffer, ref bufferPos, PerforceReflection.IoRecordInfo, out record))
				{
					response = null;
					return false;
				}
			}
			else
			{
				throw new PerforceException("Unknown return code for record: {0}", code);
			}

			// Skip over the record terminator
			if (bufferPos >= buffer.Length || buffer.Span[bufferPos] != '0')
			{
				throw new PerforceException("Unexpected record terminator");
			}
			bufferPos++;

			// Create the response
			response = new PerforceResponse(record);
			return true;
		}

		/// <summary>
		/// Parse an individual record from the server.
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="recordInfo">Reflection information for the type being serialized into.</param>
		/// <param name="record">Receives the record on success</param>
		/// <returns>The parsed object.</returns>
		static bool TryReadTypedRecord(ReadOnlyMemory<byte> buffer, ref int bufferPos, CachedRecordInfo recordInfo, [NotNullWhen(true)] out object? record)
		{
			// Create a bitmask for all the required tags
			ulong requiredTagsBitMask = 0;

			// Create the new record
			object? newRecord = recordInfo.CreateInstance();
			if (newRecord == null)
			{
				throw new InvalidDataException($"Unable to construct record of type {recordInfo.Type}");
			}

			// Get the record info, and parse it into the object
			ReadOnlySpan<byte> bufferSpan = buffer.Span;
			for (; ; )
			{
				// Check that we've got a string field
				if (bufferPos >= buffer.Length)
				{
					record = null;
					return false;
				}

				// If this is the end of the record, break out
				byte keyType = bufferSpan[bufferPos];
				if (keyType == '0')
				{
					break;
				}
				else if (keyType != 's')
				{
					throw new PerforceException("Unexpected key field type while parsing marshalled output ({0}) - expected 's', got: {1}", (int)keyType, FormatDataAsHexDump(buffer.Slice(bufferPos).Span));
				}

				// Capture the initial buffer position, in case we have to roll back
				int startBufferPos = bufferPos;
				bufferPos++;

				// Read the tag
				Utf8String tag;
				if (!TryReadString(buffer, ref bufferPos, out tag))
				{
					record = null;
					return false;
				}

				// Find the start of the array suffix
				int suffixIdx = tag.Length;
				while (suffixIdx > 0 && (tag[suffixIdx - 1] == (byte)',' || (tag[suffixIdx - 1] >= '0' && tag[suffixIdx - 1] <= '9')))
				{
					suffixIdx--;
				}

				// Separate the key into tag and suffix
				Utf8String suffix = tag.Slice(suffixIdx);
				tag = tag.Slice(0, suffixIdx);

				// Count the dimensions in the suffix
				int rank = 0;
				if (suffix.Length > 0)
				{
					rank++;
					for (int idx = 0; idx < suffix.Length; idx++)
					{
						if (suffix[idx] == (byte)',')
						{
							rank++;
						}
					}
				}

				// Try to find the matching field. Check all rank values down to zero in case we're parsing an array.
				TaggedPropertyInfo? tagInfo = null;
				for (; rank >= 0; rank--)
				{
					if (recordInfo.NameAndRankToInfo.TryGetValue(new TaggedPropertyNameAndRank(tag, rank), out tagInfo))
					{
						requiredTagsBitMask |= tagInfo.RequiredTagBitMask;
						break;
					}
				}

				// Find the target object for this tag
				object? targetRecord = null;
				if (tagInfo != null)
				{
					targetRecord = GetNestedRecord(newRecord, suffix, tagInfo.ParentRecords);
				}

				// Read the value
				if (!TryReadValue(buffer, ref bufferPos, targetRecord, tagInfo))
				{
					record = null;
					return false;
				}
			}

			// Make sure we've got all the required tags we need
			if (requiredTagsBitMask != recordInfo.RequiredTagsBitMask)
			{
				string missingTagNames = String.Join(", ", recordInfo.NameAndRankToInfo.Where(x => (requiredTagsBitMask | x.Value.RequiredTagBitMask) != requiredTagsBitMask).Select(x => x.Key.Tag));
				throw new PerforceException("Missing '{0}' tag when parsing '{1}'", missingTagNames, recordInfo.Type.Name);
			}

			// Construct the response object with the record
			record = newRecord;
			return true;
		}

		static object? GetNestedRecord(object? targetObject, Utf8String suffix, NestedRecordInfo[] nestedRecords)
		{
			object? nestedObject = targetObject;
			foreach (NestedRecordInfo nestedRecord in nestedRecords)
			{
				if (suffix.Length == 0)
				{
					return null;
				}

				// Parse the next index
				int index = 0;
				while (suffix.Length > 0)
				{
					byte character = suffix[0];
					suffix = suffix[1..];

					if (character >= '0' && character <= '9')
					{
						index = (index * 10) + (character - '0');
					}
					else
					{
						break;
					}
				}

				// Get the appropriate object
				System.Collections.IList? list = (System.Collections.IList?)nestedRecord.PropertyInfo.GetValue(nestedObject);
				if (list == null)
				{
					throw new Exception();
				}
				while (index >= list.Count)
				{
					list.Add(nestedRecord.CreateInstance());
				}

				// Move to the nested object
				nestedObject = list[index];
			}
			return nestedObject;
		}

		/// <summary>
		/// Reads a value from the input buffer
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="newRecord">The new record</param>
		/// <param name="tagInfo">The current tag</param>
		/// <returns></returns>
		static bool TryReadValue(ReadOnlyMemory<byte> buffer, ref int bufferPos, object? newRecord, TaggedPropertyInfo? tagInfo)
		{
			ReadOnlySpan<byte> bufferSpan = buffer.Span;

			// Read the value type
			byte valueType;
			if (!TryReadByte(bufferSpan, ref bufferPos, out valueType))
			{
				return false;
			}

			// Parse the appropriate value
			if (valueType == 's')
			{
				Utf8String stringValue;
				if (!TryReadString(buffer, ref bufferPos, out stringValue))
				{
					return false;
				}
				if (newRecord != null)
				{
					tagInfo?.ReadFromString(newRecord, stringValue);
				}
			}
			else if (valueType == 'i')
			{
				int integerValue;
				if (!TryReadInt(bufferSpan, ref bufferPos, out integerValue))
				{
					return false;
				}
				if (newRecord != null)
				{
					tagInfo?.ReadFromInteger(newRecord, integerValue);
				}
			}
			else
			{
				throw new PerforceException("Unrecognized value type {0}", valueType);
			}

			return true;
		}

		/// <summary>
		/// Attempts to read a single byte from the buffer
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="value">Receives the byte that was read</param>
		/// <returns>True if a byte was read from the buffer, false if there was not enough data</returns>
		static bool TryReadByte(ReadOnlySpan<byte> buffer, ref int bufferPos, out byte value)
		{
			if (bufferPos >= buffer.Length)
			{
				value = 0;
				return false;
			}

			value = buffer[bufferPos];
			bufferPos++;
			return true;
		}

		/// <summary>
		/// Attempts to read a single int from the buffer
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="value">Receives the value that was read</param>
		/// <returns>True if an int was read from the buffer, false if there was not enough data</returns>
		static bool TryReadInt(ReadOnlySpan<byte> buffer, ref int bufferPos, out int value)
		{
			if (bufferPos + 4 > buffer.Length)
			{
				value = 0;
				return false;
			}

			value = buffer[bufferPos + 0] | (buffer[bufferPos + 1] << 8) | (buffer[bufferPos + 2] << 16) | (buffer[bufferPos + 3] << 24);
			bufferPos += 4;
			return true;
		}

		/// <summary>
		/// Attempts to read a string from the buffer
		/// </summary>
		/// <param name="buffer">The buffer to read from</param>
		/// <param name="bufferPos">Current read position within the buffer</param>
		/// <param name="result">Receives the value that was read</param>
		/// <returns>True if a string was read from the buffer, false if there was not enough data</returns>
		static bool TryReadString(ReadOnlyMemory<byte> buffer, ref int bufferPos, out Utf8String result)
		{
			int length;
			if (!TryReadInt(buffer.Span, ref bufferPos, out length))
			{
				result = Utf8String.Empty;
				return false;
			}

			if (bufferPos + length > buffer.Length)
			{
				result = Utf8String.Empty;
				return false;
			}

			result = new Utf8String(buffer.Slice(bufferPos, length));
			bufferPos += length;
			return true;
		}
	}
}
