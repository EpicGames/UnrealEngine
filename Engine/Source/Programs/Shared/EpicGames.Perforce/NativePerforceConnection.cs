// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using OpenTelemetry.Trace;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Experimental implementation of <see cref="IPerforceConnection"/> which wraps the native C++ API.
	/// </summary>
	public sealed class NativePerforceConnection : IPerforceConnection, IDisposable
	{
		const string NativeDll = "EpicGames.Perforce.Native";

		[StructLayout(LayoutKind.Sequential)]
		class NativeSettings
		{
			[MarshalAs(UnmanagedType.LPStr)]
			public string? _serverAndPort;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _userName;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _password;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _hostName;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _clientName;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _appName;

			[MarshalAs(UnmanagedType.LPStr)]
			public string? _appVersion;
		}

		[StructLayout(LayoutKind.Sequential)]
		[SuppressMessage("Compiler", "CA1812")]
		class NativeReadBuffer
		{
			public IntPtr _data;
			public int _length;
			public int _count;
			public int _maxLength;
			public int _maxCount;
		};

		[StructLayout(LayoutKind.Sequential)]
		class NativeWriteBuffer
		{
			public IntPtr _data;
			public int _maxLength;
			public int _maxCount;
		};

		[UnmanagedFunctionPointer(CallingConvention.Cdecl)]
		delegate void OnBufferReadyFn(NativeReadBuffer readBuffer, [In, Out] NativeWriteBuffer writeBuffer);

		[DllImport(NativeDll, CallingConvention = CallingConvention.Cdecl)]
		static extern IntPtr Client_Create(NativeSettings? settings, NativeWriteBuffer writeBuffer, IntPtr onBufferReadyFnPtr);

		[DllImport(NativeDll, CallingConvention = CallingConvention.Cdecl, CharSet = CharSet.Ansi, BestFitMapping = false, ThrowOnUnmappableChar = true)]
		static extern void Client_Command(IntPtr client, [MarshalAs(UnmanagedType.LPStr)] string command, int numArgs, IntPtr[] args, byte[]? inputData, int inputLength, byte[]? promptResponse, bool interceptIo);

		[DllImport(NativeDll, CallingConvention = CallingConvention.Cdecl)]
		static extern void Client_CancelCommand(IntPtr client);

		[DllImport(NativeDll, CallingConvention = CallingConvention.Cdecl)]
		static extern void Client_Destroy(IntPtr client);

		/// <summary>
		/// A buffer used for native code to stream data into
		/// </summary>
		class PinnedBuffer : IDisposable
		{
			static readonly byte[] s_guardBytes = [.. Enumerable.Repeat<byte>(0xfd, 16)];

			public byte[] AllocatedData { get; private set; }
			public Memory<byte> Data { get; private set; }
			public GCHandle Handle { get; private set; }
			public IntPtr BasePtr { get; private set; }
			public int MaxLength => Data.Length;

			public PinnedBuffer(int maxLength)
			{
				AllocatedData = null!;
				AllocateBuffer(maxLength);
			}

			void AllocateBuffer(int maxLength)
			{
				AllocatedData = new byte[maxLength + (s_guardBytes.Length * 2)];
				Data = AllocatedData.AsMemory(s_guardBytes.Length, maxLength);
				Handle = GCHandle.Alloc(AllocatedData, GCHandleType.Pinned);
				BasePtr = Handle.AddrOfPinnedObject() + s_guardBytes.Length;

				s_guardBytes.AsSpan().CopyTo(AllocatedData);
				s_guardBytes.AsSpan().CopyTo(AllocatedData.AsSpan(AllocatedData.Length - s_guardBytes.Length));
			}

			public void CheckGuardBytes()
			{
				if (!AllocatedData.AsSpan(0, s_guardBytes.Length).SequenceEqual(s_guardBytes))
				{
					throw new InvalidOperationException("Guard bytes corrupted at start of memory region");
				}
				if (!AllocatedData.AsSpan(AllocatedData.Length - s_guardBytes.Length).SequenceEqual(s_guardBytes))
				{
					throw new InvalidOperationException("Guard bytes corrupted at end of memory region");
				}
			}

			public void Resize(int maxLength)
			{
				byte[] oldData = AllocatedData;
				Handle.Free();

				AllocateBuffer(maxLength);

				oldData.CopyTo(AllocatedData, 0);
			}

			public void Dispose()
			{
				Handle.Free();
			}
		}

		/// <summary>
		/// Response object for a request
		/// </summary>
		class Response : IPerforceOutput
		{
			readonly NativePerforceConnection _outer;

			PinnedBuffer? _buffer;
			int _bufferPos;
			int _bufferLen;

			PinnedBuffer? _nextBuffer;
			int _nextBufferPos;
			int _nextBufferLen;

			CancellationTokenRegistration _cancelRegistration;
			bool _cancelRegistered;

			public Channel<(PinnedBuffer Buffer, int Length)> _readBuffers = Channel.CreateUnbounded<(PinnedBuffer, int)>();

			public ReadOnlyMemory<byte> Data => (_buffer == null) ? ReadOnlyMemory<byte>.Empty : _buffer.Data.Slice(_bufferPos, _bufferLen - _bufferPos);

			public Response(NativePerforceConnection outer)
			{
				_outer = outer;
			}

			public async ValueTask DisposeAsync()
			{
				_cancelRegistration.Dispose();

				if (_buffer != null)
				{
					_outer._writeBuffers.Add(_buffer);
				}
				if (_nextBuffer != null)
				{
					_outer._writeBuffers.Add(_nextBuffer);
				}

				while (await _readBuffers.Reader.WaitToReadAsync())
				{
					(PinnedBuffer, int) buffer;
					if (_readBuffers.Reader.TryRead(out buffer))
					{
						_outer._writeBuffers.Add(buffer.Item1);
					}
				}

				_outer._responseCompleteEvent.Set();
			}

			async Task<(PinnedBuffer?, int)> GetNextReadBufferAsync(CancellationToken token = default)
			{
				for (; ; )
				{
					if (!await _readBuffers.Reader.WaitToReadAsync(token))
					{
						return (null, 0);
					}

					(PinnedBuffer, int) pair;
					if (_readBuffers.Reader.TryRead(out pair))
					{
						return pair;
					}
				}
			}

			public async Task<bool> ReadAsync(CancellationToken token)
			{
				if (!_cancelRegistered && token.CanBeCanceled)
				{
					_cancelRegistered = true;
					IntPtr clientPtr = _outer._client;
					_cancelRegistration = token.Register(() => Client_CancelCommand(clientPtr));
				}

				// If we don't have any data yet, wait until a read completes
				if (_buffer == null)
				{
					(_buffer, _bufferLen) = await GetNextReadBufferAsync(token);
					return _buffer != null;
				}

				// If we've used up all the data in the buffer, return it to the write list and move to the next one.
				int originalBufferLen = _bufferLen - _nextBufferPos;
				if (_bufferPos >= originalBufferLen)
				{
					_outer._writeBuffers.TryAdd(_buffer!);

					_buffer = _nextBuffer;
					_bufferPos -= originalBufferLen;
					_bufferLen = _nextBufferLen;

					_nextBuffer = null;
					_nextBufferPos = 0;
					_nextBufferLen = 0;

					return true;
				}

				// Ensure there's some space in the current buffer. In order to handle cases where we want to read data straddling both buffers, copy 16k chunks
				// back to the first buffer until we can read entirely from the second buffer.
				int maxAppend = _buffer.MaxLength - _bufferLen;
				if (maxAppend == 0)
				{
					if (_bufferPos > 0)
					{
						_buffer.Data.Slice(_bufferPos, _bufferLen - _bufferPos).CopyTo(_buffer.Data);
						_bufferLen -= _bufferPos;
						_bufferPos = 0;
					}
					else
					{
						_buffer.Resize(_buffer.MaxLength + 16384);
					}
					maxAppend = _buffer.MaxLength - _bufferLen;
				}

				// Read the next buffer
				if (_nextBuffer == null)
				{
					(_nextBuffer, _nextBufferLen) = await GetNextReadBufferAsync(token);
					if (_nextBuffer == null)
					{
						return false;
					}
				}

				// Try to copy some data from the next buffer
				int copyLen = Math.Min(_nextBufferLen - _nextBufferPos, Math.Min(maxAppend, 16384));
				_nextBuffer.Data.Slice(_nextBufferPos, copyLen).CopyTo(_buffer.Data.Slice(_bufferLen));
				_bufferLen += copyLen;
				_nextBufferPos += copyLen;

				// If we've read everything from the next buffer, return it to the write list
				if (_nextBufferPos == _nextBufferLen)
				{
					_outer._writeBuffers.Add(_nextBuffer, token);

					_nextBuffer = null;
					_nextBufferPos = 0;
					_nextBufferLen = 0;
				}

				return true;
			}

			public void Discard(int numBytes)
			{
				// Update the read position
				_bufferPos += numBytes;
				Debug.Assert(_bufferPos <= _bufferLen);
			}
		}

		static int s_nextUniqueId;

		IntPtr _client;
		readonly int _uniqueId;
		readonly PinnedBuffer[] _buffers;
		readonly OnBufferReadyFn _onBufferReadyInst;
		readonly IntPtr _onBufferReadyFnPtr;
		Thread? _backgroundThread;
		readonly BlockingCollection<(Action, Response)?> _requests = [];
		readonly BlockingCollection<PinnedBuffer> _writeBuffers = [];
		Response? _currentResponse;
		readonly ManualResetEvent _responseCompleteEvent;
		readonly Stopwatch _stallTimer = new();
		readonly HangMonitor _hangMonitor;
		bool _disposed;

		/// <inheritdoc/>
		public IPerforceSettings Settings { get; }

		/// <inheritdoc/>
		public ILogger Logger { get; set; }

		/// <inheritdoc/>
		public Tracer Tracer { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="settings">Settings for the connection</param>
		/// <param name="logger">Logger for messages</param>
		/// <param name="tracer">Tracer</param>
		public NativePerforceConnection(IPerforceSettings settings, ILogger logger, Tracer? tracer = null)
			: this(settings, 2, 128 * 1024, logger, tracer)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="settings">Settings for the connection</param>
		/// <param name="bufferCount">Number of buffers to create for streaming response data</param>
		/// <param name="bufferSize">Size of each buffer</param>
		/// <param name="logger">Logger for messages</param>
		/// <param name="tracer">Tracer</param> 
		public NativePerforceConnection(IPerforceSettings settings, int bufferCount, int bufferSize, ILogger logger, Tracer? tracer = null)
		{
			Settings = settings;
			Logger = logger;
			Tracer = tracer ?? PerforceConnection.DefaultTracer;

			_uniqueId = Interlocked.Increment(ref s_nextUniqueId);

			ILogger hangLogger = settings.EnableHangMonitor ? logger : NullLogger.Instance;
			_hangMonitor = new HangMonitor(TimeSpan.FromMinutes(1.0), $"Perforce connection ({_uniqueId})", hangLogger);

			_buffers = new PinnedBuffer[bufferCount];
			for (int idx = 0; idx < bufferCount; idx++)
			{
				_buffers[idx] = new PinnedBuffer(bufferSize);
				_writeBuffers.TryAdd(_buffers[idx]);
			}

			_onBufferReadyInst = new OnBufferReadyFn(OnBufferReady);
			_onBufferReadyFnPtr = Marshal.GetFunctionPointerForDelegate(_onBufferReadyInst);

			_responseCompleteEvent = new ManualResetEvent(false);

			_backgroundThread = new Thread(BackgroundThreadProc);
			_backgroundThread.IsBackground = true;
			_backgroundThread.Start();

			logger.LogTrace("Created Perforce connection {ConnectionId} (server: {ServerAndPort}, user: {UserName}, client: {ClientName})", _uniqueId, settings.ServerAndPort, settings.UserName, settings.ClientName);
		}

		/// <summary>
		/// Create an instance of the native client
		/// </summary>
		public static async Task<NativePerforceConnection> CreateAsync(IPerforceSettings settings, ILogger logger, Tracer? tracer = null)
		{
			NativePerforceConnection? connection = null;
			try
			{
				connection = new NativePerforceConnection(settings, logger, tracer);
				await connection.ConnectAsync();
				return connection;
			}
			catch
			{
				connection?.Dispose();
				throw;
			}
		}

		/// <summary>
		/// Check whether the native client is supported on the current platform
		/// </summary>
		/// <returns></returns>
		public static bool IsSupported()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				return RuntimeInformation.OSArchitecture != Architecture.Arm64;
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return RuntimeInformation.OSArchitecture != Architecture.Arm64;
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				return RuntimeInformation.OSArchitecture != Architecture.Arm64;
			}
			return false;
		}

		void GetNextWriteBuffer(NativeWriteBuffer nativeWriteBuffer, int minSize)
		{
			_stallTimer.Start();
			_hangMonitor.Tick();

			PinnedBuffer buffer = _writeBuffers.Take();
			if (buffer.MaxLength < minSize)
			{
				buffer.Resize(minSize);
			}

			nativeWriteBuffer._data = buffer.BasePtr;
			nativeWriteBuffer._maxLength = buffer.Data.Length;
			nativeWriteBuffer._maxCount = Int32.MaxValue;

			_hangMonitor.Tick();
			_stallTimer.Stop();
		}

		/// <summary>
		/// Finalizer
		/// </summary>
		~NativePerforceConnection()
		{
			Dispose(false);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		void Dispose(bool disposing)
		{
			if (disposing && !_disposed)
			{
				using IDisposable scope = _hangMonitor.Start("Disposing");

				Logger.LogTrace("Disposing Perforce connection {ConnectionId}", _uniqueId);

				if (_backgroundThread != null)
				{
					_requests.Add(null);
					_requests.CompleteAdding();

					_backgroundThread.Join();
					_backgroundThread = null!;
				}

				_requests.Dispose();
			}

			if (_client != IntPtr.Zero)
			{
				Client_Destroy(_client);
				_client = IntPtr.Zero;
			}

			if (disposing && !_disposed)
			{
				using IDisposable scope = _hangMonitor.Start("Disposing");

				_writeBuffers.Dispose();

				_responseCompleteEvent.Dispose();

				foreach (PinnedBuffer pinnedBuffer in _buffers)
				{
					pinnedBuffer.Dispose();
				}

				_hangMonitor.Dispose();
				_disposed = true;
			}
		}

		/// <summary>
		/// Gets the amount of time stalled waiting for an output buffer in the last command
		/// </summary>
		public TimeSpan StallTime => _stallTimer.Elapsed;

		/// <summary>
		/// Initializes the connection, throwing an error on failure
		/// </summary>
		private async Task ConnectAsync()
		{
			PerforceError? error = await TryConnectAsync();
			if (error != null)
			{
				throw new PerforceException(error);
			}
		}

		/// <summary>
		/// Tries to initialize the connection
		/// </summary>
		/// <returns>Error returned when attempting to connect</returns>
		private async Task<PerforceError?> TryConnectAsync()
		{
			await using Response response = new(this);

			NativeSettings? nativeSettings = null;
			if (Settings != null)
			{
				nativeSettings = new NativeSettings();
				nativeSettings._serverAndPort = Settings.ServerAndPort;
				nativeSettings._userName = Settings.UserName;
				nativeSettings._password = Settings.Password;
				nativeSettings._hostName = Settings.HostName;
				nativeSettings._clientName = Settings.ClientName;
				nativeSettings._appName = Settings.AppName;
				nativeSettings._appVersion = Settings.AppVersion;
			}

			_requests.Add((() =>
			{
				NativeWriteBuffer writeBuffer = new();
				GetNextWriteBuffer(writeBuffer, 1);
				_client = Client_Create(nativeSettings, writeBuffer, _onBufferReadyFnPtr);
			}, response));

			List<PerforceResponse> records = await ((IPerforceOutput)response).ReadResponsesAsync(null, default);
			if (records.Count != 1)
			{
				throw new PerforceException("Expected at least one record to be returned from Init() call.");
			}

			PerforceError? error = records[0].Error;
			if (error == null)
			{
				throw new PerforceException("Unexpected response from init call");
			}
			if (error.Severity != PerforceSeverityCode.Empty)
			{
				return error;
			}
			return null;
		}

		/// <summary>
		/// Background thread which sequences requests on a single thread. The Perforce API isn't async aware, but is primarily
		/// I/O bound, so this thread will mostly be idle. All processing C#-side is done using async tasks, whereas this thread
		/// blocks.
		/// </summary>
		void BackgroundThreadProc()
		{
			for (; ; )
			{
				try
				{
					(Action Action, Response Response)? request = _requests.Take();
					if (request == null)
					{
						break;
					}

					_currentResponse = request.Value.Response;
					_responseCompleteEvent.Reset();

					_stallTimer.Reset();
					request.Value.Action();

					_currentResponse._readBuffers.Writer.TryComplete();
					_responseCompleteEvent.WaitOne();

					_currentResponse = null;
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Exception while processing Perforce commands: {Message}", ex.Message);
					throw;
				}
			}
		}

		/// <summary>
		/// Callback for switching buffers
		/// </summary>
		/// <param name="readBuffer">The complete buffer</param>
		/// <param name="writeBuffer">Receives information about the next buffer to write to</param>
		void OnBufferReady(NativeReadBuffer readBuffer, [In, Out] NativeWriteBuffer writeBuffer)
		{
			PinnedBuffer buffer = _buffers.First(x => x.BasePtr == readBuffer._data);
			buffer.CheckGuardBytes();

			_currentResponse!._readBuffers.Writer.TryWrite((buffer, readBuffer._length)); // Unbounded; will always succeed

			int nextWriteSize = 0;
			if (readBuffer._length == 0)
			{
				nextWriteSize = readBuffer._maxLength * 2;
			}

			GetNextWriteBuffer(writeBuffer, nextWriteSize);
		}

		/// <inheritdoc/>
		public IPerforceOutput Command(string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, string? promptResponse, bool interceptIo)
		{
			byte[]? specData = null;
			if (inputData != null)
			{
				specData = Encoding.UTF8.GetBytes(FormatSpec(inputData));
			}

			List<string> allArguments = [.. arguments];
			if (fileArguments != null)
			{
				allArguments.AddRange(fileArguments);
			}

			Response response = new(this);
			_requests.Add((() => ExecCommand(command, allArguments, specData, promptResponse, interceptIo), response));
			return response;
		}

		private void ExecCommand(string command, List<string> args, byte[]? inputData, string? promptResponse, bool interceptIo)
		{
			StringBuilder argList = new();
			for (int idx = 0; idx < args.Count; idx++)
			{
				CommandLineArguments.Append(argList, args[idx]);
				if (argList.Length > 512)
				{
					argList.Append($" {{+{args.Count - idx - 1} more}}");
					break;
				}
			}

			Stopwatch timer = Stopwatch.StartNew();
			Logger.LogTrace("Conn {ConnectionId}: {Command} {Args}", _uniqueId, command, argList.ToString());

			using IDisposable scope = _hangMonitor.Start($"{command} {argList}");
			long initialAllocatedSize = GetAllocatedSize();

			List<IntPtr> nativeArgs = [];
			try
			{
				foreach (string arg in args)
				{
					byte[] data = Encoding.UTF8.GetBytes(arg);
					Array.Resize(ref data, data.Length + 1);
					IntPtr nativeArg = Marshal.AllocHGlobal(data.Length);
					Marshal.Copy(data, 0, nativeArg, data.Length);
					nativeArgs.Add(nativeArg);
				}

				byte[]? promptResponseBytes = null;
				if (promptResponse != null)
				{
					promptResponseBytes = new byte[Encoding.UTF8.GetByteCount(promptResponse) + 1];
					Encoding.UTF8.GetBytes(promptResponse, promptResponseBytes);
				}

				try
				{
					Client_Command(_client, command, nativeArgs.Count, [.. nativeArgs], inputData, inputData?.Length ?? 0, promptResponseBytes, interceptIo);
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Exception while executing command {Command} {Args}: {Message}", command, argList.ToString(), ex.Message);
					throw;
				}
			}
			finally
			{
				for (int idx = 0; idx < nativeArgs.Count; idx++)
				{
					Marshal.FreeHGlobal(nativeArgs[idx]);
				}
			}

			const long WarnSize = 16 * 1024 * 1024;

			long allocatedSize = GetAllocatedSize();
			if (allocatedSize > initialAllocatedSize && allocatedSize > WarnSize)
			{
				Logger.LogTrace("Native P4 connection allocation increased {InitialAllocatedSize} -> {AllocatedSize} ({Command} {Args})", initialAllocatedSize, allocatedSize, command, argList.ToString());
			}

			Logger.LogTrace("Conn {ConnectionId}: Request completed in {Time}ms", _uniqueId, timer.ElapsedMilliseconds);
		}

		long GetAllocatedSize()
		{
			long allocatedSize = 0;
			foreach (PinnedBuffer buffer in _buffers)
			{
				allocatedSize += buffer.MaxLength;
			}
			return allocatedSize;
		}

		/// <summary>
		/// Converts a Python marshalled data blob to a spec definition
		/// </summary>
		static string FormatSpec(byte[] inputData)
		{
			int pos = 0;

			List<KeyValuePair<Utf8String, PerforceValue>> rows = [];
			if (!PerforceOutput.ParseRecord(inputData, ref pos, rows))
			{
				throw new PerforceException("Unable to parse input data as record");
			}
			if (pos != inputData.Length)
			{
				throw new PerforceException("Garbage after end of spec data");
			}

			StringBuilder result = new();
			foreach ((Utf8String key, PerforceValue value) in rows)
			{
				string[] valueLines = value.ToString().Split('\n');
				if (valueLines.Length == 1)
				{
					result.AppendLine($"{key}: {valueLines[0]}");
				}
				else
				{
					result.AppendLine($"{key}:");
					foreach (string valueLine in valueLines)
					{
						result.AppendLine($"\t{valueLine}");
					}
				}
				result.AppendLine();
			}

			return result.ToString();
		}

		/// <inheritdoc/>
		public PerforceRecord CreateRecord(List<KeyValuePair<string, object>> fields) => PerforceRecord.FromFields(fields, false);
	}
}
