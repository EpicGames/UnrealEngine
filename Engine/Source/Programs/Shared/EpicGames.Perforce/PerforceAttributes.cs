// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Attributes for fields that should be deserialized from P4 tags
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class PerforceTagAttribute : Attribute
	{
		/// <summary>
		/// The tag name
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Whether this tag is required for a valid record
		/// </summary>
		public bool Optional { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the tag</param>
		public PerforceTagAttribute(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Specifies the name of an enum when converted into a P4 string
	/// </summary>
	[AttributeUsage(AttributeTargets.Field)]
	public sealed class PerforceEnumAttribute : Attribute
	{
		/// <summary>
		/// Name of the enum value
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the serialized value</param>
		public PerforceEnumAttribute(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// When attached to a list field, indicates that a list of structures can be included in the record
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class PerforceRecordListAttribute : Attribute
	{
	}
}
