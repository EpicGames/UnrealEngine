// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Text.RegularExpressions;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents a change view from a clientspec or stream
	/// </summary>
	public class PerforceChangeView
	{
		/// <summary>
		/// List of entries for this change view
		/// </summary>
		public List<PerforceChangeViewEntry> Entries { get; } = [];

		/// <summary>
		/// Determines if a file revision is visible 
		/// </summary>
		/// <param name="depotFile">Path to the depot file</param>
		/// <param name="change">Change number of the file</param>
		public bool IsVisible(string depotFile, int change)
		{
			bool visible = true;
			foreach (PerforceChangeViewEntry entry in Entries)
			{
				if (entry.Matches(depotFile))
				{
					visible = change <= entry.Change;
				}
			}
			return visible;
		}

		/// <summary>
		/// Parse a change view from a specification
		/// </summary>
		/// <param name="lines">Lines for the change view</param>
		/// <param name="ignoreCase">Whether this will be evaluated in the context of a case-insensitive server</param>
		public static PerforceChangeView Parse(IEnumerable<string> lines, bool ignoreCase)
		{
			PerforceChangeView changeView = new();
			foreach (string line in lines)
			{
				if (!String.IsNullOrWhiteSpace(line))
				{
					// Skip over lines with unsupported formatting (generally @label instead of @changenumber)
					try
					{
						changeView.Entries.Add(new PerforceChangeViewEntry(line, ignoreCase));
					}
					catch (FormatException)
					{
						// don't do anything
					}
				}
			}
			return changeView;
		}
	}

	/// <summary>
	/// Entry for a change view
	/// </summary>
	public class PerforceChangeViewEntry
	{
		/// <summary>
		/// Pattern to match
		/// </summary>
		public string Path { get; }

		/// <summary>
		/// Change to be imported at
		/// </summary>
		public long Change { get; }

		readonly Regex _pathRegex;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceChangeViewEntry(string pattern, bool ignoreCase)
		{
			int atIdx = pattern.LastIndexOf('@');
			Path = pattern.Substring(0, atIdx);
			Change = Int64.Parse(pattern.AsSpan(atIdx + 1), NumberStyles.None);

			string regexPattern = Regex.Escape(Path);
			regexPattern = regexPattern.Replace(@"\?", ".", StringComparison.Ordinal);
			regexPattern = regexPattern.Replace(@"\.\.\.", ".*", StringComparison.Ordinal);

			_pathRegex = new Regex($"^{regexPattern}$", ignoreCase ? RegexOptions.IgnoreCase : RegexOptions.None);
		}

		/// <summary>
		/// Determine
		/// </summary>
		/// <param name="depotPath"></param>
		/// <returns></returns>
		public bool Matches(string depotPath) => _pathRegex.IsMatch(depotPath);
	}
}
