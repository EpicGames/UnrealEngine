// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Wraps a call to a p4.exe child process, and allows reading data from it
	/// </summary>
	public sealed class PerforceChildProcess : IPerforceOutput, IDisposable
	{
		/// <summary>
		/// The process group
		/// </summary>
		ManagedProcessGroup _childProcessGroup;

		/// <summary>
		/// The child process instance
		/// </summary>
		ManagedProcess _childProcess;

		/// <summary>
		/// Scope object for tracing
		/// </summary>
		ITraceSpan _scope;

		/// <summary>
		/// The buffer data
		/// </summary>
		byte[] _buffer;

		/// <summary>
		/// End of the valid portion of the buffer (exclusive)
		/// </summary>
		int _bufferEnd;

		/// <inheritdoc/>
		public ReadOnlyMemory<byte> Data => _buffer.AsMemory(0, _bufferEnd);

		/// <summary>
		/// Temp file containing file arguments
		/// </summary>
		string? _tempFileName;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="command"></param>
		/// <param name="arguments">Command line arguments</param>
		/// <param name="fileArguments">File arguments, which may be placed in a response file</param>
		/// <param name="inputData">Input data to pass to the child process</param>
		/// <param name="globalOptions"></param>
		/// <param name="logger">Logging device</param>
		public PerforceChildProcess(string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, IReadOnlyList<string> globalOptions, ILogger logger)
		{
			string perforceFileName = GetExecutable();

			List<string> fullArguments = ["-G", .. globalOptions];
			if (fileArguments != null)
			{
				_tempFileName = Path.GetTempFileName();
				File.WriteAllLines(_tempFileName, fileArguments);
				fullArguments.Add($"-x{_tempFileName}");
			}
			fullArguments.Add(command);
			fullArguments.AddRange(arguments);

			string fullArgumentList = CommandLineArguments.Join(fullArguments);
			logger.LogDebug("Running {Executable} {Arguments}", perforceFileName, fullArgumentList);

			_scope = TraceSpan.Create(command, service: "perforce");
			_scope.AddMetadata("arguments", fullArgumentList);

			_childProcessGroup = new ManagedProcessGroup();
			_childProcess = new ManagedProcess(_childProcessGroup, perforceFileName, fullArgumentList, null, null, inputData, ProcessPriorityClass.Normal);

			_buffer = new byte[64 * 1024];
		}

		/// <summary>
		/// Gets the path to the P4.EXE executable
		/// </summary>
		/// <returns>Path to the executable</returns>
		public static string GetExecutable()
		{
			string perforceFileName = "p4.exe";
			if (!OperatingSystem.IsWindows())
			{
				string[] p4Paths = [
					"/usr/bin/p4", // Default path
					"/opt/homebrew/bin/p4", // Apple Silicon Homebrew Path
					"/usr/local/bin/p4", // Apple Intel Homebrew Path
					"/home/linuxbrew/.linuxbrew/bin/p4" ]; // Linux Homebrew path
				foreach (string path in p4Paths)
				{
					if (File.Exists(path))
					{
						perforceFileName = path;
						break;
					}
				}
				if (perforceFileName == "p4.exe")
				{
					perforceFileName = "p4"; // Fall back to PATH lookup
				}
			}
			return perforceFileName;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
		{
			Dispose();
			return new ValueTask(Task.CompletedTask);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_childProcess != null)
			{
				_childProcess.Dispose();
				_childProcess = null!;
			}
			if (_childProcessGroup != null)
			{
				_childProcessGroup.Dispose();
				_childProcessGroup = null!;
			}
			if (_scope != null)
			{
				_scope.Dispose();
				_scope = null!;
			}
			if (_tempFileName != null)
			{
				try
				{
					File.Delete(_tempFileName);
				}
				catch { }
				_tempFileName = null;
			}
		}

		/// <inheritdoc/>
		public async Task<bool> ReadAsync(CancellationToken cancellationToken)
		{
			// Update the buffer contents
			if (_bufferEnd == _buffer.Length)
			{
				Array.Resize(ref _buffer, Math.Min(_buffer.Length + (32 * 1024 * 1024), _buffer.Length * 2));
			}

			// Try to read more data
			int prevBufferEnd = _bufferEnd;
			while (_bufferEnd < _buffer.Length)
			{
				int count = await _childProcess!.ReadAsync(_buffer, _bufferEnd, _buffer.Length - _bufferEnd, cancellationToken);
				if (count == 0)
				{
					break;
				}
				_bufferEnd += count;
			}
			return _bufferEnd > prevBufferEnd;
		}

		/// <inheritdoc/>
		public void Discard(int numBytes)
		{
			if (numBytes > 0)
			{
				Array.Copy(_buffer, numBytes, _buffer, 0, _bufferEnd - numBytes);
				_bufferEnd -= numBytes;
			}
		}

		/// <summary>
		/// Reads all output from the child process as a string
		/// </summary>
		/// <param name="cancellationToken">Cancellation token to abort the read</param>
		/// <returns>Exit code and output from the process</returns>
		public async Task<Tuple<bool, string>> TryReadToEndAsync(CancellationToken cancellationToken)
		{
			using MemoryStream stream = new();

			while (await ReadAsync(cancellationToken))
			{
				ReadOnlyMemory<byte> dataCopy = Data;
				stream.Write(dataCopy.Span);
				Discard(dataCopy.Length);
			}

			string @string = Encoding.Default.GetString(stream.ToArray());
			return Tuple.Create(_childProcess.ExitCode == 0, @string);
		}
	}
}
