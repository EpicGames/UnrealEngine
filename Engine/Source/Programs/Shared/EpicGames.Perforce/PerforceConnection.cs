// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.CompilerServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Stores settings for communicating with a Perforce server.
	/// </summary>
	public sealed class PerforceConnection : IPerforceConnection
	{
		/// <summary>
		/// Default tracer. A custom tracer can also be passed to constructor or factory methods.
		/// </summary>
		public static Tracer DefaultTracer { get; } = TracerProvider.Default.GetTracer("Perforce");

		/// <summary>
		/// Create a new Perforce connection
		/// </summary>
		/// <param name="logger">Interface for logging</param>
		/// <param name="tracer">Tracer</param>
		public static Task<IPerforceConnection> CreateAsync(ILogger logger, Tracer? tracer = null)
		{
			return CreateAsync(PerforceSettings.Default, logger, tracer);
		}

		/// <summary>
		/// Create a new Perforce connection
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="logger">Interface for logging</param>
		/// /// <param name="tracer">Tracer</param>
		public static Task<IPerforceConnection> CreateAsync(string? serverAndPort, string? userName, ILogger logger, Tracer? tracer = null)
		{
			return CreateAsync(CombineSettings(serverAndPort, userName), logger, tracer);
		}

		/// <summary>
		/// Create a new Perforce connection
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="clientName">The client name</param>
		/// <param name="logger">Interface for logging</param>
		/// <param name="tracer">Tracer</param>
		public static Task<IPerforceConnection> CreateAsync(string? serverAndPort, string? userName, string? clientName, ILogger logger, Tracer? tracer = null)
		{
			return CreateAsync(CombineSettings(serverAndPort, userName, clientName), logger, tracer);
		}

		/// <summary>
		/// Create a new Perforce connection
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="clientName">The client name</param>
		/// <param name="appName"></param>
		/// <param name="appVersion"></param>
		/// <param name="logger">Interface for logging</param>
		/// <param name="tracer">Tracer</param>
		public static Task<IPerforceConnection> CreateAsync(string? serverAndPort, string? userName, string? clientName, string? appName, string? appVersion, ILogger logger, Tracer? tracer = null)
		{
			return CreateAsync(CombineSettings(serverAndPort, userName, clientName, appName, appVersion), logger, tracer);
		}

		/// <summary>
		/// Create a new Perforce connection
		/// </summary>
		/// <param name="settings">Settings for the connection</param>
		/// <param name="logger">Interface for logging</param>
		/// <param name="tracer">Tracer</param>
		public static async Task<IPerforceConnection> CreateAsync(IPerforceSettings settings, ILogger logger, Tracer? tracer = null)
		{
			if (settings.PreferNativeClient && NativePerforceConnection.IsSupported())
			{
				return await NativePerforceConnection.CreateAsync(settings, logger, tracer);
			}
			else
			{
				return new PerforceConnection(settings, logger, tracer);
			}
		}

		static PerforceSettings CombineSettings(string? serverAndPort, string? userName, string? clientName = null, string? appName = null, string? appVersion = null)
		{
			PerforceSettings settings = new(PerforceSettings.Default);
			if (serverAndPort != null)
			{
				settings.ServerAndPort = serverAndPort;
			}
			if (userName != null)
			{
				settings.UserName = userName;
			}
			if (clientName != null)
			{
				settings.ClientName = clientName;
			}
			if (appName != null)
			{
				settings.AppName = appName;
			}
			if (appVersion != null)
			{
				settings.AppVersion = appVersion;
			}
			return settings;
		}

		#region Legacy implementation

		/// <inheritdoc/>
		public IPerforceSettings Settings
		{
			get
			{
				PerforceSettings settings = new(PerforceEnvironment.Default);
				if (ServerAndPort != null)
				{
					settings.ServerAndPort = ServerAndPort;
				}
				if (UserName != null)
				{
					settings.UserName = UserName;
				}
				if (ClientName != null)
				{
					settings.ClientName = ClientName;
				}
				if (AppName != null)
				{
					settings.AppName = AppName;
				}
				if (AppVersion != null)
				{
					settings.AppVersion = AppVersion;
				}
				return settings;
			}
		}

		/// <summary>
		/// The current server and port
		/// </summary>
		public string? ServerAndPort
		{
			get;
			set;
		}

		/// <summary>
		/// The current user name
		/// </summary>
		public string? UserName
		{
			get;
			set;
		}

		/// <summary>
		/// The current host name
		/// </summary>
		public string? HostName
		{
			get;
			set;
		}

		/// <summary>
		/// The current client name
		/// </summary>
		public string? ClientName
		{
			get;
			set;
		}

		/// <summary>
		/// Name of this application, reported to server through -zprog arguments
		/// </summary>
		public string? AppName
		{
			get;
			set;
		}

		/// <summary>
		/// Version of this application, reported to server through -zversion arguments
		/// </summary>
		public string? AppVersion
		{
			get;
			set;
		}

		/// <summary>
		/// Additional options to append to the command line
		/// </summary>
		public List<string> GlobalOptions { get; } = [];

		/// <inheritdoc/>
		public ILogger Logger { get; set; }

		/// <inheritdoc/>
		public Tracer Tracer { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">Interface for logging</param>
		public PerforceConnection(string? serverAndPort, string? userName, ILogger logger, Tracer? tracer = null)
			: this(serverAndPort, userName, null, logger, tracer)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="clientName">The client name</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">Interface for logging</param>
		public PerforceConnection(string? serverAndPort, string? userName, string? clientName, ILogger logger, Tracer? tracer = null)
			: this(serverAndPort, userName, clientName, null, null, logger, tracer)
		{
			AssemblyName entryAssemblyName = Assembly.GetEntryAssembly()!.GetName();
			if (entryAssemblyName.Name != null)
			{
				AppName = entryAssemblyName.Name;
				AppVersion = entryAssemblyName.Version?.ToString() ?? String.Empty;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="serverAndPort">The server address and port</param>
		/// <param name="userName">The user name</param>
		/// <param name="clientName">The client name</param>
		/// <param name="appName"></param>
		/// <param name="appVersion"></param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">Interface for logging</param>
		public PerforceConnection(string? serverAndPort, string? userName, string? clientName, string? appName, string? appVersion, ILogger logger, Tracer? tracer = null)
		{
			ServerAndPort = String.IsNullOrWhiteSpace(serverAndPort) ? null : serverAndPort;
			UserName = String.IsNullOrWhiteSpace(userName) ? null : userName;
			ClientName = String.IsNullOrWhiteSpace(clientName) ? null : clientName;
			AppName = String.IsNullOrWhiteSpace(appName) ? null : appName;
			AppVersion = String.IsNullOrWhiteSpace(appVersion) ? null : appVersion;
			Logger = logger;
			Tracer = tracer ?? DefaultTracer;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceConnection(IPerforceSettings settings, ILogger logger, Tracer? tracer = null)
			: this(settings.ServerAndPort, settings.UserName, settings.ClientName, settings.AppName, settings.AppVersion, logger, tracer)
		{
			HostName = settings.HostName;
		}

		/// <summary>
		/// Constructor 
		/// </summary>
		/// <param name="other">Connection to copy settings from</param>
		public PerforceConnection(PerforceConnection other)
			: this(other.ServerAndPort, other.UserName, other.ClientName, other.AppName, other.AppVersion, other.Logger, other.Tracer)
		{
			GlobalOptions.AddRange(other.GlobalOptions);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
		}

		List<string> GetGlobalArguments()
		{
			List<string> arguments = [];
			if (ServerAndPort != null)
			{
				arguments.Add($"-p{ServerAndPort}");
			}
			if (UserName != null)
			{
				arguments.Add($"-u{UserName}");
			}
			if (HostName != null)
			{
				arguments.Add($"-H{HostName}");
			}
			if (ClientName != null)
			{
				arguments.Add($"-c{ClientName}");
			}
			if (AppName != null)
			{
				arguments.Add($"-zprog={AppName}");
			}
			if (AppVersion != null)
			{
				arguments.Add($"-zversion={AppVersion}");
			}
			arguments.AddRange(GlobalOptions);
			return arguments;
		}

		/// <inheritdoc/>
		public IPerforceOutput Command(string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, string? promptResponse, bool interceptIo)
		{
			if (promptResponse != null)
			{
				inputData = Encoding.UTF8.GetBytes(promptResponse);
			}
			if (interceptIo)
			{
				throw new NotSupportedException($"{nameof(interceptIo)} option is not supported through legacy Perforce client");
			}
			return new PerforceChildProcess(command, arguments, fileArguments, inputData, GetGlobalArguments(), Logger);
		}

		/// <summary>
		/// Sets an environment variable
		/// </summary>
		/// <param name="name">Name of the variable to set</param>
		/// <param name="value">Value for the variable</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public async Task SetAsync(string name, string value, CancellationToken cancellationToken = default)
		{
			Tuple<bool, string> result = await TrySetAsync(name, value, cancellationToken);
			if (!result.Item1)
			{
				throw new PerforceException(result.Item2);
			}
		}

		/// <summary>
		/// Sets an environment variable
		/// </summary>
		/// <param name="name">Name of the variable to set</param>
		/// <param name="value">Value for the variable</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public async Task<Tuple<bool, string>> TrySetAsync(string name, string value, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"{name}={value}"];

			using (PerforceChildProcess childProcess = new("set", arguments, null, null, GetGlobalArguments(), Logger))
			{
				return await childProcess.TryReadToEndAsync(cancellationToken);
			}
		}

		/// <inheritdoc/>
		public PerforceRecord CreateRecord(List<KeyValuePair<string, object>> fields) => PerforceRecord.FromFields(fields, true);

		#endregion
	}

	/// <summary>
	/// Extension methods for <see cref="PerforceConnection"/>
	/// </summary>
	public static class PerforceConnectionExtensions
	{
		/// <summary>
		/// Create a new connection with a different client
		/// </summary>
		/// <param name="perforce"></param>
		/// <param name="clientName"></param>
		/// <returns></returns>
		public static Task<IPerforceConnection> WithClientAsync(this IPerforceConnection perforce, string? clientName)
		{
			PerforceSettings settings = new(perforce.Settings) { ClientName = clientName };
			return PerforceConnection.CreateAsync(settings, perforce.Logger, perforce.Tracer);
		}

		/// <summary>
		/// Create a new connection with a different client
		/// </summary>
		/// <param name="perforce"></param>
		/// <returns></returns>
		public static Task<IPerforceConnection> WithoutClientAsync(this IPerforceConnection perforce)
		{
			return WithClientAsync(perforce, null);
		}

		#region Command wrappers

		private static void AddAttributes(IPerforceConnection perforce, TelemetrySpan span, string commandType, string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments)
		{
			span.SetAttribute("epic.perforce.command_type", commandType)
				.SetAttribute("epic.perforce.args", arguments.ToArray())
				.SetAttribute("epic.perforce.file_args", fileArguments?.ToArray())
				.SetAttribute("epic.perforce.native", perforce.Settings.PreferNativeClient)
				.SetAttribute("epic.perforce.client", perforce.Settings.ClientName)
				.SetAttribute("epic.perforce.username", perforce.Settings.UserName)
				.SetAttribute("epic.perforce.server_port", perforce.Settings.ServerAndPort)
				.SetAttribute("epic.perforce.hostname", perforce.Settings.HostName);

			// Useful attributes but mostly hinting for Datadog
			span.SetAttribute("type", "db")
				.SetAttribute("db.query", command + " " + String.Join(' ', arguments))
				.SetAttribute("operation.name", command)
				.SetAttribute("service.name", "Perforce");
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="perforce">The Perforce connection</param>
		/// <param name="command">Command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="statRecordType">The type of records to return for "stat" responses</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static Task<List<PerforceResponse>> CommandAsync(this IPerforceConnection perforce, string command, IReadOnlyList<string> arguments, byte[]? inputData, Type? statRecordType, CancellationToken cancellationToken = default)
		{
			return CommandAsync(perforce, command, arguments, null, inputData, statRecordType, cancellationToken);
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="perforce">The Perforce connection</param>
		/// <param name="command">Command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="fileArguments">File arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="statRecordType">The type of records to return for "stat" responses</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async Task<List<PerforceResponse>> CommandAsync(this IPerforceConnection perforce, string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, Type? statRecordType, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = perforce.Tracer.StartActiveSpan(command);
			AddAttributes(perforce, span, "normal", command, arguments, fileArguments);

			await using IPerforceOutput response = perforce.Command(command, arguments, fileArguments, inputData, null, false);
			List<PerforceResponse> responses = await response.ReadResponsesAsync(statRecordType, cancellationToken);

			span.SetAttribute("epic.perforce.res.count", responses.Count);
			return responses;
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="perforce">The Perforce connection</param>
		/// <param name="command">Command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="fileArguments">File arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="statRecordType">The type of records to return for "stat" responses</param>
		/// <param name="interceptIo">Whether to intercept Io operations and return them in the response output</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async IAsyncEnumerable<PerforceResponse> StreamCommandAsync(this IPerforceConnection perforce, string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, Type? statRecordType, bool interceptIo, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan span = perforce.Tracer.StartActiveSpan(command);
			AddAttributes(perforce, span, "stream", command, arguments, fileArguments);
			int c = 0;
#pragma warning disable CA1849 // Call async methods when in an async method
			await using (IPerforceOutput output = perforce.Command(command, arguments, fileArguments, inputData, null, interceptIo))
			{
				await foreach (PerforceResponse response in output.ReadStreamingResponsesAsync(statRecordType, cancellationToken))
				{
					c++;
					yield return response;
				}
			}
#pragma warning restore CA1849 // Call async methods when in an async method
			span.SetAttribute("epic.perforce.res.count", c);
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="perforce">The Perforce connection</param>
		/// <param name="command">Command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="fileArguments">File arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async IAsyncEnumerable<PerforceResponse<T>> StreamCommandAsync<T>(this IPerforceConnection perforce, string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments = null, byte[]? inputData = null, [EnumeratorCancellation] CancellationToken cancellationToken = default) where T : class
		{
			using TelemetrySpan span = perforce.Tracer.StartActiveSpan(command);
			AddAttributes(perforce, span, "stream", command, arguments, fileArguments);
			int c = 0;
#pragma warning disable CA1849 // Call async methods when in an async method
			await using (IPerforceOutput output = perforce.Command(command, arguments, fileArguments, inputData, null, false))
			{
				Type statRecordType = typeof(T);
				await foreach (PerforceResponse response in output.ReadStreamingResponsesAsync(statRecordType, cancellationToken))
				{
					c++;
					yield return new PerforceResponse<T>(response);
				}
			}
#pragma warning restore CA1849 // Call async methods when in an async method
			span.SetAttribute("epic.perforce.res.count", c);
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="perforce">The Perforce connection</param>
		/// <param name="command">Command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="handleRecord">Delegate used to handle each record</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async Task RecordCommandAsync(this IPerforceConnection perforce, string command, IReadOnlyList<string> arguments, byte[]? inputData, Action<PerforceRecord> handleRecord, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = perforce.Tracer.StartActiveSpan(command);
			AddAttributes(perforce, span, "record", command, arguments, null);
#pragma warning disable CA1849 // Call async methods when in an async method
			await using IPerforceOutput response = perforce.Command(command, arguments, null, inputData, null, false);
			await response.ReadRecordsAsync(handleRecord, cancellationToken);
#pragma warning restore CA1849 // Call async methods when in an async method
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="command">The command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="fileArguments">Arguments which can be passed into a -x argument</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async Task<PerforceResponseList<T>> CommandAsync<T>(this IPerforceConnection connection, string command, IReadOnlyList<string> arguments, IReadOnlyList<string>? fileArguments, byte[]? inputData, CancellationToken cancellationToken = default) where T : class
		{
			List<PerforceResponse> responses = await connection.CommandAsync(command, arguments, fileArguments, inputData, typeof(T), cancellationToken);

			PerforceResponseList<T> typedResponses = [];
			foreach (PerforceResponse response in responses)
			{
				typedResponses.Add(new PerforceResponse<T>(response));
			}
			return typedResponses;
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="command">The command to execute</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="inputData">Input data to pass to Perforce</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static Task<PerforceResponseList<T>> CommandAsync<T>(this IPerforceConnection connection, string command, List<string> arguments, byte[]? inputData, CancellationToken cancellationToken = default) where T : class
		{
			return CommandAsync<T>(connection, command, arguments, null, inputData, cancellationToken);
		}

		/// <summary>
		/// Execute a command and parse the response
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="command">The command to execute</param>
		/// <param name="commonArguments">Arguments for the command</param>
		/// <param name="batchedArguments">Arguments to pass to the command in batches</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of objects returned by the server</returns>
		public static async Task<PerforceResponseList<T>> BatchedCommandAsync<T>(IPerforceConnection connection, string command, IReadOnlyList<string> commonArguments, IReadOnlyList<string> batchedArguments, CancellationToken cancellationToken = default) where T : class
		{
			PerforceResponseList<T> responses = [];
			for (int fileSpecIdx = 0; fileSpecIdx < batchedArguments.Count;)
			{
				List<string> arguments = [.. commonArguments];

				const int PerArgumentExtra = 5;
				int length = (command.Length + PerArgumentExtra) + arguments.Sum(x => x.Length + PerArgumentExtra);

				for (; fileSpecIdx < batchedArguments.Count && length < 4096; fileSpecIdx++)
				{
					arguments.Add(batchedArguments[fileSpecIdx]);
					length += batchedArguments[fileSpecIdx].Length + PerArgumentExtra;
				}
				responses.AddRange(await CommandAsync<T>(connection, command, arguments, null, cancellationToken));
			}
			return responses;
		}

		/// <summary>
		/// Attempts to execute the given command, returning the results from the server or the first PerforceResponse object.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="command">The command to execute</param>
		/// <param name="arguments">Arguments for the command.</param>
		/// <param name="inputData">Input data for the command.</param>
		/// <param name="statRecordType">Type of element to return in the response</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either an object of type T or error.</returns>
		static async Task<PerforceResponse> SingleResponseCommandAsync(IPerforceConnection connection, string command, IReadOnlyList<string> arguments, byte[]? inputData, Type? statRecordType, CancellationToken cancellationToken = default)
		{
			List<PerforceResponse> responses = await connection.CommandAsync(command, arguments, inputData, statRecordType, cancellationToken);
			if (responses.Count != 1)
			{
				for (int idx = 0; idx < responses.Count; idx++)
				{
					connection.Logger.LogInformation("Unexpected response {Idx}: {Text}", idx, responses[idx].ToString());
				}
				throw new PerforceException("Expected one result from 'p4 {0}', got {1}", command, responses.Count);
			}
			return responses[0];
		}

		/// <summary>
		/// Attempts to execute the given command, returning the results from the server or the first PerforceResponse object.
		/// </summary>
		/// <typeparam name="T">Type of record to parse</typeparam>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="command">The command to execute</param>
		/// <param name="arguments">Arguments for the command.</param>
		/// <param name="inputData">Input data for the command.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either an object of type T or error.</returns>
		public static async Task<PerforceResponse<T>> SingleResponseCommandAsync<T>(IPerforceConnection connection, string command, List<string> arguments, byte[]? inputData, CancellationToken cancellationToken = default) where T : class
		{
			return new PerforceResponse<T>(await SingleResponseCommandAsync(connection, command, arguments, inputData, typeof(T), cancellationToken));
		}

		#endregion

		#region p4 add

		/// <summary>
		/// Adds files to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileSpecList">Files to be added</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<AddRecord>> AddAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecList, CancellationToken cancellationToken = default)
		{
			return (await TryAddAsync(connection, changeNumber, null, AddOptions.None, fileSpecList, cancellationToken)).Data;
		}

		/// <summary>
		/// Adds files to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecList">Files to be added</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<AddRecord>> AddAsync(this IPerforceConnection connection, int changeNumber, string? fileType, AddOptions options, FileSpecList fileSpecList, CancellationToken cancellationToken = default)
		{
			return (await TryAddAsync(connection, changeNumber, fileType, options, fileSpecList, cancellationToken)).Data;
		}

		/// <summary>
		/// Adds files to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileSpecList">Files to be added</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<AddRecord>> TryAddAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecList, CancellationToken cancellationToken = default)
		{
			return TryAddAsync(connection, changeNumber, null, AddOptions.None, fileSpecList, cancellationToken);
		}

		/// <summary>
		/// Adds files to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileNames">Files to be added</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<AddRecord>> TryAddAsync(this IPerforceConnection connection, int changeNumber, string? fileType, AddOptions options, FileSpecList fileNames, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & AddOptions.DowngradeToAdd) != 0)
			{
				arguments.Add("-d");
			}
			if ((options & AddOptions.IncludeWildcards) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & AddOptions.NoIgnore) != 0)
			{
				arguments.Add("-I");
			}
			if ((options & AddOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if (fileType != null)
			{
				arguments.Add($"-t{fileType}");
			}

			return BatchedCommandAsync<AddRecord>(connection, "add", arguments, fileNames.List, cancellationToken);
		}

		#endregion

		#region p4 change

		/// <summary>
		/// Creates a changelist with the p4 change command. 
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">Information for the change to create. The number field should be left set to -1.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>The changelist number, or an error.</returns>
		public static async Task<ChangeRecord> CreateChangeAsync(this IPerforceConnection connection, ChangeRecord record, CancellationToken cancellationToken = default)
		{
			return (await TryCreateChangeAsync(connection, record, cancellationToken)).Data;
		}

		/// <summary>
		/// Creates a changelist with the p4 change command. 
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">Information for the change to create. The number field should be left set to -1.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>The changelist number, or an error.</returns>
		public static async Task<PerforceResponse<ChangeRecord>> TryCreateChangeAsync(this IPerforceConnection connection, ChangeRecord record, CancellationToken cancellationToken = default)
		{
			if (record.Number != -1)
			{
				throw new PerforceException("'Number' field should be set to -1 to create a new changelist.");
			}

			PerforceResponse response = await SingleResponseCommandAsync(connection, "change", ["-i"], SerializeRecord(connection, record), null, cancellationToken);

			PerforceError? error = response.Error;
			if (error != null)
			{
				return new PerforceResponse<ChangeRecord>(error);
			}

			PerforceInfo? info = response.Info;
			if (info == null)
			{
				throw new PerforceException("Unexpected info response from change command: {0}", response);
			}

			Match match = Regex.Match(info.Data, @"^Change (\d+) created");
			if (!match.Success)
			{
				throw new PerforceException("Unexpected info response from change command: {0}", response);
			}

			record.Number = Int32.Parse(match.Groups[1].Value);
			return new PerforceResponse<ChangeRecord>(record);
		}

		/// <summary>
		/// Updates an existing changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="record">Information for the change to create. The number field should be left set to zero.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>The changelist number, or an error.</returns>
		public static async Task UpdateChangeAsync(this IPerforceConnection connection, UpdateChangeOptions options, ChangeRecord record, CancellationToken cancellationToken = default)
		{
			(await TryUpdateChangeAsync(connection, options, record, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Updates an existing changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="record">Information for the change to create. The number field should be left set to zero.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>The changelist number, or an error.</returns>
		public static Task<PerforceResponse> TryUpdateChangeAsync(this IPerforceConnection connection, UpdateChangeOptions options, ChangeRecord record, CancellationToken cancellationToken = default)
		{
			if (record.Number == -1)
			{
				throw new PerforceException("'Number' field must be set to update a changelist.");
			}

			List<string> arguments = ["-i"];
			if ((options & UpdateChangeOptions.Force) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & UpdateChangeOptions.Submitted) != 0)
			{
				arguments.Add("-u");
			}

			return SingleResponseCommandAsync(connection, "change", arguments, connection.SerializeRecord(record), null, cancellationToken);
		}

		/// <summary>
		/// Deletes a changelist (p4 change -d)
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">Changelist number to delete</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task DeleteChangeAsync(this IPerforceConnection connection, DeleteChangeOptions options, int changeNumber, CancellationToken cancellationToken = default)
		{
			(await TryDeleteChangeAsync(connection, options, changeNumber, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Deletes a changelist (p4 change -d)
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">Changelist number to delete</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TryDeleteChangeAsync(this IPerforceConnection connection, DeleteChangeOptions options, int changeNumber, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-d"];
			if ((options & DeleteChangeOptions.Submitted) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & DeleteChangeOptions.BeforeRenumber) != 0)
			{
				arguments.Add("-O");
			}
			arguments.Add($"{changeNumber}");

			return SingleResponseCommandAsync(connection, "change", arguments, null, null, cancellationToken);
		}

		/// <summary>
		/// Gets a changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">Changelist number to retrieve. -1 is the default changelist for this workspace.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<ChangeRecord> GetChangeAsync(this IPerforceConnection connection, GetChangeOptions options, int changeNumber, CancellationToken cancellationToken = default)
		{
			return (await TryGetChange(connection, options, changeNumber, cancellationToken)).Data;
		}

		/// <summary>
		/// Gets a changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">Changelist number to retrieve. -1 is the default changelist for this workspace.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse<ChangeRecord>> TryGetChange(this IPerforceConnection connection, GetChangeOptions options, int changeNumber, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o"];
			if ((options & GetChangeOptions.BeforeRenumber) != 0)
			{
				arguments.Add("-O");
			}
			if (changeNumber != -1)
			{
				arguments.Add($"{changeNumber}");
			}

			return SingleResponseCommandAsync<ChangeRecord>(connection, "change", arguments, null, cancellationToken);
		}

		/// <summary>
		/// Serializes a change record to a byte array
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="input">The record to serialize</param>
		/// <returns>Serialized record</returns>
		static byte[] SerializeRecord(this IPerforceConnection connection, ChangeRecord input)
		{
			List<KeyValuePair<string, object>> nameToValue =
			[
				input.Number == -1 ? new KeyValuePair<string, object>("Change", "new") : new KeyValuePair<string, object>("Change", input.Number.ToString()),
			];
			if (input.Type != ChangeType.Unspecified)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Type", input.Type.ToString()));
			}
			if (input.User != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("User", input.User));
			}
			if (input.Client != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Client", input.Client));
			}
			if (input.Description != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Description", input.Description));
			}
			if (input.Status != ChangeStatus.All)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Status", PerforceReflection.GetEnumText(typeof(ChangeStatus), input.Status)));
			}
			if (input.Files.Count > 0)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Files", input.Files));
			}
			return connection.CreateRecord(nameToValue).Serialize();
		}

		#endregion

		#region p4 changes

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static async Task<List<ChangesRecord>> GetChangesAsync(this IPerforceConnection connection, ChangesOptions options, int maxChanges, ChangeStatus status, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryGetChangesAsync(connection, options, maxChanges, status, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="clientName">List only changes made from the named client workspace.</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="userName">List only changes made by the named user</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static async Task<List<ChangesRecord>> GetChangesAsync(this IPerforceConnection connection, ChangesOptions options, string? clientName, int maxChanges, ChangeStatus status, string? userName, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryGetChangesAsync(connection, options, clientName, maxChanges, status, userName, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="clientName">List only changes made from the named client workspace.</param>
		/// <param name="minChangeNumber">The minimum changelist number</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="userName">List only changes made by the named user</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static async Task<List<ChangesRecord>> GetChangesAsync(this IPerforceConnection connection, ChangesOptions options, string? clientName, int minChangeNumber, int maxChanges, ChangeStatus status, string? userName, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryGetChangesAsync(connection, options, clientName, minChangeNumber, maxChanges, status, userName, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static Task<PerforceResponseList<ChangesRecord>> TryGetChangesAsync(this IPerforceConnection connection, ChangesOptions options, int maxChanges, ChangeStatus status, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryGetChangesAsync(connection, options, null, maxChanges, status, null, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="clientName">List only changes made from the named client workspace.</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="userName">List only changes made by the named user</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static Task<PerforceResponseList<ChangesRecord>> TryGetChangesAsync(this IPerforceConnection connection, ChangesOptions options, string? clientName, int maxChanges, ChangeStatus status, string? userName, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryGetChangesAsync(connection, options, clientName, -1, maxChanges, status, userName, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Enumerates changes on the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="clientName">List only changes made from the named client workspace.</param>
		/// <param name="minChangeNumber">The minimum changelist number</param>
		/// <param name="maxChanges">List only the highest numbered changes</param>
		/// <param name="status">Limit the list to the changelists with the given status (pending, submitted or shelved)</param>
		/// <param name="userName">List only changes made by the named user</param>
		/// <param name="fileSpecs">Paths to query changes for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server.</returns>
		public static Task<PerforceResponseList<ChangesRecord>> TryGetChangesAsync(this IPerforceConnection connection, ChangesOptions options, string? clientName, int minChangeNumber, int maxChanges, ChangeStatus status, string? userName, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & ChangesOptions.IncludeIntegrations) != 0)
			{
				arguments.Add("-i");
			}
			if ((options & ChangesOptions.IncludeTimes) != 0)
			{
				arguments.Add("-t");
			}
			if ((options & ChangesOptions.LongOutput) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & ChangesOptions.Reverse) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & ChangesOptions.TruncatedLongOutput) != 0)
			{
				arguments.Add("-L");
			}
			if ((options & ChangesOptions.IncludeRestricted) != 0)
			{
				arguments.Add("-f");
			}
			if (clientName != null)
			{
				arguments.Add($"-c{clientName}");
			}
			if (minChangeNumber != -1)
			{
				arguments.Add($"-e{minChangeNumber}");
			}
			if (maxChanges != -1)
			{
				arguments.Add($"-m{maxChanges}");
			}
			if (status != ChangeStatus.All)
			{
				arguments.Add($"-s{PerforceReflection.GetEnumText(typeof(ChangeStatus), status)}");
			}
			if (userName != null)
			{
				arguments.Add($"-u{userName}");
			}
			if (fileSpecs != FileSpecList.Any) // Queries are slower with a path specification
			{
				arguments.AddRange(fileSpecs.List);
			}

			return CommandAsync<ChangesRecord>(connection, "changes", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 clean

		/// <summary>
		/// Cleans the workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<CleanRecord>> CleanAsync(this IPerforceConnection connection, CleanOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryCleanAsync(connection, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Cleans the workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<CleanRecord>> TryCleanAsync(this IPerforceConnection connection, CleanOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & CleanOptions.Edited) != 0)
			{
				arguments.Add("-e");
			}
			if ((options & CleanOptions.Added) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & CleanOptions.Deleted) != 0)
			{
				arguments.Add("-d");
			}
			if ((options & CleanOptions.Preview) != 0)
			{
				arguments.Add("-n");
			}
			if ((options & CleanOptions.NoIgnoreChecking) != 0)
			{
				arguments.Add("-I");
			}
			if ((options & CleanOptions.LocalSyntax) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & CleanOptions.ModifiedTimes) != 0)
			{
				arguments.Add("-m");
			}
			arguments.AddRange(fileSpecs.List);

			PerforceResponseList<CleanRecord> records = await CommandAsync<CleanRecord>(connection, "clean", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 client

		/// <summary>
		/// Creates a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">The client record</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task CreateClientAsync(this IPerforceConnection connection, ClientRecord record, CancellationToken cancellationToken = default)
		{
			(await TryCreateClientAsync(connection, record, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Creates a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">The client record</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TryCreateClientAsync(this IPerforceConnection connection, ClientRecord record, CancellationToken cancellationToken = default)
		{
			return TryUpdateClientAsync(connection, record, cancellationToken);
		}

		/// <summary>
		/// Creates a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">The client record</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task UpdateClientAsync(this IPerforceConnection connection, ClientRecord record, CancellationToken cancellationToken = default)
		{
			(await TryUpdateClientAsync(connection, record, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Update a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">The client record</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TryUpdateClientAsync(this IPerforceConnection connection, ClientRecord record, CancellationToken cancellationToken = default)
		{
			return SingleResponseCommandAsync(connection, "client", ["-i"], connection.SerializeRecord(record), null, cancellationToken);
		}

		/// <summary>
		/// Deletes a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="clientName">Name of the client</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task DeleteClientAsync(this IPerforceConnection connection, DeleteClientOptions options, string clientName, CancellationToken cancellationToken = default)
		{
			(await TryDeleteClientAsync(connection, options, clientName, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Deletes a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="clientName">Name of the client</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TryDeleteClientAsync(this IPerforceConnection connection, DeleteClientOptions options, string clientName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-d"];
			if ((options & DeleteClientOptions.Force) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & DeleteClientOptions.DeleteShelved) != 0)
			{
				arguments.Add("-Fs");
			}
			arguments.Add(clientName);

			return SingleResponseCommandAsync(connection, "client", arguments, null, null, cancellationToken);
		}

		/// <summary>
		/// Changes the stream associated with a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">The new stream to be associated with the client</param>
		/// <param name="options">Options for this command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task SwitchClientToStreamAsync(this IPerforceConnection connection, string streamName, SwitchClientOptions options, CancellationToken cancellationToken = default)
		{
			(await TrySwitchClientToStreamAsync(connection, streamName, options, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Changes the stream associated with a client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">The new stream to be associated with the client</param>
		/// <param name="options">Options for this command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TrySwitchClientToStreamAsync(this IPerforceConnection connection, string streamName, SwitchClientOptions options, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-s"];
			if ((options & SwitchClientOptions.IgnoreOpenFiles) != 0)
			{
				arguments.Add("-f");
			}
			arguments.Add($"-S{streamName}");

			return SingleResponseCommandAsync(connection, "client", arguments, null, null, cancellationToken);
		}

		/// <summary>
		/// Changes a client to mirror a template
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="templateName">The new stream to be associated with the client</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task SwitchClientToTemplateAsync(this IPerforceConnection connection, string templateName, CancellationToken cancellationToken = default)
		{
			(await TrySwitchClientToTemplateAsync(connection, templateName, cancellationToken)).EnsureSuccess();
		}

		/// <summary>
		/// Changes a client to mirror a template
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="templateName">The new stream to be associated with the client</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponse> TrySwitchClientToTemplateAsync(this IPerforceConnection connection, string templateName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-s", $"-t{templateName}"];

			return SingleResponseCommandAsync(connection, "client", arguments, null, null, cancellationToken);
		}

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="clientName">Name of the client. Specify null for the current client.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static async Task<ClientRecord> GetClientAsync(this IPerforceConnection connection, string? clientName, CancellationToken cancellationToken = default)
		{
			return (await TryGetClientAsync(connection, clientName, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="clientName">Name of the client. Specify null for the current client.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponse<ClientRecord>> TryGetClientAsync(this IPerforceConnection connection, string? clientName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o"];
			if (clientName != null)
			{
				arguments.Add(clientName);
			}
			return SingleResponseCommandAsync<ClientRecord>(connection, "client", arguments, null, cancellationToken);
		}

		/// <summary>
		/// Queries the view for a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">Name of the stream.</param>
		/// <param name="changeNumber">Changelist at which to query the stream view</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static async Task<ClientRecord> GetStreamViewAsync(this IPerforceConnection connection, string streamName, int changeNumber, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamViewAsync(connection, streamName, changeNumber, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries the view for a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="clientName">Optional client name</param>
		/// <param name="streamName">Name of the stream.</param>
		/// <param name="changeNumber">Changelist at which to query the stream view</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static async Task<ClientRecord> GetStreamViewAsync(this IPerforceConnection connection, string? clientName, string streamName, int changeNumber, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamViewAsync(connection, clientName, streamName, changeNumber, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries the view for a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">Name of the stream.</param>
		/// <param name="changeNumber">Changelist at which to query the stream view</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponse<ClientRecord>> TryGetStreamViewAsync(this IPerforceConnection connection, string streamName, int changeNumber, CancellationToken cancellationToken = default)
		{
			return TryGetStreamViewAsync(connection, null, streamName, changeNumber, cancellationToken);
		}

		/// <summary>
		/// Queries the view for a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="clientName">Optional client name</param>
		/// <param name="streamName">Name of the stream.</param>
		/// <param name="changeNumber">Changelist at which to query the stream view</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponse<ClientRecord>> TryGetStreamViewAsync(this IPerforceConnection connection, string? clientName, string streamName, int changeNumber, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o", $"-S{streamName}"];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}

			if (!String.IsNullOrEmpty(clientName))
			{
				arguments.Add(clientName);
			}

			return SingleResponseCommandAsync<ClientRecord>(connection, "client", arguments, null, cancellationToken);
		}

		/// <summary>
		/// Serializes a client record to a byte array
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="input">The input record</param>
		/// <returns>Serialized record data</returns>
		static byte[] SerializeRecord(this IPerforceConnection connection, ClientRecord input)
		{
			List<KeyValuePair<string, object>> nameToValue = [];
			if (input.Name != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Client", input.Name));
			}
			if (input.Owner != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Owner", input.Owner));
			}
			if (input.Host != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Host", input.Host));
			}
			if (input.Description != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Description", input.Description));
			}
			if (input.Root != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Root", input.Root));
			}
			if (input.Options != ClientOptions.None)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Options", PerforceReflection.GetEnumText(typeof(ClientOptions), input.Options)));
			}
			if (input.SubmitOptions != ClientSubmitOptions.Unspecified)
			{
				nameToValue.Add(new KeyValuePair<string, object>("SubmitOptions", PerforceReflection.GetEnumText(typeof(ClientSubmitOptions), input.SubmitOptions)));
			}
			if (input.LineEnd != ClientLineEndings.Unspecified)
			{
				nameToValue.Add(new KeyValuePair<string, object>("LineEnd", PerforceReflection.GetEnumText(typeof(ClientLineEndings), input.LineEnd)));
			}
			if (input.Type != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Type", input.Type));
			}
			if (input.Stream != null)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Stream", input.Stream));
			}
			if (input.View.Count > 0)
			{
				nameToValue.Add(new KeyValuePair<string, object>("View", input.View));
			}
			return connection.CreateRecord(nameToValue).Serialize();
		}

		#endregion

		#region p4 clients

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="userName">List only client workspaces owned by this user.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static async Task<List<ClientsRecord>> GetClientsAsync(this IPerforceConnection connection, ClientsOptions options, string? userName, CancellationToken cancellationToken = default)
		{
			return (await TryGetClientsAsync(connection, options, userName, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="userName">List only client workspaces owned by this user.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponseList<ClientsRecord>> TryGetClientsAsync(this IPerforceConnection connection, ClientsOptions options, string? userName, CancellationToken cancellationToken = default)
		{
			return TryGetClientsAsync(connection, options, null, -1, null, userName, cancellationToken);
		}

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="filter">List only client workspaces matching filter. Treated as case sensitive if <ref>ClientsOptions.CaseSensitiveFilter</ref> is set.</param>
		/// <param name="maxResults">Limit the number of results to return. -1 for all.</param>
		/// <param name="stream">List client workspaces associated with the specified stream.</param>
		/// <param name="userName">List only client workspaces owned by this user.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static async Task<List<ClientsRecord>> GetClientsAsync(this IPerforceConnection connection, ClientsOptions options, string? filter, int maxResults, string? stream, string? userName, CancellationToken cancellationToken = default)
		{
			return (await TryGetClientsAsync(connection, options, filter, maxResults, stream, userName, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries the current client definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="filter">List only client workspaces matching filter. Treated as case sensitive if <ref>ClientsOptions.CaseSensitiveFilter</ref> is set.</param>
		/// <param name="maxResults">Limit the number of results to return. -1 for all.</param>
		/// <param name="stream">List client workspaces associated with the specified stream.</param>
		/// <param name="userName">List only client workspaces owned by this user.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponseList<ClientsRecord>> TryGetClientsAsync(this IPerforceConnection connection, ClientsOptions options, string? filter, int maxResults, string? stream, string? userName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & ClientsOptions.All) != 0)
			{
				arguments.Add("-a");
			}
			if (filter != null)
			{
				if ((options & ClientsOptions.CaseSensitiveFilter) != 0)
				{
					arguments.Add("-e");
					arguments.Add(filter);
				}
				else
				{
					arguments.Add("-E");
					arguments.Add(filter);
				}
			}
			if (maxResults != -1)
			{
				arguments.Add($"-m{maxResults}");
			}
			if (stream != null)
			{
				arguments.Add($"-S{stream}");
			}
			if ((options & ClientsOptions.WithTimes) != 0)
			{
				arguments.Add("-t");
			}
			if (userName != null)
			{
				arguments.Add("-u");
				arguments.Add(userName);
			}
			if ((options & ClientsOptions.Unloaded) != 0)
			{
				arguments.Add("-U");
			}
			return CommandAsync<ClientsRecord>(connection, "clients", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 delete

		/// <summary>
		/// Execute the 'delete' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The change to add deleted files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<DeleteRecord>> DeleteAsync(this IPerforceConnection connection, int changeNumber, DeleteOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryDeleteAsync(connection, changeNumber, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'delete' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The change to add deleted files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<PerforceResponseList<DeleteRecord>> TryDeleteAsync(this IPerforceConnection connection, int changeNumber, DeleteOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & DeleteOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if ((options & DeleteOptions.KeepWorkspaceFiles) != 0)
			{
				arguments.Add("-k");
			}
			if ((options & DeleteOptions.WithoutSyncing) != 0)
			{
				arguments.Add("-v");
			}

			PerforceResponseList<DeleteRecord> records = await BatchedCommandAsync<DeleteRecord>(connection, "delete", arguments, fileSpecs.List, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 depot

		/// <summary>
		/// Queries the current depot definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="depotName">Name of the client. Specify null for the current client.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<DepotRecord> GetDepotAsync(this IPerforceConnection connection, string depotName, CancellationToken cancellationToken = default)
		{
			return TryGetDepotAsync(connection, depotName, cancellationToken).UnwrapAsync();
		}

		/// <summary>
		/// Queries the current depot definition
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="depotName">Name of the client. Specify null for the current client.</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a client record or error code</returns>
		public static Task<PerforceResponse<DepotRecord>> TryGetDepotAsync(this IPerforceConnection connection, string depotName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o", depotName];
			return SingleResponseCommandAsync<DepotRecord>(connection, "depot", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 describe

		/// <summary>
		/// Describes a single changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist number to retrieve description for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a describe record or error code</returns>
		public static async Task<DescribeRecord> DescribeAsync(this IPerforceConnection connection, int changeNumber, CancellationToken cancellationToken = default)
		{
			return (await TryDescribeAsync(connection, changeNumber, cancellationToken)).Data;
		}

		/// <summary>
		/// Describes a single changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist number to retrieve description for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a describe record or error code</returns>
		public static async Task<PerforceResponse<DescribeRecord>> TryDescribeAsync(this IPerforceConnection connection, int changeNumber, CancellationToken cancellationToken = default)
		{
			PerforceResponseList<DescribeRecord> records = await TryDescribeAsync(connection, [changeNumber], cancellationToken);
			if (records.Count != 1)
			{
				throw new PerforceException("Expected only one record returned from p4 describe command, got {0}", records.Count);
			}
			return records[0];
		}

		/// <summary>
		/// Describes a single changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxNumFiles">Maximum number of files to return</param>
		/// <param name="changeNumber">The changelist number to retrieve description for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a describe record or error code</returns>
		public static async Task<DescribeRecord> DescribeAsync(this IPerforceConnection connection, DescribeOptions options, int maxNumFiles, int changeNumber, CancellationToken cancellationToken = default)
		{
			PerforceResponse<DescribeRecord> response = await TryDescribeAsync(connection, options, maxNumFiles, changeNumber, cancellationToken);
			return response.Data;
		}

		/// <summary>
		/// Describes a single changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxNumFiles">Maximum number of files to return</param>
		/// <param name="changeNumber">The changelist number to retrieve description for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; either a describe record or error code</returns>
		public static async Task<PerforceResponse<DescribeRecord>> TryDescribeAsync(this IPerforceConnection connection, DescribeOptions options, int maxNumFiles, int changeNumber, CancellationToken cancellationToken = default)
		{
			PerforceResponseList<DescribeRecord> records = await TryDescribeAsync(connection, options, maxNumFiles, [changeNumber], cancellationToken);
			if (records.Count != 1)
			{
				throw new PerforceException("Expected only one record returned from p4 describe command, got {0}", records.Count);
			}
			return records[0];
		}

		/// <summary>
		/// Describes a set of changelists
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumbers">The changelist numbers to retrieve descriptions for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<DescribeRecord>> DescribeAsync(this IPerforceConnection connection, int[] changeNumbers, CancellationToken cancellationToken = default)
		{
			return (await TryDescribeAsync(connection, changeNumbers, cancellationToken)).Data;
		}

		/// <summary>
		/// Describes a set of changelists
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumbers">The changelist numbers to retrieve descriptions for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static Task<PerforceResponseList<DescribeRecord>> TryDescribeAsync(this IPerforceConnection connection, int[] changeNumbers, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-s"];
			foreach (int changeNumber in changeNumbers)
			{
				arguments.Add($"{changeNumber}");
			}
			return CommandAsync<DescribeRecord>(connection, "describe", arguments, null, cancellationToken);
		}

		/// <summary>
		/// Describes a set of changelists
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxNumFiles">Maximum number of files to return</param>
		/// <param name="changeNumbers">The changelist numbers to retrieve descriptions for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<DescribeRecord>> DescribeAsync(this IPerforceConnection connection, DescribeOptions options, int maxNumFiles, int[] changeNumbers, CancellationToken cancellationToken = default)
		{
			return (await TryDescribeAsync(connection, options, maxNumFiles, changeNumbers, cancellationToken)).Data;
		}

		/// <summary>
		/// Describes a set of changelists
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxNumFiles">Maximum number of files to return</param>
		/// <param name="changeNumbers">The changelist numbers to retrieve descriptions for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static Task<PerforceResponseList<DescribeRecord>> TryDescribeAsync(this IPerforceConnection connection, DescribeOptions options, int maxNumFiles, int[] changeNumbers, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-s"];
			if ((options & DescribeOptions.ShowDescriptionForRestrictedChanges) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & DescribeOptions.Identity) != 0)
			{
				arguments.Add("-I");
			}
			if (maxNumFiles != -1)
			{
				arguments.Add($"-m{maxNumFiles}");
			}
			if ((options & DescribeOptions.OriginalChangeNumber) != 0)
			{
				arguments.Add("-O");
			}
			if ((options & DescribeOptions.Shelved) != 0)
			{
				arguments.Add("-S");
			}
			foreach (int changeNumber in changeNumbers)
			{
				arguments.Add($"{changeNumber}");
			}
			return CommandAsync<DescribeRecord>(connection, "describe", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 dirs

		/// <summary>
		/// List directories under a depot path
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="stream">List directories mapped for the specified stream</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<DirsRecord>> GetDirsAsync(this IPerforceConnection connection, DirsOptions options, string? stream, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryGetDirsAsync(connection, options, stream, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// List directories under a depot path
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="stream">List directories mapped for the specified stream</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<DirsRecord>> TryGetDirsAsync(this IPerforceConnection connection, DirsOptions options, string? stream, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & DirsOptions.OnlyMapped) != 0)
			{
				arguments.Add("-C");
			}
			if ((options & DirsOptions.IncludeDeleted) != 0)
			{
				arguments.Add("-D");
			}
			if ((options & DirsOptions.OnlyHave) != 0)
			{
				arguments.Add("-H");
			}
			if (stream != null)
			{
				arguments.Add("-S");
				arguments.Add(stream);
			}
			if ((options & DirsOptions.IgnoreCase) != 0)
			{
				arguments.Add("-i");
			}

			return CommandAsync<DirsRecord>(connection, "dirs", arguments, fileSpecs.List, null, cancellationToken);
		}

		#endregion

		#region p4 edit

		/// <summary>
		/// Opens files for edit
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<EditRecord>> EditAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryEditAsync(connection, changeNumber, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Opens files for edit
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<EditRecord>> TryEditAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryEditAsync(connection, changeNumber, null, EditOptions.None, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Opens files for edit
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<EditRecord>> EditAsync(this IPerforceConnection connection, int changeNumber, string? fileType, EditOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryEditAsync(connection, changeNumber, fileType, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Opens files for edit
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be opened for edit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<EditRecord>> TryEditAsync(this IPerforceConnection connection, int changeNumber, string? fileType, EditOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & EditOptions.KeepWorkspaceFiles) != 0)
			{
				arguments.Add("-k");
			}
			if ((options & EditOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if (fileType != null)
			{
				arguments.Add($"-t{fileType}");
			}

			return BatchedCommandAsync<EditRecord>(connection, "edit", arguments, fileSpecs.List, cancellationToken);
		}

		#endregion

		#region p4 filelog

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<FileLogRecord>> FileLogAsync(this IPerforceConnection connection, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryFileLogAsync(connection, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static Task<PerforceResponseList<FileLogRecord>> TryFileLogAsync(this IPerforceConnection connection, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFileLogAsync(connection, -1, -1, options, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="maxChanges">Number of changelists to show. Ignored if zero or negative.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<FileLogRecord>> FileLogAsync(this IPerforceConnection connection, int maxChanges, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryFileLogAsync(connection, maxChanges, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="maxChanges">Number of changelists to show. Ignored if zero or negative.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static Task<PerforceResponseList<FileLogRecord>> TryFileLogAsync(this IPerforceConnection connection, int maxChanges, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFileLogAsync(connection, -1, maxChanges, options, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Show only files modified by this changelist. Ignored if zero or negative.</param>
		/// <param name="maxChanges">Number of changelists to show. Ignored if zero or negative.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<List<FileLogRecord>> FileLogAsync(this IPerforceConnection connection, int changeNumber, int maxChanges, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryFileLogAsync(connection, changeNumber, maxChanges, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'filelog' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Show only files modified by this changelist. Ignored if zero or negative.</param>
		/// <param name="maxChanges">Number of changelists to show. Ignored if zero or negative.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static async Task<PerforceResponseList<FileLogRecord>> TryFileLogAsync(this IPerforceConnection connection, int changeNumber, int maxChanges, FileLogOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			// Build the argument list
			List<string> arguments = [];
			if (changeNumber > 0)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & FileLogOptions.ContentHistory) != 0)
			{
				arguments.Add("-h");
			}
			if ((options & FileLogOptions.FollowAcrossBranches) != 0)
			{
				arguments.Add("-i");
			}
			if ((options & FileLogOptions.FullDescriptions) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & FileLogOptions.LongDescriptions) != 0)
			{
				arguments.Add("-L");
			}
			if (maxChanges > 0)
			{
				arguments.Add($"-m{maxChanges}");
			}
			if ((options & FileLogOptions.DoNotFollowPromotedTaskStreams) != 0)
			{
				arguments.Add("-p");
			}
			if ((options & FileLogOptions.IgnoreNonContributoryIntegrations) != 0)
			{
				arguments.Add("-s");
			}

			// Always include times to simplify parsing
			arguments.Add("-t");

			// Append all the arguments
			PerforceResponseList<FileLogRecord> records = await BatchedCommandAsync<FileLogRecord>(connection, "filelog", arguments, fileSpecs.List, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 files

		/// <summary>
		/// Execute the 'files' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static async Task<List<FilesRecord>> FilesAsync(this IPerforceConnection connection, FilesOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryFilesAsync(connection, options, -1, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'files' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static Task<PerforceResponseList<FilesRecord>> TryFilesAsync(this IPerforceConnection connection, FilesOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFilesAsync(connection, options, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'files' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Maximum number of results to return. Ignored if less than or equal to zero.</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static async Task<List<FilesRecord>> FilesAsync(this IPerforceConnection connection, FilesOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryFilesAsync(connection, options, maxFiles, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'files' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Maximum number of results to return. Ignored if less than or equal to zero.</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static async Task<PerforceResponseList<FilesRecord>> TryFilesAsync(this IPerforceConnection connection, FilesOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & FilesOptions.AllRevisions) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & FilesOptions.LimitToArchiveDepots) != 0)
			{
				arguments.Add("-A");
			}
			if ((options & FilesOptions.ExcludeDeleted) != 0)
			{
				arguments.Add("-e");
			}
			if ((options & FilesOptions.IgnoreCase) != 0)
			{
				arguments.Add("-i");
			}
			if (maxFiles > 0)
			{
				arguments.Add($"-m{maxFiles}");
			}

			PerforceResponseList<FilesRecord> records = await BatchedCommandAsync<FilesRecord>(connection, "files", arguments, fileSpecs.List, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 fstat

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<FStatRecord> FStatAsync(this IPerforceConnection connection, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, FStatOptions.None, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<FStatRecord> FStatAsync(this IPerforceConnection connection, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, options, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<FStatRecord>> TryFStatAsync(this IPerforceConnection connection, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, -1, options, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="maxFiles">Produce fstat output for only the first max files.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<FStatRecord> FStatAsync(this IPerforceConnection connection, int maxFiles, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, maxFiles, options, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="maxFiles">Produce fstat output for only the first max files.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<FStatRecord>> TryFStatAsync(this IPerforceConnection connection, int maxFiles, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, -1, -1, null, null, maxFiles, options, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="afterChangeNumber">Return only files affected after the given changelist number.</param>
		/// <param name="onlyChangeNumber">Return only files affected by the given changelist number.</param>
		/// <param name="filter">List only those files that match the criteria specified.</param>
		/// <param name="fields">Fields to return in the output</param>
		/// <param name="maxFiles">Produce fstat output for only the first max files.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<FStatRecord> FStatAsync(this IPerforceConnection connection, int afterChangeNumber, int onlyChangeNumber, string? filter, string? fields, int maxFiles, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryFStatAsync(connection, afterChangeNumber, onlyChangeNumber, filter, fields, maxFiles, options, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'fstat' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="afterChangeNumber">Return only files affected after the given changelist number.</param>
		/// <param name="onlyChangeNumber">Return only files affected by the given changelist number.</param>
		/// <param name="filter">List only those files that match the criteria specified.</param>
		/// <param name="fields">Fields to return in the output</param>
		/// <param name="maxFiles">Produce fstat output for only the first max files.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<FStatRecord>> TryFStatAsync(this IPerforceConnection connection, int afterChangeNumber, int onlyChangeNumber, string? filter, string? fields, int maxFiles, FStatOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			// Build the argument list
			List<string> arguments = [];
			if (afterChangeNumber != -1)
			{
				arguments.Add($"-c{afterChangeNumber}");
			}
			if (onlyChangeNumber != -1)
			{
				arguments.Add($"-e{onlyChangeNumber}");
			}
			if (filter != null)
			{
				arguments.Add("-F");
				arguments.Add(filter);
			}
			if (fields != null)
			{
				arguments.Add("-T");
				arguments.Add(fields);
			}
			if ((options & FStatOptions.ReportDepotSyntax) != 0)
			{
				arguments.Add("-L");
			}
			if ((options & FStatOptions.AllRevisions) != 0)
			{
				arguments.Add("-Of");
			}
			if ((options & FStatOptions.IncludeFileSizes) != 0)
			{
				arguments.Add("-Ol");
			}
			if ((options & FStatOptions.ClientFileInPerforceSyntax) != 0)
			{
				arguments.Add("-Op");
			}
			if ((options & FStatOptions.ShowPendingIntegrations) != 0)
			{
				arguments.Add("-Or");
			}
			if ((options & FStatOptions.ShortenOutput) != 0)
			{
				arguments.Add("-Os");
			}
			if ((options & FStatOptions.ReverseOrder) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & FStatOptions.OnlyMapped) != 0)
			{
				arguments.Add("-Rc");
			}
			if ((options & FStatOptions.OnlyHave) != 0)
			{
				arguments.Add("-Rh");
			}
			if ((options & FStatOptions.OnlyOpenedBeforeHead) != 0)
			{
				arguments.Add("-Rn");
			}
			if ((options & FStatOptions.OnlyOpenInWorkspace) != 0)
			{
				arguments.Add("-Ro");
			}
			if ((options & FStatOptions.OnlyOpenAndResolved) != 0)
			{
				arguments.Add("-Rr");
			}
			if ((options & FStatOptions.OnlyShelved) != 0)
			{
				arguments.Add("-Rs");
			}
			if ((options & FStatOptions.OnlyUnresolved) != 0)
			{
				arguments.Add("-Ru");
			}
			if ((options & FStatOptions.SortByDate) != 0)
			{
				arguments.Add("-Sd");
			}
			if ((options & FStatOptions.SortByHaveRevision) != 0)
			{
				arguments.Add("-Sh");
			}
			if ((options & FStatOptions.SortByHeadRevision) != 0)
			{
				arguments.Add("-Sr");
			}
			if ((options & FStatOptions.SortByFileSize) != 0)
			{
				arguments.Add("-Ss");
			}
			if ((options & FStatOptions.SortByFileType) != 0)
			{
				arguments.Add("-St");
			}
			if ((options & FStatOptions.IncludeFilesInUnloadDepot) != 0)
			{
				arguments.Add("-U");
			}
			if (maxFiles > 0)
			{
				arguments.Add($"-m{maxFiles}");
			}

			// Execute the command
			IAsyncEnumerable<PerforceResponse<FStatRecord>> records = StreamCommandAsync<FStatRecord>(connection, "fstat", arguments, fileSpecs.List, null, cancellationToken);
			records = records.Where(x => x.Error == null || x.Error.Generic != PerforceGenericCode.Empty);

			if (onlyChangeNumber != -1)
			{
				records = records.Where(x => !x.Succeeded || x.Data.Description == null);
			}
			return records;
		}

		#endregion

		#region p4 have

		/// <summary>
		/// Determine files currently synced to the client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Files to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of file records</returns>
		public static IAsyncEnumerable<HaveRecord> HaveAsync(this IPerforceConnection connection, FileSpecList fileSpec, CancellationToken cancellationToken = default)
		{
			return TryHaveAsync(connection, fileSpec, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Determine files currently synced to the client
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Files to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of file records</returns>
		public static IAsyncEnumerable<PerforceResponse<HaveRecord>> TryHaveAsync(this IPerforceConnection connection, FileSpecList fileSpec, CancellationToken cancellationToken = default)
		{
			IAsyncEnumerable<PerforceResponse<HaveRecord>> records = StreamCommandAsync<HaveRecord>(connection, "have", [], fileSpec.List, null, cancellationToken);
			records = records.Where(x => x.Error == null || x.Error.Generic != PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 info

		/// <summary>
		/// Execute the 'info' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; an InfoRecord or error code</returns>
		public static async Task<InfoRecord> GetInfoAsync(this IPerforceConnection connection, InfoOptions options, CancellationToken cancellationToken = default)
		{
			return (await TryGetInfoAsync(connection, options, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'info' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server; an InfoRecord or error code</returns>
		public static Task<PerforceResponse<InfoRecord>> TryGetInfoAsync(this IPerforceConnection connection, InfoOptions options, CancellationToken cancellationToken = default)
		{
			// Build the argument list
			List<string> arguments = [];
			if ((options & InfoOptions.ShortOutput) != 0)
			{
				arguments.Add("-s");
			}
			return SingleResponseCommandAsync<InfoRecord>(connection, "info", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 login

		/// <summary>
		/// Log in to the server
		/// </summary>
		/// <param name="connection"></param>
		/// <param name="password"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public static async Task<LoginRecord> LoginAsync(this IPerforceConnection connection, string password, CancellationToken cancellationToken = default)
		{
			return (await TryLoginAsync(connection, password, cancellationToken)).Data;
		}

		/// <summary>
		/// Attempts to log in to the server
		/// </summary>
		/// <param name="connection">Connection to use</param>
		/// <param name="password"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public static Task<PerforceResponse<LoginRecord>> TryLoginAsync(this IPerforceConnection connection, string? password, CancellationToken cancellationToken = default)
		{
			return TryLoginAsync(connection, LoginOptions.None, null, password, null, cancellationToken);
		}

		/// <summary>
		/// Attempts to log in to the server
		/// </summary>
		/// <param name="connection">Connection to use</param>
		/// <param name="options">Options for the command</param>
		/// <param name="user">User to login as</param>
		/// <param name="password">Password for the user</param>
		/// <param name="host">Host for the ticket</param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public static async Task<PerforceResponse<LoginRecord>> TryLoginAsync(this IPerforceConnection connection, LoginOptions options, string? user, string? password, string? host, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & LoginOptions.AllHosts) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & LoginOptions.PrintTicket) != 0)
			{
				arguments.Add("-p");
			}
			if ((options & LoginOptions.Status) != 0)
			{
				arguments.Add("-s");
			}
			if (host != null)
			{
				arguments.Add($"-h{host}");
			}
			if (user != null)
			{
				arguments.Add(user);
			}

			List<PerforceResponse> parsedResponses;
#pragma warning disable CA1849 // Call async methods when in an async method
			await using (IPerforceOutput response = connection.Command("login", arguments, null, null, password, false))
			{
				for (; ; )
				{
					if (!await response.ReadAsync(cancellationToken))
					{
						break;
					}
				}

				DiscardPasswordPrompt(response);

				parsedResponses = await response.ReadResponsesAsync(typeof(LoginRecord), cancellationToken);

				// end lifetime of `response` here
				// this prevents a deadlock in case `connection` is a `NativePerforceConnection` and `response` is a `NativePerforceConnection.Response`
				// not DisposeAsync()ing here will cause a deadlock when calling `TryGetLoginStateAsync()` below
			}
#pragma warning restore CA1849 // Call async methods when in an async method

			PerforceResponse? error = parsedResponses.FirstOrDefault(x => !x.Succeeded);
			if (error != null)
			{
				return new PerforceResponse<LoginRecord>(error);
			}

			string? ticket = null;
			if ((options & LoginOptions.PrintTicket) != 0)
			{
				if (parsedResponses.Count != 2)
				{
					throw new PerforceException("Unable to parse login response; expected two records, one with ticket id");
				}

				PerforceInfo? info = parsedResponses[0].Info;
				if (info == null)
				{
					throw new PerforceException("Unable to parse login response; expected two records, one with ticket id");
				}

				ticket = info.Data;
				parsedResponses.RemoveAt(0);
			}

			LoginRecord? loginRecord = parsedResponses.First().Data as LoginRecord;
			if (loginRecord == null)
			{
				// Older versions of P4.EXE do not return a login record for succesful login, instread just returning a string. Call p4 login -s to get the login state instead.
				PerforceResponse<LoginRecord> legacyResponse = await TryGetLoginStateAsync(connection, cancellationToken);
				if (!legacyResponse.Succeeded)
				{
					return legacyResponse;
				}
				loginRecord = legacyResponse.Data;
			}

			loginRecord.Ticket = ticket;
			return new PerforceResponse<LoginRecord>(loginRecord);
		}

		static void DiscardPasswordPrompt(IPerforceOutput output)
		{
			ReadOnlySpan<byte> data = output.Data.Span;
			for (int idx = 0; idx < data.Length; idx++)
			{
				if (data[idx] == '{')
				{
					output.Discard(idx);
					break;
				}
			}
		}

		/// <summary>
		/// Gets the state of the current user's login
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static async Task<LoginRecord> GetLoginStateAsync(this IPerforceConnection connection, CancellationToken cancellationToken = default)
		{
			return (await TryGetLoginStateAsync(connection, cancellationToken)).Data;
		}

		/// <summary>
		/// Gets the state of the current user's login
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static async Task<PerforceResponse<LoginRecord>> TryGetLoginStateAsync(this IPerforceConnection connection, CancellationToken cancellationToken = default)
		{
			return await SingleResponseCommandAsync<LoginRecord>(connection, "login", ["-s"], null, cancellationToken);
		}

		#endregion

		#region p4 merge

		/// <summary>
		/// Execute the 'merge' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the merge</param>
		/// <param name="change"></param>
		/// <param name="maxFiles">Maximum number of files to merge</param>
		/// <param name="sourceFileSpec">The source filespec and revision range</param>
		/// <param name="targetFileSpec">The target filespec</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>List of records</returns>
		public static async Task<PerforceResponseList<MergeRecord>> MergeAsync(this IPerforceConnection connection, MergeOptions options, int change, int maxFiles, string sourceFileSpec, string targetFileSpec, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & MergeOptions.Preview) != 0)
			{
				arguments.Add($"-n");
			}
			if (change != -1)
			{
				arguments.Add($"-c{change}");
			}
			if (maxFiles != -1)
			{
				arguments.Add($"-m{maxFiles}");
			}
			if ((options & MergeOptions.Force) != 0)
			{
				arguments.Add("-F");
			}
			if ((options & MergeOptions.ReverseMapping) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & MergeOptions.AsStreamSpec) != 0)
			{
				arguments.Add("-As");
			}
			if ((options & MergeOptions.AsFiles) != 0)
			{
				arguments.Add("-Af");
			}
			if ((options & MergeOptions.Stream) != 0)
			{
				arguments.Add("-S");
			}

			arguments.Add(sourceFileSpec);

			if ((options & MergeOptions.Source) != 0)
			{
				arguments.Add("-s");
			}

			arguments.Add(targetFileSpec);

			PerforceResponseList<MergeRecord> records = await CommandAsync<MergeRecord>(connection, "merge", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 move

		/// <summary>
		/// Opens files for move
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="sourceFileSpec">The source file(s)</param>
		/// <param name="targetFileSpec">The target file(s)</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<MoveRecord>> MoveAsync(this IPerforceConnection connection, int changeNumber, string? fileType, MoveOptions options, string sourceFileSpec, string targetFileSpec, CancellationToken cancellationToken = default)
		{
			return (await TryMoveAsync(connection, changeNumber, fileType, options, sourceFileSpec, targetFileSpec, cancellationToken)).Data;
		}

		/// <summary>
		/// Opens files for move
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="fileType">Type for new files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="sourceFileSpec">The source file(s)</param>
		/// <param name="targetFileSpec">The target file(s)</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<MoveRecord>> TryMoveAsync(this IPerforceConnection connection, int changeNumber, string? fileType, MoveOptions options, string sourceFileSpec, string targetFileSpec, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & MoveOptions.KeepWorkspaceFiles) != 0)
			{
				arguments.Add("-k");
			}
			if ((options & MoveOptions.RenameOnly) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & MoveOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if (fileType != null)
			{
				arguments.Add($"-t{fileType}");
			}
			arguments.Add(sourceFileSpec);
			arguments.Add(targetFileSpec);

			PerforceResponseList<MoveRecord> records = await CommandAsync<MoveRecord>(connection, "move", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 opened

		/// <summary>
		/// Execute the 'opened' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Specification for the files to list</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<OpenedRecord> OpenedAsync(this IPerforceConnection connection, OpenedOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryOpenedAsync(connection, options, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'opened' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Specification for the files to list</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<OpenedRecord>> TryOpenedAsync(this IPerforceConnection connection, OpenedOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryOpenedAsync(connection, options, -1, null, null, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'opened' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">List the files in pending changelist change. To list files in the default changelist, use DefaultChange.</param>
		/// <param name="clientName">List only files that are open in the given client</param>
		/// <param name="userName">List only files that are opened by the given user</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="fileSpecs">Specification for the files to list</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<OpenedRecord> OpenedAsync(this IPerforceConnection connection, OpenedOptions options, int changeNumber, string? clientName, string? userName, int maxResults, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryOpenedAsync(connection, options, changeNumber, clientName, userName, maxResults, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Execute the 'opened' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="changeNumber">List the files in pending changelist change. To list files in the default changelist, use DefaultChange.</param>
		/// <param name="clientName">List only files that are open in the given client</param>
		/// <param name="userName">List only files that are opened by the given user</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="fileSpecs">Specification for the files to list</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<OpenedRecord>> TryOpenedAsync(this IPerforceConnection connection, OpenedOptions options, int changeNumber, string? clientName, string? userName, int maxResults, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			// Build the argument list
			List<string> arguments = [];
			if ((options & OpenedOptions.AllWorkspaces) != 0)
			{
				arguments.Add($"-a");
			}
			if ((options & OpenedOptions.Exclusive) != 0)
			{
				arguments.Add($"-x");
			}
			if (changeNumber == PerforceReflection.DefaultChange)
			{
				arguments.Add($"-cdefault");
			}
			else if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if (clientName != null)
			{
				arguments.Add($"-C{clientName}");
			}
			if (userName != null)
			{
				arguments.Add($"-u{userName}");
			}
			if (maxResults != -1)
			{
				arguments.Add($"-m{maxResults}");
			}
			if ((options & OpenedOptions.ShortOutput) != 0)
			{
				arguments.Add($"-s");
			}

			IAsyncEnumerable<PerforceResponse<OpenedRecord>> records = StreamCommandAsync<OpenedRecord>(connection, "opened", arguments, fileSpecs.List, null, cancellationToken);
			return records.Where(x => x.Error == null || x.Error.Generic != PerforceGenericCode.Empty);
		}

		#endregion

		#region p4 print

		/// <summary>
		/// Execute the 'print' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="outputFile">Output file to redirect output to</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PrintRecord> PrintAsync(this IPerforceConnection connection, string outputFile, string fileSpec, CancellationToken cancellationToken = default)
		{
			return (await TryPrintSingleInternalAsync<PrintRecord>(connection, outputFile, fileSpec, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'print' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="outputFile">Output file to redirect output to</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<PrintRecord>> TryPrintAsync(this IPerforceConnection connection, string outputFile, string fileSpec, CancellationToken cancellationToken = default)
		{
			return TryPrintInternalAsync(connection, outputFile, fileSpec, cancellationToken);
		}

		/// <summary>
		/// Execute the 'print' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponse<PrintRecord<byte[]>>> TryPrintDataAsync(this IPerforceConnection connection, string fileSpec, CancellationToken cancellationToken = default)
		{
			string tempFile = Path.GetTempFileName();
			try
			{
				PerforceResponse<PrintRecord<byte[]>> record = await TryPrintSingleInternalAsync<PrintRecord<byte[]>>(connection, tempFile, fileSpec, cancellationToken);
				if (record.Succeeded)
				{
					record.Data.Contents = await File.ReadAllBytesAsync(tempFile, cancellationToken);
				}
				return record;
			}
			finally
			{
				try
				{
					File.SetAttributes(tempFile, FileAttributes.Normal);
					File.Delete(tempFile);
				}
				catch
				{
				}
			}
		}

		/// <summary>
		/// Execute the 'print' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponse<PrintRecord<string[]>>> TryPrintLinesAsync(this IPerforceConnection connection, string fileSpec, CancellationToken cancellationToken = default)
		{
			string tempFile = Path.GetTempFileName();
			try
			{
				PerforceResponse<PrintRecord<string[]>> record = await TryPrintSingleInternalAsync<PrintRecord<string[]>>(connection, tempFile, fileSpec, cancellationToken);
				if (record.Succeeded)
				{
					record.Data.Contents = await File.ReadAllLinesAsync(tempFile, cancellationToken);
				}
				return record;
			}
			finally
			{
				try
				{
					File.SetAttributes(tempFile, FileAttributes.Normal);
					File.Delete(tempFile);
				}
				catch
				{
				}
			}
		}

		/// <summary>
		/// Execute the 'print' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="outputFile">Output file to redirect output to</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		static async Task<PerforceResponseList<PrintRecord>> TryPrintInternalAsync(this IPerforceConnection connection, string outputFile, string fileSpec, CancellationToken cancellationToken = default)
		{
			// Build the argument list
			List<string> arguments = ["-o", outputFile, fileSpec];

			PerforceResponseList<PrintRecord> records = await CommandAsync<PrintRecord>(connection, "print", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		/// <summary>
		/// Execute the 'print' command for a single file
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="outputFile">Output file to redirect output to</param>
		/// <param name="fileSpec">Specification for the files to print</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		static Task<PerforceResponse<T>> TryPrintSingleInternalAsync<T>(this IPerforceConnection connection, string outputFile, string fileSpec, CancellationToken cancellationToken = default) where T : class
		{
			// Build the argument list
			List<string> arguments = ["-o", outputFile, fileSpec];
			return SingleResponseCommandAsync<T>(connection, "print", arguments, null, cancellationToken);
		}

		#endregion

		#region property

		// ATTN: We currently support only getting a single value (p4 property -l -n)

		/// <summary>
		/// Execute the 'property' command for a single value 
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="propertyName">Name of the property</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Value of the property</returns>
		public static async Task<string> GetPropertyAsync(this IPerforceConnection connection, string propertyName, CancellationToken cancellationToken = default)
		{
			return (await TryGetSinglePropertyAsync(connection, propertyName, cancellationToken)).Data.Value;
		}

		/// <summary>
		/// Execute the 'property' command for a single value 
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="propertyName">Name of the property</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Value of the property</returns>
		public static async Task<PerforceResponse<PropertyRecord>> TryGetSinglePropertyAsync(this IPerforceConnection connection, string propertyName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-l", "-n", propertyName];

			return await SingleResponseCommandAsync<PropertyRecord>(connection, "property", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 reconcile

		/// <summary>
		/// Open files for add, delete, and/or edit in order to reconcile a workspace with changes made outside of Perforce.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<ReconcileRecord>> ReconcileAsync(this IPerforceConnection connection, int changeNumber, ReconcileOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			PerforceResponseList<ReconcileRecord> records = await TryReconcileAsync(connection, changeNumber, options, fileSpecs, cancellationToken);
			records.RemoveAll(x => x.Info != null);
			return records.Data;
		}

		/// <summary>
		/// Open files for add, delete, and/or edit in order to reconcile a workspace with changes made outside of Perforce.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<ReconcileRecord>> TryReconcileAsync(this IPerforceConnection connection, int changeNumber, ReconcileOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if ((options & ReconcileOptions.Edit) != 0)
			{
				arguments.Add("-e");
			}
			if ((options & ReconcileOptions.Add) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & ReconcileOptions.Delete) != 0)
			{
				arguments.Add("-d");
			}
			if ((options & ReconcileOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if ((options & ReconcileOptions.AllowWildcards) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & ReconcileOptions.NoIgnore) != 0)
			{
				arguments.Add("-I");
			}
			if ((options & ReconcileOptions.LocalFileSyntax) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & ReconcileOptions.UseFileModification) != 0)
			{
				arguments.Add("-m");
			}
			arguments.AddRange(fileSpecs.List);

			PerforceResponseList<ReconcileRecord> records = await CommandAsync<ReconcileRecord>(connection, "reconcile", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 reopen

		/// <summary>
		/// Reopen a file
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="fileType">New filetype</param>
		/// <param name="fileSpec">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<ReopenRecord>> ReopenAsync(this IPerforceConnection connection, int? changeNumber, string? fileType, string fileSpec, CancellationToken cancellationToken = default)
		{
			return (await TryReopenAsync(connection, changeNumber, fileType, fileSpec, cancellationToken)).Data;
		}

		/// <summary>
		/// Reopen a file
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="fileType">New filetype</param>
		/// <param name="fileSpec">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<ReopenRecord>> TryReopenAsync(this IPerforceConnection connection, int? changeNumber, string? fileType, string fileSpec, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if (changeNumber != null)
			{
				if (changeNumber == PerforceReflection.DefaultChange)
				{
					arguments.Add("-cdefault");
				}
				else
				{
					arguments.Add($"-c{changeNumber}");
				}
			}
			if (fileType != null)
			{
				arguments.Add($"-t{fileType}");
			}
			arguments.Add(fileSpec);

			PerforceResponseList<ReopenRecord> records = await CommandAsync<ReopenRecord>(connection, "reopen", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 reload

		/// <summary>
		/// Reloads a client workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="clientName">Name of the client to reload</param>
		/// <param name="sourceServerId">The source server id</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<List<PerforceResponse>> ReloadClient(this IPerforceConnection connection, string clientName, string sourceServerId, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"-c{clientName}", $"-p{sourceServerId}"];
			return connection.CommandAsync("reload", arguments, null, null, cancellationToken);
		}

		#endregion p4 reload

		#region p4 resolve

		/// <summary>
		/// Resolve conflicts between file revisions.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<ResolveRecord>> ResolveAsync(this IPerforceConnection connection, int changeNumber, ResolveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryResolveAsync(connection, changeNumber, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Resolve conflicts between file revisions.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to open files to</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<ResolveRecord>> TryResolveAsync(this IPerforceConnection connection, int changeNumber, ResolveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & ResolveOptions.Automatic) != 0)
			{
				arguments.Add("-am");
			}
			if ((options & ResolveOptions.AcceptYours) != 0)
			{
				arguments.Add("-ay");
			}
			if ((options & ResolveOptions.AcceptTheirs) != 0)
			{
				arguments.Add("-at");
			}
			if ((options & ResolveOptions.SafeAccept) != 0)
			{
				arguments.Add("-as");
			}
			if ((options & ResolveOptions.ForceAccept) != 0)
			{
				arguments.Add("-af");
			}
			if ((options & ResolveOptions.IgnoreWhitespaceOnly) != 0)
			{
				arguments.Add("-db");
			}
			if ((options & ResolveOptions.IgnoreWhitespace) != 0)
			{
				arguments.Add("-dw");
			}
			if ((options & ResolveOptions.IgnoreLineEndings) != 0)
			{
				arguments.Add("-dl");
			}
			if ((options & ResolveOptions.ResolveAgain) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & ResolveOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			arguments.AddRange(fileSpecs.List);

			PerforceResponseList<ResolveRecord> records = await CommandAsync<ResolveRecord>(connection, "resolve", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 revert

		/// <summary>
		/// Reverts files that have been added to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="clientName">Revert another user’s open files. </param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<RevertRecord>> RevertAsync(this IPerforceConnection connection, int changeNumber, string? clientName, RevertOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryRevertAsync(connection, changeNumber, clientName, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Reverts files that have been added to a pending changelist.
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist to add files to</param>
		/// <param name="clientName">Revert another user’s open files. </param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to be reverted</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<RevertRecord>> TryRevertAsync(this IPerforceConnection connection, int changeNumber, string? clientName, RevertOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & RevertOptions.Unchanged) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & RevertOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if ((options & RevertOptions.KeepWorkspaceFiles) != 0)
			{
				arguments.Add("-k");
			}
			if ((options & RevertOptions.DeleteAddedFiles) != 0)
			{
				arguments.Add("-w");
			}
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			if (clientName != null)
			{
				arguments.Add($"-C{clientName}");
			}

			PerforceResponseList<RevertRecord> records = await BatchedCommandAsync<RevertRecord>(connection, "revert", arguments, fileSpecs.List, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 server

		/// <summary>
		/// Gets information about the specified server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="serverId">The server identifier</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static async Task<ServerRecord> GetServerAsync(this IPerforceConnection connection, string serverId, CancellationToken cancellationToken = default)
		{
			PerforceResponse<ServerRecord> response = await TryGetServerAsync(connection, serverId, cancellationToken);
			return response.Data;
		}

		/// <summary>
		/// Gets information about the specified server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="serverId">The server identifier</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static Task<PerforceResponse<ServerRecord>> TryGetServerAsync(this IPerforceConnection connection, string serverId, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o", serverId];
			return SingleResponseCommandAsync<ServerRecord>(connection, "server", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 shelve

		/// <summary>
		/// Shelves a set of files
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The change number to receive the shelved files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<ShelveRecord>> ShelveAsync(this IPerforceConnection connection, int changeNumber, ShelveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryShelveAsync(connection, changeNumber, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Shelves a set of files
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The change number to receive the shelved files</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<ShelveRecord>> TryShelveAsync(this IPerforceConnection connection, int changeNumber, ShelveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"-c{changeNumber}"];
			if ((options & ShelveOptions.OnlyChanged) != 0)
			{
				arguments.Add("-aleaveunchanged");
			}
			if ((options & ShelveOptions.Overwrite) != 0)
			{
				arguments.Add("-f");
			}
			arguments.AddRange(fileSpecs.List);

			PerforceResponseList<ShelveRecord> records = await CommandAsync<ShelveRecord>(connection, "shelve", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		/// <summary>
		/// Deletes files from a shelved changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist containing shelved files to be deleted</param>
		/// <param name="fileSpecs">Files to delete</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task DeleteShelvedFilesAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<PerforceResponse> responses = await TryDeleteShelvedFilesAsync(connection, changeNumber, fileSpecs, cancellationToken);

			PerforceResponse? errorResponse = responses.FirstOrDefault(x => x.Error != null && x.Error.Generic != PerforceGenericCode.Empty);
			if (errorResponse != null)
			{
				throw new PerforceException(errorResponse.Error!);
			}
		}

		/// <summary>
		/// Deletes files from a shelved changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">Changelist containing shelved files to be deleted</param>
		/// <param name="fileSpecs">Files to delete</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<List<PerforceResponse>> TryDeleteShelvedFilesAsync(this IPerforceConnection connection, int changeNumber, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-d"];
			if (changeNumber != -1)
			{
				arguments.Add($"-c{changeNumber}");
			}
			arguments.AddRange(fileSpecs.List);

			return connection.CommandAsync("shelve", arguments, null, null, cancellationToken);
		}

		#endregion

		#region p4 sizes

		/// <summary>
		/// Execute the 'sizes' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static async Task<List<SizesRecord>> SizesAsync(this IPerforceConnection connection, SizesOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TrySizesAsync(connection, options, -1, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'sizes' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static Task<PerforceResponseList<SizesRecord>> TrySizesAsync(this IPerforceConnection connection, SizesOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySizesAsync(connection, options, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Execute the 'sizes' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxLines">Maximum number of results to return. Ignored if less than or equal to zero.</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		public static async Task<List<SizesRecord>> SizesAsync(this IPerforceConnection connection, SizesOptions options, int maxLines, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TrySizesAsync(connection, options, maxLines, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Execute the 'sizes' command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxLines">Maximum number of results to return. Ignored if less than or equal to zero.</param>
		/// <param name="fileSpecs">List of file specifications to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of response objects</returns>
		/// <see href="https://help.perforce.com/helix-core/server-apps/cmdref/current/Content/CmdRef/p4_sizes.html"/>
		public static async Task<PerforceResponseList<SizesRecord>> TrySizesAsync(this IPerforceConnection connection, SizesOptions options, int maxLines, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			// Put the sub-mode arguments first, per the p4 docs.
			// nb. Not all options are valid together, but we let p4 validate this rather than duplicating the logic.
			if ((options & SizesOptions.FileSizeAsExpectedInTheWorkspace) != 0)
			{
				arguments.Add("-C");
			}
			if ((options & SizesOptions.LimitToArchiveDepots) != 0)
			{
				arguments.Add("-A");
			}

			if ((options & SizesOptions.AllRevisions) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & SizesOptions.CalculateSum) != 0)
			{
				arguments.Add("-s");
			}
			if ((options & SizesOptions.DisplayForShelvedFilesOnly) != 0)
			{
				arguments.Add("-S");
			}
			if ((options & SizesOptions.ExcludeLazyCopies) != 0)
			{
				arguments.Add("-z");
			}
			if (maxLines > 0)
			{
				arguments.Add($"-m{maxLines}");
			}

			PerforceResponseList<SizesRecord> records = await BatchedCommandAsync<SizesRecord>(connection, "sizes", arguments, fileSpecs.List, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 stream

		/// <summary>
		/// Queries information about a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">Name of the stream to query</param>
		/// <param name="includeView">Whether to include the stream view in the output</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Stream information record</returns>
		public static async Task<StreamRecord> GetStreamAsync(this IPerforceConnection connection, string streamName, bool includeView, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamAsync(connection, streamName, includeView, cancellationToken)).Data;
		}

		/// <summary>
		/// Queries information about a stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamName">Name of the stream to query</param>
		/// <param name="includeView">Whether to include the stream view in the output</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Stream information record</returns>
		public static Task<PerforceResponse<StreamRecord>> TryGetStreamAsync(this IPerforceConnection connection, string streamName, bool includeView, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o"];
			if (includeView)
			{
				arguments.Add("-v");
			}
			arguments.Add(streamName);

			return SingleResponseCommandAsync<StreamRecord>(connection, "stream", arguments, null, cancellationToken);
		}

		/// <summary>
		/// Updates an existing stream
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="record">Information of the stream to update</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Stream information record</returns>
		public static Task<PerforceResponse> TryUpdateStreamAsync(this IPerforceConnection connection, StreamRecord record, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-i"];
			return SingleResponseCommandAsync(connection, "stream", arguments, connection.SerializeRecord(record), null, cancellationToken);
		}

		/// <summary>
		/// Serializes a client record to a byte array
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="input">The input record</param>
		/// <returns>Serialized record data</returns>
		static byte[] SerializeRecord(this IPerforceConnection connection, StreamRecord input)
		{
			List<KeyValuePair<string, object>> nameToValue = [];

			void Add(string fieldName, string? value)
			{
				if (value != null)
				{
					nameToValue.Add(new KeyValuePair<string, object>(fieldName, value));
				}
			}

			Add("Stream", input.Stream);
			Add("Owner", input.Owner);
			Add("Name", input.Name);
			Add("Parent", input.Parent);
			Add("Type", input.Type);
			Add("Description", input.Description);
			Add("ParentView", input.ParentView);

			if (input.Options != StreamOptions.None)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Options", PerforceReflection.GetEnumText(typeof(StreamOptions), input.Options)));
			}

			if (input.Paths.Count > 0)
			{
				nameToValue.Add(new KeyValuePair<string, object>("Paths", input.Paths));
			}

			if (input.View.Count > 0)
			{
				nameToValue.Add(new KeyValuePair<string, object>("View", input.View));
			}

			if (input.ChangeView.Count > 0)
			{
				nameToValue.Add(new KeyValuePair<string, object>("ChangeView", input.ChangeView));
			}

			return connection.CreateRecord(nameToValue).Serialize();
		}
		#endregion

		#region p4 streamlog

		/// <summary>
		/// List revision history of streams
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="changeNumber">Optional: Display the stream submitted at the specified changelist number</param>
		/// <param name="maxChanges">Optional: Display at most this many revisions per stream argument specified</param>
		/// <param name="streams">Streams to list history for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Stream log records</returns>
		public static async Task<List<StreamLogRecord>> GetStreamLogAsync(this IPerforceConnection connection, StreamLogOptions options, int? changeNumber, int? maxChanges, string[] streams, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamLogAsync(connection, options, changeNumber, maxChanges, streams, cancellationToken)).Data;
		}

		/// <summary>
		/// List revision history of streams
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for this command</param>
		/// <param name="changeNumber">Optional: Display the stream submitted at the specified changelist number</param>
		/// <param name="maxChanges">Optional: Display at most this many revisions per stream argument specified</param>
		/// <param name="streams">Streams to list history for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Stream log records</returns>
		public static Task<PerforceResponseList<StreamLogRecord>> TryGetStreamLogAsync(this IPerforceConnection connection, StreamLogOptions options, int? changeNumber, int? maxChanges, string[] streams, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];

			if ((options & StreamLogOptions.IncludeHistory) != 0)
			{
				arguments.Add("-i");
			}
			if ((options & StreamLogOptions.DisplayStreamContent) != 0)
			{
				arguments.Add("-h");
			}
			if ((options & StreamLogOptions.IncludeTimes) != 0)
			{
				arguments.Add("-t");
			}
			if ((options & StreamLogOptions.LongOutput) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & StreamLogOptions.TruncatedLongOutput) != 0)
			{
				arguments.Add("-L");
			}
			if (changeNumber != null)
			{
				arguments.Add($"-c{changeNumber.Value}");
			}
			if (maxChanges != null)
			{
				arguments.Add($"-m{maxChanges.Value}");
			}
			arguments.AddRange(streams);
			return CommandAsync<StreamLogRecord>(connection, "streamlog", arguments, null, cancellationToken);
		}
		#endregion

		#region p4 streams

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamPath">The path for streams to enumerate (eg. "//UE4/...")</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of streams matching the given criteria</returns>
		public static async Task<List<StreamsRecord>> GetStreamsAsync(this IPerforceConnection connection, string? streamPath, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamsAsync(connection, streamPath, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamPath">The path for streams to enumerate (eg. "//UE4/...")</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of streams matching the given criteria</returns>
		public static Task<PerforceResponseList<StreamsRecord>> TryGetStreamsAsync(this IPerforceConnection connection, string? streamPath, CancellationToken cancellationToken = default)
		{
			return TryGetStreamsAsync(connection, streamPath, -1, null, false, cancellationToken);
		}

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamPath">The path for streams to enumerate (eg. "//UE4/...")</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="filter">Additional filter to be applied to the results</param>
		/// <param name="unloaded">Whether to enumerate unloaded workspaces</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of streams matching the given criteria</returns>
		public static async Task<List<StreamsRecord>> GetStreamsAsync(this IPerforceConnection connection, string? streamPath, int maxResults, string? filter, bool unloaded, CancellationToken cancellationToken = default)
		{
			return (await TryGetStreamsAsync(connection, streamPath, maxResults, filter, unloaded, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="streamPath">The path for streams to enumerate (eg. "//UE4/...")</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="filter">Additional filter to be applied to the results</param>
		/// <param name="unloaded">Whether to enumerate unloaded workspaces</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of streams matching the given criteria</returns>
		public static async Task<PerforceResponseList<StreamsRecord>> TryGetStreamsAsync(this IPerforceConnection connection, string? streamPath, int maxResults, string? filter, bool unloaded, CancellationToken cancellationToken = default)
		{
			// Build the command line
			List<string> arguments = [];
			if (unloaded)
			{
				arguments.Add("-U");
			}
			if (filter != null)
			{
				arguments.Add("-F");
				arguments.Add(filter);
			}
			if (maxResults > 0)
			{
				arguments.Add($"-m{maxResults}");
			}
			if (streamPath != null)
			{
				arguments.Add(streamPath);
			}

			// Execute the command
			PerforceResponseList<StreamsRecord> records = await CommandAsync<StreamsRecord>(connection, "streams", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 submit

		/// <summary>
		/// Submits a pending changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist to submit</param>
		/// <param name="options">Options for the command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<SubmitRecord> SubmitAsync(this IPerforceConnection connection, int changeNumber, SubmitOptions options, CancellationToken cancellationToken = default)
		{
			return (await TrySubmitAsync(connection, changeNumber, options, cancellationToken)).Data;
		}

		/// <summary>
		/// Submits a pending changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist to submit</param>
		/// <param name="options">Options for the command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponse<SubmitRecord>> TrySubmitAsync(this IPerforceConnection connection, int changeNumber, SubmitOptions options, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & SubmitOptions.ReopenAsEdit) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & SubmitOptions.SubmitUnchanged) != 0)
			{
				arguments.Add("-f");
				arguments.Add("submitunchanged");
			}
			if ((options & SubmitOptions.RevertUnchanged) != 0)
			{
				arguments.Add("-f");
				arguments.Add("revertunchanged");
			}
			if ((options & SubmitOptions.LeaveUnchanged) != 0)
			{
				arguments.Add("-f");
				arguments.Add("leaveunchanged");
			}

			arguments.Add($"-c{changeNumber}");

			PerforceResponseList<SubmitRecord> responses = await CommandAsync<SubmitRecord>(connection, "submit", arguments, null, cancellationToken);
			return ParseSubmitResponses(responses, connection.Logger);
		}

		/// <summary>
		/// Submits a shelved changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist to submit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<SubmitRecord> SubmitShelvedAsync(this IPerforceConnection connection, int changeNumber, CancellationToken cancellationToken = default)
		{
			return (await TrySubmitShelvedAsync(connection, changeNumber, cancellationToken)).Data;
		}

		/// <summary>
		/// Submits a pending changelist
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist to submit</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponse<SubmitRecord>> TrySubmitShelvedAsync(this IPerforceConnection connection, int changeNumber, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"-e{changeNumber}"];

			PerforceResponseList<SubmitRecord> responses = await CommandAsync<SubmitRecord>(connection, "submit", arguments, null, cancellationToken);
			return ParseSubmitResponses(responses, connection.Logger);
		}

		static PerforceResponse<SubmitRecord> ParseSubmitResponses(PerforceResponseList<SubmitRecord> responses, ILogger logger)
		{
			SubmitRecord? success = null;
			PerforceResponse<SubmitRecord>? error = null;
			foreach (PerforceResponse<SubmitRecord> response in responses)
			{
				if (response.Error != null)
				{
					error ??= response;
				}
				else if (response.Info != null)
				{
					logger.LogInformation("Submit: {Info}", response.Info.Data);
				}
				else if (success == null)
				{
					success = response.Data;
				}
				else
				{
					success.Merge(response.Data);
				}
			}

			if (error != null)
			{
				return error;
			}
			else if (success != null)
			{
				return new PerforceResponse<SubmitRecord>(success);
			}
			else
			{
				return responses[0];
			}
		}

		#endregion

		#region p4 sync

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<SyncRecord> SyncAsync(this IPerforceConnection connection, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<SyncRecord>> TrySyncAsync(this IPerforceConnection connection, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, SyncOptions.None, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<SyncRecord> SyncAsync(this IPerforceConnection connection, SyncOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, options, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<SyncRecord>> TrySyncAsync(this IPerforceConnection connection, SyncOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, options, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified.</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<SyncRecord> SyncAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, options, maxFiles, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified.</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<SyncRecord>> TrySyncAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, options, maxFiles, -1, -1, -1, -1, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified</param>
		/// <param name="numThreads">Sync in parallel using the given number of threads</param>
		/// <param name="batch">The number of files in a batch</param>
		/// <param name="batchSize">The number of bytes in a batch</param>
		/// <param name="min">Minimum number of files in a parallel sync</param>
		/// <param name="minSize">Minimum number of bytes in a parallel sync</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<SyncRecord> SyncAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, int numThreads, int batch, int batchSize, int min, int minSize, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncAsync(connection, options, maxFiles, numThreads, batch, batchSize, min, minSize, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified</param>
		/// <param name="numThreads">Sync in parallel using the given number of threads</param>
		/// <param name="batch">The number of files in a batch</param>
		/// <param name="batchSize">The number of bytes in a batch</param>
		/// <param name="min">Minimum number of files in a parallel sync</param>
		/// <param name="minSize">Minimum number of bytes in a parallel sync</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<SyncRecord>> TrySyncAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, int numThreads, int batch, int batchSize, int min, int minSize, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			// Perforce annoyingly returns 'up-to-date' as an error. Ignore it.
			IAsyncEnumerable<PerforceResponse<SyncRecord>> records = SyncInternalAsync<SyncRecord>(connection, options, maxFiles, numThreads, batch, batchSize, min, minSize, fileSpecs, false, cancellationToken);
			records = records.Where(x => x.Error == null || x.Error.Generic != PerforceGenericCode.Empty);
			return records;
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified.</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<SyncSummaryRecord>> SyncQuietAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			PerforceResponseList<SyncSummaryRecord> records = await TrySyncQuietAsync(connection, options, maxFiles, fileSpecs, cancellationToken);
			records.RemoveAll(x => (x.Error != null && x.Error.Generic == PerforceGenericCode.Empty) || x.Info != null);
			return records.Data;
		}

		/// <summary>
		/// Syncs files from the server
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified.</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<SyncSummaryRecord>> TrySyncQuietAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TrySyncQuietAsync(connection, options, maxFiles, -1, -1, -1, -1, -1, fileSpecs, cancellationToken);
		}

		/// <summary>
		/// Syncs files from the server without returning detailed file info
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified</param>
		/// <param name="numThreads">Sync in parallel using the given number of threads</param>
		/// <param name="batch">The number of files in a batch</param>
		/// <param name="batchSize">The number of bytes in a batch</param>
		/// <param name="min">Minimum number of files in a parallel sync</param>
		/// <param name="minSize">Minimum number of bytes in a parallel sync</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<SyncSummaryRecord>> SyncQuietAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, int numThreads, int batch, int batchSize, int min, int minSize, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TrySyncQuietAsync(connection, options, maxFiles, numThreads, batch, batchSize, min, minSize, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Syncs files from the server without returning detailed file info
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified</param>
		/// <param name="numThreads">Sync in parallel using the given number of threads</param>
		/// <param name="batch">The number of files in a batch</param>
		/// <param name="batchSize">The number of bytes in a batch</param>
		/// <param name="min">Minimum number of files in a parallel sync</param>
		/// <param name="minSize">Minimum number of bytes in a parallel sync</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<SyncSummaryRecord>> TrySyncQuietAsync(this IPerforceConnection connection, SyncOptions options, int maxFiles, int numThreads, int batch, int batchSize, int min, int minSize, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return [.. await SyncInternalAsync<SyncSummaryRecord>(connection, options, maxFiles, numThreads, batch, batchSize, min, minSize, fileSpecs, true, cancellationToken).ToListAsync(cancellationToken)];
		}

		/// <summary>
		/// Gets arguments for a sync command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxFiles">Syncs only the first number of files specified</param>
		/// <param name="numThreads">Sync in parallel using the given number of threads</param>
		/// <param name="batch">The number of files in a batch</param>
		/// <param name="batchSize">The number of bytes in a batch</param>
		/// <param name="min">Minimum number of files in a parallel sync</param>
		/// <param name="minSize">Minimum number of bytes in a parallel sync</param>
		/// <param name="fileSpecs">Files to sync</param>
		/// <param name="quiet">Whether to use quiet output</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Arguments for the command</returns>
		private static IAsyncEnumerable<PerforceResponse<T>> SyncInternalAsync<T>(this IPerforceConnection connection, SyncOptions options, int maxFiles, int numThreads, int batch, int batchSize, int min, int minSize, FileSpecList fileSpecs, bool quiet, CancellationToken cancellationToken = default) where T : class
		{
			List<string> arguments = [];
			if ((options & SyncOptions.Force) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & SyncOptions.KeepWorkspaceFiles) != 0)
			{
				arguments.Add("-k");
			}
			if ((options & SyncOptions.FullDepotSyntax) != 0)
			{
				arguments.Add("-L");
			}
			if ((options & SyncOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if ((options & SyncOptions.NetworkPreviewOnly) != 0)
			{
				arguments.Add("-N");
			}
			if ((options & SyncOptions.DoNotUpdateHaveList) != 0)
			{
				arguments.Add("-p");
			}
			if (quiet)
			{
				arguments.Add("-q");
			}
			if ((options & SyncOptions.ReopenMovedFiles) != 0)
			{
				arguments.Add("-r");
			}
			if ((options & SyncOptions.Safe) != 0)
			{
				arguments.Add("-s");
			}
			if (maxFiles != -1)
			{
				arguments.Add($"-m{maxFiles}");
			}

			// Using multiple threads is not supported through p4.exe due to threaded output not being parsable
			if (numThreads != -1 && (connection is NativePerforceConnection))
			{
				StringBuilder argument = new($"--parallel=threads={numThreads}");
				if (batch != -1)
				{
					argument.Append($",batch={batch}");
				}
				if (batchSize != -1)
				{
					argument.Append($",batchsize={batchSize}");
				}
				if (min != -1)
				{
					argument.Append($",min={min}");
				}
				if (minSize != -1)
				{
					argument.Append($",minsize={minSize}");
				}
				arguments.Add(argument.ToString());
			}
			return StreamCommandAsync<T>(connection, "sync", arguments, fileSpecs.List, null, cancellationToken);
		}

		#endregion

		#region p4 unshelve

		/// <summary>
		/// Restore shelved files from a pending change into a workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist containing shelved files</param>
		/// <param name="intoChangeNumber">The changelist to receive the unshelved files</param>
		/// <param name="usingBranchSpec">The branchspec to use when unshelving files</param>
		/// <param name="usingStream">Specifies the use of a stream-derived branch view to map the shelved files between the specified stream and its parent stream.</param>
		/// <param name="forceParentStream">Unshelve to the specified parent stream. Overrides the parent defined in the source stream specification.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to unshelve</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<UnshelveRecord>> UnshelveAsync(this IPerforceConnection connection, int changeNumber, int intoChangeNumber, string? usingBranchSpec, string? usingStream, string? forceParentStream, UnshelveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return (await TryUnshelveAsync(connection, changeNumber, intoChangeNumber, usingBranchSpec, usingStream, forceParentStream, options, fileSpecs, cancellationToken)).Data;
		}

		/// <summary>
		/// Restore shelved files from a pending change into a workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumber">The changelist containing shelved files</param>
		/// <param name="intoChangeNumber">The changelist to receive the unshelved files</param>
		/// <param name="usingBranchSpec">The branchspec to use when unshelving files</param>
		/// <param name="usingStream">Specifies the use of a stream-derived branch view to map the shelved files between the specified stream and its parent stream.</param>
		/// <param name="forceParentStream">Unshelve to the specified parent stream. Overrides the parent defined in the source stream specification.</param>
		/// <param name="options">Options for the command</param>
		/// <param name="fileSpecs">Files to unshelve</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<UnshelveRecord>> TryUnshelveAsync(this IPerforceConnection connection, int changeNumber, int intoChangeNumber, string? usingBranchSpec, string? usingStream, string? forceParentStream, UnshelveOptions options, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"-s{changeNumber}"];
			if ((options & UnshelveOptions.ForceOverwrite) != 0)
			{
				arguments.Add("-f");
			}
			if ((options & UnshelveOptions.PreviewOnly) != 0)
			{
				arguments.Add("-n");
			}
			if (intoChangeNumber != -1)
			{
				arguments.Add($"-c{intoChangeNumber}");
			}
			if (usingBranchSpec != null)
			{
				arguments.Add($"-b{usingBranchSpec}");
			}
			if (usingStream != null)
			{
				arguments.Add($"-S{usingStream}");
			}
			if (forceParentStream != null)
			{
				arguments.Add($"-P{forceParentStream}");
			}
			arguments.AddRange(fileSpecs.List);

			PerforceResponseList<UnshelveRecord> records = await CommandAsync<UnshelveRecord>(connection, "unshelve", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 user

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="userName">Name of the user to fetch information for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<UserRecord> GetUserAsync(this IPerforceConnection connection, string userName, CancellationToken cancellationToken = default)
		{
			return (await TryGetUserAsync(connection, userName, cancellationToken))[0].Data;
		}

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="userName">Name of the user to fetch information for</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static Task<PerforceResponseList<UserRecord>> TryGetUserAsync(this IPerforceConnection connection, string userName, CancellationToken cancellationToken = default)
		{
			List<string> arguments = ["-o", userName];
			return CommandAsync<UserRecord>(connection, "user", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 users

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<List<UsersRecord>> GetUsersAsync(this IPerforceConnection connection, UsersOptions options, int maxResults, CancellationToken cancellationToken = default)
		{
			return (await TryGetUsersAsync(connection, options, maxResults, cancellationToken)).Data;
		}

		/// <summary>
		/// Enumerates all streams in a depot
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="options">Options for the command</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<UsersRecord>> TryGetUsersAsync(this IPerforceConnection connection, UsersOptions options, int maxResults, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & UsersOptions.IncludeServiceUsers) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & UsersOptions.OnlyMasterServer) != 0)
			{
				arguments.Add("-c");
			}
			if ((options & UsersOptions.IncludeLoginInfo) != 0)
			{
				arguments.Add("-l");
			}
			if ((options & UsersOptions.OnlyReplicaServer) != 0)
			{
				arguments.Add("-r");
			}
			if (maxResults > 0)
			{
				arguments.Add($"-m{maxResults}");
			}

			PerforceResponseList<UsersRecord> records = await CommandAsync<UsersRecord>(connection, "users", arguments, null, cancellationToken);
			records.RemoveAll(x => x.Error != null && x.Error.Generic == PerforceGenericCode.Empty);
			return records;
		}

		#endregion

		#region p4 where

		/// <summary>
		/// Retrieves the location of a file of set of files in the workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpecs">Patterns for the files to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<WhereRecord> WhereAsync(this IPerforceConnection connection, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			return TryWhereAsync(connection, fileSpecs, cancellationToken).Select(x => x.Data);
		}

		/// <summary>
		/// Retrieves the location of a file of set of files in the workspace
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpecs">Patterns for the files to query</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of responses from the server</returns>
		public static IAsyncEnumerable<PerforceResponse<WhereRecord>> TryWhereAsync(this IPerforceConnection connection, FileSpecList fileSpecs, CancellationToken cancellationToken = default)
		{
			IAsyncEnumerable<PerforceResponse<WhereRecord>> records = StreamCommandAsync<WhereRecord>(connection, "where", [], fileSpecs.List, null, cancellationToken);
			return records.Where(x => x.Error == null || x.Error.Generic != PerforceGenericCode.Empty);
		}

		#endregion

		#region p4 undo

		/// <summary>
		/// perform undo on a changelist (p4 undo -c [targetCL] //...@undoCL)
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumberToUndo">Changelist number to undo</param>
		/// <param name="changeNumber">Changelist number to receive the changes</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task UndoChangeAsync(this IPerforceConnection connection, int changeNumberToUndo, int changeNumber, CancellationToken cancellationToken = default)
		{
			(await TryUndoChangeAsync(connection, changeNumberToUndo, changeNumber, cancellationToken))[0].EnsureSuccess();
		}

		/// <summary>
		/// perform undo on a changelist (p4 undo -c [targetCL] //...@undoCL)
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="changeNumberToUndo">Changelist number to undo</param>
		/// <param name="changeNumber">Changelist number to receive the changes</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>Response from the server</returns>
		public static async Task<PerforceResponseList<UndoRecord>> TryUndoChangeAsync(this IPerforceConnection connection, int changeNumberToUndo, int changeNumber, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [$"-c{changeNumber}", $"//...@{changeNumberToUndo}"];

			return await CommandAsync<UndoRecord>(connection, "undo", arguments, null, cancellationToken);
		}

		#endregion

		#region p4 annotate

		/// <summary>
		/// Runs the annotate p4 command
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Depot path to the file</param>
		/// <param name="options">Options for the anotate command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of annotate records</returns>
		public static async Task<List<AnnotateRecord>> AnnotateAsync(this IPerforceConnection connection, string fileSpec, AnnotateOptions options, CancellationToken cancellationToken = default)
		{
			return (await TryAnnotateAsync(connection, fileSpec, options, cancellationToken)).Data;
		}

		/// <summary>
		/// Runs the annotate p4 command 
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="fileSpec">Depot path to the file</param>
		/// <param name="options">Options for the anotate command</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns>List of annotate records</returns>
		public static Task<PerforceResponseList<AnnotateRecord>> TryAnnotateAsync(this IPerforceConnection connection, string fileSpec, AnnotateOptions options, CancellationToken cancellationToken = default)
		{
			List<string> arguments = [];
			if ((options & AnnotateOptions.IncludeDeletedFilesAndLines) != 0)
			{
				arguments.Add("-a");
			}
			if ((options & AnnotateOptions.IgnoreWhiteSpaceChanges) != 0)
			{
				arguments.Add("-db");
			}
			if ((options & AnnotateOptions.OutputUserAndDate) != 0)
			{
				arguments.Add("-u");
			}
			if ((options & AnnotateOptions.FollowIntegrations) != 0)
			{
				arguments.Add("-I");
			}
			arguments.Add(fileSpec);

			return CommandAsync<AnnotateRecord>(connection, "annotate", arguments, null, cancellationToken);
		}

		#endregion

		#region Other

		/// <summary>
		/// Gets the current stream that a client is connected to
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static async Task<string?> GetCurrentStreamAsync(this IPerforceConnection connection, CancellationToken cancellationToken = default)
		{
			return await TryGetCurrentStreamAsync(connection, cancellationToken);
		}

		/// <summary>
		/// Gets the current stream that a client is connected to
		/// </summary>
		/// <param name="connection">Connection to the Perforce server</param>
		/// <param name="cancellationToken">Token used to cancel the operation</param>
		/// <returns></returns>
		public static async Task<string?> TryGetCurrentStreamAsync(this IPerforceConnection connection, CancellationToken cancellationToken = default)
		{
			PerforceResponse<ClientRecord> response = await connection.TryGetClientAsync(null, cancellationToken);
			if (response.Succeeded)
			{
				return response.Data.Stream;
			}
			else
			{
				return null;
			}
		}

		#endregion
	}
}
