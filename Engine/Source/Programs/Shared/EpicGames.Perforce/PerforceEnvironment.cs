// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using EpicGames.Core;
using Microsoft.Win32;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Interface describing a set of perforce environment settings
	/// </summary>
	public interface IPerforceEnvironment
	{
		/// <summary>
		/// Get the value of a particular variable
		/// </summary>
		/// <param name="name">Name of the variable to retrieve</param>
		/// <returns>The variable value, or null if it's not set</returns>
		string? GetValue(string name);
	}

	/// <summary>
	/// The global Perforce environment
	/// </summary>
	public class GlobalPerforceEnvironment : IPerforceEnvironment
	{
		/// <summary>
		/// Environment variables in the global environment
		/// </summary>
		protected Dictionary<string, string> Variables { get; } = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Constructor
		/// </summary>
		public GlobalPerforceEnvironment()
		{
			foreach (DictionaryEntry entry in Environment.GetEnvironmentVariables().OfType<DictionaryEntry>())
			{
				(string? key, string? value) = ((string?)entry.Key, (string?)entry.Value);
				if (!String.IsNullOrEmpty(key) && !String.IsNullOrEmpty(value))
				{
					if (key.StartsWith("P4", StringComparison.OrdinalIgnoreCase))
					{
						Variables[key] = value;
					}
				}
			}
		}

		/// <inheritdoc/>
		public string? GetValue(string name)
		{
			Variables.TryGetValue(name, out string? value);
			return String.IsNullOrEmpty(value) ? null : value;
		}
	}

	/// <summary>
	/// Default global environment used by Linux and MacOS, which reads settings from the registry.
	/// </summary>
	class WindowsGlobalPerforceEnvironment : GlobalPerforceEnvironment
	{
		public WindowsGlobalPerforceEnvironment()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				using (RegistryKey? key = Registry.CurrentUser.OpenSubKey("SOFTWARE\\perforce\\environment", false))
				{
					if (key != null)
					{
						foreach (string valueName in key.GetValueNames())
						{
							string? value = key.GetValue(valueName) as string;
							if (!String.IsNullOrEmpty(value) && !Variables.ContainsKey(valueName))
							{
								Variables[valueName] = value;
							}
						}
					}
				}
			}
		}
	}

	/// <summary>
	/// Environment variables read from a file
	/// </summary>
	[DebuggerDisplay("{Location}")]
	public class PerforceEnvironmentFile : IPerforceEnvironment
	{
		/// <summary>
		/// The parent environment block
		/// </summary>
		public IPerforceEnvironment Parent { get; }

		/// <summary>
		/// Location of the file containing these variables
		/// </summary>
		public FileReference Location { get; }

		readonly Dictionary<string, string> _variables = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parent"></param>
		/// <param name="location"></param>
		internal PerforceEnvironmentFile(IPerforceEnvironment parent, FileReference location)
		{
			Parent = parent;
			Location = location;

			string[] lines = FileReference.ReadAllLines(location);
			foreach (string line in lines)
			{
				string trimLine = line.Trim();
				int equalsIdx = trimLine.IndexOf('=', StringComparison.Ordinal);
				if (equalsIdx != -1)
				{
					string name = trimLine.Substring(0, equalsIdx).TrimEnd();
					string value = trimLine.Substring(equalsIdx + 1).TrimStart();
					_variables[name] = value;
				}
			}
		}

		/// <inheritdoc/>
		public string? GetValue(string name)
		{
			if (_variables.TryGetValue(name, out string? value))
			{
				return value;
			}
			else
			{
				return Parent.GetValue(name);
			}
		}
	}

	/// <summary>
	/// Static methods for retrieving the Perforce environment
	/// </summary>
	public static class PerforceEnvironment
	{
		/// <summary>
		/// Default environment regardless of directroy.
		/// </summary>
		public static IPerforceEnvironment Default { get; } = CreateDefaultEnvironment();

		static readonly Dictionary<DirectoryReference, IPerforceEnvironment> s_directoryToEnvironment = [];

		static IPerforceEnvironment CreateDefaultEnvironment()
		{
			IPerforceEnvironment environment;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				environment = new WindowsGlobalPerforceEnvironment();
			}
			else
			{
				environment = new GlobalPerforceEnvironment();
			}

			string? enviroValue = environment.GetValue("P4ENVIRO");
			if (enviroValue == null && !RuntimeInformation.IsOSPlatform(OSPlatform.Windows)) // Linux AND MacOS, despite what P4 docs say: https://www.perforce.com/manuals/v20.1/cmdref/Content/CmdRef/P4ENVIRO.html
			{
				string? homeDir = Environment.GetFolderPath(Environment.SpecialFolder.UserProfile);
				if (homeDir != null)
				{
					enviroValue = Path.Combine(homeDir, ".p4enviro");
				}
			}

			if (enviroValue != null)
			{
				FileReference location = new(enviroValue);
				if (FileReference.Exists(location))
				{
					environment = new PerforceEnvironmentFile(environment, location);
				}
			}

			return environment;
		}

		/// <summary>
		/// Read the default Perforce settings reading any config file from the given directory
		/// </summary>
		/// <param name="directory">The directory to read from</param>
		/// <returns>Default settings for the given directory</returns>
		public static IPerforceEnvironment FromDirectory(DirectoryReference directory)
		{
			IPerforceEnvironment? environment;
			if (!s_directoryToEnvironment.TryGetValue(directory, out environment))
			{
				DirectoryReference? parentDirectory = directory.ParentDirectory;
				if (parentDirectory == null)
				{
					environment = Default;
				}
				else
				{
					environment = FromDirectory(parentDirectory);

					string? configFileName = environment.GetValue("P4CONFIG");
					if (configFileName != null)
					{
						FileReference location = FileReference.Combine(directory, configFileName);
						if (FileReference.Exists(location))
						{
							environment = new PerforceEnvironmentFile(environment, location);
						}
					}
				}
				s_directoryToEnvironment[directory] = environment;
			}
			return environment;
		}
	}
}
