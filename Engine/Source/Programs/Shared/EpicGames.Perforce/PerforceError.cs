// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
#pragma warning disable CS1591

	/// <summary>
	/// Error severity codes. Taken from the p4java documentation.
	/// </summary>
	public enum PerforceSeverityCode
	{
		Empty = 0,
		Info = 1,
		Warning = 2,
		Failed = 3,
		Fatal = 4,
	}

	/// <summary>
	/// Generic error codes that can be returned by the Perforce server. Taken from the p4java documentation.
	/// </summary>
	public enum PerforceGenericCode
	{
		None = 0,
		Usage = 1,
		Unknown = 2,
		Context = 3,
		Illegal = 4,
		NotYet = 5,
		Protect = 6,
		Empty = 17,
		Fault = 33,
		Client = 34,
		Admin = 35,
		Config = 36,
		Upgrade = 37,
		Comm = 38,
		TooBig = 39,
	}

#pragma warning restore CS1591

	/// <summary>
	/// Represents a error return value from Perforce.
	/// </summary>
	public class PerforceError
	{
		/// <summary>
		/// The severity of this error
		/// </summary>
		[PerforceTagAttribute("severity")]
		public PerforceSeverityCode Severity { get; set; }

		/// <summary>
		/// The generic error code associated with this message
		/// </summary>
		[PerforceTagAttribute("generic")]
		public PerforceGenericCode Generic { get; set; }

		/// <summary>
		/// The message text
		/// </summary>
		[PerforceTagAttribute("data")]
		public string Data { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private PerforceError()
		{
			Data = null!;
		}

		/// <summary>
		/// Formats this error for display in the debugger
		/// </summary>
		/// <returns>String representation of this object</returns>
		public override string ToString()
		{
			return String.Format("{0}: {1} (Generic={2})", Severity, Data.TrimEnd(), Generic);
		}
	}
}
