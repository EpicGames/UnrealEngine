// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents an exception specific to perforce
	/// </summary>
	public class PerforceException : Exception
	{
		/// <summary>
		/// For errors returned by the server, contains the error record
		/// </summary>
		public PerforceError? Error { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message for the exception</param>
		public PerforceException(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">Format string</param>
		/// <param name="args">Arguments for the formatted string</param>
		public PerforceException(string format, params object[] args)
			: base(String.Format(format, args))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="error">The error from the server</param>
		public PerforceException(PerforceError error)
			: base(error.ToString())
		{
			Error = error;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceException(string message, PerforceException innerException)
			: base(message, innerException)
		{
			Error = innerException.Error;
		}
	}
}
