// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Generic class for parsing "info" responses from Perforce
	/// </summary>
	public class PerforceInfo
	{
		/// <summary>
		/// Message data
		/// </summary>
		[PerforceTag("data")]
		public string Data { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private PerforceInfo()
		{
			Data = null!;
		}

		/// <summary>
		/// Formats this error for display in the debugger
		/// </summary>
		/// <returns>String representation of this object</returns>
		public override string? ToString()
		{
			return Data;
		}
	}
}
