// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// 
	/// </summary>
	public enum PerforceIoCommand
	{
		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("open")]
		Open,

		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("write")]
		Write,

		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("close")]
		Close,

		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("unlink")]
		Unlink,
	}

	/// <summary>
	/// 
	/// </summary>
	public class PerforceIo
	{
		/// <summary>
		/// The severity of this error
		/// </summary>
		[PerforceTagAttribute("file")]
		public int File { get; set; }

		/// <summary>
		/// The generic error code associated with this message
		/// </summary>
		[PerforceTagAttribute("command")]
		public PerforceIoCommand Command { get; set; }

		/// <summary>
		/// The message text
		/// </summary>
		[PerforceTagAttribute("payload")]
		public ReadOnlyMemory<byte> Payload { get; set; }
	}
}
