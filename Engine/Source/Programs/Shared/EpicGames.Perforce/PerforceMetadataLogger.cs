// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Logger which adds Perforce depot path and changelist information to file annotations
	/// </summary>
	public class PerforceMetadataLogger : ILogger
	{
		class ClientView
		{
			public readonly DirectoryReference BaseDir;
			public readonly PerforceViewMap ViewMap;
			public readonly int Change;

			public ClientView(DirectoryReference baseDir, PerforceViewMap viewMap, int change)
			{
				BaseDir = baseDir;
				ViewMap = viewMap;
				Change = change;
			}
		}

		readonly ILogger _inner;
		readonly List<ClientView> _clients = [];

		/// <summary>
		/// Accessor for the inner logger
		/// </summary>
		public ILogger Inner => _inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner"></param>
		public PerforceMetadataLogger(ILogger inner)
		{
			_inner = inner;
		}

		/// <summary>
		/// Adds a new client to be included in the mapping
		/// </summary>
		/// <param name="baseDir">Base directory for the client</param>
		/// <param name="depotPath">Depot path for the workspace mapping, in the form //foo/bar...</param>
		/// <param name="change">Changelist for the client</param>
		public void AddClientView(DirectoryReference baseDir, string depotPath, int change)
		{
			PerforceViewMap viewMap = new();
			viewMap.Entries.Add(new PerforceViewMapEntry(true, depotPath, "..."));
			AddClientView(baseDir, viewMap, change);
		}

		/// <summary>
		/// Adds a new client to be included in the mapping
		/// </summary>
		public void AddClientView(DirectoryReference baseDir, PerforceViewMap viewMap, int change)
		{
			_clients.Add(new ClientView(baseDir, viewMap.Invert(), change));
		}

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			JsonLogEvent logEvent = JsonLogEvent.FromLoggerState(logLevel, eventId, state, exception, formatter);
			try
			{
				logEvent = new JsonLogEvent(logLevel, eventId, logEvent.LineIndex, logEvent.LineCount, Annotate(logEvent.Data));
			}
			catch (Exception ex)
			{
				_inner.LogWarning(KnownLogEvents.Systemic_LogEventMatcher, ex, "Unable to annotate source files with Perforce metadata: {Message}", ex.Message);
			}
			_inner.Log(logLevel, eventId, logEvent, null, JsonLogEvent.Format);
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel) => _inner.IsEnabled(logLevel);

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _inner.BeginScope(state);

		static bool ReadFirstLogProperty(ref Utf8JsonReader reader)
		{
			// Enter the main object
			if (!reader.Read() || reader.TokenType != JsonTokenType.StartObject)
			{
				return false;
			}

			// Find the 'properties' property
			for (; ; )
			{
				if (!reader.Read() || reader.TokenType != JsonTokenType.PropertyName)
				{
					return false;
				}
				if (reader.ValueTextEquals(LogEventPropertyName.Properties))
				{
					break;
				}
				reader.Skip();
			}

			// Enter the properties object
			if (!reader.Read() || reader.TokenType != JsonTokenType.StartObject)
			{
				return false;
			}

			// Find the first structured property object
			return ReadNextLogProperty(ref reader);
		}

		static bool ReadNextLogProperty(ref Utf8JsonReader reader)
		{
			for (; ; )
			{
				// Move to the next property name
				if (!reader.Read() || reader.TokenType != JsonTokenType.PropertyName)
				{
					return false;
				}

				// Move to the next property value
				if (!reader.Read())
				{
					return false;
				}

				// If it's an object, enter it
				if (reader.TokenType == JsonTokenType.StartObject)
				{
					return true;
				}

				// Otherwise skip this value
				reader.Skip();
			}
		}

		static readonly Utf8String s_sourceFileType = LogValueType.SourceFile;
		static readonly Utf8String s_assetType = LogValueType.Asset;

		static readonly Utf8String s_file = new("file");

		ReadOnlyMemory<byte> Annotate(ReadOnlyMemory<byte> data)
		{
			ReadOnlyMemory<byte> output = data;
			int shift = 0;

			Utf8JsonReader reader = new(data.Span);
			for (bool valid = ReadFirstLogProperty(ref reader); valid; valid = ReadNextLogProperty(ref reader))
			{
				ReadOnlySpan<byte> type = [];
				ReadOnlySpan<byte> text = [];
				string? file = null;

				// Get the $type and $text properties
				while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
				{
					if (type.Length == 0 && reader.ValueTextEquals(LogEventPropertyName.Type))
					{
						if (reader.Read() && reader.TokenType == JsonTokenType.String)
						{
							type = reader.GetUtf8String();
							continue;
						}
					}
					if (text.Length == 0 && reader.ValueTextEquals(LogEventPropertyName.Text))
					{
						if (reader.Read() && reader.TokenType == JsonTokenType.String)
						{
							text = reader.GetUtf8String();
							continue;
						}
					}
					if (file == null && reader.ValueTextEquals(s_file))
					{
						if (reader.Read() && reader.TokenType == JsonTokenType.String)
						{
							file = reader.GetString();
							continue;
						}
					}
					reader.Skip();
				}

				// If we're at the end of the object, append any additional data
				byte[]? annotationBytes = GetAnnotations(type, text, file);
				if (annotationBytes != null)
				{
					int position = (int)reader.TokenStartIndex + shift;

					byte[] newOutput = new byte[output.Length + annotationBytes.Length];
					output.Span.Slice(0, position).CopyTo(newOutput);
					annotationBytes.CopyTo(newOutput.AsSpan(position));
					output.Span.Slice(position).CopyTo(newOutput.AsSpan(position + annotationBytes.Length));

					output = newOutput;
					shift += annotationBytes.Length;
				}
			}

			return output;
		}

		byte[]? GetAnnotations(ReadOnlySpan<byte> type, ReadOnlySpan<byte> text, string? file)
		{
			if (type.SequenceEqual(s_sourceFileType) || type.SequenceEqual(s_assetType))
			{
				StringBuilder annotations = new();

				file ??= Encoding.UTF8.GetString(text);

				file = file.Replace(@"\\", Path.DirectorySeparatorChar.ToString(), StringComparison.Ordinal);
				file = file.Replace('\\', Path.DirectorySeparatorChar);

				foreach (ClientView client in _clients)
				{
					FileReference location = FileReference.Combine(client.BaseDir, file);
					if (location.IsUnderDirectory(client.BaseDir))
					{
						string relativePath = location.MakeRelativeTo(client.BaseDir).Replace('\\', '/');
						annotations.Append($",\"relativePath\":\"{JsonEncodedText.Encode(relativePath)}\"");

						if (client.ViewMap.TryMapFile(relativePath, StringComparison.OrdinalIgnoreCase, out string depotFile))
						{
							depotFile = $"{depotFile}@{client.Change}";
							annotations.Append($",\"depotPath\":\"{JsonEncodedText.Encode(depotFile)}\"");
							break;
						}
					}
				}

				return Encoding.UTF8.GetBytes(annotations.ToString());
			}
			return null;
		}
	}
}
