// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Buffers.Text;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// The type of a value returned by Perforce
	/// </summary>
	public enum PerforceValueType
	{
		/// <summary>
		/// A utf-8 encoded string
		/// </summary>
		String,

		/// <summary>
		/// A 32-bit integer
		/// </summary>
		Integer,
	}

	/// <summary>
	/// Wrapper for values returned by Perforce
	/// </summary>
	public readonly struct PerforceValue
	{
		/// <summary>
		/// The raw data for the value, including type, size, and payload
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">The data to construct this value from</param>
		public PerforceValue(ReadOnlyMemory<byte> data)
		{
			Data = data;
		}

		/// <summary>
		/// Create a value from a string
		/// </summary>
		/// <param name="text">String value to serialize</param>
		public PerforceValue(string text)
		{
			int length = Encoding.UTF8.GetByteCount(text);

			byte[] buffer = new byte[1 + 4 + length];
			buffer[0] = (byte)'s';
			BinaryPrimitives.WriteInt32LittleEndian(buffer.AsSpan(1, 4), length);
			Encoding.UTF8.GetBytes(text, buffer.AsSpan(5));

			Data = buffer;
		}

		/// <summary>
		/// Determines if the value is empty
		/// </summary>
		/// <returns>True if the value is empty</returns>
		public bool IsEmpty => Data.IsEmpty;

		/// <summary>
		/// Accessor for the type of value stored by this struct
		/// </summary>
		public PerforceValueType Type
		{
			get
			{
				switch (Data.Span[0])
				{
					case (byte)'s':
						return PerforceValueType.String;
					case (byte)'i':
						return PerforceValueType.Integer;
					default:
						throw new NotImplementedException("Unknown/unsupported value type");
				}
			}
		}

		/// <summary>
		/// Converts the value to a boolean
		/// </summary>
		/// <returns>The boolean value</returns>
		public bool AsBool()
		{
			Utf8String @string = GetString();
			return @string.Length == 0 || @string == StringConstants.True;
		}

		/// <summary>
		/// Converts the value to an integer
		/// </summary>
		/// <returns>Integer value</returns>
		public int AsInteger()
		{
			if (Type == PerforceValueType.Integer)
			{
				return GetInteger();
			}
			else if (Type == PerforceValueType.String)
			{
				Utf8String @string = GetString();

				int value;
				int bytesConsumed;
				if (Utf8Parser.TryParse(@string.Span, out value, out bytesConsumed) && bytesConsumed == @string.Length)
				{
					return value;
				}
				else if (@string == StringConstants.New || @string == StringConstants.None)
				{
					return -1;
				}
				else if (@string.Length > 0 && @string[0] == '#' && Utf8Parser.TryParse(@string.Span.Slice(1), out value, out bytesConsumed) && bytesConsumed + 1 == @string.Length)
				{
					return value;
				}
				else if (@string == StringConstants.Default)
				{
					return PerforceReflection.DefaultChange;
				}
				else
				{
					throw new PerforceException($"Unable to parse {@string} as an integer");
				}
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Converts this value to a long
		/// </summary>
		/// <returns>The converted value</returns>
		public long AsLong()
		{
			Utf8String @string = AsString();

			long value;
			int bytesConsumed;
			if (!Utf8Parser.TryParse(AsString().Span, out value, out bytesConsumed) || bytesConsumed != @string.Length)
			{
				throw new PerforceException($"Unable to parse {ToString()} as a long value");
			}

			return value;
		}

		/// <summary>
		/// Converts this value to a DateTimeOffset
		/// </summary>
		/// <returns>The converted value</returns>
		public DateTimeOffset AsDateTimeOffset()
		{
			string text = ToString();
			return DateTimeOffset.Parse(Regex.Replace(text, "[a-zA-Z ]*$", "")); // Strip timezone name (eg. "EST")
		}

		/// <summary>
		/// Converts this value to a string
		/// </summary>
		/// <returns>The converted value</returns>
		public Utf8String AsString()
		{
			switch (Type)
			{
				case PerforceValueType.String:
					return GetString();
				case PerforceValueType.Integer:
					return new Utf8String(GetInteger().ToString());
				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Gets the contents of this string
		/// </summary>
		/// <returns></returns>
		public int GetInteger()
		{
			if (Type != PerforceValueType.Integer)
			{
				throw new InvalidOperationException("Value is not an integer");
			}
			return BinaryPrimitives.ReadInt32LittleEndian(Data.Span.Slice(1));
		}

		/// <summary>
		/// Gets the contents of this string
		/// </summary>
		/// <returns></returns>
		public Utf8String GetString()
		{
			if (IsEmpty)
			{
				return Utf8String.Empty;
			}
			if (Type != PerforceValueType.String)
			{
				throw new InvalidOperationException("Value is not a string");
			}
			return new Utf8String(Data.Slice(5));
		}

		/// <summary>
		/// Convert to a string
		/// </summary>
		/// <returns>String representation of the value</returns>
		public override string ToString()
		{
			return AsString().ToString();
		}
	}

	/// <summary>
	/// Low-overhead record type for generic responses
	/// </summary>
	public class PerforceRecord
	{
		/// <summary>
		/// The rows in this record
		/// </summary>
		public List<KeyValuePair<Utf8String, PerforceValue>> Rows { get; } = [];

		/// <summary>
		/// Create a record from a set of fields
		/// </summary>
		/// <param name="fields">Fields to serialize</param>
		/// <param name="numberedListElements">Whether to number list elements when flattening the record</param>
		/// <returns>Record containing the given fields</returns>
		public static PerforceRecord FromFields(IEnumerable<KeyValuePair<string, object>> fields, bool numberedListElements)
		{
			PerforceRecord record = new();
			foreach ((string name, object value) in fields)
			{
				if (value is string str)
				{
					record.Rows.Add(new KeyValuePair<Utf8String, PerforceValue>(new Utf8String(name), new PerforceValue(str)));
				}
				else if (value is List<string> list)
				{
					if (numberedListElements)
					{
						record.Rows.AddRange(Enumerable.Range(0, list.Count).Select(x => new KeyValuePair<Utf8String, PerforceValue>(new Utf8String($"{name}{x}"), new PerforceValue(list[x]))));
					}
					else
					{
						record.Rows.AddRange(list.Select(x => new KeyValuePair<Utf8String, PerforceValue>(new Utf8String(name), new PerforceValue(x))));
					}
				}
				else
				{
					throw new PerforceException("Unsupported formatting type for {0}", name);
				}
			}
			return record;
		}

		/// <summary>
		/// Copy this record into the given array of values. This method is O(n) if every record key being in the list of keys in the same order, but O(n^2) if not.
		/// </summary>
		/// <param name="keys">List of keys to parse</param>
		/// <param name="values">Array to receive the list of values</param>
		public void CopyInto(Utf8String[] keys, PerforceValue[] values)
		{
			// Clear out the current values array
			for (int valueIdx = 0; valueIdx < values.Length; valueIdx++)
			{
				values[valueIdx] = new PerforceValue();
			}

			// Parse all the keys
			int rowIdx = 0;
			while (rowIdx < Rows.Count)
			{
				int initialRowIdx = rowIdx;

				// Find the key that matches this row.
				Utf8String rowKey = Rows[rowIdx].Key;
				for (int keyIdx = 0; keyIdx < keys.Length; keyIdx++)
				{
					Utf8String key = keys[keyIdx];
					if (rowKey == key)
					{
						// Copy the value to the output array, then move to the next row, and try to match that against the next key.
						values[keyIdx] = Rows[rowIdx].Value;
						if (++rowIdx == Rows.Count)
						{
							break;
						}
						rowKey = Rows[rowIdx].Key;
					}
				}

				// If we didn't match any key name for this row, move to the next one
				if (rowIdx == initialRowIdx)
				{
					rowIdx++;
				}
			}
		}

		/// <summary>
		/// Serialize to a single block of data
		/// </summary>
		/// <returns>Serialized data</returns>
		public byte[] Serialize()
		{
			MemoryStream stream = new();
			using (BinaryWriter writer = new(stream))
			{
				writer.Write((byte)'{');
				foreach ((Utf8String name, PerforceValue value) in Rows)
				{
					writer.Write('s');
					writer.Write((int)name.Length);
					writer.Write(name.Span);
					writer.Write(value.Data.Span);
				}
				writer.Write((byte)'0');
			}
			return stream.ToArray();
		}

		/// <summary>
		/// Serializes a list of key/value pairs into binary format.
		/// </summary>
		/// <param name="keyValuePairs">List of key value pairs</param>
		/// <returns>Serialized record data</returns>
		public static byte[] Serialize(List<KeyValuePair<string, object>> keyValuePairs)
		{
			MemoryStream stream = new();
			using (BinaryWriter writer = new(stream))
			{
				writer.Write((byte)'{');
				foreach (KeyValuePair<string, object> keyValuePair in keyValuePairs)
				{
					writer.Write('s');
					byte[] keyBytes = Encoding.UTF8.GetBytes(keyValuePair.Key);
					writer.Write((int)keyBytes.Length);
					writer.Write(keyBytes);

					if (keyValuePair.Value is string str)
					{
						writer.Write('s');
						byte[] valueBytes = Encoding.UTF8.GetBytes(str);
						writer.Write((int)valueBytes.Length);
						writer.Write(valueBytes);
					}
					else
					{
						throw new PerforceException("Unsupported formatting type for {0}", keyValuePair.Key);
					}
				}
				writer.Write((byte)'0');
			}
			return stream.ToArray();
		}
	}
}
