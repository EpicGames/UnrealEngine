// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Buffers.Text;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Reflection.Emit;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// String constants for perforce values
	/// </summary>
	static class StringConstants
	{
		public static readonly Utf8String True = new("true");
		public static readonly Utf8String False = new("false");
		public static readonly Utf8String New = new("new");
		public static readonly Utf8String None = new("none");
		public static readonly Utf8String Default = new("default");
	}

	/// <summary>
	/// Delegate type for creating a record instance
	/// </summary>
	/// <returns>New instance</returns>
	public delegate object CreateRecordDelegate();

	/// <summary>
	/// Information about a nested record
	/// </summary>
	record struct NestedRecordInfo(PropertyInfo PropertyInfo, CreateRecordDelegate CreateInstance);

	/// <summary>
	/// Name and rank of a tagged property within a record
	/// </summary>
	record struct TaggedPropertyNameAndRank(Utf8String Tag, int Rank);

	/// <summary>
	/// Stores cached information about a property with a <see cref="PerforceTagAttribute"/> attribute.
	/// </summary>
	class TaggedPropertyInfo
	{
		/// <summary>
		/// Name of the tag. Specified in the attribute or inferred from the field name.
		/// </summary>
		public Utf8String Tag { get; }

		/// <summary>
		/// Whether this tag is optional or not.
		/// </summary>
		public bool Optional { get; }

		/// <summary>
		/// List of nested records before getting to the target object
		/// </summary>
		public NestedRecordInfo[] ParentRecords { get; }

		/// <summary>
		/// The property containing the value of this data.
		/// </summary>
		public PropertyInfo PropertyInfo { get; }

		/// <summary>
		/// Writes an instance of this field from an object
		/// </summary>
		public Action<IMemoryWriter, object> Write { get; set; }

		/// <summary>
		/// Parser for this field type
		/// </summary>
		public Action<object, int> ReadFromInteger { get; set; }

		/// <summary>
		/// Parser for this field type
		/// </summary>
		public Action<object, Utf8String> ReadFromString { get; set; }

		/// <summary>
		/// Index into the bitmask of required types
		/// </summary>
		public ulong RequiredTagBitMask { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		/// <param name="optional"></param>
		/// <param name="parentRecords"></param>
		/// <param name="propertyInfo"></param>
		/// <param name="requiredTagBitMask"></param>
		public TaggedPropertyInfo(Utf8String name, bool optional, NestedRecordInfo[] parentRecords, PropertyInfo propertyInfo, ulong requiredTagBitMask)
		{
			Tag = name;
			Optional = optional;
			ParentRecords = parentRecords;
			PropertyInfo = propertyInfo;
			RequiredTagBitMask = requiredTagBitMask;
			Write = (obj, writer) => throw new PerforceException($"Field {name} does not have a serializer.");
			ReadFromInteger = (obj, value) => throw new PerforceException($"Field {name} was not expecting an integer value.");
			ReadFromString = (obj, str) => throw new PerforceException($"Field {name} was not expecting a string value.");
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder fullName = new(Tag.ToString());
			if (ParentRecords.Length > 0)
			{
				fullName.Append('0');
				for (int idx = 1; idx < ParentRecords.Length; idx++)
				{
					fullName.Append(",0");
				}
			}
			return fullName.ToString();
		}
	}

	/// <summary>
	/// Stores cached information about a record
	/// </summary>
	class CachedRecordInfo
	{
		/// <summary>
		/// Type of the record
		/// </summary>
		public Type Type { get; }

		/// <summary>
		/// Method to construct this record
		/// </summary>
		public CreateRecordDelegate CreateInstance { get; }

		/// <summary>
		/// List of fields in the record. These should be ordered to match P4 output for maximum efficiency.
		/// </summary>
		public List<TaggedPropertyInfo> Properties { get; } = [];

		/// <summary>
		/// Map of name to tag info
		/// </summary>
		public Dictionary<TaggedPropertyNameAndRank, TaggedPropertyInfo> NameAndRankToInfo { get; set; } = [];

		/// <summary>
		/// Bitmask of all the required tags. Formed by bitwise-or'ing the RequiredTagBitMask fields for each required CachedTagInfo.
		/// </summary>
		public ulong RequiredTagsBitMask { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The record type</param>
		/// <param name="createInstance">Method used to create an instance of this record</param>
		public CachedRecordInfo(Type type, CreateRecordDelegate createInstance)
		{
			Type = type;
			CreateInstance = createInstance;
		}
	}

	/// <summary>
	/// Information about an enum
	/// </summary>
	class CachedEnumInfo
	{
		/// <summary>
		/// The enum type
		/// </summary>
		public Type _enumType;

		/// <summary>
		/// Whether the enum has the [Flags] attribute
		/// </summary>
		public bool _bHasFlagsAttribute;

		/// <summary>
		/// Map of name to value
		/// </summary>
		public Dictionary<Utf8String, int> _nameToValue = [];

		/// <summary>
		/// Map of value to name
		/// </summary>
		public Dictionary<int, Utf8String> _valueToName = [];

		/// <summary>
		/// List of name/value pairs
		/// </summary>
		public List<KeyValuePair<string, int>> _nameValuePairs = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="enumType">The type to construct from</param>
		public CachedEnumInfo(Type enumType)
		{
			_enumType = enumType;

			_bHasFlagsAttribute = enumType.GetCustomAttribute<FlagsAttribute>() != null;

			FieldInfo[] fields = enumType.GetFields(BindingFlags.Public | BindingFlags.Static);
			foreach (FieldInfo field in fields)
			{
				PerforceEnumAttribute? attribute = field.GetCustomAttribute<PerforceEnumAttribute>();
				if (attribute != null)
				{
					object? value = field.GetValue(null);
					if (value != null)
					{
						Utf8String name = new(attribute.Name);
						_nameToValue[name] = (int)value;
						_valueToName[(int)value] = name;

						_nameValuePairs.Add(new KeyValuePair<string, int>(attribute.Name, (int)value));
					}
				}
			}
		}

		/// <summary>
		/// Gets the name of a particular enum value
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public Utf8String GetName(int value) => _valueToName[value];

		/// <summary>
		/// Parses the given integer as an enum
		/// </summary>
		/// <param name="value">The value to convert to an enum</param>
		/// <returns>The enum value corresponding to the given value</returns>
		public object ParseInteger(int value)
		{
			return Enum.ToObject(_enumType, value);
		}

		/// <summary>
		/// Parses the given text as an enum
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <returns>The enum value corresponding to the given text</returns>
		public object ParseString(Utf8String text)
		{
			return Enum.ToObject(_enumType, ParseToInteger(text));
		}

		/// <summary>
		/// Parses the given text as an enum
		/// </summary>
		/// <param name="name"></param>
		/// <returns></returns>
		public int ParseToInteger(Utf8String name)
		{
			if (_bHasFlagsAttribute)
			{
				int result = 0;
				for (int offset = 0; offset < name.Length;)
				{
					if (name.Span[offset] == (byte)' ')
					{
						offset++;
					}
					else
					{
						// Find the end of this name
						int startOffset = ++offset;
						while (offset < name.Length && name.Span[offset] != (byte)' ')
						{
							offset++;
						}

						// Take the subset
						Utf8String item = name.Slice(startOffset, offset - startOffset);

						// Get the value
						int itemValue;
						if (_nameToValue.TryGetValue(item, out itemValue))
						{
							result |= itemValue;
						}
					}
				}
				return result;
			}
			else
			{
				int result;
				_nameToValue.TryGetValue(name, out result);
				return result;
			}
		}

		/// <summary>
		/// Parses an enum value, using PerforceEnumAttribute markup for names.
		/// </summary>
		/// <param name="value">Value of the enum.</param>
		/// <returns>Text for the enum.</returns>
		public string GetEnumText(int value)
		{
			if (_bHasFlagsAttribute)
			{
				List<string> names = [];

				int combinedIntegerValue = 0;
				foreach (KeyValuePair<string, int> pair in _nameValuePairs)
				{
					if ((value & pair.Value) != 0)
					{
						names.Add(pair.Key);
						combinedIntegerValue |= pair.Value;
					}
				}

				if (combinedIntegerValue != value)
				{
					throw new ArgumentException($"Invalid enum value {value}");
				}

				return String.Join(" ", names);
			}
			else
			{
				string? name = null;
				foreach (KeyValuePair<string, int> pair in _nameValuePairs)
				{
					if (value == pair.Value)
					{
						name = pair.Key;
						break;
					}
				}

				if (name == null)
				{
					throw new ArgumentException($"Invalid enum value {value}");
				}
				return name;
			}
		}
	}

	/// <summary>
	/// Utility methods for converting to/from native types
	/// </summary>
	static class PerforceReflection
	{
		/// <summary>
		/// Unix epoch; used for converting times back into C# datetime objects
		/// </summary>
		public static readonly DateTime UnixEpoch = new(1970, 1, 1, 0, 0, 0, DateTimeKind.Utc);

		/// <summary>
		/// Constant for the default changelist, where valid.
		/// </summary>
		public const int DefaultChange = -2;

		/// <summary>
		/// Cached map of enum types to a lookup mapping from p4 strings to enum values.
		/// </summary>
		static readonly ConcurrentDictionary<Type, CachedEnumInfo> s_enumTypeToInfo = new();

		/// <summary>
		/// Cached set of record 
		/// </summary>
		static readonly ConcurrentDictionary<Type, CachedRecordInfo> s_recordTypeToInfo = new();

		/// <summary>
		/// Default type for info
		/// </summary>
		public static CachedRecordInfo InfoRecordInfo = GetCachedRecordInfo(typeof(PerforceInfo));

		/// <summary>
		/// Default type for errors
		/// </summary>
		public static CachedRecordInfo ErrorRecordInfo = GetCachedRecordInfo(typeof(PerforceError));

		/// <summary>
		/// Default type for errors
		/// </summary>
		public static CachedRecordInfo IoRecordInfo = GetCachedRecordInfo(typeof(PerforceIo));

		/// <summary>
		/// Serializes a sequence of objects to a stream
		/// </summary>
		/// <param name="obj">Object to serialize</param>
		/// <param name="writer">Writer for output data</param>
		public static void Serialize(object obj, IMemoryWriter writer)
		{
			CachedRecordInfo recordInfo = GetCachedRecordInfo(obj.GetType());
			foreach (TaggedPropertyInfo tagInfo in recordInfo.Properties)
			{
				object? value = tagInfo.PropertyInfo.GetValue(obj);
				if (value != null)
				{
					WriteUtf8StringWithTag(writer, tagInfo.Tag);
					tagInfo.Write(writer, value!);
				}
			}
		}

		static void WriteIntegerWithTag(IMemoryWriter writer, int value)
		{
			Span<byte> span = writer.GetSpanAndAdvance(5);
			span[0] = (byte)'i';
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(1, 4), value);
		}

		static void WriteStringWithTag(IMemoryWriter writer, string str)
		{
			int length = Encoding.UTF8.GetByteCount(str);
			Span<byte> span = writer.GetSpanAndAdvance(1 + length + 4);
			span[0] = (byte)'s';
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(1, 4), length);
			Encoding.UTF8.GetBytes(str, span.Slice(5));
		}

		static void WriteUtf8StringWithTag(IMemoryWriter writer, Utf8String str)
		{
			Span<byte> span = writer.GetSpanAndAdvance(1 + str.Length + 4);
			span[0] = (byte)'s';
			BinaryPrimitives.WriteInt32LittleEndian(span.Slice(1, 4), str.Length);
			str.Span.CopyTo(span.Slice(5));
		}

		/// <summary>
		/// Gets a mapping of flags to enum values for the given type
		/// </summary>
		/// <param name="enumType">The enum type to retrieve flags for</param>
		/// <returns>Map of name to enum value</returns>
		static CachedEnumInfo GetCachedEnumInfo(Type enumType)
		{
			CachedEnumInfo? enumInfo;
			if (!s_enumTypeToInfo.TryGetValue(enumType, out enumInfo))
			{
				enumInfo = new CachedEnumInfo(enumType);
				if (!s_enumTypeToInfo.TryAdd(enumType, enumInfo))
				{
					enumInfo = s_enumTypeToInfo[enumType];
				}
			}
			return enumInfo;
		}

		/// <summary>
		/// Parses an enum value, using PerforceEnumAttribute markup for names.
		/// </summary>
		/// <param name="enumType">Type of the enum to parse.</param>
		/// <param name="value">Value of the enum.</param>
		/// <returns>Text for the enum.</returns>
		public static string GetEnumText(Type enumType, object value)
		{
			return GetCachedEnumInfo(enumType).GetEnumText((int)value);
		}

		/// <summary>
		/// Gets reflection data for the given record type
		/// </summary>
		/// <param name="recordType">The type to retrieve record info for</param>
		/// <returns>The cached reflection information for the given type</returns>
		public static CachedRecordInfo GetCachedRecordInfo(Type recordType)
		{
			CachedRecordInfo? record;
			if (!s_recordTypeToInfo.TryGetValue(recordType, out record))
			{
				record = new CachedRecordInfo(recordType, GetCreateRecordDelegate(recordType));

				// Find all the properties in the record
				AddRecordProperties(recordType, record, []);
				record.NameAndRankToInfo = record.Properties.ToDictionary(x => new TaggedPropertyNameAndRank(x.Tag, x.ParentRecords.Length), x => x);

				// Try to save the record info, or get the version that's already in the cache
				if (!s_recordTypeToInfo.TryAdd(recordType, record))
				{
					record = s_recordTypeToInfo[recordType];
				}
			}
			return record;
		}

		static void AddRecordProperties(Type recordType, CachedRecordInfo rootRecord, NestedRecordInfo[] parentRecords)
		{
			// Get all the fields for this type
			PropertyInfo[] properties = recordType.GetProperties(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance);

			// Build the map of all tags for this record
			foreach (PropertyInfo property in properties)
			{
				PerforceTagAttribute? tagAttribute = property.GetCustomAttribute<PerforceTagAttribute>();
				if (tagAttribute != null)
				{
					string tagName = tagAttribute.Name ?? property.Name;

					ulong requiredTagBitMask = 0;
					if (!tagAttribute.Optional && parentRecords.Length == 0)
					{
						requiredTagBitMask = rootRecord.RequiredTagsBitMask + 1;
						if (requiredTagBitMask == 0)
						{
							throw new PerforceException("Too many required tags in {0}; max is {1}", recordType.Name, sizeof(ulong) * 8);
						}
						rootRecord.RequiredTagsBitMask |= requiredTagBitMask;
					}

					TaggedPropertyInfo tagInfo = new(new Utf8String(tagName), tagAttribute.Optional, parentRecords, property, requiredTagBitMask);

					Type fieldType = property.PropertyType;

					PropertyInfo propertyCopy = property;
					if (fieldType == typeof(DateTime))
					{
						tagInfo.Write = (writer, value) => WriteUtf8StringWithTag(writer, new Utf8String(((long)((DateTime)value - UnixEpoch).TotalSeconds).ToString()));
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsDateTime(value));
					}
					else if (fieldType == typeof(bool))
					{
						tagInfo.Write = (writer, value) => WriteUtf8StringWithTag(writer, ((bool)value) ? StringConstants.True : StringConstants.False);
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsBool(value));
					}
					else if (fieldType == typeof(Nullable<bool>))
					{
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsNullableBool(value));
					}
					else if (fieldType == typeof(int))
					{
						tagInfo.Write = (writer, value) => WriteIntegerWithTag(writer, (int)value);
						tagInfo.ReadFromInteger = (obj, value) => propertyCopy.SetValue(obj, value);
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsInt(value));
					}
					else if (fieldType == typeof(long))
					{
						tagInfo.Write = (writer, value) => WriteUtf8StringWithTag(writer, new Utf8String(((long)value).ToString()));
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsLong(value));
					}
					else if (fieldType == typeof(string))
					{
						tagInfo.Write = (writer, value) => WriteStringWithTag(writer, (string)value);
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseString(value));
					}
					else if (fieldType == typeof(Utf8String))
					{
						tagInfo.Write = (writer, value) => WriteUtf8StringWithTag(writer, (Utf8String)value);
						tagInfo.ReadFromString = (obj, str) => propertyCopy.SetValue(obj, str.Clone());
					}
					else if (fieldType.IsEnum)
					{
						CachedEnumInfo enumInfo = GetCachedEnumInfo(fieldType);
						tagInfo.Write = (writer, value) => WriteUtf8StringWithTag(writer, enumInfo.GetName((int)value));
						tagInfo.ReadFromInteger = (obj, value) => propertyCopy.SetValue(obj, enumInfo.ParseInteger(value));
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, enumInfo.ParseString(value));
					}
					else if (fieldType == typeof(DateTimeOffset?))
					{
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, ParseStringAsDateTimeOffset(value));
					}
					else if (fieldType == typeof(List<string>))
					{
						tagInfo.ReadFromString = (obj, value) => ((List<string>)propertyCopy.GetValue(obj)!).Add(value.ToString());
					}
					else if (fieldType == typeof(ReadOnlyMemory<byte>))
					{
						tagInfo.ReadFromString = (obj, value) => propertyCopy.SetValue(obj, value.Memory);
					}
					else
					{
						throw new PerforceException("Unsupported type of {0}.{1} for tag '{2}'", recordType.Name, fieldType.Name, tagName);
					}

					rootRecord.Properties.Add(tagInfo);
				}

				PerforceRecordListAttribute? subElementAttribute = property.GetCustomAttribute<PerforceRecordListAttribute>();
				if (subElementAttribute != null)
				{
					Type newRecordType = property.PropertyType.GenericTypeArguments[0];
					NestedRecordInfo newParentRecord = new(property, GetCreateRecordDelegate(newRecordType));
					AddRecordProperties(newRecordType, rootRecord, [.. parentRecords, newParentRecord]);
				}
			}
		}

		static CreateRecordDelegate GetCreateRecordDelegate(Type type)
		{
			ConstructorInfo? constructor = type.GetConstructor(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance, null, Type.EmptyTypes, null);
			if (constructor == null)
			{
				throw new PerforceException($"Unable to find default constructor for {type}");
			}

			DynamicMethod dynamicMethod = new("_", type, null);
			ILGenerator generator = dynamicMethod.GetILGenerator();
			generator.Emit(OpCodes.Newobj, constructor);
			generator.Emit(OpCodes.Ret);
			return (CreateRecordDelegate)dynamicMethod.CreateDelegate(typeof(CreateRecordDelegate));
		}

		static string ParseString(Utf8String str)
		{
			return str.ToString();
		}

		static DateTime ParseStringAsDateTime(Utf8String str)
		{
			string text = str.ToString();

			DateTime time;
			if (DateTime.TryParse(text, out time))
			{
				return time;
			}
			else
			{
				return UnixEpoch + TimeSpan.FromSeconds(Int64.Parse(text));
			}
		}

		static object ParseStringAsBool(Utf8String str)
		{
			return str.Length == 0 || str == StringConstants.True;
		}

		static object ParseStringAsNullableBool(Utf8String str)
		{
			return str == StringConstants.True;
		}

		static object ParseStringAsInt(Utf8String str)
		{
			int value;
			int bytesConsumed;
			if (Utf8Parser.TryParse(str.Span, out value, out bytesConsumed) && bytesConsumed == str.Length)
			{
				return value;
			}
			else if (str == StringConstants.New || str == StringConstants.None)
			{
				return -1;
			}
			else if (str.Length > 0 && str[0] == '#')
			{
				return ParseStringAsInt(str.Slice(1));
			}
			else if (str == StringConstants.Default)
			{
				return DefaultChange;
			}
			else
			{
				throw new PerforceException($"Unable to parse {str} as an integer");
			}
		}

		static long ParseStringAsLong(Utf8String str)
		{
			long value;
			int bytesConsumed;
			if (!Utf8Parser.TryParse(str.Span, out value, out bytesConsumed) || bytesConsumed != str.Length)
			{
				throw new PerforceException($"Unable to parse {str} as a long value");
			}
			return value;
		}

		static DateTimeOffset ParseStringAsDateTimeOffset(Utf8String str)
		{
			string text = str.ToString();
			return DateTimeOffset.Parse(Regex.Replace(text, "[^0-9]*$", "")); // Strip timezone name (eg. "EST")
		}
	}
}
