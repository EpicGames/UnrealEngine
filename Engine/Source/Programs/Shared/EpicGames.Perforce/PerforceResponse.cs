// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Base class for returning untyped response data
	/// </summary>
	public class PerforceResponse
	{
		/// <summary>
		/// Stores the response data
		/// </summary>
		protected object InternalData { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">The response data</param>
		public PerforceResponse(object data)
		{
			InternalData = data;
		}

		/// <summary>
		/// True if the response is successful
		/// </summary>
#pragma warning disable IDE0083 // Use pattern matching - "is not" is not supported by version 8 of the language and this library is used by UGS
		public bool Succeeded => !(InternalData is PerforceError);
#pragma warning restore IDE0083 // Use pattern matching

		/// <summary>
		/// True if the response is an error
		/// </summary>
		public bool Failed => InternalData is PerforceError;

		/// <summary>
		/// Accessor for the successful response data. Throws an exception if the response is an error.
		/// </summary>
		public object Data
		{
			get
			{
				EnsureSuccess();
				return InternalData;
			}
		}

		/// <summary>
		/// Returns the info data.
		/// </summary>
		public PerforceInfo? Info => InternalData as PerforceInfo;

		/// <summary>
		/// Returns the error data, or null if this is a succesful response.
		/// </summary>
		public PerforceError? Error => InternalData as PerforceError;

		/// <summary>
		/// Returns the io data, or null if this is a regular response.
		/// </summary>
		public PerforceIo? Io => InternalData as PerforceIo;

		/// <summary>
		/// Throws an exception if the response is an error
		/// </summary>
		public void EnsureSuccess()
		{
			PerforceError? error = InternalData as PerforceError;
			if (error != null)
			{
				throw new PerforceException(error);
			}
		}

		/// <summary>
		/// Returns a string representation of this object for debugging
		/// </summary>
		/// <returns>String representation of the object for debugging</returns>
		public override string? ToString()
		{
			return InternalData.ToString();
		}
	}

	/// <summary>
	/// Represents a successful Perforce response of the given type, or an error. Throws a PerforceException with the error
	/// text if the response value is attempted to be accessed and an error has occurred.
	/// </summary>
	/// <typeparam name="T">Type of data returned on success</typeparam>
	public class PerforceResponse<T> : PerforceResponse where T : class
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">The successful response data</param>
		public PerforceResponse(T data)
			: base(data)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="info">The info data</param>
		public PerforceResponse(PerforceInfo info)
			: base(info)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="error">The error data</param>
		public PerforceResponse(PerforceError error)
			: base(error)
		{
		}

		/// <summary>
		/// Construct a typed response from an untyped response
		/// </summary>
		/// <param name="untypedResponse">The untyped response</param>
		public PerforceResponse(PerforceResponse untypedResponse)
			: base(untypedResponse.Error ?? untypedResponse.Info ?? (object)(T)untypedResponse.Data)
		{

		}

		/// <summary>
		/// Accessor for the succcessful response data. Throws an exception if the response is an error.
		/// </summary>
		public new T Data
		{
			get
			{
				T? result = InternalData as T;
				if (result == null)
				{
					if (InternalData is PerforceInfo)
					{
						throw new PerforceException($"Expected record of type '{typeof(T).Name}', got info: {InternalData}");
					}
					else if (InternalData is PerforceError)
					{
						throw new PerforceException($"{InternalData}");
					}
					else
					{
						throw new PerforceException($"Expected record of type '{typeof(T).Name}', got: {InternalData}");
					}
				}
				return result;
			}
		}
	}

	/// <summary>
	/// Extension methods for responses
	/// </summary>
	public static class PerforceResponseExtensions
	{
		/// <summary>
		/// Whether all responses in this list are successful
		/// </summary>
		public static bool Succeeded<T>(this IEnumerable<PerforceResponse<T>> responses) where T : class
		{
			return responses.All(x => x.Succeeded);
		}

		/// <summary>
		/// Sequence of all the error responses.
		/// </summary>
		public static IEnumerable<PerforceError> GetErrors<T>(this IEnumerable<PerforceResponse<T>> responses) where T : class
		{
			foreach (PerforceResponse<T> response in responses)
			{
				PerforceError? error = response.Error;
				if (error != null)
				{
					yield return error;
				}
			}
		}

		/// <summary>
		/// Throws an exception if any response is an error
		/// </summary>
		public static void EnsureSuccess<T>(this IEnumerable<PerforceResponse<T>> responses) where T : class
		{
			foreach (PerforceResponse<T> response in responses)
			{
				response.EnsureSuccess();
			}
		}

		/// <summary>
		/// Unwrap a task returning a response object
		/// </summary>
		public static async Task<T> UnwrapAsync<T>(this Task<PerforceResponse<T>> response) where T : class
		{
			return (await response).Data;
		}
	}
}
