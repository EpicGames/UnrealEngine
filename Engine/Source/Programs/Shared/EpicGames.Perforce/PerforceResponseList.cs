// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents a list of responses from the Perforce server. Within the list, individual responses
	/// may indicate success or failure.
	/// </summary>
	/// <typeparam name="T">Successful response type</typeparam>
	public class PerforceResponseList<T> : List<PerforceResponse<T>> where T : class
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public PerforceResponseList()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="responses">Initial contents of the list</param>
		public PerforceResponseList(IEnumerable<PerforceResponse<T>> responses)
			: base(responses)
		{
		}

		/// <summary>
		/// Whether all responses in this list are successful
		/// </summary>
		public bool Succeeded => this.All(x => x.Succeeded);

		/// <summary>
		/// Returns the first error, or null.
		/// </summary>
		public PerforceError? FirstError => Errors.FirstOrDefault();

		/// <summary>
		/// Sequence of all the data objects from the responses.
		/// </summary>
		public List<T> Data
		{
			get
			{
				if (Count == 1)
				{
					PerforceError? error = this[0].Error;
					if (error != null && error.Generic == PerforceGenericCode.Empty)
					{
						return [];
					}
				}
				return [.. this.Where(x => x.Info == null).Select(x => x.Data).Where(x => x != null)];
			}
		}

		/// <summary>
		/// Sequence of all the error responses.
		/// </summary>
		public IEnumerable<PerforceError> Errors
		{
			get
			{
				foreach (PerforceResponse<T> response in this)
				{
					PerforceError? error = response.Error;
					if (error != null)
					{
						yield return error;
					}
				}
			}
		}

		/// <summary>
		/// Throws an exception if any response is an error
		/// </summary>
		public void EnsureSuccess()
		{
			foreach (PerforceResponse<T> response in this)
			{
				response.EnsureSuccess();
			}
		}
	}
}
