// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Text;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Settings for a new Perforce connection
	/// </summary>
	public interface IPerforceSettings
	{
		/// <summary>
		/// Server and port to connect to
		/// </summary>
		public string ServerAndPort { get; }

		/// <summary>
		/// Username to log in with
		/// </summary>
		public string UserName { get; }

		/// <summary>
		/// Password to use
		/// </summary>
		public string? Password { get; }

		/// <summary>
		/// Name of the current host
		/// </summary>
		public string? HostName { get; }

		/// <summary>
		/// Name of the client to use
		/// </summary>
		public string? ClientName { get; }

		/// <summary>
		/// The invoking application name
		/// </summary>
		public string? AppName { get; }

		/// <summary>
		/// The invoking application version
		/// </summary>
		public string? AppVersion { get; }

		/// <summary>
		/// Whether to create a native client rather than running the p4 child process, if possible
		/// </summary>
		public bool PreferNativeClient { get; }

		/// <summary>
		/// Whether to monitor the connection for commands that are not producing any output
		/// </summary>
		public bool EnableHangMonitor { get; }
	}

	/// <summary>
	/// Settings for a new Perforce connection
	/// </summary>
	public class PerforceSettings : IPerforceSettings
	{
		static readonly Assembly? s_entryAssembly = Assembly.GetEntryAssembly();

		string? _appName;
		string? _appVersion;

		/// <summary>
		/// The default settings
		/// </summary>
		public static IPerforceSettings Default { get; } = new PerforceSettings(PerforceEnvironment.Default);

		/// <summary>
		/// Default app name to use for new settings objects
		/// </summary>
		public static string? DefaultAppName { get; set; } = s_entryAssembly?.GetName()?.Name;

		/// <summary>
		/// Default version string to use for new settings objects
		/// </summary>
		public static string? DefaultAppVersion { get; set; } = s_entryAssembly?.GetCustomAttribute<AssemblyInformationalVersionAttribute>()?.InformationalVersion ?? s_entryAssembly?.GetName()?.Version?.ToString();

		/// <inheritdoc/>
		public string ServerAndPort { get; set; }

		/// <inheritdoc/>
		public string UserName { get; set; }

		/// <inheritdoc/>
		public string? Password { get; set; }

		/// <inheritdoc/>
		public string? HostName { get; set; }

		/// <inheritdoc/>
		public string? ClientName { get; set; }

		/// <inheritdoc/>
		public string? AppName
		{
			get => _appName ?? DefaultAppName;
			set => _appName = value;
		}

		/// <inheritdoc/>
		public string? AppVersion
		{
			get => _appVersion ?? DefaultAppVersion;
			set => _appVersion = value;
		}

		/// <inheritdoc/>
		public bool PreferNativeClient { get; set; }

		/// <inheritdoc/>
		public bool EnableHangMonitor { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public PerforceSettings(IPerforceEnvironment environment)
		{
			ServerAndPort = environment.GetValue("P4PORT") ?? "perforce:1666";
			UserName = environment.GetValue("P4USER") ?? Environment.UserName;
			Password = environment.GetValue("P4PASSWD");
			HostName = environment.GetValue("P4HOST");
			ClientName = environment.GetValue("P4CLIENT");
			EnableHangMonitor = true;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="serverAndPort">Server and port to connect with</param>
		/// <param name="userName">Username to connect with</param>
		public PerforceSettings(string serverAndPort, string userName)
		{
			ServerAndPort = serverAndPort;
			UserName = userName;
			EnableHangMonitor = true;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		public PerforceSettings(IPerforceSettings other)
		{
			ServerAndPort = other.ServerAndPort;
			UserName = other.UserName;
			Password = other.Password;
			HostName = other.HostName;
			ClientName = other.ClientName;
			_appName = other.AppName;
			_appVersion = other.AppVersion;
			PreferNativeClient = other.PreferNativeClient;
			EnableHangMonitor = other.EnableHangMonitor;
		}

		/// <summary>
		/// Get the Perforce settings for a particular directory, reading any necessary P4CONFIG and P4ENVIRO files
		/// </summary>
		/// <param name="directory"></param>
		/// <returns></returns>
		public static PerforceSettings FromDirectory(DirectoryReference directory)
		{
			IPerforceEnvironment environment = PerforceEnvironment.FromDirectory(directory);
			return new PerforceSettings(environment);
		}
	}

	/// <summary>
	/// Extension methods for setting objects
	/// </summary>
	public static class PerforceSettingExtensions
	{
		/// <summary>
		/// Update common fields in a IPerforceSettings object
		/// </summary>
		public static IPerforceSettings MergeWith(this IPerforceSettings settings, string? newServerAndPort = null, string? newUserName = null, string? newClientName = null)
		{
			PerforceSettings newSettings = new(settings);
			if (!String.IsNullOrEmpty(newServerAndPort))
			{
				newSettings.ServerAndPort = newServerAndPort;
			}
			if (!String.IsNullOrEmpty(newUserName))
			{
				newSettings.UserName = newUserName;
			}
			if (!String.IsNullOrEmpty(newClientName))
			{
				newSettings.ClientName = newClientName;
			}
			return newSettings;
		}

		/// <summary>
		/// Gets the command line arguments to launch an external program, such as P4V or P4VC
		/// </summary>
		/// <param name="settings"></param>
		/// <param name="includeClient"></param>
		/// <returns></returns>
		public static string GetArgumentsForExternalProgram(this IPerforceSettings settings, bool includeClient)
		{
			StringBuilder basicCommandArgs = new();

			basicCommandArgs.AppendFormat("-p \"{0}\"", settings.ServerAndPort);
			basicCommandArgs.AppendFormat(" -u \"{0}\"", settings.UserName);

			if (includeClient && settings.ClientName != null)
			{
				basicCommandArgs.AppendFormat(" -c \"{0}\" ", settings.ClientName);
			}

			return basicCommandArgs.ToString();
		}
	}
}
