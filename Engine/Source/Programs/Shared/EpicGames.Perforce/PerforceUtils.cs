// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Utility methods for dealing with Perforce paths
	/// </summary>
	public static class PerforceUtils
	{
		/// <summary>
		/// Set of extensions to treat as code
		/// </summary>
		public static readonly HashSet<string> CodeExtensions =
		[
			".c",
			".cc",
			".cpp",
			".inl",
			".m",
			".mm",
			".rc",
			".cs",
			".csproj",
			".h",
			".hpp",
			".inl",
			".usf",
			".ush",
			".uproject",
			".uplugin",
			".sln",
			".native.verse"
		];

		/// <summary>
		/// Tests if a path is a code file
		/// </summary>
		/// <param name="path">Path to test</param>
		/// <returns>True if the path is a code file</returns>
		public static bool IsCodeFile(string path)
		{
			foreach (string codeExtension in CodeExtensions)
			{
				if (path.EndsWith(codeExtension, StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Escape a path to Perforce syntax
		/// </summary>
		public static string EscapePath(string path)
		{
			string newPath = path;
			newPath = newPath.Replace("%", "%25", StringComparison.Ordinal);
			newPath = newPath.Replace("*", "%2A", StringComparison.Ordinal);
			newPath = newPath.Replace("#", "%23", StringComparison.Ordinal);
			newPath = newPath.Replace("@", "%40", StringComparison.Ordinal);
			return newPath;
		}

		/// <summary>
		/// Remove escape characters from a path
		/// </summary>
		public static string UnescapePath(string path)
		{
			string newPath = path;
			newPath = newPath.Replace("%40", "@", StringComparison.Ordinal);
			newPath = newPath.Replace("%23", "#", StringComparison.Ordinal);
			newPath = newPath.Replace("%2A", "*", StringComparison.Ordinal);
			newPath = newPath.Replace("%2a", "*", StringComparison.Ordinal);
			newPath = newPath.Replace("%25", "%", StringComparison.Ordinal);
			return newPath;
		}

		/// <summary>
		/// Remove escape characters from a UTF8 path
		/// </summary>
		public static Utf8String UnescapePath(Utf8String path)
		{
			ReadOnlySpan<byte> pathSpan = path.Span;
			for (int inputIdx = 0; inputIdx < pathSpan.Length - 2; inputIdx++)
			{
				if (pathSpan[inputIdx] == '%')
				{
					// Allocate the output buffer
					byte[] buffer = new byte[path.Length];
					pathSpan.Slice(0, inputIdx).CopyTo(buffer.AsSpan());

					// Copy the data to the output buffer
					int outputIdx = inputIdx;
					while (inputIdx < pathSpan.Length)
					{
						// Parse the character code
						int value = StringUtils.ParseHexByte(pathSpan, inputIdx + 1);
						if (value == -1)
						{
							buffer[outputIdx++] = (byte)'%';
							inputIdx++;
						}
						else
						{
							buffer[outputIdx++] = (byte)value;
							inputIdx += 3;
						}

						// Keep copying until we get to another percent character
						while (inputIdx < pathSpan.Length && (pathSpan[inputIdx] != '%' || inputIdx + 2 >= pathSpan.Length))
						{
							buffer[outputIdx++] = pathSpan[inputIdx++];
						}
					}

					// Copy the last chunk of data to the output buffer
					path = new Utf8String(buffer.AsMemory(0, outputIdx));
					break;
				}
			}
			return path;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="depotPath"></param>
		/// <param name="depotName"></param>
		/// <returns></returns>
		public static bool TryGetDepotName(string depotPath, [NotNullWhen(true)] out string? depotName)
		{
			return TryGetClientName(depotPath, out depotName);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="clientPath"></param>
		/// <param name="clientName"></param>
		/// <returns></returns>
		public static bool TryGetClientName(string clientPath, [NotNullWhen(true)] out string? clientName)
		{
			if (!clientPath.StartsWith("//", StringComparison.Ordinal))
			{
				clientName = null;
				return false;
			}

			int slashIdx = clientPath.IndexOf('/', 2);
			if (slashIdx == -1)
			{
				clientName = null;
				return false;
			}

			clientName = clientPath.Substring(2, slashIdx - 2);
			return true;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="clientFile"></param>
		/// <returns></returns>
		public static string GetClientOrDepotDirectoryName(string clientFile)
		{
			int index = clientFile.LastIndexOf('/');
			if (index == -1)
			{
				return "";
			}
			else
			{
				return clientFile.Substring(0, index);
			}
		}

		/// <summary>
		/// Get the relative path of a client file (eg. //ClientName/Foo/Bar.txt -> Foo/Bar.txt)
		/// </summary>
		/// <param name="clientFile">Path to the client file</param>
		/// <returns></returns>
		/// <exception cref="ArgumentException"></exception>
		public static string GetClientRelativePath(string clientFile)
		{
			if (!clientFile.StartsWith("//", StringComparison.Ordinal))
			{
				throw new ArgumentException("Invalid client path", nameof(clientFile));
			}

			int idx = clientFile.IndexOf('/', 2);
			if (idx == -1)
			{
				throw new ArgumentException("Invalid client path", nameof(clientFile));
			}

			return clientFile.Substring(idx + 1);
		}

		/// <summary>
		/// Get the relative path within a client from a filename
		/// </summary>
		/// <param name="workspaceRoot">Dierctory containing the file</param>
		/// <param name="workspaceFile">File to get the path for</param>
		/// <returns></returns>
		public static string GetClientRelativePath(DirectoryReference workspaceRoot, FileReference workspaceFile)
		{
			if (!workspaceFile.IsUnderDirectory(workspaceRoot))
			{
				throw new ArgumentException($"File '{workspaceFile}' is not under workspace root '{workspaceRoot}'", nameof(workspaceFile));
			}

			return workspaceFile.MakeRelativeTo(workspaceRoot).Replace(Path.DirectorySeparatorChar, '/');
		}
	}
}
