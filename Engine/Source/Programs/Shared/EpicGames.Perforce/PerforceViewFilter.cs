// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Filters a view of files
	/// </summary>
	public class PerforceViewFilter
	{
		/// <summary>
		/// List of entries making up the view
		/// </summary>
		public List<PerforceViewFilterEntry> Entries { get; }

		/// <summary>
		/// Construct from an existing set of entries
		/// </summary>
		/// <param name="entries"></param>
		public PerforceViewFilter(IEnumerable<PerforceViewFilterEntry> entries)
		{
			Entries = [.. entries];

			// When first entry is excluding, start by including everything. Otherwise, default is to exclude.
			// This is to emulate the behavior of ManagedWorkspace.UpdateClientHaveTableAsync()
			if (Entries.Count == 0 || !Entries[0].Include)
			{
				Entries.Insert(0, new PerforceViewFilterEntry(true, "...", "", ""));
			}
		}

		/// <summary>
		/// Construct a view filter from a set of entries
		/// </summary>
		/// <param name="entries"></param>
		public static PerforceViewFilter Parse(IEnumerable<string> entries)
		{
			return new PerforceViewFilter(entries.Select(x => PerforceViewFilterEntry.Parse(x)));
		}

		/// <summary>
		/// Determines if a file is included in the view
		/// </summary>
		/// <param name="file">The file to test</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the file is included in the view</returns>
		public bool IncludeFile(string file, StringComparison comparison)
		{
			bool included = false;
			foreach (PerforceViewFilterEntry entry in Entries)
			{
				if (entry.MatchFile(file, comparison))
				{
					included = entry.Include;
				}
			}
			return included;
		}
	}

	/// <summary>
	/// Entry within a view filter
	/// </summary>
	public class PerforceViewFilterEntry
	{
		/// <summary>
		/// Whether to include files matching this pattern
		/// </summary>
		public bool Include { get; }

		/// <summary>
		/// The wildcard string - either '*' or '...'
		/// </summary>
		public string Wildcard { get; }

		/// <summary>
		/// The part of pattern before the wildcard
		/// </summary>
		public string Prefix { get; }

		/// <summary>
		/// The part of pattern after the wildcard. Perforce does not permit a slash to be in this part of the file spec filter.
		/// </summary>
		public string Suffix { get; }

		/// <summary>
		/// Tests if the entry has a file wildcard ('*')
		/// </summary>
		/// <returns>True if the entry has a file wildcard</returns>
		public bool IsFileWildcard() => Wildcard.Length == 1;

		/// <summary>
		/// Tests if the entry has a path wildcard ('...')
		/// </summary>
		/// <returns>True if the entry has a path wildcard</returns>
		public bool IsPathWildcard() => Wildcard.Length == 3;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="pathExpression"></param>
		public PerforceViewFilterEntry(bool include, string pathExpression)
		{
			Include = include;

			Match match = Regex.Match(pathExpression, @"^(.*?)(\*|\.\.\.|%%1)(.*)$");
			if (match.Success)
			{
				Prefix = match.Groups[1].Value;
				Suffix = match.Groups[3].Value;
				Wildcard = match.Groups[2].Value;
			}
			else
			{
				Prefix = pathExpression;
				Suffix = String.Empty;
				Wildcard = String.Empty;
			}

			if (!Prefix.StartsWith('/'))
			{
				Prefix = '/' + Prefix;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="wildcard"></param>
		/// <param name="prefix"></param>
		/// <param name="suffix"></param>
		public PerforceViewFilterEntry(bool include, string wildcard, string prefix, string suffix)
		{
			Include = include;
			Wildcard = wildcard;
			Prefix = prefix;
			Suffix = suffix;
		}

		/// <summary>
		/// Parse a view filter entry from a string, as returned by spec documents
		/// </summary>
		/// <param name="entry"></param>
		/// <returns></returns>
		public static PerforceViewFilterEntry Parse(string entry)
		{
			Match match = Regex.Match(entry, @"^\s*(-?)\s*(.*[^ ])\s*$");
			if (!match.Success)
			{
				throw new PerforceException($"Unable to parse view filter entry: {entry}");
			}
			return new PerforceViewFilterEntry(match.Groups[1].Length == 0, match.Groups[2].Value);
		}

		/// <summary>
		/// Determine if a file matches the current entry
		/// </summary>
		/// <param name="path">Path to the file</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the path matches the entry</returns>
		public bool MatchFile(string path, StringComparison comparison)
		{
			if (!path.StartsWith('/'))
			{
				path = '/' + path;
			}

			if (Wildcard.Length == 0)
			{
				// Expect an exact match if no wildcard is specified
				return String.Equals(path, Prefix, comparison);
			}

			if (!path.StartsWith(Prefix, comparison) || !path.EndsWith(Suffix, comparison))
			{
				return false;
			}
			if (IsFileWildcard() && path.AsSpan(Prefix.Length, path.Length - Suffix.Length - Prefix.Length).IndexOf('/') != -1)
			{
				return false;
			}
			return true;
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder builder = new();
			if (!Include)
			{
				builder.Append('-');
			}
			builder.Append($"{Prefix}{Wildcard}{Suffix}");
			return builder.ToString();
		}
	}
}
