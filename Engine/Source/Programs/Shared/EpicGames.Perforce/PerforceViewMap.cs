// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Stores a mapping from one set of paths to another
	/// </summary>
	public class PerforceViewMap
	{
		/// <summary>
		/// List of entries making up the view
		/// </summary>
		public List<PerforceViewMapEntry> Entries { get; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public PerforceViewMap()
		{
			Entries = [];
		}

		/// <summary>
		/// Construct from an existing set of entries
		/// </summary>
		/// <param name="entries"></param>
		public PerforceViewMap(IEnumerable<PerforceViewMapEntry> entries)
		{
			Entries = [.. entries];
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="other"></param>
		public PerforceViewMap(PerforceViewMap other)
		{
			Entries = [.. other.Entries];
		}

		/// <summary>
		/// Construct a view map from a set of entries
		/// </summary>
		/// <param name="entries"></param>
		public static PerforceViewMap Parse(IEnumerable<string> entries)
		{
			return new PerforceViewMap(entries.Select(x => PerforceViewMapEntry.Parse(x)));
		}

		/// <summary>
		/// Gets the inverse of this mapping
		/// </summary>
		/// <returns></returns>
		public PerforceViewMap Invert()
		{
			List<PerforceViewMapEntry> entries = [];
			foreach (PerforceViewMapEntry entry in Entries)
			{
				entries.Add(new PerforceViewMapEntry(entry.Include, entry.Target, entry.Source));
			}
			return new PerforceViewMap(entries);
		}

		/// <summary>
		/// Determines if a file is included in the view
		/// </summary>
		/// <param name="file">The file to test</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the file is included in the view</returns>
		public bool MatchFile(string file, StringComparison comparison)
		{
			bool included = false;
			foreach (PerforceViewMapEntry entry in Entries)
			{
				if (entry.MatchFile(file, comparison))
				{
					included = entry.Include;
				}
			}
			return included;
		}

		/// <summary>
		/// Maps a set of files into the target files
		/// </summary>
		/// <param name="sourceFiles">List of source files</param>
		/// <param name="comparison">Comparison to use for strings</param>
		/// <returns>List of files in the target domain, excluding any not covered by the mapping</returns>
		public IEnumerable<string> MapFiles(IEnumerable<string> sourceFiles, StringComparison comparison)
		{
			foreach (string sourceFile in sourceFiles)
			{
				if (TryMapFile(sourceFile, comparison, out string? targetFile))
				{
					yield return targetFile;
				}
			}
		}

		/// <summary>
		/// Attempts to convert a source file to its target path
		/// </summary>
		/// <param name="sourceFile"></param>
		/// <param name="comparison">The comparison type</param>
		/// <param name="targetFile"></param>
		/// <returns></returns>
		public bool TryMapFile(string sourceFile, StringComparison comparison, out string targetFile)
		{
			PerforceViewMapEntry? mapEntry = null;
			foreach (PerforceViewMapEntry entry in Entries)
			{
				if (entry.MatchFile(sourceFile, comparison))
				{
					mapEntry = entry;
				}
			}

			if (mapEntry != null && mapEntry.Include)
			{
				targetFile = mapEntry.MapFile(sourceFile);
				return true;
			}
			else
			{
				targetFile = String.Empty;
				return false;
			}
		}

		/// <summary>
		/// Gets the root paths from the view entries
		/// </summary>
		/// <returns></returns>
		public List<string> GetRootPaths(StringComparison comparison)
		{
			List<string> rootPaths = [];
			foreach (PerforceViewMapEntry entry in Entries)
			{
				if (entry.Include)
				{
					int lastSlashIdx = entry.SourcePrefix.LastIndexOf('/');
					ReadOnlySpan<char> rootPath = entry.SourcePrefix.AsSpan(0, lastSlashIdx + 1);

					for (int idx = 0; ; idx++)
					{
						if (idx == rootPaths.Count)
						{
							rootPaths.Add(rootPath.ToString());
							break;
						}
						else if (rootPaths[idx].AsSpan().StartsWith(rootPath, comparison))
						{
							rootPaths[idx] = rootPath.ToString();
							break;
						}
						else if (rootPath.StartsWith(rootPaths[idx], comparison))
						{
							break;
						}
					}
				}
			}
			return rootPaths;
		}
	}

	/// <summary>
	/// Entry within a ViewMap
	/// </summary>
	public class PerforceViewMapEntry
	{
		/// <summary>
		/// Whether to include files matching this pattern
		/// </summary>
		public bool Include { get; }

		/// <summary>
		/// The wildcard string - either '*' or '...'
		/// </summary>
		public string Wildcard { get; }

		/// <summary>
		/// The source part of the pattern before the wildcard
		/// </summary>
		public string SourcePrefix { get; }

		/// <summary>
		/// The source part of the pattern after the wildcard. Perforce does not permit a slash to be in this part of the mapping.
		/// </summary>
		public string SourceSuffix { get; }

		/// <summary>
		/// The target mapping for the pattern before the wildcard
		/// </summary>
		public string TargetPrefix { get; }

		/// <summary>
		/// The target mapping for the pattern after the wildcard
		/// </summary>
		public string TargetSuffix { get; }

		/// <summary>
		/// The full source pattern
		/// </summary>
		public string Source => $"{SourcePrefix}{Wildcard}{SourceSuffix}";

		/// <summary>
		/// The full target pattern
		/// </summary>
		public string Target => $"{TargetPrefix}{Wildcard}{TargetSuffix}";

		/// <summary>
		/// Tests if the entry has a file wildcard ('*')
		/// </summary>
		/// <returns>True if the entry has a file wildcard</returns>
		public bool IsFileWildcard() => Wildcard.Length == 1;

		/// <summary>
		/// Tests if the entry has a path wildcard ('...')
		/// </summary>
		/// <returns>True if the entry has a path wildcard</returns>
		public bool IsPathWildcard() => Wildcard.Length == 3;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="other"></param>
		public PerforceViewMapEntry(PerforceViewMapEntry other)
			: this(other.Include, other.Wildcard, other.SourcePrefix, other.SourceSuffix, other.TargetPrefix, other.TargetSuffix)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="source"></param>
		/// <param name="target"></param>
		public PerforceViewMapEntry(bool include, string source, string target)
		{
			Include = include;

			Match match = Regex.Match(source, @"^(.*?)(\*|\.\.\.|%%1)(.*)$");
			if (match.Success)
			{
				string wildcardStr = match.Groups[2].Value;

				SourcePrefix = match.Groups[1].Value;
				SourceSuffix = match.Groups[3].Value;
				Wildcard = match.Groups[2].Value;

				int otherIdx = target.IndexOf(wildcardStr, StringComparison.Ordinal);
				TargetPrefix = target.Substring(0, otherIdx);
				TargetSuffix = target.Substring(otherIdx + Wildcard.Length);

				if (wildcardStr.Equals("%%1", StringComparison.Ordinal))
				{
					Wildcard = "*";
				}
			}
			else
			{
				SourcePrefix = source;
				SourceSuffix = String.Empty;
				TargetPrefix = target;
				TargetSuffix = String.Empty;
				Wildcard = String.Empty;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="wildcard"></param>
		/// <param name="sourcePrefix"></param>
		/// <param name="sourceSuffix"></param>
		/// <param name="targetPrefix"></param>
		/// <param name="targetSuffix"></param>
		public PerforceViewMapEntry(bool include, string wildcard, string sourcePrefix, string sourceSuffix, string targetPrefix, string targetSuffix)
		{
			Include = include;
			Wildcard = wildcard;
			SourcePrefix = sourcePrefix;
			SourceSuffix = sourceSuffix;
			TargetPrefix = targetPrefix;
			TargetSuffix = targetSuffix;
		}

		/// <summary>
		/// Parse a view map entry from a string, as returned by spec documents
		/// </summary>
		/// <param name="entry"></param>
		/// <returns></returns>
		public static PerforceViewMapEntry Parse(string entry)
		{
			Match match = Regex.Match(entry, @"^\s*(-?)\s*([^ ]+)\s+([^ ]+)\s*$");
			if (!match.Success)
			{
				throw new PerforceException($"Unable to parse view map entry: {entry}");
			}
			return new PerforceViewMapEntry(match.Groups[1].Length == 0, match.Groups[2].Value, match.Groups[3].Value);
		}

		/// <summary>
		/// Maps a file to the target path
		/// </summary>
		/// <param name="sourceFile"></param>
		/// <returns></returns>
		public string MapFile(string sourceFile)
		{
			int count = sourceFile.Length - SourceSuffix.Length - SourcePrefix.Length;
			return String.Concat(TargetPrefix, sourceFile.AsSpan(SourcePrefix.Length, count), TargetSuffix);
		}

		/// <summary>
		/// Determine if a file matches the current entry
		/// </summary>
		/// <param name="path">Path to the file</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the path matches the entry</returns>
		public bool MatchFile(string path, StringComparison comparison)
		{
			if (Wildcard.Length == 0)
			{
				return String.Equals(path, SourcePrefix, comparison);
			}
			else
			{
				if (!path.StartsWith(SourcePrefix, comparison) || !path.EndsWith(SourceSuffix, comparison))
				{
					return false;
				}
				if (IsFileWildcard() && path.AsSpan(SourcePrefix.Length, path.Length - SourceSuffix.Length - SourcePrefix.Length).IndexOf('/') != -1)
				{
					return false;
				}
				return true;
			}
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder builder = new();
			if (!Include)
			{
				builder.Append('-');
			}
			builder.Append($"{SourcePrefix}{Wildcard}{SourceSuffix} {TargetPrefix}{Wildcard}{TargetSuffix}");
			return builder.ToString();
		}
	}
}
