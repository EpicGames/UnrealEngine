// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 add command
	/// </summary>
	[Flags]
	public enum AddOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Downgrade file open status to simple add.
		/// </summary>
		DowngradeToAdd = 1,

		/// <summary>
		/// Force inclusion of wildcards in filenames.
		/// </summary>
		IncludeWildcards = 2,

		/// <summary>
		/// Do not perform any ignore checking; ignore any settings specified by P4IGNORE.
		/// </summary>
		NoIgnore = 4,

		/// <summary>
		/// Preview which files would be opened for add, without actually changing any files or metadata.
		/// </summary>
		PreviewOnly = 8,
	}
}
