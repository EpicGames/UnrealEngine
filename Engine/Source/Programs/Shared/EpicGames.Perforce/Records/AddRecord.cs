// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about an added file
	/// </summary>
	public class AddRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// Path to the file in the workspace
		/// </summary>
		[PerforceTag("clientFile")]
		public string ClientFile { get; set; }

		/// <summary>
		/// The working revision number of the file that was synced
		/// </summary>
		[PerforceTag("workRev")]
		public int WorkingRevision { get; set; }

		/// <summary>
		/// Action taken when syncing the file
		/// </summary>
		[PerforceTag("action")]
		public string Action { get; set; }

		/// <summary>
		/// Type of the file
		/// </summary>
		[PerforceTag("type")]
		public string Type { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private AddRecord()
		{
			DepotFile = null!;
			ClientFile = null!;
			Action = null!;
			Type = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string? ToString()
		{
			return DepotFile;
		}
	}
}
