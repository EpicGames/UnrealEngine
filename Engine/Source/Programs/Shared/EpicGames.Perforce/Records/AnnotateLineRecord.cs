// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// 
	/// </summary>
	public class AnnotateLineRecord
	{
		/// <summary>
		/// 
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// The upper changelist 
		/// </summary>
		[PerforceTag("upper")]
		public int UpperRev { get; set; }

		/// <summary>
		/// The lower changelist 
		/// </summary>
		[PerforceTag("lower")]
		public int LowerRev { get; set; }

		/// <summary>
		/// Timestamp of this modification
		/// </summary>
		[PerforceTag("time", Optional = true)]
		public DateTime Time { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private AnnotateLineRecord()
		{
			DepotFile = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return String.Format("FileLoc {0}, from rev {1} to {2} on {3}", DepotFile, LowerRev, UpperRev, Time);
		}
	}
}
