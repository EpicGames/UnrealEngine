// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// 
	/// </summary>
	[Flags]
	public enum AnnotateOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// includes both deleted files and lines no longer present
		/// at the head revision. In the latter case, both the starting and ending
		///	revision for each line is displayed.
		/// </summary>
		IncludeDeletedFilesAndLines = 1,

		/// <summary>
		/// ignore whitespace changes
		/// </summary>
		IgnoreWhiteSpaceChanges = 2,

		/// <summary>
		/// follows all integrations into the file.  If a line was
		/// introduced into the file by a merge, the source of the merge is
		/// displayed as the changelist that introduced the line. If the source
		/// itself was the result of an integration, that source is used instead,
		/// and so on.
		/// </summary>
		FollowIntegrations = 4,

		/// <summary>
		/// directs the annotate command to output the user who
		/// modified the line in the file and the date it was modified.
		/// </summary>
		OutputUserAndDate = 8,

	}
}
