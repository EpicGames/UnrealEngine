// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// 
	/// </summary>
	public class AnnotateRecord
	{
		/// <summary>
		/// The revision number of this file
		/// </summary>
		[PerforceTag("rev", Optional = true)]
		public int RevisionNumber { get; set; }

		/// <summary>
		/// The changelist responsible for this revision of the file
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int ChangeNumber { get; set; }

		/// <summary>
		/// Type of the file
		/// </summary>
		[PerforceTag("type", Optional = true)]
		public string Type { get; set; }

		/// <summary>
		/// The upper changelist 
		/// </summary>
		[PerforceTag("upper", Optional = true)]
		public string UpperCl { get; set; }

		/// <summary>
		/// The lower changelist 
		/// </summary>
		[PerforceTag("lower", Optional = true)]
		public string LowerCl { get; set; }

		/// <summary>
		/// Author of the changelist
		/// </summary>
		[PerforceTag("user", Optional = true)]
		public string UserName { get; set; }

		/// <summary>
		/// Timestamp of this modification
		/// </summary>
		[PerforceTag("time", Optional = true)]
		public DateTime Time { get; set; }

		/// <summary>
		/// Client that submitted this changelist
		/// </summary>
		[PerforceTag("client", Optional = true)]
		public string ClientName { get; set; }

		/// <summary>
		/// The actual line
		/// </summary>
		[PerforceTag("data", Optional = true)]
		public string Data { get; set; }

		/// <summary>
		/// 
		/// </summary>
		[PerforceRecordList]
		public List<AnnotateLineRecord> LineRecord { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private AnnotateRecord()
		{
			Type = null!;
			UserName = null!;
			ClientName = null!;
			Data = null!;
			UpperCl = null!;
			LowerCl = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			if (Data == null) // this is the header
			{
				return String.Format("Data for revision {0}, CL {1}", RevisionNumber, ChangeNumber);
			}
			else
			{
				return String.Format("CL {0} by {1}, on {2}, line = {3}", LowerCl, UserName, Time, Data);
			}
		}
	}
}
