// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains summary information about a changelist
	/// </summary>
	public class ChangeRecord
	{
		/// <summary>
		/// The changelist number. -1 for "new".
		/// </summary>
		[PerforceTag("Change")]
		public int Number { get; set; } = -1;

		/// <summary>
		/// The date the change was last modified
		/// </summary>
		[PerforceTag("Date", Optional = true)]
		public DateTime Date { get; set; }

		/// <summary>
		/// The user that owns or submitted the change
		/// </summary>
		[PerforceTag("User")]
		public string? User { get; set; }

		/// <summary>
		/// The client that owns the change
		/// </summary>
		[PerforceTag("Client")]
		public string? Client { get; set; }

		/// <summary>
		/// Current changelist status
		/// </summary>
		[PerforceTag("Status")]
		public ChangeStatus Status { get; set; }

		/// <summary>
		/// Whether the change is restricted or not
		/// </summary>
		[PerforceTag("Type")]
		public ChangeType Type { get; set; }

		/// <summary>
		/// Description for the changelist
		/// </summary>
		[PerforceTag("Description")]
		public string? Description { get; set; }

		/// <summary>
		/// Files that are open in this changelist
		/// </summary>
		[PerforceTag("Files", Optional = true)]
		public List<string> Files { get; } = [];
	}
}
