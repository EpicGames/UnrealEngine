// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Indicates the status of a changelist
	/// </summary>
	public enum ChangeStatus
	{
		/// <summary>
		/// Include all changes
		/// </summary>
		All,

		/// <summary>
		/// The change is still pending
		/// </summary>
		[PerforceEnum("pending")]
		Pending,

		/// <summary>
		/// The change has been submitted
		/// </summary>
		[PerforceEnum("submitted")]
		Submitted,

		/// <summary>
		/// The change is shelved
		/// </summary>
		[PerforceEnum("shelved")]
		Shelved,
	}
}
