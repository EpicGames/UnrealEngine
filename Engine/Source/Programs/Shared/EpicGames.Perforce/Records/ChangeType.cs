// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// The type of a changelist
	/// </summary>
	public enum ChangeType
	{
		/// <summary>
		/// When creating a new changelist, leaves the changelist type unspecified.
		/// </summary>
		Unspecified,

		/// <summary>
		/// This change is visible to anyone
		/// </summary>
		[PerforceEnum("public")]
		Public,

		/// <summary>
		/// This change is restricted
		/// </summary>
		[PerforceEnum("restricted")]
		Restricted,
	}
}
