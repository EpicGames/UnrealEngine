// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the 'changes' command
	/// </summary>
	[Flags]
	public enum ChangesOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Include changelists that affected files that were integrated with the specified files.
		/// </summary>
		IncludeIntegrations = 1,

		/// <summary>
		/// Display the time as well as the date of each change.
		/// </summary>
		IncludeTimes = 2,

		/// <summary>
		/// List long output, with the full text of each changelist description.
		/// </summary>
		LongOutput = 4,

		/// <summary>
		/// List long output, with the full text of each changelist description truncated at 250 characters.
		/// </summary>
		TruncatedLongOutput = 8,

		/// <summary>
		/// View restricted changes (requires admin permission)
		/// </summary>
		IncludeRestricted = 16,

		/// <summary>
		/// Gets oldest changes first
		/// </summary>
		Reverse = 32,
	}
}
