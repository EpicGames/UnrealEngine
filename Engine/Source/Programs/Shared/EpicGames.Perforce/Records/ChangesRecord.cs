// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains summary information about a changelist
	/// </summary>
	[DebuggerDisplay("{Number}: {Description}")]
	public class ChangesRecord
	{
		/// <summary>
		/// The changelist number
		/// </summary>
		[PerforceTag("change")]
		public int Number { get; set; }

		/// <summary>
		/// The date the change was last modified
		/// </summary>
		[PerforceTag("time")]
		public DateTime Time { get; set; }

		/// <summary>
		/// The user that owns or submitted the change
		/// </summary>
		[PerforceTag("user")]
		public string User { get; set; } = String.Empty;

		/// <summary>
		/// The client that owns the change
		/// </summary>
		[PerforceTag("client")]
		public string? Client { get; set; }

		/// <summary>
		/// Current changelist status
		/// </summary>
		[PerforceTag("status")]
		public ChangeStatus Status { get; set; }

		/// <summary>
		/// Whether the change is restricted or not
		/// </summary>
		[PerforceTag("changeType")]
		public ChangeType Type { get; set; }

		/// <summary>
		/// The path affected by this change.
		/// </summary>
		[PerforceTag("path", Optional = true)]
		public string? Path { get; set; }

		/// <summary>
		/// Description for the changelist
		/// </summary>
		[PerforceTag("desc")]
		public string Description { get; set; } = String.Empty;
	}
}
