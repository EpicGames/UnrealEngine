// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 clean command
	/// </summary>
	[Flags]
	public enum CleanOptions
	{
		/// <summary>
		/// User default options
		/// </summary>
		None = 0,

		/// <summary>
		/// Edited files: Find files in the workspace that have been modified and restore them to the last file version that has synced from the depot.
		/// </summary>
		Edited = 1,

		/// <summary>
		/// Added files: Find files in the workspace that have no corresponding files in the depot and delete them.
		/// </summary>
		Added = 2,

		/// <summary>
		/// Deleted files: Find those files in the depot that do not exist in your workspace and add them to the workspace.
		/// </summary>
		Deleted = 4,

		/// <summary>
		/// Preview the results of the operation without performing any action.
		/// </summary>
		Preview = 8,

		/// <summary>
		/// Do not perform any ignore checking; ignore any settings specified by P4IGNORE for added files.
		/// </summary>
		NoIgnoreChecking = 16,

		/// <summary>
		/// Include file paths in local syntax
		/// </summary>
		LocalSyntax = 32,

		/// <summary>
		/// Use modified times to determine whether files are out of date (see https://community.perforce.com/s/article/15133)
		/// </summary>
		ModifiedTimes = 64,
	};
}
