// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the p4 clean command
	/// </summary>
	public class CleanRecord
	{
		/// <summary>
		/// Depot path of the file, according to the workspace mapping
		/// </summary>
		[PerforceTag("depotFile", Optional = true)]
		public string? DepotFile { get; set; }

		/// <summary>
		/// Client path to the file (in local path syntax)
		/// </summary>
		[PerforceTag("clientFile", Optional = true)]
		public string? ClientFile { get; set; }

		/// <summary>
		/// Path to the file in local syntax
		/// </summary>
		[PerforceTag("localFile", Optional = true)]
		public string? LocalFile { get; set; }

		/// <summary>
		/// Revision of the file
		/// </summary>
		[PerforceTag("rev", Optional = true)]
		public string? Revision { get; set; }

		/// <summary>
		/// Action to be taken to the file
		/// </summary>
		[PerforceTag("action", Optional = true)]
		public FileAction Action { get; set; }
	}
}
