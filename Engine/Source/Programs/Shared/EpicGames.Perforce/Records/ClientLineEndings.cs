// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Specifies how to treat line endings on files synced to the client
	/// </summary>
	public enum ClientLineEndings
	{
		/// <summary>
		/// This value is not specified
		/// </summary>
		Unspecified,

		/// <summary>
		/// Use mode native to the client (default)
		/// </summary>
		[PerforceEnum("local")]
		Local,

		/// <summary>
		/// UNIX-style (and Mac OS X) line endings: LF
		/// </summary>
		[PerforceEnum("unix")]
		Unix,

		/// <summary>
		/// Mac pre-OS X: CR only
		/// </summary>
		[PerforceEnum("mac")]
		Mac,

		/// <summary>
		/// Windows-style: CR + LF.
		/// </summary>
		[PerforceEnum("win")]
		Windows,

		/// <summary>
		/// The share option normalizes mixed line-endings into UNIX line-end format.
		/// </summary>
		[PerforceEnum("share")]
		Share,
	}
}
