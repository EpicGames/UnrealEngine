// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the client
	/// </summary>
	[Flags]
	public enum ClientOptions
	{
		/// <summary>
		/// No options are set
		/// </summary>
		None = 0,

		/// <summary>
		/// If set, unopened files in the workspace are left writable.
		/// </summary>
		[PerforceEnum("allwrite")]
		AllWrite = 1,

		/// <summary>
		/// Opposite of AllWrite.
		/// </summary>
		[PerforceEnum("noallwrite")]
		NoAllWrite = 2,

		/// <summary>
		/// If set, a p4 sync overwrites ("clobbers") writable-but-unopened files in the workspace that have the same name as the newly-synced files.
		/// </summary>
		[PerforceEnum("clobber")]
		Clobber = 4,

		/// <summary>
		/// Opposite of Clobber.
		/// </summary>
		[PerforceEnum("noclobber")]
		NoClobber = 8,

		/// <summary>
		/// If set, a p4 sync overwrites ("clobbers") writable-but-unopened files in the workspace that have the same name as the newly-synced files.
		/// </summary>
		[PerforceEnum("compress")]
		Compress = 16,

		/// <summary>
		/// Opposite of Compress.
		/// </summary>
		[PerforceEnum("nocompress")]
		NoCompress = 32,

		/// <summary>
		/// If set, a p4 sync overwrites ("clobbers") writable-but-unopened files in the workspace that have the same name as the newly-synced files.
		/// </summary>
		[PerforceEnum("locked")]
		Locked = 64,

		/// <summary>
		/// Opposite of Locked.
		/// </summary>
		[PerforceEnum("unlocked")]
		Unlocked = 128,

		/// <summary>
		/// For files without the +m (modtime) file type modifier, the modification date (on the local filesystem) of a newly synced file is the datestamp on the file when the file was last modified.
		/// </summary>
		[PerforceEnum("modtime")]
		ModTime = 256,

		/// <summary>
		/// For files without the +m (modtime) file type modifier, the modification date is the date and time of sync, regardless of version.
		/// </summary>
		[PerforceEnum("nomodtime")]
		NoModTime = 512,

		/// <summary>
		/// If set, p4 sync deletes empty directories in a workspace if all files in the directory have been removed.
		/// </summary>
		[PerforceEnum("rmdir")]
		Rmdir = 1024,

		/// <summary>
		/// Opposite of Rmdir.
		/// </summary>
		[PerforceEnum("normdir")]
		NoRmdir = 2048,

		/// <summary>
		/// If set, and a Helix Core Virtual File Service (P4VFS) application is installed to make use of P4ALTSYNC, the client can create, delete, or update placeholders of the file metadata instead of always taking time to download the full file content. 
		/// </summary>
		[PerforceEnum("altsync")]
		AltSync = 4096,

		/// <summary>
		/// Opposite of AltSync.
		/// </summary>
		[PerforceEnum("noaltsync")]
		NoAltSync = 8192,
	}
}
