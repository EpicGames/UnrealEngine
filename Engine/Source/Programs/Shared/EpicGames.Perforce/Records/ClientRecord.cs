// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents a Perforce clientspec
	/// </summary>
	public class ClientRecord
	{
		/// <summary>
		/// The client workspace name, as specified in the P4CLIENT environment variable or its equivalents.
		/// </summary>
		[PerforceTag("Client")]
		public string Name { get; set; }

		/// <summary>
		/// The Perforce user name of the user who owns the workspace.
		/// </summary>
		[PerforceTag("Owner", Optional = true)]
		public string? Owner { get; set; }

		/// <summary>
		/// The date the workspace specification was last modified.
		/// </summary>
		[PerforceTag("Update", Optional = true)]
		public DateTime Update { get; set; }

		/// <summary>
		/// The date and time that the workspace was last used in any way.
		/// </summary>
		[PerforceTag("Access", Optional = true)]
		public DateTime Access { get; set; }

		/// <summary>
		/// The name of the workstation on which this workspace resides.
		/// </summary>
		[PerforceTag("Host", Optional = true)]
		public string? Host { get; set; }

		/// <summary>
		/// A textual description of the workspace.
		/// </summary>
		[PerforceTag("Description", Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// Name of the server containing this change.
		/// </summary>
		[PerforceTag("ServerID", Optional = true)]
		public string? ServerId { get; set; }

		/// <summary>
		/// The directory (on the local host) relative to which all the files in the View: are specified. 
		/// </summary>
		[PerforceTag("Root")]
		public string Root { get; set; }

		/// <summary>
		/// A set of seven switches that control particular workspace options.
		/// </summary>
		[PerforceTag("Options")]
		public ClientOptions Options { get; set; }

		/// <summary>
		/// Options to govern the default behavior of p4 submit.
		/// </summary>
		[PerforceTag("SubmitOptions")]
		public ClientSubmitOptions SubmitOptions { get; set; }

		/// <summary>
		/// Configure carriage-return/linefeed (CR/LF) conversion. 
		/// </summary>
		[PerforceTag("LineEnd")]
		public ClientLineEndings LineEnd { get; set; }

		/// <summary>
		/// Associates the workspace with the specified stream.
		/// </summary>
		[PerforceTag("Stream", Optional = true)]
		public string? Stream { get; set; }

		/// <summary>
		/// The type of client.
		/// </summary>
		[PerforceTag("Type", Optional = true)]
		public string? Type { get; set; }

		/// <summary>
		/// Specifies the mappings between files in the depot and files in the workspace.
		/// </summary>
		[PerforceTag("View")]
		public List<string> View { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private ClientRecord()
		{
			Name = null!;
			Owner = null!;
			Root = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the client</param>
		/// <param name="owner">Owner of the client</param>
		/// <param name="root">The root directory to sync</param>
		public ClientRecord(string name, string owner, string root)
		{
			Name = name;
			Owner = owner;
			Root = root;
		}
	}
}
