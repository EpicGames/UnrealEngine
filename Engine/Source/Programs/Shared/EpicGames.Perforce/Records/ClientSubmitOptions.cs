// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Client options for controlling the default behavior of p4 submit
	/// </summary>
	public enum ClientSubmitOptions
	{
		/// <summary>
		/// This field is not set
		/// </summary>
		Unspecified,

		/// <summary>
		/// All open files (with or without changes) are submitted to the depot.
		/// </summary>
		[PerforceEnum("submitunchanged")]
		SubmitUnchanged,

		/// <summary>
		/// All open files (with or without changes) are submitted to the depot, and all files are automatically reopened in the default changelist.
		/// </summary>
		[PerforceEnum("submitunchanged+reopen")]
		SubmitUnchangedAndReopen,

		/// <summary>
		/// Only those files with content, type, or resolved changes are submitted to the depot.
		/// </summary>
		[PerforceEnum("revertunchanged")]
		RevertUnchanged,

		/// <summary>
		/// Only those files with content, type, or resolved changes are submitted to the depot and reopened in the default changelist.
		/// </summary>
		[PerforceEnum("revertunchanged+reopen")]
		RevertUnchangedAndReopen,

		/// <summary>
		/// Only those files with content, type, or resolved changes are submitted to the depot. Any unchanged files are moved to the default changelist.
		/// </summary>
		[PerforceEnum("leaveunchanged")]
		LeaveUnchanged,

		/// <summary>
		/// Only those files with content, type, or resolved changes are submitted to the depot. 
		/// </summary>
		[PerforceEnum("leaveunchanged+reopen")]
		LeaveUnchangedAndReopen,
	}
}
