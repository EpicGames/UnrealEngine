// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 clients command
	/// </summary>
	[Flags]
	public enum ClientsOptions
	{
		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// List all client workspaces, not just workspaces bound to this server
		/// </summary>
		All = 1,

		/// <summary>
		/// Treat the filter argument as case sensitive
		/// </summary>
		CaseSensitiveFilter = 2,

		/// <summary>
		/// Display the time as well as the date of the last update to the workspace.
		/// </summary>
		WithTimes = 4,

		/// <summary>
		/// List only client workspaces unloaded with p4 unload.
		/// </summary>
		Unloaded = 8,
	}
}
