// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the p4 clients command
	/// </summary>
	public class ClientsRecord
	{
		/// <summary>
		/// The client workspace name, as specified in the P4CLIENT environment variable or its equivalents.
		/// </summary>
		[PerforceTag("client")]
		public string Name { get; set; }

		/// <summary>
		/// The Perforce user name of the user who owns the workspace.
		/// </summary>
		[PerforceTag("Owner")]
		public string Owner { get; set; }

		/// <summary>
		/// The date the workspace specification was last modified.
		/// </summary>
		[PerforceTag("Update")]
		public DateTime Update { get; set; }

		/// <summary>
		/// The date and time that the workspace was last used in any way.
		/// </summary>
		[PerforceTag("Access")]
		public DateTime Access { get; set; }

		/// <summary>
		/// The name of the workstation on which this workspace resides.
		/// </summary>
		[PerforceTag("Host", Optional = true)]
		public string? Host { get; set; }

		/// <summary>
		/// A textual description of the workspace.
		/// </summary>
		[PerforceTag("Description", Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// The directory (on the local host) relative to which all the files in the View: are specified. 
		/// </summary>
		[PerforceTag("Root")]
		public string Root { get; set; }

		/// <summary>
		/// A set of seven switches that control particular workspace options.
		/// </summary>
		[PerforceTag("Options")]
		public ClientOptions Options { get; set; }

		/// <summary>
		/// Options to govern the default behavior of p4 submit.
		/// </summary>
		[PerforceTag("SubmitOptions")]
		public ClientSubmitOptions SubmitOptions { get; set; }

		/// <summary>
		/// Configure carriage-return/linefeed (CR/LF) conversion. 
		/// </summary>
		[PerforceTag("LineEnd")]
		public ClientLineEndings LineEnd { get; set; }

		/// <summary>
		/// Associates the workspace with the specified stream.
		/// </summary>
		[PerforceTag("Stream", Optional = true)]
		public string? Stream { get; set; }

		/// <summary>
		/// The edge server ID
		/// </summary>
		[PerforceTag("ServerID", Optional = true)]
		public string? ServerId { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private ClientsRecord()
		{
			Name = null!;
			Owner = null!;
			Root = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string? ToString()
		{
			return Name;
		}
	}
}
