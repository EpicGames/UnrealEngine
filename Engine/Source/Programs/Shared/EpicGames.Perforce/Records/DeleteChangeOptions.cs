// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the DeleteChange command (p4 change -d)
	/// </summary>
	public enum DeleteChangeOptions
	{
		/// <summary>
		/// No options are specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Forcibly delete a previously submitted changelist.
		/// </summary>
		Submitted = 1,

		/// <summary>
		/// If a changelist was renumbered on submit, and you know only the original changelist number, use OriginalChangeNumber and the original changelist number to view or edit the changelist.
		/// </summary>
		BeforeRenumber = 2,
	}
}
