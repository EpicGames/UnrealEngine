// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 client -d command
	/// </summary>
	[Flags]
	public enum DeleteClientOptions
	{
		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// Administrators can use the -f option to delete or modify locked workspaces owned by other users.
		/// </summary>
		Force = 1,

		/// <summary>
		/// Allows the deletion of a client even when that client contains shelved changes. Requires the Force option.
		/// </summary>
		DeleteShelved = 2,
	}
}
