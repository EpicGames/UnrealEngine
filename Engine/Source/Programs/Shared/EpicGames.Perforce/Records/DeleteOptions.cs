// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 delete command
	/// </summary>
	[Flags]
	public enum DeleteOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Keep existing workspace files; mark the file as open for edit even if the file is not in the client view. 
		/// </summary>
		KeepWorkspaceFiles = 1,

		/// <summary>
		/// Preview which files would be opened for edit, without actually changing any files or metadata.
		/// </summary>
		PreviewOnly = 2,

		/// <summary>
		/// Perform the delete without syncing files
		/// </summary>
		WithoutSyncing = 4,
	}
}
