// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about a file opened for delete
	/// </summary>
	public class DepotRecord
	{
		/// <summary>
		/// Depot name
		/// </summary>
		[PerforceTag("Depot")]
		public string Depot { get; set; } = String.Empty;

		/// <summary>
		/// Owner of the depot
		/// </summary>
		[PerforceTag("Owner", Optional = true)]
		public string Owner { get; set; } = String.Empty;

		/// <summary>
		/// Creation time
		/// </summary>
		[PerforceTag("Date", Optional = true)]
		public DateTime Date { get; set; }

		/// <summary>
		/// Depot Type 
		/// </summary>
		[PerforceTag("Type", Optional = true)]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// Depot Description
		/// </summary>
		[PerforceTag("Description", Optional = true)]
		public string Description { get; set; } = String.Empty;

		/// <summary>
		/// Depot Stream Depth (number of slashes after depot that define the stream name)
		/// A string containing either an integer value or an example path (i.e. //depot/foo/bar represents a depth of 2)
		/// </summary>
		[PerforceTag("StreamDepth", Optional = true)]
		public string StreamDepthString { get; set; } = String.Empty;

		/// <summary>
		/// Helper to get Depot Stream Depth (number of slashes after depot that define the stream name) as an integer
		/// </summary>
		public int GetStreamDepth()
		{
			int result;
			if (Int32.TryParse(StreamDepthString, out result))
			{
				return result;
			}
			int slashCount = StreamDepthString.Count(f => f == '/');
			if (slashCount > 0)
			{
				return slashCount - 2; // depth is the number of slashes not counting the leading //
			}
			return -1;
		}
	}
}
