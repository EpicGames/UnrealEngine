// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about an individual file in a returned DescribeRecord
	/// </summary>
	public class DescribeFileRecord
	{
		/// <summary>
		/// Path to the modified file in depot syntax
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// The action performed on this file
		/// </summary>
		[PerforceTag("action")]
		public FileAction Action { get; set; }

		/// <summary>
		/// The file type after this change
		/// </summary>
		[PerforceTag("type")]
		public string Type { get; set; }

		/// <summary>
		/// The revision number for this file
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// Size of the file, or -1 if not specified
		/// </summary>
		[PerforceTag("fileSize", Optional = true)]
		public long FileSize { get; set; }

		/// <summary>
		/// Digest of the file, or null if not specified
		/// </summary>
		[PerforceTag("digest", Optional = true)]
		public string? Digest { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private DescribeFileRecord()
		{
			DepotFile = null!;
			Type = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return DepotFile;
		}
	}
}
