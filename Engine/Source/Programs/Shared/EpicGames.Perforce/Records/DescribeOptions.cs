// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 describe command
	/// </summary>
	[Flags]
	public enum DescribeOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Force the display of descriptions for restricted changelists. This option requires admin permission.
		/// </summary>
		ShowDescriptionForRestrictedChanges = 1,

		/// <summary>
		/// Specifies that the changelist number is the Identity field of a changelist.
		/// </summary>
		Identity = 2,

		/// <summary>
		/// If a changelist was renumbered on submit, and you know only the original changelist number, use -O and the original changelist number to describe the changelist.
		/// </summary>
		OriginalChangeNumber = 4,

		/// <summary>
		/// Display the names of files shelved for the specified changelist, including the diff of each file against its previous depot revision.
		/// </summary>
		Shelved = 8,
	}
}
