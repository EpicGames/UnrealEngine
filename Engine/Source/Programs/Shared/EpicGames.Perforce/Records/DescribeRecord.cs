// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Detailed description of an individual changelist
	/// </summary>
	public class DescribeRecord
	{
		/// <summary>
		/// The changelist number
		/// </summary>
		[PerforceTag("change")]
		public int Number { get; set; }

		/// <summary>
		/// The user that owns or submitted the change
		/// </summary>
		[PerforceTag("user")]
		public string User { get; set; }

		/// <summary>
		/// The workspace that contains or submitted the change
		/// </summary>
		[PerforceTag("client")]
		public string Client { get; set; }

		/// <summary>
		/// Time at which the change was submitted
		/// </summary>
		[PerforceTag("time")]
		public DateTime Time { get; set; }

		/// <summary>
		/// The changelist description
		/// </summary>
		[PerforceTag("desc")]
		public string Description { get; set; }

		/// <summary>
		/// The change status (submitted, pending, etc...)
		/// </summary>
		[PerforceTag("status")]
		public ChangeStatus Status { get; set; }

		/// <summary>
		/// Whether the change is restricted or not
		/// </summary>
		[PerforceTag("changeType")]
		public ChangeType Type { get; set; }

		/// <summary>
		/// Narrowest path which contains all files affected by this change
		/// </summary>
		[PerforceTag("path", Optional = true)]
		public string? Path { get; set; }

		/// <summary>
		/// The files affected by this change
		/// </summary>
		[PerforceRecordList]
		public List<DescribeFileRecord> Files { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private DescribeRecord()
		{
			User = null!;
			Client = null!;
			Description = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return String.Format("{0}: {1}", Number, Description);
		}
	}
}
