// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 dirs command
	/// </summary>
	[Flags]
	public enum DirsOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Display only those directories that are mapped through the current client view.
		/// </summary>
		OnlyMapped = 1,

		/// <summary>
		/// Include only those directories that contain files on the current client workspace’s p4 have list.
		/// </summary>
		OnlyHave = 2,

		/// <summary>
		/// Include subdirectories that contain only deleted files. By default, these directories are not displayed.
		/// </summary>
		IncludeDeleted = 4,

		/// <summary>
		/// Ignore the case of the directory argument when listing directories in a case-sensitive server.
		/// </summary>
		IgnoreCase = 8,
	}
}
