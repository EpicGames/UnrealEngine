// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Subdirectory under a requested depot path
	/// </summary>
	public class DirsRecord
	{
		/// <summary>
		/// The directory name
		/// </summary>
		[PerforceTag("dir")]
		public string Dir { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private DirsRecord()
		{
			Dir = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return Dir;
		}
	}
}
