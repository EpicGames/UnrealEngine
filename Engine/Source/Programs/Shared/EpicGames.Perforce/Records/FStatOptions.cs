// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the Fstat command
	/// </summary>
	[Flags]
	public enum FStatOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0x0,

		/// <summary>
		/// For scripting purposes, report file information on a list of valid file arguments in full depot syntax with a valid revision number.
		/// </summary>
		ReportDepotSyntax = 0x01,

		/// <summary>
		/// Output all revisions for the given files, suppressing the other[...] and resolve[...] fields.
		/// </summary>
		AllRevisions = 0x02,

		/// <summary>
		/// Output a fileSize field displaying the length of the file and a digest field for each revision.
		/// </summary>
		IncludeFileSizes = 0x04,

		/// <summary>
		/// Display the clientFile in Perforce syntax, as opposed to local syntax.
		/// </summary>
		ClientFileInPerforceSyntax = 0x08,

		/// <summary>
		/// Display pending integration record data for files open in the current workspace.
		/// </summary>
		ShowPendingIntegrations = 0x10,

		/// <summary>
		/// Shorten output by excluding client workspace data (for instance, the clientFile field).
		/// </summary>
		ShortenOutput = 0x20,

		/// <summary>
		/// Sort the output in reverse order.
		/// </summary>
		ReverseOrder = 0x40,

		/// <summary>
		/// Limit output to files mapped into the current workspace.
		/// </summary>
		OnlyMapped = 0x80,

		/// <summary>
		/// Limit output to files on your have list; that is, to files synced to the current workspace.
		/// </summary>
		OnlyHave = 0x100,

		/// <summary>
		/// Limit output to files opened at revisions not at the head revision.
		/// </summary>
		OnlyOpenedBeforeHead = 0x200,

		/// <summary>
		/// Limit output to open files in the current workspace.
		/// </summary>
		OnlyOpenInWorkspace = 0x400,

		/// <summary>
		/// Limit output to open files that have been resolved.
		/// </summary>
		OnlyOpenAndResolved = 0x800,

		/// <summary>
		/// Limit output to shelved files.
		/// </summary>
		OnlyShelved = 0x1000,

		/// <summary>
		/// Limit output to open files that are unresolved.
		/// </summary>
		OnlyUnresolved = 0x2000,

		/// <summary>
		/// Sort by date.
		/// </summary>
		SortByDate = 0x4000,

		/// <summary>
		/// Sort by have revision.
		/// </summary>
		SortByHaveRevision = 0x8000,

		/// <summary>
		/// Sort by head revision.
		/// </summary>
		SortByHeadRevision = 0x10000,

		/// <summary>
		/// Sort by file size.
		/// </summary>
		SortByFileSize = 0x20000,

		/// <summary>
		/// Sort by file type.
		/// </summary>
		SortByFileType = 0x40000,

		/// <summary>
		/// Include files in the unload depot when displaying data.
		/// </summary>
		IncludeFilesInUnloadDepot = 0x80000,
	}
}
