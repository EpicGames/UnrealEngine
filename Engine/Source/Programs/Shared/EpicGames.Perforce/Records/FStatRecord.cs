// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the fstat command
	/// </summary>
	public class FStatRecord
	{
		/// <summary>
		/// Depot path to file
		/// </summary>
		[PerforceTag("depotFile", Optional = true)]
		public string? DepotFile { get; set; }

		/// <summary>
		/// Local path to file (in local syntax by default, or in Perforce syntax with the FStatOptions.ClientFileInPerforceSyntax option)
		/// </summary>
		[PerforceTag("clientFile", Optional = true)]
		public string? ClientFile { get; set; }

		/// <summary>
		/// For move operations, shows the counterpart file
		/// </summary>
		[PerforceTag("movedFile", Optional = true)]
		public string? MovedFile { get; set; }

		/// <summary>
		/// Local path to file
		/// </summary>
		[PerforceTag("path", Optional = true)]
		public string? Path { get; set; }

		/// <summary>
		/// Set if the file is open for add and it is mapped to current client workspace
		/// </summary>
		[PerforceTag("isMapped", Optional = true)]
		public bool IsMapped { get; set; }

		/// <summary>
		/// Set if file is shelved
		/// </summary>
		[PerforceTag("shelved", Optional = true)]
		public bool Shelved { get; set; }

		/// <summary>
		/// Action taken at head revision, if in depot
		/// </summary>
		[PerforceTag("headAction", Optional = true)]
		public FileAction HeadAction { get; set; }

		/// <summary>
		/// Head revision type, if in depot
		/// </summary>
		[PerforceTag("headType", Optional = true)]
		public string? HeadType { get; set; }

		/// <summary>
		/// Head revision changelist time, if in depot.
		/// </summary>
		[PerforceTag("headTime", Optional = true)]
		public DateTime HeadTime { get; set; }

		/// <summary>
		/// Revision for opened files
		/// </summary>
		[PerforceTag("rev", Optional = true)]
		public int Revision { get; set; }

		/// <summary>
		/// Head revision number, if in depot
		/// </summary>
		[PerforceTag("headRev", Optional = true)]
		public int HeadRevision { get; set; }

		/// <summary>
		/// Head revision changelist number, if in depot
		/// </summary>
		[PerforceTag("headChange", Optional = true)]
		public int HeadChange { get; set; }

		/// <summary>
		/// Head revision modification time (the time that the file was last modified on the client before submit), if in depot.
		/// </summary>
		[PerforceTag("headModTime", Optional = true)]
		public DateTime HeadModTime { get; set; }

		/// <summary>
		/// Head revision of moved file
		/// </summary>
		[PerforceTag("movedRev", Optional = true)]
		public int MovedRevision { get; set; }

		/// <summary>
		/// Revision last synced to workspace, if on workspace
		/// </summary>
		[PerforceTag("haveRev", Optional = true)]
		public int HaveRevision { get; set; }

		/// <summary>
		/// Changelist description (if using -e changelist and if the file was part of changelist)
		/// </summary>
		[PerforceTag("desc", Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// MD5 digest of a file (requires -Ol option)
		/// </summary>
		[PerforceTag("digest", Optional = true)]
		public string? Digest { get; set; }

		/// <summary>
		/// File length in bytes (requires -Ol option)
		/// </summary>
		[PerforceTag("fileSize", Optional = true)]
		public long FileSize { get; set; }

		/// <summary>
		/// Open action, if opened in your workspace
		/// </summary>
		[PerforceTag("action", Optional = true)]
		public FileAction Action { get; set; }

		/// <summary>
		/// Open type, if opened in your workspace
		/// </summary>
		[PerforceTag("type", Optional = true)]
		public string? Type { get; set; }

		/// <summary>
		/// User who opened the file, if opens
		/// </summary>
		[PerforceTag("actionOwner", Optional = true)]
		public string? ActionOwner { get; set; }

		/// <summary>
		/// Open changelist number, if opened in your workspace
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int ChangeNumber { get; set; }

		/// <summary>
		/// Summarize this record for display in the debugger
		/// </summary>
		/// <returns>Formatted record</returns>
		public override string? ToString()
		{
			return DepotFile ?? ClientFile ?? Path ?? base.ToString();
		}
	}
}
