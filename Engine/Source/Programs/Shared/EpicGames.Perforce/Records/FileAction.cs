// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Actions applied to a file in a particular revision
	/// </summary>
	public enum FileAction
	{
		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("none")]
		None,

		/// <summary>
		/// The file was added
		/// </summary>
		[PerforceEnum("add")]
		Add,

		/// <summary>
		/// The file has been edited
		/// </summary>
		[PerforceEnum("edit")]
		Edit,

		/// <summary>
		/// The file was deleted
		/// </summary>
		[PerforceEnum("delete")]
		Delete,

		/// <summary>
		/// The file was branched from elsewhere in the depot
		/// </summary>
		[PerforceEnum("branch")]
		Branch,

		/// <summary>
		/// The file at this path was added as part of a move from another location
		/// </summary>
		[PerforceEnum("move/add")]
		MoveAdd,

		/// <summary>
		/// The file at this path was deleted as part of a move into another location
		/// </summary>
		[PerforceEnum("move/delete")]
		MoveDelete,

		/// <summary>
		/// The file was merged with another file in the depot
		/// </summary>
		[PerforceEnum("integrate")]
		Integrate,

		/// <summary>
		/// 
		/// </summary>
		[PerforceEnum("import")]
		Import,

		/// <summary>
		/// The file was purged from the server
		/// </summary>
		[PerforceEnum("purge")]
		Purge,

		/// <summary>
		/// The file was archived
		/// </summary>
		[PerforceEnum("archive")]
		Archive,

		/// <summary>
		/// Unknown
		/// </summary>
		[PerforceEnum("unknown")]
		Unknown,
	}
}
