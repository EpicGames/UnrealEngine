// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the filelog command
	/// </summary>
	[Flags]
	public enum FileLogOptions
	{
		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// Display file content history instead of file name history.
		/// </summary>
		ContentHistory = 1,

		/// <summary>
		/// Follow file history across branches.
		/// </summary>
		FollowAcrossBranches = 2,

		/// <summary>
		/// List long output, with the full text of each changelist description.
		/// </summary>
		FullDescriptions = 4,

		/// <summary>
		/// List long output, with the full text of each changelist description truncated at 250 characters.
		/// </summary>
		LongDescriptions = 8,

		/// <summary>
		/// When used with the ContentHistory option, do not follow content of promoted task streams. 
		/// </summary>
		DoNotFollowPromotedTaskStreams = 16,

		/// <summary>
		/// Display a shortened form of output by ignoring non-contributory integrations
		/// </summary>
		IgnoreNonContributoryIntegrations = 32,
	}
}
