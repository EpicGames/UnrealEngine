// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record output by the filelog command
	/// </summary>
	public class FileLogRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotPath { get; set; }

		/// <summary>
		/// Revisions of this file
		/// </summary>
		[PerforceRecordList]
		public List<RevisionRecord> Revisions { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private FileLogRecord()
		{
			DepotPath = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="depotPath">Path to the file in the depot</param>
		/// <param name="revisions">Revisions of this file</param>
		public FileLogRecord(string depotPath, List<RevisionRecord> revisions)
		{
			DepotPath = depotPath;
			Revisions = revisions;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return DepotPath;
		}
	}
}
