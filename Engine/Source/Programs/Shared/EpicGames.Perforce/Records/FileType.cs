// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;

namespace EpicGames.Perforce.Managed
{
	/// <summary>
	/// Base file type
	/// </summary>
	public enum FileTypeBase : int
	{
		/// <summary>
		/// Text file
		/// </summary>
		Text = 0,

		/// <summary>
		/// Non-text file
		/// </summary>
		Binary = 1,

		/// <summary>
		/// Symbolic link
		/// </summary>
		Symlink = 2,

		/// <summary>
		/// Multi-forked Macintosh file
		/// </summary>
		Apple = 3,

		/// <summary>
		/// Macintosh resource fork
		/// </summary>
		Resource = 4,

		/// <summary>
		/// Unicode file
		/// </summary>
		Unicode = 5,

		/// <summary>
		/// Unicode file, encoded as UTF-16
		/// </summary>
		Utf16 = 6,

		/// <summary>
		/// Unicode file, encoded as UTF-8
		/// </summary>
		Utf8 = 7,
	}

	/// <summary>
	/// Modifiers on top of the base filetype
	/// </summary>
	[Flags]
	public enum FileTypeModifiers : int
	{
		/// <summary>
		/// File is always writable on the client
		/// </summary>
		AlwaysWritable = 0x8, // +w

		/// <summary>
		/// Executable bit set on client
		/// </summary>
		Executable = 0x10, // +x

		/// <summary>
		/// RCS keyword expansion
		/// </summary>
		KeywordExpansion = 0x20, // +k

		/// <summary>
		/// Old-style keyword expansion
		/// </summary>
		KeywordExpansionOld = 0x40, // +ko

		/// <summary>
		/// Exclusive open (locking)
		/// </summary>
		ExclusiveOpen = 0x80, // +l

		/// <summary>
		/// Server stores the full compressed version of each file revision
		/// </summary>
		StoreFullRevisionsCompressed = 0x100, // +C

		/// <summary>
		/// Server stores deltas in RCS format
		/// </summary>
		StoreDeltas = 0x200, // +D

		/// <summary>
		/// Server stores full file per revision, uncompressed
		/// </summary>
		StoreFullRevisionsUncompressed = 0x400, // +F

		/// <summary>
		/// Preserve original modtime
		/// </summary>
		Modtime = 0x800, // +m

		/// <summary>
		/// Archive trigger required
		/// </summary>
		ArchiveTriggerRequired = 0x1000, // +X
	}

	/// <summary>
	/// Indicates the type of a file
	/// </summary>
	public readonly struct FileType : IEquatable<FileType>
	{
		/// <summary>
		/// Size of the object when serialized to memory
		/// </summary>
		public const int SerializedSize = sizeof(uint);

		/// <summary>
		/// Mask for the base bits
		/// </summary>
		public const uint BaseMask = 7;

		/// <summary>
		/// Mask for the modifiers
		/// </summary>
		public const uint ModifiersMask = 0xff & ~BaseMask;

		/// <summary>
		/// Array of file type bases, with order matching <see cref="FileTypeBase"/>
		/// </summary>
		static readonly Utf8String[] s_baseNames =
		[
			new Utf8String("text"),
			new Utf8String("binary"),
			new Utf8String("symlink"),
			new Utf8String("apple"),
			new Utf8String("resource"),
			new Utf8String("unicode"),
			new Utf8String("utf16"),
			new Utf8String("utf8")
		];

		/// <summary>
		/// Encoded value of the filetype
		/// </summary>
		public uint Encoded { get; }

		/// <summary>
		/// Base file type
		/// </summary>
		public FileTypeBase Base => (FileTypeBase)(Encoded & BaseMask);

		/// <summary>
		/// File type modifiers
		/// </summary>
		public FileTypeModifiers Modifiers => (FileTypeModifiers)(Encoded & ModifiersMask);

		/// <summary>
		/// Number of revisions to be stored
		/// </summary>
		public int NumRevisions => (int)(Encoded >> 16);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="encoded">The encoded value</param>
		public FileType(uint encoded)
		{
			Encoded = encoded;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="base">Base file type</param>
		/// <param name="modifiers">File type modifiers</param>
		/// <param name="numRevisions">Number of revisions to store</param>
		public FileType(FileTypeBase @base, FileTypeModifiers modifiers = 0, int numRevisions = 0)
		{
			Encoded = (uint)@base | (uint)modifiers | (uint)(numRevisions << 16);
		}

		/// <summary>
		/// Parse a string as a filetype
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static FileType Parse(ReadOnlySpan<byte> text)
		{
			if (TryParse(text, out FileType type))
			{
				return type;
			}
			else
			{
				throw new InvalidCastException($"Cannot parse text ('{Encoding.UTF8.GetString(text)}') as FileType");
			}
		}

		/// <summary>
		/// Try to parse a utf8 string as a filetype
		/// </summary>
		/// <param name="text"></param>
		/// <param name="type"></param>
		/// <returns></returns>
		public static bool TryParse(ReadOnlySpan<byte> text, out FileType type)
		{
			int plusIdx = text.IndexOf((byte)'+');
			if (plusIdx == -1)
			{
				if (TryParseBase(text, out FileTypeBase @base))
				{
					type = new FileType(@base);
					return true;
				}
			}
			else
			{
				if (TryParseBase(text.Slice(0, plusIdx), out FileTypeBase @base) && TryParseModifiers(text.Slice(plusIdx + 1), out (FileTypeModifiers, int) modifiers))
				{
					type = new FileType(@base, modifiers.Item1, modifiers.Item2);
					return true;
				}
			}

			type = default;
			return false;
		}

		/// <summary>
		/// Try to parse a utf8 string as a filetype base
		/// </summary>
		/// <param name="text"></param>
		/// <param name="base"></param>
		/// <returns></returns>
		public static bool TryParseBase(ReadOnlySpan<byte> text, out FileTypeBase @base)
		{
			for (int idx = 0; idx < s_baseNames.Length; idx++)
			{
				if (text.SequenceEqual(s_baseNames[idx].Span))
				{
					@base = (FileTypeBase)idx;
					return true;
				}
			}

			@base = 0;
			return false;
		}

		/// <summary>
		/// Try to parse modifiers from a utf8 string
		/// </summary>
		/// <param name="text"></param>
		/// <param name="result"></param>
		/// <returns></returns>
		public static bool TryParseModifiers(ReadOnlySpan<byte> text, out (FileTypeModifiers, int) result)
		{
			FileTypeModifiers modifiers = 0;
			int numRevisions = 0;

			for (int idx = 0; idx < text.Length; idx++)
			{
				switch (text[idx])
				{
					case (byte)'w':
						modifiers |= FileTypeModifiers.AlwaysWritable;
						break;
					case (byte)'x':
						modifiers |= FileTypeModifiers.Executable;
						break;
					case (byte)'k':
						modifiers |= FileTypeModifiers.KeywordExpansion;
						break;
					case (byte)'o':
						modifiers |= FileTypeModifiers.KeywordExpansionOld;
						break;
					case (byte)'l':
						modifiers |= FileTypeModifiers.ExclusiveOpen;
						break;
					case (byte)'C':
						modifiers |= FileTypeModifiers.StoreFullRevisionsCompressed;
						break;
					case (byte)'D':
						modifiers |= FileTypeModifiers.StoreDeltas;
						break;
					case (byte)'F':
						modifiers |= FileTypeModifiers.StoreFullRevisionsUncompressed;
						break;
					case (byte)'S':
						while (idx + 1 < text.Length && (text[idx + 1] >= '0' && text[idx + 1] <= '9'))
						{
							numRevisions = (numRevisions * 10) + (text[++idx] - '0');
						}
						if (numRevisions == 0)
						{
							numRevisions = 1;
						}
						break;
					case (byte)'m':
						modifiers |= FileTypeModifiers.Modtime;
						break;
					default:
						result = (0, 0);
						return false;
				}
			}

			result = (modifiers, numRevisions);
			return true;
		}

		/// <summary>
		/// Compares two filetypes for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator ==(FileType a, FileType b)
		{
			return a.Encoded == b.Encoded;
		}

		/// <summary>
		/// Compares two filetypes for equality
		/// </summary>
		/// <param name="a"></param>
		/// <param name="b"></param>
		/// <returns></returns>
		public static bool operator !=(FileType a, FileType b)
		{
			return a.Encoded != b.Encoded;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			return (obj is FileType other) && Equals(other);
		}

		/// <inheritdoc/>
		public bool Equals(FileType other)
		{
			return Encoded == other.Encoded;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return (int)Encoded;
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder type = new(s_baseNames[(int)Base].ToString());

			int remainingModifiers = (int)Modifiers;
			if (remainingModifiers != 0 || NumRevisions > 0)
			{
				type.Append('+');
				while (remainingModifiers != 0)
				{
					FileTypeModifiers modifier = (FileTypeModifiers)(remainingModifiers & ~(remainingModifiers - 1));
					switch (modifier)
					{
						case FileTypeModifiers.AlwaysWritable:
							type.Append('w');
							break;
						case FileTypeModifiers.Executable:
							type.Append('x');
							break;
						case FileTypeModifiers.KeywordExpansion:
							type.Append('k');
							break;
						case FileTypeModifiers.KeywordExpansionOld:
							type.Append('o');
							break;
						case FileTypeModifiers.ExclusiveOpen:
							type.Append('l');
							break;
						case FileTypeModifiers.StoreFullRevisionsCompressed:
							type.Append('C');
							break;
						case FileTypeModifiers.StoreDeltas:
							type.Append('D');
							break;
						case FileTypeModifiers.StoreFullRevisionsUncompressed:
							type.Append('F');
							break;
						case FileTypeModifiers.Modtime:
							type.Append('m');
							break;
						case FileTypeModifiers.ArchiveTriggerRequired:
							type.Append('X');
							break;
					}
					remainingModifiers ^= (int)modifier;
				}

				if (NumRevisions == 1)
				{
					type.Append('S');
				}
				else if (NumRevisions > 1)
				{
					type.Append($"S{NumRevisions}");
				}
			}

			return type.ToString();
		}
	}

	/// <summary>
	/// Extension methods for file types
	/// </summary>
	public static class FileTypeExtensions
	{
		/// <summary>
		/// Constructor for reading a file info from disk
		/// </summary>
		/// <param name="reader">Binary reader to read data from</param>
		public static FileType ReadFileType(this MemoryReader reader)
		{
			return new FileType(reader.ReadUInt32());
		}

		/// <summary>
		/// Save the file info to disk
		/// </summary>
		/// <param name="writer">Writer to output to</param>
		/// <param name="fileType">The file type to serialize</param>
		public static void WriteFileType(this MemoryWriter writer, FileType fileType)
		{
			writer.WriteUInt32(fileType.Encoded);
		}
	}
}
