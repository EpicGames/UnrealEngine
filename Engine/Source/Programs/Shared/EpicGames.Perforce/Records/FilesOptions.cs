// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 files command
	/// </summary>
	[Flags]
	public enum FilesOptions
	{
		/// <summary>
		/// Default options
		/// </summary>
		None = 0,

		/// <summary>
		/// For each file, list all revisions within a specified revision range, rather than only the highest revision in the range.
		/// </summary>
		AllRevisions = 1,

		/// <summary>
		/// Limit output to files in archive depots.
		/// </summary>
		LimitToArchiveDepots = 2,

		/// <summary>
		/// Exclude deleted, purged, or archived files; the files that remain are those available for syncing or integration.
		/// </summary>
		ExcludeDeleted = 4,

		/// <summary>
		/// Ignore the case of the file argument when listing files in a case sensitive server.
		/// </summary>
		IgnoreCase = 8,
	}
}
