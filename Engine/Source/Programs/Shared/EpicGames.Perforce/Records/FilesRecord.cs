// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the p4 files command
	/// </summary>
	[DebuggerDisplay("{DepotFile}#{Revision}")]
	public class FilesRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; } = String.Empty;

		/// <summary>
		/// File revision number
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// Changelist number that the file was modified in
		/// </summary>
		[PerforceTag("change")]
		public int Change { get; set; }

		/// <summary>
		/// The action performed on this file
		/// </summary>
		[PerforceTag("action")]
		public FileAction Action { get; set; }

		/// <summary>
		/// The file type after this change
		/// </summary>
		[PerforceTag("type")]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// Modification time (the time that the file was last modified on the client before submit), if in depot.
		/// </summary>
		[PerforceTag("time", Optional = true)]
		public DateTime Time { get; set; }
	}
}
