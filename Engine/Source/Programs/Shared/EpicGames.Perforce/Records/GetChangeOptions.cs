// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the GetChange command (p4 change -o)
	/// </summary>
	public enum GetChangeOptions
	{
		/// <summary>
		/// No options are specified
		/// </summary>
		None = 0,

		/// <summary>
		/// If a changelist was renumbered on submit, and you know only the original changelist number, use OriginalChangeNumber and the original changelist number to view or edit the changelist.
		/// </summary>
		BeforeRenumber = 1,
	}
}
