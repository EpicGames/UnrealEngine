// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the have command
	/// </summary>
	[DebuggerDisplay("{DepotFile}")]
	public class HaveRecord
	{
		/// <summary>
		/// Depot path to file
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; } = String.Empty;

		/// <summary>
		/// Client path to file
		/// </summary>
		[PerforceTag("clientFile")]
		public string ClientFile { get; set; } = String.Empty;

		/// <summary>
		/// Local path to file
		/// </summary>
		[PerforceTag("path")]
		public string Path { get; set; } = String.Empty;

		/// <summary>
		/// Revision of the file on the client
		/// </summary>
		[PerforceTag("haveRev")]
		public int HaveRev { get; set; }

		/// <summary>
		/// Time that the file was synced (local timestamp?)
		/// </summary>
		[PerforceTag("syncTime", Optional = true)]
		public int SyncTime { get; set; }
	}
}
