// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 info command
	/// </summary>
	[Flags]
	public enum InfoOptions
	{
		/// <summary>
		/// No addiional options
		/// </summary>
		None = 0,

		/// <summary>
		/// Shortened output; exclude information that requires a database lookup.
		/// </summary>
		ShortOutput = 1,
	}
}
