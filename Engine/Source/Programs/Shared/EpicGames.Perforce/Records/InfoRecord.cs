// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about the current client and server configuration
	/// </summary>
	public class InfoRecord
	{
		/// <summary>
		/// The current user according to the Perforce environment
		/// </summary>
		[PerforceTag("userName", Optional = true)]
		public string? UserName { get; set; }

		/// <summary>
		/// The current client
		/// </summary>
		[PerforceTag("clientName", Optional = true)]
		public string? ClientName { get; set; }

		/// <summary>
		/// The current host
		/// </summary>
		[PerforceTag("clientHost", Optional = true)]
		public string? ClientHost { get; set; }

		/// <summary>
		/// Root directory for the current client
		/// </summary>
		[PerforceTag("clientRoot", Optional = true)]
		public string? ClientRoot { get; set; }

		/// <summary>
		/// Selected stream in the current client
		/// </summary>
		[PerforceTag("clientStream", Optional = true)]
		public string? ClientStream { get; set; }

		/// <summary>
		/// Address of the Perforce server
		/// </summary>
		[PerforceTag("serverAddress", Optional = true)]
		public string? ServerAddress { get; set; }
		/// <summary>
		/// Address of the Perforce server
		/// </summary>
		[PerforceTag("serverCluster", Optional = true)]
		public string? ServerCluster { get; set; }

		/// <summary>
		/// Date and time on the server
		/// </summary>
		[PerforceTag("serverDate", Optional = true)]
		public DateTimeOffset? ServerDate { get; set; }

		/// <summary>
		/// Case handling setting on the server
		/// </summary>
		[PerforceTag("caseHandling", Optional = true)]
		public string? CaseHandling { get; set; }

		/// <summary>
		/// Whether the server is case sensitive
		/// </summary>
		public bool IsCaseSensitive => !String.Equals(CaseHandling, "insensitive", StringComparison.OrdinalIgnoreCase);

		/// <summary>
		/// How to compare paths on this server
		/// </summary>
		public StringComparer PathComparer => IsCaseSensitive ? StringComparer.Ordinal : StringComparer.OrdinalIgnoreCase;

		/// <summary>
		/// How to compare paths on this server
		/// </summary>
		public StringComparison PathComparison => IsCaseSensitive ? StringComparison.Ordinal : StringComparison.OrdinalIgnoreCase;

		/// <summary>
		/// How to compare paths on this server
		/// </summary>
		public Utf8StringComparer Utf8PathComparer => IsCaseSensitive ? Utf8StringComparer.Ordinal : Utf8StringComparer.OrdinalIgnoreCase;

		/// <summary>
		/// List of services provided by this server
		/// </summary>
		[PerforceTag("serverServices", Optional = true)]
		public string? Services { get; set; }

		/// <summary>
		/// The server unique id
		/// </summary>
		[PerforceTag("ServerID", Optional = true)]
		public string? ServerId { get; set; }

		/// <summary>
		/// Timezone offset from UTC, in seconds
		/// </summary>
		[PerforceTag("tzoffset", Optional = true)]
		public int TimeZoneOffsetSecs { get; set; }
	}
}
