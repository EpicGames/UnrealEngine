// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Describes the action performed by the user when resolving the integration
	/// </summary>
	public enum IntegrateAction
	{
		/// <summary>
		/// file did not previously exist; it was created as a copy of partner-file
		/// </summary>
		[PerforceEnum("branch from")]
		BranchFrom,

		/// <summary>
		/// partner-file did not previously exist; it was created as a copy of file.
		/// </summary>
		[PerforceEnum("branch into")]
		BranchInto,

		/// <summary>
		/// file was integrated from partner-file, accepting merge.
		/// </summary>
		[PerforceEnum("merge from")]
		MergeFrom,

		/// <summary>
		/// file was integrated into partner-file, accepting merge.
		/// </summary>
		[PerforceEnum("merge into")]
		MergeInto,

		/// <summary>
		/// file was integrated from partner-file, accepting theirs and deleting the original.
		/// </summary>
		[PerforceEnum("moved from")]
		MovedFrom,

		/// <summary>
		/// file was integrated into partner-file, accepting theirs and creating partner-file if it did not previously exist.
		/// </summary>
		[PerforceEnum("moved into")]
		MovedInto,

		/// <summary>
		/// file was integrated from partner-file, accepting theirs.
		/// </summary>
		[PerforceEnum("copy from")]
		CopyFrom,

		/// <summary>
		/// file was integrated into partner-file, accepting theirs.
		/// </summary>
		[PerforceEnum("copy into")]
		CopyInto,

		/// <summary>
		/// file was integrated from partner-file, accepting yours.
		/// </summary>
		[PerforceEnum("ignored")]
		Ignored,

		/// <summary>
		/// file was integrated into partner-file, accepting yours.
		/// </summary>
		[PerforceEnum("ignored by")]
		IgnoredBy,

		/// <summary>
		/// file was integrated from partner-file, and partner-file had been previously deleted.
		/// </summary>
		[PerforceEnum("delete from")]
		DeleteFrom,

		/// <summary>
		/// file was integrated into partner-file, and file had been previously deleted.
		/// </summary>
		[PerforceEnum("delete into")]
		DeleteInto,

		/// <summary>
		/// file was integrated from partner-file, and file was edited within the p4 resolve process.
		/// </summary>
		[PerforceEnum("edit from")]
		EditFrom,

		/// <summary>
		/// file was integrated into partner-file, and partner-file was reopened for edit before submission.
		/// </summary>
		[PerforceEnum("edit into")]
		EditInto,

		/// <summary>
		/// file was integrated from a deleted partner-file, and partner-file was reopened for add (that is, someone restored a deleted file by syncing back to a pre-deleted revision and adding the file).
		/// </summary>
		[PerforceEnum("add from")]
		AddFrom,

		/// <summary>
		/// file was integrated into previously nonexistent partner-file, and partner-file was reopened for add before submission.
		/// </summary>
		[PerforceEnum("add into")]
		AddInto,

		/// <summary>
		/// The resulting revision has "undid" the edited changes (content and history) from the previously submitted revision(s)
		/// </summary>
		[PerforceEnum("undid")]
		Undid,

		/// <summary>
		/// The previous revision(s) have been "undone by" the resulting revision, known as the "undone" revision.
		/// </summary>
		[PerforceEnum("undone by")]
		UndoneBy
	}
}
