// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Stores integration information for a file revision
	/// </summary>
	public class IntegrationRecord
	{
		/// <summary>
		/// The integration action performed for this file
		/// </summary>
		[PerforceTag("how")]
		public IntegrateAction Action { get; set; }

		/// <summary>
		/// The partner file for this integration
		/// </summary>
		[PerforceTag("file")]
		public string OtherFile { get; set; }

		/// <summary>
		/// Min revision of the partner file for this integration
		/// </summary>
		[PerforceTag("srev")]
		public int StartRevisionNumber { get; set; }

		/// <summary>
		/// Max revision of the partner file for this integration
		/// </summary>
		[PerforceTag("erev")]
		public int EndRevisionNumber { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private IntegrationRecord()
		{
			OtherFile = null!;
		}

		/// <summary>
		/// Summarize this record for display in the debugger
		/// </summary>
		/// <returns>Formatted integration record</returns>
		public override string ToString()
		{
			if (StartRevisionNumber + 1 == EndRevisionNumber)
			{
				return String.Format("{0} {1}#{2}", Action, OtherFile, EndRevisionNumber);
			}
			else
			{
				return String.Format("{0} {1}#{2},#{3}", Action, OtherFile, StartRevisionNumber + 1, EndRevisionNumber);
			}
		}
	}
}
