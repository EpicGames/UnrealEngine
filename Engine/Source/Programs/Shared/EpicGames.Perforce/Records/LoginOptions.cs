// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the login command
	/// </summary>
	[Flags]
	public enum LoginOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Obtain a ticket that is valid for all IP addresses.
		/// </summary>
		AllHosts = 1,

		/// <summary>
		/// Display the ticket, rather than storing it in the local ticket file.
		/// </summary>
		PrintTicket = 2,

		/// <summary>
		/// Display the status of the current ticket, if one exists.
		/// Use with -a to display status for all hosts, or with -h host to display status for a specfic host.
		/// Users with super access can provide a username argument to display the status of that username's ticket.
		/// </summary>
		Status = 4,
	}
}
