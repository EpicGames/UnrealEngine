// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about the logged in user
	/// </summary>
	public class LoginRecord
	{
		/// <summary>
		/// The current user according to the Perforce environment
		/// </summary>
		[PerforceTag("User")]
		public string? User { get; set; }

		/// <summary>
		/// The ticket created for the user. Set only if <see cref="LoginOptions.PrintTicket"/> is set.
		/// </summary>
		public string? Ticket { get; set; }

		/// <summary>
		/// Time at which the current ticket expires
		/// </summary>
		[PerforceTag("TicketExpiration", Optional = true)]
		public long TicketExpiration { get; set; }

		/// <summary>
		/// Method by which the current user was authorized
		/// </summary>
		[PerforceTag("AuthedBy", Optional = true)]
		public string? AuthedBy { get; set; }
	}
}
