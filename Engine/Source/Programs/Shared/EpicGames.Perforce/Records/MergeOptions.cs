// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the merge command
	/// </summary>
	[Flags]
	public enum MergeOptions
	{
		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// Preview the merge
		/// </summary>
		Preview = 1,

		/// <summary>
		/// Forces the operation
		/// </summary>
		Force = 2,

		/// <summary>
		/// Performs the merge with files only
		/// </summary>
		AsFiles = 4,

		/// <summary>
		/// Performs the merge as a stream spec
		/// </summary>
		AsStreamSpec = 8,

		/// <summary>
		/// Use stream based merging
		/// </summary>
		Stream = 16,

		/// <summary>
		/// Revere the mapping direction
		/// </summary>
		ReverseMapping = 32,

		/// <summary>
		/// Treat the from path as the source
		/// </summary>
		Source = 64,
	}
}
