// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Response from a merge
	/// </summary>
	public class MergeRecord
	{
		/// <summary>
		/// The file being merged from
		/// </summary>
		[PerforceTag("fromFile")]
		public string FromFile { get; set; } = null!;

		/// <summary>
		/// The starting revision, as an integer (null if 'none')
		/// </summary>
		public int? StartFromRev => String.Equals(StartFromRevText, "none", StringComparison.Ordinal) ? (int?)null : Int32.Parse(StartFromRevText);

		/// <summary>
		/// The first revision being merged (or 'none')
		/// </summary>
		[PerforceTag("startFromRev")]
		public string StartFromRevText { get; set; } = null!;

		/// <summary>
		/// The last revision being merged
		/// </summary>
		[PerforceTag("endFromRev")]
		public int EndFromRev { get; set; }
	}
}
