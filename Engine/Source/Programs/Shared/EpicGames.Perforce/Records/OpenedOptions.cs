// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 opened command
	/// </summary>
	[Flags]
	public enum OpenedOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// List open files in all client workspaces
		/// </summary>
		AllWorkspaces = 1,

		/// <summary>
		/// Short output; do not output the revision number or file type. This option is more efficient, particularly when using the -a (all-workspaces) option at large sites.
		/// </summary>
		ShortOutput = 2,

		/// <summary>
		/// List files opened exclusively
		/// </summary>
		Exclusive = 4,
	}
}
