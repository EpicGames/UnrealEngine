// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the opened command
	/// </summary>
	[DebuggerDisplay("{DepotFile}")]
	public class OpenedRecord
	{
		/// <summary>
		/// The stream spec opened for edit
		/// </summary>
		[PerforceTag("stream", Optional = true)]
		public string Stream { get; set; } = String.Empty;

		/// <summary>
		/// Depot path to file. May be unset if this record describes a stream spec.
		/// </summary>
		[PerforceTag("depotFile", Optional = true)]
		public string DepotFile { get; set; } = String.Empty;

		/// <summary>
		/// Client path to file (not returned if OpenedOptions.ShortOutput is set)
		/// </summary>
		[PerforceTag("clientFile", Optional = true)]
		public string ClientFile { get; set; } = String.Empty;

		/// <summary>
		/// For a move action (either move/add or move/delete) indicates the counterpart depot path
		/// </summary>
		[PerforceTag("movedFile", Optional = true)]
		public string? MovedFile { get; set; }

		/// <summary>
		/// The revision of the file (not returned if OpenedOptions.ShortOutput is set)
		/// </summary>
		[PerforceTag("rev", Optional = true)]
		public int Revision { get; set; }

		/// <summary>
		/// The synced revision of the file (may be 'none' for adds) (not returned if OpenedOptions.ShortOutput is set)
		/// </summary>
		[PerforceTag("haveRev", Optional = true)]
		public int HaveRevision { get; set; }

		/// <summary>
		/// Open action, if opened in your workspace
		/// </summary>
		[PerforceTag("action", Optional = true)]
		public FileAction Action { get; set; }

		/// <summary>
		/// Change containing the open file
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int Change { get; set; }

		/// <summary>
		/// New filetype for the file
		/// </summary>
		[PerforceTag("type", Optional = true)]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// User with the file open
		/// </summary>
		[PerforceTag("user")]
		public string User { get; set; } = String.Empty;

		/// <summary>
		/// Client with the file open
		/// </summary>
		[PerforceTag("client", Optional = true)]
		public string Client { get; set; } = String.Empty;
	}
}
