// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record containing information about a printed file
	/// </summary>
	public class PrintRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; } = String.Empty;

		/// <summary>
		/// Revision number of the file
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// Change number of the file
		/// </summary>
		[PerforceTag("change")]
		public int ChangeNumber { get; set; }

		/// <summary>
		/// Last action to the file
		/// </summary>
		[PerforceTag("action")]
		public FileAction Action { get; set; }

		/// <summary>
		/// File type
		/// </summary>
		[PerforceTag("type")]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// Submit time of the file
		/// </summary>
		[PerforceTag("time")]
		public DateTime Time { get; set; }

		/// <summary>
		/// Size of the file in bytes
		/// </summary>
		[PerforceTag("fileSize", Optional = true)]
		public long FileSize { get; set; }
	}

	/// <summary>
	/// Information about a printed file, with its data
	/// </summary>
	/// <typeparam name="T">The type of data</typeparam>
	public class PrintRecord<T> : PrintRecord where T : class
	{
		/// <summary>
		/// Data for the file
		/// </summary>
		public T? Contents { get; set; }
	}
}
