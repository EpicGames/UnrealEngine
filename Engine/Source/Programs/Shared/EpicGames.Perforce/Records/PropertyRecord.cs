// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a Perforce property
	/// </summary>
	public class PropertyRecord
	{
		/// <summary>
		/// The name of the property
		/// </summary>
		[PerforceTag("name")]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The value of the property
		/// </summary>
		[PerforceTag("value")]
		public string Value { get; set; } = String.Empty;
	}
}
