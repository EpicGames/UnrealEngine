// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 reconcile command
	/// </summary>
	[Flags]
	public enum ReconcileOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Find files in the client workspace that have been modified outside of Perforce, and open them for edit.
		/// </summary>
		Edit = 1,

		/// <summary>
		/// Find files in the client workspace that are not under Perforce control and open them for add.
		/// </summary>
		Add = 2,

		/// <summary>
		/// Find files missing from the client workspace, but present on the server; open these files for delete, but only if these files are in the user's have list.
		/// </summary>
		Delete = 4,

		/// <summary>
		/// Preview which files would be opened for add, without actually changing any files or metadata.
		/// </summary>
		PreviewOnly = 8,

		/// <summary>
		/// Add filenames that contain special (wildcard) characters.
		/// </summary>
		AllowWildcards = 16,

		/// <summary>
		/// Do not perform any ignore checking.
		/// </summary>
		NoIgnore = 32,

		/// <summary>
		/// Display output in local file syntax with relative paths, similar to the workspace-centric view of p4 status.
		/// </summary>
		LocalFileSyntax = 64,

		/// <summary>
		/// use file modification time before digest to optimize
		/// </summary>
		UseFileModification = 128,
	}
}
