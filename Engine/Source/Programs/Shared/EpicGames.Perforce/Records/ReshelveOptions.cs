// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 reshelve command
	/// </summary>
	[Flags]
	public enum ReshelveOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// When the same file already exists in the target changelist, force the overwriting of it.
		/// </summary>
		Force = 1,

		/// <summary>
		/// Promote the new or target changelist where it can be accessed by other edge servers participating in the multi-server configuration. Once a shelved change has been promoted, all subsequent local modifications to the shelf are also pushed to the commit server and remain until the shelf is deleted.
		/// </summary>
		Promote = 2,
	}
}
