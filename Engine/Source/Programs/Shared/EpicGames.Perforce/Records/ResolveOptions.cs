// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 resolve command
	/// </summary>
	[Flags]
	public enum ResolveOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Automatically accept the Perforce-recommended file revision.
		/// </summary>
		Automatic = 1,

		/// <summary>
		/// Accept Yours, ignore theirs.
		/// </summary>
		AcceptYours = 2,

		/// <summary>
		/// Accept Theirs.
		/// </summary>
		AcceptTheirs = 4,

		/// <summary>
		/// If either yours or theirs is different from base, (and the changes are in common) accept that revision. If both are different from base, skip this file.
		/// </summary>
		SafeAccept = 8,

		/// <summary>
		/// Automatically accept the Perforce-recommended file revision.
		/// </summary>
		ForceAccept = 16,

		/// <summary>
		/// Ignore whitespace-only changes (for instance, a tab replaced by eight spaces)
		/// </summary>
		IgnoreWhitespaceOnly = 32,

		/// <summary>
		/// Ignore whitespace altogether (for instance, deletion of tabs or other whitespace)
		/// </summary>
		IgnoreWhitespace = 64,

		/// <summary>
		/// Ignore differences in line-ending convention
		/// </summary>
		IgnoreLineEndings = 128,

		/// <summary>
		/// Allow already resolved, but not yet submitted, files to be resolved again.
		/// </summary>
		ResolveAgain = 256,

		/// <summary>
		/// List the files that need resolving without actually performing the resolve.
		/// </summary>
		PreviewOnly = 512,
	}
}
