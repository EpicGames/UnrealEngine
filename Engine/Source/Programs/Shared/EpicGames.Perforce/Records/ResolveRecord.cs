// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about a resolved file
	/// </summary>
	public class ResolveRecord
	{
		/// <summary>
		/// Path to the file in the workspace
		/// </summary>
		[PerforceTag("clientFile", Optional = true)]
		public string? ClientFile { get; set; }

		/// <summary>
		/// Path to the depot file that needs to be resolved
		/// </summary>
		[PerforceTag("fromFile", Optional = true)]
		public string? FromFile { get; set; }

		/// <summary>
		/// Target file for the resolve
		/// </summary>
		[PerforceTag("toFile", Optional = true)]
		public string? ToFile { get; set; }

		/// <summary>
		/// How the file was resolved
		/// </summary>
		[PerforceTag("how", Optional = true)]
		public string? How { get; set; }

		/// <summary>
		/// Start range of changes to be resolved
		/// </summary>
		[PerforceTag("startFromRev", Optional = true)]
		public int FromRevisionStart { get; set; }

		/// <summary>
		/// Ending range of changes to be resolved
		/// </summary>
		[PerforceTag("endFromRev", Optional = true)]
		public int FromRevisionEnd { get; set; }

		/// <summary>
		/// The type of resolve to perform
		/// </summary>
		[PerforceTag("resolveType", Optional = true)]
		public string? ResolveType { get; set; }

		/// <summary>
		/// For content resolves, the type of resolve to be performed
		/// </summary>
		[PerforceTag("contentResolveType", Optional = true)]
		public string? ContentResolveType { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private ResolveRecord()
		{
			ClientFile = null!;
			FromFile = null!;
			ResolveType = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string? ToString()
		{
			return ClientFile;
		}
	}
}
