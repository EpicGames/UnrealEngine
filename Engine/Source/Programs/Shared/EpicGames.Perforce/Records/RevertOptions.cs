// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the revert command
	/// </summary>
	[Flags]
	public enum RevertOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Revert only those files that haven’t changed (in terms of content or filetype) since they were opened.
		/// </summary>
		Unchanged = 1,

		/// <summary>
		/// Keep workspace files; the file(s) are removed from any changelists and Perforce records that the files as being no longer open, but the file(s) are unchanged in the client workspace.
		/// </summary>
		KeepWorkspaceFiles = 2,

		/// <summary>
		/// List the files that would be reverted without actually performing the revert.
		/// </summary>
		PreviewOnly = 4,

		/// <summary>
		/// Files that are open for add are to be deleted (wiped) from the workspace when reverted.
		/// </summary>
		DeleteAddedFiles = 8,
	}
}
