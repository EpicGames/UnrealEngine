// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about a reverted file
	/// </summary>
	public class RevertRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// Path to the file in the workspace
		/// </summary>
		[PerforceTag("clientFile")]
		public string ClientFile { get; set; }

		/// <summary>
		/// The revision number that is now in the workspace
		/// </summary>
		[PerforceTag("haveRev")]
		public int HaveRevision { get; set; }

		/// <summary>
		/// The previous action that the file was opened as
		/// </summary>
		[PerforceTag("oldAction")]
		public string OldAction { get; set; }

		/// <summary>
		/// Action taken to revert the file
		/// </summary>
		[PerforceTag("action")]
		public string Action { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private RevertRecord()
		{
			DepotFile = null!;
			ClientFile = null!;
			OldAction = null!;
			Action = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revert record</returns>
		public override string ToString()
		{
			return ClientFile;
		}
	}
}
