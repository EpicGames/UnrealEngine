// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Stores a revision record for a file
	/// </summary>
	public class RevisionRecord
	{
		/// <summary>
		/// The revision number of this file
		/// </summary>
		[PerforceTag("rev")]
		public int RevisionNumber { get; set; }

		/// <summary>
		/// The changelist responsible for this revision of the file
		/// </summary>
		[PerforceTag("change")]
		public int ChangeNumber { get; set; }

		/// <summary>
		/// Action performed to the file in this revision
		/// </summary>
		[PerforceTag("action")]
		public FileAction Action { get; set; }

		/// <summary>
		/// Type of the file
		/// </summary>
		[PerforceTag("type")]
		public string Type { get; set; }

		/// <summary>
		/// Timestamp of this modification
		/// </summary>
		[PerforceTag("time")]
		public DateTime Time { get; set; }

		/// <summary>
		/// Author of the changelist
		/// </summary>
		[PerforceTag("user")]
		public string UserName { get; set; }

		/// <summary>
		/// Client that submitted this changelist
		/// </summary>
		[PerforceTag("client")]
		public string ClientName { get; set; }

		/// <summary>
		/// Size of the file, or -1 if not specified
		/// </summary>
		[PerforceTag("fileSize", Optional = true)]
		public long FileSize { get; set; }

		/// <summary>
		/// Digest of the file, or null if not specified
		/// </summary>
		[PerforceTag("digest", Optional = true)]
		public string? Digest { get; set; }

		/// <summary>
		/// Description of this changelist
		/// </summary>
		[PerforceTag("desc")]
		public string Description { get; set; }

		/// <summary>
		/// Integration records for this revision
		/// </summary>
		[PerforceRecordList]
		public List<IntegrationRecord> Integrations { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private RevisionRecord()
		{
			Type = null!;
			UserName = null!;
			ClientName = null!;
			Description = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return String.Format("#{0} change {1} {2} on {3} by {4}@{5}", RevisionNumber, ChangeNumber, Action, Time, UserName, ClientName);
		}
	}
}
