// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents a Perforce clientspec
	/// </summary>
	public class ServerRecord
	{
		/// <summary>
		/// The server id
		/// </summary>
		[PerforceTag("ServerID")]
		public string ServerId { get; set; } = String.Empty;

		/// <summary>
		/// The server type
		/// </summary>
		[PerforceTag("Type")]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// The P4NAME used by this server (optional).
		/// </summary>
		[PerforceTag("Name", Optional = true)]
		public string? Name { get; set; }

		/// <summary>
		/// The P4PORT used by this server (optional).
		/// </summary>
		[PerforceTag("Address", Optional = true)]
		public string? Address { get; set; }
	}
}
