// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 shelve command
	/// </summary>
	[Flags]
	public enum ShelveOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Only shelve files that have changed
		/// </summary>
		OnlyChanged = 1,

		/// <summary>
		/// Force the overwriting of any existing shelved files in a pending changelist with the contents of their client workspace copies.
		/// </summary>
		Overwrite = 2,
	}
}
