// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a shelved file
	/// </summary>
	public class ShelveRecord
	{
		/// <summary>
		/// The changelist containing the shelved file
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int Change { get; set; }

		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// The revision number of the file that was shelved
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// The action to be applied to the file
		/// </summary>
		[PerforceTag("action")]
		public string Action { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private ShelveRecord()
		{
			DepotFile = null!;
			Action = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return DepotFile;
		}
	}
}
