// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the sizes command
	/// </summary>
	/// <see href="https://help.perforce.com/helix-core/server-apps/cmdref/current/Content/CmdRef/p4_sizes.html"/>
	[Flags]
	public enum SizesOptions
	{
		/// <summary>
		/// No options are set
		/// </summary>
		None = 0,

		/// <summary>
		/// For each file, list all revisions within a specified revision range, rather than only the highest revision in the range.
		/// </summary>
		AllRevisions = 1,

		/// <summary>
		/// Display files in archive depots.
		/// </summary>
		LimitToArchiveDepots = 2,

		/// <summary>
		/// Calculate the sum of the file sizes for the specified file argument.
		/// </summary>
		CalculateSum = 4,

		/// <summary>
		/// Display size information for shelved files only. If you use this option, revision specifications are not permitted.
		/// </summary>
		DisplayForShelvedFilesOnly = 8,

		/// <summary>
		/// When calculating size information, exclude lazy copies.
		/// </summary>
		ExcludeLazyCopies = 16,

		/// <summary>
		/// Reports the file size as expected in the workspace, defaulting to the client's environment, accounting for line-endings,
		/// charset conversion of unicode/utf8/utf16 filetypes (including the BOM for utf8 type files) and keyword expansion for +k type files.
		/// </summary>
		FileSizeAsExpectedInTheWorkspace = 32,
	}
}
