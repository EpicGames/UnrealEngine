// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Record returned by the sizes command
	/// </summary>
	public class SizesRecord
	{
		/// <summary>
		/// Depot path to file
		/// </summary>
		[PerforceTag("depotFile", Optional = true)]
		public string? DepotFile { get; set; }

		/// <summary>
		/// Revision for opened file(s)
		/// </summary>
		[PerforceTag("rev", Optional = true)]
		public int Revision { get; set; }

		/// <summary>
		/// Size for file(s)
		/// </summary>
		[PerforceTag("fileSize")]
		public long Size { get; set; }

		/// <summary>
		/// Count of file(s)
		/// </summary>
		[PerforceTag("fileCount", Optional = true)]
		public int Count { get; set; }
	}
}
