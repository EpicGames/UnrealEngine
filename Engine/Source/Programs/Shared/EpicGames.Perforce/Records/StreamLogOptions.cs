// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the 'streamlog' command
	/// </summary>
	[Flags]
	public enum StreamLogOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Include inherited stream history (-i)
		/// </summary>
		IncludeHistory = 1,

		/// <summary>
		/// Display the stream content history instead of stream name history (-h)
		/// </summary>
		DisplayStreamContent = 2,

		/// <summary>
		/// Display the time as well as the date (-t)
		/// </summary>
		IncludeTimes = 4,

		/// <summary>
		/// List long output, with the full text of each changelist description (-l)
		/// </summary>
		LongOutput = 8,

		/// <summary>
		/// List long output, with the full text of each changelist description truncated at 250 characters (-L)
		/// </summary>
		TruncatedLongOutput = 16,
	}
}
