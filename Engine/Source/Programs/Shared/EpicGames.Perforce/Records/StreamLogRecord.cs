// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains summary information about a stream log record
	/// </summary>
	[DebuggerDisplay("{Number}: {Description}")]
	public class StreamLogRecord
	{
		/// <summary>
		/// The changelist number
		/// </summary>
		[PerforceTag("change")]
		public int Number { get; set; }

		/// <summary>
		/// The date the change was last modified
		/// </summary>
		[PerforceTag("time")]
		public DateTime Time { get; set; }

		/// <summary>
		/// The user that owns or submitted the change
		/// </summary>
		[PerforceTag("user")]
		public string User { get; set; } = String.Empty;

		/// <summary>
		/// The client that owns the change
		/// </summary>
		[PerforceTag("client")]
		public string? Client { get; set; }

		/// <summary>
		/// Description for the changelist
		/// </summary>
		[PerforceTag("desc")]
		public string Description { get; set; } = String.Empty;
	}
}
