// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for a stream
	/// </summary>
	[Flags]
	public enum StreamOptions
	{
		/// <summary>
		/// No options are set
		/// </summary>
		None = 0,

		/// <summary>
		/// Enable other users' ability to edit or delete the stream. If locked, the stream specification cannot be deleted, and only its owner can modify it.
		/// </summary>
		[PerforceEnum("unlocked")]
		Unlocked = 1,

		/// <summary>
		/// Disable other users' ability to edit or delete the stream. If locked, the stream specification cannot be deleted, and only its owner can modify it.
		/// </summary>
		[PerforceEnum("locked")]
		Locked = 2,

		/// <summary>
		/// Specifies that all users can submit changes to the stream.
		/// </summary>
		[PerforceEnum("allsubmit")]
		AllSubmit = 4,

		/// <summary>
		/// Specifies that only the owner of the stream can submit changes to the stream.
		/// </summary>
		[PerforceEnum("ownersubmit")]
		OwnerSubmit = 8,

		/// <summary>
		/// Specifies whether integrations from the stream to its parent are expected.
		/// </summary>
		[PerforceEnum("toparent")]
		ToParent = 16,

		/// <summary>
		/// Specifies whether integrations from the stream to its parent are expected.
		/// </summary>
		[PerforceEnum("notoparent")]
		NoToParent = 32,

		/// <summary>
		/// Specifies whether integrations to the stream from its parent are expected.
		/// </summary>
		[PerforceEnum("fromparent")]
		FromParent = 64,

		/// <summary>
		/// Specifies whether integrations to the stream from its parent are expected.
		/// </summary>
		[PerforceEnum("notoparent")]
		NoFromParent = 128,

		/// <summary>
		/// Specifies whether the merge flow is restricted or whether merge is permitted from any other stream.
		/// </summary>
		[PerforceEnum("mergeany")]
		MergeAny = 256,

		/// <summary>
		/// Specifies whether the merge flow is restricted or whether merge is permitted from any other stream.
		/// </summary>
		[PerforceEnum("mergedown")]
		MergeDown = 512,
	}
}
