// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Represents a Perforce stream spec
	/// </summary>
	public class StreamRecord
	{
		/// <summary>
		/// The stream path
		/// </summary>
		[PerforceTag("Stream")]
		public string Stream { get; set; }

		/// <summary>
		/// The stream name
		/// </summary>
		[PerforceTag("Name")]
		public string Name { get; set; }

		/// <summary>
		/// The Perforce user name of the user who owns the stream.
		/// </summary>
		[PerforceTag("Owner")]
		public string Owner { get; set; }

		/// <summary>
		/// The date the stream specification was last modified.
		/// </summary>
		[PerforceTag("Update", Optional = true)]
		public DateTime Update { get; set; }

		/// <summary>
		/// The date and time that the stream was last used in any way.
		/// </summary>
		[PerforceTag("Access", Optional = true)]
		public DateTime Access { get; set; }

		/// <summary>
		/// The parent stream
		/// </summary>
		[PerforceTag("Parent", Optional = true)]
		public string? Parent { get; set; }

		/// <summary>
		/// The stream type
		/// </summary>
		[PerforceTag("Type", Optional = true)]
		public string? Type { get; set; }

		/// <summary>
		/// A textual description of the stream.
		/// </summary>
		[PerforceTag("Description", Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// Options for this stream
		/// </summary>
		[PerforceTag("Options")]
		public StreamOptions Options { get; set; }

		/// <summary>
		/// Parent view
		/// </summary>
		[PerforceTag("ParentView", Optional = true)]
		public string? ParentView { get; set; }

		/// <summary>
		/// List of paths in the stream spec
		/// </summary>
		[PerforceTag("Paths")]
		public List<string> Paths { get; } = [];

		/// <summary>
		/// Computed view for the stream
		/// </summary>
		[PerforceTag("View", Optional = true)]
		public List<string> View { get; } = [];

		/// <summary>
		/// Specific changelists to import
		/// </summary>
		[PerforceTag("ChangeView", Optional = true)]
		public List<string> ChangeView { get; } = [];

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private StreamRecord()
		{
			Stream = null!;
			Name = null!;
			Owner = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="stream">Stream</param>
		/// <param name="owner">Owner</param>
		/// <param name="name">Name of the stream</param>
		/// <param name="parent">Parent stream</param>
		/// <param name="type">Stream type</param>
		/// <param name="parentView">Parent view</param>
		/// <param name="paths">List of paths in the stream spec</param>
		public StreamRecord(string stream, string owner, string name, string parent, string type, string parentView, List<string> paths)
		{
			Stream = stream;
			Owner = owner;
			Name = name;
			Parent = parent;
			Type = type;
			ParentView = parentView;
			Paths = paths;
		}
	}
}
