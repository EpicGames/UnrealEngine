// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Type of a Perforce stream
	/// </summary>
	public enum StreamType
	{
		/// <summary>
		/// A mainline stream
		/// </summary>
		[PerforceEnum("mainline")]
		Mainline,

		/// <summary>
		/// A development stream
		/// </summary>
		[PerforceEnum("development")]
		Development,

		/// <summary>
		/// A release stream
		/// </summary>
		[PerforceEnum("release")]
		Release,

		/// <summary>
		/// A virtual stream
		/// </summary>
		[PerforceEnum("virtual")]
		Virtual,

		/// <summary>
		/// A task stream
		/// </summary>
		[PerforceEnum("task")]
		Task,
	}
}
