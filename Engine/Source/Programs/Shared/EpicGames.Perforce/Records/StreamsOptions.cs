// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for a stream definition
	/// </summary>
	[Flags]
	public enum StreamsOptions
	{
		/// <summary>
		/// The stream is locked
		/// </summary>
		[PerforceEnum("locked")]
		Locked = 1,

		/// <summary>
		/// The stream is unlocked
		/// </summary>
		[PerforceEnum("unlocked")]
		Unlocked = 2,

		/// <summary>
		/// Only the owner may submit to the stream
		/// </summary>
		[PerforceEnum("ownersubmit")]
		OwnerSubmit = 4,

		/// <summary>
		/// Anyone may submit to the stream
		/// </summary>
		[PerforceEnum("allsubmit")]
		AllSubmit = 8,

		/// <summary>
		/// Integrations from this stream to its parent are expected
		/// </summary>
		[PerforceEnum("toparent")]
		ToParent = 16,

		/// <summary>
		/// Integrations from this stream from its parent are expected
		/// </summary>
		[PerforceEnum("notoparent")]
		NotToParent = 32,

		/// <summary>
		/// Integrations from this stream from its parent are expected
		/// </summary>
		[PerforceEnum("fromparent")]
		FromParent = 64,

		/// <summary>
		/// Integrations from this stream from its parent are expected
		/// </summary>
		[PerforceEnum("nofromparent")]
		NotFromParent = 128,

		/// <summary>
		/// Undocumented?
		/// </summary>
		[PerforceEnum("mergedown")]
		MergeDown = 256,

		/// <summary>
		/// Undocumented?
		/// </summary>
		[PerforceEnum("mergeany")]
		MergeAny = 512,
	}
}
