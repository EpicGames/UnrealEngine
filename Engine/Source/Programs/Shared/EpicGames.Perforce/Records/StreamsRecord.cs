// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a stream, as returned by the 'p4 streams' command
	/// </summary>
	public class StreamsRecord
	{
		/// <summary>
		/// Path to the stream
		/// </summary>
		[PerforceTag("Stream")]
		public string Stream { get; set; }

		/// <summary>
		/// Last time the stream definition was updated
		/// </summary>
		[PerforceTag("Update")]
		public DateTime Update { get; set; }

		/// <summary>
		/// Last time the stream definition was accessed
		/// </summary>
		[PerforceTag("Access")]
		public DateTime Access { get; set; }

		/// <summary>
		/// Owner of this stream
		/// </summary>
		[PerforceTag("Owner")]
		public string Owner { get; set; }

		/// <summary>
		/// Name of the stream. This may be modified after the stream is initially created, but it's underlying depot path will not change.
		/// </summary>
		[PerforceTag("Name")]
		public string Name { get; set; }

		/// <summary>
		/// The parent stream
		/// </summary>
		[PerforceTag("Parent", Optional = true)]
		public string? Parent { get; set; }

		/// <summary>
		/// Type of the stream
		/// </summary>
		[PerforceTag("Type")]
		public StreamType Type { get; set; }

		/// <summary>
		/// User supplied description of the stream
		/// </summary>
		[PerforceTag("desc", Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// Options for the stream definition
		/// </summary>
		[PerforceTag("Options")]
		public StreamsOptions Options { get; set; }

		/// <summary>
		/// Whether this stream is more stable than the parent stream
		/// </summary>
		[PerforceTag("firmerThanParent")]
		public bool? FirmerThanParent { get; set; }

		/// <summary>
		/// Whether changes from this stream flow to the parent stream
		/// </summary>
		[PerforceTag("changeFlowsToParent")]
		public bool ChangeFlowsToParent { get; set; }

		/// <summary>
		/// Whether changes from this stream flow from the parent stream
		/// </summary>
		[PerforceTag("changeFlowsFromParent")]
		public bool ChangeFlowsFromParent { get; set; }

		/// <summary>
		/// The mainline branch associated with this stream
		/// </summary>
		[PerforceTag("baseParent", Optional = true)]
		public string? BaseParent { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private StreamsRecord()
		{
			Stream = null!;
			Owner = null!;
			Name = null!;
		}

		/// <summary>
		/// Summarize this record for display in the debugger
		/// </summary>
		/// <returns>Formatted stream information</returns>
		public override string? ToString()
		{
			return Name;
		}
	}
}
