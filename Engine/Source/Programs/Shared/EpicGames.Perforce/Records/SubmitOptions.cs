// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the 'submit' command
	/// </summary>
	[Flags]
	public enum SubmitOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Reopen files for edit in the default changelist after submission.
		/// </summary>
		ReopenAsEdit = 1,

		/// <summary>
		/// All open files (with or without changes) are submitted to the depot. This is the default behavior of Helix Server.
		/// </summary>
		SubmitUnchanged = 2,

		/// <summary>
		/// Only those files with content or type changes are submitted to the depot. Unchanged files are reverted.
		/// </summary>
		RevertUnchanged = 4,

		/// <summary>
		/// Only those files with content or type changes are submitted to the depot. Any unchanged files are moved to the default changelist.
		/// </summary>
		LeaveUnchanged = 8,
	}
}
