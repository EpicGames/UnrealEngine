// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a submitted changelist
	/// </summary>
	public class SubmitRecord
	{
		/// <summary>
		/// Original changelist number
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int OriginalChangeNumber { get; set; } = -1;

		/// <summary>
		/// Submitted changelist number
		/// </summary>
		[PerforceTag("submittedChange", Optional = true)]
		public int SubmittedChangeNumber { get; set; } = -1;

		/// <summary>
		/// Merge another submit record into this one. Perforce outputs multiple records for a submit command, so we merge them together for convenience.
		/// </summary>
		/// <param name="other"></param>
		public void Merge(SubmitRecord other)
		{
			if (other.OriginalChangeNumber != -1)
			{
				OriginalChangeNumber = other.OriginalChangeNumber;
			}
			if (other.SubmittedChangeNumber != -1)
			{
				SubmittedChangeNumber = other.SubmittedChangeNumber;
			}
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return SubmittedChangeNumber.ToString();
		}
	}
}
