// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 client -s command
	/// </summary>
	[Flags]
	public enum SwitchClientOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Ignore files which are open for edit
		/// </summary>
		IgnoreOpenFiles = 1,
	}
}
