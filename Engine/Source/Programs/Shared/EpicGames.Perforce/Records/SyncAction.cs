// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Action performed to the file during a sync
	/// </summary>
	public enum SyncAction
	{
		/// <summary>
		/// Unknown value
		/// </summary>
		[PerforceEnum("none")]
		Unknown,

		/// <summary>
		/// The file was added during the sync
		/// </summary>
		[PerforceEnum("added")]
		Added,

		/// <summary>
		/// The file was updated during the sync
		/// </summary>
		[PerforceEnum("updated")]
		Updated,

		/// <summary>
		/// The file was deleted during the sync
		/// </summary>
		[PerforceEnum("deleted")]
		Deleted,
	}
}
