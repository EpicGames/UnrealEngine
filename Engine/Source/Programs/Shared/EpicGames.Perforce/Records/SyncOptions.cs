// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 sync command
	/// </summary>
	[Flags]
	public enum SyncOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Force the sync. Perforce performs the sync even if the client workspace already has the file at the specified revision. If the file is writable, it is overwritten.
		/// </summary>
		Force = 1,

		/// <summary>
		/// Keep existing workspace files; update the have list without updating the client workspace.
		/// </summary>
		KeepWorkspaceFiles = 2,

		/// <summary>
		/// For scripting purposes, perform the sync on a list of valid file arguments in full depot syntax with a valid revision number.
		/// </summary>
		FullDepotSyntax = 4,

		/// <summary>
		/// Preview mode: Display the results of the sync without actually performing the sync.
		/// </summary>
		PreviewOnly = 8,

		/// <summary>
		/// Preview mode: Display a summary of the expected network traffic associated with a sync, without performing the sync.
		/// </summary>
		NetworkPreviewOnly = 16,

		/// <summary>
		/// Populate a client workspace, but do not update the have list. Any file that is already synced or opened is bypassed with a warning message.
		/// </summary>
		DoNotUpdateHaveList = 32,

		/// <summary>
		/// Reopen files that are mapped to new locations in the depot, in the new location.
		/// </summary>
		ReopenMovedFiles = 64,

		/// <summary>
		/// Safe sync: Compare the content in your client workspace against what was last synced.
		/// </summary>
		Safe = 128,
	}
}
