// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about a synced file
	/// </summary>
	[DebuggerDisplay("{DepotFile}")]
	public class SyncRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public Utf8String DepotFile { get; set; } = Utf8String.Empty;

		/// <summary>
		/// Path to the file in the workspace. Note: despite being a property called 'clientFile', this is not in client syntax.
		/// </summary>
		[PerforceTag("clientFile")]
		public Utf8String Path { get; set; } = Utf8String.Empty;

		/// <summary>
		/// The revision number of the file that was synced
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// Action taken when syncing the file
		/// </summary>
		[PerforceTag("action")]
		public SyncAction Action { get; set; }

		/// <summary>
		/// Size of the file
		/// </summary>
		[PerforceTag("fileSize", Optional = true)]
		public long FileSize { get; set; }

		/// <summary>
		/// Stores the total size of all files that are being synced (only set for the first sync record)
		/// </summary>
		[PerforceTag("totalFileSize", Optional = true)]
		public long TotalFileSize { get; set; }

		/// <summary>
		/// Stores the total number of files that will be synced (only set for the first sync record)
		/// </summary>
		[PerforceTag("totalFileCount", Optional = true)]
		public long TotalFileCount { get; set; }

		/// <summary>
		/// Change that modified the file
		/// </summary>
		[PerforceTag("change", Optional = true)]
		public int Change { get; set; }
	}
}
