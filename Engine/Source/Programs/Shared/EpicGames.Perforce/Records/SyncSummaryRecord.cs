// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about a sync operation
	/// </summary>
	public class SyncSummaryRecord
	{
		/// <summary>
		/// The total size of all files synced
		/// </summary>
		[PerforceTag("totalFileSize")]
		public long TotalFileSize { get; set; }

		/// <summary>
		/// The total number of files synced
		/// </summary>
		[PerforceTag("totalFileCount")]
		public long TotalFileCount { get; set; }

		/// <summary>
		/// The changelist that was synced to
		/// </summary>
		[PerforceTag("change")]
		public int Change { get; set; }
	}
}
