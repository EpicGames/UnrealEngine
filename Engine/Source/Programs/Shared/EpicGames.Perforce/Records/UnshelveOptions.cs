// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 unshelve command
	/// </summary>
	[Flags]
	public enum UnshelveOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Force the overwriting of writable (but unopened) files during the unshelve operation.
		/// </summary>
		ForceOverwrite = 1,

		/// <summary>
		/// Preview the results of the unshelve operation without actually restoring the files to your workspace.
		/// </summary>
		PreviewOnly = 2,
	}
}
