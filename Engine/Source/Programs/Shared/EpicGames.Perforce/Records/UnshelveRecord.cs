// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Information about an unshelved file
	/// </summary>
	public class UnshelveRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// The revision number of the file
		/// </summary>
		[PerforceTag("rev")]
		public int Revision { get; set; }

		/// <summary>
		/// Open action for the file
		/// </summary>
		[PerforceTag("action")]
		public string Action { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private UnshelveRecord()
		{
			DepotFile = null!;
			Action = null!;
		}

		/// <summary>
		/// Format this record for display in the debugger
		/// </summary>
		/// <returns>Summary of this revision</returns>
		public override string ToString()
		{
			return DepotFile;
		}
	}
}
