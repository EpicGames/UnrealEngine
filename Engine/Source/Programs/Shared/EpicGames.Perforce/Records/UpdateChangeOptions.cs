// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the UpdateChange command
	/// </summary>
	[Flags]
	public enum UpdateChangeOptions
	{
		/// <summary>
		/// No options are specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Allows the description, modification date, or user of a submitted changelist to be edited. Editing a submitted changelist requires admin or super access. 
		/// </summary>
		Force = 2,

		/// <summary>
		/// Update a submitted changelist. Only the Jobs:, Description:, or Type: fields can be updated, and only the submitter of the changelist can update the changelist.
		/// </summary>
		Submitted = 4,
	}
}
