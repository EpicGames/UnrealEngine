// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a Perforce user
	/// </summary>
	public class UserRecord
	{
		/// <summary>
		/// The name for the user
		/// </summary>
		[PerforceTag("User")]
		public string UserName { get; set; }

		/// <summary>
		/// Registered email address for reviews
		/// </summary>
		[PerforceTag("Email")]
		public string Email { get; set; }

		/// <summary>
		/// Last time the user's information was updated
		/// </summary>
		[PerforceTag("Update", Optional = true)]
		public DateTime Update { get; set; }

		/// <summary>
		/// Last time the user's information was accessed
		/// </summary>
		[PerforceTag("Access", Optional = true)]
		public DateTime Access { get; set; }

		/// <summary>
		/// The user's full name
		/// </summary>
		[PerforceTag("FullName")]
		public string FullName { get; set; }

		/// <summary>
		/// Paths which the user is watching
		/// </summary>
		[PerforceTag("Reviews", Optional = true)]
		public List<string> Reviews { get; } = [];

		/// <summary>
		/// The type of user
		/// </summary>
		[PerforceTag("Type", Optional = true)]
		public string? Type { get; set; }

		/// <summary>
		/// Method used to authenticate
		/// </summary>
		[PerforceTag("AuthMethod")]
		public string? AuthMethod { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private UserRecord()
		{
			UserName = null!;
			Email = null!;
			FullName = null!;
			Type = null!;
		}

		/// <summary>
		/// Summarize this record for display in the debugger
		/// </summary>
		/// <returns>Formatted record</returns>
		public override string ToString()
		{
			return FullName;
		}
	}
}
