// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Options for the p4 users command
	/// </summary>
	[Flags]
	public enum UsersOptions
	{
		/// <summary>
		/// No options specified
		/// </summary>
		None = 0,

		/// <summary>
		/// Include service users in list.
		/// </summary>
		IncludeServiceUsers = 1,

		/// <summary>
		/// On replica servers, only user information from the master server are reported.
		/// </summary>
		OnlyMasterServer = 2,

		/// <summary>
		/// Login information: includes time of last password change and login ticket expiry, if applicable. You must be a Helix server superuser to use this option.
		/// </summary>
		IncludeLoginInfo = 4,

		/// <summary>
		/// On replica servers, only users who have used this replica server are reported.
		/// </summary>
		OnlyReplicaServer = 8,
	}
}
