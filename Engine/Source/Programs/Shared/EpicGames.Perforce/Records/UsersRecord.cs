// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a Perforce user
	/// </summary>
	[DebuggerDisplay("{FullName}")]
	public class UsersRecord
	{
		/// <summary>
		/// The name for the user
		/// </summary>
		[PerforceTag("User")]
		public string UserName { get; set; } = String.Empty;

		/// <summary>
		/// Registered email address for reviews
		/// </summary>
		[PerforceTag("Email")]
		public string Email { get; set; } = String.Empty;

		/// <summary>
		/// Last time the user's information was updated
		/// </summary>
		[PerforceTag("Update", Optional = true)]
		public DateTime Update { get; set; }

		/// <summary>
		/// Last time the user's information was accessed
		/// </summary>
		[PerforceTag("Access", Optional = true)]
		public DateTime Access { get; set; }

		/// <summary>
		/// The user's full name
		/// </summary>
		[PerforceTag("FullName")]
		public string FullName { get; set; } = String.Empty;

		/// <summary>
		/// The type of user
		/// </summary>
		[PerforceTag("Type", Optional = true)]
		public string? Type { get; set; }
	}
}
