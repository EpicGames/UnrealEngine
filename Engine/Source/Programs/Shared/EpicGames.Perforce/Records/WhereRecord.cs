// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Perforce
{
	/// <summary>
	/// Contains information about a file's location in the workspace
	/// </summary>
	public class WhereRecord
	{
		/// <summary>
		/// Path to the file in the depot
		/// </summary>
		[PerforceTag("depotFile")]
		public string DepotFile { get; set; }

		/// <summary>
		/// Path to the file in client syntax
		/// </summary>
		[PerforceTag("clientFile")]
		public string ClientFile { get; set; }

		/// <summary>
		/// Path to the file on dist
		/// </summary>
		[PerforceTag("path")]
		public string Path { get; set; }

		/// <summary>
		/// Indicates that the given file or path is being unmapped from the workspace
		/// </summary>
		[PerforceTag("unmap", Optional = true)]
		public bool Unmap { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		private WhereRecord()
		{
			DepotFile = null!;
			ClientFile = null!;
			Path = null!;
		}

		/// <summary>
		/// Summarize this record for display in the debugger
		/// </summary>
		/// <returns>Formatted record</returns>
		public override string ToString()
		{
			return DepotFile;
		}
	}
}
