// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.OIDC;
using EpicGames.Serialization;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace EpicGames.ProjectStore
{
	/// <inheritdoc />
	public class CloudStorage : ICloudStorage
	{
		private readonly IOptionsSnapshot<CloudStorageOptions> _options;
		private readonly ILogger<CloudStorage> _logger;
		private readonly IServiceProvider _provider;

		/// <summary>
		/// Zens documented exit codes
		/// </summary>
		private enum ZenExitCodes
		{
			Success = 0,
			OtherError = 1,
			BadInput = 2,
			OutOfMemory = 16,
			OutOfDisk = 17,
			AssertError = 70,
			HttpOtherClientError = 80,
			HttpCantConnectError = 81,
			HttpNotFound = 66,
			HttpUnauthorized = 77,
			HttpSLLError = 82,
			HttpForbidden = 83,
			HttpTimeout = 84,
			HttpConflict = 85,
			HttpNoHost = 86,
			HttpOtherServerError = 90,
			HttpInternalServerError = 91,
			HttpServiceUnavailable = 69,
			HttpBadGateway = 92,
			HttpGatewayTimeout = 93
		}

		static readonly Regex s_logLinePattern = new(@"\[.*\]\s?\[(?<severity>warning|error|info)\]\s?(\[(?<file>\S+)\:(?<line>\d+)\])?(?<message>.*)");
		/// <summary>
		/// CTOR
		/// </summary>
		public CloudStorage(IOptionsSnapshot<CloudStorageOptions> options, ILogger<CloudStorage> logger, IServiceProvider provider)
		{
			_options = options;
			_logger = logger;
			_provider = provider;
		}

		/// <inheritdoc />
		public async Task<ICloudStorage.LoggedInStatus> LoginAsync(bool unattended = true, CloudBuildOptionsOverride? optionsOverride = null)
		{
			if (_options.Value.OidcTokenExePath == null && _options.Value.AccessToken == null)
			{
				return ICloudStorage.LoggedInStatus.LoginNotConfigured;
			}

			if (_options.Value.AccessToken != null)
			{
				// if the access token is set we assume its valid and thus say we are logged in
				return ICloudStorage.LoggedInStatus.LoggedIn;
			}

			try
			{
				string? _ = await GetAccessTokenAsync(unattended, optionsOverride);
			}
			catch (NotLoggedInException)
			{
				return ICloudStorage.LoggedInStatus.InteractiveLoginRequired;
			}
			return ICloudStorage.LoggedInStatus.LoggedIn;
		}

		private async Task<string?> GetAccessTokenAsync(bool unattended = true, CloudBuildOptionsOverride? optionsOverride = null)
		{
			if (_options.Value.AccessToken != null)
			{
				return _options.Value.AccessToken;
			}

			if (optionsOverride?.AccessToken != null)
			{
				return optionsOverride.AccessToken;
			}

			if (optionsOverride?.DefaultHost == null && optionsOverride?.OverrideHost == null && _options.Value.Host == null)
			{
				throw new CloudStorageException("No host configured");
			}

			Uri host = new(optionsOverride?.OverrideHost ?? optionsOverride?.DefaultHost ?? _options.Value.Host!);

			// TODO: Should we support local file configuration?
			ClientAuthConfigurationV1? authConfig = await ProviderConfigurationFactory.ReadRemoteAuthConfigurationAsync(host, ProviderConfigurationFactory.DefaultEncryptionKey);
			if (authConfig?.DefaultProvider == null)
			{
				throw new CloudStorageException($"Failed to determine default auth provider from {host}");
			}

			IConfiguration config = ProviderConfigurationFactory.BindOptions(authConfig);
			using ITokenStore tokenStore = _provider.GetService<ITokenStore>() ?? TokenStoreFactory.CreateTokenStore();
			// TODO: Should we fetch the token manager from DI as well? Then we can not reconfigure it
			OidcTokenManager tokenManager = OidcTokenManager.CreateTokenManager(config, tokenStore);

			bool allowInteractiveLogin = !unattended || _options.Value.AllowAuthPrompt;
			try
			{
				IOidcTokenInfo tokenInfo = await tokenManager.GetAccessToken(authConfig.DefaultProvider);

				return tokenInfo.AccessToken;
			}
			catch (NotLoggedInException)
			{
				if (!allowInteractiveLogin)
				{
					throw;
				}

				IOidcTokenInfo tokenInfo = await tokenManager.LoginAsync(authConfig.DefaultProvider);

				return tokenInfo.AccessToken;
			}
		}

		/// <inheritdoc />
		public Task DownloadBuildAsync(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			NamespaceId? ns = _options.Value.NamespaceId != null ? new NamespaceId(_options.Value.NamespaceId) : null;
			return DownloadBuildAsync(ns, bucketId, buildId, outputDir, includeFiles, excludeFiles, downloadOptions, progress, optionsOverride, cancellationToken);
		}

		/// <inheritdoc />
		public Task DownloadBuildAsyncEx(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<string>? includeFiles = null, List<string>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			NamespaceId? ns = _options.Value.NamespaceId != null ? new NamespaceId(_options.Value.NamespaceId) : null;
			return DownloadBuildAsync(ns, bucketId, buildId, outputDir, includeFiles, excludeFiles, downloadOptions, progress, optionsOverride, cancellationToken);
		}

		/// <inheritdoc />
		public void DownloadBuild(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			DownloadBuildAsync(bucketId, buildId, outputDir, includeFiles, excludeFiles, downloadOptions, progress, optionsOverride, cancellationToken).Wait(cancellationToken);
		}

		/// <inheritdoc />
		public void DownloadBuildEx(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<string>? includeFiles = null, List<string>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			DownloadBuildAsyncEx(bucketId, buildId, outputDir, includeFiles, excludeFiles, downloadOptions, progress, optionsOverride, cancellationToken).Wait(cancellationToken);
		}

		/// <inheritdoc />
		public Task DownloadBuildFromUriAsync(Uri uri, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			string path = uri.AbsolutePath;
			// remove api path if specified
			path = path.Replace("api/v2/builds/", "", StringComparison.OrdinalIgnoreCase);
			string[] components = path.Split('/', StringSplitOptions.RemoveEmptyEntries);
			if (components.Length != 3)
			{
				_logger.LogError("Failed to base uri path {Uri}, did not match expected format of namespace/bucket/buildId", path);
				throw new ArgumentException("Uri did not match expected format of namespace/bucket/buildId", nameof(uri));
			}

			string namespaceId = components[0];
			string bucketId = components[1];
			string buildId = components[2];

			optionsOverride ??= new CloudBuildOptionsOverride();
			optionsOverride.DefaultHost = uri.GetLeftPart(UriPartial.Authority);
			return DownloadBuildAsync(new NamespaceId(namespaceId), new StringId(bucketId), CbObjectId.Parse(buildId), outputDir, includeFiles, excludeFiles, downloadOptions, progress, optionsOverride, cancellationToken);
		}

		private async Task DownloadBuildAsync(NamespaceId? namespaceId, StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<FilePattern>? includeFiles, List<FilePattern>? excludeFiles, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			// Since wildcard options essentially boil down to a string, we transform the File Pattern list(s) here to string list/enumerable
			await DownloadBuildAsync(namespaceId, bucketId, buildId, outputDir, 
						includeFiles?.Select(p => ToZenWildcard(outputDir, p)), 
						excludeFiles?.Select(p => ToZenWildcard(outputDir, p)), 
						downloadOptions, progress, optionsOverride, cancellationToken);
		}

		private async Task DownloadBuildAsync(NamespaceId? namespaceId, StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, IEnumerable<string>? includeFiles, IEnumerable<string>? excludeFiles, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			NamespaceId? ns = optionsOverride?.NamespaceId ?? namespaceId;
			if (ns == null)
			{
				throw new ArgumentNullException(nameof(namespaceId), "No namespace specified when downloading build, make sure to set default namespace or override it for this download");
			}
			string http2Options = downloadOptions?.AssumeHttp2 ?? false ? "--assume-http2" : "";
			string boostWorkersOption = downloadOptions?.BoostWorkers ?? false ? "--boost-workers" : "";
			string boostWorkerMemoryOption = downloadOptions?.BoostWorkerMemory ?? false ? "--boost-worker-memory" : "";
			string allowMultipartOption = downloadOptions?.AllowMultipart != null ? $"--allow-multipart={downloadOptions.AllowMultipart}" : "";
			string append = downloadOptions?.Append ?? false ? "--append=true" : String.Empty;
			string clean = downloadOptions?.Clean ?? false ? "--clean" : "";
			// trace file option needs to be passed before other arguments including the verbs
			string traceOption = downloadOptions?.TraceFile != null ? $"--tracefile={downloadOptions.TraceFile.FullName} " : "";
			string zenFolderPath = downloadOptions?.ZenStateFolder != null ? $"--zen-folder-path={downloadOptions.ZenStateFolder.FullName} " : "";

			string wildcardOption = includeFiles == null ? "" : $"--wildcard \"{String.Join(';', includeFiles)}\"";
			string excludeOption = excludeFiles == null ? "" : $"--exclude-wildcard \"{String.Join(';', excludeFiles)}\"";
			// pass the access token via environment variable to avoid showing it on the cli
			string cmdline = $"{traceOption}builds download {_options.Value.GetHostCliArgs(optionsOverride)} --namespace {ns} --bucket {bucketId} {wildcardOption} {excludeOption} {allowMultipartOption} {append} {clean} {http2Options} {boostWorkersOption} {boostWorkerMemoryOption} {zenFolderPath} \"{outputDir}\" {buildId}";

			int exitCode = await RunZenCliAsync(cmdline, progress: progress, optionsOverride: optionsOverride, cancellationToken: cancellationToken);
			EnsureZenExitCodeSuccess(exitCode);
		}

		private static string ToZenWildcard(DirectoryReference outputDirectory, FilePattern filePattern)
		{
			static IEnumerable<string> GetTokens(FilePattern pattern)
			{
				foreach (string token in pattern.Tokens)
				{
					// convert directory wildcards (...) to * as zen only has general wildcards
					yield return token == "..." ? "*" : token;
				}
			}

			string s = Path.Combine(filePattern.BaseDirectory.MakeRelativeTo(outputDirectory), String.Join(String.Empty, GetTokens(filePattern)));
			// zen assumes all wildcards are rooted in the local folder and does not support specifying it
			if (s.StartsWith(".\\", StringComparison.OrdinalIgnoreCase))
			{
				s = s.Replace(".\\", "", StringComparison.OrdinalIgnoreCase);
			}
			return s;
		}

		/// <inheritdoc />
		public IEnumerable<FindBuildResponse> FindBuilds(Regex bucketRegex, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			string? ns = GetCurrentNamespace(optionsOverride);
			if (ns == null)
			{
				throw new CloudStorageException("Unable to determine namespace to find builds for, make sure to set the default namespace or override it in the options override");
			}

			string tempInputFile = Path.Combine(Path.GetTempPath(), Path.GetRandomFileName());
			string tempOutputFile = Path.Combine(Path.GetTempPath(), Path.GetRandomFileName());
			tempInputFile = Path.ChangeExtension(tempInputFile, "cbo");
			tempOutputFile = Path.ChangeExtension(tempOutputFile, "cbo");
			try
			{
				// copy the query and append the bucket regex filter
				CbWriter writer = new();
				writer.BeginObject();
				writer.WriteObject("query", query);
				writer.WriteString("bucketRegex", bucketRegex.ToString());
				if (searchOptions != null)
				{
					CloudBuildSearchOptions options = searchOptions;
					writer.BeginObject("options");
					if (options.Skip.HasValue)
					{
						writer.WriteInteger("skip", options.Skip.Value);
					}
					if (options.Max.HasValue)
					{
						writer.WriteInteger("max", options.Max.Value);
					}
					if (options.Limit.HasValue)
					{
						writer.WriteInteger("limit", options.Limit.Value);
					}
					if (options.IncludeTTL.HasValue)
					{
						writer.WriteBool("includeTtl", options.IncludeTTL.Value);
					}
					writer.EndObject();
				}
				writer.EndObject();

				File.WriteAllBytes(tempInputFile, writer.ToByteArray());

				string cmdline = $"builds list {_options.Value.GetHostCliArgs(optionsOverride)} --namespace {ns} --query-path \"{tempInputFile}\" --result-path \"{tempOutputFile}\"";

				int exitCode = RunZenCliAsync(cmdline, optionsOverride: optionsOverride, cancellationToken: cancellationToken).Result;

				EnsureZenExitCodeSuccess(exitCode);
				byte[] b = File.ReadAllBytes(tempOutputFile);
				CbObject o = new(b);

				bool partialResult = o["partialResult"].AsBool();
				if (partialResult)
				{
					// TODO: Use a much smaller options then default to trigger pagination and return reasonable page sizes always so we do not have this warning
					_logger.LogWarning("Partial request result, not returning all possible values. Try tweaking your search options to allow for scanning more objects, reducing the wide your search goes or implementing pagination");
				}

				foreach (CbField result in o["results"].AsArray())
				{
					CbObject metadata = result["metadata"].AsObject();
					if (metadata.Equals(CbObject.Empty))
					{
						// we require metadata to be able to know which commit this build belongs to
						continue;
					}

					string buildName = metadata["name"].AsString();
					string bucketIdString = result["bucketId"].AsString();
					StringId.TryParse(new Utf8String(bucketIdString), 256, out StringId bucketId, out string? _);
					string buildIdString = result["buildId"].AsString();
					CbObjectId buildId = CbObjectId.Parse(buildIdString);
					DateTime createdAt = metadata["createdAt"].IsDateTime() ? metadata["createdAt"].AsDateTime() : DateTime.Parse(metadata["createdAt"].AsString());
					CbField commitField = metadata["commit"];

					string? commit = null;
					if (commitField.Equals(CbField.Empty))
					{
						// commit field is missing
						commit = null;
					}
					else if (commitField.IsInteger())
					{
						commit = commitField.AsInt64().ToString();
					}
					else if (commitField.IsFloat())
					{
						// due to jsons problematic number types we sometimes get doubles for integers, these are likely changelists and thus whole numbers so we just cast them
						commit = ((long)commitField.AsDouble()).ToString();
					}
					else if (commitField.IsString())
					{
						commit = commitField.AsString();
					}

					yield return new FindBuildResponse(buildName, buildId, bucketId, commit, createdAt, metadata);
				}
			}
			finally
			{
				if (File.Exists(tempInputFile))
				{
					File.Delete(tempInputFile);
				}

				if (File.Exists(tempOutputFile))
				{
					File.Delete(tempOutputFile);
				}
			}
		}

		/// <inheritdoc />
		public async IAsyncEnumerable<FindBuildResponse> FindBuildsAsync(Regex bucketRegex, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			await Task.CompletedTask;

			foreach (FindBuildResponse build in FindBuilds(bucketRegex, query, searchOptions, optionsOverride, cancellationToken))
			{
				yield return build;
			}
		}

		private static Regex BuildRegex(string project, string? type, string? branch)
		{
			string regex = ICloudStorage.SanitizeBucketValue(project);
			if (!String.IsNullOrEmpty(type))
			{
				regex += $"\\.{ICloudStorage.SanitizeBucketValue(type)}";
			}
			if (!String.IsNullOrEmpty(branch))
			{
				regex += $"\\.{ICloudStorage.SanitizeBucketValue(branch)}";
			}
			// wildcard after this prefix
			regex += "\\.*";
			return new Regex(regex);
		}

		/// <inheritdoc />
		public IAsyncEnumerable<FindBuildResponse> FindBuildsAsync(string project, string? type, string? branch, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			Regex regex = BuildRegex(project, type, branch);

			return FindBuildsAsync(regex, query, searchOptions, optionsOverride, cancellationToken);
		}

		/// <inheritdoc />
		public IEnumerable<FindBuildResponse> FindBuilds(string project, string? type, string? branch, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			Regex regex = BuildRegex(project, type, branch);

			return FindBuilds(regex, query, searchOptions, optionsOverride, cancellationToken);
		}

		private static ZenExitCodes AsZenExitCodeValue(int exitCode)
		{
			if (!Enum.IsDefined(typeof(ZenExitCodes), exitCode))
			{
				throw new NotImplementedException($"Unknown zen exit code: {exitCode}");
			}

			return (ZenExitCodes)exitCode;
		}

		/// <inheritdoc />
		public async Task<BuildContents> ListBuildContentsAsync(StringId bucketId, CbObjectId buildId, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			string? ns = GetCurrentNamespace(optionsOverride);
			if (ns == null)
			{
				throw new Exception("Unable to determine namespace, so unable to list contents");
			}

			string? hostString = optionsOverride?.OverrideHost ?? optionsOverride?.DefaultHost ?? _options.Value.Host;
			if (String.IsNullOrEmpty(hostString))
			{
				throw new ArgumentException("Expected host option to be set either in the global config or the per invocation options override to be able to list contents");
			}
			Uri host = new(hostString, UriKind.Absolute);
			Uri uri = new(host,$"/api/v2/builds/{ns}/{bucketId}/{buildId}");

			string tempOutputFile = Path.Combine(Path.GetTempPath(), Path.GetRandomFileName());
			tempOutputFile = Path.ChangeExtension(tempOutputFile, "cbo");

			CbObject o;
			try
			{
				string cmdline = $"builds ls --cloud-url={uri} --result-path \"{tempOutputFile}\"";
				int exitCode = await RunZenCliAsync(cmdline, optionsOverride: optionsOverride, cancellationToken: cancellationToken);

				EnsureZenExitCodeSuccess(exitCode);

				byte[] b = await File.ReadAllBytesAsync(tempOutputFile, cancellationToken);
				o = new(b);
			}
			finally
			{
				if (File.Exists(tempOutputFile))
				{
					File.Delete(tempOutputFile);
				}
			}

			CbField parts = o["parts"];
			if (parts.Equals(CbField.Empty))
			{
				throw new Exception("Failed to parse output file when listing content, unable to find parts field");
			}

			Dictionary<string, PartContents> partsDict = [];

			foreach (CbField part in parts.AsArray())
			{
				Dictionary<string, FileDescription> foundFiles = [];

				CbObject objectPart = part.AsObject();
				CbObjectId partId = objectPart["id"].AsObjectId();
				string partName = objectPart["partName"].AsString();
				ulong partSize = 0;

				foreach (CbField files in objectPart["files"].AsArray())
				{
					CbObject filesObject = files.AsObject();
					string path = filesObject["path"].AsString();

					CbField attributesField = filesObject.Find("attributes");
					int attributes = 0;
					if (!attributesField.Equals(CbField.Empty))
					{
						attributes = attributesField.AsInt32();
					}

					CbField modeField = filesObject.Find("mode");
					int mode = 0;
					if (!modeField.Equals(CbField.Empty))
					{
						mode = modeField.AsInt32();
					}
					ulong rawSize = filesObject["rawSize"].AsUInt64();

					IoHash rawHash = IoHash.Zero;

					CbField rawHashField = filesObject.Find("rawHash");
					if (!rawHashField.Equals(CbField.Empty))
					{
						// zen builds ls does not currently output the raw hash but will in a future release so making this optional for now
						rawHash = rawHashField.AsHash();
					}
					partSize += rawSize;
					foundFiles.Add(path, new FileDescription {Attributes = attributes, RawHash = rawHash, Mode = mode, RawSize = rawSize});
				}

				partsDict.Add(partName, new PartContents
				{
					PartId = partId,
					PartName = partName,
					TotalSize = partSize,
					Files = foundFiles
				});
			}

			return new BuildContents { Parts = partsDict };
		}

		private void EnsureZenExitCodeSuccess(int exitCode)
		{
			if (exitCode != 0)
			{
				string msg = AsZenExitCodeValue(exitCode).ToString();
				_logger.LogError("Zen non-zero exit code {ExitCode} ({Message})", exitCode, msg);
				throw new CloudStorageException($"Zen exited with non-success exit code: {exitCode} ({msg})");
			}
		}

		/// <inheritdoc />
		public Task Reconfigure(Action<CloudStorageOptions> action)
		{
			action(_options.Value);

			// throws validation exceptions if there are any issues
			Validator.ValidateObject(_options.Value, new ValidationContext(_options.Value), validateAllProperties: true);

			return Task.CompletedTask;
		}

		/// <inheritdoc />
		public string? GetCurrentNamespace(CloudBuildOptionsOverride? optionsOverride)
		{
			return optionsOverride?.NamespaceId?.ToString() ?? _options.Value.NamespaceId;
		}

		/// <inheritdoc />
		public async Task ImportSnapshotAsync(string localFileName, string localRootPath, string engineDir, string cookedPlatformDir, bool forceImport = false, bool asyncImport = false, CancellationToken cancellationToken = default)
		{
			/*string projectId = GetProjectID(projectFile);
			CreateProject(projectFile, rootDirectory, engineDirectory);

			platformCookedDirectory = new DirectoryReference(platformCookedDirectory.FullName.Replace("{Platform}", _buildData.Metadata.CookPlatform, StringComparison.Ordinal));
			FileReference projectStoreFile = WriteProjectStoreFile(projectId, _buildData.Metadata.CookPlatform, platformCookedDirectory);

			string oplogName = _buildData.Metadata.CookPlatform;

			StringBuilder oplogCreateCommandLine = new StringBuilder();
			oplogCreateCommandLine.AppendFormat("oplog-create --force-update {0} {1}", projectId, oplogName);

			StringBuilder oplogImportCommandLine = new StringBuilder();
			oplogImportCommandLine.AppendFormat("oplog-import {0} {1} {2} --clean", projectId, oplogName, projectStoreFile.FullName);

			if (forceImport)
			{
				oplogImportCommandLine.AppendFormat(" --force");
			}

			if (asyncImport)
			{
				oplogImportCommandLine.AppendFormat(" --async");
			}

			oplogImportCommandLine.AppendFormat(" --namespace {0}", _namespace);
			oplogImportCommandLine.AppendFormat(" --access-token {0}", _accessToken);
			oplogImportCommandLine.AppendFormat(" --bucket {0}", _bucket);
			oplogImportCommandLine.AppendFormat(" --builds {0}", _host);
			oplogImportCommandLine.AppendFormat(" --builds-id {0}", _buildData.BuildId);

			await RunZenCliAsync(oplogCreateCommandLine.ToString());
			await RunZenCliAsync(oplogImportCommandLine.ToString());*/

			await Task.CompletedTask;
			throw new NotImplementedException();
		}

		private async Task<int> RunZenCliAsync(string cmdline, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default)
		{
			FileReference zenExe = _options.Value.ZenExePath!;

			string? accessToken = _options.Value.AccessToken;
			if (_options.Value.FetchAccessTokenInProcess)
			{
				accessToken = await GetAccessTokenAsync(unattended: true, optionsOverride);
			}

			// append auth arguments
			Dictionary<string, string>? env = null;
			if (!String.IsNullOrEmpty(accessToken))
			{
				cmdline += " --access-token-env UE_CloudDataCacheAccessToken ";

				env = [];
				// forward the current environment and then override the token
				foreach (DictionaryEntry entry in Environment.GetEnvironmentVariables())
				{
					if (entry is { Key: string k, Value: string v })
					{
						env[k] = v;
					}
				}

				env.Add("UE_CloudDataCacheAccessToken", accessToken);
			}
			else if (_options.Value.OidcTokenExePath != null)
			{
				string unattendedOption = _options.Value.AllowAuthPrompt ? "--oidctoken-exe-unattended=true" : "";
				cmdline += $" --oidctoken-exe-path=\"{_options.Value.OidcTokenExePath}\" {unattendedOption}";
			}

			// if we have a progress callback with to the log progress mode
			if (progress != null)
			{
				cmdline += " --log-progress";
			}
			else
			{
				// when no progress reporting is requested we use the plain progress which simply outputs console output about the progress in a way that works well with redirected output
				cmdline += " --plain-progress";
			}

			_logger.Log(LogLevel.Information, "Running: {App} with {CommandLine}", zenExe.FullName, cmdline);

			ILogger? zenLogger = null;

			if (_options.Value.EchoZenCliToLog && optionsOverride?.EchoLogger == null)
			{
				ILoggerFactory loggerFactory = _provider.GetRequiredService<ILoggerFactory>();
				zenLogger = loggerFactory.CreateLogger("ZenCli");
			}
			else if (_options.Value.EchoZenCliToLog)
			{
				zenLogger = optionsOverride?.EchoLogger;
			}

			if (_options.Value.EchoZenCliToStdout && _options.Value.WriteLogEventStatements)
			{
				// if we are echoing zen output to stdout we instruct the horde log event parsing to not run its usually categorization of errors and instead allow us to control errors
				Console.WriteLine("<-- Suspend Log Parsing -->");
			}

			try
			{
				const int WarnAtAttempts = 2;
				// increasingly long time to wait before retrying
				int[] waitDurationSeconds = [60, 300, 900, 3600];
				int maxRetryAttempts = waitDurationSeconds.Length;

				for (int retryAttempt = 0; retryAttempt < maxRetryAttempts; retryAttempt++)
				{
					int exitCode = await RunProcessAsync(zenExe.FullName, cmdline, env: env, echoStdout: _options.Value.EchoZenCliToStdout, echoLogger: zenLogger, progress: progress, cancellationToken: cancellationToken);

					int waitDuration = waitDurationSeconds[retryAttempt];

					ZenExitCodes zenExitCode = AsZenExitCodeValue(exitCode);
					// retry on zen exit codes that indicate transient issues
					if (zenExitCode is ZenExitCodes.HttpServiceUnavailable or ZenExitCodes.HttpBadGateway or ZenExitCodes.HttpGatewayTimeout or ZenExitCodes.HttpTimeout)
					{
						if (retryAttempt + 1 == maxRetryAttempts)
						{
							// give up
							_logger.LogError("Zen returned http status code {ExitCode}({ExitCodeMessage}) after {RetryAttempts} retries. Giving up.",
								exitCode, AsZenExitCodeValue(exitCode), retryAttempt);

							// we return the exit code even though it's a failure to allow the caller to handle this issue that isn't fixed by retrying
						}
						else
						{
							if (retryAttempt + 1 == WarnAtAttempts)
							{
								_logger.LogWarning(
									"Zen returned http status code {ExitCode}({ExitCodeMessage}) after {RetryAttempts} retries. This is a indication of an infrastructure issue.",
									exitCode, AsZenExitCodeValue(exitCode), retryAttempt);
							}
							else
							{
								_logger.LogInformation(
									"Http Status code {ExitCode}({ExitCodeMessage}) returned from zen. Waiting {WaitDuration}s before trying.",
									exitCode, AsZenExitCodeValue(exitCode), waitDuration);
							}

							await Task.Delay(TimeSpan.FromSeconds(waitDuration), cancellationToken);
							continue; // retry
						}
					}

					return exitCode;
				}

				throw new CloudStorageException($"Failed to run zen cli, gave up after {maxRetryAttempts} attempts");
			}
			finally
			{
				if (_options.Value.EchoZenCliToStdout && _options.Value.WriteLogEventStatements)
				{
					// resume normal log parsing
					Console.WriteLine("<-- Resume Log Parsing -->");
				}
			}
		}

		private async Task<int> RunProcessAsync(string cmd, string arguments, string? workingDirectory = null, Dictionary<string, string>? env = null, IProgress<(float, string)>? progress = null, bool echoStdout = false, ILogger? echoLogger = null, CancellationToken cancellationToken = default)
		{
			using ManagedProcessGroup managedProcessGroup = new();
			using ManagedProcess managedProcess = new(managedProcessGroup, cmd, arguments, workingDirectory, env, ProcessPriorityClass.Normal, flags: ManagedProcessFlags.MergeOutputPipes);

			_options.Value.OnProcessCreated?.Invoke(managedProcess);

			ProgressValue progressValue = new();
			while (managedProcess.TryReadLine(out string? line, cancellationToken))
			{
				if (progress != null && line.StartsWith(ProgressTextReader.DirectivePrefix, StringComparison.Ordinal))
				{
					line = ProgressTextReader.ParseLine(line, progressValue);
					progress.Report((progressValue.Current.Item2, progressValue.Current.Item1));

					if (String.IsNullOrEmpty(line))
					{
						// line only had progress reporting, move on
						continue;
					}
				}

				if (echoLogger != null)
				{
					EchoToLogger(echoLogger, line);
				}
				if (echoStdout)
				{
					Console.WriteLine(line);
				}
			}

			cancellationToken.ThrowIfCancellationRequested();

			await managedProcess.WaitForExitAsync(cancellationToken);
			_options.Value.OnProcessExited?.Invoke(managedProcess);

			cancellationToken.ThrowIfCancellationRequested();

			return managedProcess.ExitCode;
		}

		private static void EchoToLogger(ILogger zenLogger, string line)
		{
			// TODO: Ideally zen cli has structured logging here that we can parse and output back as a structured logging event, for now we just text match
			Match match = s_logLinePattern.Match(line);
			if (match.Success)
			{
				// we ignore the file matching for now
				//string file = match.Groups["file"].Value;
				//string lineNumber = match.Groups["line"].Value;
				string message = match.Groups["message"].Value;

				LogLevel level = match.Groups["severity"].Value switch
				{
					"error" => LogLevel.Error,
					"warning" => LogLevel.Warning,
					_ => LogLevel.Information,
				};

#pragma warning disable CA2254
				zenLogger.Log(level, message);
#pragma warning restore CA2254
			}
			else
			{
				// didn't match the structured output
#pragma warning disable CA2254
				zenLogger.LogInformation(line);
#pragma warning restore CA2254
			}
		}
	}
}
