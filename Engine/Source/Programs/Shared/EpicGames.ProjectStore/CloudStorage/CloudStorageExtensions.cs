// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.Extensions.DependencyInjection;

namespace EpicGames.ProjectStore
{
	/// <summary>
	/// Extension methods for Cloud Storage
	/// </summary>
	public static class CloudStorageExtensions
	{
		/// <summary>
		/// Adds Cloud Storage-related services with the default settings
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		public static void AddCloudStorage(this IServiceCollection serviceCollection)
		{
			// This is registered as transient before of the Reconfigure call which will mutate the options of a ICloudStorage
			// That option is required due to some cases where configuration is read after all our services would normally be configured
			// This is not a very big deal as CloudStorage itself has limited local state
			serviceCollection.AddTransient<ICloudStorage, CloudStorage>();

			serviceCollection.AddOptions<CloudStorageOptions>();
		}

		/// <summary>
		/// Adds Cloud Storage-related services with configuration
		/// </summary>
		/// <param name="serviceCollection">Collection to register services with</param>
		/// <param name="configureCloudStorage">Callback to configure options</param>
		public static void AddCloudStorage(this IServiceCollection serviceCollection, Action<CloudStorageOptions> configureCloudStorage)
		{
			serviceCollection.Configure(configureCloudStorage);
			AddCloudStorage(serviceCollection);
		}
	}
}
