// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Serialization;

namespace EpicGames.ProjectStore
{
	/// <summary>
	/// Helper class to build a query for finding builds using cloud storage
	/// </summary>
	public class CloudStorageQueryBuilder
	{
		private readonly CbWriter _writer = new();
		private CbObject? _generatedObject = null;

		/// <summary>
		/// CTOR
		/// </summary>
		public CloudStorageQueryBuilder()
		{
			// open the writer so that we can start appending request to it
			_writer.BeginObject();
		}

		/// <summary>
		/// Search for exact value in a field
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void Equals<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$eq", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any value except value in a field
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void NotEquals<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$neq", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any value less than, type must be able to be compared
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void LessThen<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$lt", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any value less than or equal, type must be able to be compared
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void LessThenOrEqual<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$lte", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any value greater than, type must be able to be compared
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void GreaterThen<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$gt", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any value greater than or equal, type must be able to be compared
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="value">The value to filter for</param>
		public void GreaterThenOrEqual<T>(string fieldName, T value)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			WriteValue("$gte", value);
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any of the exact values in a list
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="values">The values to filter for</param>
		public void In<T>(string fieldName, IEnumerable<T> values)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			_writer.BeginArray("$in");
			foreach (T value in values)
			{
				WriteValue(null, value);
			}
			_writer.EndArray();
			_writer.EndObject();
		}

		/// <summary>
		/// Search for any of the exact values to not be in this list
		/// </summary>
		/// <typeparam name="T">Type</typeparam>
		/// <param name="fieldName">The field name</param>
		/// <param name="values">The values to filter for</param>
		public void NotIn<T>(string fieldName, IEnumerable<T> values)
		{
			ThrowOnLocked();

			_writer.BeginObject(fieldName);
			_writer.BeginArray("$nin");
			foreach (T value in values)
			{
				WriteValue(null, value);
			}
			_writer.EndArray();
			_writer.EndObject();
		}

		/// <summary>
		/// Returns the built query and locks the builder to prevent further appended filters
		/// </summary>
		/// <returns></returns>
		public CbObject Done()
		{
			if (_generatedObject != null)
			{
				return _generatedObject;
			}

			_writer.EndObject();
			_generatedObject = _writer.ToObject();

			return _generatedObject;
		}

		private void WriteValue<T>(string? fieldName, T value)
		{
			CbConverter<T> converter = CbConverter.GetConverter<T>();

			if (converter == null)
			{
				throw new NotImplementedException($"Failed to find a converter for type \"{typeof(T)}\", you may want to try to use a basic type like string");
			}

			if (fieldName != null)
			{
				converter.WriteNamed(_writer, fieldName, value);
			}
			else
			{
				converter.Write(_writer, value);
			}
		}

		private void ThrowOnLocked()
		{
			if (_generatedObject != null)
			{
				throw new Exception("Query builder is locked, more filters can not be added after the object has been fetched");
			}
		}
	}
}
