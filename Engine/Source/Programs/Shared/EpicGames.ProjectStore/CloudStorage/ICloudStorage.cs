// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.IO;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace EpicGames.ProjectStore
{
	/// <summary>
	/// A build found during search
	/// </summary>
	/// <param name="Name">The name of the build</param>
	/// <param name="BuildId">The unique id of the build</param>
	/// <param name="BucketId">The bucket the build belongs to</param>
	/// <param name="Commit">The commit of the build, optional and can be a changelist, hash or other global unique value</param>
	/// <param name="CreatedAt">The time this build was created</param>
	/// <param name="Metadata">Metadata associated with the build</param>
	public record FindBuildResponse(
		string Name,
		CbObjectId BuildId,
		StringId BucketId,
		string? Commit,
		DateTime CreatedAt,
		CbObject Metadata)
	{
		/// <summary>
		/// The globally unique id of the build
		/// </summary>
		public CbObjectId BuildId { get; init; } = BuildId;

		/// <summary>
		/// Which bucket the build is in
		/// </summary>
		public StringId BucketId { get; init; } = BucketId;

		/// <summary>
		/// The commit identifier as defined in source control typically an incrementing number (changelist) or hash (git commit), can be null if missing
		/// </summary>
		public string? Commit { get; init; } = Commit;

		/// <summary>
		/// The name of the build
		/// </summary>
		public string Name { get; init; } = Name;

		/// <summary>
		/// When the build was created
		/// </summary>
		public DateTime CreatedAt { get; init; } = CreatedAt;

		/// <summary>
		/// The metadata of the build
		/// </summary>
		public CbObject Metadata { get; init; } = Metadata;
	}

	/// <summary>
	/// The contents available for a part
	/// </summary>
	public readonly record struct PartContents(string PartName, CbObjectId PartId, ulong TotalSize, Dictionary<string, FileDescription> Files)
	{
		/// <summary>
		/// The name of the part
		/// </summary>
		public required string PartName { get; init; } = PartName;

		/// <summary>
		/// The id of the part, usually used in apis to uniquely identify this part
		/// </summary>
		public CbObjectId PartId { get; init; } = PartId;

		/// <summary>
		/// The total uncompressed size of the part
		/// </summary>
		public required ulong TotalSize { get; init; } = TotalSize;

		/// <summary>
		/// All the files in the part
		/// </summary>
		public required Dictionary<string, FileDescription> Files { get; init; } = Files;
	}

	/// <summary>
	/// Description of an individual file
	/// </summary>
	public readonly record struct FileDescription(IoHash RawHash, ulong RawSize, int Attributes, int Mode)
	{
		/// <summary>
		/// Hash of uncompressed content
		/// </summary>
		public required IoHash RawHash { get; init; } = RawHash;

		/// <summary>
		/// Size of the file before compression
		/// </summary>
		public required ulong RawSize { get; init; } = RawSize;

		/// <summary>
		/// Windows file attributes applied to the file
		/// </summary>
		public required int Attributes { get; init; } = Attributes;

		/// <summary>
		/// Linux/Mac attributes applied to the file
		/// </summary>
		public required int Mode { get; init; } = Mode;
	}

	/// <summary>
	/// The contents available for a build
	/// </summary>
	public record struct BuildContents
	{
		/// <summary>
		/// Each part of the build
		/// </summary>
		public required Dictionary<string, PartContents> Parts { get; init; }
	}

	/// <summary>
	/// Options passed to the search
	/// </summary>
	public record CloudBuildSearchOptions
	{
		/// <summary>
		/// Max number of builds returned
		/// </summary>
		public int? Limit { get; set; }

		/// <summary>
		/// Number of builds to skip before returning result, useful to paginate results
		/// </summary>
		public int? Skip { get; set; }

		/// <summary>
		/// Max number of builds considered in the search request, you will usually want Limit rather then Max
		/// </summary>
		public int? Max { get; set; }

		/// <summary>
		/// Set to true to include how long the build will live, makes the search slower
		/// </summary>
		public bool? IncludeTTL { get; set; }
	}

	/// <summary>
	/// Per command overrides of common options
	/// </summary>
	public record CloudBuildOptionsOverride
	{
		/// <summary>
		/// Override which host is being used by default for this command, this host is used with service discovery to determine optimal host for this invocation
		/// </summary>
		public string? DefaultHost { get; set; }

		/// <summary>
		/// Explicitly define which host to use ignoring any service discovery, only use this is you explicitly want to hit a specific host
		/// </summary>
		public string? OverrideHost { get; set; }

		/// <summary>
		/// Override which namespace is being used
		/// </summary>
		public NamespaceId? NamespaceId { get; set; }

		/// <summary>
		/// Override the access token used for auth
		/// </summary>
		public string? AccessToken { get; set; }

		/// <summary>
		/// Logger to echo output to, echo log to output option must also be set. If this logger is not provided but that option is set a global ZenCli logger is used instead
		/// </summary>
		public ILogger? EchoLogger { get; set; }
	}

	/// <summary>
	/// Advanced options for downloading builds
	/// </summary>
	public record CloudBuildDownloadOptions
	{
		/// <summary>
		/// Set to override if we allow usage of multipart downloads
		/// </summary>
		public bool? AllowMultipart { get; set; }

		/// <summary>
		/// Set to opt out of http upgrade - needed if using http2 without TLS
		/// </summary>
		public bool AssumeHttp2 { get; set; }

		/// <summary>
		/// Set to boost the number of worker threads used by zen. Will usually speed up downloads if there is CPU capacity available but can cause you machine to become unresponsive
		/// </summary>
		public bool BoostWorkers { get; set; }

		/// <summary>
		/// Set to increase memory used by workers when BoostWorkers is enabled to reduce disk pressure by buffering more in memory
		/// </summary>
		public bool BoostWorkerMemory { get; set; }

		/// <summary>
		/// Optional path to an utrace file, if specified Zen will output its utrace file here
		/// </summary>
		public FileReference? TraceFile { get; set; }

		/// <summary>
		/// Override directory location for the zen state folder
		/// </summary>
		public DirectoryReference? ZenStateFolder { get; set; }

		/// <summary>
		/// Whether to append downloads to an existing directory (true) or to clear the directory before download (false)
		/// Set this to true if you want to download an artifact to a directory, then download an additional artifact from another without wiping state between
		/// </summary>
		public bool Append { get; set; }

		/// <summary>
		/// Delete all data in target folder that is not part of the downloaded content
		/// </summary>
		public bool Clean { get; set; }
	}

	/// <summary>
	/// General exception thrown from Cloud Storage
	/// </summary>
	/// <param name="msg">Error message</param>
	public class CloudStorageException(string msg) : Exception(msg);

	/// <summary>
	/// Interface for interacting with UE Cloud Storage
	/// </summary>
	public interface ICloudStorage
	{
		/// <summary>
		/// The possible login states
		/// </summary>
		enum LoggedInStatus
		{
			/// <summary>
			/// Has an active access token
			/// </summary>
			LoggedIn,
			/// <summary>
			/// No access token found, will require interactive login
			/// </summary>
			InteractiveLoginRequired,

			/// <summary>
			/// Unattended login specified but no login configuration given
			/// </summary>
			LoginNotConfigured
		}

		/// <summary>
		/// Login against cloud storage, will trigger an interactive login unless set to unattended
		/// </summary>
		/// <param name="unattended">Set as false to allow interactive logins</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>
		/// <returns></returns>
		Task<LoggedInStatus> LoginAsync(bool unattended = true, CloudBuildOptionsOverride? optionsOverride = null);

		/// <summary>
		/// Download a build using all the required fields
		/// </summary>
		/// <param name="bucketId">The bucket</param>
		/// <param name="buildId">The id of the build</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="includeFiles">List of file patterns of files to download, if specified only files matching these patterns are downloaded</param>
		/// <param name="excludeFiles">List of file patterns to exclude. If combined with include files these apply after the include files</param>
		/// <param name="downloadOptions">Advanced options for reconfiguring the download</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		Task DownloadBuildAsync(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Download a build using all the required fields. Ex: Pass file patterns as strings directly to <paramref name="includeFiles"/> and <paramref name="excludeFiles"/> params. 
		/// </summary>
		/// <param name="bucketId">The bucket</param>
		/// <param name="buildId">The id of the build</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="includeFiles">List of file patterns of files to download, if specified only files matching these patterns are downloaded</param>
		/// <param name="excludeFiles">List of file patterns to exclude. If combined with include files these apply after the include files</param>
		/// <param name="downloadOptions">Advanced options for reconfiguring the download</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>
		/// <param name="cancellationToken">A cancellation token</param>
		Task DownloadBuildAsyncEx(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<string>? includeFiles = null, List<string>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Download a build using all the required fields
		/// </summary>
		/// <param name="bucketId">The bucket</param>
		/// <param name="buildId">The id of the build</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="includeFiles">List of file patterns of files to download, if specified only files matching these patterns are downloaded</param>
		/// <param name="excludeFiles">List of file patterns to exclude. If combined with include files these apply after the include files</param>
		/// <param name="downloadOptions">Advanced options for reconfiguring the download</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		void DownloadBuild(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Download a build using all the required fields. Ex: Pass file patterns as strings directly to <paramref name="includeFiles"/> and <paramref name="excludeFiles"/> params. 
		/// </summary>
		/// <param name="bucketId">The bucket</param>
		/// <param name="buildId">The id of the build</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="includeFiles">List of file patterns of files to download, if specified only files matching these patterns are downloaded</param>
		/// <param name="excludeFiles">List of file patterns to exclude. If combined with include files these apply after the include files</param>
		/// <param name="downloadOptions">Advanced options for reconfiguring the download</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		void DownloadBuildEx(StringId bucketId, CbObjectId buildId, DirectoryReference outputDir, List<string>? includeFiles = null, List<string>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Download a build using a single uri value that encodes everything needed to locate the build
		/// </summary>
		/// <param name="uri">The uri to download from</param>
		/// <param name="outputDir">The directory to write the build to</param>
		/// <param name="includeFiles">List of file patterns of files to download, if specified only files matching these patterns are downloaded</param>
		/// <param name="excludeFiles">List of file patterns to exclude. If combined with include files these apply after the include files</param>
		/// <param name="downloadOptions">Advanced options for reconfiguring the download</param>
		/// <param name="progress">Progress reporting</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		Task DownloadBuildFromUriAsync(Uri uri, DirectoryReference outputDir, List<FilePattern>? includeFiles = null, List<FilePattern>? excludeFiles = null, CloudBuildDownloadOptions? downloadOptions = null, IProgress<(float, string)>? progress = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Search for builds in UE Cloud Storage using a regular expression to describe which buckets the build might be in
		/// </summary>
		/// <param name="bucketRegex">A regex that describe which buckets the build might be in</param>
		/// <param name="query">CbObject that describes the query, see UE Cloud Storage documentation</param>
		/// <param name="searchOptions">Optional set of options for the search</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		IAsyncEnumerable<FindBuildResponse> FindBuildsAsync(Regex bucketRegex, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Search for builds in UE Cloud Storage using a regular expression to describe which buckets the build might be in
		/// </summary>
		/// <param name="bucketRegex">A regex that describe which buckets the build might be in</param>
		/// <param name="query">CbObject that describes the query, see UE Cloud Storage documentation</param>
		/// <param name="searchOptions">Optional set of options for the search</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		IEnumerable<FindBuildResponse> FindBuilds(Regex bucketRegex, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Search for builds in UE Cloud Storage using common components
		/// </summary>
		/// <param name="project">The project you are searching builds for</param>
		/// <param name="type">The type of build you are searching for, if omitted all types are considered</param>
		/// <param name="branch">The branch you which to search, if omitted all branches are searched</param>
		/// <param name="query">CbObject that describes the query, see UE Cloud Storage documentation</param>
		/// <param name="searchOptions">Optional set of options for the search</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		IAsyncEnumerable<FindBuildResponse> FindBuildsAsync(string project, string? type, string? branch, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Search for builds in UE Cloud Storage using common components
		/// </summary>
		/// <param name="project">The project you are searching builds for</param>
		/// <param name="type">The type of build you are searching for, if omitted all types are considered</param>
		/// <param name="branch">The branch you which to search, if omitted all branches are searched</param>
		/// <param name="query">CbObject that describes the query, see UE Cloud Storage documentation</param>
		/// <param name="searchOptions">Optional set of options for the search</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		IEnumerable<FindBuildResponse> FindBuilds(string project, string? type, string? branch, CbObject query, CloudBuildSearchOptions? searchOptions = null, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// List the contents of a build
		/// </summary>
		/// <param name="bucketId">The bucket the build is in</param>
		/// <param name="buildId">The unique id of the build</param>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>		
		/// <param name="cancellationToken">A cancellation token</param>
		/// <returns></returns>
		Task<BuildContents> ListBuildContentsAsync(StringId bucketId, CbObjectId buildId, CloudBuildOptionsOverride? optionsOverride = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Override the configuration of this instance of cloud storage, mostly useful if you get configuration from another source then the default inis / environment variables
		/// </summary>
		/// <param name="action">Action that reconfigures the options</param>
		/// <returns></returns>
		Task Reconfigure(Action<CloudStorageOptions> action);

		/// <summary>
		/// Helper method to turn a bucket component into a value that is compatible with cloud storage bucket names
		/// </summary>
		/// <param name="s">The value to clean up</param>
		/// <returns></returns>
		static string SanitizeBucketValue(string s)
		{
			// bucket components are lower cased
			s = s.ToLower();
			// trim starting + and then replace any + and . with -
			s = s.TrimStart('+');
			s = s.Replace("+", "-", StringComparison.OrdinalIgnoreCase);
			// . have special meaning in the bucket names, its used as a separator between components, as such component can not themselves contain .
			return s.Replace(".", "-", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Returns the current namespace used
		/// </summary>
		/// <param name="optionsOverride">Optional per invocation overrides of options</param>	
		/// <returns></returns>
		string? GetCurrentNamespace(CloudBuildOptionsOverride? optionsOverride);

		/// <summary>
		/// Import a snapshot to the local zen
		/// </summary>
		/// <param name="localFileName"></param>
		/// <param name="localRootPath"></param>
		/// <param name="engineDir"></param>
		/// <param name="cookedPlatformDir"></param>
		/// <param name="forceImport"></param>
		/// <param name="asyncImport"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task ImportSnapshotAsync(string localFileName, string localRootPath, string engineDir, string cookedPlatformDir, bool forceImport = false, bool asyncImport = false, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Options for cloud storage
	/// </summary>
	public record CloudStorageOptions : IValidatableObject
	{
		/// <summary>
		/// Set too true to allow auth prompts, recommended to set to false for un-interactive scenarios
		/// </summary>
		public bool AllowAuthPrompt { get; set; } = false;

		/// <summary>
		/// The host name to use to connect to cloud storage
		/// </summary>
		[Url]
		[Required]
		public string? Host { get; set; }

		/// <summary>
		///  The namespace to use when connecting to cloud storage
		/// </summary>
		[Required]
		public string? NamespaceId { get; set; }

		/// <summary>
		/// The Baseline branch to use for builds within cloud storage
		/// </summary>
		[Required]
		public string? BaselineBranch { get; set; }

		/// <summary>
		/// The path to the zen cli executable
		/// </summary>
		[Required]
		public FileReference? ZenExePath { get; set; }

		/// <summary>
		/// The path to the OidcToken executable
		/// </summary>
		public FileReference? OidcTokenExePath { get; set; }

		/// <summary>
		/// The explicit access token to use when communicating with Cloud Storage, it's usually best to not set this and specify the OidcTokenExePath instead to allow for refreshing of tokens if the background task end up long-running
		/// </summary>
		public string? AccessToken { get; set; } = null;

		/// <summary>
		/// Set to fetch access token via EpicGames.Oidc instead of using the OidcToken executable. This lets you build a tool that doesn't need to ship the token executable but prevents zen from refreshing the access token as needed so long-running operations can fail
		/// </summary>
		public bool FetchAccessTokenInProcess { get; set; } = false;

		/// <summary>
		/// Callback when a process has been created
		/// </summary>
		public Action<ManagedProcess>? OnProcessCreated { get; set; } = null;

		/// <summary>
		/// Callback when a process we created has exited
		/// </summary>
		public Action<ManagedProcess>? OnProcessExited { get; set; } = null;

		/// <summary>
		/// Set to true to echo all zencli output to stdout
		/// </summary>
		public bool EchoZenCliToStdout { get; set; } = false;

		/// <summary>
		/// Set to true to echo all zen cli logs to a Microsoft.Extensions.Logger called "ZenCli"
		/// </summary>
		public bool EchoZenCliToLog { get; set; } = false;

		/// <summary>
		/// Used to control if log event directives are written into sources that it will consume
		/// </summary>
		public bool WriteLogEventStatements { get; set; } = true;

		/// <summary>
		/// Implements verification of the settings object for IValidatableObject
		/// </summary>
		/// <param name="validationContext"></param>
		/// <returns></returns>
		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> results = [];
			if (String.IsNullOrEmpty(AccessToken) && OidcTokenExePath == null && FetchAccessTokenInProcess == false)
			{
				results.Add(new ValidationResult("AccessToken or OidcTokenExePath needs to be defined for auth to function"));
			}

			if (FetchAccessTokenInProcess == false && OidcTokenExePath != null && !FileReference.Exists(OidcTokenExePath))
			{
				results.Add(new ValidationResult($"Unable to find OidcToken at path \"{OidcTokenExePath}\""));
			}

			if (ZenExePath != null && !FileReference.Exists(ZenExePath))
			{
				results.Add(new ValidationResult($"Unable to find Zen executable at path \"{ZenExePath}\""));
			}

			return results;
		}

		/// <summary>
		/// Finds paths for executables based on unreal root directory
		/// </summary>
		public void FindExecutablesFromUnrealRoot()
		{
			string binaryType = "Win64";
			if (OperatingSystem.IsMacOS())
			{
				binaryType = "Mac";
			}
			else if (OperatingSystem.IsLinux())
			{
				binaryType = "Linux";
			}

			DirectoryReference engineBinariesDir = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Binaries", binaryType);
			FileReference zenExe = new(Path.Combine(engineBinariesDir.FullName, "zen" + RuntimePlatform.ExeExtension));
			ZenExePath = zenExe;

			string binaryTypeDotnet = "win-x64";
			if (OperatingSystem.IsMacOS())
			{
				binaryTypeDotnet = "osx-x64";
			}
			else if (OperatingSystem.IsLinux())
			{
				binaryTypeDotnet = "linux-x64";
			}

			DirectoryReference oidcDir = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Binaries", "DotNET", "OidcToken", binaryTypeDotnet);
			FileReference oidcExe = new(Path.Combine(oidcDir.FullName, "OidcToken" + RuntimePlatform.ExeExtension));

			OidcTokenExePath = oidcExe;
		}

		/// <summary>
		/// Returns the zen cli arguments to use for this set of host arguments
		/// </summary>
		/// <param name="optionsOverride">The options overrides used</param>
		/// <returns></returns>
		internal string GetHostCliArgs(CloudBuildOptionsOverride? optionsOverride)
		{
			string? defaultHost = optionsOverride?.DefaultHost ?? Host;
			string args = "";

			if (!String.IsNullOrEmpty(defaultHost))
			{
				args += $"--host {defaultHost}";
			}

			if (optionsOverride?.OverrideHost != null)
			{
				args += $" --override-host {optionsOverride.OverrideHost}";
			}

			return args;
		}
	}
}
