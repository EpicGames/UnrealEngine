<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>
</Project>
