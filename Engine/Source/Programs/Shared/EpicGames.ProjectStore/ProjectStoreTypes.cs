// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net.Http.Headers;
using System.Net.Http;
using System.Net;
using System.Runtime.InteropServices;
using System.Text.Json.Serialization;
using System.Text.Json;
using System.Text;

using EpicGames.Core;
using EpicGames.Horde;
using UnrealBuildBase;
using EpicGames.OIDC;
using System.Threading.Tasks;
using Microsoft.Extensions.Configuration;

namespace EpicGames.ProjectStore
{
	/// <summary>
	/// How the client should operate. Whether it should stream from zenserver
	/// or use the local filesystem
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum ZenOperatingMode
	{
		/// <summary>
		/// Use Zen streaming
		/// </summary>
		Stream,
		/// <summary>
		/// Access data from local filesystem
		/// </summary>
		Filesystem
	}

	/// <summary>
	/// A specification of storage details with zenserver
	/// </summary>
	public class ZenServerStoreData
	{
		/// <summary>
		/// Whether zenserver is running locally
		/// </summary>
		public bool IsLocalHost { get; set; } = true;
		/// <summary>
		/// The primary or local host name for zenserver
		/// </summary>
		public string? HostName { get; set; } = "localhost";
		/// <summary>
		/// The list of remote host names for zenserver
		/// </summary>
		public List<string> RemoteHostNames { get; } = [];
		/// <summary>
		/// The port for zenserver on the host
		/// </summary>
		public int HostPort { get; set; } = 8558;
		/// <summary>
		/// Authentication details for connecting to zenserver. Null if no authentication is required.
		/// Stored as an opaque JSON object that is passed through from the platform storage.
		/// </summary>
		public JsonElement? HostAuth { get; set; }
		/// <summary>
		/// The project identifier for the stored data in zenserver
		/// </summary>
		public string? ProjectId { get; set; }
		/// <summary>
		/// The oplog identifier for the stored data in zenserver
		/// </summary>
		public string? OplogId { get; set; }
		/// <summary>
		/// The zen workspace the build's share is available from
		/// </summary>
		public string? WorkspaceId { get; set; }
		/// <summary>
		/// The zen workspace share pointing to the build
		/// </summary>
		public string? ShareId { get; set; }
		/// <summary>
		/// Selected mode of operation. If missing, assume ZenOperatingMode.Stream
		/// </summary>
		public ZenOperatingMode? OperatingMode { get; set; } = ZenOperatingMode.Stream;
	}
	/// <summary>
	/// A desctiptor of a project store
	/// </summary>
	public class ProjectStoreData
	{
		/// <summary>
		/// The details for zenserver for this project store
		/// </summary>
		public ZenServerStoreData? ZenServer { get; set; }
	}

	/// <summary>
	/// Exception indicating errors when finding or importing snapshots.
	/// </summary>
	public class BuildIndexException : Exception
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public BuildIndexException() { }

		/// <summary>
		/// Constructor with message
		/// </summary>
		public BuildIndexException(string message) : base(message) { }

		/// <summary>
		/// Constructor with message and innerException
		/// </summary>
		public BuildIndexException(string message, Exception innerException) : base(message, innerException) { }
	};

	/// <summary>
	/// This is a bit of a hack to avoid depending on CommandUtils, etc from AutomationTool.
	/// 
	/// Calling apps (specifically UAT) may want to control process lifetimes for spawned processes in a way we can't
	/// know about here. So we allow them to run any commands we need to run using whatever process-creation function
	/// they want, by passing a delegate to the build index.
	/// </summary>
	/// <param name="app"></param>
	/// <param name="commandLine"></param>
	public delegate void RunAppDelegate(string app, string commandLine);

	/// <summary>
	/// Base class for all builds we could want to import, whether discovered via looking them up in the P:/ drive or
	/// discovered via the Unreal Cloud DDC builds API.
	/// </summary>
#pragma warning disable CA1724
	public abstract class Build
#pragma warning restore CA1724
	{
		/// <summary>
		/// Delegate used to run command-line applications
		/// </summary>
#pragma warning disable CA1051
		protected readonly RunAppDelegate RunApp;
#pragma warning restore CA1051

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="runApp">Delegate used to run command-line applications.</param>
		protected Build(RunAppDelegate runApp)
		{
			RunApp = runApp;
		}

		/// <summary>
		/// Performs snapshot import for the discovered descriptors into a specified project and cooked output directory.
		/// </summary>
		/// <param name="projectFile"></param>
		/// <param name="platformCookedDirectory"></param>
		/// <param name="forceImport"></param>
		/// <param name="asyncImport"></param>
		public abstract void Import(FileReference projectFile, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport);

		/// <summary>
		/// Performs snapshot import for the discovered descriptors into a specified project and cooked output directory,
		/// using a specified Unreal root/engine directory.
		/// </summary>
		/// <param name="projectFile"></param>
		/// <param name="rootDirectory"></param>
		/// <param name="engineDirectory"></param>
		/// <param name="platformCookedDirectory"></param>
		/// <param name="forceImport"></param>
		/// <param name="asyncImport"></param>
		public abstract void Import(FileReference projectFile, DirectoryReference rootDirectory, DirectoryReference engineDirectory, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport);

		/// <summary>
		/// Returns the changelist associated with this snapshot.
		/// </summary>
		public abstract int GetChangelist();

		private static string[] GetHostAddresses()
		{
			return [.. Dns.GetHostAddresses("", System.Net.Sockets.AddressFamily.InterNetwork).Select(addr => addr.ToString())];
		}

		/// <summary>
		/// JSON serialization options for writing to ue.projectstore and reading data from Unreal Cloud.
		/// </summary>
		public static JsonSerializerOptions GetDefaultJsonSerializerOptions()
		{
			JsonSerializerOptions options = new();
			options.AllowTrailingCommas = true;
			options.ReadCommentHandling = JsonCommentHandling.Skip;
			options.PropertyNameCaseInsensitive = true;
			options.PropertyNamingPolicy = new LowerCaseNamingPolicy();
			options.Converters.Add(new JsonStringEnumConverter());
			options.NumberHandling = JsonNumberHandling.AllowReadingFromString;
			return options;
		}

		private class LowerCaseNamingPolicy : JsonNamingPolicy
		{
			public override string ConvertName(string name) => name.ToLower();
		}

		/// <summary>
		/// Returns a project ID of the form {project}.{hash} where {hash} is the first eight
		/// bytes of the MD5 hash of the full project filename, lowercased.
		/// 
		/// This matches conventions used by other tooling such as ushell.
		/// </summary>
		/// <param name="projectFile">Full path to a .uproject file to generate an ID for.</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA5351", Justification = "Not using MD5 for cryptographically secure use case")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA1308", Justification = "We know the hashed strings are safe to lowercase using ToLowerInvariant()")]
		protected static string GetProjectID(FileReference projectFile)
		{
			byte[] bytes = Encoding.UTF8.GetBytes(projectFile.FullName.Replace("\\", "/", StringComparison.InvariantCulture));
			byte[] hashed = System.Security.Cryptography.MD5.HashData(bytes);

			string hexString = Convert.ToHexString(hashed).ToLowerInvariant();

			return projectFile.GetFileNameWithoutAnyExtensions() + "." + hexString.Substring(0, 8);
		}

		/// <summary>
		/// Creates a new project in zenserver if it doesn't exist, using the default Unreal root and engine directories.
		/// </summary>
		/// <param name="projectFile">Full path to a .uproject file to create a zenserver project for.</param>
		protected void CreateProject(FileReference projectFile)
		{
			CreateProject(projectFile, Unreal.RootDirectory, Unreal.EngineDirectory);
		}

		/// <summary>
		/// Creates a new project in zenserver if it doesn't exist.
		/// </summary>
		/// <param name="projectFile">Full path to a .uproject file to create a zenserver project for.</param>
		/// <param name="rootDirectory">Full path to the Unreal root directory.</param>
		/// <param name="engineDirectory">Full path to Engine/.</param>
		protected void CreateProject(FileReference projectFile, DirectoryReference rootDirectory, DirectoryReference engineDirectory)
		{
			string projectId = GetProjectID(projectFile);
			StringBuilder projectCreateCommandLine = new();
			projectCreateCommandLine.AppendFormat("project-create --force-update {0} {1} {2} {3} {4}", projectId, rootDirectory.FullName, engineDirectory.FullName, projectFile.Directory.FullName, projectFile.FullName);

			RunApp(GetZenExecutable(engineDirectory).FullName, projectCreateCommandLine.ToString());
		}

		/// <summary>
		/// Returns a FileReference to the published Zen executable in Engine/Binaries/...
		/// </summary>
		protected static FileReference GetZenExecutable(DirectoryReference engineDirectory)
		{
			string platform = (OperatingSystem.IsWindows() ? "Win64"
				: (OperatingSystem.IsMacOS() ? "Mac" : "Linux"));

			return new FileReference(Path.Combine(engineDirectory.FullName, "Binaries", platform, String.Format("zen{0}", RuntimePlatform.ExeExtension)));
		}

		/// <summary>
		/// Retrieves the Zen host auth data from the platform credential store as an opaque JSON element,
		/// matching the storage used by FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("Zen"), TEXT("HostAuth")).
		/// Returns null if no auth data is stored or if reading/parsing fails.
		/// </summary>
		private static JsonElement? GetZenHostAuth()
		{
			try
			{
				string? authJson = null;
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					using Microsoft.Win32.RegistryKey? key = Microsoft.Win32.Registry.CurrentUser.OpenSubKey(@"Software\Epic Games\Zen");
					authJson = key?.GetValue("HostAuth") as string;
				}
				else
				{
					// On Mac and Linux, FPlatformMisc::SetStoredValue writes to an INI file:
					//   ApplicationSettingsDir() / "Epic Games" / "KeyValueStore.ini"
					// with section [Zen] and key HostAuth=<value>
					// Mac ApplicationSettingsDir: ~/Library/Application Support/Epic/
					// Linux ApplicationSettingsDir: ~/.config/Epic/
					string appSettingsDir;
					if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
					{
						appSettingsDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
							"Library", "Application Support", "Epic");
					}
					else
					{
						appSettingsDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile),
							".config", "Epic");
					}

					string iniPath = Path.Combine(appSettingsDir, "Epic Games", "KeyValueStore.ini");
					if (File.Exists(iniPath))
					{
						authJson = ReadIniValue(iniPath, "Zen", "HostAuth");
					}
				}

				if (!String.IsNullOrEmpty(authJson))
				{
					using JsonDocument doc = JsonDocument.Parse(authJson);
					return doc.RootElement.Clone();
				}
			}
			catch
			{
				// Silently fail - auth is optional
			}
			return null;
		}

		/// <summary>
		/// Reads a value from a UE-style INI file given a section name and key name.
		/// Handles UE's quoted-string format: if the value is wrapped in double quotes,
		/// the quotes are stripped and escape sequences are processed to match the
		/// behavior of FParse::QuotedString / FConfigFile::ShouldExportQuotedString.
		/// </summary>
		private static string? ReadIniValue(string iniPath, string sectionName, string keyName)
		{
			bool inSection = false;
			string sectionHeader = $"[{sectionName}]";
			foreach (string rawLine in File.ReadLines(iniPath))
			{
				string line = rawLine.Trim();
				if (line.StartsWith('['))
				{
					inSection = String.Equals(line, sectionHeader, StringComparison.OrdinalIgnoreCase);
					continue;
				}
				if (inSection)
				{
					int eqIndex = line.IndexOf('=', StringComparison.Ordinal);
					if (eqIndex > 0 && String.Equals(line.Substring(0, eqIndex), keyName, StringComparison.OrdinalIgnoreCase))
					{
						string raw = line.Substring(eqIndex + 1);
						if (raw.Length >= 2 && raw[0] == '"' && raw[^1] == '"')
						{
							return UnescapeIniQuotedString(raw.AsSpan(1, raw.Length - 2));
						}
						return raw;
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Unescapes a UE INI quoted string value, matching FParse::QuotedString.
		/// </summary>
		private static string UnescapeIniQuotedString(ReadOnlySpan<char> value)
		{
			StringBuilder sb = new(value.Length);
			for (int i = 0; i < value.Length; i++)
			{
				if (value[i] != '\\' || i + 1 >= value.Length)
				{
					sb.Append(value[i]);
					continue;
				}
				char esc = value[++i];
				switch (esc)
				{
					case '\\': sb.Append('\\'); break;
					case '"':  sb.Append('"');  break;
					case '\'': sb.Append('\''); break;
					case 'n':  sb.Append('\n'); break;
					case 'r':  sb.Append('\r'); break;
					case 't':  sb.Append('\t'); break;
					default:   sb.Append('\\'); sb.Append(esc); break;
				}
			}
			return sb.ToString();
		}

		/// <summary>
		/// Writes a ue.projectstore file for the given project and platform to the platform cooked data directory.
		/// </summary>
		/// <param name="projectId">A project ID in zenserver, generated by GetProjectID().</param>
		/// <param name="platform">The platform this project contains cooked data for.</param>
		/// <param name="platformCookDirectory">The output directory for the ue.projectstore marker.</param>
		protected static FileReference WriteProjectStoreFile(string projectId, string platform, DirectoryReference platformCookDirectory)
		{
			if (!DirectoryReference.Exists(platformCookDirectory))
			{
				DirectoryReference.CreateDirectory(platformCookDirectory);
			}

			FileReference projectStoreFile = new(Path.Combine(platformCookDirectory.FullName, "ue.projectstore"));

			ProjectStoreData data = new();
			data.ZenServer = new ZenServerStoreData();
			data.ZenServer.IsLocalHost = true;
			data.ZenServer.HostName = "[::1]";
			data.ZenServer.HostPort = 8558;
			data.ZenServer.HostAuth = GetZenHostAuth();
			data.ZenServer.ProjectId = projectId;
			data.ZenServer.OplogId = platform;
			data.ZenServer.RemoteHostNames.AddRange(GetHostAddresses());

			string json = JsonSerializer.Serialize(data, GetDefaultJsonSerializerOptions());
			FileReference.WriteAllText(projectStoreFile, json);

			return projectStoreFile;
		}
	}

	/// <summary>
	/// Implementation of Build for builds discovered from snapshot descriptors on the network fileshare.
	/// </summary>
	public class FileshareBuild : Build
	{
		/// <summary>
		/// Enumeration of different storage options for snapshots.
		/// </summary>
		public enum SnapshotStorageType
		{
			/// <summary>
			/// A reserved non-valid storage type for snapshots.
			/// </summary>
			Invalid,
			/// <summary>
			/// Snapshot stored in cloud repositories such as Unreal Cloud DDC.
			/// </summary>
			Cloud,
			/// <summary>
			/// Snapshot stored in a zenserver.
			/// </summary>
			Zen,
			/// <summary>
			/// Snapshot stored as a file on disk.
			/// </summary>
			File,
		}

		/// <summary>
		/// Metadata about a snapshot
		/// </summary>
		class SnapshotDescriptor
		{
			/// <summary>
			/// Name of the snapshot
			/// </summary>
			public string Name { get; set; } = String.Empty;

			/// <summary>
			/// Storage type used for the snapshot
			/// </summary>
			public SnapshotStorageType Type { get; set; }

			/// <summary>
			/// Target platform for this snapshot
			/// </summary>
			public string TargetPlatform { get; set; } = String.Empty;

			/// <summary>
			/// For cloud or Zen snapshots, the host they are stored on.
			/// </summary>
			public string Host { get; set; } = String.Empty;

			/// <summary>
			/// For Zen snapshots, the project ID to import from.
			/// </summary>
			public string ProjectId { get; set; } = String.Empty;

			/// <summary>
			/// For Zen snapshots, the oplog ID to import from.
			/// </summary>
			public string OplogId { get; set; } = String.Empty;

			/// <summary>
			/// For cloud snapshots, the namespace they are stored in.
			/// </summary>
			public string Namespace { get; set; } = String.Empty;

			/// <summary>
			/// For cloud snapshots, the bucket they are stored in.
			/// </summary>
			public string Bucket { get; set; } = String.Empty;

			/// <summary>
			/// For cloud snapshots, the key they are stored in.
			/// </summary>
			public string Key { get; set; } = String.Empty;

			/// <summary>
			/// For file snapshots, the directory it is stored in.
			/// </summary>
			public string Directory { get; set; } = String.Empty;

			/// <summary>
			/// For file snapshots, the filename (not including path) that they are stored in.
			/// </summary>
			public string Filename { get; set; } = String.Empty;
		}

		/// <summary>
		/// A collection of one or more snapshot descriptors
		/// </summary>
		class SnapshotDescriptorCollection
		{
			/// <summary>
			/// The list of snapshots contained within this collection.
			/// </summary>
			public List<SnapshotDescriptor> Snapshots { get; set; } = [];
		}

		private readonly SnapshotDescriptorCollection _snapshotDescriptors = new();
		private readonly int _changelist;

		private void LoadSnapshotDescriptors(string snapshotDescriptorJSON)
		{
			SnapshotDescriptorCollection? jsonDescriptors = JsonSerializer.Deserialize<SnapshotDescriptorCollection>(snapshotDescriptorJSON, GetDefaultJsonSerializerOptions());
			if (jsonDescriptors == null || jsonDescriptors.Snapshots == null)
			{
				throw new BuildIndexException("Failed to deserialize snapshot descriptors");
			}

			_snapshotDescriptors.Snapshots.AddRange(jsonDescriptors.Snapshots);
		}

		/// <summary>
		/// Constructs a fileshare build using a JSON string
		/// </summary>
		/// <param name="snapshotDescriptorJSON"></param>
		/// <param name="changelist">The changelist associated with this build.</param>
		/// <param name="runApp">A delegate to use to run command-line apps.</param>
		public FileshareBuild(string snapshotDescriptorJSON, int changelist, RunAppDelegate runApp)
			: base(runApp)
		{
			_changelist = changelist;
			LoadSnapshotDescriptors(snapshotDescriptorJSON);
		}

		/// <summary>
		/// Constructs a fileshare build using a snapshot descriptor from a network fileshare.
		/// </summary>
		/// <param name="snapshotDescriptor"></param>
		/// <param name="changelist">The changelist associated with this build.</param>
		/// <param name="runApp">A delegate to use to run command-line apps.</param>
		public FileshareBuild(FileReference snapshotDescriptor, int changelist, RunAppDelegate runApp)
			: base(runApp)
		{
			_changelist = changelist;
			string data = FileReference.ReadAllText(snapshotDescriptor);
			LoadSnapshotDescriptors(data);
		}

		/// <inheritdoc/>
		public override void Import(FileReference projectFile, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport)
		{
			Import(projectFile, Unreal.RootDirectory, Unreal.EngineDirectory, platformCookedDirectory, forceImport, asyncImport);
		}

		/// <inheritdoc />
		public override void Import(FileReference projectFile, DirectoryReference rootDirectory, DirectoryReference engineDirectory, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport)
		{
			string projectId = GetProjectID(projectFile);
			CreateProject(projectFile, rootDirectory, engineDirectory);

			if (_snapshotDescriptors == null || _snapshotDescriptors.Snapshots == null)
			{
				throw new BuildIndexException("Failed to deserialize snapshot descriptors");
			}

			foreach (SnapshotDescriptor descriptor in _snapshotDescriptors.Snapshots)
			{
				platformCookedDirectory = new DirectoryReference(platformCookedDirectory.FullName.Replace("{Platform}", descriptor.TargetPlatform, StringComparison.Ordinal));
				FileReference projectStoreFile = WriteProjectStoreFile(projectId, descriptor.TargetPlatform, platformCookedDirectory);

				string oplogName = descriptor.TargetPlatform;

				StringBuilder oplogCreateCommandLine = new();
				oplogCreateCommandLine.AppendFormat("oplog-create --force-update {0} {1}", projectId, oplogName);

				StringBuilder oplogImportCommandLine = new();
				oplogImportCommandLine.AppendFormat("oplog-import {0} {1} {2} --clean", projectId, oplogName, projectStoreFile.FullName);

				if (forceImport)
				{
					oplogImportCommandLine.AppendFormat(" --force");
				}

				if (asyncImport)
				{
					oplogImportCommandLine.AppendFormat(" --async");
				}

				switch (descriptor.Type)
				{
					case SnapshotStorageType.Cloud:
						oplogImportCommandLine.AppendFormat(" --cloud {0}", descriptor.Host);
						oplogImportCommandLine.AppendFormat(" --namespace {0}", descriptor.Namespace);
						oplogImportCommandLine.AppendFormat(" --bucket {0}", descriptor.Bucket);
						oplogImportCommandLine.AppendFormat(" --key {0}", descriptor.Key);
						oplogImportCommandLine.AppendFormat(" --assume-http2");
						break;
					case SnapshotStorageType.Zen:
						oplogImportCommandLine.AppendFormat(" --zen {0}", descriptor.Host);
						oplogImportCommandLine.AppendFormat(" --source-project {0}", descriptor.ProjectId);
						oplogImportCommandLine.AppendFormat(" --source-oplog {0}", descriptor.OplogId);
						break;
					case SnapshotStorageType.File:
						oplogImportCommandLine.AppendFormat(" --file {0}", descriptor.Directory);
						oplogImportCommandLine.AppendFormat(" --name {0}", descriptor.Filename);
						break;
					default:
						break;
				}

				FileReference zenExe = GetZenExecutable(engineDirectory);
				RunApp(zenExe.FullName, oplogCreateCommandLine.ToString());
				RunApp(zenExe.FullName, oplogImportCommandLine.ToString());
			}
		}

		/// <inheritdoc/>
		public override int GetChangelist()
		{
			return _changelist;
		}
	}

	/// <summary>
	/// Implementation of Build for snapshots discovered via the Unreal Cloud DDC builds API.
	/// </summary>
	public class UnrealCloudDDCBuild : Build
	{
		private readonly UnrealCloudDDCBuildData _buildData;
		private readonly string _host;
		private readonly string _namespace;
		private readonly string _bucket;
		private readonly string _accessToken;

		/// <summary>
		/// Creates a build that can be used to import the given buildData from an Unreal Cloud host, namespace and bucket.
		/// </summary>
		/// <param name="buildData">Build data retrieved from Unreal Cloud build index's QueryBuilds.</param>
		/// <param name="host"></param>
		/// <param name="ns"></param>
		/// <param name="bucket"></param>
		/// <param name="accessToken">An OIDC token used for authorization.</param>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		public UnrealCloudDDCBuild(UnrealCloudDDCBuildData buildData, string host, string ns, string bucket, string accessToken, RunAppDelegate runApp)
			: base(runApp)
		{
			_buildData = buildData;
			_host = host;
			_namespace = ns;
			_bucket = bucket;
			_accessToken = accessToken;
		}

		/// <inheritdoc />
		public override void Import(FileReference projectFile, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport)
		{
			Import(projectFile, Unreal.RootDirectory, Unreal.EngineDirectory, platformCookedDirectory, forceImport, asyncImport);
		}

		/// <inheritdoc />
		public override void Import(FileReference projectFile, DirectoryReference rootDirectory, DirectoryReference engineDirectory, DirectoryReference platformCookedDirectory, bool forceImport, bool asyncImport)
		{
			string projectId = GetProjectID(projectFile);
			CreateProject(projectFile, rootDirectory, engineDirectory);

			platformCookedDirectory = new DirectoryReference(platformCookedDirectory.FullName.Replace("{Platform}", _buildData.Metadata.CookPlatform, StringComparison.Ordinal));
			FileReference projectStoreFile = WriteProjectStoreFile(projectId, _buildData.Metadata.CookPlatform, platformCookedDirectory);

			string oplogName = _buildData.Metadata.CookPlatform;

			StringBuilder oplogCreateCommandLine = new();
			oplogCreateCommandLine.AppendFormat("oplog-create --force-update {0} {1}", projectId, oplogName);

			StringBuilder oplogImportCommandLine = new();
			oplogImportCommandLine.AppendFormat("oplog-import {0} {1} {2} --clean", projectId, oplogName, projectStoreFile.FullName);

			if (forceImport)
			{
				oplogImportCommandLine.AppendFormat(" --force");
			}

			if (asyncImport)
			{
				oplogImportCommandLine.AppendFormat(" --async");
			}

			oplogImportCommandLine.AppendFormat(" --namespace {0}", _namespace);
			oplogImportCommandLine.AppendFormat(" --access-token {0}", _accessToken);
			oplogImportCommandLine.AppendFormat(" --bucket {0}", _bucket);
			oplogImportCommandLine.AppendFormat(" --builds {0}", _host);
			oplogImportCommandLine.AppendFormat(" --builds-id {0}", _buildData.BuildId);

			FileReference zenExe = GetZenExecutable(engineDirectory);
			RunApp(zenExe.FullName, oplogCreateCommandLine.ToString());
			RunApp(zenExe.FullName, oplogImportCommandLine.ToString());
		}

		/// <inheritdoc/>
		public override int GetChangelist()
		{
			return _buildData.Metadata.Changelist;
		}
	}

	/// <summary>
	/// Base class representing a location we can discover builds from, whether from the P:/ drive
	/// or from an Unreal Cloud DDC host's builds API.
	/// </summary>
	public abstract class BuildIndex
	{
		/// <summary>
		/// </summary>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		protected BuildIndex(RunAppDelegate runApp)
		{
			RunApp = runApp;
		}

		/// <summary>
		/// A delegate used to run command-line apps. See documentation of RunAppDelegate for details.
		/// </summary>
#pragma warning disable CA1051
		protected readonly RunAppDelegate RunApp;
#pragma warning restore CA1051

		/// <summary>
		/// Retrieves a list of changelists for a given branch/project/platform/runtime that can be imported
		/// from this build index.
		/// </summary>
		/// <param name="branch"></param>
		/// <param name="project"></param>
		/// <param name="platform"></param>
		/// <param name="runtime"></param>
		/// <returns></returns>
		public abstract List<int> GetAvailableChangelists(string branch, string project, string platform, string runtime);

		/// <summary>
		/// Returns a Build which can be used to import the referenced snapshot into zen.
		/// </summary>
		/// <param name="branch"></param>
		/// <param name="project"></param>
		/// <param name="platform"></param>
		/// <param name="runtime"></param>
		/// <param name="changelist"></param>
		/// <param name="getClosestPreceding"></param>
		/// <returns></returns>
		public abstract Build GetBuild(string branch, string project, string platform, string runtime, int changelist, bool getClosestPreceding);
	}

	/// <summary>
	/// Implementation of BuildIndex for builds discovered from a network drive.
	/// </summary>
	public class FileshareBuildIndex : BuildIndex
	{
		private readonly DirectoryReference _buildRoot;

		/// <summary>
		/// </summary>
		/// <param name="buildRoot">A directory tree including a set of directories, one per changelist with published snapshots. 
		/// Snapshot descriptors are .json files within the changelist directories. Optionally can be one level up from
		/// the changelist directories themselves, and include directories for multiple projects in the root directory.</param>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		public FileshareBuildIndex(DirectoryReference buildRoot, RunAppDelegate runApp)
			: base(runApp)
		{
			if (buildRoot == null || String.IsNullOrEmpty(buildRoot.FullName))
			{
				throw new BuildIndexException("Must provide a root directory to search for snapshots");
			}
			else
			{
				if (!DirectoryReference.Exists(buildRoot))
				{
					throw new BuildIndexException(String.Format("Unable to find build root {0}", buildRoot.FullName));
				}

				_buildRoot = buildRoot;
			}
		}

		private DirectoryReference GetBuildsDir(string branch, string project, string platform, string runtime)
		{
			string escapedBranch = branch.Replace("/", "+", StringComparison.InvariantCultureIgnoreCase);
			DirectoryReference buildsDir = DirectoryReference.Combine(_buildRoot, escapedBranch);
			DirectoryReference projectBuildsDir = DirectoryReference.Combine(buildsDir, project);

			if (DirectoryReference.Exists(projectBuildsDir))
			{
				buildsDir = projectBuildsDir;
			}

			TextInfo textInfo = new CultureInfo("en-US", false).TextInfo;

			string runtimeSuffix = (runtime == "game" ? "" : textInfo.ToTitleCase(runtime));

			if (platform.Equals("win64", StringComparison.OrdinalIgnoreCase))
			{
				buildsDir = DirectoryReference.Combine(buildsDir, "Windows" + runtimeSuffix);
			}
			else
			{
				buildsDir = DirectoryReference.Combine(buildsDir, textInfo.ToTitleCase(platform) + runtimeSuffix);
			}

			if (!DirectoryReference.Exists(buildsDir))
			{
				throw new BuildIndexException("Unable to find build index");
			}

			return buildsDir;
		}

		/// <inheritdoc />
		public override List<int> GetAvailableChangelists(string branch, string project, string platform, string runtime)
		{
			DirectoryReference buildsDir = GetBuildsDir(branch, project, platform, runtime);
			List<DirectoryReference> directories = [.. DirectoryReference.EnumerateDirectories(buildsDir)];
			List<int> changelists = [];

			foreach (DirectoryReference d in directories)
			{
				if (Int32.TryParse(d.GetDirectoryName(), out int changelist))
				{
					changelists.Add(changelist);
				}
			}

			return changelists;
		}

		/// <inheritdoc />
		public override Build GetBuild(string branch, string project, string platform, string runtime, int changelist, bool getClosestPreceding)
		{
			DirectoryReference buildsDir = GetBuildsDir(branch, project, platform, runtime);
			List<int> changelists = GetAvailableChangelists(branch, project, platform, runtime);
			int foundChangelistIndex = changelists.FindLastIndex(cl => cl < changelist);

			if (foundChangelistIndex == -1)
			{
				throw new BuildIndexException("Unable to find changelists in build index");
			}

			int foundChangelist = changelists[foundChangelistIndex];
			if (foundChangelist != changelist && !getClosestPreceding)
			{
				throw new BuildIndexException(String.Format("Unable to find exact match for changelist {0}", changelist));
			}

			DirectoryReference buildDir = DirectoryReference.Combine(buildsDir, changelist.ToString());
			return new FileshareBuild(DirectoryReference.EnumerateFiles(buildDir).First(), foundChangelist, RunApp);
		}
	}

	/// <summary>
	/// Metadata associated with an Unreal Cloud DDC snapshot discovered via the builds API.
	/// </summary>
	public class UnrealCloudDDCBuildMetadata
	{
		/// <summary>
		/// The name of the published snapshot.
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The branch of the published snapshot.
		/// </summary>
		public string Branch { get; set; } = String.Empty;

		/// <summary>
		/// The baseline branch of the published snapshot.
		/// </summary>
		public string BaselineBranch { get; set; } = String.Empty;

		/// <summary>
		/// The platform the snapshot is cooked for.
		/// </summary>
		public string Platform { get; set; } = String.Empty;

		/// <summary>
		/// The cookPlatform the snapshot is cooked for (includes details such as cook flavor).
		/// </summary>
		public string CookPlatform { get; set; } = String.Empty;

		/// <summary>
		/// The name of the project the snapshot includes.
		/// </summary>
		public string Project { get; set; } = String.Empty;

		/// <summary>
		/// The changelist the snapshot was cooked at.
		/// </summary>
		public int Changelist { get; set; }
	}

	/// <summary>
	/// Data associated with an Unreal Cloud DDC snapshot discovered via the builds API.
	/// </summary>
	public class UnrealCloudDDCBuildData
	{
		/// <summary>
		/// The builds ID used to retrieve the snapshot's data.
		/// </summary>
		public string BuildId { get; set; } = String.Empty;

		/// <summary>
		/// The build metadata retrieved from Unreal Cloud.
		/// </summary>
		public UnrealCloudDDCBuildMetadata Metadata { get; set; } = new UnrealCloudDDCBuildMetadata();
	}

	/// <summary>
	/// Implementation of BuildIndex for builds discovered from the Unreal Cloud DDC builds API.
	/// </summary>
	public class UnrealCloudDDCBuildIndex : BuildIndex
	{
		private readonly string _host;
		private readonly string _namespace;
		private readonly string _accessToken;

		/// <summary>
		/// Constructs an Unreal Cloud build index that will search the given Unreal Cloud host and namespace.
		/// Uses OidcToken.exe to perform authorization with the host.
		/// </summary>
		/// <param name="host">The Unreal Cloud host to search.</param>
		/// <param name="ns">The Unreal Cloud namespace to search.</param>
		/// <param name="cloudAuthServiceName">The service name used to obtain cloud authorization tokens.</param>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		public UnrealCloudDDCBuildIndex(string host, string ns, string cloudAuthServiceName, RunAppDelegate runApp)
			: base(runApp)
		{
			_host = host;
			_namespace = ns;
			_accessToken = GetAccessToken(cloudAuthServiceName);
		}

		/// <summary>
		/// Creates an Unreal Cloud build index that will search the given Unreal Cloud host and namespace,
		/// using an existing OidcTokenInfo from OidcTokenManager.
		/// </summary>
		/// <param name="host">The Unreal CLoud host to search.</param>
		/// <param name="ns">The Unreal Cloud namespace to search.</param>
		/// <param name="accessTokenInfo"></param>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		public UnrealCloudDDCBuildIndex(string host, string ns, IOidcTokenInfo accessTokenInfo, RunAppDelegate runApp)
			: base(runApp)
		{
			_host = host;
			_namespace = ns;
			_accessToken = accessTokenInfo.AccessToken!;
		}

		/// <summary>
		/// Creates an Unreal Cloud build index asynchronously using remote auth configuration provided by a given host.
		/// </summary>
		/// <param name="host">The Unreal CLoud host to search.</param>
		/// <param name="ns">The Unreal Cloud namespace to search.</param>
		/// <param name="runApp">A delegate used to run command-line apps.</param>
		/// <param name="allowInteractiveLogin">Whether we should allow an interactive login or raise NotLoggedInException.</param>
		/// <returns>A task that can be awaited to get the authorized Unreal Cloud build index.</returns>
		/// <exception cref="BuildIndexException">Raised if we fail to determine the auth provider from the host.</exception>
		/// <exception cref="NotLoggedInException">Raised if we require interactive login and allowInteractiveLogin is false..</exception>
		public static async Task<UnrealCloudDDCBuildIndex> CreateBuildIndexAsync(string host, string ns, RunAppDelegate runApp, bool allowInteractiveLogin = false)
		{
			ClientAuthConfigurationV1? authConfig = await ProviderConfigurationFactory.ReadRemoteAuthConfigurationAsync(new Uri(host), ProviderConfigurationFactory.DefaultEncryptionKey);
			if (authConfig?.DefaultProvider == null)
			{
				throw new BuildIndexException(String.Format("Failed to determine default auth provider from {0}", host));
			}

			IConfiguration config = ProviderConfigurationFactory.BindOptions(authConfig);
			using (ITokenStore tokenStore = TokenStoreFactory.CreateTokenStore())
			{
				OidcTokenManager tokenManager = OidcTokenManager.CreateTokenManager(config, tokenStore);

				try
				{
					IOidcTokenInfo tokenInfo = await tokenManager.GetAccessToken(authConfig.DefaultProvider);
					return new UnrealCloudDDCBuildIndex(host, ns, tokenInfo, runApp);
				}
				catch (NotLoggedInException)
				{
					if (!allowInteractiveLogin)
					{
						throw;
					}

					IOidcTokenInfo tokenInfo = await tokenManager.LoginAsync(authConfig.DefaultProvider);
					return new UnrealCloudDDCBuildIndex(host, ns, tokenInfo, runApp);
				}
			}
		}

		class AccessTokenOutput
		{
			public string? Token { get; set; }
			public string? ExpiresAt { get; set; }
		}

		private string GetAccessToken(string cloudAuthServiceName)
		{
			string binaryType = "win-x64";
			if (OperatingSystem.IsMacOS())
			{
				binaryType = "osx-x64";
			}
			else if (OperatingSystem.IsLinux())
			{
				binaryType = "linux-x64";
			}

			DirectoryReference oidcDir = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Binaries", "DotNET", "OidcToken", binaryType);
			FileReference oidcExe = new(Path.Combine(oidcDir.FullName, "OidcToken" + RuntimePlatform.ExeExtension));
			FileReference outputFile = new(Path.Combine(Path.GetTempPath(), Path.GetTempFileName()));

			StringBuilder commandLine = new();

			commandLine.AppendFormat(" --Service={0} --OutFile={1}", cloudAuthServiceName, outputFile.FullName);
			RunApp(oidcExe.FullName, commandLine.ToString());

			string oidcOutput = FileReference.ReadAllText(outputFile);
			AccessTokenOutput? token = JsonSerializer.Deserialize<AccessTokenOutput>(oidcOutput);
			if (token == null || token.Token == null)
			{
				throw new BuildIndexException("Failed to parse OidcToken output");
			}

			return token.Token;
		}

		class QueryBuildsResponse
		{
			public List<UnrealCloudDDCBuildData> Results { get; set; } = [];
			public bool PartialResult { get; set; }
		}

		class QueryBucketsResponse
		{
			public List<string> Buckets { get; set; } = [];
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA1308", Justification = "We know the provided strings are safe to lowercase using ToLowerInvariant()")]
		private static string GetBucket(string project, string branch, string platform, string runtime)
		{
			project = project.ToLowerInvariant();
			branch = StringId.Sanitize(branch).ToString().Replace(".", "-", StringComparison.Ordinal);
			platform = platform.ToLowerInvariant();
			runtime = runtime.ToLowerInvariant();

			if (platform == "win64")
			{
				platform = "windows";
			}

			if (runtime != "game")
			{
				platform += runtime;
			}

			if (platform == "windowsserver")
			{
				platform = "linuxserver";
			}

			return String.Format("{0}.oplog.{1}.{2}", project, branch, platform);
		}

		/// <summary>
		/// Gets a list of the available platforms this build index can find for a given branch and project.
		/// </summary>
		/// <param name="branch">The branch to search for.</param>
		/// <param name="project">The project to search for.</param>
		/// <returns></returns>
		/// <exception cref="BuildIndexException">Raised on request failure or unexpected response format.</exception>
		public List<string> GetAvailablePlatforms(string branch, string project)
		{
			Uri uri = new(String.Format("{0}/api/v2/builds/{1}", _host, _namespace));

			HttpResponseMessage response;
			using (HttpClient client = new())
			{
				using (HttpRequestMessage request = new())
				{
					request.Method = HttpMethod.Get;
					request.RequestUri = uri;
					request.Headers.Authorization = new AuthenticationHeaderValue("Bearer", _accessToken);

					response = client.Send(request);
				}
			}

			string body;
			using (StreamReader reader = new(response.Content.ReadAsStream()))
			{
				body = reader.ReadToEnd();
			}

			if (!response.IsSuccessStatusCode)
			{
				throw new BuildIndexException(String.Format("Failed to query builds: {0} ({1})", response.ReasonPhrase, body));
			}

			QueryBucketsResponse? buckets = JsonSerializer.Deserialize<QueryBucketsResponse>(body, Build.GetDefaultJsonSerializerOptions());
			if (buckets == null)
			{
				throw new BuildIndexException("Failed to deserialize bucket info returned from Unreal Cloud");
			}

			branch = StringId.Sanitize(branch).ToString();
			// Bucket format for snapshot buckets is <project>.oplog.<branch>.<platform>.
			// Find buckets matching the format, with the specified project and branch,
			// then return the associated platform.
			return [.. buckets.Buckets
				.Select(b => b.Split("."))
				.Where(b =>
					b.Length == 4
					&& b[0].Equals(project, StringComparison.OrdinalIgnoreCase)
					&& b[1].Equals("oplog", StringComparison.OrdinalIgnoreCase)
					&& b[2].Equals(branch, StringComparison.OrdinalIgnoreCase))
				.Select(b => b[3].ToLower())];
		}

		private List<UnrealCloudDDCBuildData> QueryBuilds(string branch, string project, string platform, string runtime, int changelist = 0)
		{
			string bucket = GetBucket(project, branch, platform, runtime);
			Uri uri = new(String.Format("{0}/api/v2/builds/{1}/{2}/search", _host, _namespace, bucket));

			HttpResponseMessage response;
			using (HttpClient client = new())
			{
				using (HttpRequestMessage request = new())
				{
					request.RequestUri = uri;
					request.Method = HttpMethod.Post;

					if (changelist > 0)
					{
						request.Content = new StringContent(String.Format("{{\"query\": {{\"isPreflight\": {{\"$eq\": false}}, \"runtime\": {{\"$eq\": \"{0}\"}}, \"changelist\": {{\"$lte\": {1}}}, \"branch\": {{\"$eq\": \"{2}\"}}}}}}", runtime, changelist, branch), MediaTypeHeaderValue.Parse("application/json"));
					}
					else
					{
						request.Content = new StringContent(String.Format("{{\"query\": {{\"isPreflight\": {{\"$eq\": false}}, \"runtime\": {{\"$eq\": \"{0}\"}}, \"branch\": {{\"$eq\": \"{1}\"}}}}}}", runtime, branch), MediaTypeHeaderValue.Parse("application/json"));
					}

					request.Headers.Authorization = new AuthenticationHeaderValue("Bearer", _accessToken);

					response = client.Send(request);
				}
			}

			string body;
			using (StreamReader reader = new(response.Content.ReadAsStream()))
			{
				body = reader.ReadToEnd();
			}

			if (!response.IsSuccessStatusCode)
			{
				throw new BuildIndexException(String.Format("Failed to query builds: {0} ({1})", response.ReasonPhrase, body));
			}

			QueryBuildsResponse? builds = JsonSerializer.Deserialize<QueryBuildsResponse>(body, Build.GetDefaultJsonSerializerOptions());
			if (builds == null)
			{
				throw new BuildIndexException("Failed to deserialize build info returned from Unreal Cloud DDC");
			}

			return builds.Results;
		}

		/// <inheritdoc />
		public override List<int> GetAvailableChangelists(string branch, string project, string platform, string runtime)
		{
			return [.. QueryBuilds(branch, project, platform, runtime).Select(b => b.Metadata.Changelist)];
		}

		/// <inheritdoc />
		public override Build GetBuild(string branch, string project, string platform, string runtime, int changelist, bool getClosestPreceding)
		{
			List<UnrealCloudDDCBuildData> builds = QueryBuilds(branch, project, platform, runtime, changelist);

			if (builds.Count == 0)
			{
				throw new BuildIndexException("No snapshots found");
			}

			builds.SortBy(b => b.Metadata.Changelist);

			if (!getClosestPreceding && builds.Last().Metadata.Changelist != changelist)
			{
				throw new BuildIndexException("Exact match for target changelist requested, but no snapshots matching that changelist were found");
			}

			return new UnrealCloudDDCBuild(builds.Last(), _host, _namespace, GetBucket(project, branch, platform, runtime), _accessToken, RunApp);
		}
	}
}
