<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Redis\EpicGames.Redis.csproj" />
  </ItemGroup>
</Project>
