// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using EpicGames.Core;
using EpicGames.Redis.Converters;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using ProtoBuf;
using StackExchange.Redis;

namespace EpicGames.Redis.Tests
{
	[TestClass]
	public class SerializationTests
	{
		record class TestRecord(int Number, string String, Utf8String StringU8);

		[TestMethod]
		public void RoundTrip()
		{
			TestRecord input = new(123, "hello", new Utf8String("world"));

			RedisValue value = RedisSerializer.Serialize(input);

			Assert.AreEqual("123|hello|world", (string?)value);

			TestRecord output = RedisSerializer.Deserialize<TestRecord>(value);
			Assert.AreEqual(123, output.Number);
			Assert.AreEqual("hello", output.String);
			Assert.AreEqual(new Utf8String("world"), output.StringU8);
		}

		[TestMethod]
		public void EscapedCharacters()
		{
			TestRecord input = new(123, "|||", new Utf8String("\\"));

			RedisValue value = RedisSerializer.Serialize(input);

			Assert.AreEqual("123|\\|\\|\\||\\\\", (string?)value);

			TestRecord output = RedisSerializer.Deserialize<TestRecord>(value);
			Assert.AreEqual(123, output.Number);
			Assert.AreEqual("|||", output.String);
			Assert.AreEqual(new Utf8String("\\"), output.StringU8);
		}

		[TypeConverter(typeof(TestStringTypeConverter))]
		class TestStringType
		{
			public int Number { get; init; }
		}

		class TestStringTypeConverter : TypeConverter
		{
			public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

			public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new TestStringType { Number = 123 };

			public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

			public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType) => "hello";
		}

		record class TestStringRecord(int Number, TestStringType Instance);

		[TestMethod]
		public void CustomStringConverter()
		{
			TestStringRecord input = new(123, new TestStringType { Number = 0 });

			RedisClassConverter<TestStringRecord> converter = new();
			RedisValue value = converter.ToRedisValue(input);

			Assert.AreEqual("123|hello", (string?)value);

			TestStringRecord output = converter.FromRedisValue(value);
			Assert.AreEqual(123, output.Number);
			Assert.AreEqual(123, output.Instance.Number);
		}

		[TypeConverter(typeof(TestUtf8StringTypeConverter))]
		class TestUtf8StringType
		{
			public int Number { get; init; }
		}

		class TestUtf8StringTypeConverter : TypeConverter
		{
			public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(Utf8String);

			public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new TestUtf8StringType { Number = 123 };

			public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(Utf8String);

			public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType) => new Utf8String("hello");
		}

		record class TestUtf8StringRecord(int Number, TestUtf8StringType Instance);

		[TestMethod]
		public void CustomUtf8StringConverter()
		{
			TestUtf8StringRecord input = new(123, new TestUtf8StringType { Number = 0 });

			RedisClassConverter<TestUtf8StringRecord> converter = new();
			RedisValue value = converter.ToRedisValue(input);

			Assert.AreEqual("123|hello", (string?)value);

			TestUtf8StringRecord output = converter.FromRedisValue(value);
			Assert.AreEqual(123, output.Number);
			Assert.AreEqual(123, output.Instance.Number);
		}

		[ProtoContract]
		class ProtoClass
		{
			[ProtoMember(1)]
			public string Text { get; set; } = null!;
		}

		[TestMethod]
		public void ProtobufTest()
		{
			ProtoClass cls = new();
			cls.Text = "hello world";

			RedisValue value = RedisSerializer.Serialize(cls);

			ProtoClass clsPb = Serializer.Deserialize<ProtoClass>(((byte[]?)value).AsMemory());
			Assert.AreEqual("hello world", clsPb.Text);

			ProtoClass clsRedis = RedisSerializer.Deserialize<ProtoClass>(value);
			Assert.AreEqual("hello world", clsRedis.Text);
		}
	}
}
