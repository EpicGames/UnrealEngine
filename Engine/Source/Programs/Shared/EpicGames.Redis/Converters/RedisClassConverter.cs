// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using EpicGames.Core;
using StackExchange.Redis;

namespace EpicGames.Redis.Converters
{
	/// <summary>
	/// Converter for records to Redis values.
	/// </summary>
	/// <typeparam name="T">The record type</typeparam>
	public class RedisClassConverter<T> : IRedisConverter<T>
	{
		readonly PropertyInfo[] _properties;
		readonly Func<Utf8String, object?>[] _typeReaders; // Utf8String -> object
		readonly Action<object?, Utf8StringBuilder>[] _typeWriters;
		readonly ConstructorInfo _constructor;

		/// <summary>
		/// Constructor
		/// </summary>
		public RedisClassConverter()
		{
			Type type = typeof(T);
			_properties = type.GetProperties(BindingFlags.Public | BindingFlags.Instance);

			_typeReaders = new Func<Utf8String, object?>[_properties.Length];
			_typeWriters = new Action<object?, Utf8StringBuilder>[_properties.Length];

			for (int idx = 0; idx < _properties.Length; idx++)
			{
				Type propertyType = _properties[idx].PropertyType;
				if (propertyType == typeof(Utf8String))
				{
					_typeReaders[idx] = str => str;
					_typeWriters[idx] = (obj, builder) => WriteEscapedString((Utf8String)obj!, builder);
				}
				else if (TryGetUtf8StringTypeConverter(propertyType, out TypeConverter? converter))
				{
					_typeReaders[idx] = str => converter.ConvertFrom(str);
					_typeWriters[idx] = (obj, builder) => WriteEscapedString((Utf8String)converter.ConvertTo(obj, typeof(Utf8String))!, builder);
				}
				else if (TryGetStringTypeConverter(propertyType, out converter))
				{
					_typeReaders[idx] = str => converter.ConvertFromInvariantString(str.ToString());
					_typeWriters[idx] = (obj, builder) => WriteEscapedString(new Utf8String(converter.ConvertToInvariantString(obj) ?? String.Empty), builder);
				}
				else
				{
					throw new InvalidOperationException($"Unable to create converter to/from strings for {propertyType.Name}");
				}
			}

			ConstructorInfo? constructor = type.GetConstructor(BindingFlags.Instance | BindingFlags.Public, _properties.ConvertAll(x => x.PropertyType));
			if (constructor == null)
			{
				throw new InvalidOperationException("No constructor found with same arguments as properties. Try using a record type.");
			}
			_constructor = constructor;
		}

		static bool TryGetUtf8StringTypeConverter(Type propertyType, [NotNullWhen(true)] out TypeConverter? converter)
		{
			converter = TypeDescriptor.GetConverter(propertyType);
			return converter != null && converter.CanConvertFrom(typeof(Utf8String)) && converter.CanConvertTo(typeof(Utf8String));
		}

		static bool TryGetStringTypeConverter(Type propertyType, [NotNullWhen(true)] out TypeConverter? converter)
		{
			converter = TypeDescriptor.GetConverter(propertyType);
			return converter != null && converter.CanConvertFrom(typeof(string)) && converter.CanConvertTo(typeof(string));
		}

		static Utf8String ReadEscapedString(byte[] data, ref int idx)
		{
			int startIdx = idx;
			int endIdx = startIdx;
			int outIdx = startIdx;

			for (; endIdx < data.Length; endIdx++, outIdx++)
			{
				if (data[endIdx] == '|')
				{
					endIdx++;
					break;
				}
				else if (data[endIdx] == '\\')
				{
					for (; endIdx < data.Length; endIdx++, outIdx++)
					{
						if (data[endIdx] == '|')
						{
							endIdx++;
							break;
						}
						if (data[endIdx] == '\\')
						{
							endIdx++;
						}
						data[outIdx] = data[endIdx];
					}
					break;
				}
			}

			idx = endIdx;
			return new Utf8String(data.AsMemory(startIdx, outIdx - startIdx));
		}

		static void WriteEscapedString(Utf8String str, Utf8StringBuilder builder)
		{
			if (str.Length > 0)
			{
				int minIdx = 0;
				for (int idx = 0; idx < str.Length; idx++)
				{
					if (str[idx] == '\\' || str[idx] == '|')
					{
						builder.Append(str.Substring(minIdx, idx - minIdx));
						builder.Append((byte)'\\');
						minIdx = idx;
					}
				}
				builder.Append(str.Substring(minIdx));
			}
		}

		void AppendProperty(T value, int propertyIdx, Utf8StringBuilder builder)
		{
			object? arg = _properties[propertyIdx].GetValue(value);
			_typeWriters[propertyIdx](arg, builder);
		}

		/// <inheritdoc/>
		public RedisValue ToRedisValue(T value)
		{
			Utf8StringBuilder builder = new();
			if (_properties.Length > 0)
			{
				AppendProperty(value, 0, builder);
				for (int idx = 1; idx < _properties.Length; idx++)
				{
					builder.Append((byte)'|');
					AppendProperty(value, idx, builder);
				}
			}
			return builder.ToString();
		}

		/// <inheritdoc/>
		public T FromRedisValue(RedisValue value) => FromBytes((byte[]?)value);

		T FromBytes(byte[]? data)
		{
			if (data == null)
			{
				return default!;
			}

			object?[] arguments = new object?[_properties.Length];
			int argumentIdx = 0;

			for (int idx = 0; idx < data.Length;)
			{
				Utf8String str = ReadEscapedString(data, ref idx);
				arguments[argumentIdx] = _typeReaders[argumentIdx].Invoke(str);
				argumentIdx++;
			}

			return (T)_constructor.Invoke(arguments);
		}
	}
}
