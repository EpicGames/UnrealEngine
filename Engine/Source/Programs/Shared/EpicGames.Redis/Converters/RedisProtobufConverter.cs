// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;
using Google.Protobuf;
using Google.Protobuf.Reflection;
using ProtoBuf;
using StackExchange.Redis;

namespace EpicGames.Redis.Converters
{
	/// <summary>
	/// Converter for records to Redis values using protobuf serialization.
	/// </summary>
	/// <typeparam name="T">The record type</typeparam>
	public class RedisProtobufConverter<T> : IRedisConverter<T> where T : IMessage<T>, new()
	{
		readonly MessageDescriptor _descriptor;

		/// <summary>
		/// Constructor
		/// </summary>
		public RedisProtobufConverter()
		{
			_descriptor = new T().Descriptor;
		}

		/// <inheritdoc/>
		public T FromRedisValue(RedisValue value)
		{
			return (T)_descriptor.Parser.ParseFrom((byte[])value!);
		}

		/// <inheritdoc/>
		public RedisValue ToRedisValue(T value)
		{
			return value.ToByteArray();
		}
	}

	/// <summary>
	/// Converter for records to Redis values using ProtoBuf-Net annotations.
	/// </summary>
	/// <typeparam name="T">The record type</typeparam>
	public class RedisProtobufNetConverter<T> : IRedisConverter<T>
	{
		/// <inheritdoc/>
		public RedisValue ToRedisValue(T value)
		{
			using (MemoryStream stream = new())
			{
				Serializer.Serialize(stream, value);
				return stream.ToArray();
			}
		}

		/// <inheritdoc/>
		public T FromRedisValue(RedisValue value)
		{
			using (ReadOnlyMemoryStream stream = new(value))
			{
				T result = Serializer.Deserialize<T>(stream);
				return result;
			}
		}
	}
}
