<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Google.Protobuf" Version="3.25.8" />
    <PackageReference Include="protobuf-net" Version="3.2.56" />
    <PackageReference Include="StackExchange.Redis" Version="2.8.58" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>
</Project>
