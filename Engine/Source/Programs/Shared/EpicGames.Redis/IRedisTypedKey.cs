// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Interface for typed redis keys
	/// </summary>
	public interface IRedisTypedKey
	{
		/// <summary>
		/// The inner untyped key
		/// </summary>
		public RedisKey Inner { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IRedisTypedKey"/>
	/// </summary>
	public static class RedisTypedKeyExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.KeyExists(RedisKey)"/>
		public static Condition KeyExists(this IRedisTypedKey key)
			=> Condition.KeyExists(key.Inner);

		/// <inheritdoc cref="Condition.KeyExists(RedisKey)"/>
		public static Condition KeyNotExists(this IRedisTypedKey key)
			=> Condition.KeyNotExists(key.Inner);

		#endregion

		#region KeyDeleteAsync

		/// <inheritdoc cref="IDatabaseAsync.KeyDeleteAsync(RedisKey, CommandFlags)"/>
		public static Task KeyDeleteAsync(this IDatabaseAsync target, IRedisTypedKey key, CommandFlags flags = CommandFlags.None)
			=> target.KeyDeleteAsync(key.Inner, flags);

		/// <inheritdoc cref="IDatabaseAsync.KeyDeleteAsync(RedisKey[], CommandFlags)"/>
		public static Task KeyDeleteAsync(this IDatabaseAsync target, IRedisTypedKey[] keys, CommandFlags flags = CommandFlags.None)
			=> target.KeyDeleteAsync(keys, flags);

		#endregion

		#region KeyExistsAsync

		/// <inheritdoc cref="IDatabaseAsync.KeyExistsAsync(RedisKey, CommandFlags)"/>
		public static Task<bool> KeyExistsAsync(this IDatabaseAsync target, IRedisTypedKey key, CommandFlags flags = CommandFlags.None)
			=> target.KeyExistsAsync(key.Inner, flags);

		/// <inheritdoc cref="IDatabaseAsync.KeyExistsAsync(RedisKey[], CommandFlags)"/>
		public static Task<long> KeyExistsAsync(this IDatabaseAsync target, IEnumerable<IRedisTypedKey> keys, CommandFlags flags = CommandFlags.None)
			=> target.KeyExistsAsync(keys.ToArray(), flags);

		#endregion

		#region KeyExpireAsync

		/// <inheritdoc cref="IDatabaseAsync.KeyExpireAsync(RedisKey, DateTime?, ExpireWhen, CommandFlags)"/>
		public static Task KeyExpireAsync(this IDatabaseAsync target, IRedisTypedKey key, DateTime? expiry = null, ExpireWhen when = ExpireWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.KeyExpireAsync(key.Inner, expiry, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.KeyExpireAsync(RedisKey, TimeSpan?, ExpireWhen, CommandFlags)"/>
		public static Task KeyExpireAsync(this IDatabaseAsync target, IRedisTypedKey key, TimeSpan? expiry, ExpireWhen when = ExpireWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.KeyExpireAsync(key.Inner, expiry, when, flags);

		#endregion
	}
}
