// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using EpicGames.Core;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed pub/sub channel with a particular value
	/// </summary>
	/// <typeparam name="T">The type of element stored in the channel</typeparam>
	public readonly struct RedisChannel<T> : IEquatable<RedisChannel<T>>
	{
		/// <summary>
		/// The key for the list
		/// </summary>
		public readonly RedisChannel Channel { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="channel"></param>
		public RedisChannel(RedisChannel channel) => Channel = channel;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="value">The channel name</param>
		/// <param name="mode">Pattern mode</param>
		public RedisChannel(Utf8String value, RedisChannel.PatternMode mode) => Channel = new RedisChannel(value.Memory.ToArray(), mode);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="value">The channel name</param>
		/// <param name="mode">Pattern mode</param>
		public RedisChannel(string value, RedisChannel.PatternMode mode) => Channel = new RedisChannel(value, mode);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="value">The channel name</param>
		/// <param name="mode">Pattern mode</param>
		public RedisChannel(byte[] value, RedisChannel.PatternMode mode) => Channel = new RedisChannel(value, mode);

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is RedisChannel<T> list && Channel == list.Channel;

		/// <inheritdoc/>
		public bool Equals(RedisChannel<T> other) => Channel == other.Channel;

		/// <inheritdoc/>
		public override int GetHashCode() => Channel.GetHashCode();

		/// <summary>Compares two instances for equality</summary>
		public static bool operator ==(RedisChannel<T> left, RedisChannel<T> right) => left.Channel == right.Channel;

		/// <summary>Compares two instances for equality</summary>
		public static bool operator !=(RedisChannel<T> left, RedisChannel<T> right) => left.Channel != right.Channel;

		/// <summary>
		/// Create a channel name from a <see cref="Utf8String"/>.
		/// </summary>
		/// <param name="key">The string to get a channel from.</param>
		public static implicit operator RedisChannel<T>(Utf8String key) => new(key, RedisChannel.PatternMode.Auto);

		/// <summary>
		/// Create a channel name from a byte array.
		/// </summary>
		/// <param name="key">The byte array to get a channel from.</param>
		public static implicit operator RedisChannel<T>(string key) => new(key, RedisChannel.PatternMode.Auto);

		/// <summary>
		/// Create a channel name from a byte array.
		/// </summary>
		/// <param name="key">The byte array to get a channel from.</param>
		public static implicit operator RedisChannel<T>(byte[] key) => new(key, RedisChannel.PatternMode.Auto);
	}

	/// <summary>
	/// Extension methods for channels
	/// </summary>
	public static class RedisChannelExtensions
	{
		/// <inheritdoc cref="IDatabaseAsync.PublishAsync(RedisChannel, RedisValue, CommandFlags)"/>
		public static Task PublishAsync<T>(this IDatabaseAsync database, RedisChannel<T> channel, T item, CommandFlags flags = CommandFlags.None)
		{
			RedisValue value = RedisSerializer.Serialize(item);
			return database.PublishAsync(channel.Channel, value, flags);
		}
	}
}
