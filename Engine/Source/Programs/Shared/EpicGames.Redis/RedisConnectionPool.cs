// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// A connection pool for Redis client
	///
	/// Wraps multiple ConnectionMultiplexers as lazy values and initializes them as needed.
	/// If full, the least loaded connection will be picked.
	/// </summary>
	public sealed class RedisConnectionPool : IDisposable
	{
		private readonly Lazy<IConnectionMultiplexer>[] _connections;
		private readonly int _defaultDatabaseIndex;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="poolSize">Size of the pool (i.e max number of connections)</param>
		/// <param name="redisConfString">Configuration string for Redis</param>
		/// <param name="defaultDatabaseIndex">The Redis database index to use. Use -1 for the default one</param>
		public RedisConnectionPool(int poolSize, string redisConfString, int defaultDatabaseIndex = -1)
		{
			_defaultDatabaseIndex = defaultDatabaseIndex;
			_connections = new Lazy<IConnectionMultiplexer>[poolSize];
			for (int i = 0; i < poolSize; i++)
			{
				static void ConfigureOptions(ConfigurationOptions options)
				{
					if (Debugger.IsAttached)
					{
						options.SyncTimeout = 1_000_000;
					}
				}

				_connections[i] = new Lazy<IConnectionMultiplexer>(() => ConnectionMultiplexer.Connect(redisConfString, ConfigureOptions));
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			for (int idx = 0; idx < _connections.Length; idx++)
			{
				if (_connections[idx].IsValueCreated)
				{
					_connections[idx].Value.Dispose();
				}
			}
		}

		/// <summary>
		/// Get a connection from the pool
		///
		/// It will pick the least loaded connection or create a new one (if pool size allows)
		/// </summary>
		/// <returns>A Redis database connection</returns>
		public IConnectionMultiplexer GetConnection()
		{
			long existingConnectionLoad = 0;
			IConnectionMultiplexer? existingConnection = null;
			Lazy<IConnectionMultiplexer>? newConnection = null;

			// Find the least loaded connection
			foreach (Lazy<IConnectionMultiplexer> connection in _connections)
			{
				if (connection.IsValueCreated)
				{
					ServerCounters counters = connection.Value.GetCounters();
					if (existingConnection == null || counters.TotalOutstanding < existingConnectionLoad)
					{
						existingConnection = connection.Value;
						existingConnectionLoad = counters.TotalOutstanding;
					}
				}
				else
				{
					newConnection ??= connection;
				}
			}

			// Check if we should try to create a new connection
			if (existingConnection == null || existingConnectionLoad >= 10)
			{
				if (newConnection != null)
				{
					return newConnection.Value;
				}
			}

			// Otherwise return the best connection we already have
			Debug.Assert(existingConnection != null);
			return existingConnection;
		}

		/// <summary>
		/// Shortcut to getting a IDatabase
		/// </summary>
		/// <returns>A Redis database</returns>
		public IDatabase GetDatabase()
		{
			return GetConnection().GetDatabase(_defaultDatabaseIndex);
		}
	}
}