// Copyright Epic Games, Inc. All Rights Reserved.

using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Extension methods for Redis
	/// </summary>
	public static class RedisExtensions
	{
		/// <summary>
		/// Convert a RedisValue to a RedisKey
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		public static RedisKey AsKey(this RedisValue value)
		{
			return (byte[])value!;
		}

		/// <summary>
		/// Convert a RedisKey to a RedisValue
		/// </summary>
		/// <param name="key"></param>
		/// <returns></returns>
		public static RedisValue AsValue(this RedisKey key)
		{
			return (byte[])key!;
		}
	}
}
