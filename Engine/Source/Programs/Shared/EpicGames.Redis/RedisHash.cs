// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq.Expressions;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Redis hash, with members corresponding to the property names of a type
	/// </summary>
	public record struct RedisHash<T>(IDatabaseAsync Database, RedisHashKey<T> Key);

	/// <summary>
	/// Redis hash, with members of the given key and value types
	/// </summary>
	public record struct RedisHash<TName, TValue>(IDatabaseAsync Database, RedisHashKey<TName, TValue> Key);

	/// <summary>
	/// Extension methods for hashes
	/// </summary>
	public static class RedisHashExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashEqual<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector, TValue value)
			=> target.Key.HashEqual(selector, value);

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashEqual<TName, TValue>(this RedisHash<TName, TValue> target, TName name, TValue value)
			=> target.Key.HashEqual(name, value);

		/// <inheritdoc cref="Condition.HashExists(RedisKey, RedisValue)"/>
		public static Condition HashExists<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector)
			=> target.Key.HashExists(selector);

		/// <inheritdoc cref="Condition.HashExists(RedisKey, RedisValue)"/>
		public static Condition HashExists<TName, TValue>(this RedisHash<TName, TValue> target, TName name)
			=> target.Key.HashExists(name);

		/// <inheritdoc cref="Condition.HashLengthEqual(RedisKey, Int64)"/>
		public static Condition HashLengthEqual<TName, TValue>(this RedisHash<TName, TValue> target, long length)
			=> target.Key.HashLengthEqual(length);

		/// <inheritdoc cref="Condition.HashLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition HashLengthGreaterThan<TName, TValue>(this RedisHash<TName, TValue> target, long length)
			=> target.Key.HashLengthGreaterThan(length);

		/// <inheritdoc cref="Condition.HashLengthLessThan(RedisKey, Int64)"/>
		public static Condition HashLengthLessThan<TName, TValue>(this RedisHash<TName, TValue> target, long length)
			=> target.Key.HashLengthLessThan(length);

		/// <inheritdoc cref="Condition.HashNotExists(RedisKey, RedisValue)"/>
		public static Condition HashNotExists<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector)
			=> target.Key.HashNotExists(selector);

		/// <inheritdoc cref="Condition.HashNotExists(RedisKey, RedisValue)"/>
		public static Condition HashNotExists<TName, TValue>(this RedisHash<TName, TValue> target, TName name)
			=> target.Key.HashNotExists(name);

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashNotEqual<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector, TValue value)
			=> target.Key.HashNotEqual(selector, value);

		/// <inheritdoc cref="Condition.HashNotEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashNotEqual<TName, TValue>(this RedisHash<TName, TValue> target, TName name, TValue value)
			=> target.Key.HashNotEqual(name, value);

		#endregion

		#region HashDecrementAsync

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> DecrementAsync<TRecord>(this RedisHash<TRecord> target, Expression<Func<TRecord, long>> selector, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDecrementAsync(target.Key, selector, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> DecrementAsync<TRecord>(this RedisHash<TRecord> target, Expression<Func<TRecord, double>> selector, double value = 1.0, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDecrementAsync(target.Key, selector, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> DecrementAsync<TName>(this RedisHash<TName, long> target, TName name, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDecrementAsync(target.Key, name, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> DecrementAsync<TName>(this RedisHash<TName, double> target, TName name, double value = 1.0, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDecrementAsync(target.Key, name, value, flags);

		#endregion

		#region HashDeleteAsync

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> DeleteAsync<TRecord>(this RedisHash<TRecord> target, Expression<Func<TRecord, double>> selector, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDeleteAsync(target.Key, selector, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> DeleteAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName name, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDeleteAsync(target.Key, name, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> DeleteAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName[] names, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashDeleteAsync(target.Key, names, flags);

		#endregion

		#region HashExistsAsync

		/// <inheritdoc cref="IDatabaseAsync.HashExistsAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> ExistsAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName name, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashExistsAsync(target.Key, name, flags);

		#endregion

		#region HashGetAsync

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<TValue> GetAsync<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashGetAsync(target.Key, selector, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<TValue> GetAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName name, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashGetAsync(target.Key, name, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<TValue[]> GetAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName[] names, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashGetAsync(target.Key, names, flags);

		#endregion

		#region HashGetAllAsync

		/// <inheritdoc cref="IDatabaseAsync.HashGetAllAsync(RedisKey, CommandFlags)"/>
		public static Task<HashEntry<TName, TValue>[]> GetAllAsync<TName, TValue>(this RedisHash<TName, TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashGetAllAsync(target.Key, flags);

		#endregion

		#region HashIncrementAsync

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> IncrementAsync<TRecord>(this RedisHash<TRecord> target, Expression<Func<TRecord, long>> selector, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashIncrementAsync(target.Key, selector, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> IncrementAsync<TRecord>(this RedisHash<TRecord> target, Expression<Func<TRecord, double>> selector, double value = 1.0, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashIncrementAsync(target.Key, selector, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<long> IncrementAsync<TName>(this RedisHash<TName, long> target, TName name, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashIncrementAsync(target.Key, name, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> IncrementAsync<TName>(this RedisHash<TName, double> target, TName name, double value = 1.0, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashIncrementAsync(target.Key, name, value, flags);

		#endregion

		#region HashKeysAsync

		/// <inheritdoc cref="IDatabaseAsync.HashLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<TName[]> KeysAsync<TName, TValue>(this RedisHash<TName, TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashKeysAsync(target.Key, flags);

		#endregion

		#region HashLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.HashLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> LengthAsync<TName, TValue>(this RedisHash<TName, TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashLengthAsync(target.Key, flags);

		#endregion

		#region HashScanAsync

		/// <inheritdoc cref="IDatabaseAsync.HashScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static IAsyncEnumerable<HashEntry<TName, TValue>> ScanAsync<TName, TValue>(this RedisHash<TName, TValue> target, RedisValue pattern, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashScanAsync(target.Key, pattern, pageSize, cursor, pageOffset, flags);

		#endregion

		#region HashSetAsync

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, RedisValue, RedisValue, When, CommandFlags)"/>
		public static Task SetAsync<TRecord, TValue>(this RedisHash<TRecord> target, Expression<Func<TRecord, TValue>> selector, TValue value, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashSetAsync(target.Key, selector, value, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, RedisValue, RedisValue, When, CommandFlags)"/>
		public static Task SetAsync<TName, TValue>(this RedisHash<TName, TValue> target, TName name, TValue value, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashSetAsync(target.Key, name, value, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, HashEntry[], CommandFlags)"/>
		public static Task SetAsync<TName, TValue>(this RedisHash<TName, TValue> target, IEnumerable<HashEntry<TName, TValue>> entries, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashSetAsync(target.Key, entries, flags);

		#endregion

		#region HashValuesAsync

		/// <inheritdoc cref="IDatabaseAsync.HashValuesAsync(RedisKey, CommandFlags)"/>
		public static Task<TValue[]> ValuesAsync<TName, TValue>(this RedisHash<TName, TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.HashValuesAsync(target.Key, flags);

		#endregion
	}
}
