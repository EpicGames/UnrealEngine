// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Linq.Expressions;
using System.Reflection;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a redis hash key, with members corresponding to the property names of a type
	/// </summary>
	/// <typeparam name="T">Type of the hash fields</typeparam>
	public record struct RedisHashKey<T>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisHashKey<T>(string key) => new(new RedisKey(key));
	}

	/// <summary>
	/// Represents a typed Redis hash with given key/value types
	/// </summary>
	/// <typeparam name="TName">Type of the hash key</typeparam>
	/// <typeparam name="TValue">Type of the hash value</typeparam>
	public record struct RedisHashKey<TName, TValue>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisHashKey<TName, TValue>(string key) => new(new RedisKey(key));
	}

	/// <inheritdoc cref="HashEntry"/>
	public readonly struct HashEntry<TName, TValue>
	{
		/// <inheritdoc cref="HashEntry.Name"/>
		public readonly TName Name { get; }

		/// <inheritdoc cref="HashEntry.Value"/>
		public readonly TValue Value { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public HashEntry(TName name, TValue value)
		{
			Name = name;
			Value = value;
		}

		/// <summary>
		/// Deconstructor helper method
		/// </summary>
		public void Deconstruct(out TName name, out TValue value)
		{
			name = Name;
			value = Value;
		}

		/// <summary>
		/// Implicit conversion to a <see cref="HashEntry"/>
		/// </summary>
		/// <param name="entry"></param>
		public static implicit operator HashEntry(HashEntry<TName, TValue> entry)
		{
			return new HashEntry(RedisSerializer.Serialize(entry.Name), RedisSerializer.Serialize(entry.Value));
		}

		/// <summary>
		/// Implicit conversion to a <see cref="KeyValuePair{TName, TValue}"/>
		/// </summary>
		/// <param name="entry"></param>
		public static implicit operator KeyValuePair<TName, TValue>(HashEntry<TName, TValue> entry)
		{
			return new KeyValuePair<TName, TValue>(entry.Name, entry.Value);
		}
	}

	/// <summary>
	/// Extension methods for hashes
	/// </summary>
	public static class RedisHashKeyExtensions
	{
		/// <summary>
		/// Helper method to convert an array of hash entries into a dictionary
		/// </summary>
		public static async Task<Dictionary<TName, TValue>> ToDictionaryAsync<TName, TValue>(this Task<HashEntry<TName, TValue>[]> entries) where TName : notnull
			=> (await entries).ToDictionary(x => x.Name, x => x.Value);

		/// <summary>
		/// Helper method to convert an array of hash entries into a dictionary
		/// </summary>
		public static async Task<Dictionary<TName, TValue>> ToDictionaryAsync<TName, TValue>(this Task<HashEntry<TName, TValue>[]> entries, IEqualityComparer<TName>? comparer) where TName : notnull
			=> (await entries).ToDictionary(x => x.Name, x => x.Value, comparer);

		#region Conditions

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashEqual<TRecord, TValue>(this RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector, TValue value)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return Condition.HashEqual(key.Inner, memberExpression.Member.Name, RedisSerializer.Serialize(value));
		}

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashEqual<TName, TValue>(this RedisHashKey<TName, TValue> key, TName name, TValue value)
			=> Condition.HashEqual(key.Inner, RedisSerializer.Serialize(name), RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.HashExists(RedisKey, RedisValue)"/>
		public static Condition HashExists<TRecord, TValue>(this RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return Condition.HashExists(key.Inner, memberExpression.Member.Name);
		}

		/// <inheritdoc cref="Condition.HashExists(RedisKey, RedisValue)"/>
		public static Condition HashExists<TName, TValue>(this RedisHashKey<TName, TValue> key, TName name)
			=> Condition.HashExists(key.Inner, RedisSerializer.Serialize(name));

		/// <inheritdoc cref="Condition.HashLengthEqual(RedisKey, Int64)"/>
		public static Condition HashLengthEqual<TName, TValue>(this RedisHashKey<TName, TValue> key, long length)
			=> Condition.HashLengthEqual(key.Inner, length);

		/// <inheritdoc cref="Condition.HashLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition HashLengthGreaterThan<TName, TValue>(this RedisHashKey<TName, TValue> key, long length)
			=> Condition.HashLengthGreaterThan(key.Inner, length);

		/// <inheritdoc cref="Condition.HashLengthLessThan(RedisKey, Int64)"/>
		public static Condition HashLengthLessThan<TName, TValue>(this RedisHashKey<TName, TValue> key, long length)
			=> Condition.HashLengthLessThan(key.Inner, length);

		/// <inheritdoc cref="Condition.HashNotExists(RedisKey, RedisValue)"/>
		public static Condition HashNotExists<TRecord, TValue>(this RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return Condition.HashNotExists(key.Inner, memberExpression.Member.Name);
		}

		/// <inheritdoc cref="Condition.HashNotExists(RedisKey, RedisValue)"/>
		public static Condition HashNotExists<TName, TValue>(this RedisHashKey<TName, TValue> key, TName name)
			=> Condition.HashNotExists(key.Inner, RedisSerializer.Serialize(name));

		/// <inheritdoc cref="Condition.HashEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashNotEqual<TRecord, TValue>(this RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector, TValue value)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return Condition.HashNotEqual(key.Inner, memberExpression.Member.Name, RedisSerializer.Serialize(value));
		}

		/// <inheritdoc cref="Condition.HashNotEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition HashNotEqual<TName, TValue>(this RedisHashKey<TName, TValue> key, TName name, TValue value)
			=> Condition.HashNotEqual(key.Inner, RedisSerializer.Serialize(name), RedisSerializer.Serialize(value));

		#endregion

		#region HashDecrementAsync

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> HashDecrementAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, long>> selector, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashDecrementAsync(key.Inner, memberExpression.Member.Name, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> HashDecrementAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, double>> selector, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashDecrementAsync(key.Inner, memberExpression.Member.Name, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> HashDecrementAsync<TName>(this IDatabaseAsync target, RedisHashKey<TName, long> key, TName name, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.HashDecrementAsync(key.Inner, RedisSerializer.Serialize<TName>(name), value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashDecrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> HashDecrementAsync<TName>(this IDatabaseAsync target, RedisHashKey<TName, double> key, TName name, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			return target.HashDecrementAsync(key.Inner, RedisSerializer.Serialize<TName>(name), value, flags);
		}

		#endregion

		#region HashDeleteAsync

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> HashDeleteAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, double>> selector, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashDeleteAsync(key.Inner, memberExpression.Member.Name, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> HashDeleteAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName name, CommandFlags flags = CommandFlags.None)
		{
			return target.HashDeleteAsync(key.Inner, RedisSerializer.Serialize(name), flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashDeleteAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> HashDeleteAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName[] names, CommandFlags flags = CommandFlags.None)
		{
			return target.HashDeleteAsync(key.Inner, RedisSerializer.Serialize(names), flags);
		}

		#endregion

		#region HashExistsAsync

		/// <inheritdoc cref="IDatabaseAsync.HashExistsAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> HashExistsAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName name, CommandFlags flags = CommandFlags.None)
		{
			return target.HashExistsAsync(key.Inner, RedisSerializer.Serialize(name), flags);
		}

		#endregion

		#region HashGetAsync

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static async Task<TRecord> HashGetAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, object>>[] selectors, CommandFlags flags = CommandFlags.None) where TRecord : new()
		{
			RedisValue[] names = new RedisValue[selectors.Length];
			for (int idx = 0; idx < selectors.Length; idx++)
			{
				Expression expr = selectors[idx].Body;
				if (expr is UnaryExpression unaryExpr && unaryExpr.NodeType == ExpressionType.Convert)
				{
					expr = unaryExpr.Operand;
				}

				MemberExpression memberExpression = (expr as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
				names[idx] = memberExpression.Member.Name;
			}

			RedisValue[] values = await target.HashGetAsync(key.Inner, RedisSerializer.Serialize(names), flags);

			HashRecordInfo<TRecord> recordInfo = HashRecordInfo<TRecord>.Instance;

			TRecord record = new();
			for (int idx = 0; idx < selectors.Length; idx++)
			{
				recordInfo.SetProperty(record, names[idx], values[idx]);
			}

			return record;
		}

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<TValue> HashGetAsync<TRecord, TValue>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashGetAsync(key.Inner, memberExpression.Member.Name, flags).DeserializeAsync<TValue>();
		}

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<TValue> HashGetAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName name, CommandFlags flags = CommandFlags.None)
		{
			return target.HashGetAsync(key.Inner, RedisSerializer.Serialize(name), flags).DeserializeAsync<TValue>();
		}

		/// <inheritdoc cref="IDatabaseAsync.HashGetAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<TValue[]> HashGetAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName[] names, CommandFlags flags = CommandFlags.None)
		{
			return target.HashGetAsync(key.Inner, RedisSerializer.Serialize(names), flags).DeserializeAsync<TValue>();
		}

		#endregion

		#region HashGetAllAsync

		/// <inheritdoc cref="IDatabaseAsync.HashGetAllAsync(RedisKey, CommandFlags)"/>
		public static async Task<HashEntry<TName, TValue>[]> HashGetAllAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, CommandFlags flags = CommandFlags.None)
		{
			HashEntry[] entries = await target.HashGetAllAsync(key.Inner, flags);
			return Array.ConvertAll(entries, x => new HashEntry<TName, TValue>(RedisSerializer.Deserialize<TName>(x.Name)!, RedisSerializer.Deserialize<TValue>(x.Value)!));
		}

		/// <inheritdoc cref="IDatabaseAsync.HashGetAllAsync(RedisKey, CommandFlags)"/>
		public static async Task<TRecord> HashGetAllAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, CommandFlags flags = CommandFlags.None) where TRecord : new()
		{
			HashRecordInfo<TRecord> recordInfo = HashRecordInfo<TRecord>.Instance;

			TRecord value = new();

			HashEntry[] entries = await target.HashGetAllAsync(key.Inner, flags);
			foreach (HashEntry entry in entries)
			{
				recordInfo.SetProperty(value, entry.Name, entry.Value);
			}

			return value;
		}

		#endregion

		#region HashIncrementAsync

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> HashIncrementAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, long>> selector, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashIncrementAsync(key.Inner, memberExpression.Member.Name, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> HashIncrementAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, double>> selector, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashIncrementAsync(key.Inner, memberExpression.Member.Name, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> HashIncrementAsync<TName>(this IDatabaseAsync target, RedisHashKey<TName, long> key, TName name, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.HashIncrementAsync(key.Inner, RedisSerializer.Serialize<TName>(name), value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashIncrementAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<double> HashIncrementAsync<TName>(this IDatabaseAsync target, RedisHashKey<TName, double> key, TName name, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			return target.HashIncrementAsync(key.Inner, RedisSerializer.Serialize<TName>(name), value, flags);
		}

		#endregion

		#region HashKeysAsync

		/// <inheritdoc cref="IDatabaseAsync.HashKeysAsync(RedisKey, CommandFlags)"/>
		public static Task<TName[]> HashKeysAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, CommandFlags flags = CommandFlags.None)
		{
			return target.HashKeysAsync(key.Inner, flags).DeserializeAsync<TName>();
		}

		#endregion

		#region HashLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.HashLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> HashLengthAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, CommandFlags flags = CommandFlags.None)
		{
			return target.HashLengthAsync(key, flags);
		}

		#endregion

		#region HashScanAsync

		/// <inheritdoc cref="IDatabaseAsync.HashScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static async IAsyncEnumerable<HashEntry<TName, TValue>> HashScanAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, RedisValue pattern, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
		{
			await foreach (HashEntry entry in target.HashScanAsync(key.Inner, pattern, pageSize, cursor, pageOffset, flags))
			{
				yield return new HashEntry<TName, TValue>(RedisSerializer.Deserialize<TName>(entry.Name), RedisSerializer.Deserialize<TValue>(entry.Value));
			}
		}

		#endregion

		#region HashSetAsync

		class HashRecordInfo<T>
		{
			public static HashRecordInfo<T> Instance { get; } = new HashRecordInfo<T>();

			public IReadOnlyList<HashPropertyInfo<T>> Properties { get; }

			public IReadOnlyDictionary<string, HashPropertyInfo<T>> PropertiesByName { get; }

			public HashRecordInfo()
			{
				List<HashPropertyInfo<T>> properties = [];
				foreach (PropertyInfo propertyInfo in typeof(T).GetProperties(BindingFlags.Public | BindingFlags.Instance))
				{
					properties.Add(new HashPropertyInfo<T>(propertyInfo));
				}
				Properties = properties;
				PropertiesByName = properties.ToDictionary(x => x.Name, x => x);
			}

			public bool SetProperty(T record, RedisValue name, RedisValue value)
			{
				string? nameStr = (string?)name;
				if (nameStr != null)
				{
					HashPropertyInfo<T>? propertyInfo;
					if (PropertiesByName.TryGetValue(nameStr, out propertyInfo))
					{
						propertyInfo.SetValue(record, value);
						return true;
					}
				}
				return false;
			}
		}

		record class HashPropertyInfo<T>(string Name, Func<T, RedisValue> GetValue, Action<T, RedisValue> SetValue)
		{
			public HashPropertyInfo(PropertyInfo propertyInfo)
				: this(
					propertyInfo.Name,
					record => RedisSerializer.Serialize(propertyInfo.GetValue(record), propertyInfo.PropertyType),
					(record, value) => propertyInfo.SetValue(record, RedisSerializer.Deserialize(value, propertyInfo.PropertyType))
				)
			{ }
		}

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, RedisValue, RedisValue, When, CommandFlags)"/>
		public static Task<bool> HashSetAsync<TRecord, TValue>(this IDatabaseAsync target, RedisHashKey<TRecord> key, Expression<Func<TRecord, TValue>> selector, TValue value, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			MemberExpression memberExpression = (selector.Body as MemberExpression) ?? throw new InvalidOperationException("Expression must be a property accessor");
			return target.HashSetAsync(key.Inner, memberExpression.Member.Name, RedisSerializer.Serialize(value), when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, HashEntry[], CommandFlags)"/>
		public static Task HashSetAsync<TRecord>(this IDatabaseAsync target, RedisHashKey<TRecord> key, TRecord value, CommandFlags flags = CommandFlags.None)
		{
			HashRecordInfo<TRecord> recordInfo = HashRecordInfo<TRecord>.Instance;

			HashEntry[] entries = new HashEntry[recordInfo.Properties.Count];
			for (int idx = 0; idx < recordInfo.Properties.Count; idx++)
			{
				HashPropertyInfo<TRecord> property = recordInfo.Properties[idx];
				entries[idx] = new HashEntry(property.Name, property.GetValue(value));
			}

			return target.HashSetAsync(key.Inner, entries, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, RedisValue, RedisValue, When, CommandFlags)"/>
		public static Task<bool> HashSetAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, TName name, TValue value, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.HashSetAsync(key.Inner, RedisSerializer.Serialize(name), RedisSerializer.Serialize(value), when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.HashSetAsync(RedisKey, HashEntry[], CommandFlags)"/>
		public static Task HashSetAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, IEnumerable<HashEntry<TName, TValue>> entries, CommandFlags flags = CommandFlags.None)
		{
			return target.HashSetAsync(key.Inner, [.. entries.Select(x => (HashEntry)x)], flags);
		}

		#endregion

		#region HashValuesAsync

		/// <inheritdoc cref="IDatabaseAsync.HashValuesAsync(RedisKey, CommandFlags)"/>
		public static Task<TValue[]> HashValuesAsync<TName, TValue>(this IDatabaseAsync target, RedisHashKey<TName, TValue> key, CommandFlags flags = CommandFlags.None)
		{
			return target.HashValuesAsync(key.Inner, flags).DeserializeAsync<TValue>();
		}

		#endregion
	}
}
