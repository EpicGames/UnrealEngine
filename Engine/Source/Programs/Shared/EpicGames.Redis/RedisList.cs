// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis list with a given key
	/// </summary>
	public record struct RedisList<TElement>(IDatabaseAsync Database, RedisListKey<TElement> Key);

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisListExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.ListIndexEqual(RedisKey, Int64, RedisValue)"/>
		public static Condition ListIndexEqual<TElement>(this RedisList<TElement> target, long index, TElement value)
			=> target.Key.ListIndexEqual(index, value);

		/// <inheritdoc cref="Condition.ListIndexExists(RedisKey, Int64)"/>
		public static Condition ListIndexExists<TElement>(this RedisList<TElement> target, long index)
			=> target.Key.ListIndexExists(index);

		/// <inheritdoc cref="Condition.ListIndexNotEqual(RedisKey, Int64, RedisValue)"/>
		public static Condition ListIndexNotEqual<TElement>(this RedisList<TElement> target, long index, TElement value)
			=> target.Key.ListIndexNotEqual(index, value);

		/// <inheritdoc cref="Condition.ListIndexNotExists(RedisKey, Int64)"/>
		public static Condition ListIndexNotExists<TElement>(this RedisList<TElement> target, long index)
			=> target.Key.ListIndexNotExists(index);

		/// <inheritdoc cref="Condition.ListLengthEqual(RedisKey, Int64)"/>
		public static Condition ListLengthEqual<TElement>(this RedisList<TElement> target, long length)
			=> target.Key.ListLengthEqual(length);

		/// <inheritdoc cref="Condition.ListLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition ListLengthGreaterThan<TElement>(this RedisList<TElement> target, long length)
			=> target.Key.ListLengthGreaterThan(length);

		/// <inheritdoc cref="Condition.ListLengthLessThan(RedisKey, Int64)"/>
		public static Condition ListLengthLessThan<TElement>(this RedisList<TElement> target, long length)
			=> target.Key.ListLengthLessThan(length);

		#endregion

		#region ListGetByIndexAsync

		/// <inheritdoc cref="IDatabaseAsync.ListGetByIndexAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<TElement> GetByIndexAsync<TElement>(this RedisList<TElement> target, long index, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListGetByIndexAsync(target.Key, index, flags);

		#endregion

		#region ListInsertAfterAsync

		/// <inheritdoc cref="IDatabaseAsync.ListInsertAfterAsync(RedisKey, RedisValue, RedisValue, CommandFlags)"/>
		public static Task<long> InsertAfterAsync<TElement>(this RedisList<TElement> target, TElement pivot, TElement item, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListInsertAfterAsync(target.Key, pivot, item, flags);

		#endregion

		#region ListInsertBeforeAsync

		/// <inheritdoc cref="IDatabaseAsync.ListInsertBeforeAsync(RedisKey, RedisValue, RedisValue, CommandFlags)"/>
		public static Task<long> InsertBeforeAsync<TElement>(this RedisList<TElement> target, TElement pivot, TElement item, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListInsertBeforeAsync(target.Key, pivot, item, flags);

		#endregion

		#region ListLeftPopAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> LeftPopAsync<TElement>(this RedisList<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListLeftPopAsync(target.Key, flags);

		#endregion

		#region ListLeftPushAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPushAsync(RedisKey, RedisValue, When, CommandFlags)"/>
		public static Task<long> LeftPushAsync<TElement>(this RedisList<TElement> target, TElement item, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListLeftPushAsync(target.Key, item, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPushAsync(RedisKey, RedisValue[], When, CommandFlags)"/>
		public static Task<long> LeftPushAsync<TElement>(this RedisList<TElement> target, TElement[] values, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListLeftPushAsync(target.Key, values, when, flags);

		#endregion

		#region ListLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> LengthAsync<TElement>(this RedisList<TElement> target)
			=> target.Database.ListLengthAsync(target.Key);

		#endregion

		#region ListRangeAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRangeAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> RangeAsync<TElement>(this RedisList<TElement> target, long start = 0, long stop = -1, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListRangeAsync(target.Key, start, stop, flags);

		#endregion

		#region ListRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRemoveAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> RemoveAsync<TElement>(this RedisList<TElement> target, TElement value, long count = 0L, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListRemoveAsync(target.Key, value, count, flags);

		#endregion

		#region ListRightPopAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRightPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> RightPopAsync<TElement>(this RedisList<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListRightPopAsync(target.Key, flags);

		#endregion

		#region ListRightPushAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRightPushAsync(RedisKey, RedisValue, When, CommandFlags)"/>
		public static Task<long> RightPushAsync<TElement>(this RedisList<TElement> target, TElement item, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListRightPushAsync(target.Key, item, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.ListRightPushAsync(RedisKey, RedisValue[], When, CommandFlags)"/>
		public static Task<long> RightPushAsync<TElement>(this RedisList<TElement> target, IEnumerable<TElement> values, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListRightPushAsync(target.Key, values, when, flags);

		#endregion

		#region ListSetByIndexAsync

		/// <inheritdoc cref="IDatabaseAsync.ListSetByIndexAsync(RedisKey, Int64, RedisValue, CommandFlags)"/>
		public static Task SetByIndexAsync<TElement>(this RedisList<TElement> target, long index, TElement value, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListSetByIndexAsync(target.Key, index, value, flags);

		#endregion

		#region ListTrimAsync

		/// <inheritdoc cref="IDatabaseAsync.ListTrimAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task TrimAsync<TElement>(this RedisList<TElement> target, long start, long stop, CommandFlags flags = CommandFlags.None)
			=> target.Database.ListTrimAsync(target.Key, start, stop, flags);

		#endregion
	}
}
