// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis list with a given key
	/// </summary>
	/// <typeparam name="TElement">The type of element stored in the set</typeparam>
	public record struct RedisListKey<TElement>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisListKey<TElement>(string key) => new(new RedisKey(key));
	}

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisListKeyExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.ListIndexEqual(RedisKey, Int64, RedisValue)"/>
		public static Condition ListIndexEqual<TElement>(this RedisListKey<TElement> key, long index, TElement value)
			=> Condition.ListIndexEqual(key.Inner, index, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.ListIndexExists(RedisKey, Int64)"/>
		public static Condition ListIndexExists<TElement>(this RedisListKey<TElement> key, long index)
			=> Condition.ListIndexExists(key.Inner, index);

		/// <inheritdoc cref="Condition.ListIndexNotEqual(RedisKey, Int64, RedisValue)"/>
		public static Condition ListIndexNotEqual<TElement>(this RedisListKey<TElement> key, long index, TElement value)
			=> Condition.ListIndexNotEqual(key.Inner, index, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.ListIndexNotExists(RedisKey, Int64)"/>
		public static Condition ListIndexNotExists<TElement>(this RedisListKey<TElement> key, long index)
			=> Condition.ListIndexNotExists(key.Inner, index);

		/// <inheritdoc cref="Condition.ListLengthEqual(RedisKey, Int64)"/>
		public static Condition ListLengthEqual<TElement>(this RedisListKey<TElement> key, long length)
			=> Condition.ListLengthEqual(key.Inner, length);

		/// <inheritdoc cref="Condition.ListLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition ListLengthGreaterThan<TElement>(this RedisListKey<TElement> key, long length)
			=> Condition.ListLengthGreaterThan(key.Inner, length);

		/// <inheritdoc cref="Condition.ListLengthLessThan(RedisKey, Int64)"/>
		public static Condition ListLengthLessThan<TElement>(this RedisListKey<TElement> key, long length)
			=> Condition.ListLengthLessThan(key.Inner, length);

		#endregion

		#region ListGetByIndexAsync

		/// <inheritdoc cref="IDatabaseAsync.ListGetByIndexAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<TElement> ListGetByIndexAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, long index, CommandFlags flags = CommandFlags.None)
		{
			return target.ListGetByIndexAsync(key.Inner, index, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region ListInsertAfterAsync

		/// <inheritdoc cref="IDatabaseAsync.ListInsertAfterAsync(RedisKey, RedisValue, RedisValue, CommandFlags)"/>
		public static Task<long> ListInsertAfterAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement pivot, TElement item, CommandFlags flags = CommandFlags.None)
		{
			return target.ListInsertAfterAsync(key.Inner, RedisSerializer.Serialize(pivot), RedisSerializer.Serialize(item), flags);
		}

		#endregion

		#region ListInsertBeforeAsync

		/// <inheritdoc cref="IDatabaseAsync.ListInsertBeforeAsync(RedisKey, RedisValue, RedisValue, CommandFlags)"/>
		public static Task<long> ListInsertBeforeAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement pivot, TElement item, CommandFlags flags = CommandFlags.None)
		{
			RedisValue pivotValue = RedisSerializer.Serialize(pivot);
			RedisValue itemValue = RedisSerializer.Serialize(item);
			return target.ListInsertBeforeAsync(key.Inner, pivotValue, itemValue, flags);
		}

		#endregion

		#region ListLeftPopAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> ListLeftPopAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.ListLeftPopAsync(key.Inner, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region ListLeftPushAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPushAsync(RedisKey, RedisValue, When, CommandFlags)"/>
		public static Task<long> ListLeftPushAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement item, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.ListLeftPushAsync(key.Inner, RedisSerializer.Serialize(item), when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.ListLeftPushAsync(RedisKey, RedisValue[], When, CommandFlags)"/>
		public static Task<long> ListLeftPushAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement[] values, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.ListLeftPushAsync(key.Inner, RedisSerializer.Serialize(values), when, flags);
		}

		#endregion

		#region ListLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.ListLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> ListLengthAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key)
		{
			return target.ListLengthAsync(key.Inner);
		}

		#endregion

		#region ListRangeAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRangeAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> ListRangeAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, long start = 0, long stop = -1, CommandFlags flags = CommandFlags.None)
		{
			return target.ListRangeAsync(key.Inner, start, stop, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region ListRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRemoveAsync(RedisKey, RedisValue, Int64, CommandFlags)"/>
		public static Task<long> ListRemoveAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement value, long count = 0L, CommandFlags flags = CommandFlags.None)
		{
			return target.ListRemoveAsync(key.Inner, RedisSerializer.Serialize(value), count, flags);
		}

		#endregion

		#region ListRightPopAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRightPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> ListRightPopAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.ListRightPopAsync(key.Inner, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region ListRightPushAsync

		/// <inheritdoc cref="IDatabaseAsync.ListRightPushAsync(RedisKey, RedisValue, When, CommandFlags)"/>
		public static Task<long> ListRightPushAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, TElement item, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.ListRightPushAsync(key.Inner, RedisSerializer.Serialize(item), when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.ListRightPushAsync(RedisKey, RedisValue[], When, CommandFlags)"/>
		public static Task<long> ListRightPushAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, IEnumerable<TElement> values, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.ListRightPushAsync(key.Inner, RedisSerializer.Serialize(values), when, flags);
		}

		#endregion

		#region ListSetByIndexAsync

		/// <inheritdoc cref="IDatabaseAsync.ListSetByIndexAsync(RedisKey, Int64, RedisValue, CommandFlags)"/>
		public static Task ListSetByIndexAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, long index, TElement value, CommandFlags flags = CommandFlags.None)
		{
			return target.ListSetByIndexAsync(key.Inner, index, RedisSerializer.Serialize(value), flags);
		}

		#endregion

		#region ListTrimAsync

		/// <inheritdoc cref="IDatabaseAsync.ListTrimAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task ListTrimAsync<TElement>(this IDatabaseAsync target, RedisListKey<TElement> key, long start, long stop, CommandFlags flags = CommandFlags.None)
		{
			return target.ListTrimAsync(key.Inner, start, stop, flags);
		}

		#endregion
	}
}
