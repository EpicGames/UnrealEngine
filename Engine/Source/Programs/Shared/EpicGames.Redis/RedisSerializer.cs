// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel;
using System.Reflection;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Redis.Converters;
using EpicGames.Serialization;
using Google.Protobuf;
using ProtoBuf;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Attribute specifying the converter type to use for a class
	/// </summary>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct)]
	public sealed class RedisConverterAttribute : Attribute
	{
		/// <summary>
		/// Type of the converter to use
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="converterType">The converter type</param>
		public RedisConverterAttribute(Type converterType)
		{
			ConverterType = converterType;
		}
	}

	/// <summary>
	/// Converter to and from RedisValue types
	/// </summary>
	public interface IRedisConverter<T>
	{
		/// <summary>
		/// Serailize an object to a RedisValue
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		RedisValue ToRedisValue(T value);

		/// <summary>
		/// Deserialize an object from a RedisValue
		/// </summary>
		/// <param name="value"></param>
		/// <returns></returns>
		T FromRedisValue(RedisValue value);
	}

	/// <summary>
	/// Redis serializer that uses compact binary to serialize objects
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public sealed class RedisCbConverter<T> : IRedisConverter<T>
	{
		/// <inheritdoc/>
		public RedisValue ToRedisValue(T value)
		{
			return CbSerializer.Serialize(value).GetView();
		}

		/// <inheritdoc/>
		public T FromRedisValue(RedisValue value)
		{
			return CbSerializer.Deserialize<T>(new CbField((byte[])value!));
		}
	}

	/// <summary>
	/// Redis serializer that uses JSON to serialize objects
	/// </summary>
	public sealed class RedisJsonConverter<T> : IRedisConverter<T>
	{
		static readonly JsonSerializerOptions s_options = new() { PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase };

		/// <inheritdoc/>
		public RedisValue ToRedisValue(T value)
		{
			return JsonSerializer.Serialize(value, s_options);
		}

		/// <inheritdoc/>
		public T FromRedisValue(RedisValue value)
		{
			return JsonSerializer.Deserialize<T>((string?)value ?? String.Empty, s_options) ?? throw new FormatException("Expected non-null value");
		}
	}

	/// <summary>
	/// Handles serialization of types to RedisValue instances
	/// </summary>
	public static class RedisSerializer
	{
		class RedisStringConverter<T> : IRedisConverter<T>
		{
			readonly TypeConverter _typeConverter;

			public RedisStringConverter(TypeConverter typeConverter)
			{
				_typeConverter = typeConverter;
			}

			public RedisValue ToRedisValue(T value) => (string?)_typeConverter.ConvertTo(value, typeof(string));
			public T FromRedisValue(RedisValue value) => (T)_typeConverter.ConvertFrom((string)value!)!;
		}

		class RedisUtf8StringConverter<T> : IRedisConverter<T>
		{
			readonly TypeConverter _typeConverter;

			public RedisUtf8StringConverter(TypeConverter typeConverter)
			{
				_typeConverter = typeConverter;
			}

			public RedisValue ToRedisValue(T value) => ((Utf8String)_typeConverter.ConvertTo(value, typeof(Utf8String))!).Memory;
			public T FromRedisValue(RedisValue value) => (T)_typeConverter.ConvertFrom(new Utf8String((ReadOnlyMemory<byte>)value!))!;
		}

		class RedisNativeConverter<T> : IRedisConverter<T>
		{
			readonly Func<RedisValue, T> _fromRedisValueFunc;
			readonly Func<T, RedisValue> _toRedisValueFunc;

			public RedisNativeConverter(Func<RedisValue, T> fromRedisValueFunc, Func<T, RedisValue> toRedisValueFunc)
			{
				_fromRedisValueFunc = fromRedisValueFunc;
				_toRedisValueFunc = toRedisValueFunc;
			}

			public T FromRedisValue(RedisValue value) => _fromRedisValueFunc(value);
			public RedisValue ToRedisValue(T value) => _toRedisValueFunc(value);
		}

		static readonly Dictionary<Type, object> s_nativeConverters = CreateNativeConverterLookup();

		static Dictionary<Type, object> CreateNativeConverterLookup()
		{
			KeyValuePair<Type, object>[] converters =
			[
				CreateNativeConverter<RedisValue>(x => x, x => x),
				CreateNativeConverter(x => (bool)x, x => x),
				CreateNativeConverter(x => (int)x, x => x),
				CreateNativeConverter(x => (int?)x, x => x),
				CreateNativeConverter(x => (uint)x, x => x),
				CreateNativeConverter(x => (uint?)x, x => x),
				CreateNativeConverter(x => (long)x, x => x),
				CreateNativeConverter(x => (long?)x, x => x),
				CreateNativeConverter(x => (ulong)x, x => x),
				CreateNativeConverter(x => (ulong?)x, x => x),
				CreateNativeConverter(x => (double)x, x => x),
				CreateNativeConverter(x => (double?)x, x => x),
				CreateNativeConverter(x => (ReadOnlyMemory<byte>)x, x => x),
				CreateNativeConverter(x => (byte[])x!, x => x),
				CreateNativeConverter(x => (string)x!, x => x),
				CreateNativeConverter(x => new DateTime((long)x, DateTimeKind.Utc), x => x.ToUniversalTime().Ticks)
			];
			return new Dictionary<Type, object>(converters);
		}

		static KeyValuePair<Type, object> CreateNativeConverter<T>(Func<RedisValue, T> fromRedisValueFunc, Func<T, RedisValue> toRedisValueFunc)
		{
			return new KeyValuePair<Type, object>(typeof(T), new RedisNativeConverter<T>(fromRedisValueFunc, toRedisValueFunc));
		}

		static readonly Dictionary<Type, Type> s_typeToConverterType = [];

		class RedisObjectConverter<T> : IRedisConverter<object>
		{
			public object FromRedisValue(RedisValue value)
				=> GetConverter<T>().FromRedisValue(value)!;

			public RedisValue ToRedisValue(object value)
				=> GetConverter<T>().ToRedisValue((T)value);
		}

		static readonly ConcurrentDictionary<Type, IRedisConverter<object>> s_typeToObjectConverter = new();

		/// <summary>
		/// Register a custom converter for a particular type
		/// </summary>
		public static void RegisterConverter<T, TConverter>() where TConverter : IRedisConverter<T>
		{
			lock (s_typeToConverterType)
			{
				if (!s_typeToConverterType.TryGetValue(typeof(T), out Type? converterType) || converterType != typeof(TConverter))
				{
					s_typeToConverterType.Add(typeof(T), typeof(TConverter));
				}
			}
		}

		/// <summary>
		/// Creates a converter for a given type
		/// </summary>
		static IRedisConverter<T> CreateConverter<T>()
		{
			Type type = typeof(T);

			// Check for a registered converter type
			lock (s_typeToConverterType)
			{
				if (s_typeToConverterType.TryGetValue(type, out Type? converterType))
				{
					return (IRedisConverter<T>)Activator.CreateInstance(converterType)!;
				}
			}

			// Check for a custom converter
			RedisConverterAttribute? attribute = type.GetCustomAttribute<RedisConverterAttribute>();
			if (attribute != null)
			{
				Type converterType = attribute.ConverterType;
				if (converterType.IsGenericTypeDefinition)
				{
					converterType = converterType.MakeGenericType(type);
				}
				return (IRedisConverter<T>)Activator.CreateInstance(converterType)!;
			}

			// Check for known basic types
			object? nativeConverter;
			if (s_nativeConverters.TryGetValue(typeof(T), out nativeConverter))
			{
				return (IRedisConverter<T>)nativeConverter;
			}

			// Check if the type is a protobuf message
			if (type.IsAssignableTo(typeof(IMessage)))
			{
				Type converterType = typeof(RedisProtobufConverter<>).MakeGenericType(type);
				return (IRedisConverter<T>)Activator.CreateInstance(converterType)!;
			}

			// Check if the type supports protobuf serialization
			ProtoContractAttribute? protoAttribute = type.GetCustomAttribute<ProtoContractAttribute>();
			if (protoAttribute != null)
			{
				return new RedisProtobufNetConverter<T>();
			}

			// Check if there's a regular converter we can use to convert to/from a string
			TypeConverter? converter = TypeDescriptor.GetConverter(type);
			if (converter != null)
			{
				if (converter.CanConvertFrom(typeof(Utf8String)) && converter.CanConvertTo(typeof(Utf8String)))
				{
					return new RedisUtf8StringConverter<T>(converter);
				}
				if (converter.CanConvertFrom(typeof(string)) && converter.CanConvertTo(typeof(string)))
				{
					return new RedisStringConverter<T>(converter);
				}
			}

			// If it's a compound type, try to create a class converter
			if (type.IsClass)
			{
				return new RedisClassConverter<T>();
			}

			// Otherwise fail
			throw new Exception($"Unable to find Redis converter for {type.Name}");
		}

		/// <summary>
		/// Static class for caching converter lookups
		/// </summary>
		/// <typeparam name="T"></typeparam>
		class CachedConverter<T>
		{
			public static IRedisConverter<T> Converter = CreateConverter<T>();
		}

		/// <summary>
		/// Gets the converter for a particular type
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <returns></returns>
		public static IRedisConverter<T> GetConverter<T>()
		{
			return CachedConverter<T>.Converter;
		}

		/// <summary>
		/// Gets a type converter which casts to/from an object value
		/// </summary>
		/// <param name="type">The concrete type for the converter</param>
		/// <returns></returns>
		public static IRedisConverter<object> GetObjectConverter(Type type)
		{
			IRedisConverter<object>? converter;
			if (!s_typeToObjectConverter.TryGetValue(type, out converter))
			{
				converter = s_typeToObjectConverter.GetOrAdd(type, (IRedisConverter<object>)Activator.CreateInstance(typeof(RedisObjectConverter<>).MakeGenericType(type))!);
			}
			return converter;
		}

		/// <summary>
		/// Serialize an object to a <see cref="RedisValue"/>
		/// </summary>
		/// <param name="value"></param>
		/// <param name="type">Type of the object</param>
		/// <returns></returns>
		public static RedisValue Serialize(object? value, Type type)
		{
			return GetObjectConverter(type).ToRedisValue(value!);
		}

		/// <summary>
		/// Serialize an object to a <see cref="RedisValue"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static RedisValue Serialize<T>(T value)
		{
			return CachedConverter<T>.Converter.ToRedisValue(value);
		}

		/// <summary>
		/// Serialize an object to a <see cref="RedisValue"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="inputs"></param>
		/// <returns></returns>
		public static RedisValue[] Serialize<T>(T[] inputs)
		{
			RedisValue[] outputs = new RedisValue[inputs.Length];

			for (int idx = 0; idx < inputs.Length; idx++)
			{
				outputs[idx] = Serialize<T>(inputs[idx]);
			}

			return outputs;
		}

		/// <summary>
		/// Deserialize a <see cref="RedisValue"/>
		/// </summary>
		/// <param name="value"></param>
		/// <param name="type">Type of the value to return</param>
		/// <returns></returns>
		public static object? Deserialize(RedisValue value, Type type)
		{
			return GetObjectConverter(type).FromRedisValue(value);
		}

		/// <summary>
		/// Deserialize a <see cref="RedisValue"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static T Deserialize<T>(RedisValue value)
		{
			return CachedConverter<T>.Converter.FromRedisValue(value);
		}

		/// <summary>
		/// Deserialize an array of <see cref="RedisValue"/> objects
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="inputs"></param>
		/// <returns></returns>
		public static T[] Deserialize<T>(RedisValue[] inputs)
		{
			T[] outputs = new T[inputs.Length];

			for (int idx = 0; idx < inputs.Length; idx++)
			{
				outputs[idx] = Deserialize<T>(inputs[idx]);
			}

			return outputs;
		}
	}

	/// <summary>
	/// Extension methods for serialization
	/// </summary>
	public static class RedisSerializerExtensions
	{
		/// <summary>
		/// Deserialize a <see cref="RedisValue"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static async Task<T> DeserializeAsync<T>(this Task<RedisValue> value)
		{
			RedisValue result = await value;
			if (result.IsNull)
			{
				return default!;
			}
			else
			{
				return RedisSerializer.Deserialize<T>(await value);
			}
		}

		/// <summary>
		/// Deserialize a <see cref="RedisValue"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="values"></param>
		/// <returns></returns>
		public static async Task<T[]> DeserializeAsync<T>(this Task<RedisValue[]> values)
		{
			return RedisSerializer.Deserialize<T>(await values);
		}
	}
}
