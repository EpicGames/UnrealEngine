// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis set with a given key
	/// </summary>
	/// <typeparam name="TElement"></typeparam>
	public record struct RedisSet<TElement>(IDatabaseAsync Database, RedisSetKey<TElement> Key);

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisSetExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.SetContains(RedisKey, RedisValue)"/>
		public static Condition SetContains<TElement>(this RedisSet<TElement> target, TElement value)
			=> target.Key.SetContains(value);

		/// <inheritdoc cref="Condition.SetLengthEqual(RedisKey, Int64)"/>
		public static Condition SetLengthEqual<TElement>(this RedisSet<TElement> target, long length)
			=> target.Key.SetLengthEqual(length);

		/// <inheritdoc cref="Condition.SetLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition SetLengthGreaterThan<TElement>(this RedisSet<TElement> target, long length)
			=> target.Key.SetLengthGreaterThan(length);

		/// <inheritdoc cref="Condition.SetLengthLessThan(RedisKey, Int64)"/>
		public static Condition SetLengthLessThan<TElement>(this RedisSet<TElement> target, long length)
			=> target.Key.SetLengthLessThan(length);

		/// <inheritdoc cref="Condition.SetNotContains(RedisKey, RedisValue)"/>
		public static Condition SetNotContains<TElement>(this RedisSet<TElement> target, TElement value)
			=> target.Key.SetNotContains(value);

		#endregion

		#region SetAddAsync

		/// <inheritdoc cref="IDatabaseAsync.SetAddAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> AddAsync<TElement>(this RedisSet<TElement> target, TElement item, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetAddAsync(target.Key, item, flags);

		/// <inheritdoc cref="IDatabaseAsync.SetAddAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> AddAsync<TElement>(this RedisSet<TElement> target, TElement[] values, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetAddAsync(target.Key, values, flags);

		#endregion

		#region SetContainsAsync

		/// <inheritdoc cref="IDatabaseAsync.SetContainsAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> ContainsAsync<TElement>(this RedisSet<TElement> target, TElement value, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetContainsAsync(target.Key, value, flags);

		#endregion

		#region SetLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.SetLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> LengthAsync<TElement>(this RedisSet<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetLengthAsync(target.Key, flags);

		#endregion

		#region SetMembersAsync

		/// <inheritdoc cref="IDatabaseAsync.SetMembersAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement[]> MembersAsync<TElement>(this RedisSet<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetMembersAsync(target.Key, flags);

		#endregion

		#region SetPopAsync

		/// <inheritdoc cref="IDatabaseAsync.SetPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> PopAsync<TElement>(this RedisSet<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetPopAsync(target.Key, flags);

		/// <inheritdoc cref="IDatabaseAsync.SetPopAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<TElement[]> PopAsync<TElement>(this RedisSet<TElement> target, long count, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetPopAsync(target.Key, count, flags);

		#endregion

		#region SetRandomMemberAsync

		/// <inheritdoc cref="IDatabaseAsync.SetRandomMemberAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> RandomMemberAsync<TElement>(this RedisSet<TElement> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetRandomMemberAsync(target.Key, flags);

		#endregion

		#region SetRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.SetRemoveAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> RemoveAsync<TElement>(this RedisSet<TElement> target, TElement item, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetRemoveAsync(target.Key, item, flags);

		/// <inheritdoc cref="IDatabaseAsync.SetRemoveAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> RemoveAsync<TElement>(this RedisSet<TElement> target, TElement[] values, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetRemoveAsync(target.Key, values, flags);

		#endregion

		#region SetScanAsync

		/// <inheritdoc cref="IDatabaseAsync.SetScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static IAsyncEnumerable<TElement> ScanAsync<TElement>(this RedisSet<TElement> target, RedisValue pattern = default, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
			=> target.Database.SetScanAsync(target.Key, pattern, pageSize, cursor, pageOffset, flags);

		#endregion
	}
}
