// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis set with a given key
	/// </summary>
	/// <typeparam name="TElement">The type of element stored in the set</typeparam>
	public record struct RedisSetKey<TElement>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisSetKey<TElement>(string key) => new(new RedisKey(key));
	}

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisSetKeyExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.SetContains(RedisKey, RedisValue)"/>
		public static Condition SetContains<TElement>(this RedisSetKey<TElement> key, TElement value)
			=> Condition.SetContains(key.Inner, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.SetLengthEqual(RedisKey, Int64)"/>
		public static Condition SetLengthEqual<TElement>(this RedisSetKey<TElement> key, long length)
			=> Condition.SetLengthEqual(key.Inner, length);

		/// <inheritdoc cref="Condition.SetLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition SetLengthGreaterThan<TElement>(this RedisSetKey<TElement> key, long length)
			=> Condition.SetLengthGreaterThan(key.Inner, length);

		/// <inheritdoc cref="Condition.SetLengthLessThan(RedisKey, Int64)"/>
		public static Condition SetLengthLessThan<TElement>(this RedisSetKey<TElement> key, long length)
			=> Condition.SetLengthLessThan(key.Inner, length);

		/// <inheritdoc cref="Condition.SetContains(RedisKey, RedisValue)"/>
		public static Condition SetNotContains<TElement>(this RedisSetKey<TElement> key, TElement value)
			=> Condition.SetNotContains(key.Inner, RedisSerializer.Serialize(value));

		#endregion

		#region SetAddAsync

		/// <inheritdoc cref="IDatabaseAsync.SetAddAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> SetAddAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, TElement item, CommandFlags flags = CommandFlags.None)
		{
			return target.SetAddAsync(key.Inner, RedisSerializer.Serialize(item), flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.SetAddAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> SetAddAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, TElement[] values, CommandFlags flags = CommandFlags.None)
		{
			return target.SetAddAsync(key.Inner, RedisSerializer.Serialize(values), flags);
		}

		#endregion

		#region SetContainsAsync

		/// <inheritdoc cref="IDatabaseAsync.SetContainsAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> SetContainsAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, TElement value, CommandFlags flags = CommandFlags.None)
		{
			return target.SetContainsAsync(key.Inner, RedisSerializer.Serialize(value), flags);
		}

		#endregion

		#region SetLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.SetLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> SetLengthAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.SetLengthAsync(key.Inner, flags);
		}

		#endregion

		#region SetMembersAsync

		/// <inheritdoc cref="IDatabaseAsync.SetMembersAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement[]> SetMembersAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.SetMembersAsync(key.Inner, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SetPopAsync

		/// <inheritdoc cref="IDatabaseAsync.SetPopAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> SetPopAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.SetPopAsync(key.Inner, flags).DeserializeAsync<TElement>();
		}

		/// <inheritdoc cref="IDatabaseAsync.SetPopAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<TElement[]> SetPopAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, long count, CommandFlags flags = CommandFlags.None)
		{
			return target.SetPopAsync(key.Inner, count, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SetRandomMemberAsync

		/// <inheritdoc cref="IDatabaseAsync.SetRandomMemberAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> SetRandomMemberAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, CommandFlags flags = CommandFlags.None)
			=> target.SetRandomMemberAsync(key.Inner, flags).DeserializeAsync<TElement>();

		#endregion

		#region SetRandomMembersAsync

		/// <inheritdoc cref="IDatabaseAsync.SetRandomMembersAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<TElement[]> SetRandomMembersAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, long count, CommandFlags flags = CommandFlags.None)
			=> target.SetRandomMembersAsync(key.Inner, count, flags).DeserializeAsync<TElement>();

		#endregion

		#region SetRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.SetRemoveAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> SetRemoveAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, TElement item, CommandFlags flags = CommandFlags.None)
		{
			return target.SetRemoveAsync(key.Inner, RedisSerializer.Serialize(item), flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.SetRemoveAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> SetRemoveAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, TElement[] values, CommandFlags flags = CommandFlags.None)
		{
			return target.SetRemoveAsync(key.Inner, RedisSerializer.Serialize(values), flags);
		}

		#endregion

		#region SetScanAsync

		/// <inheritdoc cref="IDatabaseAsync.SetScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static async IAsyncEnumerable<TElement> SetScanAsync<TElement>(this IDatabaseAsync target, RedisSetKey<TElement> key, RedisValue pattern = default, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
		{
			await foreach (RedisValue value in target.SetScanAsync(key.Inner, pattern, pageSize, cursor, pageOffset, flags))
			{
				yield return RedisSerializer.Deserialize<TElement>(value);
			}
		}

		#endregion
	}
}
