// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis sorted set with a given key
	/// </summary>
	public record struct RedisSortedSet<TElement>(IDatabaseAsync Database, RedisSortedSetKey<TElement> Key);

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisSortedSetExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.SortedSetContains(RedisKey, RedisValue)"/>
		public static Condition SortedSetContains<TElement>(this RedisSortedSet<TElement> target, TElement value)
			=> target.Key.SortedSetContains(value);

		/// <inheritdoc cref="Condition.SortedSetEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetEqual<TElement>(this RedisSortedSet<TElement> target, TElement value, RedisValue score)
			=> target.Key.SortedSetEqual(value, score);

		/// <inheritdoc cref="Condition.SortedSetLengthEqual(RedisKey, Int64)"/>
		public static Condition SortedSetLengthEqual<TElement>(this RedisSortedSet<TElement> target, long length)
			=> target.Key.SortedSetLengthLessThan(length);

		/// <inheritdoc cref="Condition.SortedSetLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition SortedSetLengthGreaterThan<TElement>(this RedisSortedSet<TElement> target, long length)
			=> target.Key.SortedSetLengthGreaterThan(length);

		/// <inheritdoc cref="Condition.SortedSetLengthLessThan(RedisKey, Int64)"/>
		public static Condition SortedSetLengthLessThan<TElement>(this RedisSortedSet<TElement> target, long length)
			=> target.Key.SortedSetLengthLessThan(length);

		/// <inheritdoc cref="Condition.SortedSetNotContains(RedisKey, RedisValue)"/>
		public static Condition SortedSetNotContains<TElement>(this RedisSortedSet<TElement> target, TElement value)
			=> target.Key.SortedSetNotContains(value);

		/// <inheritdoc cref="Condition.SortedSetNotEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetNotEqual<TElement>(this RedisSortedSet<TElement> target, TElement value, RedisValue score)
			=> target.Key.SortedSetNotEqual(value, score);

		/// <inheritdoc cref="Condition.SortedSetScoreExists(RedisKey, RedisValue)"/>
		public static Condition SortedSetScoreExists<TElement>(this RedisSortedSet<TElement> target, RedisValue score)
			=> target.Key.SortedSetScoreExists(score);

		/// <inheritdoc cref="Condition.SortedSetScoreExists(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetScoreExists<TElement>(this RedisSortedSet<TElement> target, RedisValue score, RedisValue count)
			=> target.Key.SortedSetScoreExists(score, count);

		#endregion

		#region SortedSetAddAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetAddAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<bool> AddAsync<TElement>(this RedisSortedSet<TElement> target, TElement value, double score, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetAddAsync(target.Key, value, score, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.SortedSetAddAsync(RedisKey, RedisValue, Double, When, CommandFlags)"/>
		public static Task<long> AddAsync<TElement>(this RedisSortedSet<TElement> target, SortedSetEntry<TElement>[] values, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetAddAsync(target.Key, values, when, flags);

		#endregion

		#region SortedSetLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetLengthAsync(RedisKey, Double, Double, Exclude, CommandFlags)"/>
		public static Task<long> LengthAsync<TElement>(this RedisSortedSet<TElement> target, double min = Double.NegativeInfinity, double max = Double.PositiveInfinity, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetLengthAsync(target.Key, min, max, exclude, flags);

		#endregion

		#region SortedSetLengthByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetLengthByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, CommandFlags)"/>
		public static Task<long> LengthByValueAsync<TElement>(this RedisSortedSet<TElement> target, TElement min, TElement max, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetLengthByValueAsync(target.Key, min, max, exclude, flags);

		#endregion

		#region SortedSetRangeByRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByRankAsync(RedisKey, Int64, Int64, Order, CommandFlags)"/>
		public static Task<TElement[]> RangeByRankAsync<TElement>(this RedisSortedSet<TElement> target, long start = 0, long stop = -1, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRangeByRankAsync(target.Key, start, stop, order, flags);

		#endregion

		#region SortedSetRangeByRankWithScoresAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByRankWithScoresAsync(RedisKey, Int64, Int64, Order, CommandFlags)"/>
		public static Task<SortedSetEntry<TElement>[]> RangeByRankWithScoresAsync<TElement>(this RedisSortedSet<TElement> target, long start, long stop = -1, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRangeByRankWithScoresAsync(target.Key, start, stop, order, flags);

		#endregion

		#region SortedSetRangeByScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByScoreAsync(RedisKey, Double, Double, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> RangeByScoreAsync<TElement>(this RedisSortedSet<TElement> target, double start = Double.NegativeInfinity, double stop = Double.PositiveInfinity, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRangeByScoreAsync(target.Key, start, stop, exclude, order, skip, take, flags);

		#endregion

		#region SortedSetRangeByScoreWithScoresAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByScoreWithScoresAsync(RedisKey, Double, Double, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static Task<SortedSetEntry<TElement>[]> SortedSetRangeByScoreWithScoresAsync<TElement>(this RedisSortedSet<TElement> target, double start = Double.NegativeInfinity, double stop = Double.PositiveInfinity, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRangeByScoreWithScoresAsync(target.Key, start, stop, exclude, order, skip, take, flags);

		#endregion

		#region SortedSetRangeByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> RangeByValueAsync<TElement>(this RedisSortedSet<TElement> target, TElement min, TElement max, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRangeByValueAsync(target.Key, min, max, exclude, order, skip, take, flags);

		#endregion

		#region SortedSetRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRankAsync(RedisKey, RedisValue, Order, CommandFlags)"/>
		public static Task<long?> RankAsync<TElement>(this RedisSortedSet<TElement> target, TElement item, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRankAsync(target.Key, item, order, flags);

		#endregion

		#region SortedSetRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> RemoveAsync<TElement>(this RedisSortedSet<TElement> target, TElement value, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRemoveAsync(target.Key, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> RemoveAsync<TElement>(this RedisSortedSet<TElement> target, TElement[] values, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRemoveAsync(target.Key, values, flags);

		#endregion

		#region SortedSetRemoveRangeByRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByRankAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task<long> RemoveRangeByRankAsync<TElement>(this RedisSortedSet<TElement> target, long start, long stop, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRemoveRangeByRankAsync(target.Key, start, stop, flags);

		#endregion

		#region SortedSetRemoveRangeByScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByScoreAsync(RedisKey, Double, Double, Exclude, CommandFlags)"/>
		public static Task<long> RemoveRangeByScoreAsync<TElement>(this RedisSortedSet<TElement> target, double start, double stop, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRemoveRangeByScoreAsync(target.Key, start, stop, exclude, flags);

		#endregion

		#region SortedSetRemoveRangeByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, CommandFlags)"/>
		public static Task<long> RemoveRangeByValueAsync<TElement>(this RedisSortedSet<TElement> target, TElement min, TElement max, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetRemoveRangeByValueAsync(target.Key, min, max, exclude, flags);

		#endregion

		#region SortedSetScanAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static IAsyncEnumerable<SortedSetEntry<TElement>> ScanAsync<TElement>(this RedisSortedSet<TElement> target, RedisValue pattern = default, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetScanAsync(target.Key, pattern, pageSize, cursor, pageOffset, flags);

		#endregion

		#region SortedSetScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetScoreAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<double?> ScoreAsync<TElement>(this RedisSortedSet<TElement> target, TElement member, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetScoreAsync(target.Key, member, flags);

		#endregion

		#region SortedSetUpdateAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetUpdateAsync(RedisKey, RedisValue, Double, SortedSetWhen, CommandFlags)"/>
		public static Task<bool> UpdateAsync<TElement>(this RedisSortedSet<TElement> target, TElement member, double score, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetUpdateAsync(target.Key, member, score, when, flags);

		/// <inheritdoc cref="IDatabaseAsync.SortedSetUpdateAsync(RedisKey, RedisValue, Double, SortedSetWhen, CommandFlags)"/>
		public static Task<long> UpdateAsync<TElement>(this RedisSortedSet<TElement> target, SortedSetEntry<TElement>[] values, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.SortedSetUpdateAsync(target.Key, values, when, flags);

		#endregion
	}
}
