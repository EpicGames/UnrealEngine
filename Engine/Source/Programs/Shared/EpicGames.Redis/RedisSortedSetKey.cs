// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis sorted set with a given key
	/// </summary>
	/// <typeparam name="TElement">The type of element stored in the set</typeparam>
	public record struct RedisSortedSetKey<TElement>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisSortedSetKey<TElement>(string key) => new(new RedisKey(key));
	}

	/// <summary>
	/// Typed implementation of <see cref="SortedSetEntry"/>
	/// </summary>
	/// <typeparam name="T">The element type</typeparam>
	public readonly struct SortedSetEntry<T> : IEquatable<SortedSetEntry<T>>, IComparable, IComparable<SortedSetEntry<T>>
	{
		/// <summary>
		/// Accessor for the element type
		/// </summary>
		public readonly T Element { get; }

		/// <summary>
		/// The encoded element value
		/// </summary>
		public readonly RedisValue ElementValue { get; }

		/// <summary>
		/// Score for the entry
		/// </summary>
		public readonly double Score { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="entry"></param>
		public SortedSetEntry(SortedSetEntry entry)
		{
			Element = RedisSerializer.Deserialize<T>(entry.Element)!;
			ElementValue = entry.Element;
			Score = entry.Score;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="element"></param>
		/// <param name="score"></param>
		public SortedSetEntry(T element, double score)
		{
			Element = element;
			ElementValue = RedisSerializer.Serialize<T>(element);
			Score = score;
		}

		/// <summary>
		/// Deconstruct this item into a tuple
		/// </summary>
		/// <param name="outElement"></param>
		/// <param name="outScore"></param>
		public void Deconstruct(out T outElement, out double outScore)
		{
			outElement = Element;
			outScore = Score;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
			=> obj is SortedSetEntry ssObj && Equals(ssObj);

		/// <inheritdoc/>
		public override int GetHashCode()
			=> HashCode.Combine(Score, Element);

		/// <inheritdoc/>
		public bool Equals(SortedSetEntry<T> other)
			=> Score == other.Score && Equals(Element, other.Element);

		/// <inheritdoc/>
		public int CompareTo(SortedSetEntry<T> other)
			=> Score.CompareTo(other.Score);

		/// <inheritdoc/>
		public int CompareTo(object? obj)
			=> obj is SortedSetEntry<T> ssObj ? CompareTo(ssObj) : -1;

		/// <inheritdoc/>
		public static bool operator ==(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> a.Equals(b);

		/// <inheritdoc/>
		public static bool operator !=(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> !a.Equals(b);

		/// <inheritdoc/>
		public static bool operator <=(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> a.CompareTo(b) <= 0;

		/// <inheritdoc/>
		public static bool operator <(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> a.CompareTo(b) < 0;

		/// <inheritdoc/>
		public static bool operator >=(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> a.CompareTo(b) >= 0;

		/// <inheritdoc/>
		public static bool operator >(SortedSetEntry<T> a, SortedSetEntry<T> b)
			=> a.CompareTo(b) > 0;
	}

	/// <summary>
	/// Extension methods for sets
	/// </summary>
	public static class RedisSortedSetKeyExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.SortedSetContains(RedisKey, RedisValue)"/>
		public static Condition SortedSetContains<TElement>(this RedisSortedSetKey<TElement> key, TElement value)
			=> Condition.SortedSetContains(key.Inner, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.SortedSetEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetEqual<TElement>(this RedisSortedSetKey<TElement> key, TElement value, RedisValue score)
			=> Condition.SortedSetEqual(key.Inner, RedisSerializer.Serialize(value), score);

		/// <inheritdoc cref="Condition.SortedSetLengthEqual(RedisKey, Int64)"/>
		public static Condition SortedSetLengthEqual<TElement>(this RedisSortedSetKey<TElement> key, long length)
			=> Condition.SortedSetLengthEqual(key.Inner, length);

		/// <inheritdoc cref="Condition.SortedSetLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition SortedSetLengthGreaterThan<TElement>(this RedisSortedSetKey<TElement> key, long length)
			=> Condition.SortedSetLengthGreaterThan(key.Inner, length);

		/// <inheritdoc cref="Condition.SortedSetLengthLessThan(RedisKey, Int64)"/>
		public static Condition SortedSetLengthLessThan<TElement>(this RedisSortedSetKey<TElement> key, long length)
			=> Condition.SortedSetLengthLessThan(key.Inner, length);

		/// <inheritdoc cref="Condition.SortedSetNotContains(RedisKey, RedisValue)"/>
		public static Condition SortedSetNotContains<TElement>(this RedisSortedSetKey<TElement> key, TElement value)
			=> Condition.SortedSetNotContains(key.Inner, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.SortedSetNotEqual(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetNotEqual<TElement>(this RedisSortedSetKey<TElement> key, TElement value, RedisValue score)
			=> Condition.SortedSetNotEqual(key.Inner, RedisSerializer.Serialize(value), score);

		/// <inheritdoc cref="Condition.SortedSetScoreExists(RedisKey, RedisValue)"/>
		public static Condition SortedSetScoreExists<TElement>(this RedisSortedSetKey<TElement> key, RedisValue score)
			=> Condition.SortedSetScoreExists(key.Inner, score);

		/// <inheritdoc cref="Condition.SortedSetScoreExists(RedisKey, RedisValue, RedisValue)"/>
		public static Condition SortedSetScoreExists<TElement>(this RedisSortedSetKey<TElement> key, RedisValue score, RedisValue count)
			=> Condition.SortedSetScoreExists(key.Inner, score, count);

		#endregion

		#region SortedSetAddAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetAddAsync(RedisKey, RedisValue, Double, CommandFlags)"/>
		public static Task<bool> SortedSetAddAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement value, double score, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetAddAsync(key.Inner, RedisSerializer.Serialize(value), score, when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.SortedSetAddAsync(RedisKey, RedisValue, Double, When, CommandFlags)"/>
		public static Task<long> SortedSetAddAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, SortedSetEntry<TElement>[] values, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetAddAsync(key.Inner, values.ConvertAll(x => new SortedSetEntry(x.ElementValue, x.Score)), when, flags);
		}

		#endregion

		#region SortedSetLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetLengthAsync(RedisKey, Double, Double, Exclude, CommandFlags)"/>
		public static Task<long> SortedSetLengthAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, double min = Double.NegativeInfinity, double max = Double.PositiveInfinity, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetLengthAsync(key.Inner, min, max, exclude, flags);
		}

		#endregion

		#region SortedSetLengthByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetLengthByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, CommandFlags)"/>
		public static Task<long> SortedSetLengthByValueAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement min, TElement max, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetLengthByValueAsync(key.Inner, RedisSerializer.Serialize(min), RedisSerializer.Serialize(max), exclude, flags);
		}

		#endregion

		#region SortedSetRandomMemberAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRandomMemberAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement> SortedSetRandomMemberAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRandomMemberAsync(key.Inner, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SortedSetRandomMembersAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRandomMemberAsync(RedisKey, CommandFlags)"/>
		public static Task<TElement[]> SortedSetRandomMembersAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, long count, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRandomMembersAsync(key.Inner, count, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SortedSetRangeByRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByRankAsync(RedisKey, Int64, Int64, Order, CommandFlags)"/>
		public static Task<TElement[]> SortedSetRangeByRankAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, long start = 0, long stop = -1, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRangeByRankAsync(key.Inner, start, stop, order, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SortedSetRangeByRankWithScoresAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByRankWithScoresAsync(RedisKey, Int64, Int64, Order, CommandFlags)"/>
		public static async Task<SortedSetEntry<TElement>[]> SortedSetRangeByRankWithScoresAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, long start = 0, long stop = -1, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
		{
			SortedSetEntry[] values = await target.SortedSetRangeByRankWithScoresAsync(key.Inner, start, stop, order, flags);
			return Array.ConvertAll(values, x => new SortedSetEntry<TElement>(x));
		}

		#endregion

		#region SortedSetRangeByScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByScoreAsync(RedisKey, Double, Double, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> SortedSetRangeByScoreAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, double start = Double.NegativeInfinity, double stop = Double.PositiveInfinity, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRangeByScoreAsync(key.Inner, start, stop, exclude, order, skip, take, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SortedSetRangeByScoreWithScoresAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByScoreWithScoresAsync(RedisKey, Double, Double, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static async Task<SortedSetEntry<TElement>[]> SortedSetRangeByScoreWithScoresAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, double start = Double.NegativeInfinity, double stop = Double.PositiveInfinity, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
		{
			SortedSetEntry[] values = await target.SortedSetRangeByScoreWithScoresAsync(key.Inner, start, stop, exclude, order, skip, take, flags);
			return Array.ConvertAll(values, x => new SortedSetEntry<TElement>(x));
		}

		#endregion

		#region SortedSetRangeByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRangeByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, Order, Int64, Int64, CommandFlags)"/>
		public static Task<TElement[]> SortedSetRangeByValueAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement min, TElement max, Exclude exclude = Exclude.None, Order order = Order.Ascending, long skip = 0L, long take = -1L, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRangeByValueAsync(key.Inner, RedisSerializer.Serialize(min), RedisSerializer.Serialize(max), exclude, order, skip, take, flags).DeserializeAsync<TElement>();
		}

		#endregion

		#region SortedSetRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRankAsync(RedisKey, RedisValue, Order, CommandFlags)"/>
		public static Task<long?> SortedSetRankAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement item, Order order = Order.Ascending, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRankAsync(key.Inner, RedisSerializer.Serialize(item), order, flags);
		}

		#endregion

		#region SortedSetRemoveAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<bool> SortedSetRemoveAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement value, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRemoveAsync(key.Inner, RedisSerializer.Serialize(value), flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveAsync(RedisKey, RedisValue[], CommandFlags)"/>
		public static Task<long> SortedSetRemoveAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement[] values, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRemoveAsync(key.Inner, RedisSerializer.Serialize(values), flags);
		}

		#endregion

		#region SortedSetRemoveRangeByRankAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByRankAsync(RedisKey, Int64, Int64, CommandFlags)"/>
		public static Task<long> SortedSetRemoveRangeByRankAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, long start, long stop, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRemoveRangeByRankAsync(key.Inner, start, stop, flags);
		}

		#endregion

		#region SortedSetRemoveRangeByScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByScoreAsync(RedisKey, Double, Double, Exclude, CommandFlags)"/>
		public static Task<long> SortedSetRemoveRangeByScoreAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, double start, double stop, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRemoveRangeByScoreAsync(key.Inner, start, stop, exclude, flags);
		}

		#endregion

		#region SortedSetRemoveRangeByValueAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetRemoveRangeByValueAsync(RedisKey, RedisValue, RedisValue, Exclude, CommandFlags)"/>
		public static Task<long> SortedSetRemoveRangeByValueAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement min, TElement max, Exclude exclude = Exclude.None, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetRemoveRangeByValueAsync(key.Inner, RedisSerializer.Serialize(min), RedisSerializer.Serialize(max), exclude, flags);
		}

		#endregion

		#region SortedSetScanAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetScanAsync(RedisKey, RedisValue, Int32, Int64, Int32, CommandFlags)"/>
		public static async IAsyncEnumerable<SortedSetEntry<TElement>> SortedSetScanAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, RedisValue pattern = default, int pageSize = 250, long cursor = 0, int pageOffset = 0, CommandFlags flags = CommandFlags.None)
		{
			await foreach (SortedSetEntry entry in target.SortedSetScanAsync(key.Inner, pattern, pageSize, cursor, pageOffset, flags))
			{
				yield return new SortedSetEntry<TElement>(entry);
			}
		}

		#endregion

		#region SortedSetScoreAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetScoreAsync(RedisKey, RedisValue, CommandFlags)"/>
		public static Task<double?> SortedSetScoreAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement member, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetScoreAsync(key.Inner, RedisSerializer.Serialize(member), flags);
		}

		#endregion

		#region SortedSetUpdateAsync

		/// <inheritdoc cref="IDatabaseAsync.SortedSetUpdateAsync(RedisKey, RedisValue, Double, SortedSetWhen, CommandFlags)"/>
		public static Task<bool> SortedSetUpdateAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, TElement member, double score, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetUpdateAsync(key.Inner, RedisSerializer.Serialize(member), score, when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.SortedSetUpdateAsync(RedisKey, RedisValue, Double, SortedSetWhen, CommandFlags)"/>
		public static Task<long> SortedSetUpdateAsync<TElement>(this IDatabaseAsync target, RedisSortedSetKey<TElement> key, SortedSetEntry<TElement>[] values, SortedSetWhen when = SortedSetWhen.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.SortedSetUpdateAsync(key.Inner, values.ConvertAll(x => new SortedSetEntry(RedisSerializer.Serialize(x.Element), x.Score)), when, flags);
		}

		#endregion
	}
}
