// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Accessor for a typed Redis string
	/// </summary>
	/// <param name="Database">Database to operate on</param>
	/// <param name="Key">Key for the string</param>
	/// <typeparam name="TValue">Type of element stored in the string</typeparam>
	public record struct RedisString<TValue>(IDatabaseAsync Database, RedisStringKey<TValue> Key);

	/// <summary>
	/// Extension methods for strings
	/// </summary>
	public static class RedisStringExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.StringEqual(RedisKey, RedisValue)"/>
		public static Condition StringEqual<TElement>(this RedisString<TElement> target, TElement value)
			=> target.Key.StringEqual(value);

		/// <inheritdoc cref="Condition.StringLengthEqual(RedisKey, Int64)"/>
		public static Condition StringLengthEqual<TElement>(this RedisString<TElement> target, long length)
			=> target.Key.StringLengthEqual(length);

		/// <inheritdoc cref="Condition.StringLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition StringLengthGreaterThan<TElement>(this RedisString<TElement> target, long length)
			=> target.Key.StringLengthGreaterThan(length);

		/// <inheritdoc cref="Condition.StringLengthLessThan(RedisKey, Int64)"/>
		public static Condition StringLengthLessThan<TElement>(this RedisString<TElement> target, long length)
			=> target.Key.StringLengthLessThan(length);

		/// <inheritdoc cref="Condition.StringNotEqual(RedisKey, RedisValue)"/>
		public static Condition StringNotEqual<TElement>(this RedisString<TElement> target, TElement value)
			=> target.Key.StringNotEqual(value);

		#endregion

		#region StringDecrementAsync

		/// <inheritdoc cref="IDatabaseAsync.StringDecrementAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<long> StringDecrementAsync(this RedisString<long> target, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringDecrementAsync(target.Key, value, flags);

		/// <inheritdoc cref="IDatabaseAsync.StringDecrementAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<double> StringDecrementAsync(this RedisString<double> target, double value = 1.0, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringDecrementAsync(target.Key, value, flags);

		#endregion

		#region StringGetAsync

		/// <inheritdoc cref="IDatabaseAsync.StringGetAsync(RedisKey, CommandFlags)"/>
		public static Task<TValue?> GetAsync<TValue>(this RedisString<TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringGetAsync(target.Key, flags);

		/// <inheritdoc cref="IDatabaseAsync.StringGetAsync(RedisKey, CommandFlags)"/>
		public static Task<TValue> GetAsync<TValue>(this RedisString<TValue> target, TValue defaultValue, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringGetAsync(target.Key, defaultValue, flags);

		#endregion

		#region StringIncrementAsync

		/// <inheritdoc cref="IDatabaseAsync.StringIncrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<long> IncrementAsync(this RedisString<long> target, long value = 1L, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringIncrementAsync(target.Key, value, flags);

		#endregion

		#region StringLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.StringLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> LengthAsync<TValue>(this RedisString<TValue> target, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringLengthAsync(target.Key, flags);

		#endregion

		#region StringSetAsync

		/// <inheritdoc cref="IDatabaseAsync.StringSetAsync(RedisKey, RedisValue, TimeSpan?, When, CommandFlags)"/>
		public static Task<bool> SetAsync<TValue>(this RedisString<TValue> target, TValue value, TimeSpan? expiry = null, When when = When.Always, CommandFlags flags = CommandFlags.None)
			=> target.Database.StringSetAsync<TValue>(target.Key, value, expiry, when, flags);

		#endregion
	}
}
