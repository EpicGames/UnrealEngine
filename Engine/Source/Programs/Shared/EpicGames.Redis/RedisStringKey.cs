// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Represents a typed Redis string with a given value type
	/// </summary>
	/// <typeparam name="TValue">The type of element stored in the list</typeparam>
	public record struct RedisStringKey<TValue>(RedisKey Inner) : IRedisTypedKey
	{
		/// <summary>
		/// Implicit conversion to typed redis key.
		/// </summary>
		/// <param name="key">Key to convert</param>
		public static implicit operator RedisStringKey<TValue>(string key) => new(new RedisKey(key));
	}

	/// <summary>
	/// Extension methods for strings
	/// </summary>
	public static class RedisStringKeyExtensions
	{
		#region Conditions

		/// <inheritdoc cref="Condition.StringEqual(RedisKey, RedisValue)"/>
		public static Condition StringEqual<TElement>(this RedisStringKey<TElement> key, TElement value)
			=> Condition.StringEqual(key.Inner, RedisSerializer.Serialize(value));

		/// <inheritdoc cref="Condition.StringLengthEqual(RedisKey, Int64)"/>
		public static Condition StringLengthEqual<TElement>(this RedisStringKey<TElement> key, long length)
			=> Condition.StringLengthEqual(key.Inner, length);

		/// <inheritdoc cref="Condition.StringLengthGreaterThan(RedisKey, Int64)"/>
		public static Condition StringLengthGreaterThan<TElement>(this RedisStringKey<TElement> key, long length)
			=> Condition.StringLengthGreaterThan(key.Inner, length);

		/// <inheritdoc cref="Condition.StringLengthLessThan(RedisKey, Int64)"/>
		public static Condition StringLengthLessThan<TElement>(this RedisStringKey<TElement> key, long length)
			=> Condition.StringLengthLessThan(key.Inner, length);

		/// <inheritdoc cref="Condition.StringNotEqual(RedisKey, RedisValue)"/>
		public static Condition StringNotEqual<TElement>(this RedisStringKey<TElement> key, TElement value)
			=> Condition.StringNotEqual(key.Inner, RedisSerializer.Serialize(value));

		#endregion

		#region StringDecrementAsync

		/// <inheritdoc cref="IDatabaseAsync.StringDecrementAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<long> StringDecrementAsync(this IDatabaseAsync target, RedisKey key, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.StringDecrementAsync(key, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringDecrementAsync(RedisKey, Int64, CommandFlags)"/>
		public static Task<long> StringDecrementAsync(this IDatabaseAsync target, RedisStringKey<long> key, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.StringDecrementAsync(key.Inner, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringDecrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<double> StringDecrementAsync(this IDatabaseAsync target, RedisStringKey<double> key, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			return target.StringDecrementAsync(key.Inner, value, flags);
		}

		#endregion

		#region StringGetAsync

		/// <inheritdoc cref="IDatabaseAsync.StringGetAsync(RedisKey, CommandFlags)"/>
		public static async Task<TValue?> StringGetAsync<TValue>(this IDatabaseAsync target, RedisStringKey<TValue> key, CommandFlags flags = CommandFlags.None)
		{
			RedisValue value = await target.StringGetAsync(key.Inner, flags);
			if (value.IsNull)
			{
				return default;
			}
			return RedisSerializer.Deserialize<TValue>(value)!;
		}

		/// <inheritdoc cref="IDatabaseAsync.StringGetAsync(RedisKey, CommandFlags)"/>
		public static async Task<TValue> StringGetAsync<TValue>(this IDatabaseAsync target, RedisStringKey<TValue> key, TValue defaultValue, CommandFlags flags = CommandFlags.None)
		{
			RedisValue value = await target.StringGetAsync(key.Inner, flags);
			if (value.IsNull)
			{
				return defaultValue;
			}
			return RedisSerializer.Deserialize<TValue>(value)!;
		}

		#endregion

		#region StringIncrementAsync

		/// <inheritdoc cref="IDatabaseAsync.StringIncrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<long> StringIncrementAsync(this IDatabaseAsync target, RedisKey key, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.StringIncrementAsync(key, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringIncrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<long> StringIncrementAsync(this IDatabaseAsync target, RedisStringKey<long> key, long value = 1L, CommandFlags flags = CommandFlags.None)
		{
			return target.StringIncrementAsync(key.Inner, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringIncrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<double> StringIncrementAsync(this IDatabaseAsync target, RedisKey key, double value, CommandFlags flags = CommandFlags.None)
		{
			return target.StringIncrementAsync(key, value, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringIncrementAsync(RedisKey, Double, CommandFlags)"/>
		public static Task<double> StringIncrementAsync(this IDatabaseAsync target, RedisStringKey<double> key, double value = 1.0, CommandFlags flags = CommandFlags.None)
		{
			return target.StringIncrementAsync(key.Inner, value, flags);
		}

		#endregion

		#region StringLengthAsync

		/// <inheritdoc cref="IDatabaseAsync.StringLengthAsync(RedisKey, CommandFlags)"/>
		public static Task<long> StringLengthAsync<TValue>(this IDatabaseAsync target, RedisStringKey<TValue> key, CommandFlags flags = CommandFlags.None)
		{
			return target.StringLengthAsync(key.Inner, flags);
		}

		#endregion

		#region StringSetAsync

		/// <inheritdoc cref="IDatabaseAsync.StringSetAsync(RedisKey, RedisValue, TimeSpan?, When, CommandFlags)"/>
		public static Task<bool> StringSetAsync<TValue>(this IDatabaseAsync target, RedisStringKey<TValue> key, TValue value, TimeSpan? expiry = null, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.StringSetAsync(key.Inner, RedisSerializer.Serialize(value), expiry, when, flags);
		}

		/// <inheritdoc cref="IDatabaseAsync.StringSetAsync(RedisKey, RedisValue, TimeSpan?, When, CommandFlags)"/>
		public static Task<bool> StringSetAsync<TValue>(this IDatabaseAsync target, KeyValuePair<RedisStringKey<TValue>, TValue>[] pairs, When when = When.Always, CommandFlags flags = CommandFlags.None)
		{
			return target.StringSetAsync(pairs.ConvertAll(x => new KeyValuePair<RedisKey, RedisValue>(x.Key.Inner, RedisSerializer.Serialize(x.Value))), when, flags);
		}

		#endregion
	}
}
