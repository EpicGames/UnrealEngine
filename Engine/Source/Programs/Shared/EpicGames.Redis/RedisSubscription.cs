// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Subscription to a <see cref="RedisChannel"/>
	/// </summary>
	public sealed class RedisSubscription : IDisposable, IAsyncDisposable
	{
		ISubscriber? _subscriber;
		readonly RedisChannel _channel;
		readonly Action<RedisChannel, RedisValue> _handler;

		/// <summary>
		/// Accessor for the channel
		/// </summary>
		public RedisChannel Channel => _channel;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="channel">The channel that is subscribed to</param>
		/// <param name="handler">Callback method</param>
		public RedisSubscription(RedisChannel channel, Action<RedisChannel, RedisValue> handler)
		{
			_channel = channel;
			_handler = handler;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_ = UnsubscribeAsync(CommandFlags.FireAndForget);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await UnsubscribeAsync();
		}

		/// <summary>
		/// Subscribes to the channel
		/// </summary>
		/// <param name="subscriber">Connection to use for the subscription</param>
		/// <param name="flags">Flags for the operation</param>
		public async Task SubscribeAsync(ISubscriber subscriber, CommandFlags flags = CommandFlags.None)
		{
			if (_subscriber != null)
			{
				throw new InvalidOperationException("Cannot subscribe to a connection while a subscription is already active");
			}

			_subscriber = subscriber;
			await subscriber.SubscribeAsync(_channel, _handler, flags);
		}

		/// <summary>
		/// Unsubscribe from the channel
		/// </summary>
		/// <param name="flags">Flags for the operation</param>
		public async Task UnsubscribeAsync(CommandFlags flags = CommandFlags.None)
		{
			if (_subscriber != null)
			{
				await _subscriber.UnsubscribeAsync(_channel, _handler, flags);
				_subscriber = null;
			}
		}
	}

	/// <summary>
	/// Extension methods for typed lists
	/// </summary>
	public static class RedisSubscriptionExtensions
	{
		#region SubscribeAsync

		/// <inheritdoc cref="ISubscriber.SubscribeAsync(RedisChannel, Action{RedisChannel, RedisValue}, CommandFlags)"/>
		public static Task<RedisSubscription> SubscribeAsync(this IConnectionMultiplexer connection, RedisChannel channel, Action<RedisValue> handler, CommandFlags flags = CommandFlags.None)
		{
			return SubscribeAsync(connection, channel, (_, v) => handler(v), flags);
		}

		/// <inheritdoc cref="ISubscriber.SubscribeAsync(RedisChannel, Action{RedisChannel, RedisValue}, CommandFlags)"/>
		public static async Task<RedisSubscription> SubscribeAsync(this IConnectionMultiplexer connection, RedisChannel channel, Action<RedisChannel, RedisValue> handler, CommandFlags flags = CommandFlags.None)
		{
			RedisSubscription subscription = new(channel, handler);
			await subscription.SubscribeAsync(connection.GetSubscriber(), flags);
			return subscription;
		}

		/// <inheritdoc cref="ISubscriber.SubscribeAsync(RedisChannel, Action{RedisChannel, RedisValue}, CommandFlags)"/>
		public static Task<RedisSubscription> SubscribeAsync<T>(this IConnectionMultiplexer connection, RedisChannel<T> channel, Action<T> handler, CommandFlags flags = CommandFlags.None)
		{
			void Action(RedisChannel _, RedisValue v) => handler(RedisSerializer.Deserialize<T>(v));
			return SubscribeAsync(connection, channel.Channel, Action, flags);
		}

		/// <inheritdoc cref="ISubscriber.SubscribeAsync(RedisChannel, Action{RedisChannel, RedisValue}, CommandFlags)"/>
		public static Task<RedisSubscription> SubscribeAsync<T>(this IConnectionMultiplexer connection, RedisChannel<T> channel, Action<RedisChannel<T>, T> handler, CommandFlags flags = CommandFlags.None)
		{
			void Action(RedisChannel _, RedisValue v) => handler(channel, RedisSerializer.Deserialize<T>(v));
			return SubscribeAsync(connection, channel.Channel, Action, flags);
		}

		#endregion
	}
}
