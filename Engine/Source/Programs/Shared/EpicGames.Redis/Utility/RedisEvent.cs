// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using StackExchange.Redis;

namespace EpicGames.Redis
{
	/// <summary>
	/// Distributed version of <see cref="AsyncEvent"/>. The event will be pulsed via a pub/sub channel in Redis.
	/// </summary>
	public sealed class RedisEvent : IAsyncDisposable
	{
		readonly IConnectionMultiplexer _multiplexer;
		readonly RedisChannel _channel;
		readonly AsyncEvent _asyncEvent;
		readonly RedisSubscription _subscription;

		/// <summary>
		/// Accessor for the inner task. Can be captured and awaited by clients.
		/// </summary>
		public Task Task => _asyncEvent.Task;

		RedisEvent(IConnectionMultiplexer multiplexer, RedisChannel channel, AsyncEvent asyncEvent, RedisSubscription subscription)
		{
			_multiplexer = multiplexer;
			_channel = channel;
			_asyncEvent = asyncEvent;
			_subscription = subscription;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
			=> _subscription.DisposeAsync();

		/// <summary>
		/// Create a new async event using the given channel name
		/// </summary>
		/// <param name="multiplexer">Multiplexer for the </param>
		/// <param name="channel">Channel for posting event updates</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<RedisEvent> CreateAsync(IConnectionMultiplexer multiplexer, RedisChannel channel, CancellationToken cancellationToken = default)
		{
			AsyncEvent asyncEvent = new();

			RedisSubscription subscription = await multiplexer.SubscribeAsync(channel, x => asyncEvent.Pulse());
			if (cancellationToken.IsCancellationRequested)
			{
				await subscription.DisposeAsync();
				cancellationToken.ThrowIfCancellationRequested();
			}

			return new RedisEvent(multiplexer, channel, asyncEvent, subscription);
		}

		/// <summary>
		/// Pulse the event, allowing any captured copies of the task to continue.
		/// </summary>
		public void Pulse()
		{
			_ = _multiplexer.GetDatabase().PublishAsync(_channel, RedisValue.EmptyString, CommandFlags.FireAndForget);
		}
	}
}
