// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis.Utility
{
	/// <summary>
	/// Implements a named single-entry lock which expires after a period of time if the process terminates.
	/// </summary>
	public class RedisLock : IAsyncDisposable, IDisposable
	{
		readonly IDatabase _database;
		readonly RedisKey _key;
		CancellationTokenSource? _cancellationSource;
		Task? _backgroundTask;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="database"></param>
		/// <param name="key"></param>
		public RedisLock(IDatabase database, RedisKey key)
		{
			_database = database;
			_key = key;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Dispose pattern
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				DisposeAsync().AsTask().Wait();
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_cancellationSource != null)
			{
				await _cancellationSource.CancelAsync();
			}
			if (_backgroundTask != null)
			{
				await _backgroundTask;
			}
			_backgroundTask?.Dispose();
			_cancellationSource?.Dispose();
			_backgroundTask = null;
			_cancellationSource = null;
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Attempts to acquire the lock for the given period of time. The lock will be renewed once half of this interval has elapsed.
		/// </summary>
		/// <param name="duration">Time after which the lock expires</param>
		/// <param name="releaseOnDispose">Whether the lock should be released when disposed. If false, the lock will be held for the given time, but not renewed once disposed.</param>
		/// <returns>True if the lock was acquired, false if another service already has it</returns>
		public async ValueTask<bool> AcquireAsync(TimeSpan duration, bool releaseOnDispose = true)
		{
			if (await _database.StringSetAsync(_key, RedisValue.EmptyString, duration, When.NotExists))
			{
				_cancellationSource = new CancellationTokenSource();
				_backgroundTask = Task.Run(() => RenewAsync(duration, releaseOnDispose, _cancellationSource.Token));
				return true;
			}
			return false;
		}

		/// <summary>
		/// Background task which renews the lock while the service is running
		/// </summary>
		/// <param name="duration"></param>
		/// <param name="releaseOnDispose">Whether the lock should be released when the cancellation token is fired</param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		async Task RenewAsync(TimeSpan duration, bool releaseOnDispose, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();
			for (; ; )
			{
				await Task.Delay(duration / 2, cancellationToken).ContinueWith(x => { }, CancellationToken.None, TaskContinuationOptions.None, TaskScheduler.Default); // Do not throw

				if (cancellationToken.IsCancellationRequested)
				{
					timer.Stop();
					if (releaseOnDispose || timer.Elapsed > duration)
					{
						await _database.StringSetAsync(_key, RedisValue.Null);
					}
					else
					{
						await _database.StringSetAsync(_key, RedisValue.EmptyString, duration - timer.Elapsed, When.Exists);
					}
					break;
				}
				if (!await _database.StringSetAsync(_key, RedisValue.EmptyString, duration, When.Exists))
				{
					break;
				}
			}
		}
	}

	/// <summary>
	/// Implements a named single-entry lock which expires after a period of time if the process terminates.
	/// </summary>
	/// <typeparam name="T">Type of the value identifying the lock uniqueness</typeparam>
	public class RedisLock<T> : RedisLock
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="database"></param>
		/// <param name="baseKey"></param>
		/// <param name="value"></param>
		public RedisLock(IDatabase database, RedisKey baseKey, T value)
			: base(database, baseKey.Append(RedisSerializer.Serialize<T>(value).AsKey()))
		{
		}
	}
}
