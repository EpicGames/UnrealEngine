// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using StackExchange.Redis;

namespace EpicGames.Redis.Utility
{
	/// <summary>
	/// Utility method for queue instances
	/// </summary>
	public static class RedisQueue
	{
		/// <summary>
		/// Create a new queue instance
		/// </summary>
		/// <param name="multiplexer">Multiplexer for the connection</param>
		/// <param name="queueKey">Key for the queue</param>
		/// <param name="eventChannel">Channel to use for posting update notifications</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task<RedisQueue<T>> CreateAsync<T>(IConnectionMultiplexer multiplexer, RedisKey queueKey, RedisChannel eventChannel, CancellationToken cancellationToken = default)
		{
			RedisEvent asyncEvent = await RedisEvent.CreateAsync(multiplexer, eventChannel, cancellationToken);
			return new RedisQueue<T>(multiplexer, asyncEvent, new RedisListKey<T>(queueKey));
		}
	}

	/// <summary>
	/// Implements a waitable FIFO queue with Redis. Wraps a pub/sub channel and list, ensuring that each item is only popped from the queue once.
	/// </summary>
	/// <typeparam name="T">Type of item in the queue</typeparam>
	public sealed class RedisQueue<T> : IAsyncDisposable
	{
		readonly IConnectionMultiplexer _multiplexer;
		readonly RedisEvent _asyncEvent;
		readonly RedisListKey<T> _listKey;

		internal RedisQueue(IConnectionMultiplexer multiplexer, RedisEvent asyncEvent, RedisListKey<T> listKey)
		{
			_multiplexer = multiplexer;
			_asyncEvent = asyncEvent;
			_listKey = listKey;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
			=> _asyncEvent.DisposeAsync();

		/// <summary>
		/// Push a new item onto the queue
		/// </summary>
		/// <param name="item">Item to add to the queue</param>
		/// <param name="flags">Flags for the push operation</param>
		public async Task PushAsync(T item, CommandFlags flags = CommandFlags.None)
		{
			IDatabase database = _multiplexer.GetDatabase();
			await database.ListRightPushAsync(_listKey, item, flags: flags);
			_asyncEvent.Pulse();
		}

		/// <summary>
		/// Attempt to pop an item from the front of the queue. Returns the default value for the item if the queue is empty.
		/// </summary>
		public async Task<T> TryPopAsync(CancellationToken cancellationToken = default)
		{
			_ = cancellationToken; // Don't currently support cancellation due to potential of items being dropped

			IDatabase database = _multiplexer.GetDatabase();
			return await database.ListLeftPopAsync(_listKey);
		}

		/// <summary>
		/// Wait for a new item to be pushed onto the queue
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task WaitForDataAsync(CancellationToken cancellationToken = default)
		{
			Task task = _asyncEvent.Task;

			IDatabase database = _multiplexer.GetDatabase();
			if (await database.ListLengthAsync(_listKey) == 0)
			{
				await task.WaitAsync(cancellationToken);
			}
		}
	}
}
