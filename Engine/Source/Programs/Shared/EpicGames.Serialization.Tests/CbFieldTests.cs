// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	[TestClass]
	public class CbFieldTests
	{
		static readonly IoHash s_testHash = IoHash.Parse("7b0142eafb4d0bed8265978281a1398099b1d727");

		[TestMethod]
		[DataRow(5)]
		[DataRow(22)]
		[DataRow(30)]
		public void CbBinaryAttachmentMultiSegment(int splitPoint)
		{
			CbBinaryAttachment expected = s_testHash;
			string json = $"\"{s_testHash}\"";
			byte[] bytes = Encoding.UTF8.GetBytes(json);
			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			CbBinaryAttachment result = JsonSerializer.Deserialize<CbBinaryAttachment>(ref reader);
			Assert.AreEqual(expected, result);
		}

		[TestMethod]
		[DataRow(5)]
		[DataRow(22)]
		[DataRow(30)]
		public void CbObjectAttachmentMultiSegment(int splitPoint)
		{
			CbObjectAttachment expected = s_testHash;
			string json = $"\"{s_testHash}\"";
			byte[] bytes = Encoding.UTF8.GetBytes(json);
			ReadOnlySequence<byte> sequence = ReadOnlySequence.Create<byte>([bytes.AsMemory(0, splitPoint), bytes.AsMemory(splitPoint)]);

			Utf8JsonReader reader = new(sequence);
			reader.Read();
			CbObjectAttachment result = JsonSerializer.Deserialize<CbObjectAttachment>(ref reader);
			Assert.AreEqual(expected, result);
		}
	}
}
