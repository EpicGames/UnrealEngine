// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	class TestObject
	{
		[CbField]
		public CbBinaryAttachment BinaryAttachment { get; set; }

		[CbField]
		public CbObjectAttachment ObjectAttachment { get; set; }
	}

	class SimpleObject
	{
		[CbField] public int A { get; set; } = 0;

		[CbField]
		public string B { get; set; } = String.Empty;
	}

	[TestClass]
	public class CbPackageTests
	{
		[TestMethod]
		public async Task RoundTripAsync()
		{
			TestObject root = new()
			{

			};
			CbObject rootObject = CbSerializer.Serialize(root);
			IoHash rootHash = IoHash.Compute(rootObject.GetView().Span);

			using CbPackageBuilder builder = new();
			builder.AddAttachment(rootHash, CbPackageAttachmentFlags.IsObject, rootObject.GetView().ToArray());
			byte[] bytes = await builder.ToByteArrayAsync();

			await using MemoryStream ms = new(bytes);
			CbPackageReader reader = await CbPackageReader.CreateAsync(ms);

			Assert.AreEqual(rootHash, reader.RootHash);
			Assert.AreEqual(rootObject, reader.RootObject);

			await foreach ((CbPackageAttachmentEntry, byte[]) _ in reader.IterateAttachmentsAsync())
			{
				Assert.Fail("No Attachments expected");
			}
		}

		[TestMethod]
		public async Task RoundTripComplexAsync()
		{
			byte[] blob = Encoding.UTF8.GetBytes("This is blob contents");
			IoHash blobHash = IoHash.Compute(blob);

			SimpleObject simple = new();
			CbObject simpleObject = CbSerializer.Serialize(simple);
			IoHash simpleHash = IoHash.Compute(simpleObject.GetView().Span);

			TestObject root = new()
			{
				BinaryAttachment = blobHash,
				ObjectAttachment = simpleHash,
			};
			CbObject rootObject = CbSerializer.Serialize(root);
			IoHash rootHash = IoHash.Compute(rootObject.GetView().Span);

			using CbPackageBuilder builder = new();
			builder.AddAttachment(rootHash, CbPackageAttachmentFlags.IsObject, rootObject.GetView().ToArray());
			builder.AddAttachment(simpleHash, CbPackageAttachmentFlags.IsObject, simpleObject.GetView().ToArray());
			builder.AddAttachment(blobHash, 0, blob);

			byte[] bytes = await builder.ToByteArrayAsync();

			await using MemoryStream ms = new(bytes);
			CbPackageReader reader = await CbPackageReader.CreateAsync(ms);

			Assert.AreEqual(rootHash, reader.RootHash);
			Assert.AreEqual(rootObject, reader.RootObject);

			int countOfAttachments = 0;
			await foreach ((CbPackageAttachmentEntry entry, byte[] attachmentBytes) in reader.IterateAttachmentsAsync())
			{
				countOfAttachments++;
				if (entry.AttachmentHash == blobHash)
				{
					Assert.AreEqual((ulong)blob.Length, entry.PayloadSize);
					CollectionAssert.AreEqual(blob, attachmentBytes);
					continue;
				}

				if (entry.AttachmentHash == simpleHash)
				{
					Assert.AreEqual((ulong)simpleObject.GetView().Length, entry.PayloadSize);

					CbObject roundTrippedObject = new(attachmentBytes);
					Assert.AreEqual(simpleObject, roundTrippedObject);
					continue;
				}

				Assert.Fail($"Unknown attachment {entry.AttachmentHash}");
			}
			Assert.AreEqual(2, countOfAttachments);
		}

		[TestMethod]
		public async Task ReadNoAttachmentsAsync()
		{
			using CbPackageBuilder packageBuilder = new();
			SimpleObject simple = new();
			CbObject simpleObject = CbSerializer.Serialize(simple);
			IoHash simpleHash = IoHash.Compute(simpleObject.GetView().Span);
			packageBuilder.AddAttachment(simpleHash, CbPackageAttachmentFlags.IsObject, simpleObject.GetView().ToArray());

			byte[] buf = await packageBuilder.ToByteArrayAsync();
			await using MemoryStream ms = new(buf);

			try
			{
				await CbPackageReader.CreateAsync(ms);
				Assert.Fail("Exception should be thrown");
			}
			catch (Exception)
			{
			}
		}
	}
}
