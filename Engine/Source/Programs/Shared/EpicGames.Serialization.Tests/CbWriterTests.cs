// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	[TestClass]
	public class CbWriterTests
	{
		[TestMethod]
		public void EmptyObject()
		{
			CbWriter writer = new();
			writer.BeginObject();
			writer.EndObject();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(2, data.Length);
			Assert.AreEqual((byte)CbFieldType.Object, data[0]);
			Assert.AreEqual(0, data[1]);
		}

		[TestMethod]
		public void EmptyArray()
		{
			CbWriter writer = new();
			writer.BeginArray();
			writer.EndArray();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(3, data.Length);
			Assert.AreEqual((byte)CbFieldType.Array, data[0]);
			Assert.AreEqual(1, data[1]);
			Assert.AreEqual(0, data[2]);
		}

		[TestMethod]
		public void Object()
		{
			CbWriter writer = new();
			writer.BeginObject();
			writer.WriteInteger("a", 1);
			writer.WriteInteger("b", 2);
			writer.WriteInteger("c", 3);
			writer.EndArray();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(14, data.Length);
			Assert.AreEqual((byte)CbFieldType.Object, data[0]);
			Assert.AreEqual(12, data[1]); // Length

			Assert.AreEqual((byte)(CbFieldType.IntegerPositive | CbFieldType.HasFieldName), data[2]);
			Assert.AreEqual(1, data[3]);
			Assert.AreEqual((byte)'a', data[4]);
			Assert.AreEqual(1, data[5]);

			Assert.AreEqual((byte)(CbFieldType.IntegerPositive | CbFieldType.HasFieldName), data[6]);
			Assert.AreEqual(1, data[7]);
			Assert.AreEqual((byte)'b', data[8]);
			Assert.AreEqual(2, data[9]);

			Assert.AreEqual((byte)(CbFieldType.IntegerPositive | CbFieldType.HasFieldName), data[10]);
			Assert.AreEqual(1, data[11]);
			Assert.AreEqual((byte)'c', data[12]);
			Assert.AreEqual(3, data[13]);
		}

		[TestMethod]
		public void Array()
		{
			CbWriter writer = new();
			writer.BeginArray();
			writer.WriteIntegerValue(1);
			writer.WriteIntegerValue(2);
			writer.WriteIntegerValue(3);
			writer.EndArray();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(9, data.Length);
			Assert.AreEqual((byte)CbFieldType.Array, data[0]);
			Assert.AreEqual(7, data[1]); // Length
			Assert.AreEqual(3, data[2]); // Item count
			Assert.AreEqual((byte)CbFieldType.IntegerPositive, data[3]);
			Assert.AreEqual(1, data[4]);
			Assert.AreEqual((byte)CbFieldType.IntegerPositive, data[5]);
			Assert.AreEqual(2, data[6]);
			Assert.AreEqual((byte)CbFieldType.IntegerPositive, data[7]);
			Assert.AreEqual(3, data[8]);
		}

		[TestMethod]
		public void UniformArray()
		{
			CbWriter writer = new();
			writer.BeginUniformArray(CbFieldType.IntegerPositive);
			writer.WriteIntegerValue(1);
			writer.WriteIntegerValue(2);
			writer.WriteIntegerValue(3);
			writer.EndArray();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(7, data.Length);
			Assert.AreEqual((byte)CbFieldType.UniformArray, data[0]);
			Assert.AreEqual(5, data[1]); // Length
			Assert.AreEqual(3, data[2]); // Item count
			Assert.AreEqual((byte)CbFieldType.IntegerPositive, data[3]);
			Assert.AreEqual(1, data[4]);
			Assert.AreEqual(2, data[5]);
			Assert.AreEqual(3, data[6]);
		}

		[TestMethod]
		public void NestedArray()
		{
			CbWriter writer = new();
			writer.BeginObject();
			writer.BeginArray("a");
			writer.WriteIntegerValue(1);
			writer.EndArray();
			writer.EndObject();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(9, data.Length);
			Assert.AreEqual((byte)CbFieldType.Object, data[0]);
			Assert.AreEqual(7, data[1]); // Length

			Assert.AreEqual((byte)(CbFieldType.Array | CbFieldType.HasFieldName), data[2]);
			Assert.AreEqual(1, data[3]); // Name length
			Assert.AreEqual((byte)'a', data[4]); // Name
			Assert.AreEqual(3, data[5]); // Length
			Assert.AreEqual(1, data[6]); // Item count

			Assert.AreEqual((byte)CbFieldType.IntegerPositive, data[7]);
			Assert.AreEqual(1, data[8]);
		}

		[TestMethod]
		public void RawData()
		{
			byte[] test = [1, 2, 3, 4];

			CbWriter writer = new();
			writer.BeginObject();
			writer.WriteBinaryReference("a", test);
			writer.EndObject();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(10, data.Length);
			Assert.AreEqual((byte)CbFieldType.Object, data[0]);
			Assert.AreEqual(8, data[1]); // Length

			Assert.AreEqual((byte)(CbFieldType.Binary | CbFieldType.HasFieldName), data[2]);
			Assert.AreEqual(1, data[3]); // Name length
			Assert.AreEqual((byte)'a', data[4]); // Name
			Assert.AreEqual(4, data[5]); // Length

			Assert.AreEqual(1, data[6]);
			Assert.AreEqual(2, data[7]);
			Assert.AreEqual(3, data[8]);
			Assert.AreEqual(4, data[9]);
		}

		[TestMethod]
		public void UniformArrayOfArrays()
		{
			CbWriter writer = new();
			writer.BeginUniformArray(CbFieldType.Array);
			writer.BeginArray();
			writer.WriteNullValue();
			writer.EndArray();
			writer.EndUniformArray();

			byte[] data = writer.ToByteArray();
			Assert.AreEqual(7, data.Length);
			Assert.AreEqual((byte)CbFieldType.UniformArray, data[0]);
			Assert.AreEqual(5, data[1]); // Length
			Assert.AreEqual(1, data[2]); // Item Count

			Assert.AreEqual((byte)CbFieldType.Array, data[3]);
			Assert.AreEqual(2, data[4]); // Length
			Assert.AreEqual(1, data[5]); // Item Count

			Assert.AreEqual((byte)CbFieldType.Null, data[6]);
		}

		class TestObj
		{
			[CbField("obj")]
			public List<CbObjectAttachment> ObjectAttachments { get; set; } = [];

			[CbField("bin")]
			public List<CbBinaryAttachment> BinaryAttachments { get; set; } = [];
		}

		[TestMethod]
		public void Attachments()
		{
			List<IoHash> objHashes = [.. Enumerable.Range(0, 5).Select(x => IoHash.Compute(Encoding.UTF8.GetBytes($"obj{x}")))];
			List<IoHash> binHashes = [.. Enumerable.Range(0, 5).Select(x => IoHash.Compute(Encoding.UTF8.GetBytes($"bin{x}")))];

			CbWriter writer = new();
			writer.BeginObject();
			writer.BeginUniformArray("obj", CbFieldType.ObjectAttachment);
			for (int idx = 0; idx < 5; idx++)
			{
				writer.WriteObjectAttachmentValue(objHashes[idx]);
			}
			writer.EndUniformArray();
			writer.BeginArray("bin");
			for (int idx = 0; idx < 5; idx++)
			{
				writer.WriteBinaryAttachmentValue(binHashes[idx]);
			}
			writer.EndArray();
			writer.EndObject();

			byte[] data = writer.ToByteArray();

			TestObj obj = CbSerializer.Deserialize<TestObj>(data);
			Assert.IsTrue(obj.ObjectAttachments.Select(x => x.Hash).SequenceEqual(objHashes));
			Assert.IsTrue(obj.BinaryAttachments.Select(x => x.Hash).SequenceEqual(binHashes));
		}

		[TestMethod]
		public void EmbeddedObject()
		{
			// Create an empty CbObject
			CbObject obj;
			{
				CbWriter writer = new();
				writer.BeginObject();
				writer.EndObject();
				obj = writer.ToObject();
			}

			// Embed it in another writer at the root
			{
				CbWriter writer = new();
				writer.WriteObject(obj);

				byte[] data = writer.ToByteArray();
				Assert.AreEqual(2, data.Length);
				Assert.AreEqual((byte)CbFieldType.Object, data[0]);
				Assert.AreEqual(0, data[1]);
			}
		}
	}
}
