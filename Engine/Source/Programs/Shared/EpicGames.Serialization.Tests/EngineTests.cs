// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	[TestClass]
	public class EngineTests
	{
		class CbFieldAccessors(object defaultValue, Func<CbField, bool> isType, Func<CbField, object> asType, Func<CbField, object, object> asTypeWithDefault)
		{
			public object _defaultValue = defaultValue;
			public Func<CbField, bool> _isType = isType;
			public Func<CbField, object> _asType = asType;
			public Func<CbField, object, object> _asTypeWithDefault = asTypeWithDefault;
			public Func<object, object, bool> _comparer = (x, y) => x.Equals(y);

			public static CbFieldAccessors FromStruct<T>(Func<CbField, bool> isType, Func<CbField, T, object> asTypeWithDefault) where T : struct
			{
				return new CbFieldAccessors(new T(), isType, x => asTypeWithDefault(x, new T()), (x, y) => asTypeWithDefault(x, (T)y));
			}

			public static CbFieldAccessors FromStruct<T>(T @default, Func<CbField, bool> isType, Func<CbField, T, object> asTypeWithDefault) where T : struct
			{
				return new CbFieldAccessors(@default, isType, x => asTypeWithDefault(x, new T()), (x, y) => asTypeWithDefault(x, (T)y));
			}

			public static CbFieldAccessors FromStruct<T>(Func<CbField, bool> isType, Func<CbField, T, object> asTypeWithDefault, Func<T, T, bool> comparer) where T : struct
			{
				return new CbFieldAccessors(new T(), isType, x => asTypeWithDefault(x, new T()), (x, y) => asTypeWithDefault(x, (T)y)) { _comparer = (x, y) => comparer((T)x, (T)y) };
			}
		}

		readonly Dictionary<CbFieldType, CbFieldAccessors> _typeAccessors = new()
		{
			[CbFieldType.Object] = new CbFieldAccessors(CbObject.Empty, x => x.IsObject(), x => x.AsObject(), (x, y) => x.AsObject()),
			[CbFieldType.UniformObject] = new CbFieldAccessors(CbObject.Empty, x => x.IsObject(), x => x.AsObject(), (x, y) => x.AsObject()),
			[CbFieldType.Array] = new CbFieldAccessors(CbArray.Empty, x => x.IsArray(), x => x.AsArray(), (x, y) => x.AsArray()),
			[CbFieldType.UniformArray] = new CbFieldAccessors(CbArray.Empty, x => x.IsArray(), x => x.AsArray(), (x, y) => x.AsArray()),
			[CbFieldType.Binary] = CbFieldAccessors.FromStruct<ReadOnlyMemory<byte>>(x => x.IsBinary(), (x, y) => x.AsBinary(y), (x, y) => x.Span.SequenceEqual(y.Span)),
			[CbFieldType.String] = new CbFieldAccessors(Utf8String.Empty, x => x.IsString(), x => x.AsUtf8String(), (x, @default) => x.AsUtf8String((Utf8String)@default)),
			[CbFieldType.IntegerPositive] = CbFieldAccessors.FromStruct<ulong>(x => x.IsInteger(), (x, y) => x.AsUInt64(y)),
			[CbFieldType.IntegerNegative] = CbFieldAccessors.FromStruct<long>(x => x.IsInteger(), (x, y) => x.AsInt64(y)),
			[CbFieldType.Float32] = CbFieldAccessors.FromStruct<float>(x => x.IsFloat(), (x, y) => x.AsFloat(y)),
			[CbFieldType.Float64] = CbFieldAccessors.FromStruct<double>(x => x.IsFloat(), (x, y) => x.AsDouble(y)),
			[CbFieldType.BoolTrue] = CbFieldAccessors.FromStruct<bool>(x => x.IsBool(), (x, y) => x.AsBool(y)),
			[CbFieldType.BoolFalse] = CbFieldAccessors.FromStruct<bool>(x => x.IsBool(), (x, y) => x.AsBool(y)),
			[CbFieldType.ObjectAttachment] = new CbFieldAccessors(new CbObjectAttachment(IoHash.Zero), x => x.IsObjectAttachment(), x => x.AsObjectAttachment(), (x, @default) => x.AsObjectAttachment((CbObjectAttachment)@default)),
			[CbFieldType.BinaryAttachment] = new CbFieldAccessors(new CbBinaryAttachment(IoHash.Zero), x => x.IsBinaryAttachment(), x => x.AsBinaryAttachment(), (x, @default) => x.AsBinaryAttachment((CbBinaryAttachment)@default)),
			[CbFieldType.Hash] = new CbFieldAccessors(IoHash.Zero, x => x.IsHash(), x => x.AsHash(), (x, @default) => x.AsHash((IoHash)@default)),
			[CbFieldType.Uuid] = CbFieldAccessors.FromStruct<Guid>(x => x.IsUuid(), (x, y) => x.AsUuid(y)),
			[CbFieldType.DateTime] = CbFieldAccessors.FromStruct<DateTime>(new DateTime(0, DateTimeKind.Utc), x => x.IsDateTime(), (x, y) => x.AsDateTime(y)),
			[CbFieldType.TimeSpan] = CbFieldAccessors.FromStruct<TimeSpan>(x => x.IsTimeSpan(), (x, y) => x.AsTimeSpan(y)),
		};

		void TestField(CbFieldType fieldType, CbField field, object? expectedValue = null, object? defaultValue = null, CbFieldError expectedError = CbFieldError.None, CbFieldAccessors? accessors = null)
		{
			accessors ??= _typeAccessors[fieldType];
			expectedValue ??= accessors._defaultValue;
			defaultValue ??= accessors._defaultValue;

			Assert.AreEqual(accessors._isType(field), expectedError != CbFieldError.TypeError);
			if (expectedError == CbFieldError.None && !field.IsBool())
			{
				Assert.IsFalse(field.AsBool());
				Assert.IsTrue(field.HasError());
				Assert.AreEqual(CbFieldError.TypeError, field.GetError());
			}

			object value = accessors._asTypeWithDefault(field, defaultValue);
			Assert.IsTrue(accessors._comparer(accessors._asTypeWithDefault(field, defaultValue), expectedValue));
			Assert.AreEqual(field.HasError(), expectedError != CbFieldError.None);
			Assert.AreEqual(expectedError, field.GetError());
		}

		void TestField(CbFieldType fieldType, byte[] payload, object? expectedValue = null, object? defaultValue = null, CbFieldError expectedError = CbFieldError.None, CbFieldAccessors? accessors = null)
		{
			CbField field = new(payload, fieldType);
			Assert.AreEqual(field.GetSize(), payload.Length + (CbFieldUtils.HasFieldType(fieldType) ? 0 : 1));
			Assert.IsTrue(field.HasValue());
			Assert.IsFalse(field.HasError());
			Assert.AreEqual(CbFieldError.None, field.GetError());
			TestField(fieldType, field, expectedValue, defaultValue, expectedError, accessors);
		}

		void TestFieldError(CbFieldType fieldType, CbField field, CbFieldError expectedError, object? expectedValue = null, CbFieldAccessors? accessors = null)
		{
			TestField(fieldType, field, expectedValue, expectedValue, expectedError, accessors);
		}

		void TestFieldError(CbFieldType fieldType, ReadOnlyMemory<byte> payload, CbFieldError expectedError, object? expectedValue = null, CbFieldAccessors? accessors = null)
		{
			CbField field = new(payload, fieldType);
			TestFieldError(fieldType, field, expectedError, expectedValue, accessors);
		}

		[TestMethod]
		public void CbFieldNoneTest()
		{
			// Test CbField()
			{
				CbField defaultField = new();
				Assert.IsFalse(defaultField.HasName());
				Assert.IsFalse(defaultField.HasValue());
				Assert.IsFalse(defaultField.HasError());
				Assert.IsTrue(defaultField.GetError() == CbFieldError.None);
				Assert.AreEqual(1, defaultField.GetSize());
				Assert.AreEqual(0, defaultField.GetName().Length);
				Assert.IsFalse(defaultField.HasName());
				Assert.IsFalse(defaultField.HasValue());
				Assert.IsFalse(defaultField.HasError());
				Assert.AreEqual(CbFieldError.None, defaultField.GetError());
				Assert.AreEqual(defaultField.GetHash(), Blake3Hash.Compute([(byte)CbFieldType.None]));
				Assert.IsFalse(defaultField.TryGetView(out _));
			}

			// Test CbField(None)
			{
				CbField noneField = new(ReadOnlyMemory<byte>.Empty, CbFieldType.None);
				Assert.AreEqual(1, noneField.GetSize());
				Assert.AreEqual(0, noneField.GetName().Length);
				Assert.IsFalse(noneField.HasName());
				Assert.IsFalse(noneField.HasValue());
				Assert.IsFalse(noneField.HasError());
				Assert.AreEqual(CbFieldError.None, noneField.GetError());
				Assert.AreEqual(noneField.GetHash(), new CbField().GetHash());
				Assert.IsFalse(noneField.TryGetView(out _));
			}

			// Test CbField(None|Type|Name)
			{
				CbFieldType fieldType = CbFieldType.None | CbFieldType.HasFieldName;
				byte[] noneBytes = [(byte)fieldType, 4, (byte)'N', (byte)'a', (byte)'m', (byte)'e'];
				CbField noneField = new(noneBytes);
				Assert.AreEqual(noneField.GetSize(), noneBytes.Length);
				Assert.AreEqual(noneField.GetName(), new Utf8String("Name"));
				Assert.IsTrue(noneField.HasName());
				Assert.IsFalse(noneField.HasValue());
				Assert.AreEqual(noneField.GetHash(), Blake3Hash.Compute(noneBytes));
				ReadOnlyMemory<byte> view;
				Assert.IsTrue(noneField.TryGetView(out view) && view.Span.SequenceEqual(noneBytes));

				byte[] copyBytes = new byte[noneBytes.Length];
				noneField.CopyTo(copyBytes);
				Assert.IsTrue(noneBytes.AsSpan().SequenceEqual(copyBytes));
			}

			// Test CbField(None|Type)
			{
				CbFieldType fieldType = CbFieldType.None;
				byte[] noneBytes = [(byte)fieldType];
				CbField noneField = new(noneBytes);
				Assert.AreEqual(noneField.GetSize(), noneBytes.Length);
				Assert.AreEqual(0, noneField.GetName().Length);
				Assert.IsFalse(noneField.HasName());
				Assert.IsFalse(noneField.HasValue());
				Assert.AreEqual(noneField.GetHash(), new CbField().GetHash());
				ReadOnlyMemory<byte> view;
				Assert.IsTrue(noneField.TryGetView(out view) && view.Span.SequenceEqual(noneBytes));
			}

			// Test CbField(None|Name)
			{
				CbFieldType fieldType = CbFieldType.None | CbFieldType.HasFieldName;
				byte[] noneBytes = [(byte)fieldType, 4, (byte)'N', (byte)'a', (byte)'m', (byte)'e'];
				CbField noneField = new(noneBytes.AsMemory(1), fieldType);
				Assert.AreEqual(noneField.GetSize(), noneBytes.Length);
				Assert.AreEqual(noneField.GetName(), new Utf8String("Name"));
				Assert.IsTrue(noneField.HasName());
				Assert.IsFalse(noneField.HasValue());
				Assert.AreEqual(noneField.GetHash(), Blake3Hash.Compute(noneBytes));
				Assert.IsFalse(noneField.TryGetView(out _));

				byte[] copyBytes = new byte[noneBytes.Length];
				noneField.CopyTo(copyBytes);
				Assert.IsTrue(noneBytes.AsSpan().SequenceEqual(copyBytes));
			}

			// Test CbField(None|EmptyName)
			{
				CbFieldType fieldType = CbFieldType.None | CbFieldType.HasFieldName;
				byte[] noneBytes = [(byte)fieldType, 0];
				CbField noneField = new(noneBytes.AsMemory(1), fieldType);
				Assert.AreEqual(noneBytes.Length, noneField.GetSize());
				Assert.AreEqual(Utf8String.Empty, noneField.GetName());
				Assert.IsTrue(noneField.HasName());
				Assert.IsFalse(noneField.HasValue());
				Assert.AreEqual(noneField.GetHash(), Blake3Hash.Compute(noneBytes));
				Assert.IsFalse(noneField.TryGetView(out _));
			}
		}

		[TestMethod]
		public void CbFieldNullTest()
		{
			// Test CbField(Null)
			{
				CbField nullField = new(ReadOnlyMemory<byte>.Empty, CbFieldType.Null);
				Assert.AreEqual(1, nullField.GetSize());
				Assert.IsTrue(nullField.IsNull());
				Assert.IsTrue(nullField.HasValue());
				Assert.IsFalse(nullField.HasError());
				Assert.AreEqual(CbFieldError.None, nullField.GetError());
				Assert.AreEqual(nullField.GetHash(), Blake3Hash.Compute([(byte)CbFieldType.Null]));
			}

			// Test CbField(None) as Null
			{
				CbField field = new();
				Assert.IsFalse(field.IsNull());
			}
		}

		[TestMethod]
		public void CbFieldObjectTest()
		{
			static void TestIntObject(CbObject obj, int expectedNum, int expectedPayloadSize)
			{
				Assert.AreEqual(obj.GetSize(), expectedPayloadSize + sizeof(CbFieldType));

				int actualNum = 0;
				foreach (CbField field in obj)
				{
					++actualNum;
					Assert.AreNotEqual(0, field.GetName().Length);
					Assert.AreEqual(field.AsInt32(), actualNum);
				}
				Assert.AreEqual(expectedNum, actualNum);
			}

			// Test CbField(Object, Empty)
			TestField(CbFieldType.Object, new byte[1]);

			// Test CbField(Object, Empty)
			{
				CbObject obj = CbObject.Empty;
				TestIntObject(obj, 0, 1);

				// Find fields that do not exist.
				Assert.IsFalse(obj.Find("Field").HasValue());
				Assert.IsFalse(obj.FindIgnoreCase("Field").HasValue());
				Assert.IsFalse(obj["Field"].HasValue());

				// Advance an iterator past the last field.
				CbFieldIterator it = obj.CreateIterator();
				Assert.IsFalse((bool)it);
				Assert.IsTrue(!it);
				for (int count = 16; count > 0; --count)
				{
					++it;
					it.Current.AsInt32();
				}
				Assert.IsFalse((bool)it);
				Assert.IsTrue(!it);
			}

			// Test CbField(Object, NotEmpty)
			{
				byte intType = (byte)(CbFieldType.HasFieldName | CbFieldType.IntegerPositive);
				byte[] payload = [12, intType, 1, (byte)'A', 1, intType, 1, (byte)'B', 2, intType, 1, (byte)'C', 3];
				CbField field = new(payload, CbFieldType.Object);
				TestField(CbFieldType.Object, field, new CbObject(payload, CbFieldType.Object));
				CbObject @object = CbObject.Clone(field.AsObject());
				TestIntObject(@object, 3, payload.Length);
				TestIntObject(field.AsObject(), 3, payload.Length);
				Assert.IsTrue(@object.Equals(field.AsObject()));
				Assert.AreEqual(2, @object.Find("B").AsInt32());
				Assert.AreEqual(4, @object.Find("b").AsInt32(4));
				Assert.AreEqual(2, @object.FindIgnoreCase("B").AsInt32());
				Assert.AreEqual(2, @object.FindIgnoreCase("b").AsInt32());
				Assert.AreEqual(2, @object["B"].AsInt32());
				Assert.AreEqual(4, @object["b"].AsInt32(4));
			}

			// Test CbField(UniformObject, NotEmpty)
			{
				byte intType = (byte)(CbFieldType.HasFieldName | CbFieldType.IntegerPositive);
				byte[] payload = [10, intType, 1, (byte)'A', 1, 1, (byte)'B', 2, 1, (byte)'C', 3];
				CbField field = new(payload, CbFieldType.UniformObject);
				TestField(CbFieldType.UniformObject, field, new CbObject(payload, CbFieldType.UniformObject));
				CbObject @object = CbObject.Clone(field.AsObject());
				TestIntObject(@object, 3, payload.Length);
				TestIntObject(field.AsObject(), 3, payload.Length);
				Assert.IsTrue(@object.Equals(field.AsObject()));
				Assert.AreEqual(2, @object.Find("B").AsInt32());
				Assert.AreEqual(2, @object.Find("B").AsInt32());
				Assert.AreEqual(4, @object.Find("b").AsInt32(4));
				Assert.AreEqual(4, @object.Find("b").AsInt32(4));
				Assert.AreEqual(2, @object.FindIgnoreCase("B").AsInt32());
				Assert.AreEqual(2, @object.FindIgnoreCase("B").AsInt32());
				Assert.AreEqual(2, @object.FindIgnoreCase("b").AsInt32());
				Assert.AreEqual(2, @object.FindIgnoreCase("b").AsInt32());
				Assert.AreEqual(2, @object["B"].AsInt32());
				Assert.AreEqual(4, @object["b"].AsInt32(4));

				// Equals
				byte[] namedPayload = [1, (byte)'O', 10, intType, 1, (byte)'A', 1, 1, (byte)'B', 2, 1, (byte)'C', 3];
				CbField namedField = new(namedPayload, CbFieldType.UniformObject | CbFieldType.HasFieldName);
				Assert.IsTrue(field.AsObject().Equals(namedField.AsObject()));

				// CopyTo
				byte[] copyBytes = new byte[payload.Length + 1];
				field.AsObject().CopyTo(copyBytes);
				Assert.IsTrue(payload.AsSpan().SequenceEqual(copyBytes.AsSpan(1)));
				namedField.AsObject().CopyTo(copyBytes);
				Assert.IsTrue(payload.AsSpan().SequenceEqual(copyBytes.AsSpan(1)));

				// TryGetView
				Assert.IsFalse(field.AsObject().TryGetView(out _));
				Assert.IsFalse(namedField.AsObject().TryGetView(out _));
			}

			// Test CbField(None) as Object
			{
				CbField field = CbField.Empty;
				TestFieldError(CbFieldType.Object, field, CbFieldError.TypeError);
				field.AsObject();
			}

			// Test FCbObjectView(ObjectWithName) and CreateIterator
			{
				byte objectType = (byte)(CbFieldType.Object | CbFieldType.HasFieldName);
				byte[] buffer = [objectType, 3, (byte)'K', (byte)'e', (byte)'y', 4, (byte)(CbFieldType.HasFieldName | CbFieldType.IntegerPositive), 1, (byte)'F', 8];
				CbObject @object = new(buffer);
				Assert.AreEqual(6, @object.GetSize());
				CbObject objectClone = CbObject.Clone(@object);
				Assert.AreEqual(6, objectClone.GetSize());
				Assert.IsTrue(@object.Equals(objectClone));
				Assert.AreEqual(objectClone.GetHash(), @object.GetHash());
				for (CbFieldIterator it = objectClone.CreateIterator(); it; ++it)
				{
					CbField field = it.Current;
					Assert.AreEqual(field.GetName(), new Utf8String("F"));
					Assert.AreEqual(8, field.AsInt32());
				}
				for (CbFieldIterator it = objectClone.CreateIterator(), end = new(); it != end; ++it)
				{
				}
				foreach (CbField _ in objectClone)
				{
				}
			}

			// Test FCbObjectView as CbFieldIterator
			{
				int count = 0;
				CbObject @object = CbObject.Empty;
				for (CbFieldIterator it = CbFieldIterator.MakeSingle(@object.AsField()); it; ++it)
				{
					CbField field = it.Current;
					Assert.IsTrue(field.IsObject());
					++count;
				}
				Assert.AreEqual(1, count);
			}
		}

		public void CbFieldArrayTest()
		{
			static void TestIntArray(CbArray array, int expectedNum, int expectedPayloadSize)
			{
				Assert.AreEqual(array.GetSize(), expectedPayloadSize + sizeof(CbFieldType));
				Assert.AreEqual(expectedNum, array.Count);

				int actualNum = 0;
				for (CbFieldIterator it = array.CreateIterator(); it; ++it)
				{
					++actualNum;
					Assert.AreEqual(it.Current.AsInt32(), actualNum);
				}
				Assert.AreEqual(expectedNum, actualNum);

				actualNum = 0;
				foreach (CbField field in array)
				{
					++actualNum;
					Assert.AreEqual(field.AsInt32(), actualNum);
				}
				Assert.AreEqual(expectedNum, actualNum);

				actualNum = 0;
				foreach (CbField field in array.AsField())
				{
					++actualNum;
					Assert.AreEqual(field.AsInt32(), actualNum);
				}
				Assert.AreEqual(expectedNum, actualNum);
			}

			// Test CbField(Array, Empty)
			TestField(CbFieldType.Array, [1, 0]);

			// Test CbField(Array, Empty)
			{
				CbArray array = new();
				TestIntArray(array, 0, 2);

				// Advance an iterator past the last field.
				CbFieldIterator it = array.CreateIterator();
				Assert.IsFalse((bool)it);
				Assert.IsTrue(!it);
				for (int count = 16; count > 0; --count)
				{
					++it;
					it.Current.AsInt32();
				}
				Assert.IsFalse((bool)it);
				Assert.IsTrue(!it);
			}

			// Test CbField(Array, NotEmpty)
			{
				byte intType = (byte)CbFieldType.IntegerPositive;
				byte[] payload = [7, 3, intType, 1, intType, 2, intType, 3];
				CbField field = new(payload, CbFieldType.Array);
				TestField(CbFieldType.Array, field, new CbArray(payload, CbFieldType.Array));
				CbArray array = field.AsArray();
				TestIntArray(array, 3, payload.Length);
				TestIntArray(field.AsArray(), 3, payload.Length);
				Assert.IsTrue(array.Equals(field.AsArray()));
			}

			// Test CbField(UniformArray)
			{
				byte intType = (byte)(CbFieldType.IntegerPositive);
				byte[] payload = [5, 3, intType, 1, 2, 3];
				CbField field = new(payload, CbFieldType.UniformArray);
				TestField(CbFieldType.UniformArray, field, new CbArray(payload, CbFieldType.UniformArray));
				CbArray array = field.AsArray();
				TestIntArray(array, 3, payload.Length);
				TestIntArray(field.AsArray(), 3, payload.Length);
				Assert.IsTrue(array.Equals(field.AsArray()));

				//				Assert.IsTrue(Array.GetOuterBuffer() == Array.AsField().AsArray().GetOuterBuffer());

				// Equals
				byte[] namedPayload = [1, (byte)'A', 5, 3, intType, 1, 2, 3];
				CbField namedField = new(namedPayload, CbFieldType.UniformArray | CbFieldType.HasFieldName);
				Assert.IsTrue(field.AsArray().Equals(namedField.AsArray()));
				Assert.IsTrue(field.Equals(field.AsArray().AsField()));
				Assert.IsTrue(namedField.Equals(namedField.AsArray().AsField()));

				// CopyTo
				byte[] copyBytes = new byte[payload.Length + 1];
				field.AsArray().CopyTo(copyBytes);
				Assert.IsTrue(payload.AsSpan().SequenceEqual(copyBytes.AsSpan(1)));
				namedField.AsArray().CopyTo(copyBytes);
				Assert.IsTrue(payload.AsSpan().SequenceEqual(copyBytes.AsSpan(1)));

				// TryGetView
				//				Assert.IsTrue(Array.TryGetView(out View) && View == Array.GetOuterBuffer().GetView());
				Assert.IsFalse(field.AsArray().TryGetView(out _));
				Assert.IsFalse(namedField.AsArray().TryGetView(out _));

				//				// GetBuffer
				//				Assert.IsTrue(Array.GetBuffer().Flatten().GetView() == Array.GetOuterBuffer().GetView());
				//				Assert.IsTrue(CbField.MakeView(Field).AsArray().GetBuffer().Flatten().GetView().Span.SequenceEqual(Array.GetOuterBuffer().GetView()));
				//				Assert.IsTrue(CbField.MakeView(NamedField).AsArray().GetBuffer().Flatten().GetView().Span.SequenceEqual(Array.GetOuterBuffer().GetView()));
			}

			// Test CbField(None) as Array
			{
				CbField field = new();
				//				TestFieldError(CbFieldType.Array, Field, CbFieldError.TypeError);
				field.AsArray();
			}

			// Test CbArray(ArrayWithName) and CreateIterator
			{
				byte arrayType = (byte)(CbFieldType.Array | CbFieldType.HasFieldName);
				byte[] buffer = [arrayType, 3, (byte)'K', (byte)'e', (byte)'y', 3, 1, (byte)(CbFieldType.IntegerPositive), 8];
				CbArray array = new(buffer);
				Assert.AreEqual(5, array.GetSize());
				CbArray arrayClone = array;
				Assert.AreEqual(5, arrayClone.GetSize());
				Assert.IsTrue(array.Equals(arrayClone));
				Assert.AreEqual(arrayClone.GetHash(), array.GetHash());
				for (CbFieldIterator it = arrayClone.CreateIterator(); it; ++it)
				{
					CbField field = it.Current;
					Assert.AreEqual(8, field.AsInt32());
					//					Assert.IsTrue(Field.IsOwned());
				}
				for (CbFieldIterator it = arrayClone.CreateIterator(), end = new(); it != end; ++it)
				{
				}
				foreach (CbField _ in arrayClone)
				{
				}

				// CopyTo
				byte[] copyBytes = new byte[5];
				array.CopyTo(copyBytes);
				//				Assert.IsTrue(ArrayClone.GetOuterBuffer().GetView().Span.SequenceEqual(CopyBytes));
				arrayClone.CopyTo(copyBytes);
				//				Assert.IsTrue(ArrayClone.GetOuterBuffer().GetView().Span.SequenceEqual(CopyBytes));

				//				// GetBuffer
				//				Assert.IsTrue(CbField(FSharedBuffer.MakeView(Buffer)).GetBuffer().Flatten().GetView().Span.SequenceEqual(Buffer));
				//				Assert.IsTrue(TEXT("CbField(ArrayWithNameNoType).GetBuffer()"),
				//					CbField(CbField(Buffer + 1, CbFieldType(ArrayType)), FSharedBuffer.MakeView(Buffer)).GetBuffer().Flatten().GetView().Span.SequenceEqual(Buffer));
			}

			// Test CbArray as CbFieldIterator
			{
				uint count = 0;
				CbArray array = new();
				for (CbFieldIterator iter = CbFieldIterator.MakeSingle(array.AsField()); iter; ++iter)
				{
					CbField field = iter.Current;
					Assert.IsTrue(field.IsArray());
					++count;
				}
				Assert.AreEqual(1u, count);
			}

			// Test CbArray as CbFieldIterator
			{
				uint count = 0;
				CbArray array = new();
				//				Array.MakeOwned();
				for (CbFieldIterator iter = CbFieldIterator.MakeSingle(array.AsField()); iter; ++iter)
				{
					CbField field = iter.Current;
					Assert.IsTrue(field.IsArray());
					++count;
				}
				Assert.AreEqual(1u, count);
			}
		}

		[TestMethod]
		public void CbFieldBinaryTest()
		{
			// Test CbField(Binary, Empty)
			TestField(CbFieldType.Binary, [0]);

			// Test CbField(Binary, Value)
			{
				byte[] payload = [3, 4, 5, 6]; // Size: 3, Data: 4/5/6
				CbField fieldView = new(payload, CbFieldType.Binary);
				TestField(CbFieldType.Binary, fieldView, (ReadOnlyMemory<byte>)payload.AsMemory(1, 3));

				CbField field = fieldView;
				field.AsBinary();
				//				Assert.IsFalse(Field.GetOuterBuffer().IsNull());
				//				MoveTemp(Field).AsBinary();
				//				Assert.IsTrue(Field.GetOuterBuffer().IsNull());
			}

			// Test CbField(None) as Binary
			{
				CbField fieldView = new();
				byte[] @default = [1, 2, 3];
				TestFieldError(CbFieldType.Binary, fieldView, CbFieldError.TypeError, (ReadOnlyMemory<byte>)@default);

				//				CbField Field = CbField.Clone(FieldView);
				//				TestFieldError(CbFieldType.Binary, FSharedBuffer, Field, CbFieldError.TypeError, FSharedBuffer.MakeView(Default), FCbBinaryAccessors());
			}
		}

		[TestMethod]
		public void CbFieldStringTest()
		{
			// Test CbField(String, Empty)
			TestField(CbFieldType.String, [0]);

			// Test CbField(String, Value)
			{
				byte[] payload = [3, (byte)'A', (byte)'B', (byte)'C']; // Size: 3, Data: ABC
				TestField(CbFieldType.String, payload, new Utf8String(payload.AsMemory(1, 3)));
			}

			// Test CbField(String, OutOfRangeSize)
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, (ulong)(1) << 31);
				TestFieldError(CbFieldType.String, payload, CbFieldError.RangeError, new Utf8String("ABC"));
			}

			// Test CbField(None) as String
			{
				CbField field = new();
				TestFieldError(CbFieldType.String, field, CbFieldError.TypeError, new Utf8String("ABC"));
			}
		}

		[Flags]
		enum EIntType : byte
		{
			None = 0x00,
			Int8 = 0x01,
			Int16 = 0x02,
			Int32 = 0x04,
			Int64 = 0x08,
			UInt8 = 0x10,
			UInt16 = 0x20,
			UInt32 = 0x40,
			UInt64 = 0x80,
			// Masks for positive values requiring the specified number of bits.
			Pos64 = UInt64,
			Pos63 = Pos64 | Int64,
			Pos32 = Pos63 | UInt32,
			Pos31 = Pos32 | Int32,
			Pos16 = Pos31 | UInt16,
			Pos15 = Pos16 | Int16,
			Pos8 = Pos15 | UInt8,
			Pos7 = Pos8 | Int8,
			// Masks for negative values requiring the specified number of bits.
			Neg63 = Int64,
			Neg31 = Neg63 | Int32,
			Neg15 = Neg31 | Int16,
			Neg7 = Neg15 | Int8,
		};

		void TestIntegerField(CbFieldType fieldType, EIntType expectedMask, ulong magnitude)
		{
			byte[] payload = new byte[9];
			ulong negative = (ulong)((byte)fieldType & 1);
			VarInt.WriteUnsigned(payload, magnitude - negative);
			ulong defaultValue = 8;
			ulong expectedValue = (negative != 0) ? (ulong)(-(long)(magnitude)) : magnitude;
			CbField field = new(payload, fieldType);

			TestField(CbFieldType.IntegerNegative, field, (sbyte)(((expectedMask & EIntType.Int8) != 0) ? expectedValue : defaultValue),
				(sbyte)(defaultValue), ((expectedMask & EIntType.Int8) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<sbyte>(x => x.IsInteger(), (x, y) => x.AsInt8(y)));

			TestField(CbFieldType.IntegerNegative, field, (short)(((expectedMask & EIntType.Int16) != 0) ? expectedValue : defaultValue),
				(short)(defaultValue), ((expectedMask & EIntType.Int16) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<short>(x => x.IsInteger(), (x, y) => x.AsInt16(y)));

			TestField(CbFieldType.IntegerNegative, field, (int)(((expectedMask & EIntType.Int32) != 0) ? expectedValue : defaultValue),
				(int)(defaultValue), ((expectedMask & EIntType.Int32) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<int>(x => x.IsInteger(), (x, y) => x.AsInt32(y)));

			TestField(CbFieldType.IntegerNegative, field, (long)(((expectedMask & EIntType.Int64) != 0) ? expectedValue : defaultValue),
				(long)(defaultValue), ((expectedMask & EIntType.Int64) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<long>(x => x.IsInteger(), (x, y) => x.AsInt64(y)));

			TestField(CbFieldType.IntegerPositive, field, (byte)(((expectedMask & EIntType.UInt8) != 0) ? expectedValue : defaultValue),
				(byte)(defaultValue), ((expectedMask & EIntType.UInt8) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<byte>(x => x.IsInteger(), (x, y) => x.AsUInt8(y)));

			TestField(CbFieldType.IntegerPositive, field, (ushort)(((expectedMask & EIntType.UInt16) != 0) ? expectedValue : defaultValue),
				(ushort)(defaultValue), ((expectedMask & EIntType.UInt16) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<ushort>(x => x.IsInteger(), (x, y) => x.AsUInt16(y)));

			TestField(CbFieldType.IntegerPositive, field, (uint)(((expectedMask & EIntType.UInt32) != 0) ? expectedValue : defaultValue),
				(uint)(defaultValue), ((expectedMask & EIntType.UInt32) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<uint>(x => x.IsInteger(), (x, y) => x.AsUInt32(y)));
			TestField(CbFieldType.IntegerPositive, field, (ulong)(((expectedMask & EIntType.UInt64) != 0) ? expectedValue : defaultValue),
				(ulong)(defaultValue), ((expectedMask & EIntType.UInt64) != 0) ? CbFieldError.None : CbFieldError.RangeError, CbFieldAccessors.FromStruct<ulong>(x => x.IsInteger(), (x, y) => x.AsUInt64(y)));
		}

		[TestMethod]
		public void CbFieldIntegerTest()
		{
			// Test CbField(IntegerPositive)
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos7, 0x00);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos7, 0x7f);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos8, 0x80);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos8, 0xff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos15, 0x0100);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos15, 0x7fff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos16, 0x8000);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos16, 0xffff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos31, 0x0001_0000);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos31, 0x7fff_ffff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos32, 0x8000_0000);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos32, 0xffff_ffff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos63, 0x0000_0001_0000_0000);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos63, 0x7fff_ffff_ffff_ffff);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos64, 0x8000_0000_0000_0000);
			TestIntegerField(CbFieldType.IntegerPositive, EIntType.Pos64, 0xffff_ffff_ffff_ffff);

			// Test CbField(IntegerNegative)
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg7, 0x01);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg7, 0x80);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg15, 0x81);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg15, 0x8000);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg31, 0x8001);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg31, 0x8000_0000);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg63, 0x8000_0001);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.Neg63, 0x8000_0000_0000_0000);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.None, 0x8000_0000_0000_0001);
			TestIntegerField(CbFieldType.IntegerNegative, EIntType.None, 0xffff_ffff_ffff_ffff);

			// Test CbField(None) as Integer
			{
				CbField field = new();
				TestFieldError(CbFieldType.IntegerPositive, field, CbFieldError.TypeError, (ulong)(8));
				TestFieldError(CbFieldType.IntegerNegative, field, CbFieldError.TypeError, (long)(8));
			}
		}

		[TestMethod]
		public void CbFieldFloatTest()
		{
			// Test CbField(Float, 32-bit)
			{
				byte[] payload = [0xc0, 0x12, 0x34, 0x56]; // -2.28444433f
				TestField(CbFieldType.Float32, payload, -2.28444433f);

				CbField field = new(payload, CbFieldType.Float32);
				TestField(CbFieldType.Float64, field, (double)-2.28444433f);
			}

			// Test CbField(Float, 64-bit)
			{
				byte[] payload = [0xc1, 0x23, 0x45, 0x67, 0x89, 0xab, 0xcd, 0xef]; // -631475.76888888876
				TestField(CbFieldType.Float64, payload, -631475.76888888876);

				CbField field = new(payload, CbFieldType.Float64);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
			}

			// Test CbField(Integer+, MaxBinary32) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 24) - 1); // 16,777,215
				CbField field = new(payload, CbFieldType.IntegerPositive);
				TestField(CbFieldType.Float32, field, 16_777_215.0f);
				TestField(CbFieldType.Float64, field, 16_777_215.0);
			}

			// Test CbField(Integer+, MaxBinary32+1) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, (ulong)(1) << 24); // 16,777,216
				CbField field = new(payload, CbFieldType.IntegerPositive);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestField(CbFieldType.Float64, field, 16_777_216.0);
			}

			// Test CbField(Integer+, MaxBinary64) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 53) - 1); // 9,007,199,254,740,991
				CbField field = new(payload, CbFieldType.IntegerPositive);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestField(CbFieldType.Float64, field, 9_007_199_254_740_991.0);
			}

			// Test CbField(Integer+, MaxBinary64+1) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, (ulong)(1) << 53); // 9,007,199,254,740,992
				CbField field = new(payload, CbFieldType.IntegerPositive);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestFieldError(CbFieldType.Float64, field, CbFieldError.RangeError, 8.0);
			}

			// Test CbField(Integer+, MaxUInt64) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ~(ulong)0); // Max uint64
				CbField field = new(payload, CbFieldType.IntegerPositive);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestFieldError(CbFieldType.Float64, field, CbFieldError.RangeError, 8.0);
			}

			// Test CbField(Integer-, MaxBinary32) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 24) - 2); // -16,777,215
				CbField field = new(payload, CbFieldType.IntegerNegative);
				TestField(CbFieldType.Float32, field, -16_777_215.0f);
				TestField(CbFieldType.Float64, field, -16_777_215.0);
			}

			// Test CbField(Integer-, MaxBinary32+1) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 24) - 1); // -16,777,216
				CbField field = new(payload, CbFieldType.IntegerNegative);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestField(CbFieldType.Float64, field, -16_777_216.0);
			}

			// Test CbField(Integer-, MaxBinary64) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 53) - 2); // -9,007,199,254,740,991
				CbField field = new(payload, CbFieldType.IntegerNegative);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestField(CbFieldType.Float64, field, -9_007_199_254_740_991.0);
			}

			// Test CbField(Integer-, MaxBinary64+1) as Float
			{
				byte[] payload = new byte[9];
				VarInt.WriteUnsigned(payload, ((ulong)(1) << 53) - 1); // -9,007,199,254,740,992
				CbField field = new(payload, CbFieldType.IntegerNegative);
				TestFieldError(CbFieldType.Float32, field, CbFieldError.RangeError, 8.0f);
				TestFieldError(CbFieldType.Float64, field, CbFieldError.RangeError, 8.0);
			}

			// Test CbField(None) as Float
			{
				CbField field = new();
				TestFieldError(CbFieldType.Float32, field, CbFieldError.TypeError, 8.0f);
				TestFieldError(CbFieldType.Float64, field, CbFieldError.TypeError, 8.0);
			}
		}

		[TestMethod]
		public void CbFieldBoolTest()
		{
			// Test CbField(Bool, False)
			TestField(CbFieldType.BoolFalse, Array.Empty<byte>(), false, true);

			// Test CbField(Bool, True)
			TestField(CbFieldType.BoolTrue, Array.Empty<byte>(), true, false);

			// Test CbField(None) as Bool
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.BoolFalse, defaultField, CbFieldError.TypeError, false);
				TestFieldError(CbFieldType.BoolTrue, defaultField, CbFieldError.TypeError, true);
			}
		}

		[TestMethod]
		public void CbFieldObjectAttachmentTest()
		{
			byte[] zeroBytes = new byte[20];
			byte[] sequentialBytes = [1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20];

			// Test CbField(ObjectAttachment, Zero)
			TestField(CbFieldType.ObjectAttachment, zeroBytes);

			// Test CbField(ObjectAttachment, NonZero)
			TestField(CbFieldType.ObjectAttachment, sequentialBytes, (CbObjectAttachment)new IoHash(sequentialBytes));

			// Test CbField(ObjectAttachment, NonZero) AsAttachment
			{
				CbField field = new(sequentialBytes, CbFieldType.ObjectAttachment);
				TestField(CbFieldType.ObjectAttachment, field, (CbObjectAttachment)new IoHash(sequentialBytes), (CbObjectAttachment)new IoHash(), CbFieldError.None);
			}

			// Test CbField(None) as ObjectAttachment
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.ObjectAttachment, defaultField, CbFieldError.TypeError, (CbObjectAttachment)new IoHash(sequentialBytes));
			}
		}

		[TestMethod]
		public void CbFieldBinaryAttachmentTest()
		{
			byte[] zeroBytes = new byte[20];
			byte[] sequentialBytes = [1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20];

			// Test CbField(BinaryAttachment, Zero)
			TestField(CbFieldType.BinaryAttachment, zeroBytes);

			// Test CbField(BinaryAttachment, NonZero)
			TestField(CbFieldType.BinaryAttachment, sequentialBytes, (CbBinaryAttachment)new IoHash(sequentialBytes));

			// Test CbField(BinaryAttachment, NonZero) AsAttachment
			{
				CbField field = new(sequentialBytes, CbFieldType.BinaryAttachment);
				TestField(CbFieldType.BinaryAttachment, field, (CbBinaryAttachment)new IoHash(sequentialBytes), (CbBinaryAttachment)new IoHash(), CbFieldError.None);
			}

			// Test CbField(None) as BinaryAttachment
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.BinaryAttachment, defaultField, CbFieldError.TypeError, (CbBinaryAttachment)new IoHash(sequentialBytes));
			}
		}

		[TestMethod]
		public void CbFieldHashTest()
		{
			byte[] zeroBytes = new byte[20];
			byte[] sequentialBytes = [1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20];

			// Test CbField(Hash, Zero)
			TestField(CbFieldType.Hash, zeroBytes);

			// Test CbField(Hash, NonZero)
			TestField(CbFieldType.Hash, sequentialBytes, new IoHash(sequentialBytes));

			// Test CbField(None) as Hash
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.Hash, defaultField, CbFieldError.TypeError, new IoHash(sequentialBytes));
			}

			// Test CbField(ObjectAttachment) as Hash
			{
				CbField field = new(sequentialBytes, CbFieldType.ObjectAttachment);
				TestField(CbFieldType.Hash, field, new IoHash(sequentialBytes));
			}

			// Test CbField(BinaryAttachment) as Hash
			{
				CbField field = new(sequentialBytes, CbFieldType.BinaryAttachment);
				TestField(CbFieldType.Hash, field, new IoHash(sequentialBytes));
			}
		}

		[TestMethod]
		public void CbFieldUuidTest()
		{
			byte[] zeroBytes = [0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0];
			byte[] sequentialBytes = [0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15];
			Guid sequentialGuid = Guid.Parse("00010203-0405-0607-0809-0a0b0c0d0e0f");

			// Test CbField(Uuid, Zero)
			TestField(CbFieldType.Uuid, zeroBytes, new Guid(), sequentialGuid);

			// Test CbField(Uuid, NonZero)
			TestField(CbFieldType.Uuid, sequentialBytes, sequentialGuid, new Guid());

			// Test CbField(None) as Uuid
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.Uuid, defaultField, CbFieldError.TypeError, Guid.NewGuid());
			}
		}

		[TestMethod]
		public void CbFieldDateTimeTest()
		{
			// Test CbField(DateTime, Zero)
			TestField(CbFieldType.DateTime, [0, 0, 0, 0, 0, 0, 0, 0]);

			// Test CbField(DateTime, 0x1020_3040_5060_7080)
			TestField(CbFieldType.DateTime, [0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80], new DateTime(0x1020_3040_5060_7080L, DateTimeKind.Utc));

			// Test CbField(DateTime, Zero) as FDateTime
			{
				byte[] payload = [0, 0, 0, 0, 0, 0, 0, 0];
				CbField field = new(payload, CbFieldType.DateTime);
				Assert.AreEqual(field.AsDateTime(), new DateTime(0));
			}

			// Test CbField(None) as DateTime
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.DateTime, defaultField, CbFieldError.TypeError);
				DateTime defaultValue = new(0x1020_3040_5060_7080L, DateTimeKind.Utc);
				Assert.AreEqual(defaultField.AsDateTime(defaultValue), defaultValue);
			}
		}

		[TestMethod]
		public void CbFieldTimeSpanTest()
		{
			// Test CbField(TimeSpan, Zero)
			TestField(CbFieldType.TimeSpan, [0, 0, 0, 0, 0, 0, 0, 0]);

			// Test CbField(TimeSpan, 0x1020_3040_5060_7080)
			TestField(CbFieldType.TimeSpan, [0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80], new TimeSpan(0x1020_3040_5060_7080L));

			// Test CbField(TimeSpan, Zero) as FTimeSpan
			{
				byte[] payload = [0, 0, 0, 0, 0, 0, 0, 0];
				CbField field = new(payload, CbFieldType.TimeSpan);
				Assert.AreEqual(field.AsTimeSpan(), new TimeSpan(0));
			}

			// Test CbField(None) as TimeSpan
			{
				CbField defaultField = new();
				TestFieldError(CbFieldType.TimeSpan, defaultField, CbFieldError.TypeError);
				TimeSpan defaultValue = new(0x1020_3040_5060_7080L);
				Assert.AreEqual(defaultField.AsTimeSpan(defaultValue), defaultValue);
			}
		}

#if false
		[TestMethod]
		public void CbFieldObjectIdTest()
		{
			// Test CbField(ObjectId, Zero)
			TestField(CbFieldType.ObjectId, {0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0});

			// Test CbField(ObjectId, 0x102030405060708090A0B0C0)
			TestField(CbFieldType.ObjectId, {0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0},
				FCbObjectId(MakeMemoryView<byte>({0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0})));

			// Test CbField(ObjectId, Zero) as FCbObjectId
			{
				byte[] Payload = new byte[]{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0};
				CbField Field = new CbField(Payload, CbFieldType.ObjectId);
				Assert.AreEqual(Field.AsObjectId(), FCbObjectId());
			}

			// Test CbField(None) as ObjectId
			{
				CbField DefaultField;
				TestFieldError(CbFieldType.ObjectId, DefaultField, CbFieldError.TypeError);
				FCbObjectId DefaultValue(MakeMemoryView<byte>({0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0}));
				Assert.AreEqual(DefaultField.AsObjectId(DefaultValue), DefaultValue);
			}
		}

		[TestMethod]
		public void CbFieldCustomByIdTest()
		{
			struct FCustomByIdAccessor
			{
				explicit FCustomByIdAccessor(uint64 Id)
					: AsType([Id](CbField& Field, FMemoryView Default) { return Field.AsCustom(Id, Default); })
				{
				}

				bool (CbField.*IsType)() = &CbField.IsCustomById;
				TUniqueFunction<FMemoryView (CbField& Field, FMemoryView Default)> AsType;
			};

			// Test CbField(CustomById, MinId, Empty)
			{
				byte[] Payload = new byte[]{1, 0};
				TestField(CbFieldType.CustomById, Payload, FCbCustomById{0});
				TestField(CbFieldType.CustomById, Payload, FMemoryView(), MakeMemoryView<byte>({1, 2, 3}), CbFieldError.None, FCustomByIdAccessor(0));
				TestFieldError(CbFieldType.CustomById, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByIdAccessor(MAX_uint64));
			}

			// Test CbField(CustomById, MinId, Value)
			{
				byte[] Payload = new byte[]{5, 0, 1, 2, 3, 4};
				TestFieldNoClone(CbFieldType.CustomById, Payload, FCbCustomById{0, Payload.Right(4)});
				TestFieldNoClone(CbFieldType.CustomById, Payload, Payload.Right(4), FMemoryView(), CbFieldError.None, FCustomByIdAccessor(0));
				TestFieldError(CbFieldType.CustomById, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByIdAccessor(MAX_uint64));
			}

			// Test CbField(CustomById, MaxId, Empty)
			{
				byte[] Payload = new byte[]{9, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff};
				TestField(CbFieldType.CustomById, Payload, FCbCustomById{MAX_uint64});
				TestField(CbFieldType.CustomById, Payload, FMemoryView(), MakeMemoryView<byte>({1, 2, 3}), CbFieldError.None, FCustomByIdAccessor(MAX_uint64));
				TestFieldError(CbFieldType.CustomById, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByIdAccessor(0));
			}

			// Test CbField(CustomById, MaxId, Value)
			{
				byte[] Payload = new byte[]{13, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 1, 2, 3, 4};
				TestFieldNoClone(CbFieldType.CustomById, Payload, FCbCustomById{MAX_uint64, Payload.Right(4)});
				TestFieldNoClone(CbFieldType.CustomById, Payload, Payload.Right(4), FMemoryView(), CbFieldError.None, FCustomByIdAccessor(MAX_uint64));
				TestFieldError(CbFieldType.CustomById, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByIdAccessor(0));
			}

			// Test CbField(None) as CustomById
			{
				CbField DefaultField;
				TestFieldError(CbFieldType.CustomById, DefaultField, CbFieldError.TypeError, FCbCustomById{4, MakeMemoryView<byte>({1, 2, 3})});
				TestFieldError(CbFieldType.CustomById, DefaultField, CbFieldError.TypeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByIdAccessor(0));
				byte[] DefaultValue = new byte[]{1, 2, 3};
				Assert.AreEqual(DefaultField.AsCustom(0, DefaultValue), DefaultValue);
			}

			return true;
		}

		[TestMethod]
		void CbFieldCustomByNameTest()
		{
			struct FCustomByNameAccessor
			{
				explicit FCustomByNameAccessor(FAnsiStringView Name)
					: AsType([Name = FString(Name)](CbField& Field, FMemoryView Default) { return Field.AsCustom(TCHAR_TO_ANSI(*Name), Default); })
				{
				}

				bool (CbField.*IsType)() = &CbField.IsCustomByName;
				TUniqueFunction<FMemoryView (CbField& Field, FMemoryView Default)> AsType;
			};

			// Test CbField(CustomByName, ABC, Empty)
			{
				byte[] Payload = new byte[]{4, 3, 'A', 'B', 'C'};
				TestField(CbFieldType.CustomByName, Payload, FCbCustomByName{"ABC"});
				TestField(CbFieldType.CustomByName, Payload, FMemoryView(), MakeMemoryView<byte>({1, 2, 3}), CbFieldError.None, FCustomByNameAccessor("ABC"));
				TestFieldError(CbFieldType.CustomByName, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByNameAccessor("abc"));
			}

			// Test CbField(CustomByName, ABC, Value)
			{
				byte[] Payload = new byte[]{8, 3, 'A', 'B', 'C', 1, 2, 3, 4};
				TestFieldNoClone(CbFieldType.CustomByName, Payload, FCbCustomByName{"ABC", Payload.Right(4)});
				TestFieldNoClone(CbFieldType.CustomByName, Payload, Payload.Right(4), FMemoryView(), CbFieldError.None, FCustomByNameAccessor("ABC"));
				TestFieldError(CbFieldType.CustomByName, Payload, CbFieldError.RangeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByNameAccessor("abc"));
			}

			// Test CbField(None) as CustomByName
			{
				CbField DefaultField;
				TestFieldError(CbFieldType.CustomByName, DefaultField, CbFieldError.TypeError, FCbCustomByName{"ABC", MakeMemoryView<byte>({1, 2, 3})});
				TestFieldError(CbFieldType.CustomByName, DefaultField, CbFieldError.TypeError, MakeMemoryView<byte>({1, 2, 3}), FCustomByNameAccessor("ABC"));
				byte[] DefaultValue = new byte[]{1, 2, 3};
				Assert.AreEqual(DefaultField.AsCustom("ABC", DefaultValue), DefaultValue);
			}
		}

		[TestMethod]
		public void CbFieldIterateAttachmentsTest()
		{
			Func<uint, IoHash> MakeTestHash = (uint Index) =>
			{
				byte[] Data = new byte[4];
				BinaryPrimitives.WriteUInt32LittleEndian(Data, Index);
				return IoHash.Compute(Data);
			};

			CbFieldIterator Fields;
			{
				CbWriter Writer = new CbWriter();

				Writer.SetName("IgnoredTypeInRoot").AddHash(MakeTestHash(100));
				Writer.AddObjectAttachment(MakeTestHash(0));
				Writer.AddBinaryAttachment(MakeTestHash(1));
				Writer.SetName("ObjAttachmentInRoot").AddObjectAttachment(MakeTestHash(2));
				Writer.SetName("BinAttachmentInRoot").AddBinaryAttachment(MakeTestHash(3));

				// Uniform array of type to ignore.
				Writer.BeginArray();
				{
					Writer << 1;
					Writer << 2;
				}
				Writer.EndArray();
				// Uniform array of binary attachments.
				Writer.BeginArray();
				{
					Writer.AddBinaryAttachment(MakeTestHash(4));
					Writer.AddBinaryAttachment(MakeTestHash(5));
				}
				Writer.EndArray();
				// Uniform array of uniform arrays.
				Writer.BeginArray();
				{
					Writer.BeginArray();
					Writer.AddBinaryAttachment(MakeTestHash(6));
					Writer.AddBinaryAttachment(MakeTestHash(7));
					Writer.EndArray();
					Writer.BeginArray();
					Writer.AddBinaryAttachment(MakeTestHash(8));
					Writer.AddBinaryAttachment(MakeTestHash(9));
					Writer.EndArray();
				}
				Writer.EndArray();
				// Uniform array of non-uniform arrays.
				Writer.BeginArray();
				{
					Writer.BeginArray();
					Writer << 0;
					Writer << false;
					Writer.EndArray();
					Writer.BeginArray();
					Writer.AddObjectAttachment(MakeTestHash(10));
					Writer << false;
					Writer.EndArray();
				}
				Writer.EndArray();
				// Uniform array of uniform objects.
				Writer.BeginArray();
				{
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInUniObjInUniObj1").AddObjectAttachment(MakeTestHash(11));
					Writer.SetName("ObjAttachmentInUniObjInUniObj2").AddObjectAttachment(MakeTestHash(12));
					Writer.EndObject();
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInUniObjInUniObj3").AddObjectAttachment(MakeTestHash(13));
					Writer.SetName("ObjAttachmentInUniObjInUniObj4").AddObjectAttachment(MakeTestHash(14));
					Writer.EndObject();
				}
				Writer.EndArray();
				// Uniform array of non-uniform objects.
				Writer.BeginArray();
				{
					Writer.BeginObject();
					Writer << "Int" << 0;
					Writer << "Bool" << false;
					Writer.EndObject();
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInNonUniObjInUniObj").AddObjectAttachment(MakeTestHash(15));
					Writer << "Bool" << false;
					Writer.EndObject();
				}
				Writer.EndArray();

				// Uniform object of type to ignore.
				Writer.BeginObject();
				{
					Writer << "Int1" << 1;
					Writer << "Int2" << 2;
				}
				Writer.EndObject();
				// Uniform object of binary attachments.
				Writer.BeginObject();
				{
					Writer.SetName("BinAttachmentInUniObj1").AddBinaryAttachment(MakeTestHash(16));
					Writer.SetName("BinAttachmentInUniObj2").AddBinaryAttachment(MakeTestHash(17));
				}
				Writer.EndObject();
				// Uniform object of uniform arrays.
				Writer.BeginObject();
				{
					Writer.SetName("Array1");
					Writer.BeginArray();
					Writer.AddBinaryAttachment(MakeTestHash(18));
					Writer.AddBinaryAttachment(MakeTestHash(19));
					Writer.EndArray();
					Writer.SetName("Array2");
					Writer.BeginArray();
					Writer.AddBinaryAttachment(MakeTestHash(20));
					Writer.AddBinaryAttachment(MakeTestHash(21));
					Writer.EndArray();
				}
				Writer.EndObject();
				// Uniform object of non-uniform arrays.
				Writer.BeginObject();
				{
					Writer.SetName("Array1");
					Writer.BeginArray();
					Writer << 0;
					Writer << false;
					Writer.EndArray();
					Writer.SetName("Array2");
					Writer.BeginArray();
					Writer.AddObjectAttachment(MakeTestHash(22));
					Writer << false;
					Writer.EndArray();
				}
				Writer.EndObject();
				// Uniform object of uniform objects.
				Writer.BeginObject();
				{
					Writer.SetName("Object1");
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInUniObjInUniObj1").AddObjectAttachment(MakeTestHash(23));
					Writer.SetName("ObjAttachmentInUniObjInUniObj2").AddObjectAttachment(MakeTestHash(24));
					Writer.EndObject();
					Writer.SetName("Object2");
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInUniObjInUniObj3").AddObjectAttachment(MakeTestHash(25));
					Writer.SetName("ObjAttachmentInUniObjInUniObj4").AddObjectAttachment(MakeTestHash(26));
					Writer.EndObject();
				}
				Writer.EndObject();
				// Uniform object of non-uniform objects.
				Writer.BeginObject();
				{
					Writer.SetName("Object1");
					Writer.BeginObject();
					Writer << "Int" << 0;
					Writer << "Bool" << false;
					Writer.EndObject();
					Writer.SetName("Object2");
					Writer.BeginObject();
					Writer.SetName("ObjAttachmentInNonUniObjInUniObj").AddObjectAttachment(MakeTestHash(27));
					Writer << "Bool" << false;
					Writer.EndObject();
				}
				Writer.EndObject();

				Fields = Writer.Save();
			}

			Assert.AreEqual(ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode.All), ECbValidateError.None);

			uint AttachmentIndex = 0;
			Fields.IterateRangeAttachments([this, &AttachmentIndex, &MakeTestHash](CbField Field)
				{
					Assert.IsTrue(FString.Printf(AttachmentIndex), Field.IsAttachment());
					Assert.AreEqual(FString.Printf(AttachmentIndex), Field.AsAttachment(), MakeTestHash(AttachmentIndex));
					++AttachmentIndex;
				});
			Assert.AreEqual(AttachmentIndex, 28);
		}

		[TestMethod]
		void CbFieldBufferTest()
		{
			static_assert(std.is_constructible<CbField>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&&>.value, "Missing constructor for CbField");

			static_assert(std.is_constructible<CbField, FSharedBuffer&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, FSharedBuffer&&>.value, "Missing constructor for CbField");

			static_assert(std.is_constructible<CbField, CbField&, FSharedBuffer&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbFieldIterator&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbField&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbArray&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, FCbObject&>.value, "Missing constructor for CbField");

			static_assert(std.is_constructible<CbField, CbField&, FSharedBuffer&&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbFieldIterator&&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbField&&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, CbArray&&>.value, "Missing constructor for CbField");
			static_assert(std.is_constructible<CbField, CbField&, FCbObject&&>.value, "Missing constructor for CbField");

			// Test CbField()
			{
				CbField DefaultField;
				Assert.IsFalse(DefaultField.HasValue());
				Assert.IsFalse(DefaultField.IsOwned());
				DefaultField.MakeOwned();
				Assert.IsTrue(DefaultField.IsOwned());
			}

			// Test Field w/ Type from Shared Buffer
			{
				byte[] Payload = new byte[]{ (byte)(CbFieldType.Binary), 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
				FSharedBuffer ViewBuffer = FSharedBuffer.MakeView(Payload);
				FSharedBuffer OwnedBuffer = FSharedBuffer.Clone(ViewBuffer);

				CbField View(ViewBuffer);
				CbField ViewMove{FSharedBuffer(ViewBuffer)};
				CbField ViewOuterField(ImplicitConv<CbField>(View), ViewBuffer);
				CbField ViewOuterBuffer(ImplicitConv<CbField>(View), View);
				CbField Owned(OwnedBuffer);
				CbField OwnedMove{FSharedBuffer(OwnedBuffer)};
				CbField OwnedOuterField(ImplicitConv<CbField>(Owned), OwnedBuffer);
				CbField OwnedOuterBuffer(ImplicitConv<CbField>(Owned), Owned);

				// These lines are expected to assert when uncommented.
				//CbField InvalidOuterBuffer(ImplicitConv<CbField>(Owned), ViewBuffer);
				//CbField InvalidOuterBufferMove(ImplicitConv<CbField>(Owned), FSharedBuffer(ViewBuffer));

				Assert.AreEqual(View.AsBinaryView(), ViewBuffer.GetView().Right(3));
				Assert.AreEqual(ViewMove.AsBinaryView(), View.AsBinaryView());
				Assert.AreEqual(ViewOuterField.AsBinaryView(), View.AsBinaryView());
				Assert.AreEqual(ViewOuterBuffer.AsBinaryView(), View.AsBinaryView());
				Assert.AreEqual(Owned.AsBinaryView(), OwnedBuffer.GetView().Right(3));
				Assert.AreEqual(OwnedMove.AsBinaryView(), Owned.AsBinaryView());
				Assert.AreEqual(OwnedOuterField.AsBinaryView(), Owned.AsBinaryView());
				Assert.AreEqual(OwnedOuterBuffer.AsBinaryView(), Owned.AsBinaryView());

				Assert.IsFalse(View.IsOwned());
				Assert.IsFalse(ViewMove.IsOwned());
				Assert.IsFalse(ViewOuterField.IsOwned());
				Assert.IsFalse(ViewOuterBuffer.IsOwned());
				Assert.IsTrue(Owned.IsOwned());
				Assert.IsTrue(OwnedMove.IsOwned());
				Assert.IsTrue(OwnedOuterField.IsOwned());
				Assert.IsTrue(OwnedOuterBuffer.IsOwned());

				View.MakeOwned();
				Owned.MakeOwned();
				Assert.AreNotEqual(View.AsBinaryView(), ViewBuffer.GetView().Right(3));
				Assert.IsTrue(View.IsOwned());
				Assert.AreEqual(Owned.AsBinaryView(), OwnedBuffer.GetView().Right(3));
				Assert.IsTrue(Owned.IsOwned());
			}

			// Test Field w/ Type
			{
				byte[] Payload = new byte[]{ (byte)(CbFieldType.Binary), 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
				CbField Field = new CbField(Payload);

				CbField VoidView = CbField.MakeView(Payload);
				CbField VoidClone = CbField.Clone(Payload);
				CbField FieldView = CbField.MakeView(Field);
				CbField FieldClone = CbField.Clone(Field);
				CbField FieldViewClone = CbField.Clone(FieldView);

				Assert.AreEqual(VoidView.AsBinaryView(), Payload.Right(3));
				Assert.AreNotEqual(VoidClone.AsBinaryView(), Payload.Right(3));
				Assert.IsTrue(VoidClone.AsBinaryView().Span.SequenceEqual(VoidView.AsBinaryView()));
				Assert.AreEqual(FieldView.AsBinaryView(), Payload.Right(3));
				Assert.AreNotEqual(FieldClone.AsBinaryView(), Payload.Right(3));
				Assert.IsTrue(FieldClone.AsBinaryView().Span.SequenceEqual(VoidView.AsBinaryView()));
				Assert.AreNotEqual(FieldViewClone.AsBinaryView(), FieldView.AsBinaryView());
				Assert.IsTrue(FieldViewClone.AsBinaryView().Span.SequenceEqual(VoidView.AsBinaryView()));

				Assert.IsFalse(VoidView.IsOwned());
				Assert.IsTrue(VoidClone.IsOwned());
				Assert.IsFalse(FieldView.IsOwned());
				Assert.IsTrue(FieldClone.IsOwned());
				Assert.IsTrue(FieldViewClone.IsOwned());
			}

			// Test Field w/o Type
			{
				byte[] Payload = new byte[]{ 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
				CbField Field = new CbField(Payload, CbFieldType.Binary);

				CbField FieldView = CbField.MakeView(Field);
				CbField FieldClone = CbField.Clone(Field);
				CbField FieldViewClone = CbField.Clone(FieldView);

				Assert.AreEqual(FieldView.AsBinaryView(), Payload.Right(3));
				Assert.IsTrue(FieldClone.AsBinaryView().Span.SequenceEqual(FieldView.AsBinaryView()));
				Assert.IsTrue(FieldViewClone.AsBinaryView().Span.SequenceEqual(FieldView.AsBinaryView()));

				Assert.IsFalse(FieldView.IsOwned());
				Assert.IsTrue(FieldClone.IsOwned());
				Assert.IsTrue(FieldViewClone.IsOwned());

				FieldView.MakeOwned();
				Assert.IsTrue(FieldView.AsBinaryView().Span.SequenceEqual(Payload.Right(3)));
				Assert.IsTrue(FieldView.IsOwned());
			}

			return true;
		}

		[TestMethod]
		void CbArrayBufferTest()
		{
			static_assert(std.is_constructible<CbArray>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&&>.value, "Missing constructor for CbArray");

			static_assert(std.is_constructible<CbArray, CbArray&, FSharedBuffer&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbFieldIterator&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbField&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbArray&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, FCbObject&>.value, "Missing constructor for CbArray");

			static_assert(std.is_constructible<CbArray, CbArray&, FSharedBuffer&&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbFieldIterator&&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbField&&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, CbArray&&>.value, "Missing constructor for CbArray");
			static_assert(std.is_constructible<CbArray, CbArray&, FCbObject&&>.value, "Missing constructor for CbArray");

			// Test CbArray()
			{
				CbArray DefaultArray;
				Assert.IsFalse(DefaultArray.IsOwned());
				DefaultArray.MakeOwned();
				Assert.IsTrue(DefaultArray.IsOwned());
			}
		}

		[TestMethod]
		public void CbObjectBufferTest()
		{
			static_assert(std.is_constructible<FCbObject>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObject&&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObject&&>.value, "Missing constructor for FCbObject");

			static_assert(std.is_constructible<FCbObject, FCbObjectView&, FSharedBuffer&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbFieldIterator&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbField&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbArray&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, FCbObject&>.value, "Missing constructor for FCbObject");

			static_assert(std.is_constructible<FCbObject, FCbObjectView&, FSharedBuffer&&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbFieldIterator&&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbField&&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, CbArray&&>.value, "Missing constructor for FCbObject");
			static_assert(std.is_constructible<FCbObject, FCbObjectView&, FCbObject&&>.value, "Missing constructor for FCbObject");

			// Test FCbObject()
			{
				FCbObject DefaultObject;
				Assert.IsFalse(DefaultObject.IsOwned());
				DefaultObject.MakeOwned();
				Assert.IsTrue(DefaultObject.IsOwned());
			}

			return true;
		}

		[TestMethod]
		public void CbFieldBufferIterator()
		{
			static_assert(std.is_constructible<CbFieldIterator, CbFieldIterator&>.value, "Missing constructor for CbFieldIterator");
			static_assert(std.is_constructible<CbFieldIterator, CbFieldIterator&&>.value, "Missing constructor for CbFieldIterator");

			static_assert(std.is_constructible<CbFieldIterator, CbFieldIterator&>.value, "Missing constructor for CbFieldIterator");
			static_assert(std.is_constructible<CbFieldIterator, CbFieldIterator&&>.value, "Missing constructor for CbFieldIterator");

			auto GetCount = [](auto It) -> uint
			{
				uint Count = 0;
				for (; It; ++It)
				{
					++Count;
				}
				return Count;
			};

			// Test CbField[View]Iterator()
			{
				Assert.AreEqual(GetCount(CbFieldIterator()), 0);
				Assert.AreEqual(GetCount(CbFieldIterator()), 0);
			}

			// Test CbField[View]Iterator(Range)
			{
				byte T = (byte)(CbFieldType.IntegerPositive);
				byte[] Payload = new byte[]{ T, 0, T, 1, T, 2, T, 3 };

				FSharedBuffer View = FSharedBuffer.MakeView(Payload);
				FSharedBuffer Clone = FSharedBuffer.Clone(View);

				FMemoryView EmptyView;
				FSharedBuffer NullBuffer;

				CbFieldIterator FieldViewIt = CbFieldIterator.MakeRange(View);
				CbFieldIterator FieldIt = CbFieldIterator.MakeRange(View);

				Assert.AreEqual(FieldViewIt.GetRangeHash(), FBlake3.HashBuffer(View));
				Assert.AreEqual(FieldIt.GetRangeHash(), FBlake3.HashBuffer(View));

				FMemoryView RangeView;
				Assert.IsTrue(FieldViewIt.TryGetRangeView(RangeView) && RangeView == Payload);
				Assert.IsTrue(FieldIt.TryGetRangeView(RangeView) && RangeView == Payload);

				Assert.AreEqual(GetCount(CbFieldIterator.CloneRange(CbFieldIterator())), 0);
				Assert.AreEqual(GetCount(CbFieldIterator.CloneRange(CbFieldIterator())), 0);
				CbFieldIterator FieldViewItClone = CbFieldIterator.CloneRange(FieldViewIt);
				CbFieldIterator FieldItClone = CbFieldIterator.CloneRange(FieldIt);
				Assert.AreEqual(GetCount(FieldViewItClone), 4);
				Assert.AreEqual(GetCount(FieldItClone), 4);
				Assert.AreNotEqual(FieldViewItClone, FieldIt);
				Assert.AreNotEqual(FieldItClone, FieldIt);

				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(EmptyView)), 0);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(NullBuffer)), 0);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(FSharedBuffer(NullBuffer))), 0);

				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(Payload)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(Clone)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRange(FSharedBuffer(Clone))), 4);

				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(View), NullBuffer)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(View), FSharedBuffer(NullBuffer))), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(View), View)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(View), FSharedBuffer(View))), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(Clone), Clone)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(Clone), FSharedBuffer(Clone))), 4);

				Assert.AreEqual(GetCount(CbFieldIterator(FieldIt)), 4);
				Assert.AreEqual(GetCount(CbFieldIterator(CbFieldIterator(FieldIt))), 4);

				// Uniform
				byte[] UniformPayload = new byte[]{ 0, 1, 2, 3 };
				CbFieldIterator UniformFieldViewIt = CbFieldIterator.MakeRange(UniformPayload, CbFieldType.IntegerPositive);
				Assert.AreEqual(UniformFieldViewIt.GetRangeHash(), FieldViewIt.GetRangeHash());
				Assert.IsFalse(UniformFieldViewIt.TryGetRangeView(RangeView));
				FSharedBuffer UniformView = FSharedBuffer.MakeView(UniformPayload);
				CbFieldIterator UniformFieldIt = CbFieldIterator.MakeRange(UniformView, CbFieldType.IntegerPositive);
				Assert.AreEqual(UniformFieldIt.GetRangeHash(), FieldViewIt.GetRangeHash());
				Assert.IsFalse(UniformFieldIt.TryGetRangeView(RangeView));

				// Equals
				Assert.IsTrue(FieldViewIt.Equals(FieldViewIt));
				Assert.IsTrue(FieldViewIt.Equals(FieldIt));
				Assert.IsTrue(FieldIt.Equals(FieldIt));
				Assert.IsTrue(FieldIt.Equals(FieldViewIt));
				Assert.IsFalse(FieldViewIt.Equals(FieldViewItClone));
				Assert.IsFalse(FieldIt.Equals(FieldItClone));
				Assert.IsTrue(UniformFieldViewIt.Equals(UniformFieldViewIt));
				Assert.IsTrue(UniformFieldViewIt.Equals(UniformFieldIt));
				Assert.IsTrue(UniformFieldIt.Equals(UniformFieldIt));
				Assert.IsTrue(UniformFieldIt.Equals(UniformFieldViewIt));
				Assert.IsFalse(TEXT("CbFieldIterator.Equals(SamePayload, DifferentEnd)"),
					CbFieldIterator.MakeRange(UniformPayload, CbFieldType.IntegerPositive)
						.Equals(CbFieldIterator.MakeRange(UniformPayload.LeftChop(1), CbFieldType.IntegerPositive)));
				Assert.IsFalse(TEXT("CbFieldIterator.Equals(DifferentPayload, SameEnd)"),
					CbFieldIterator.MakeRange(UniformPayload, CbFieldType.IntegerPositive)
						.Equals(CbFieldIterator.MakeRange(UniformPayload.RightChop(1), CbFieldType.IntegerPositive)));

				// CopyRangeTo
				byte[] CopyBytes = new byte[Payload.Length];
				FieldViewIt.CopyRangeTo(CopyBytes);
				Assert.IsTrue(CopyBytes.Span.SequenceEqual(Payload));
				FieldIt.CopyRangeTo(CopyBytes);
				Assert.IsTrue(CopyBytes.Span.SequenceEqual(Payload));
				UniformFieldViewIt.CopyRangeTo(CopyBytes);
				Assert.IsTrue(CopyBytes.Span.SequenceEqual(Payload));

				// MakeRangeOwned
				CbFieldIterator OwnedFromView = UniformFieldIt;
				OwnedFromView.MakeRangeOwned();
				Assert.IsTrue(OwnedFromView.TryGetRangeView(RangeView) && RangeView.Span.SequenceEqual(Payload));
				CbFieldIterator OwnedFromOwned = OwnedFromView;
				OwnedFromOwned.MakeRangeOwned();
				Assert.AreEqual(OwnedFromOwned, OwnedFromView);

				// These lines are expected to assert when uncommented.
				//FSharedBuffer ShortView = FSharedBuffer.MakeView(Payload.LeftChop(2));
				//Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(*View), ShortView)), 4);
				//Assert.AreEqual(GetCount(CbFieldIterator.MakeRangeView(CbFieldIterator.MakeRange(*View), FSharedBuffer(ShortView))), 4);
			}

			// Test CbField[View]Iterator(Scalar)
			{
				byte T = (byte)(CbFieldType.IntegerPositive);
				byte[] Payload = new byte[]{ T, 0 };

				FSharedBuffer View = FSharedBuffer.MakeView(Payload);
				FSharedBuffer Clone = FSharedBuffer.Clone(View);

				CbField FieldView(Payload);
				CbField Field = new CbField(View);

				Assert.AreEqual(GetCount(CbFieldIterator.MakeSingle(FieldView)), 1);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeSingle(CbField(FieldView))), 1);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeSingle(Field)), 1);
				Assert.AreEqual(GetCount(CbFieldIterator.MakeSingle(CbField(Field))), 1);
			}

			return true;
		}
#endif

		[TestMethod]
		public void CbFieldParseTest()
		{
			// Test the optimal object parsing loop because it is expected to be required for high performance.
			// Under ideal conditions, when the fields are in the expected order and there are no extra fields,
			// the loop will execute once and only one comparison will be performed for each field name. Either
			// way, each field will only be visited once even if the loop needs to execute several times.
			static void ParseObject(CbObject obj, ref uint a, ref uint b, ref uint c, ref uint d)
			{
				for (CbFieldIterator it = obj.CreateIterator(); it;)
				{
					CbFieldIterator last = it;
					if (it.Current.GetName() == new Utf8String("A"))
					{
						a = it.Current.AsUInt32();
						++it;
					}
					if (it.Current.GetName() == new Utf8String("B"))
					{
						b = it.Current.AsUInt32();
						++it;
					}
					if (it.Current.GetName() == new Utf8String("C"))
					{
						c = it.Current.AsUInt32();
						++it;
					}
					if (it.Current.GetName() == new Utf8String("D"))
					{
						d = it.Current.AsUInt32();
						++it;
					}
					if (last == it)
					{
						++it;
					}
				}
			}

			static bool TestParseObject(byte[] data, uint a, uint b, uint c, uint d)
			{
				uint parsedA = 0, parsedB = 0, parsedC = 0, parsedD = 0;
				ParseObject(new CbObject(data, CbFieldType.Object), ref parsedA, ref parsedB, ref parsedC, ref parsedD);
				return a == parsedA && b == parsedB && c == parsedC && d == parsedD;
			}

			byte t = (byte)(CbFieldType.IntegerPositive | CbFieldType.HasFieldName);
			Assert.IsTrue(TestParseObject([0], 0, 0, 0, 0));
			Assert.IsTrue(TestParseObject([16, t, 1, (byte)'A', 1, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3, t, 1, (byte)'D', 4], 1, 2, 3, 4));
			Assert.IsTrue(TestParseObject([16, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3, t, 1, (byte)'D', 4, t, 1, (byte)'A', 1], 1, 2, 3, 4));
			Assert.IsTrue(TestParseObject([12, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3, t, 1, (byte)'D', 4], 0, 2, 3, 4));
			Assert.IsTrue(TestParseObject([8, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3], 0, 2, 3, 0));
			Assert.IsTrue(TestParseObject([20, t, 1, (byte)'A', 1, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3, t, 1, (byte)'D', 4, t, 1, (byte)'E', 5], 1, 2, 3, 4));
			Assert.IsTrue(TestParseObject([20, t, 1, (byte)'E', 5, t, 1, (byte)'A', 1, t, 1, (byte)'B', 2, t, 1, (byte)'C', 3, t, 1, (byte)'D', 4], 1, 2, 3, 4));
			Assert.IsTrue(TestParseObject([16, t, 1, (byte)'D', 4, t, 1, (byte)'C', 3, t, 1, (byte)'B', 2, t, 1, (byte)'A', 1], 1, 2, 3, 4));
		}
	}
}
