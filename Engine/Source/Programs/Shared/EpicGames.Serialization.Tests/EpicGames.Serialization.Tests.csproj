<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\EpicGames.Serialization\EpicGames.Serialization.csproj" />
  </ItemGroup>

  <ItemGroup>
    <None Update="CompactBinaryObjects\build">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Update="CompactBinaryObjects\compact_binary">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
    <None Update="CompactBinaryObjects\ReferenceOutput">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>
</Project>
