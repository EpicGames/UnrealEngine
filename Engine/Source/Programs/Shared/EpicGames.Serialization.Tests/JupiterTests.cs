// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	[TestClass]
	public class JupiterTests
	{
		[TestMethod]
		public void OverflowTest()
		{
			CbWriter writer = new();

			writer.BeginArray();
			for (int idx = 0; idx < 128; idx++)
			{
				writer.WriteNullValue();
			}
			writer.EndArray();

			writer.ToByteArray();
		}

		[TestMethod]
		public void EmbeddedObject()
		{
			CbWriter writer1 = new();
			writer1.BeginObject();
			writer1.WriteInteger("a", 1);
			writer1.WriteUtf8String("b", new Utf8String("hello"));
			writer1.EndObject();

			CbObject object1 = writer1.ToObject();

			CbWriter writer2 = new();
			writer2.BeginObject();
			writer2.WriteObject("ref", object1);
			writer2.EndObject();

			CbObject object2 = writer2.ToObject();
			Assert.AreEqual(21, object2.GetSize());
		}

		[TestMethod]
		public void BuildObject()
		{
			byte[] bytes = File.ReadAllBytes("CompactBinaryObjects/build");

			ReadOnlyMemory<byte> memory = new(bytes);
			CbField o = new(memory);

			Assert.AreEqual(new Utf8String("BuildAction"), o.Name);
			List<CbField> buildActionFields = [.. o];
			Assert.AreEqual(3, buildActionFields.Count);
			Assert.AreEqual(new Utf8String("Function"), buildActionFields[0].Name);
			Assert.AreEqual(new Utf8String("Constants"), buildActionFields[1].Name);
			Assert.AreEqual(new Utf8String("Inputs"), buildActionFields[2].Name);

			List<CbField> constantsFields = [.. buildActionFields[1]];
			Assert.AreEqual(3, constantsFields.Count);
			Assert.AreEqual(new Utf8String("TextureBuildSettings"), constantsFields[0].Name);
			Assert.AreEqual(new Utf8String("TextureOutputSettings"), constantsFields[1].Name);
			Assert.AreEqual(new Utf8String("TextureSource"), constantsFields[2].Name);

			List<CbField> inputsFields = [.. buildActionFields[2]];
			Assert.AreEqual(1, inputsFields.Count);
			Assert.AreEqual(new Utf8String("7587B323422942733DDD048A91709FDE"), inputsFields[0].Name);
			Assert.IsTrue(inputsFields[0].IsBinaryAttachment());
			Assert.IsTrue(inputsFields[0].IsAttachment());
			Assert.IsFalse(inputsFields[0].IsObjectAttachment());
			Assert.AreEqual(IoHash.Parse("f855382171a0b1e5a1c653aa6c5121a05cbf4ba0"), inputsFields[0].AsHash());
		}

		[TestMethod]
		public void ReferenceOutput()
		{
			byte[] bytes = File.ReadAllBytes("CompactBinaryObjects/ReferenceOutput");

			ReadOnlyMemory<byte> memory = new(bytes);
			CbField o = new(memory);

			Assert.AreEqual(new Utf8String("BuildOutput"), o.Name);
			List<CbField> buildActionFields = [.. o];
			Assert.AreEqual(1, buildActionFields.Count);
			CbField payloads = buildActionFields[0];
			List<CbField> payloadFields = [.. payloads];
			Assert.AreEqual(3, payloadFields.Count);

			Assert.AreEqual(IoHash.Parse("5d8a6dc277c968f0d027c98f879c955c1905c293"), payloadFields[0]["RawHash"]!.AsHash());
			Assert.AreEqual(IoHash.Parse("313f0d0d334100d83aeb1ee2c42794fd087cb0ae"), payloadFields[1]["RawHash"]!.AsHash());
			Assert.AreEqual(IoHash.Parse("c7a03f83c08cdca882110ecf2b5654ee3b09b11e"), payloadFields[2]["RawHash"]!.AsHash());
		}

		[TestMethod]
		public void CompactBinary()
		{
			byte[] bytes = File.ReadAllBytes("CompactBinaryObjects/compact_binary");

			CbObject o = new(bytes);
			Assert.AreEqual(Utf8String.Empty, o.AsField().Name);
			List<CbField> buildActionFields = [.. o];
			Assert.AreEqual(3, buildActionFields.Count);
			CbField payloads = buildActionFields[0];
			List<CbField> payloadFields = [.. payloads];
			Assert.AreEqual(2, payloadFields.Count);

			Assert.AreEqual("{\"Key\":{\"Bucket\":\"EditorDomainPackage\",\"Hash\":\"37dbaa409ef30ba67f18c8fc2faaf606636cb915\"},\"Meta\":{\"FileSize\":24789},\"Attachments\":[{\"Id\":\"000000000000000000000001\",\"RawHash\":\"da6fc57e4b9f91377c9509ea0ad567bacb3796c5\",\"RawSize\":24789}]}", o.ToJson());
		}

		[TestMethod]
		public void WriteArray()
		{
			IoHash hash1 = IoHash.Parse("5d8a6dc277c968f0d027c98f879c955c1905c293");
			IoHash hash2 = IoHash.Parse("313f0d0d334100d83aeb1ee2c42794fd087cb0ae");

			CbWriter writer = new();
			writer.BeginObject();
			writer.BeginUniformArray("needs", CbFieldType.Hash);
			writer.WriteHashValue(hash1);
			writer.WriteHashValue(hash2);
			writer.EndUniformArray();
			writer.EndObject();

			byte[] objectData = writer.ToByteArray();
			ReadOnlyMemory<byte> memory = new(objectData);
			CbField o = new(memory);

			// the top object has no name
			Assert.AreEqual(Utf8String.Empty, o.Name);
			List<CbField> fields = [.. o];
			Assert.AreEqual(1, fields.Count);
			CbField? needs = o["needs"];
			List<CbField> blobList = [.. needs!.AsArray()];
			IoHash[] blobs = [.. blobList.Select(field => field.AsHash())];
			CollectionAssert.AreEqual(new IoHash[] { hash1, hash2 }, blobs);
		}

		[TestMethod]
		public void WriteObject()
		{
			IoHash hash1 = IoHash.Parse("5d8a6dc277c968f0d027c98f879c955c1905c293");

			CbWriter writer = new();
			writer.BeginObject();
			writer.WriteUtf8String("string", new Utf8String("test"));
			writer.WriteBinaryAttachment("hash", hash1);
			writer.EndObject();

			byte[] objectData = writer.ToByteArray();
			ReadOnlyMemory<byte> memory = new(objectData);
			CbField o = new(memory);

			// the object has no name and 2 fields
			Assert.AreEqual(Utf8String.Empty, o.Name);
			List<CbField> fields = [.. o];
			Assert.AreEqual(2, fields.Count);

			CbField? stringField = o["string"];
			Assert.AreEqual(new Utf8String("test"), stringField!.AsUtf8String());

			CbField? hashField = o["hash"];
			Assert.AreEqual(hash1, hashField!.AsAttachment());
		}
	}
}
