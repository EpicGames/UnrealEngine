// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Serialization.Tests
{
	[TestClass]
	public class SerializerTests
	{
		enum TestEnum
		{
			Value1,
			Value2,
			Value3
		}

		class TestObject : IEquatable<TestObject>
		{
			public int A { get; set; }
			public string B { get; set; } = String.Empty;
			public IoHash C { get; set; }
			public List<TestObject> Children { get; set; } = [];

			[CbField("D")]
			public string PropertyWithCustomName { get; set; } = "hello";

			[CbIgnore]
			public string IgnoredProperty { get; set; } = "world";

			public TestEnum Enum { get; set; } = TestEnum.Value3;

			public override bool Equals(object? obj) => obj is TestObject other && Equals(other);

			public bool Equals(TestObject? other) => other != null && A == other.A && B == other.B && C == other.C && Enumerable.SequenceEqual(Children, other.Children);

			public override int GetHashCode() => HashCode.Combine(A, B, C);
		}

		[TestMethod]
		public void RoundTrip()
		{
			TestObject obj1 = CreateTestObject(new Random(123));
			CbObject cbObj1 = CbSerializer.Serialize(obj1);
			ReadOnlyMemory<byte> mem1 = cbObj1.GetView();

			TestObject obj2 = CbSerializer.Deserialize<TestObject>(mem1);
			CbObject cbObj2 = CbSerializer.Serialize(obj2);
			ReadOnlyMemory<byte> mem2 = cbObj2.GetView();

			Assert.AreEqual(obj1, obj2);
			Assert.IsTrue(mem1.Span.SequenceEqual(mem2.Span));

			Assert.AreEqual("hello", cbObj1.Find("D").AsString());
			Assert.IsFalse(cbObj1.Find(nameof(TestObject.PropertyWithCustomName)).HasValue());
			Assert.IsFalse(cbObj1.Find(nameof(TestObject.IgnoredProperty)).HasValue());

			Assert.AreEqual((int)TestEnum.Value3, cbObj2.Find("Enum").AsInt32());
		}

		[TestMethod]
		public void StreamTest()
		{
			TestObject obj1 = CreateTestObject(new Random(123));

			CbWriter writer = new();
			CbSerializer.Serialize(writer, obj1);

			byte[] data1 = writer.ToByteArray();
			byte[] data2;
			using (Stream stream = writer.AsStream())
			{
				data2 = new byte[stream.Length];

				int readLength = stream.Read(data2, 0, data2.Length);
				Assert.AreEqual(data2.Length, readLength);

				readLength = stream.Read(new byte[10], 0, 10);
				Assert.AreEqual(0, readLength);
			}
			Assert.IsTrue(data1.AsSpan().SequenceEqual(data2));

			TestObject obj2 = CbSerializer.Deserialize<TestObject>(data1);
			Assert.AreEqual(obj1, obj2);
		}

		static TestObject CreateTestObject(Random random)
		{
			TestObject obj = new();
			obj.A = random.Next();
			obj.B = random.NextInt64().ToString();
			obj.C = IoHash.Compute(Encoding.UTF8.GetBytes(obj.B));

			for (int idx = 0; idx < random.Next(0, 3); idx++)
			{
				obj.Children.Add(CreateTestObject(random));
			}

			return obj;
		}

		class EmptyObject
		{
		}

		[CbObject]
		class EmptyObjectWithAttribute
		{
		}

		[TestMethod]
		public void EmptyObjectSerializationTest()
		{
			Assert.ThrowsExactly<CbEmptyClassException>(() => CbSerializer.Serialize(new EmptyObject()));
			Assert.ThrowsExactly<CbEmptyClassException>(() => CbSerializer.Deserialize<EmptyObject>(CbField.Empty));

			EmptyObjectWithAttribute obj2 = new();
			CbObject cbObj2 = CbSerializer.Serialize(obj2);
			CbSerializer.Deserialize<EmptyObjectWithAttribute>(cbObj2);
		}

		class ObjectWithNullField
		{
			[CbField("a")]
			public int Value { get; set; }

			[CbField("b")]
			public List<int>? Items { get; set; }
		}

		[TestMethod]
		public void NullFieldTest()
		{
			ObjectWithNullField input = new() { Value = 123 };
			CbObject obj = CbSerializer.Serialize(input);

			ObjectWithNullField output = CbSerializer.Deserialize<ObjectWithNullField>(obj.AsField());
			Assert.AreEqual(123, output.Value);
			Assert.AreEqual(null, output.Items);
		}

		class ObjectWithReadOnlyCollection
		{
			[CbField("s")]
			public List<string> Strings { get; } = [];

			[CbField("i")]
			public List<int> Integers { get; } = [];

			[CbField("d")]
			public Dictionary<string, int> Dictionary { get; } = new Dictionary<string, int>(StringComparer.Ordinal);
		}

		[TestMethod]
		public void ObjectWithReadOnlyCollectionTest()
		{
			ObjectWithReadOnlyCollection input = new();
			input.Strings.Add("Hello");
			input.Strings.Add("World");
			input.Integers.Add(1);
			input.Integers.Add(2);
			input.Integers.Add(3);
			input.Integers.Add(4);
			input.Dictionary.Add("hello", 123);
			input.Dictionary.Add("world", 456);
			CbObject obj = CbSerializer.Serialize(input);

			ObjectWithReadOnlyCollection output = CbSerializer.Deserialize<ObjectWithReadOnlyCollection>(obj.AsField());
			Assert.AreEqual(2, output.Strings.Count);
			Assert.AreEqual("Hello", output.Strings[0]);
			Assert.AreEqual("World", output.Strings[1]);
			Assert.AreEqual(4, output.Integers.Count);
			Assert.AreEqual(1, output.Integers[0]);
			Assert.AreEqual(2, output.Integers[1]);
			Assert.AreEqual(3, output.Integers[2]);
			Assert.AreEqual(4, output.Integers[3]);
			Assert.AreEqual(2, output.Dictionary.Count);
			Assert.AreEqual(123, output.Dictionary["hello"]);
			Assert.AreEqual(456, output.Dictionary["world"]);
		}

		class ObjectWithStringDictCollection
		{
			[CbField("d")]
			public Dictionary<string, int> Dictionary { get; set; } = new Dictionary<string, int>(StringComparer.Ordinal);
		}

		[TestMethod]
		public void ObjectWithStringDictCollectionTest()
		{
			ObjectWithStringDictCollection input = new();
			input.Dictionary.Add("hello", 123);
			input.Dictionary.Add("world", 456);
			CbObject obj = CbSerializer.Serialize(input);

			ObjectWithStringDictCollection output = CbSerializer.Deserialize<ObjectWithStringDictCollection>(obj.AsField());
			Assert.AreEqual(2, output.Dictionary.Count);
			Assert.AreEqual(123, output.Dictionary["hello"]);
			Assert.AreEqual(456, output.Dictionary["world"]);

			// make sure string keyed dictionaries are serialized as objects
			CbWriter writer = new();
			writer.BeginObject();
			writer.BeginObject("d");
			writer.WriteInteger("hello", 123);
			writer.WriteInteger("world", 456);
			writer.EndObject();
			writer.EndObject();
			CbObject manualCbObject = writer.ToObject();

			ObjectWithStringDictCollection output2 = CbSerializer.Deserialize<ObjectWithStringDictCollection>(manualCbObject.AsField());
			Assert.AreEqual(2, output2.Dictionary.Count);
			Assert.AreEqual(123, output2.Dictionary["hello"]);
			Assert.AreEqual(456, output2.Dictionary["world"]);
		}

		class ObjectWithByteArray
		{
			[CbField("b")]
			public byte[] Bytes { get; set; } = null!;
		}

		[TestMethod]
		public void ObjectWithByteArrayTest()
		{
			ObjectWithByteArray input = new();
			byte[] b = Encoding.ASCII.GetBytes("test-string");
			input.Bytes = b;
			CbObject obj = CbSerializer.Serialize(input);

			ObjectWithByteArray output = CbSerializer.Deserialize<ObjectWithByteArray>(obj.AsField());
			CollectionAssert.AreEqual(b, output.Bytes.ToArray());
		}
	}
}
