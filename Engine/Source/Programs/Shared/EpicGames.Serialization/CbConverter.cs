// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq.Expressions;
using System.Reflection;
using System.Threading;
using EpicGames.Core;
using EpicGames.Serialization.Converters;

namespace EpicGames.Serialization
{
	/// <summary>
	/// Attribute declaring the converter to use for a type
	/// </summary>
	/// <param name="converterType"></param>
	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Struct | AttributeTargets.Property)]
	public sealed class CbConverterAttribute(Type converterType) : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type ConverterType { get; } = converterType;
	}

	/// <summary>
	/// Base class for all converters.
	/// </summary>
	public abstract partial class CbConverter
	{
		/// <summary>
		/// Reads an object from a field
		/// </summary>
		/// <param name="field"></param>
		/// <returns></returns>
		public abstract object? ReadObject(CbField field);

		/// <summary>
		/// Writes an object to a field
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		public abstract void WriteObject(CbWriter writer, object? value);

		/// <summary>
		/// Writes an object to a named field, if not equal to the default value
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public abstract void WriteNamedObject(CbWriter writer, CbFieldName name, object? value);
	}

	/// <summary>
	/// Converter for a particular type
	/// </summary>
	public abstract class CbConverter<T> : CbConverter
	{
		/// <inheritdoc/>
		public sealed override object? ReadObject(CbField field) => Read(field);

		/// <inheritdoc/>
		public sealed override void WriteObject(CbWriter writer, object? value) => Write(writer, (T)value!);

		/// <inheritdoc/>
		public sealed override void WriteNamedObject(CbWriter writer, CbFieldName name, object? value) => WriteNamed(writer, name, (T)value!);

		/// <summary>
		/// Reads an object from a field
		/// </summary>
		/// <param name="field"></param>
		/// <returns></returns>
		public abstract T Read(CbField field);

		/// <summary>
		/// Writes an object to a field
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		public abstract void Write(CbWriter writer, T value);

		/// <summary>
		/// Writes an object to a named field, if not equal to the default value
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public abstract void WriteNamed(CbWriter writer, CbFieldName name, T value);
	}

	/// <summary>
	/// Interface for obtaining static methods that can be invoked directly
	/// </summary>
	public interface ICbConverterMethods
	{
		/// <summary>
		/// Method with the signature CbField -> T
		/// </summary>
		public MethodInfo ReadMethod { get; }

		/// <summary>
		/// Method with the signature CbWriter, T -> void
		/// </summary>
		public MethodInfo WriteMethod { get; }

		/// <summary>
		/// Method with the signature CbWriter, CbFieldName, T -> void
		/// </summary>
		public MethodInfo WriteNamedMethod { get; }
	}

	/// <summary>
	/// Helper class for wrapping regular converters in a ICbConverterMethods interface
	/// </summary>
	public static class CbConverterMethods
	{
		class CbConverterMethodsWrapper<TConverter, T> : ICbConverterMethods where TConverter : CbConverter<T>
		{
			static TConverter s_staticConverter = null!;

			static T Read(CbField field) => s_staticConverter.Read(field);
			static void Write(CbWriter writer, T value) => s_staticConverter.Write(writer, value);
			static void WriteNamed(CbWriter writer, CbFieldName name, T value) => s_staticConverter.WriteNamed(writer, name, value);

			public MethodInfo ReadMethod { get; } = GetMethodInfo(() => Read(null!));
			public MethodInfo WriteMethod { get; } = GetMethodInfo(() => Write(null!, default!));
			public MethodInfo WriteNamedMethod { get; } = GetMethodInfo(() => WriteNamed(null!, default, default!));

			public CbConverterMethodsWrapper(TConverter converter)
			{
				s_staticConverter = converter;
			}

			static MethodInfo GetMethodInfo(Expression<Action> expr)
			{
				return ((MethodCallExpression)expr.Body).Method;
			}
		}

		static readonly Dictionary<PropertyInfo, ICbConverterMethods> s_propertyToMethods = [];
		static readonly Dictionary<Type, ICbConverterMethods> s_typeToMethods = [];

		static ICbConverterMethods CreateWrapper(Type type, CbConverter converter)
		{
			Type converterMethodsType = typeof(CbConverterMethodsWrapper<,>).MakeGenericType(converter.GetType(), type);
			return (ICbConverterMethods)Activator.CreateInstance(converterMethodsType, [converter])!;
		}

		/// <summary>
		/// Gets a <see cref="ICbConverterMethods"/> interface for the given property
		/// </summary>
		/// <param name="property"></param>
		/// <returns></returns>
		public static ICbConverterMethods Get(PropertyInfo property)
		{
			ICbConverterMethods? methods;
			if (!s_propertyToMethods.TryGetValue(property, out methods))
			{
				CbConverter converter = CbConverter.GetConverter(property);
				methods = (converter as ICbConverterMethods) ?? CreateWrapper(property.PropertyType, converter);
				s_propertyToMethods.Add(property, methods);
			}
			return methods;
		}

		/// <summary>
		/// Gets a <see cref="ICbConverterMethods"/> interface for the given type
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		public static ICbConverterMethods Get(Type type)
		{
			ICbConverterMethods? methods;
			if (!s_typeToMethods.TryGetValue(type, out methods))
			{
				CbConverter converter = CbConverter.GetConverter(type);
				methods = (converter as ICbConverterMethods) ?? CreateWrapper(type, converter);
				s_typeToMethods.Add(type, methods);
			}
			return methods;
		}
	}

	/// <summary>
	/// Base class for converter factories
	/// </summary>
	public abstract class CbConverterFactory
	{
		/// <summary>
		/// Create a converter for the given type
		/// </summary>
		/// <param name="type">The type to create a converter for</param>
		/// <returns>The converter instance, or null if this factory does not support the given type</returns>
		public abstract CbConverter? CreateConverter(Type type);
	}

	/// <summary>
	/// Utility functions for creating converters
	/// </summary>
	public partial class CbConverter
	{
		/// <summary>
		/// Object used for locking access to shared objects
		/// </summary>
		static readonly Lock s_lockObject = new();

		/// <summary>
		/// Cache of property to converter
		/// </summary>
		static readonly Dictionary<PropertyInfo, CbConverter> s_propertyToConverter = [];

		/// <summary>
		/// Cache of type to converter
		/// </summary>
		private static Dictionary<Type, CbConverter> TypeToConverter { get; } = new Dictionary<Type, CbConverter>()
		{
			[typeof(bool)] = new CbPrimitiveConverter<bool>(x => x.AsBool(), (w, v) => w.WriteBoolValue(v), (w, n, v) => w.WriteBool(n, v)),
			[typeof(int)] = new CbPrimitiveConverter<int>(x => x.AsInt32(), (w, v) => w.WriteIntegerValue(v), (w, n, v) => w.WriteInteger(n, v)),
			[typeof(uint)] = new CbPrimitiveConverter<uint>(x => x.AsUInt32(), (w, v) => w.WriteIntegerValue(v), (w, n, v) => w.WriteInteger(n, v)),
			[typeof(long)] = new CbPrimitiveConverter<long>(x => x.AsInt64(), (w, v) => w.WriteIntegerValue(v), (w, n, v) => w.WriteInteger(n, v)),
			[typeof(ulong)] = new CbPrimitiveConverter<ulong>(x => x.AsUInt64(), (w, v) => w.WriteIntegerValue(v), (w, n, v) => w.WriteInteger(n, v)),
			[typeof(double)] = new CbPrimitiveConverter<double>(x => x.AsDouble(), (w, v) => w.WriteDoubleValue(v), (w, n, v) => w.WriteDouble(n, v)),
			[typeof(Utf8String)] = new CbPrimitiveConverter<Utf8String>(x => x.AsUtf8String(), (w, v) => w.WriteUtf8StringValue(v), (w, n, v) => w.WriteUtf8String(n, v)),
			[typeof(IoHash)] = new CbPrimitiveConverter<IoHash>(x => x.AsHash(), (w, v) => w.WriteHashValue(v), (w, n, v) => w.WriteHash(n, v)),
			[typeof(CbObjectAttachment)] = new CbPrimitiveConverter<CbObjectAttachment>(x => x.AsObjectAttachment(), (w, v) => w.WriteObjectAttachmentValue(v.Hash), (w, n, v) => w.WriteObjectAttachment(n, v.Hash)),
			[typeof(CbBinaryAttachment)] = new CbPrimitiveConverter<CbBinaryAttachment>(x => x.AsBinaryAttachment(), (w, v) => w.WriteBinaryAttachmentValue(v.Hash), (w, n, v) => w.WriteBinaryAttachment(n, v.Hash)),
			[typeof(DateTime)] = new CbPrimitiveConverter<DateTime>(x => x.AsDateTime(), (w, v) => w.WriteDateTimeValue(v), (w, n, v) => w.WriteDateTime(n, v)),
			[typeof(string)] = new CbPrimitiveConverter<string>(x => x.AsString(), (w, v) => w.WriteStringValue(v), (w, n, v) => w.WriteString(n, v)),
			[typeof(Guid)] = new CbPrimitiveConverter<Guid>(x => x.AsUuid(), (w, v) => w.WriteUuidValue(v), (w, n, v) => w.WriteUuid(n, v)),
			[typeof(ReadOnlyMemory<byte>)] = new CbPrimitiveConverter<ReadOnlyMemory<byte>>(x => x.AsBinary(), (w, v) => w.WriteBinaryValue(v), (w, n, v) => w.WriteBinary(n, v)),
			[typeof(byte[])] = new CbPrimitiveConverter<byte[]>(x => x.AsBinaryArray(), (w, v) => w.WriteBinaryArrayValue(v), (w, n, v) => w.WriteBinaryArray(n, v)),
			[typeof(CbField)] = new CbFieldConverter(),
			[typeof(CbObject)] = new CbObjectConverter()
		};

		/// <summary>
		/// List of converter factories. Must be 
		/// </summary>
		public static List<CbConverterFactory> ConverterFactories { get; } =
		[
			new CbClassConverterFactory(),
			new CbStringConverterFactory(),
			new CbEnumConverterFactory(),
			new CbListConverterFactory(),
			new CbArrayConverterFactory(),
			new CbDictionaryConverterFactory(),
			new CbNullableConverterFactory(),
		];

		/// <summary>
		/// Class used to cache values returned by CreateConverterInfo without having to do a dictionary lookup.
		/// </summary>
		/// <typeparam name="T">The type to be converted</typeparam>
		class CbConverterCache<T>
		{
			/// <summary>
			/// The converter instance
			/// </summary>
			public static CbConverter<T> Instance { get; } = CreateConverter();

			/// <summary>
			/// Create a typed converter
			/// </summary>
			/// <returns></returns>
			static CbConverter<T> CreateConverter()
			{
				CbConverter converter = GetConverter(typeof(T));
				return (converter as CbConverter<T>) ?? new CbConverterWrapper(converter);
			}

			/// <summary>
			/// Wrapper class to convert an untyped converter into a typed one
			/// </summary>
			class CbConverterWrapper(CbConverter inner) : CbConverter<T>
			{
				/// <inheritdoc/>
				public override T Read(CbField field) => (T)inner.ReadObject(field)!;

				/// <inheritdoc/>
				public override void Write(CbWriter writer, T value) => inner.WriteObject(writer, value);

				/// <inheritdoc/>
				public override void WriteNamed(CbWriter writer, CbFieldName name, T value) => inner.WriteNamedObject(writer, name, value);
			}
		}

		/// <summary>
		/// Gets the converter for a particular property
		/// </summary>
		/// <param name="property"></param>
		/// <returns></returns>
		public static CbConverter GetConverter(PropertyInfo property)
		{
			CbConverterAttribute? converterAttribute = property.GetCustomAttribute<CbConverterAttribute>();
			if (converterAttribute != null)
			{
				Type converterType = converterAttribute.ConverterType;
				lock (s_lockObject)
				{
					CbConverter? converter;
					if (!s_propertyToConverter.TryGetValue(property, out converter))
					{
						converter = (CbConverter?)Activator.CreateInstance(converterType)!;
						s_propertyToConverter.Add(property, converter);
					}
					return converter;
				}
			}
			return GetConverter(property.PropertyType);
		}

		/// <summary>
		/// Adds a new converter to the type to converter mapping if it does not exist
		/// </summary>
		/// <param name="type">The type to convert to/from</param>
		/// <param name="converter">The converter that converts to/from the type</param>
		public static void AddConverter(Type type, CbConverter converter)
		{
			lock (s_lockObject)
			{
				TypeToConverter.TryAdd(type, converter);
			}
		}

		/// <summary>
		/// Gets the converter for a particular type
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		public static CbConverter GetConverter(Type type)
		{
			CbConverter? converter;
			lock (s_lockObject)
			{
				if (!TypeToConverter.TryGetValue(type, out converter))
				{
					CbConverterAttribute? converterAttribute = type.GetCustomAttribute<CbConverterAttribute>();
					if (converterAttribute != null)
					{
						Type converterType = converterAttribute.ConverterType;
						if (type.IsGenericType && converterType.IsGenericTypeDefinition)
						{
							converterType = converterType.MakeGenericType(type.GetGenericArguments());
						}
						converter = (CbConverter?)Activator.CreateInstance(converterType)!;
					}
					else
					{
						for (int idx = ConverterFactories.Count - 1; idx >= 0 && converter == null; idx--)
						{
							converter = ConverterFactories[idx].CreateConverter(type);
						}

						if (converter == null)
						{
							throw new CbException($"Unable to create converter for {type.Name}");
						}
					}
					TypeToConverter.Add(type, converter!);
				}
			}
			return converter;
		}

		/// <summary>
		/// Gets the converter for a given type
		/// </summary>
		/// <typeparam name="T">Type to retrieve the converter for</typeparam>
		/// <returns></returns>
		public static CbConverter<T> GetConverter<T>()
		{
			try
			{
				return CbConverterCache<T>.Instance;
			}
			catch (TypeInitializationException ex) when (ex.InnerException != null)
			{
				throw ex.InnerException;
			}
		}
	}
}

