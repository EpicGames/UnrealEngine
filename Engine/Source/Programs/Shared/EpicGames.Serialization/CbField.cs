// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.Globalization;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using EpicGames.Core;

#pragma warning disable CA1721 // Property names should not match get methods
#pragma warning disable CA1028 // Enum Storage should be Int32

namespace EpicGames.Serialization
{
	/// <summary>
	/// Field types and flags for FCbField[View].
	///
	/// DO NOT CHANGE THE VALUE OF ANY MEMBERS OF THIS ENUM!
	/// BACKWARD COMPATIBILITY REQUIRES THAT THESE VALUES BE FIXED!
	/// SERIALIZATION USES HARD-CODED CONSTANTS BASED ON THESE VALUES!
	/// </summary>
	[Flags]
	public enum CbFieldType : byte
	{
		/// <summary>
		/// A field type that does not occur in a valid object.
		/// </summary>
		None = 0x00,

		/// <summary>
		/// Null. Payload is empty.
		/// </summary>
		Null = 0x01,

		/// <summary>
		/// Object is an array of fields with unique non-empty names.
		///
		/// Payload is a VarUInt byte count for the encoded fields followed by the fields.
		/// </summary>
		Object = 0x02,

		/// <summary>
		/// UniformObject is an array of fields with the same field types and unique non-empty names.
		/// 
		/// Payload is a VarUInt byte count for the encoded fields followed by the fields.
		/// </summary>
		UniformObject = 0x03,

		/// <summary>
		/// Array is an array of fields with no name that may be of different types.
		/// 
		/// Payload is a VarUInt byte count, followed by a VarUInt item count, followed by the fields.
		/// </summary>
		Array = 0x04,

		/// <summary>
		/// UniformArray is an array of fields with no name and with the same field type.
		///
		/// Payload is a VarUInt byte count, followed by a VarUInt item count, followed by field type,
		/// followed by the fields without their field type.
		/// </summary>
		UniformArray = 0x05,

		/// <summary> 
		/// Binary. Payload is a VarUInt byte count followed by the data. 
		/// /// </summary>
		Binary = 0x06,

		/// <summary>
		/// String in UTF-8. Payload is a VarUInt byte count then an unterminated UTF-8 string. 
		/// </summary>
		String = 0x07,

		/// <summary>
		/// Non-negative integer with the range of a 64-bit unsigned integer.
		/// 
		/// Payload is the value encoded as a VarUInt.
		/// </summary>
		IntegerPositive = 0x08,

		/// <summary>
		/// Negative integer with the range of a 64-bit signed integer.
		///
		/// Payload is the ones' complement of the value encoded as a VarUInt.
		/// </summary>
		IntegerNegative = 0x09,

		/// <summary>
		/// Single precision float. Payload is one big endian IEEE 754 binary32 float.
		/// /// </summary>
		Float32 = 0x0a,

		/// <summary>
		/// Double precision float. Payload is one big endian IEEE 754 binary64 float. 
		/// </summary>
		Float64 = 0x0b,

		/// <summary>
		/// Boolean false value. Payload is empty. 
		/// </summary>
		BoolFalse = 0x0c,

		/// <summary>
		/// Boolean true value. Payload is empty. 
		/// </summary>
		BoolTrue = 0x0d,

		/// <summary>
		/// CompactBinaryAttachment is a reference to a compact binary attachment stored externally.
		///
		/// Payload is a 160-bit hash digest of the referenced compact binary data.
		/// </summary>
		ObjectAttachment = 0x0e,

		/// <summary>
		/// BinaryAttachment is a reference to a binary attachment stored externally.
		///
		/// Payload is a 160-bit hash digest of the referenced binary data.
		/// </summary>
		BinaryAttachment = 0x0f,

		/// <summary>
		/// Hash. Payload is a 160-bit hash digest. 
		/// </summary>
		Hash = 0x10,

		/// <summary>
		/// UUID/GUID. Payload is a 128-bit UUID as defined by RFC 4122. 
		/// </summary>
		Uuid = 0x11,

		/// <summary>
		/// Date and time between 0001-01-01 00:00:00.0000000 and 9999-12-31 23:59:59.9999999.
		///
		/// Payload is a big endian int64 count of 100ns ticks since 0001-01-01 00:00:00.0000000.
		/// </summary>
		DateTime = 0x12,

		/// <summary>
		/// Difference between two date/time values.
		/// 
		/// Payload is a big endian int64 count of 100ns ticks in the span, and may be negative.
		/// </summary>
		TimeSpan = 0x13,

		/// <summary>
		/// ObjectId is an opaque object identifier. See FCbObjectId.
		///
		/// Payload is a 12-byte object identifier.
		/// </summary>
		ObjectId = 0x14,

		/// <summary>
		/// CustomById identifies the sub-type of its payload by an integer identifier.
		///
		/// Payload is a VarUInt byte count of the sub-type identifier and the sub-type payload, followed
		/// by a VarUInt of the sub-type identifier then the payload of the sub-type.
		/// </summary>
		CustomById = 0x1e,

		/// <summary>
		/// CustomByType identifies the sub-type of its payload by a string identifier.
		///
		/// Payload is a VarUInt byte count of the sub-type identifier and the sub-type payload, followed
		/// by a VarUInt byte count of the unterminated sub-type identifier, then the sub-type identifier
		/// without termination, then the payload of the sub-type.
		/// </summary>
		CustomByName = 0x1f,

		/// <summary>
		/// A transient flag which indicates that the object or array containing this field has stored
		/// the field type before the payload and name. Non-uniform objects and fields will set this.
		/// 
		/// Note: Since the flag must never be serialized, this bit may be repurposed in the future.
		/// </summary>
		HasFieldType = 0x40,

		/// <summary>
		/// A persisted flag which indicates that the field has a name stored before the payload. 
		/// </summary>
		HasFieldName = 0x80,
	}

	/// <summary>
	/// A binary attachment, referenced by <see cref="IoHash"/>
	/// </summary>
	/// <param name="hash">Hash of the referenced object</param>
	[DebuggerDisplay("{Hash}")]
	[JsonConverter(typeof(CbBinaryAttachmentJsonConverter))]
	[TypeConverter(typeof(CbBinaryAttachmentTypeConverter))]
	public readonly struct CbBinaryAttachment(IoHash hash) : IEquatable<CbBinaryAttachment>
	{
		/// <summary>
		/// Attachment with a hash of zero
		/// </summary>
		public static CbBinaryAttachment Zero { get; } = new CbBinaryAttachment(IoHash.Zero);

		/// <summary>
		/// Hash of the referenced object
		/// </summary>
		public IoHash Hash { get; } = hash;

		/// <inheritdoc/>
		public override string ToString() => Hash.ToString();

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is CbBinaryAttachment other && Equals(other);

		/// <inheritdoc/>
		public bool Equals(CbBinaryAttachment other) => other.Hash == Hash;

		/// <inheritdoc/>
		public override int GetHashCode() => Hash.GetHashCode();

		/// <inheritdoc/>
		public static bool operator ==(CbBinaryAttachment lhs, CbBinaryAttachment rhs) => lhs.Hash == rhs.Hash;

		/// <inheritdoc/>
		public static bool operator !=(CbBinaryAttachment lhs, CbBinaryAttachment rhs) => lhs.Hash != rhs.Hash;

		/// <summary>
		/// Convert a hash to a binary attachment 
		/// </summary>
		/// <param name="hash">The attachment to convert</param>
		public static implicit operator CbBinaryAttachment(IoHash hash) => new(hash);

		/// <summary>
		/// Use a binary attachment as a hash
		/// </summary>
		/// <param name="attachment">The attachment to convert</param>
		public static implicit operator IoHash(CbBinaryAttachment attachment) => attachment.Hash;
	}

	/// <summary>
	/// Type converter for IoHash to and from JSON
	/// </summary>
	sealed class CbBinaryAttachmentJsonConverter : JsonConverter<CbBinaryAttachment>
	{
		/// <inheritdoc/>
		public override CbBinaryAttachment Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => IoHash.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CbBinaryAttachment value, JsonSerializerOptions options) => writer.WriteStringValue(value.Hash.ToUtf8String().Span);
	}

	/// <summary>
	/// Type converter from strings to IoHash objects
	/// </summary>
	sealed class CbBinaryAttachmentTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new CbBinaryAttachment(IoHash.Parse((string)value));
	}

	/// <summary>
	/// An object attachment, referenced by <see cref="IoHash"/>
	/// </summary>
	/// <param name="hash">Hash of the referenced object</param>
	[DebuggerDisplay("{Hash}")]
	[JsonConverter(typeof(CbObjectAttachmentJsonConverter))]
	[TypeConverter(typeof(CbObjectAttachmentTypeConverter))]
	public readonly struct CbObjectAttachment(IoHash hash) : IEquatable<CbObjectAttachment>
	{
		/// <summary>
		/// Attachment with a hash of zero
		/// </summary>
		public static CbObjectAttachment Zero { get; } = new CbObjectAttachment(IoHash.Zero);

		/// <summary>
		/// Hash of the referenced object
		/// </summary>
		public IoHash Hash { get; } = hash;

		/// <inheritdoc/>
		public override string ToString() => Hash.ToString();

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is CbObjectAttachment other && Equals(other);

		/// <inheritdoc/>
		public bool Equals(CbObjectAttachment other) => other.Hash == Hash;

		/// <inheritdoc/>
		public override int GetHashCode() => Hash.GetHashCode();

		/// <inheritdoc/>
		public static bool operator ==(CbObjectAttachment lhs, CbObjectAttachment rhs) => lhs.Hash == rhs.Hash;

		/// <inheritdoc/>
		public static bool operator !=(CbObjectAttachment lhs, CbObjectAttachment rhs) => lhs.Hash != rhs.Hash;

		/// <summary>
		/// Use an object attachment as a hash
		/// </summary>
		/// <param name="hash">The attachment to convert</param>
		public static implicit operator CbObjectAttachment(IoHash hash) => new(hash);

		/// <summary>
		/// Use an object attachment as a hash
		/// </summary>
		/// <param name="attachment">The attachment to convert</param>
		public static implicit operator IoHash(CbObjectAttachment attachment) => attachment.Hash;
	}

	/// <summary>
	/// Type converter for IoHash to and from JSON
	/// </summary>
	sealed class CbObjectAttachmentJsonConverter : JsonConverter<CbObjectAttachment>
	{
		/// <inheritdoc/>
		public override CbObjectAttachment Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => IoHash.Parse(reader.GetUtf8String());

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CbObjectAttachment value, JsonSerializerOptions options) => writer.WriteStringValue(value.Hash.ToUtf8String().Span);
	}

	/// <summary>
	/// Type converter from strings to IoHash objects
	/// </summary>
	sealed class CbObjectAttachmentTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new CbObjectAttachment(IoHash.Parse((string)value));
	}

	/// <summary>
	/// Methods that operate on <see cref="CbFieldType"/>.
	/// </summary>
	public static class CbFieldUtils
	{
		private const CbFieldType SerializedTypeMask = (CbFieldType)0b_1001_1111;
		private const CbFieldType TypeMask = (CbFieldType)0b_0001_1111;

		private const CbFieldType ObjectMask = (CbFieldType)0b_0001_1110;
		private const CbFieldType ObjectBase = (CbFieldType)0b_0000_0010;

		private const CbFieldType ArrayMask = (CbFieldType)0b_0001_1110;
		private const CbFieldType ArrayBase = (CbFieldType)0b_0000_0100;

		private const CbFieldType IntegerMask = (CbFieldType)0b_0011_1110;
		private const CbFieldType IntegerBase = (CbFieldType)0b_0000_1000;

		private const CbFieldType FloatMask = (CbFieldType)0b_0001_1100;
		private const CbFieldType FloatBase = (CbFieldType)0b_0000_1000;

		private const CbFieldType BoolMask = (CbFieldType)0b_0001_1110;
		private const CbFieldType BoolBase = (CbFieldType)0b_0000_1100;

		private const CbFieldType AttachmentMask = (CbFieldType)0b_0001_1110;
		private const CbFieldType AttachmentBase = (CbFieldType)0b_0000_1110;

		/// <summary>
		/// Removes flags from the given type
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>Type without flag fields</returns>
		public static CbFieldType GetType(CbFieldType type)
		{
			return type & TypeMask;
		}

		/// <summary>
		/// Gets the serialized type
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>Type without flag fields</returns>
		public static CbFieldType GetSerializedType(CbFieldType type)
		{
			return type & SerializedTypeMask;
		}

		/// <summary>
		/// Tests if the given field has a type
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field has a type</returns>
		public static bool HasFieldType(CbFieldType type)
		{
			return (type & CbFieldType.HasFieldType) != 0;
		}

		/// <summary>
		/// Tests if the given field has a name
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field has a name</returns>
		public static bool HasFieldName(CbFieldType type)
		{
			return (type & CbFieldType.HasFieldName) != 0;
		}

		/// <summary>
		/// Tests if the given field type is none
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is none</returns>
		public static bool IsNone(CbFieldType type)
		{
			return GetType(type) == CbFieldType.None;
		}

		/// <summary>
		/// Tests if the given field type is a null value
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a null</returns>
		public static bool IsNull(CbFieldType type)
		{
			return GetType(type) == CbFieldType.Null;
		}

		/// <summary>
		/// Tests if the given field type is an object
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an object type</returns>
		public static bool IsObject(CbFieldType type)
		{
			return (type & ObjectMask) == ObjectBase;
		}

		/// <summary>
		/// Tests if the given field type is an array
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an array type</returns>
		public static bool IsArray(CbFieldType type)
		{
			return (type & ArrayMask) == ArrayBase;
		}

		/// <summary>
		/// Tests if the given field type is binary
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is binary</returns>
		public static bool IsBinary(CbFieldType type)
		{
			return GetType(type) == CbFieldType.Binary;
		}

		/// <summary>
		/// Tests if the given field type is a string
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an array type</returns>
		public static bool IsString(CbFieldType type)
		{
			return GetType(type) == CbFieldType.String;
		}

		/// <summary>
		/// Tests if the given field type is an integer
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an integer type</returns>
		public static bool IsInteger(CbFieldType type)
		{
			return (type & IntegerMask) == IntegerBase;
		}

		/// <summary>
		/// Tests if the given field type is a float (or integer, due to implicit conversion)
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a float type</returns>
		public static bool IsFloat(CbFieldType type)
		{
			return (type & FloatMask) == FloatBase;
		}

		/// <summary>
		/// Tests if the given field type is a boolean
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an bool type</returns>
		public static bool IsBool(CbFieldType type)
		{
			return (type & BoolMask) == BoolBase;
		}

		/// <summary>
		/// Tests if the given field type is a compact binary attachment
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a compact binary attachment</returns>
		public static bool IsObjectAttachment(CbFieldType type)
		{
			return GetType(type) == CbFieldType.ObjectAttachment;
		}

		/// <summary>
		/// Tests if the given field type is a binary attachment
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a binary attachment</returns>
		public static bool IsBinaryAttachment(CbFieldType type)
		{
			return GetType(type) == CbFieldType.BinaryAttachment;
		}

		/// <summary>
		/// Tests if the given field type is an attachment
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is an attachment type</returns>
		public static bool IsAttachment(CbFieldType type)
		{
			return (type & AttachmentMask) == AttachmentBase;
		}

		/// <summary>
		/// Tests if the given field type is a hash
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a hash</returns>
		public static bool IsHash(CbFieldType type)
		{
			return GetType(type) == CbFieldType.Hash || IsAttachment(type);
		}

		/// <summary>
		/// Tests if the given field type is a UUID
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a UUID</returns>
		public static bool IsUuid(CbFieldType type)
		{
			return GetType(type) == CbFieldType.Uuid;
		}

		/// <summary>
		/// Tests if the given field type is a date/time
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a date/time</returns>
		public static bool IsDateTime(CbFieldType type)
		{
			return GetType(type) == CbFieldType.DateTime;
		}

		/// <summary>
		/// Tests if the given field type is a timespan
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a timespan</returns>
		public static bool IsTimeSpan(CbFieldType type)
		{
			return GetType(type) == CbFieldType.TimeSpan;
		}

		/// <summary>
		/// Tests if the given field type is a object id
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field is a object id</returns>
		public static bool IsObjectId(CbFieldType type)
		{
			return GetType(type) == CbFieldType.ObjectId;
		}

		/// <summary>
		/// Tests if the given field type has fields
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field has fields</returns>
		public static bool HasFields(CbFieldType type)
		{
			CbFieldType noFlags = GetType(type);
			return noFlags >= CbFieldType.Object && noFlags <= CbFieldType.UniformArray;
		}

		/// <summary>
		/// Tests if the given field type has uniform fields (array/object)
		/// </summary>
		/// <param name="type">Type to check</param>
		/// <returns>True if the field has uniform fields</returns>
		public static bool HasUniformFields(CbFieldType type)
		{
			CbFieldType localType = GetType(type);
			return localType == CbFieldType.UniformObject || localType == CbFieldType.UniformArray;
		}

		/// <summary>
		/// Tests if the type is or may contain fields of any attachment type.
		/// </summary>
		public static bool MayContainAttachments(CbFieldType type)
		{
			return IsObject(type) | IsArray(type) | IsAttachment(type);
		}
	}

	/// <summary>
	/// Errors that can occur when accessing a field. */
	/// </summary>
	public enum CbFieldError : byte
	{
		/// <summary>
		/// The field is not in an error state.
		/// </summary>
		None,

		/// <summary>
		/// The value type does not match the requested type.
		/// </summary>
		TypeError,

		/// <summary>
		/// The value is out of range for the requested type.
		/// </summary>
		RangeError,
	}

	/// <summary>
	/// Simplified view of <see cref="CbField"/> in the debugger, for fields with a name
	/// </summary>
	class CbFieldWithNameDebugView
	{
		public string? Name { get; set; }
		public object? Value { get; set; }
	}

	/// <summary>
	/// Simplified view of <see cref="CbField"/> for the debugger
	/// </summary>
	class CbFieldDebugView
	{
		public CbFieldDebugView(CbField field) => Value = field.HasName()
				? new CbFieldWithNameDebugView { Name = field.Name.ToString(), Value = field.Value }
				: field.Value;

		[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
		public object? Value { get; }
	}

	/// <summary>
	/// An atom of data in the compact binary format.
	///
	/// Accessing the value of a field is always a safe operation, even if accessed as the wrong type.
	/// An invalid access will return a default value for the requested type, and set an error code on
	/// the field that can be checked with GetLastError and HasLastError. A valid access will clear an
	/// error from a previous invalid access.
	///
	/// A field is encoded in one or more bytes, depending on its type and the type of object or array
	/// that contains it. A field of an object or array which is non-uniform encodes its field type in
	/// the first byte, and includes the HasFieldName flag for a field in an object. The field name is
	/// encoded in a variable-length unsigned integer of its size in bytes, for named fields, followed
	/// by that many bytes of the UTF-8 encoding of the name with no null terminator.The remainder of
	/// the field is the payload and is described in the field type enum. Every field must be uniquely
	/// addressable when encoded, which means a zero-byte field is not permitted, and only arises in a
	/// uniform array of fields with no payload, where the answer is to encode as a non-uniform array.
	/// </summary>
	[DebuggerDisplay("{DebugValue,nq}")]
	[DebuggerTypeProxy(typeof(CbFieldDebugView))]
	[JsonConverter(typeof(CbFieldJsonConverter))]
	public class CbField : IEquatable<CbField>, IEnumerable<CbField>
	{
		/// <summary>
		/// Type returned for none values
		/// </summary>
		[DebuggerDisplay("<none>")]
		class NoneValueType
		{
		}

		/// <summary>
		/// Special value returned for "none" fields.
		/// </summary>
		static NoneValueType None { get; } = new NoneValueType();

		/// <summary>
		/// Formatter for the debug string
		/// </summary>
		object? DebugValue => HasName() ? $"{Name} = {Value}" : Value;

		/// <summary>
		/// Default empty field
		/// </summary>
		public static CbField Empty { get; } = new CbField();

		/// <summary>
		/// The field type, with the transient HasFieldType flag if the field contains its type
		/// </summary>
		public CbFieldType TypeWithFlags { get; }

		/// <summary>
		/// Data for this field
		/// </summary>
		public ReadOnlyMemory<byte> Memory { get; }

		/// <summary>
		/// Offset of the name with the memory
		/// </summary>
		internal int _nameLen;

		/// <summary>
		/// Offset of the payload within the memory
		/// </summary>
		internal int _payloadOffset;

		/// <summary>
		/// Error for parsing the current field type
		/// </summary>
		public CbFieldError Error { get; private set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public CbField()
			: this(ReadOnlyMemory<byte>.Empty, CbFieldType.None)
		{
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other"></param>
		public CbField(CbField other)
		{
			TypeWithFlags = other.TypeWithFlags;
			Memory = other.Memory;
			_nameLen = other._nameLen;
			_payloadOffset = other._payloadOffset;
			Error = other.Error;
		}

		/// <summary>
		/// Construct a field from a pointer to its data and an optional externally-provided type.
		/// </summary>
		/// <param>Data Pointer to the start of the field data.</param>
		/// <param>Type HasFieldType means that Data contains the type. Otherwise, use the given type.</param>
		public CbField(ReadOnlyMemory<byte> data, CbFieldType type = CbFieldType.HasFieldType)
		{
			int offset = 0;
			if (CbFieldUtils.HasFieldType(type))
			{
				type = (CbFieldType)data.Span[offset] | CbFieldType.HasFieldType;
				offset++;
			}

			if (CbFieldUtils.HasFieldName(type))
			{
				_nameLen = (int)VarInt.ReadUnsigned(data.Slice(offset).Span, out int nameLenByteCount);
				offset += nameLenByteCount + _nameLen;
			}

			Memory = data;
			TypeWithFlags = type;
			_payloadOffset = offset;

			Memory = Memory.Slice(0, (int)Math.Min((ulong)Memory.Length, (ulong)_payloadOffset + GetPayloadSize()));
		}

		/// <summary>
		/// Returns the name of the field if it has a name, otherwise an empty view.
		/// </summary>
		public Utf8String Name => new(Memory.Slice(_payloadOffset - _nameLen, _nameLen));

		/// <summary>
		/// Gets the value of this field
		/// </summary>
		public object? Value
		{
			get
			{
				CbFieldType fieldType = CbFieldUtils.GetType(TypeWithFlags);
				switch (fieldType)
				{
					case CbFieldType.None:
						return None;
					case CbFieldType.Null:
						return null;
					case CbFieldType.Object:
					case CbFieldType.UniformObject:
						return AsObject();
					case CbFieldType.Array:
					case CbFieldType.UniformArray:
						return AsArray();
					case CbFieldType.Binary:
						return AsBinary();
					case CbFieldType.String:
						return AsString();
					case CbFieldType.IntegerPositive:
						return AsUInt64();
					case CbFieldType.IntegerNegative:
						return AsInt64();
					case CbFieldType.Float32:
						return AsFloat();
					case CbFieldType.Float64:
						return AsDouble();
					case CbFieldType.BoolFalse:
						return false;
					case CbFieldType.BoolTrue:
						return true;
					case CbFieldType.ObjectAttachment:
						return AsObjectAttachment();
					case CbFieldType.BinaryAttachment:
						return AsBinaryAttachment();
					case CbFieldType.Hash:
						return AsHash();
					case CbFieldType.Uuid:
						return AsUuid();
					case CbFieldType.DateTime:
						return AsDateTime();
					case CbFieldType.TimeSpan:
						return AsTimeSpan();
					case CbFieldType.ObjectId:
						return AsObjectId();
					default:
						throw new NotImplementedException($"Unknown field type ({fieldType})");
				}
			}
		}

		/// <inheritdoc cref="Name"/>
		public Utf8String GetName() => Name;

		/// <summary>
		/// Access the field as an object. Defaults to an empty object on error. 
		/// </summary>
		/// <returns></returns>
		public CbObject AsObject()
		{
			if (CbFieldUtils.IsObject(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return CbObject.FromFieldNoCheck(this);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return CbObject.Empty;
			}
		}

		/// <summary>
		/// Access the field as an array. Defaults to an empty array on error. 
		/// </summary>
		/// <returns></returns>
		public CbArray AsArray()
		{
			if (CbFieldUtils.IsArray(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return CbArray.FromFieldNoCheck(this);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return CbArray.Empty;
			}
		}

		/// <summary>
		/// Access the field as binary data.
		/// </summary>
		/// <returns></returns>
		public ReadOnlyMemory<byte> AsBinary()
		{
			return AsBinary(ReadOnlyMemory<byte>.Empty);
		}

		/// <summary>
		/// Access the field as binary data.
		/// </summary>
		/// <returns></returns>
		public ReadOnlyMemory<byte> AsBinary(ReadOnlyMemory<byte> defaultValue)
		{
			if (CbFieldUtils.IsBinary(TypeWithFlags))
			{
				Error = CbFieldError.None;

				ulong length = VarInt.ReadUnsigned(Payload.Span, out int bytesRead);
				return Payload.Slice(bytesRead, (int)length);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as binary data.
		/// </summary>
		/// <returns></returns>
		public byte[] AsBinaryArray()
		{
			return AsBinaryArray([]);
		}

		/// <summary>
		/// Access the field as binary data.
		/// </summary>
		/// <returns></returns>
		public byte[] AsBinaryArray(byte[] defaultValue)
		{
			return AsBinary(defaultValue).ToArray();
		}

		/// <summary>
		/// Access the field as a UTF-8 string.
		/// </summary>
		/// <returns></returns>
		public string AsString() => AsUtf8String().ToString();

		/// <summary>
		/// Access the field as a UTF-8 string.
		/// </summary>
		/// <returns></returns>
		public string AsString(string defaultValue) => AsUtf8String(new Utf8String(defaultValue)).ToString();

		/// <summary>
		/// Access the field as a UTF-8 string.
		/// </summary>
		/// <returns></returns>
		public Utf8String AsUtf8String()
		{
			return AsUtf8String(default);
		}

		/// <summary>
		/// Access the field as a UTF-8 string. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value to return</param>
		/// <returns></returns>
		public Utf8String AsUtf8String(Utf8String defaultValue)
		{
			if (CbFieldUtils.IsString(TypeWithFlags))
			{
				ulong valueSize = VarInt.ReadUnsigned(Payload.Span, out int valueSizeByteCount);
				if (valueSize >= (1UL << 31))
				{
					Error = CbFieldError.RangeError;
					return defaultValue;
				}
				else
				{
					Error = CbFieldError.None;
					return new Utf8String(Payload.Slice(valueSizeByteCount, (int)valueSize));
				}
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as an int8. Returns the provided default on error.
		/// </summary>
		public sbyte AsInt8(sbyte defaultValue = 0)
		{
			return (sbyte)AsInteger((ulong)defaultValue, 7, true);
		}

		/// <summary>
		/// Access the field as an int16. Returns the provided default on error.
		/// </summary>
		public short AsInt16(short defaultValue = 0)
		{
			return (short)AsInteger((ulong)defaultValue, 15, true);
		}

		/// <summary>
		/// Access the field as an int32. Returns the provided default on error.
		/// </summary>
		public int AsInt32()
		{
			return AsInt32(0);
		}

		/// <summary>
		/// Access the field as an int32. Returns the provided default on error.
		/// </summary>
		public int AsInt32(int defaultValue)
		{
			return (int)AsInteger((ulong)defaultValue, 31, true);
		}

		/// <summary>
		/// Access the field as an int64. Returns the provided default on error.
		/// </summary>
		public long AsInt64()
		{
			return AsInt64(0);
		}

		/// <summary>
		/// Access the field as an int64. Returns the provided default on error.
		/// </summary>
		public long AsInt64(long defaultValue)
		{
			return (long)AsInteger((ulong)defaultValue, 63, true);
		}

		/// <summary>
		/// Access the field as an int8. Returns the provided default on error.
		/// </summary>
		public byte AsUInt8(byte defaultValue = 0)
		{
			return (byte)AsInteger(defaultValue, 8, false);
		}

		/// <summary>
		/// Access the field as an int16. Returns the provided default on error.
		/// </summary>
		public ushort AsUInt16(ushort defaultValue = 0)
		{
			return (ushort)AsInteger(defaultValue, 16, false);
		}

		/// <summary>
		/// Access the field as an int32. Returns the provided default on error.
		/// </summary>
		public uint AsUInt32()
		{
			return AsUInt32(0);
		}

		/// <summary>
		/// Access the field as an int32. Returns the provided default on error.
		/// </summary>
		public uint AsUInt32(uint defaultValue)
		{
			return (uint)AsInteger(defaultValue, 32, false);
		}

		/// <summary>
		/// Access the field as an int64. Returns the provided default on error.
		/// </summary>
		public ulong AsUInt64()
		{
			return AsUInt64(0);
		}

		/// <summary>
		/// Access the field as an int64. Returns the provided default on error.
		/// </summary>
		public ulong AsUInt64(ulong defaultValue)
		{
			return (ulong)AsInteger(defaultValue, 64, false);
		}

		/// <summary>
		/// Access the field as an integer, checking that it's in the correct range
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <param name="magnitudeBits"></param>
		/// <param name="isSigned"></param>
		/// <returns></returns>
		private ulong AsInteger(ulong defaultValue, int magnitudeBits, bool isSigned)
		{
			if (CbFieldUtils.IsInteger(TypeWithFlags))
			{
				// A shift of a 64-bit value by 64 is undefined so shift by one less because magnitude is never zero.
				ulong outOfRangeMask = ~(ulong)1 << (magnitudeBits - 1);
				ulong isNegative = (ulong)(byte)(TypeWithFlags) & 1;

				ulong magnitude = VarInt.ReadUnsigned(Payload.Span, out _);
				ulong value = magnitude ^ (ulong)-(long)(isNegative);

				if ((magnitude & outOfRangeMask) == 0 && (isNegative == 0 || isSigned))
				{
					Error = CbFieldError.None;
					return value;
				}
				else
				{
					Error = CbFieldError.RangeError;
					return defaultValue;
				}
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a float. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public float AsFloat(float defaultValue = 0.0f)
		{
			switch (GetType())
			{
				case CbFieldType.IntegerPositive:
				case CbFieldType.IntegerNegative:
					{
						ulong isNegative = (ulong)TypeWithFlags & 1;
						ulong outOfRangeMask = ~((1UL << /*FLT_MANT_DIG*/ 24) - 1);

						ulong magnitude = VarInt.ReadUnsigned(Payload.Span, out _) + isNegative;
						bool isInRange = (magnitude & outOfRangeMask) == 0;
						Error = isInRange ? CbFieldError.None : CbFieldError.RangeError;
						return isInRange ? (float)((isNegative != 0) ? (float)-(long)magnitude : (float)magnitude) : defaultValue;
					}
				case CbFieldType.Float32:
					Error = CbFieldError.None;
					return BitConverter.Int32BitsToSingle(BinaryPrimitives.ReadInt32BigEndian(Payload.Span));
				case CbFieldType.Float64:
					Error = CbFieldError.RangeError;
					return defaultValue;
				default:
					Error = CbFieldError.TypeError;
					return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a double.
		/// </summary>
		/// <returns>Value of the field</returns>
		public double AsDouble() => AsDouble(0.0);

		/// <summary>
		/// Access the field as a double. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public double AsDouble(double defaultValue)
		{
			switch (GetType())
			{
				case CbFieldType.IntegerPositive:
				case CbFieldType.IntegerNegative:
					{
						ulong isNegative = (ulong)TypeWithFlags & 1;
						ulong outOfRangeMask = ~((1UL << /*DBL_MANT_DIG*/ 53) - 1);

						ulong magnitude = VarInt.ReadUnsigned(Payload.Span, out _) + isNegative;
						bool isInRange = (magnitude & outOfRangeMask) == 0;
						Error = isInRange ? CbFieldError.None : CbFieldError.RangeError;
						return isInRange ? (double)((isNegative != 0) ? (double)-(long)magnitude : (double)magnitude) : defaultValue;
					}
				case CbFieldType.Float32:
					{
						Error = CbFieldError.None;
						return BitConverter.Int32BitsToSingle(BinaryPrimitives.ReadInt32BigEndian(Payload.Span));
					}
				case CbFieldType.Float64:
					{
						Error = CbFieldError.None;
						return BitConverter.Int64BitsToDouble(BinaryPrimitives.ReadInt64BigEndian(Payload.Span));
					}
				default:
					Error = CbFieldError.TypeError;
					return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a bool. Returns the provided default on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public bool AsBool() => AsBool(false);

		/// <summary>
		/// Access the field as a bool. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public bool AsBool(bool defaultValue)
		{
			switch (GetType())
			{
				case CbFieldType.BoolTrue:
					Error = CbFieldError.None;
					return true;
				case CbFieldType.BoolFalse:
					Error = CbFieldError.None;
					return false;
				default:
					Error = CbFieldError.TypeError;
					return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a hash referencing an object attachment. Returns the provided default on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public CbObjectAttachment AsObjectAttachment() => AsObjectAttachment(CbObjectAttachment.Zero);

		/// <summary>
		/// Access the field as a hash referencing an object attachment. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public CbObjectAttachment AsObjectAttachment(CbObjectAttachment defaultValue)
		{
			if (CbFieldUtils.IsObjectAttachment(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return new IoHash(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a hash referencing a binary attachment. Returns the provided default on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public CbBinaryAttachment AsBinaryAttachment() => AsBinaryAttachment(CbBinaryAttachment.Zero);

		/// <summary>
		/// Access the field as a hash referencing a binary attachment. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public CbBinaryAttachment AsBinaryAttachment(CbBinaryAttachment defaultValue)
		{
			if (CbFieldUtils.IsBinaryAttachment(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return new IoHash(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a hash referencing an attachment. Returns the provided default on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public IoHash AsAttachment() => AsAttachment(IoHash.Zero);

		/// <summary>
		/// Access the field as a hash referencing an attachment. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public IoHash AsAttachment(IoHash defaultValue)
		{
			if (CbFieldUtils.IsAttachment(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return new IoHash(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a hash referencing an attachment. Returns the provided default on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public IoHash AsHash() => AsHash(IoHash.Zero);

		/// <summary>
		/// Access the field as a hash referencing an attachment. Returns the provided default on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public IoHash AsHash(IoHash defaultValue)
		{
			if (CbFieldUtils.IsHash(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return new IoHash(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a UUID. Returns a nil UUID on error.
		/// </summary>
		/// <param name="defaultValue">Default value</param>
		/// <returns>Value of the field</returns>
		public Guid AsUuid(Guid defaultValue = default)
		{
			if (CbFieldUtils.IsUuid(TypeWithFlags))
			{
				Error = CbFieldError.None;

				ReadOnlySpan<byte> span = Payload.Span;
				uint a = BinaryPrimitives.ReadUInt32BigEndian(span);
				ushort b = BinaryPrimitives.ReadUInt16BigEndian(span.Slice(4));
				ushort c = BinaryPrimitives.ReadUInt16BigEndian(span.Slice(6));

				return new Guid(a, b, c, span[8], span[9], span[10], span[11], span[12], span[13], span[14], span[15]);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a UUID. Returns a nil UUID on error.
		/// </summary>
		/// <returns>Value of the field</returns>
		public Guid AsUuid() => AsUuid(Guid.Empty);

		/// <summary>
		/// Reads a date time as number of ticks from the stream
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public long AsDateTimeTicks(long defaultValue = 0)
		{
			if (CbFieldUtils.IsDateTime(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return BinaryPrimitives.ReadInt64BigEndian(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Access the field as a DateTime.
		/// </summary>
		/// <returns></returns>
		public DateTime AsDateTime()
		{
			return AsDateTime(new DateTime(0, DateTimeKind.Utc));
		}

		/// <summary>
		/// Access the field as a DateTime.
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public DateTime AsDateTime(DateTime defaultValue)
		{
			return new DateTime(AsDateTimeTicks(defaultValue.ToUniversalTime().Ticks), DateTimeKind.Utc);
		}

		/// <summary>
		/// Reads a timespan as number of ticks from the stream
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public long AsTimeSpanTicks(long defaultValue = 0)
		{
			if (CbFieldUtils.IsTimeSpan(TypeWithFlags))
			{
				Error = CbFieldError.None;
				return BinaryPrimitives.ReadInt64BigEndian(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <summary>
		/// Reads a timespan as number of ticks from the stream
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public TimeSpan AsTimeSpan(TimeSpan defaultValue = default) => new(AsTimeSpanTicks(defaultValue.Ticks));

		/// <summary>
		/// Access the field as a object id
		/// </summary>
		/// <returns></returns>
		public CbObjectId AsObjectId() => AsObjectId(CbObjectId.Zero);

		/// <summary>
		/// Access the field as a object id
		/// </summary>
		/// <param name="defaultValue"></param>
		/// <returns></returns>
		public CbObjectId AsObjectId(CbObjectId defaultValue)
		{
			if (CbFieldUtils.IsObjectId(TypeWithFlags))
			{
				Error = CbFieldError.None;

				return new CbObjectId(Payload.Span);
			}
			else
			{
				Error = CbFieldError.TypeError;
				return defaultValue;
			}
		}

		/// <inheritdoc cref="CbFieldUtils.HasFieldName(CbFieldType)"/>
		public bool HasName() => CbFieldUtils.HasFieldName(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsNull(CbFieldType)"/>
		public bool IsNull() => CbFieldUtils.IsNull(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsObject(CbFieldType)"/>
		public bool IsObject() => CbFieldUtils.IsObject(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsArray(CbFieldType)"/>
		public bool IsArray() => CbFieldUtils.IsArray(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsBinary(CbFieldType)"/>
		public bool IsBinary() => CbFieldUtils.IsBinary(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsString(CbFieldType)"/>
		public bool IsString() => CbFieldUtils.IsString(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsInteger(CbFieldType)"/>
		public bool IsInteger() => CbFieldUtils.IsInteger(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsFloat(CbFieldType)"/>
		public bool IsFloat() => CbFieldUtils.IsFloat(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsBool(CbFieldType)"/>
		public bool IsBool() => CbFieldUtils.IsBool(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsObjectAttachment(CbFieldType)"/>
		public bool IsObjectAttachment() => CbFieldUtils.IsObjectAttachment(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsBinaryAttachment(CbFieldType)"/>
		public bool IsBinaryAttachment() => CbFieldUtils.IsBinaryAttachment(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsAttachment(CbFieldType)"/>
		public bool IsAttachment() => CbFieldUtils.IsAttachment(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsHash(CbFieldType)"/>
		public bool IsHash() => CbFieldUtils.IsHash(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsUuid(CbFieldType)"/>
		public bool IsUuid() => CbFieldUtils.IsUuid(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsDateTime(CbFieldType)"/>
		public bool IsDateTime() => CbFieldUtils.IsDateTime(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsTimeSpan(CbFieldType)"/>
		public bool IsTimeSpan() => CbFieldUtils.IsTimeSpan(TypeWithFlags);

		/// <inheritdoc cref="CbFieldUtils.IsObjectId(CbFieldType)"/>
		public bool IsObjectId() => CbFieldUtils.IsObjectId(TypeWithFlags);

		/// <summary>
		/// Whether the field has a value
		/// </summary>
		/// <param name="field"></param>
		public static explicit operator bool(CbField field) => field.HasValue();

		/// <summary>
		/// Whether the field has a value.
		///
		/// All fields in a valid object or array have a value. A field with no value is returned when
		/// finding a field by name fails or when accessing an iterator past the end.
		/// </summary>
		public bool HasValue() => !CbFieldUtils.IsNone(TypeWithFlags);

		/// <summary>
		/// Whether the last field access encountered an error.
		/// </summary>
		public bool HasError() => Error != CbFieldError.None;

		/// <inheritdoc cref="Error"/>
		public CbFieldError GetError() => Error;

		/// <summary>
		/// Returns the size of the field in bytes, including the type and name
		/// </summary>
		/// <returns></returns>
		public int GetSize() => sizeof(CbFieldType) + GetViewNoType().Length;

		/// <summary>
		/// Calculate the hash of the field, including the type and name.
		/// </summary>
		/// <returns></returns>
		public Blake3Hash GetHash()
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				AppendHash(hasher);

				byte[] hash = new byte[32];
				hasher.Finalize(hash);

				return new Blake3Hash(hash);
			}
		}

		/// <summary>
		/// Append the hash of the field, including the type and name
		/// </summary>
		/// <param name="hasher"></param>
		void AppendHash(Blake3.Hasher hasher)
		{
			Span<byte> data = [(byte)CbFieldUtils.GetSerializedType(TypeWithFlags)];
			hasher.Update(data);

			hasher.Update(GetViewNoType().Span);
		}

		/// <inheritdoc/>
		public override int GetHashCode() => throw new NotImplementedException();

		/// <inheritdoc/>
		public override bool Equals(object? other) => Equals(other as CbField);

		/// <summary>
		/// Whether this field is identical to the other field.
		/// 
		/// Performs a deep comparison of any contained arrays or objects and their fields. Comparison
		/// assumes that both fields are valid and are written in the canonical format. Fields must be
		/// written in the same order in arrays and objects, and name comparison is case sensitive. If
		/// these assumptions do not hold, this may return false for equivalent inputs. Validation can
		/// be performed with ValidateCompactBinary, except for field order and field name case.
		/// </summary>
		/// <param name="other"></param>
		/// <returns></returns>
		public bool Equals(CbField? other)
		{
			return other != null && CbFieldUtils.GetSerializedType(TypeWithFlags) == CbFieldUtils.GetSerializedType(other.TypeWithFlags) && GetViewNoType().Span.SequenceEqual(other.GetViewNoType().Span);
		}

		/// <summary>
		/// Copy the field into a buffer of exactly GetSize() bytes, including the type and name.
		/// </summary>
		/// <param name="buffer"></param>
		public void CopyTo(Span<byte> buffer)
		{
			buffer[0] = (byte)CbFieldUtils.GetSerializedType(TypeWithFlags);
			GetViewNoType().Span.CopyTo(buffer.Slice(1));
		}

		/// <summary>
		/// Invoke the visitor for every attachment in the field.
		/// </summary>
		/// <param name="visitor"></param>
		public void IterateAttachments(Action<CbField> visitor)
		{
			switch (GetType())
			{
				case CbFieldType.Object:
				case CbFieldType.UniformObject:
					CbObject.FromFieldNoCheck(this).IterateAttachments(visitor);
					break;
				case CbFieldType.Array:
				case CbFieldType.UniformArray:
					CbArray.FromFieldNoCheck(this).IterateAttachments(visitor);
					break;
				case CbFieldType.ObjectAttachment:
				case CbFieldType.BinaryAttachment:
					visitor(this);
					break;
			}
		}

		/// <summary>
		/// Try to get a view of the field as it would be serialized, such as by CopyTo.
		///
		/// A view is available if the field contains its type. Access the equivalent for other fields
		/// through FCbField::GetBuffer, FCbField::Clone, or CopyTo.
		/// </summary>
		/// <param name="outView"></param>
		/// <returns></returns>
		public bool TryGetView(out ReadOnlyMemory<byte> outView)
		{
			if (CbFieldUtils.HasFieldType(TypeWithFlags))
			{
				outView = Memory;
				return true;
			}
			else
			{
				outView = ReadOnlyMemory<byte>.Empty;
				return false;
			}
		}

		/// <summary>
		/// Find a field of an object by case-sensitive name comparison, otherwise a field with no value.
		/// </summary>
		/// <param name="name"></param>
		/// <returns></returns>
#pragma warning disable CA1043 // Use Integral Or String Argument For Indexers
		public CbField this[CbFieldName name] => this.FirstOrDefault(field => field.Name == name.Text) ?? Empty;
#pragma warning restore CA1043 // Use Integral Or String Argument For Indexers

		/// <summary>
		/// Create an iterator for the fields of an array or object, otherwise an empty iterator.
		/// </summary>
		/// <returns></returns>
		public CbFieldIterator CreateIterator()
		{
			CbFieldType localTypeWithFlags = TypeWithFlags;
			if (CbFieldUtils.HasFields(localTypeWithFlags))
			{
				ReadOnlyMemory<byte> payloadBytes = Payload;
				int payloadSizeByteCount;
				int payloadSize = (int)VarInt.ReadUnsigned(payloadBytes.Span, out payloadSizeByteCount);
				payloadBytes = payloadBytes.Slice(payloadSizeByteCount);
				int numByteCount = CbFieldUtils.IsArray(localTypeWithFlags) ? (int)VarInt.Measure(payloadBytes.Span) : 0;
				if (payloadSize > numByteCount)
				{
					payloadBytes = payloadBytes.Slice(numByteCount);

					CbFieldType uniformType = CbFieldType.HasFieldType;
					if (CbFieldUtils.HasUniformFields(TypeWithFlags))
					{
						uniformType = (CbFieldType)payloadBytes.Span[0];
						payloadBytes = payloadBytes.Slice(1);
					}

					return new CbFieldIterator(payloadBytes, uniformType);
				}
			}
			return new CbFieldIterator(ReadOnlyMemory<byte>.Empty, CbFieldType.HasFieldType);
		}

		/// <inheritdoc/>
		public IEnumerator<CbField> GetEnumerator()
		{
			for (CbFieldIterator iter = CreateIterator(); iter; iter.MoveNext())
			{
				yield return iter.Current;
			}
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <summary>
		/// Returns a view of the name and value payload, which excludes the type.
		/// </summary>
		/// <returns></returns>
		private ReadOnlyMemory<byte> GetViewNoType()
		{
			int nameSize = CbFieldUtils.HasFieldName(TypeWithFlags) ? _nameLen + (int)VarInt.MeasureUnsigned((uint)_nameLen) : 0;
			return Memory.Slice(_payloadOffset - nameSize);
		}

		/// <summary>
		/// Accessor for the payload
		/// </summary>
		internal ReadOnlyMemory<byte> Payload => Memory.Slice(_payloadOffset);

		/// <summary>
		/// Returns a view of the value payload, which excludes the type and name.
		/// </summary>
		/// <returns></returns>
		internal ReadOnlyMemory<byte> GetPayloadView() => Memory.Slice(_payloadOffset);

		/// <summary>
		/// Returns the type of the field excluding flags.
		/// </summary>
		internal new CbFieldType GetType() => CbFieldUtils.GetType(TypeWithFlags);

		/// <summary>
		/// Returns the type of the field excluding flags.
		/// </summary>
		internal CbFieldType GetTypeWithFlags() => TypeWithFlags;

		/// <summary>
		/// Returns the size of the value payload in bytes, which is the field excluding the type and name.
		/// </summary>
		/// <returns>Size of the payload</returns>
		public ulong GetPayloadSize()
		{
			switch (GetType())
			{
				case CbFieldType.None:
				case CbFieldType.Null:
					return 0;
				case CbFieldType.Object:
				case CbFieldType.UniformObject:
				case CbFieldType.Array:
				case CbFieldType.UniformArray:
				case CbFieldType.Binary:
				case CbFieldType.String:
					{
						ulong payloadSize = VarInt.ReadUnsigned(Payload.Span, out int bytesRead);
						return payloadSize + (ulong)bytesRead;
					}
				case CbFieldType.IntegerPositive:
				case CbFieldType.IntegerNegative:
					{
						return (ulong)VarInt.Measure(Payload.Span);
					}
				case CbFieldType.Float32:
					return 4;
				case CbFieldType.Float64:
					return 8;
				case CbFieldType.BoolFalse:
				case CbFieldType.BoolTrue:
					return 0;
				case CbFieldType.ObjectAttachment:
				case CbFieldType.BinaryAttachment:
				case CbFieldType.Hash:
					return 20;
				case CbFieldType.Uuid:
					return 16;
				case CbFieldType.DateTime:
				case CbFieldType.TimeSpan:
					return 8;
				case CbFieldType.ObjectId:
					return 12;
				default:
					return 0;
			}
		}

		/// <summary>
		/// Verifies if type and value is equal between two fields, ignoring the name
		/// </summary>
		/// <param name="other"></param>
		/// <returns></returns>
		public bool ValueEquals(CbField? other)
		{
			return other != null && CbFieldUtils.GetType(TypeWithFlags) == CbFieldUtils.GetType(other.TypeWithFlags) && GetPayloadView().Span.SequenceEqual(other.GetPayloadView().Span);
		}
	}

	/// <summary>
	/// Converter to and from JSON objects
	/// </summary>
	public class CbFieldJsonConverter : JsonConverter<CbField>
	{
		/// <inheritdoc/>
		public override CbField Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CbField field, JsonSerializerOptions options)
		{
			WriteField(writer, field, options);
		}

		/// <summary>
		/// Writes a CbField to a Jtf8JsonWriter
		/// </summary>
		/// <param name="writer">The json writer</param>
		/// <param name="field">The field to write</param>
		/// <param name="options">Serialization options</param>
		/// <exception cref="NotImplementedException"></exception>
		public static void WriteField(Utf8JsonWriter writer, CbField field, JsonSerializerOptions options)
		{
			switch (field.GetType())
			{
				case CbFieldType.Null:
					if (field.HasName())
					{
						writer.WriteNull(field.Name.Span);
					}
					else
					{
						writer.WriteNullValue();
					}
					break;
				case CbFieldType.Object:
				case CbFieldType.UniformObject:
					if (field.HasName())
					{
						writer.WriteStartObject(field.Name.Span);
					}
					else
					{
						writer.WriteStartObject();
					}

					foreach (CbField member in field.AsObject())
					{
						WriteField(writer, member, options);
					}
					writer.WriteEndObject();
					break;
				case CbFieldType.Array:
				case CbFieldType.UniformArray:
					if (field.HasName())
					{
						writer.WriteStartArray(field.Name.Span);
					}
					else
					{
						writer.WriteStartArray();
					}

					foreach (CbField element in field.AsArray())
					{
						WriteField(writer, element, options);
					}
					writer.WriteEndArray();
					break;
				case CbFieldType.Binary:
					if (field.HasName())
					{
						writer.WriteBase64String(field.Name.Span, field.AsBinary().Span);
					}
					else
					{
						writer.WriteBase64StringValue(field.AsBinary().Span);
					}
					break;
				case CbFieldType.String:
					if (field.HasName())
					{
						writer.WriteString(field.Name.Span, field.AsUtf8String().Span);
					}
					else
					{
						writer.WriteStringValue(field.AsUtf8String().Span);
					}
					break;
				case CbFieldType.IntegerPositive:
					if (field.HasName())
					{
						writer.WriteNumber(field.Name.Span, field.AsUInt64());
					}
					else
					{
						writer.WriteNumberValue(field.AsUInt64());
					}
					break;
				case CbFieldType.IntegerNegative:
					if (field.HasName())
					{
						writer.WriteNumber(field.Name.Span, field.AsInt64());
					}
					else
					{
						writer.WriteNumberValue(field.AsInt64());
					}
					break;
				case CbFieldType.Float32:
				case CbFieldType.Float64:
					if (field.HasName())
					{
						writer.WriteNumber(field.Name.Span, field.AsDouble());
					}
					else
					{
						writer.WriteNumberValue(field.AsDouble());
					}
					break;
				case CbFieldType.BoolFalse:
				case CbFieldType.BoolTrue:
					if (field.HasName())
					{
						writer.WriteBoolean(field.Name.Span, field.AsBool());
					}
					else
					{
						writer.WriteBooleanValue(field.AsBool());
					}
					break;
				case CbFieldType.ObjectAttachment:
				case CbFieldType.BinaryAttachment:
				case CbFieldType.Hash:
					if (field.HasName())
					{
						writer.WriteString(field.Name.Span, field.AsHash().ToString());
					}
					else
					{
						writer.WriteStringValue(field.AsHash().ToString());
					}
					break;
				case CbFieldType.ObjectId:
					if (field.HasName())
					{
						writer.WriteString(field.Name.Span, field.AsObjectId().ToString());
					}
					else
					{
						writer.WriteStringValue(field.AsObjectId().ToString());
					}
					break;
				case CbFieldType.Uuid:
					if (field.HasName())
					{
						writer.WriteString(field.Name.Span, field.AsUuid().ToString());
					}
					else
					{
						writer.WriteStringValue(field.AsUuid().ToString());
					}
					break;
				case CbFieldType.DateTime:
					if (field.HasName())
					{
						writer.WriteNumber(field.Name.Span, field.AsDateTimeTicks());
					}
					else
					{
						writer.WriteNumberValue(field.AsDateTimeTicks());
					}
					break;
				case CbFieldType.TimeSpan:
					if (field.HasName())
					{
						writer.WriteNumber(field.Name.Span, field.AsTimeSpanTicks());
					}
					else
					{
						writer.WriteNumberValue(field.AsTimeSpanTicks());
					}
					break;
				default:
					throw new NotImplementedException($"Unhandled type '{field.GetType()}' in cb-json converter");
			}
		}
	}

	/// <summary>
	/// Converter to and from JSON objects
	/// </summary>
	public class CbObjectJsonConverter : JsonConverter<CbObject>
	{
		/// <inheritdoc/>
		public override CbObject Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			CbWriter writer = new();

			using JsonDocument doc = JsonDocument.ParseValue(ref reader);
			CbJsonReader.ReadJsonField(doc.RootElement, null, writer);
			return writer.ToObject();
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CbObject o, JsonSerializerOptions options)
		{
			writer.WriteStartObject();

			foreach (CbField member in o)
			{
				CbFieldJsonConverter.WriteField(writer, member, options);
			}
			writer.WriteEndObject();
		}
	}
	/// <summary>
	/// Enumerator for contents of a field
	/// </summary>
	/// <param name="data"></param>
	/// <param name="uniformType"></param>
	public sealed class CbFieldEnumerator(ReadOnlyMemory<byte> data, CbFieldType uniformType) : IEnumerator<CbField>
	{

		/// <inheritdoc/>
		public CbField Current { get; private set; } = null!;

		/// <inheritdoc/>
		object? IEnumerator.Current => Current;

		/// <inheritdoc/>
		public void Dispose()
		{
		}

		/// <inheritdoc/>
		public void Reset()
		{
			throw new InvalidOperationException();
		}

		/// <inheritdoc/>
		public bool MoveNext()
		{
			if (data.Length > 0)
			{
				Current = new CbField(data, uniformType);
				return true;
			}
			else
			{
				Current = null!;
				return false;
			}
		}

		/// <summary>
		/// Clone this enumerator
		/// </summary>
		/// <returns></returns>
		public CbFieldEnumerator Clone()
		{
			return new CbFieldEnumerator(data, uniformType);
		}
	}

	/// <summary>
	/// Iterator for fields
	/// </summary>
	public class CbFieldIterator
	{
		/// <summary>
		/// The underlying buffer
		/// </summary>
		ReadOnlyMemory<byte> _nextData;

		/// <summary>
		/// Type for all fields
		/// </summary>
		readonly CbFieldType _uniformType;

		/// <summary>
		/// The current iterator
		/// </summary>
		public CbField Current { get; private set; } = null!;

		/// <summary>
		/// Default constructor
		/// </summary>
		public CbFieldIterator()
			: this(ReadOnlyMemory<byte>.Empty, CbFieldType.HasFieldType)
		{
		}

		/// <summary>
		/// Constructor for single field iterator
		/// </summary>
		/// <param name="field"></param>
		private CbFieldIterator(CbField field)
		{
			_nextData = ReadOnlyMemory<byte>.Empty;
			Current = field;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data"></param>
		/// <param name="uniformType"></param>
		public CbFieldIterator(ReadOnlyMemory<byte> data, CbFieldType uniformType)
		{
			_nextData = data;
			_uniformType = uniformType;

			MoveNext();
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other"></param>
		public CbFieldIterator(CbFieldIterator other)
		{
			_nextData = other._nextData;
			_uniformType = other._uniformType;
			Current = other.Current;
		}

		/// <summary>
		/// Construct a field range that contains exactly one field.
		/// </summary>
		/// <param name="field"></param>
		/// <returns></returns>
		public static CbFieldIterator MakeSingle(CbField field)
		{
			return new CbFieldIterator(field);
		}

		/// <summary>
		/// Construct a field range from a buffer containing zero or more valid fields.
		/// </summary>
		/// <param name="view">A buffer containing zero or more valid fields.</param>
		/// <param name="type">HasFieldType means that View contains the type.Otherwise, use the given type.</param>
		/// <returns></returns>
		public static CbFieldIterator MakeRange(ReadOnlyMemory<byte> view, CbFieldType type = CbFieldType.HasFieldType)
		{
			return new CbFieldIterator(view, type);
		}

		/// <summary>
		/// Check if the current value is valid
		/// </summary>
		/// <returns></returns>
		public bool IsValid()
		{
			return Current.GetType() != CbFieldType.None;
		}

		/// <summary>
		/// Accessor for the current value
		/// </summary>
		/// <returns></returns>
		public CbField GetCurrent()
		{
			return Current;
		}

		/// <summary>
		/// Invoke the visitor for every attachment in the field range.
		/// </summary>
		/// <param name="visitor"></param>
		public void IterateRangeAttachments(Action<CbField> visitor)
		{
			// Always iterate over non-uniform ranges because we do not know if they contain an attachment.
			if (CbFieldUtils.HasFieldType(Current.GetTypeWithFlags()))
			{
				for (CbFieldIterator it = new(this); it; ++it)
				{
					if (CbFieldUtils.MayContainAttachments(it.Current.GetTypeWithFlags()))
					{
						it.Current.IterateAttachments(visitor);
					}
				}
			}
			// Only iterate over uniform ranges if the uniform type may contain an attachment.
			else
			{
				if (CbFieldUtils.MayContainAttachments(Current.GetTypeWithFlags()))
				{
					for (CbFieldIterator it = new(this); it; ++it)
					{
						it.Current.IterateAttachments(visitor);
					}
				}
			}
		}

		/// <summary>
		/// Move to the next element
		/// </summary>
		/// <returns></returns>
		public bool MoveNext()
		{
			if (_nextData.Length > 0)
			{
				Current = new CbField(_nextData, _uniformType);
				_nextData = _nextData.Slice(Current.Memory.Length);
				return true;
			}
			else
			{
				Current = CbField.Empty;
				return false;
			}
		}

		/// <summary>
		/// Test whether the iterator is valid
		/// </summary>
		/// <param name="iterator"></param>
		public static implicit operator bool(CbFieldIterator iterator)
		{
			return iterator.IsValid();
		}

		/// <summary>
		/// Move to the next item
		/// </summary>
		/// <param name="iterator"></param>
		/// <returns></returns>
		public static CbFieldIterator operator ++(CbFieldIterator iterator)
		{
			return new CbFieldIterator(iterator._nextData, iterator._uniformType);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public static bool operator ==(CbFieldIterator a, CbFieldIterator b)
		{
			return a.Current.Equals(b.Current);
		}

		/// <inheritdoc/>
		public static bool operator !=(CbFieldIterator a, CbFieldIterator b)
		{
			return !a.Current.Equals(b.Current);
		}
	}

	/// <summary>
	/// Simplified view of <see cref="CbArray"/> for display in the debugger
	/// </summary>
	class CbArrayDebugView(CbArray array)
	{
		[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
		public object?[] Value => [.. array.Select(x => x.Value)];
	}

	/// <summary>
	/// Array of CbField that have no names.
	///
	/// Accessing a field of the array requires iteration. Access by index is not provided because the
	/// cost of accessing an item by index scales linearly with the index.
	/// </summary>
	[DebuggerDisplay("Count = {Count}")]
	[DebuggerTypeProxy(typeof(CbArrayDebugView))]
	public class CbArray : IEnumerable<CbField>
	{
		/// <summary>
		/// The field containing this array
		/// </summary>
		readonly CbField _innerField;

		/// <summary>
		/// Empty array constant
		/// </summary>
		public static CbArray Empty { get; } = new CbArray(new byte[] { (byte)CbFieldType.Array, 1, 0 });

		/// <summary>
		/// Construct an array with no fields
		/// </summary>
		public CbArray()
		{
			_innerField = Empty._innerField;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="field"></param>
		private CbArray(CbField field)
		{
			_innerField = field;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data"></param>
		/// <param name="type"></param>
		public CbArray(ReadOnlyMemory<byte> data, CbFieldType type = CbFieldType.HasFieldType)
		{
			_innerField = new CbField(data, type);
		}

		/// <summary>
		/// Number of items in this array
		/// </summary>
		public int Count
		{
			get
			{
				ReadOnlyMemory<byte> payloadBytes = _innerField.Payload;
				payloadBytes = payloadBytes.Slice(VarInt.Measure(payloadBytes.Span));
				return (int)VarInt.ReadUnsigned(payloadBytes.Span, out int _);
			}
		}

		/// <summary>
		/// Access the array as an array field.
		/// </summary>
		/// <returns></returns>
		public CbField AsField() => _innerField;

		/// <summary>
		/// Construct an array from an array field. No type check is performed!
		/// </summary>
		/// <param name="field"></param>
		/// <returns></returns>
		public static CbArray FromFieldNoCheck(CbField field) => new(field);

		/// <summary>
		/// Returns the size of the array in bytes if serialized by itself with no name.
		/// </summary>
		/// <returns></returns>
		public int GetSize()
		{
			return (int)Math.Min((ulong)sizeof(CbFieldType) + _innerField.GetPayloadSize(), Int32.MaxValue);
		}

		/// <summary>
		/// Calculate the hash of the array if serialized by itself with no name.
		/// </summary>
		/// <returns></returns>
		public Blake3Hash GetHash()
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				AppendHash(hasher);

				byte[] result = new byte[Blake3Hash.NumBytes];
				hasher.Finalize(result);

				return new Blake3Hash(result);
			}
		}

		/// <summary>
		/// Append the hash of the array if serialized by itself with no name.
		/// </summary>
		public void AppendHash(Blake3.Hasher hasher)
		{
			byte[] serializedType = [(byte)_innerField.GetType()];
			hasher.Update(serializedType);
			hasher.Update(_innerField.Payload.Span);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => Equals(obj as CbArray);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32BigEndian(GetHash().Span);

		/// <summary>
		/// Whether this array is identical to the other array.
		///
		/// Performs a deep comparison of any contained arrays or objects and their fields. Comparison
		/// assumes that both fields are valid and are written in the canonical format.Fields must be
		/// written in the same order in arrays and objects, and name comparison is case sensitive.If
		/// these assumptions do not hold, this may return false for equivalent inputs. Validation can
		/// be done with the All mode to check these assumptions about the format of the inputs.
		/// </summary>
		/// <param name="other"></param>
		/// <returns></returns>
		public bool Equals(CbArray? other)
		{
			return other != null && GetType() == other.GetType() && GetPayloadView().Span.SequenceEqual(other.GetPayloadView().Span);
		}

		/// <summary>
		/// Copy the array into a buffer of exactly GetSize() bytes, with no name.
		/// </summary>
		/// <param name="buffer"></param>
		public void CopyTo(Span<byte> buffer)
		{
			buffer[0] = (byte)GetType();
			GetPayloadView().Span.CopyTo(buffer.Slice(1));
		}

		/** Invoke the visitor for every attachment in the array. */
		public void IterateAttachments(Action<CbField> visitor) => CreateIterator().IterateRangeAttachments(visitor);

		/// <summary>
		/// Try to get a view of the array as it would be serialized, such as by CopyTo.
		/// 
		/// A view is available if the array contains its type and has no name. Access the equivalent
		/// for other arrays through FCbArray::GetBuffer, FCbArray::Clone, or CopyTo.
		/// </summary>
		public bool TryGetView(out ReadOnlyMemory<byte> outView)
		{
			if (_innerField.HasName())
			{
				outView = ReadOnlyMemory<byte>.Empty;
				return false;
			}
			return _innerField.TryGetView(out outView);
		}

		/// <inheritdoc cref="CbField.CreateIterator"/>
		public CbFieldIterator CreateIterator() => _innerField.CreateIterator();

		/// <inheritdoc/>
		public IEnumerator<CbField> GetEnumerator() => _innerField.GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		#region Mimic inheritance from CbField

		/// <inheritdoc cref="CbField.GetType"/>
		internal new CbFieldType GetType() => _innerField.GetType();

		/// <inheritdoc cref="CbField.GetPayloadView"/>
		internal ReadOnlyMemory<byte> GetPayloadView() => _innerField.GetPayloadView();

		#endregion
	}

	/// <summary>
	/// Simplified view of <see cref="CbObject"/> for display in the debugger
	/// </summary>
	class CbObjectDebugView(CbObject obj)
	{
		[DebuggerDisplay("{Name}: {Value}")]
		public class Property
		{
			public string? Name { get; set; }
			public object? Value { get; set; }
		}

		[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
		public Property[] Properties => [.. obj.Select(x => new Property { Name = x.Name.ToString(), Value = x.Value })];
	}

	/// <summary>
	/// Array of CbField that have unique names.
	///
	/// Accessing the fields of an object is always a safe operation, even if the requested field does
	/// not exist. Fields may be accessed by name or through iteration. When a field is requested that
	/// is not found in the object, the field that it returns has no value (evaluates to false) though
	/// attempting to access the empty field is also safe, as described by FCbFieldView.
	/// </summary>
	[DebuggerTypeProxy(typeof(CbObjectDebugView))]
	[JsonConverter(typeof(CbObjectJsonConverter))]
	public class CbObject : IEnumerable<CbField>
	{
		/// <summary>
		/// Empty array constant
		/// </summary>
		public static CbObject Empty { get; } = FromFieldNoCheck(new CbField(new byte[] { (byte)CbFieldType.Object, 0 }));

		/// <summary>
		/// The inner field object
		/// </summary>
		private readonly CbField _innerField;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="field"></param>
		private CbObject(CbField field)
		{
			_innerField = new CbField(field.Memory, field.TypeWithFlags);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="buffer"></param>
		/// <param name="fieldType">Explicit type of the data in buffer</param>
		public CbObject(ReadOnlyMemory<byte> buffer, CbFieldType fieldType = CbFieldType.HasFieldType)
		{
			_innerField = new CbField(buffer, fieldType);
		}

		/// <summary>
		/// Builds an object by calling a delegate with a writer
		/// </summary>
		/// <param name="build"></param>
		/// <returns></returns>
		public static CbObject Build(Action<CbWriter> build)
		{
			CbWriter writer = new();
			writer.BeginObject();
			build(writer);
			writer.EndObject();
			return new CbObject(writer.ToByteArray());
		}

		/// <summary>
		/// Find a field by case-sensitive name comparison.
		///
		/// The cost of this operation scales linearly with the number of fields in the object. Prefer to
		/// iterate over the fields only once when consuming an object.
		/// </summary>
		/// <param name="name">The name of the field.</param>
		/// <returns>The matching field if found, otherwise a field with no value.</returns>
		public CbField Find(CbFieldName name) => _innerField[name.Text];

		/// <summary>
		/// Find a field by case-insensitive name comparison.
		/// </summary>
		/// <param name="name">The name of the field.</param>
		/// <returns>The matching field if found, otherwise a field with no value.</returns>
		public CbField FindIgnoreCase(CbFieldName name) => _innerField.FirstOrDefault(field => Utf8StringComparer.OrdinalIgnoreCase.Equals(field.Name, name.Text)) ?? new CbField();

		/// <summary>
		/// Find a field by case-sensitive name comparison.
		/// </summary>
		/// <param name="name">The name of the field.</param>
		/// <returns>The matching field if found, otherwise a field with no value.</returns>
#pragma warning disable CA1043 // Use Integral Or String Argument For Indexers
		public CbField this[CbFieldName name] => _innerField[name.Text];
#pragma warning restore CA1043 // Use Integral Or String Argument For Indexers

		/// <summary>
		/// Gets the underlying field for this object
		/// </summary>
		/// <returns></returns>
		public CbField AsField() => _innerField;

		/// <summary>
		/// Construct an object from an object field. No type check is performed!
		/// </summary>
		/// <param name="field"></param>
		/// <returns></returns>
		public static CbObject FromFieldNoCheck(CbField field) => new(field);

		/// <summary>
		/// Returns the size of the object in bytes if serialized by itself with no name.
		/// </summary>
		/// <returns></returns>
		public int GetSize()
		{
			return sizeof(CbFieldType) + _innerField.Payload.Length;
		}

		/// <summary>
		/// Calculate the hash of the object if serialized by itself with no name.
		/// </summary>
		/// <returns></returns>
		public Blake3Hash GetHash()
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				AppendHash(hasher);

				byte[] data = new byte[Blake3Hash.NumBytes];
				hasher.Finalize(data);

				return new Blake3Hash(data);
			}
		}

		/// <summary>
		/// Append the hash of the object if serialized by itself with no name.
		/// </summary>
		/// <param name="hasher"></param>
		public void AppendHash(Blake3.Hasher hasher)
		{
			byte[] temp = [(byte)_innerField.GetType()];
			hasher.Update(temp);
			hasher.Update(_innerField.Payload.Span);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => Equals(obj as CbObject);

		/// <inheritdoc/>
		public override int GetHashCode() => BinaryPrimitives.ReadInt32BigEndian(GetHash().Span);

		/// <summary>
		/// Whether this object is identical to the other object.
		/// 
		/// Performs a deep comparison of any contained arrays or objects and their fields. Comparison
		/// assumes that both fields are valid and are written in the canonical format. Fields must be
		/// written in the same order in arrays and objects, and name comparison is case sensitive. If
		/// these assumptions do not hold, this may return false for equivalent inputs. Validation can
		/// be done with the All mode to check these assumptions about the format of the inputs.
		/// </summary>
		/// <param name="other"></param>
		/// <returns></returns>
		public bool Equals(CbObject? other)
		{
			return other != null && _innerField.GetType() == other._innerField.GetType() && _innerField.Payload.Span.SequenceEqual(other._innerField.Payload.Span);
		}

		/// <summary>
		/// Copy the object into a buffer of exactly GetSize() bytes, with no name.
		/// </summary>
		/// <param name="buffer"></param>
		public void CopyTo(Span<byte> buffer)
		{
			buffer[0] = (byte)_innerField.GetType();
			_innerField.Payload.Span.CopyTo(buffer.Slice(1));
		}

		/// <summary>
		/// Invoke the visitor for every attachment in the object.
		/// </summary>
		/// <param name="visitor"></param>
		public void IterateAttachments(Action<CbField> visitor) => CreateIterator().IterateRangeAttachments(visitor);

		/// <summary>
		/// Creates a view of the object, excluding the name
		/// </summary>
		/// <returns></returns>
		public ReadOnlyMemory<byte> GetView()
		{
			ReadOnlyMemory<byte> memory;
			if (!TryGetView(out memory))
			{
				byte[] data = new byte[GetSize()];
				CopyTo(data);
				memory = data;
			}
			return memory;
		}

		/// <summary>
		/// Try to get a view of the object as it would be serialized, such as by CopyTo.
		/// 
		/// A view is available if the object contains its type and has no name. Access the equivalent
		/// for other objects through FCbObject::GetBuffer, FCbObject::Clone, or CopyTo.
		/// </summary>
		/// <param name="outView"></param>
		/// <returns></returns>
		public bool TryGetView(out ReadOnlyMemory<byte> outView)
		{
			if (_innerField.HasName())
			{
				outView = ReadOnlyMemory<byte>.Empty;
				return false;
			}
			return _innerField.TryGetView(out outView);
		}

		/// <inheritdoc cref="CbField.CreateIterator"/>
		public CbFieldIterator CreateIterator() => _innerField.CreateIterator();

		/// <inheritdoc/>
		public IEnumerator<CbField> GetEnumerator() => _innerField.GetEnumerator();

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => _innerField.GetEnumerator();

		/// <summary>
		/// Clone this object
		/// </summary>
		/// <param name="obj"></param>
		/// <returns></returns>
		public static CbObject Clone(CbObject obj) => obj;

		#region Conversion to Json
		/// <summary>
		/// Convert this object to JSON
		/// </summary>
		/// <returns></returns>
		public string ToJson()
		{
			ArrayBufferWriter<byte> buffer = new();
			using (Utf8JsonWriter jsonWriter = new(buffer))
			{
				ToJson(jsonWriter, JsonSerializerOptions.Default);
			}
			return Encoding.UTF8.GetString(buffer.WrittenMemory.Span);
		}

		/// <summary>
		/// Write this object to JSON
		/// </summary>
		/// <param name="writer"></param>
		public void ToJson(Utf8JsonWriter writer)
		{
			writer.WriteStartObject();
			foreach (CbField field in _innerField)
			{
				CbFieldJsonConverter.WriteField(writer, field, JsonSerializerOptions.Default);
			}
			writer.WriteEndObject();
		}

		/// <summary>
		/// Write this object to JSON
		/// </summary>
		/// <param name="writer"></param>
		/// <param name="options"></param>
		public void ToJson(Utf8JsonWriter writer, JsonSerializerOptions options)
		{
			writer.WriteStartObject();
			foreach (CbField field in _innerField)
			{
				CbFieldJsonConverter.WriteField(writer, field, options);
			}
			writer.WriteEndObject();
		}

		/// <summary>
		/// Converts a json object into a CbObject
		/// </summary>
		/// <returns></returns>
		public static CbObject FromJson(string json)
		{
			return CbJsonReader.FromJson(json);
		}

		#endregion
	}

	/// <summary>
	/// ObjectId used within compact binary, similar to an uuid type 4
	/// </summary>
	/// <param name="a">time component</param>
	/// <param name="b">serial component</param>
	/// <param name="c">run component</param>
	[JsonConverter(typeof(CbObjectIdJsonConverter))]
	[TypeConverter(typeof(CbObjectIdTypeConverter))]
#pragma warning disable CA1036 // we do not need the compare operators
	public readonly struct CbObjectId(uint a, uint b, uint c) : IEquatable<CbObjectId>, IComparer<CbObjectId>, IComparable<CbObjectId>
#pragma warning restore CA1036
	{
		readonly uint _a = a;
		readonly uint _b = b;
		readonly uint _c = c;

		// initialize the serial to a random integer
		private static uint s_objectIdSerial = CalculateRunId();

		/// <summary>
		/// The zero value of an object id
		/// </summary>
		public static CbObjectId Zero { get; } = new CbObjectId(0u, 0u, 0u);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="payload"></param>
		public CbObjectId(ReadOnlySpan<byte> payload) : this(BinaryPrimitives.ReadUInt32BigEndian(payload[0..4]), BinaryPrimitives.ReadUInt32BigEndian(payload[4..8]), BinaryPrimitives.ReadUInt32BigEndian(payload[8..12]))
		{
		}

		/// <summary>
		/// Parses a object id from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		public static CbObjectId Parse(string text)
		{
			byte[] bytes = StringUtils.ParseHexString(text);
			return new CbObjectId(bytes);
		}

		/// <summary>
		/// Parses a digest from the given hex string
		/// </summary>
		/// <param name="text"></param>
		/// <param name="objectId">Receives the object id if successful</param>
		/// <returns></returns>
		public static bool TryParse(string text, out CbObjectId objectId)
		{
			if (!StringUtils.TryParseHexString(text, out byte[]? bytes))
			{
				objectId = Zero;
				return false;
			}
			if (bytes.Length != 12)
			{
				objectId = Zero;
				return false;
			}
			objectId = new CbObjectId(bytes);
			return true;
		}

		/// <summary>
		/// Generates a new object id
		/// </summary>
		/// <returns></returns>
		public static CbObjectId NewObjectId()
		{
			const long Offset = 1_609_459_200; // Seconds from 1970 -> 2021
			long longTime = DateTimeOffset.Now.ToUnixTimeSeconds() - Offset;
			uint time = (uint)longTime;
			uint serial = Interlocked.Increment(ref s_objectIdSerial);
			uint runId = RunId;
			return new CbObjectId(time, serial, runId);
		}

		private static uint RunId { get; } = CalculateRunId();

		private static uint CalculateRunId()
		{
			Guid g = Guid.NewGuid();
			return (uint)g.GetHashCode();
		}

		/// <summary>
		/// Copies this objectid into a span
		/// </summary>
		/// <param name="span"></param>
		public void CopyTo(Span<byte> span)
		{
			BinaryPrimitives.WriteUInt32BigEndian(span, _a);
			BinaryPrimitives.WriteUInt32BigEndian(span[4..], _b);
			BinaryPrimitives.WriteUInt32BigEndian(span[8..], _c);
		}

		/// <summary>
		/// Converts this object id to a byte array
		/// </summary>
		/// <returns></returns>
		public byte[] ToByteArray()
		{
			byte[] bytes = new byte[12];
			CopyTo(bytes);
			return bytes;
		}

		/// <summary>
		/// Creates a utf8string of the object id
		/// </summary>
		/// <returns></returns>
		public Utf8String ToUtf8String()
		{
			return StringUtils.FormatUtf8HexString(ToByteArray());
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return StringUtils.FormatHexString(ToByteArray());
		}

		/// <inheritdoc/>
		public bool Equals(CbObjectId other) => _a == other._a && _b == other._b && _c == other._c;

		/// <inheritdoc/>
		public override bool Equals(object? obj) => (obj is CbObjectId cid) && Equals(cid);

		/// <inheritdoc/>
		public override int GetHashCode() => (int)_a;

		/// <summary>
		/// Test two object ids for equality
		/// </summary>
		public static bool operator ==(CbObjectId a, CbObjectId b) => a.Equals(b);

		/// <summary>
		/// Test two object ids for equality
		/// </summary>
		public static bool operator !=(CbObjectId a, CbObjectId b) => !(a == b);

		/// <summary>
		/// Compare two object ids by comparing their components
		/// </summary>
		public int Compare(CbObjectId x, CbObjectId y)
		{
			int aComparison = x._a.CompareTo(y._a);
			if (aComparison != 0)
			{
				return aComparison;
			}

			int bComparison = x._b.CompareTo(y._b);
			if (bComparison != 0)
			{
				return bComparison;
			}

			return x._c.CompareTo(y._c);
		}

		/// <summary>
		///  Compare two object ids by comparing their components
		/// </summary>
		public int CompareTo(CbObjectId other)
		{
			return Compare(this, other);
		}
	}

	/// <summary>
	/// A comparer for inverse comparisons of a cb object (newest first)
	/// </summary>
	public sealed class CbObjectIdReverseComparer : IComparer<CbObjectId>
	{
		/// <summary>
		/// Compare two cb objects returning the newst first
		/// </summary>
		public int Compare(CbObjectId x, CbObjectId y)
		{
			return y.CompareTo(x);
		}
	}

	/// <summary>
	/// Type converter for CbObjectId to and from JSON
	/// </summary>
	sealed class CbObjectIdJsonConverter : JsonConverter<CbObjectId>
	{
		/// <inheritdoc/>
		public override CbObjectId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => CbObjectId.Parse(reader.GetString()!);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, CbObjectId value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
	}

	/// <summary>
	/// Type converter for CbObjectId
	/// </summary>
	sealed class CbObjectIdTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => CbObjectId.Parse((string)value);
	}
}
