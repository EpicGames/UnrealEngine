// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;

#pragma warning disable CA1028 // Enum Storage should be Int32

namespace EpicGames.Serialization
{
	static class CbPackageConstants
	{
		public const uint CbPackageHeaderMagic = 0xaa77aacc;
	}

	/// <summary>
	/// Header in the CbPackage format
	/// </summary>
	/// <remarks>see CbPackage definition in Zen https://github.com/EpicGames/zen/blob/main/zenhttp/include/zenhttp/httpshared.h</remarks>
	struct CbPackageHeader
	{
		private CbPackageHeader(uint attachmentCount, uint reserved1 = 0, uint reserved2 = 0)
		{
			HeaderMagic = CbPackageConstants.CbPackageHeaderMagic;
			AttachmentCount = attachmentCount;

			Reserved1 = reserved1;
			Reserved2 = reserved2;
		}

		public uint HeaderMagic { get; set; }

		public uint AttachmentCount { get; set; }

		public uint Reserved1 { get; set; }

		public uint Reserved2 { get; set; }

		private const int Length = sizeof(uint) * 4;

		public static async Task<CbPackageHeader> ReadAsync(Stream s)
		{
			// using async reading of stream because Asp.net requires this
			byte[] buf = new byte[Length];
			int readBytes = await s.ReadAsync(buf, 0, Length);
			if (readBytes != Length)
			{
				throw new Exception($"Was not able to read package header, not enough bytes. Bytes read: {readBytes}");
			}

			using MemoryStream ms = new(buf);
			using BinaryReader reader = new(ms);
			uint headerMagic = reader.ReadUInt32();
			if (headerMagic != CbPackageConstants.CbPackageHeaderMagic)
			{
				throw new Exception($"Magic did not match, expected {CbPackageConstants.CbPackageHeaderMagic} got {headerMagic}");
			}

			uint attachmentCount = reader.ReadUInt32();
			uint reserved1 = reader.ReadUInt32();
			uint reserved2 = reader.ReadUInt32();

			return new CbPackageHeader(attachmentCount, reserved1, reserved2);
		}

		public readonly void Write(Stream stream)
		{
			using BinaryWriter writer = new(stream, Encoding.ASCII, leaveOpen: true);
			writer.Write(HeaderMagic);
			writer.Write(AttachmentCount);
			writer.Write(Reserved1);
			writer.Write(Reserved2);
		}
	}

	/// <summary>
	/// Per attachment entry flags
	/// </summary>
	[Flags]
	public enum CbPackageAttachmentFlags : uint
	{
		/// <summary>
		/// Is marshaled using compressed buffer storage format
		/// </summary>
		IsCompressed = (1u << 0),

		/// <summary>
		/// Is compact binary object
		/// </summary>
		IsObject = (1u << 1),

		/// <summary>
		/// Is error (compact binary formatted) object
		/// </summary>
		IsError = (1u << 2),
	}

	/// <summary>
	/// Header for each attachment in the package
	/// </summary>
	public struct CbPackageAttachmentEntry
	{
		/// <summary>
		/// The size of the attachment
		/// </summary>
		public ulong PayloadSize { get; set; }

		/// <summary>
		/// Flags description the attachment
		/// </summary>
		public CbPackageAttachmentFlags Flags { get; set; }

		/// <summary>
		/// The hash of the attachment
		/// </summary>
		public IoHash AttachmentHash { get; set; }

		/// <summary>
		/// Reads a package attachment struct from a stream
		/// </summary>
		/// <param name="s">The stream to read from</param>
		/// <returns>The read package attachment</returns>
		internal static async Task<CbPackageAttachmentEntry> ReadAsync(Stream s)
		{
			byte[] buf = new byte[Length];
			int readBytes = await s.ReadAsync(buf, 0, Length);
			if (readBytes != Length)
			{
				throw new Exception($"Was not able to read package attachment entry, not enough bytes. Bytes read: {readBytes}");
			}
			using MemoryStream ms = new(buf);
			using BinaryReader reader = new(ms);
			ulong payloadSize = reader.ReadUInt64();

			uint flags = reader.ReadUInt32();
			byte[] hashBytes = reader.ReadBytes(20);

			return new CbPackageAttachmentEntry
			{
				PayloadSize = payloadSize,
				Flags = (CbPackageAttachmentFlags)flags,
				AttachmentHash = new IoHash(hashBytes),
			};
		}

		private const int Length = IoHash.NumBytes + sizeof(ulong) + sizeof(uint);

		/// <summary>
		/// Write a package attachment entry to the stream
		/// </summary>
		/// <param name="stream"></param>
		internal readonly void Write(Stream stream)
		{
			using BinaryWriter writer = new(stream, Encoding.ASCII, leaveOpen: true);
			writer.Write(PayloadSize);
			writer.Write((uint)Flags);
			writer.Write(AttachmentHash.ToByteArray());
		}
	}

	/// <summary>
	/// Reads a CbPackage (a bundle of both a cb object and its attachments)
	/// </summary>
	public class CbPackageReader
	{
#pragma warning disable IDE0052 // Remove unread private members
		private readonly CbPackageHeader _header;
#pragma warning restore IDE0052 // Remove unread private members
		private readonly Stream _attachmentsStream;
		private readonly List<CbPackageAttachmentEntry> _attachmentEntries = [];

		private CbPackageReader(CbPackageHeader header, CbObject rootObject, IoHash rootHash, List<CbPackageAttachmentEntry> attachmentEntries, Stream attachmentsStream)
		{
			_header = header;
			_attachmentEntries = attachmentEntries;
			_attachmentsStream = attachmentsStream;
			RootObject = rootObject;
			RootHash = rootHash;
		}

		private static async Task<byte[]> ReadStreamAsync(Stream s, int count)
		{
			int index = 0;
			byte[] bytes = new byte[count];
			do
			{
				int n = await s.ReadAsync(new Memory<byte>(bytes, index, count - index))
					.ConfigureAwait(false);
				if (n == 0)
				{
					throw new Exception("Unexpected end of stream");
				}

				index += n;
			} while (index < count);

			return bytes;
		}

		/// <summary>
		/// Create a package reader from a stream, used to stream in the attachments of the package
		/// </summary>
		/// <param name="s">A stream</param>
		/// <returns>A package reader instance</returns>
		public static async Task<CbPackageReader> CreateAsync(Stream s)
		{
			CbPackageHeader header = await CbPackageHeader.ReadAsync(s);

			List<CbPackageAttachmentEntry> entries = [];
			// we expect a extra entry for the root object
			uint attachmentsCount = header.AttachmentCount + 1;

			for (int i = 0; i < attachmentsCount; i++)
			{
				CbPackageAttachmentEntry entry = await CbPackageAttachmentEntry.ReadAsync(s);
				entries.Add(entry);
			}

			if (entries.Count < 1)
			{
				throw new Exception("Did not find any root object entry when reading CbPackage");
			}

			// the first object is assumed to be a CbObject, the root object which references the other attachments
			CbPackageAttachmentEntry rootObject = entries.First();
			entries.RemoveAt(0);
			if (!rootObject.Flags.HasFlag(CbPackageAttachmentFlags.IsObject))
			{
				throw new Exception("First attachment must be a CbObject for a package");
			}
			if (rootObject.Flags.HasFlag(CbPackageAttachmentFlags.IsError))
			{
				throw new Exception("First attachment was a error object");
			}
			if (rootObject.Flags.HasFlag(CbPackageAttachmentFlags.IsCompressed))
			{
				// TODO: We could support this being a compressed buffer and just remove it
				throw new Exception("First attachment must not be compressed");
			}

			if (rootObject.PayloadSize > Int32.MaxValue)
			{
				throw new Exception($"Package attachments larger then {Int32.MaxValue} not supported");
			}
			byte[] rootObjectBytes = await ReadStreamAsync(s, (int)rootObject.PayloadSize);

			return new CbPackageReader(header, new CbObject(rootObjectBytes), rootObject.AttachmentHash, entries, s);
		}

		/// <summary>
		/// Iterates over the attachments, returning the attachment entry and the attachment (in memory)
		/// </summary>
		/// <returns></returns>
		public async IAsyncEnumerable<(CbPackageAttachmentEntry, byte[])> IterateAttachmentsAsync()
		{
			// close the stream after we have iterated the attachments as there should be nothing left in it
			await using Stream s = _attachmentsStream;

			foreach (CbPackageAttachmentEntry entry in _attachmentEntries)
			{
				if (entry.PayloadSize > Int32.MaxValue)
				{
					throw new Exception($"Package attachments larger then {Int32.MaxValue} not supported");
				}
				byte[] blob = await ReadStreamAsync(s, (int)entry.PayloadSize);
				yield return (entry, blob);
			}
		}

		/// <summary>
		/// The CbObject that is the root object (that references the other attachments in the package)
		/// </summary>
		public CbObject RootObject { get; }

		/// <summary>
		/// The hash of the root object
		/// </summary>
		public IoHash RootHash { get; }
	}

	/// <summary>
	/// Builds a in-memory representation of a CbPackage
	/// </summary>
	public sealed class CbPackageBuilder : IDisposable
	{
		private readonly List<(CbPackageAttachmentEntry, Stream)> _streamAttachments = [];

		/// <summary>
		/// Constructor for the package builder
		/// </summary>
		public CbPackageBuilder()
		{
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			foreach ((CbPackageAttachmentEntry _, Stream stream) in _streamAttachments)
			{
				stream.Dispose();
			}
		}

		/// <summary>
		/// Add a attachment to the package builder using in memory buffer
		/// </summary>
		/// <param name="attachmentHash">The hash of the attachment</param>
		/// <param name="flags">The flags that apply to the attachment</param>
		/// <param name="blobMemory">In-memory buffer of the attachment</param>
		/// <returns></returns>
		public void AddAttachment(IoHash attachmentHash, CbPackageAttachmentFlags flags, byte[] blobMemory)
		{
			CbPackageAttachmentEntry entry = new()
			{
				AttachmentHash = attachmentHash,
				Flags = flags,
				PayloadSize = (ulong)blobMemory.LongLength
			};

			_streamAttachments.Add((entry, new MemoryStream(blobMemory)));
		}

		/// <summary>
		/// Add a attachment to the package builder from a stream, will be cache in memory for the lifetime of the package builder
		/// </summary>
		/// <param name="attachmentHash">The hash of the attachment</param>
		/// <param name="flags">The flags that apply to the attachment</param>
		/// <param name="stream">The stream to read the attachment from</param>
		/// <param name="length">The count of bytes to read from the stream</param>
		/// <returns></returns>
		public void AddAttachment(IoHash attachmentHash, CbPackageAttachmentFlags flags, Stream stream, ulong length)
		{
			CbPackageAttachmentEntry entry = new()
			{
				AttachmentHash = attachmentHash,
				Flags = flags,
				PayloadSize = length
			};

			_streamAttachments.Add((entry, stream));
		}

		/// <summary>
		/// Generate a contiguous buffer of the cb package
		/// </summary>
		/// <returns></returns>
		public async Task<byte[]> ToByteArrayAsync()
		{
			MemoryStream packageBuffer = new();

			if (_streamAttachments.Count == 0)
			{
				throw new Exception("Expected the first attachment to be a CbObject but found no attachment");
			}

			// we will overwrite this header again after building the package builder
			CbPackageHeader header = new()
			{
				HeaderMagic = CbPackageConstants.CbPackageHeaderMagic,
				AttachmentCount = (uint)_streamAttachments.Count - 1, // the root object does not count as a attachment
				Reserved1 = 0,
				Reserved2 = 0,
			};

			header.Write(packageBuffer);
			foreach ((CbPackageAttachmentEntry entry, Stream _) in _streamAttachments)
			{
				entry.Write(packageBuffer);
			}

			foreach ((CbPackageAttachmentEntry _, Stream stream) in _streamAttachments)
			{
				await stream.CopyToAsync(packageBuffer);
			}
			return packageBuffer.ToArray();
		}
	}
}
