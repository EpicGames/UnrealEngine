// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json;
using EpicGames.Core;

namespace EpicGames.Serialization
{
	/// <summary>
	/// Marks a class as supporting serialization to compact-binary, even if it does not have exposed fields. This suppresses errors
	/// when base class objects are empty.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class CbObjectAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute used to mark a property that should be serialized to compact binary
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class CbFieldAttribute : Attribute
	{
		/// <summary>
		/// Name of the serialized field
		/// </summary>
		public string? Name { get; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public CbFieldAttribute()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		public CbFieldAttribute(string name)
		{
			Name = name;
		}
	}

	/// <summary>
	/// Attribute used to mark that a property should not be serialized to compact binary
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class CbIgnoreAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute used to indicate that this object is the base for a class hierarchy. Each derived class must have a [CbDiscriminator] attribute.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class CbPolymorphicAttribute : Attribute
	{
	}

	/// <summary>
	/// Sets the name used for discriminating between derived classes during serialization
	/// </summary>
	/// <param name="name">Name used to identify this class</param>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class CbDiscriminatorAttribute(string name) : Attribute
	{
		/// <summary>
		/// Name used to identify this class
		/// </summary>
		public string Name { get; } = name;
	}

	/// <summary>
	/// Exception thrown when serializing cb objects
	/// </summary>
	public class CbException : Exception
	{
		/// <inheritdoc cref="Exception(String?)"/>
		public CbException(string message) : base(message)
		{
		}

		/// <inheritdoc cref="Exception(String?, Exception)"/>
		public CbException(string message, Exception inner) : base(message, inner)
		{
		}
	}

	/// <summary>
	/// Exception indicating that a class does not have any fields to serialize
	/// </summary>
	/// <remarks>
	/// Constructor
	/// </remarks>
	public sealed class CbEmptyClassException(Type classType) : CbException($"{classType.Name} does not have any fields marked with a [CbField] attribute. If this is intended, explicitly mark the class with a [CbObject] attribute.")
	{
		/// <summary>
		/// Type with missing field annotations
		/// </summary>
		public Type ClassType { get; } = classType;
	}

	/// <summary>
	/// Attribute-driven compact binary serializer
	/// </summary>
	public static class CbSerializer
	{
		/// <summary>
		/// Serialize an object
		/// </summary>
		/// <param name="type">Type of the object to serialize</param>
		/// <param name="value"></param>
		/// <returns></returns>
		public static CbObject Serialize(Type type, object value)
		{
			CbWriter writer = new();
			CbConverter.GetConverter(type).WriteObject(writer, value);
			return writer.ToObject();
		}

		/// <summary>
		/// Serialize an object
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static CbObject Serialize<T>(T value)
		{
			CbWriter writer = new();
			CbConverter.GetConverter<T>().Write(writer, value);
			return writer.ToObject();
		}

		/// <summary>
		/// Serialize an object
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="value"></param>
		/// <returns></returns>
		public static byte[] SerializeToByteArray<T>(T value)
		{
			CbWriter writer = new();
			CbConverter.GetConverter<T>().Write(writer, value);
			return writer.ToByteArray();
		}

		/// <summary>
		/// Serialize a property to a given writer
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="writer"></param>
		/// <param name="value"></param>
		public static void Serialize<T>(CbWriter writer, T value)
		{
			CbConverter.GetConverter<T>().Write(writer, value);
		}

		/// <summary>
		/// Serialize a named property to the given writer
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="writer"></param>
		/// <param name="name"></param>
		/// <param name="value"></param>
		public static void Serialize<T>(CbWriter writer, CbFieldName name, T value)
		{
			CbConverter.GetConverter<T>().WriteNamed(writer, name, value);
		}

		/// <summary>
		/// Deserialize an object from a <see cref="CbObject"/>
		/// </summary>
		/// <param name="field"></param>
		/// <param name="type">Type of the object to read</param>
		/// <returns></returns>
		public static object? Deserialize(CbField field, Type type)
		{
			return CbConverter.GetConverter(type).ReadObject(field);
		}

		/// <summary>
		/// Deserialize an object from a <see cref="CbField"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="field"></param>
		/// <returns></returns>
		public static T Deserialize<T>(CbField field)
		{
			return CbConverter.GetConverter<T>().Read(field);
		}

		/// <summary>
		/// Deserialize an object from a <see cref="CbObject"/>
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="obj"></param>
		/// <returns></returns>
		public static T Deserialize<T>(CbObject obj) => Deserialize<T>(obj.AsField());

		/// <summary>
		/// Deserialize an object from a block of memory
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="data"></param>
		/// <returns></returns>
		public static T Deserialize<T>(ReadOnlyMemory<byte> data) => Deserialize<T>(new CbField(data));
	}

	/// <summary>
	/// Helper to convert a json string into a cb object with conventions for how to detect types that json can not naturally encode
	/// </summary>
	public static class CbJsonReader
	{
		/// <summary>
		/// Converts a json object into a CbObject
		/// </summary>
		/// <returns></returns>
		public static CbObject FromJson(string json)
		{
			using JsonDocument document = JsonDocument.Parse(json);
			CbWriter writer = new();
			ReadJsonField(document.RootElement, null, writer);

			return writer.ToObject();
		}

		internal static void ReadJsonField(JsonElement jsonElement, string? fieldName, CbWriter writer)
		{
			switch (jsonElement.ValueKind)
			{
				case JsonValueKind.Object:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.BeginObject();
						}
						else
						{
							writer.BeginObject(fieldName);
						}

						foreach (JsonProperty prop in jsonElement.EnumerateObject())
						{
							ReadJsonField(prop.Value, prop.Name, writer);
						}

						writer.EndObject();
						break;
					}
				case JsonValueKind.Array:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.BeginArray();
						}
						else
						{
							writer.BeginArray(fieldName);
						}

						foreach (JsonElement prop in jsonElement.EnumerateArray())
						{
							ReadJsonField(prop, null, writer);
						}

						writer.EndArray();
						break;
					}
				case JsonValueKind.Null:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.WriteNullValue();
						}
						else
						{
							writer.WriteNull(fieldName);
						}
						break;
					}
				case JsonValueKind.True:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.WriteBoolValue(true);
						}
						else
						{
							writer.WriteBool(fieldName, true);
						}
						break;
					}
				case JsonValueKind.False:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.WriteBoolValue(false);
						}
						else
						{
							writer.WriteBool(fieldName, false);
						}
						break;
					}
				case JsonValueKind.Number:
					{
						if (String.IsNullOrEmpty(fieldName))
						{
							writer.WriteDoubleValue(jsonElement.GetDouble());
						}
						else
						{
							writer.WriteDouble(fieldName, jsonElement.GetDouble());
						}
						break;
					}
				case JsonValueKind.String:
					{
						string? value = jsonElement.GetString() ?? "";

						if (CbObjectId.TryParse(value, out CbObjectId oid))
						{
							if (String.IsNullOrEmpty(fieldName))
							{
								writer.WriteObjectIdValue(oid);
							}
							else
							{
								writer.WriteObjectId(fieldName, oid);
							}

							return;
						}

						{
							if (IoHash.TryParse(value, out IoHash hash))
							{
								if (String.IsNullOrEmpty(fieldName))
								{
									writer.WriteHashValue(hash);
								}
								else
								{
									writer.WriteHash(fieldName, hash);
								}

								return;
							}
						}

						int pos = value.IndexOf(BinaryAttachmentPrefix, StringComparison.OrdinalIgnoreCase);
						if (pos != -1)
						{
							value = value.Substring(pos);
							if (IoHash.TryParse(value, out IoHash hash))
							{
								if (String.IsNullOrEmpty(fieldName))
								{
									writer.WriteBinaryAttachmentValue(hash);
								}
								else
								{
									writer.WriteBinaryAttachment(fieldName, hash);
								}

								return;
							}
						}

						pos = value.IndexOf(CompactBinaryAttachmentPrefix, StringComparison.OrdinalIgnoreCase);
						if (pos != -1)
						{
							value = value.Substring(pos);
							if (IoHash.TryParse(value, out IoHash hash))
							{
								if (String.IsNullOrEmpty(fieldName))
								{
									writer.WriteObjectAttachmentValue(hash);
								}
								else
								{
									writer.WriteObjectAttachment(fieldName, hash);
								}

								return;
							}
						}

						if (String.IsNullOrEmpty(fieldName))
						{
							writer.WriteStringValue(value);
						}
						else
						{
							writer.WriteString(fieldName, value);
						}
						break;
					}
				case JsonValueKind.Undefined:
					break;
				default:
					throw new NotImplementedException();
			}
		}

		private const string CompactBinaryAttachmentPrefix = "obj/";
		private const string BinaryAttachmentPrefix = "bin/";
	}
}
