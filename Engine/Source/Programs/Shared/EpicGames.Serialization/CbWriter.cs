// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using EpicGames.Core;

namespace EpicGames.Serialization
{
	/// <summary>
	/// Exception for <see cref="CbWriter"/>
	/// </summary>
	public class CbWriterException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		public CbWriterException(string message)
			: base(message)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		/// <param name="ex"></param>
		public CbWriterException(string message, Exception? ex)
			: base(message, ex)
		{
		}
	}

	/// <summary>
	/// Interface for compact binary writers
	/// </summary>
	public interface ICbWriter
	{
		/// <summary>
		/// Begin writing an object field
		/// </summary>
		/// <param name="name">Name of the field. May be empty for fields that are not part of another object.</param>
		void BeginObject(CbFieldName name);

		/// <summary>
		/// End the current object
		/// </summary>
		void EndObject();

		/// <summary>
		/// Begin writing a named array field
		/// </summary>
		/// <param name="name">Name of the field, or an empty string.</param>
		/// <param name="elementType">Type of the field. May be <see cref="CbFieldType.None"/> for non-uniform arrays.</param>
		void BeginArray(CbFieldName name, CbFieldType elementType);

		/// <summary>
		/// End the current array
		/// </summary>
		void EndArray();

		/// <summary>
		/// Writes the header for a named field
		/// </summary>
		/// <param name="type">Type of the field</param>
		/// <param name="name">Name of the field. May be empty for fields that are not part of another object.</param>
		/// <param name="length">Length of data for the field</param>
		Span<byte> WriteField(CbFieldType type, CbFieldName name, int length);

		/// <summary>
		/// Writes a reference to an external binary field into the output stream
		/// </summary>
		/// <param name="data">Data to reference</param>
		void WriteReference(ReadOnlyMemory<byte> data);
	}

	/// <summary>
	/// Wrapper for a field name, which can come from an encoded UTF8 string or regular C# string
	/// </summary>
	public record struct CbFieldName(Utf8String Text)
	{
		/// <summary>
		/// Implicit conversion from a string
		/// </summary>
		public static implicit operator CbFieldName(string text) => new(new Utf8String(text));

		/// <summary>
		/// Implicit conversion from a utf8 string
		/// </summary>
		public static implicit operator CbFieldName(Utf8String text) => new(text);
	}

	/// <summary>
	/// Base class for <see cref="ICbWriter"/> implementations. Tracks structural data for fixing up lengths and offsets, without managing any buffers for field data.
	/// </summary>
	public abstract class CbWriterBase : ICbWriter
	{
		/// <summary>
		/// Stores information about an object or array scope within the written buffer which requires a header to be inserted containing
		/// the size or number of elements when copied to an output buffer
		/// </summary>
		class Scope
		{
			public CbFieldType _fieldType;
			public bool _writeFieldType;
			public CbFieldType _uniformFieldType;
			public Utf8String _name;
			public int _itemCount;
			public ReadOnlyMemory<byte> _data;
			public int _length;
			public Scope? _firstChild;
			public Scope? _lastChild;
			public Scope? _nextSibling;

			public void Reset()
			{
				_fieldType = CbFieldType.None;
				_writeFieldType = true;
				_uniformFieldType = CbFieldType.None;
				_name = default;
				_itemCount = 0;
				_data = default;
				_length = 0;
				_firstChild = null;
				_lastChild = null;
				_nextSibling = null;
			}

			public void AddChild(Scope child)
			{
				if (_lastChild == null)
				{
					_firstChild = child;
				}
				else
				{
					_lastChild._nextSibling = child;
				}
				_lastChild = child;
			}
		}

		readonly Scope _rootScope = new() { _fieldType = CbFieldType.Array };
		readonly Stack<Scope> _openScopes = new();
		readonly Stack<Scope> _freeScopes = new();

		Memory<byte> _buffer = Memory<byte>.Empty;
		int _bufferPos = 0; // Offset of the first field in the current buffer
		int _bufferEnd = 0; // Current end of the buffer

		/// <summary>
		/// Constructor
		/// </summary>
		protected CbWriterBase()
		{
			_openScopes.Push(_rootScope);
		}

		/// <summary>
		/// Resets the current contents of the writer
		/// </summary>
		protected void Reset()
		{
			AddChildrenToFreeList(_rootScope);
			_rootScope.Reset();
		}

		void AddChildrenToFreeList(Scope root)
		{
			for (Scope? child = root._firstChild; child != null; child = child._nextSibling)
			{
				AddChildrenToFreeList(child);
				child.Reset();
				_freeScopes.Push(child);
			}
		}

		/// <summary>
		/// Allocate a scope object
		/// </summary>
		/// <returns>New scope object</returns>
		Scope AllocScope()
		{
			Scope? scope;
			if (!_freeScopes.TryPop(out scope))
			{
				scope = new Scope();
			}
			return scope;
		}

		/// <summary>
		/// Creates a scope containing leaf data
		/// </summary>
		/// <param name="data"></param>
		void AddLeafData(ReadOnlyMemory<byte> data)
		{
			Scope scope = AllocScope();
			scope._data = data;
			scope._length = data.Length;

			Scope currentScope = _openScopes.Peek();
			currentScope.AddChild(scope);
		}

		/// <summary>
		/// Insert a new scope
		/// </summary>
		Scope EnterScope(CbFieldType fieldType, CbFieldName name)
		{
			Scope currentScope = _openScopes.Peek();

			Scope scope = AllocScope();
			scope._fieldType = fieldType;
			scope._writeFieldType = currentScope._uniformFieldType == CbFieldType.None;
			scope._name = name.Text;

			currentScope.AddChild(scope);

			_openScopes.Push(scope);
			return scope;
		}

		/// <summary>
		/// Pop a scope from the current open list
		/// </summary>
		void LeaveScope()
		{
			WriteFields();
			Scope scope = _openScopes.Peek();

			// Measure the length of all children
			int childrenLength = 0;
			for (Scope? child = scope._firstChild; child != null; child = child._nextSibling)
			{
				childrenLength += child._length;
			}

			// Measure the total length of this scope
			if (scope._fieldType != CbFieldType.None)
			{
				// Measure the size of the field header
				int headerLength = 0;
				if (scope._writeFieldType)
				{
					headerLength++;
					if (!scope._name.IsEmpty)
					{
						headerLength += VarInt.MeasureUnsigned(scope._name.Length) + scope._name.Length;
					}
				}

				// Measure the size of the payload
				int payloadLength = 0;
				if (CbFieldUtils.IsArray(scope._fieldType))
				{
					payloadLength += VarInt.MeasureUnsigned(scope._itemCount);
				}
				if (scope._fieldType == CbFieldType.UniformObject || scope._fieldType == CbFieldType.UniformArray)
				{
					payloadLength++;
				}

				// Measure the size of writing the payload
				int payloadLengthBytes = VarInt.MeasureUnsigned(payloadLength + childrenLength);

				// Allocate the header
				Memory<byte> header = Allocate(headerLength + payloadLengthBytes + payloadLength);
				scope._data = header;
				_bufferPos = _bufferEnd;

				// Write all the fields to the header buffer
				Span<byte> span = header.Span;
				if (scope._writeFieldType)
				{
					if (scope._name.IsEmpty)
					{
						span[0] = (byte)scope._fieldType;
						span = span.Slice(1);
					}
					else
					{
						span[0] = (byte)(scope._fieldType | CbFieldType.HasFieldName);
						span = span.Slice(1);

						int bytesWritten = VarInt.WriteUnsigned(span, scope._name.Length);
						span = span.Slice(bytesWritten);

						scope._name.Span.CopyTo(span);
						span = span.Slice(scope._name.Length);
					}
				}

				VarInt.WriteUnsigned(span, payloadLength + childrenLength);
				span = span.Slice(payloadLengthBytes);

				if (CbFieldUtils.IsArray(scope._fieldType))
				{
					int itemCountBytes = VarInt.WriteUnsigned(span, scope._itemCount);
					span = span.Slice(itemCountBytes);
				}

				// Write the type for uniform arrays
				if (scope._fieldType == CbFieldType.UniformObject || scope._fieldType == CbFieldType.UniformArray)
				{
					span[0] = (byte)scope._uniformFieldType;
					span = span.Slice(1);
				}

				Debug.Assert(span.Length == 0);
			}

			// Set the final size of this scope, and pop it from the current open stack
			scope._length = childrenLength + scope._data.Length;
			_openScopes.Pop();
		}

		/// <summary>
		/// Begin writing an object field
		/// </summary>
		/// <param name="name">Name of the field</param>
		public void BeginObject(CbFieldName name)
		{
			WriteFields();

			Scope parentScope = _openScopes.Peek();
			parentScope._itemCount++;

			EnterScope(CbFieldType.Object, name);
		}

		/// <summary>
		/// Begin writing a uniform object field, an object which fields are all the same type
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="elementType">The type of the uniform field</param>
		public void BeginUniformObject(CbFieldName name, CbFieldType elementType)
		{
			WriteFields();

			Scope parentScope = _openScopes.Peek();
			parentScope._itemCount++;

			Scope scope = EnterScope(CbFieldType.UniformObject, name);
			scope._uniformFieldType = elementType;
		}

		/// <summary>
		/// End the current object
		/// </summary>
		public void EndObject() => LeaveScope();

		/// <summary>
		/// Begin writing a named array field
		/// </summary>
		/// <param name="name"></param>
		/// <param name="elementType">Type of elements in the array</param>
		public void BeginArray(CbFieldName name, CbFieldType elementType)
		{
			WriteFields();

			Scope parentScope = _openScopes.Peek();
			parentScope._itemCount++;

			Scope scope = EnterScope((elementType == CbFieldType.None) ? CbFieldType.Array : CbFieldType.UniformArray, name);
			scope._uniformFieldType = elementType;
		}

		/// <summary>
		/// End the current array
		/// </summary>
		public void EndArray() => LeaveScope();

		void WriteFields()
		{
			if (_bufferPos < _bufferEnd)
			{
				AddLeafData(_buffer.Slice(_bufferPos, _bufferEnd - _bufferPos));
				_bufferPos = _bufferEnd;
			}
		}

		private Memory<byte> Allocate(int length)
		{
			if (_bufferEnd + length > _buffer.Length)
			{
				WriteFields();
				_buffer = AllocateChunk(length);
				_bufferPos = 0;
				_bufferEnd = 0;
			}

			Memory<byte> data = _buffer.Slice(_bufferEnd, length);
			_bufferEnd += length;

			return data;
		}

		/// <summary>
		/// Allocates a chunk of data for storing CB fragments. This should be relatively coarse; the returned chunk will be reused for subsequent writes until full.
		/// </summary>
		/// <param name="minSize">Minimum size of the chunk</param>
		/// <returns>New chunk of memory</returns>
		protected abstract Memory<byte> AllocateChunk(int minSize);

		/// <summary>
		/// Writes the header for a named field
		/// </summary>
		/// <param name="type">Type of the field</param>
		/// <param name="name">Name of the field</param>
		/// <param name="size">Size of the field</param>
		public Span<byte> WriteField(CbFieldType type, CbFieldName name, int size)
		{
			WriteFieldHeader(type, name.Text);

			Span<byte> span = Allocate(size).Span;
			if (_openScopes.Count == 1)
			{
				// If this field is at the root, flush it immediately
				WriteFields();
			}
			return span;
		}

		void WriteFieldHeader(CbFieldType type, Utf8String name)
		{
			Scope scope = _openScopes.Peek();
			if (name.IsEmpty)
			{
				CbFieldType scopeType = scope._fieldType;
				if (!CbFieldUtils.IsArray(scopeType))
				{
					throw new CbWriterException($"Anonymous fields are not allowed within fields of type {scopeType}");
				}

				CbFieldType elementType = scope._uniformFieldType;
				if (elementType == CbFieldType.None)
				{
					Allocate(1).Span[0] = (byte)type;
				}
				else if (elementType != type)
				{
					throw new CbWriterException($"Mismatched type for uniform array - expected {elementType}, not {type}");
				}

				scope._itemCount++;
			}
			else
			{
				CbFieldType scopeType = scope._fieldType;
				if (!CbFieldUtils.IsObject(scopeType))
				{
					throw new CbWriterException($"Named fields are not allowed within fields of type {scopeType}");
				}

				CbFieldType elementType = scope._uniformFieldType;

				int nameVarIntLength = VarInt.MeasureUnsigned(name.Length);
				if (elementType == CbFieldType.None)
				{
					Span<byte> buffer = Allocate(1 + nameVarIntLength + name.Length).Span;
					buffer[0] = (byte)(type | CbFieldType.HasFieldName);
					WriteBinaryPayload(buffer[1..], name.Span);
				}
				else
				{
					if (elementType != (type | CbFieldType.HasFieldName))
					{
						throw new CbWriterException($"Mismatched type for uniform object - expected {elementType}, not {type}");
					}
					Memory<byte> buffer = Allocate(nameVarIntLength + name.Length);
					WriteBinaryPayload(buffer.Span, name.Span);
				}

				scope._itemCount++;
			}
		}

		/// <inheritdoc/>
		public void WriteReference(ReadOnlyMemory<byte> data)
		{
			WriteFields();
			AddLeafData(data);
		}

		/// <summary>
		/// Writes the payload for a binary value
		/// </summary>
		/// <param name="output">Output buffer</param>
		/// <param name="value">Value to be written</param>
		static void WriteBinaryPayload(Span<byte> output, ReadOnlySpan<byte> value)
		{
			int varIntLength = VarInt.WriteUnsigned(output, value.Length);
			output = output[varIntLength..];

			value.CopyTo(output);
		}

		/// <summary>
		/// Gets the size of the serialized data
		/// </summary>
		/// <returns></returns>
		public int GetSize()
		{
			if (_openScopes.Count > 1)
			{
				throw new CbWriterException("Unfinished scope in writer");
			}

			int length = 0;
			for (Scope? child = _rootScope._firstChild; child != null; child = child._nextSibling)
			{
				length += child._length;
			}

			return length;
		}

		/// <summary>
		/// Copy the data from this writer to a buffer
		/// </summary>
		/// <param name="buffer"></param>
		public void CopyTo(Span<byte> buffer)
		{
			Copy(_rootScope, buffer);
		}

		static Span<byte> Copy(Scope scope, Span<byte> span)
		{
			if (scope._data.Length > 0)
			{
				scope._data.Span.CopyTo(span);
				span = span.Slice(scope._data.Length);
			}
			for (Scope? child = scope._firstChild; child != null; child = child._nextSibling)
			{
				span = Copy(child, span);
			}
			return span;
		}

		/// <summary>
		/// Computes the hash for this object
		/// </summary>
		/// <returns>Hash for the object</returns>
		public IoHash ComputeHash()
		{
			using (Blake3.Hasher hasher = Blake3.Hasher.New())
			{
				foreach (ReadOnlyMemory<byte> segment in GetSegments())
				{
					hasher.Update(segment.Span);
				}
				return IoHash.FromBlake3(hasher);
			}
		}

		/// <summary>
		/// Convert the data into a compact binary object
		/// </summary>
		/// <returns></returns>
		public CbObject ToObject()
		{
			return new CbObject(ToByteArray());
		}

		/// <summary>
		/// Convert the data into a flat array
		/// </summary>
		/// <returns></returns>
		public byte[] ToByteArray()
		{
			byte[] buffer = new byte[GetSize()];
			CopyTo(buffer);
			return buffer;
		}

		/// <summary>
		/// Enumerate all the segments in the data that has been written
		/// </summary>
		/// <returns>Sequence of segments</returns>
		public List<ReadOnlyMemory<byte>> GetSegments()
		{
			List<ReadOnlyMemory<byte>> segments = [];
			GetSegments(_rootScope, segments);
			return segments;
		}

		static void GetSegments(Scope scope, List<ReadOnlyMemory<byte>> segments)
		{
			if (scope._data.Length > 0)
			{
				segments.Add(scope._data);
			}
			for (Scope? child = scope._firstChild; child != null; child = child._nextSibling)
			{
				GetSegments(child, segments);
			}
		}

		/// <summary>
		/// Gets the contents of this writer as a stream
		/// </summary>
		/// <returns>New stream for the contents of this object</returns>
		public Stream AsStream() => new ReadStream(GetSegments().GetEnumerator(), GetSize());

		class ReadStream(IEnumerator<ReadOnlyMemory<byte>> segments, long length) : Stream
		{
			ReadOnlyMemory<byte> _segment;
			long _positionInternal;

			/// <inheritdoc/>
			public override bool CanRead => true;

			/// <inheritdoc/>
			public override bool CanSeek => false;

			/// <inheritdoc/>
			public override bool CanWrite => false;

			/// <inheritdoc/>
			public override long Length { get; } = length;

			/// <inheritdoc/>
			public override long Position
			{
				get => _positionInternal;
				set => throw new NotSupportedException();
			}

			/// <inheritdoc/>
			public override void Flush() { }

			/// <inheritdoc/>
			public override int Read(Span<byte> buffer)
			{
				int readLength = 0;
				while (readLength < buffer.Length)
				{
					while (_segment.Length == 0)
					{
						if (!segments.MoveNext())
						{
							return readLength;
						}
						_segment = segments.Current;
					}

					int copyLength = Math.Min(_segment.Length, buffer.Length);

					_segment.Span.Slice(0, copyLength).CopyTo(buffer.Slice(readLength));
					_segment = _segment.Slice(copyLength);

					_positionInternal += copyLength;
					readLength += copyLength;
				}
				return readLength;
			}

			/// <inheritdoc/>
			public override int Read(byte[] buffer, int offset, int count) => Read(buffer.AsSpan(offset, count));

			/// <inheritdoc/>
			public override long Seek(long offset, SeekOrigin origin) => throw new NotSupportedException();

			/// <inheritdoc/>
			public override void SetLength(long value) => throw new NotSupportedException();

			/// <inheritdoc/>
			public override void Write(byte[] buffer, int offset, int count) => throw new NotSupportedException();
		}
	}

	/// <summary>
	/// Forward-only writer for compact binary objects
	/// </summary>
	public class CbWriter : CbWriterBase
	{
		/// <summary>
		/// Size of data to preallocate by default
		/// </summary>
		public const int DefaultChunkSize = 1024;

		readonly List<byte[]> _chunks = [];
		readonly List<byte[]> _freeChunks = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public CbWriter()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="reserve">Amount of data to reserve for output</param>
		public CbWriter(int reserve)
		{
			_freeChunks.Add(new byte[reserve]);
		}

		/// <summary>
		/// Clear the current contents of the writer
		/// </summary>
		public void Clear()
		{
			Reset();

			_freeChunks.AddRange(_chunks);
			_chunks.Clear();
		}

		/// <summary>
		/// Ensures that the required space is available in a contiguous chunk
		/// </summary>
		/// <param name="reserve">Minimum required space</param>
		public void Reserve(int reserve)
		{
			int allocate = Math.Max(DefaultChunkSize, reserve);
			if (!_freeChunks.Any(x => x.Length >= reserve))
			{
				_freeChunks.Add(new byte[allocate + 4096]);
			}
		}

		/// <inheritdoc/>
		protected override Memory<byte> AllocateChunk(int minSize)
		{
			for (int idx = 0; idx < _freeChunks.Count; idx++)
			{
				byte[] data = _freeChunks[idx];
				if (data.Length >= minSize)
				{
					_freeChunks.RemoveAt(idx);
					return data;
				}
			}
			return new byte[Math.Max(minSize, DefaultChunkSize)];
		}
	}

	/// <summary>
	/// Extension methods for <see cref="CbWriter"/>
	/// </summary>
	public static class CbWriterExtensions
	{
		static int MeasureFieldWithLength(int length) => length + VarInt.MeasureUnsigned(length);

		static Span<byte> WriteFieldWithLength(this ICbWriter writer, CbFieldType type, CbFieldName name, int length)
		{
			int fullLength = MeasureFieldWithLength(length);
			Span<byte> buffer = writer.WriteField(type, name, fullLength);

			int lengthLength = VarInt.WriteUnsigned(buffer, length);
			return buffer.Slice(lengthLength);
		}

		/// <summary>
		/// Begin writing an object field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public static void BeginObject(this ICbWriter writer) => writer.BeginObject(default);

		/// <summary>
		/// Begin writing an array field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public static void BeginArray(this ICbWriter writer) => writer.BeginArray(default, CbFieldType.None);

		/// <summary>
		/// Begin writing a named array field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		public static void BeginArray(this ICbWriter writer, CbFieldName name) => writer.BeginArray(name, CbFieldType.None);

		/// <summary>
		/// Begin writing a uniform array field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="fieldType">The field type for elements in the array</param>
		public static void BeginUniformArray(this ICbWriter writer, CbFieldType fieldType) => BeginUniformArray(writer, default, fieldType);

		/// <summary>
		/// Begin writing a named uniform array field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="fieldType">The field type for elements in the array</param>
		public static void BeginUniformArray(this ICbWriter writer, CbFieldName name, CbFieldType fieldType) => writer.BeginArray(name, fieldType);

		/// <summary>
		/// End writing a uniform array field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public static void EndUniformArray(this ICbWriter writer) => writer.EndArray();

		/// <summary>
		/// Copies an entire field value to the output
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="field"></param>
		public static void WriteFieldValue(this ICbWriter writer, CbField field) => WriteField(writer, default, field);

		/// <summary>
		/// Copies an entire field value to the output, using the name from the field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="field"></param>
		public static void WriteField(this ICbWriter writer, CbField field) => WriteField(writer, field.GetName(), field);

		/// <summary>
		/// Copies an entire field value to the output
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="field"></param>
		public static void WriteField(this ICbWriter writer, CbFieldName name, CbField field)
		{
			ReadOnlySpan<byte> source = field.GetPayloadView().Span;
			Span<byte> target = writer.WriteField(field.GetType(), name, source.Length);
			source.CopyTo(target);
		}

		/// <summary>
		/// Write a null field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		public static void WriteNullValue(this ICbWriter writer) => WriteNull(writer, default);

		/// <summary>
		/// Write a named null field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		public static void WriteNull(this ICbWriter writer, CbFieldName name) => writer.WriteField(CbFieldType.Null, name, 0);

		/// <summary>
		/// Writes a boolean value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBoolValue(this ICbWriter writer, bool value) => WriteBool(writer, default, value);

		/// <summary>
		/// Writes a boolean value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBool(this ICbWriter writer, CbFieldName name, bool value) => writer.WriteField(value ? CbFieldType.BoolTrue : CbFieldType.BoolFalse, name, 0);

		/// <summary>
		/// Writes an unnamed integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteIntegerValue(this ICbWriter writer, int value) => WriteInteger(writer, default, value);

		/// <summary>
		/// Writes an unnamed integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteIntegerValue(this ICbWriter writer, long value) => WriteInteger(writer, default, value);

		/// <summary>
		/// Writes an named integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteInteger(this ICbWriter writer, CbFieldName name, int value) => WriteInteger(writer, name, (long)value);

		/// <summary>
		/// Writes an named integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteInteger(this ICbWriter writer, CbFieldName name, long value)
		{
			if (value >= 0)
			{
				int length = VarInt.MeasureUnsigned((ulong)value);
				Span<byte> data = writer.WriteField(CbFieldType.IntegerPositive, name, length);
				VarInt.WriteUnsigned(data, (ulong)value);
			}
			else
			{
				int length = VarInt.MeasureUnsigned((ulong)-value);
				Span<byte> data = writer.WriteField(CbFieldType.IntegerNegative, name, length);
				VarInt.WriteUnsigned(data, (ulong)-value);
			}
		}

		/// <summary>
		/// Writes an unnamed integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteIntegerValue(this ICbWriter writer, ulong value) => WriteInteger(writer, default, value);

		/// <summary>
		/// Writes a named integer field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteInteger(this ICbWriter writer, CbFieldName name, ulong value)
		{
			int length = VarInt.MeasureUnsigned((ulong)value);
			Span<byte> data = writer.WriteField(CbFieldType.IntegerPositive, name, length);
			VarInt.WriteUnsigned(data, (ulong)value);
		}

		/// <summary>
		/// Writes an unnamed double field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteDoubleValue(this ICbWriter writer, double value) => WriteDouble(writer, default, value);

		/// <summary>
		/// Writes a named double field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteDouble(this ICbWriter writer, CbFieldName name, double value)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.Float64, name, sizeof(double));
			BinaryPrimitives.WriteDoubleBigEndian(buffer, value);
		}

		/// <summary>
		/// Writes an unnamed <see cref="DateTime"/> field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteDateTimeValue(this ICbWriter writer, DateTime value) => WriteDateTime(writer, default, value);

		/// <summary>
		/// Writes a named <see cref="DateTime"/> field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteDateTime(this ICbWriter writer, CbFieldName name, DateTime value)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.DateTime, name, sizeof(long));
			BinaryPrimitives.WriteInt64BigEndian(buffer, value.Ticks);
		}

		/// <summary>
		/// Writes an unnamed <see cref="IoHash"/> field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteHashValue(this ICbWriter writer, IoHash value) => WriteHash(writer, default, value);

		/// <summary>
		/// Writes a named <see cref="IoHash"/> field
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteHash(this ICbWriter writer, CbFieldName name, IoHash value)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.Hash, name, IoHash.NumBytes);
			value.CopyTo(buffer);
		}

		/// <summary>
		/// Writes an unnamed reference to a binary attachment
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="hash">Hash of the attachment</param>
		public static void WriteBinaryAttachmentValue(this ICbWriter writer, IoHash hash) => WriteBinaryAttachment(writer, default, hash);

		/// <summary>
		/// Writes a named reference to a binary attachment
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="hash">Hash of the attachment</param>
		public static void WriteBinaryAttachment(this ICbWriter writer, CbFieldName name, IoHash hash)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.BinaryAttachment, name, IoHash.NumBytes);
			hash.CopyTo(buffer);
		}

		/// <summary>
		/// Writes an object directly into the writer
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="obj">Object to write</param>
		public static void WriteObject(this ICbWriter writer, CbObject obj) => WriteObject(writer, default, obj);

		/// <summary>
		/// Writes an object directly into the writer
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the object</param>
		/// <param name="obj">Object to write</param>
		public static void WriteObject(this ICbWriter writer, CbFieldName name, CbObject obj)
		{
			ReadOnlyMemory<byte> view = obj.AsField().Payload;
			Span<byte> buffer = writer.WriteField(CbFieldType.Object, name, view.Length);
			view.Span.CopyTo(buffer);
		}

		/// <summary>
		/// Writes an unnamed reference to an object attachment
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="hash">Hash of the attachment</param>
		public static void WriteObjectAttachmentValue(this ICbWriter writer, IoHash hash) => WriteObjectAttachment(writer, default, hash);

		/// <summary>
		/// Writes a named reference to an object attachment
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="hash">Hash of the attachment</param>
		public static void WriteObjectAttachment(this ICbWriter writer, CbFieldName name, IoHash hash)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.ObjectAttachment, name, IoHash.NumBytes);
			hash.CopyTo(buffer);
		}

		/// <summary>
		/// Writes a named object id
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="objectId">The object id</param>
		/// <exception cref="NotImplementedException"></exception>
		public static void WriteObjectId(this ICbWriter writer, CbFieldName name, CbObjectId objectId)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.ObjectId, name, 12);
			objectId.CopyTo(buffer);
		}

		/// <summary>
		/// Writes a unnamed object id
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="objectId">The object id</param>
		/// <exception cref="NotImplementedException"></exception>
		public static void WriteObjectIdValue(this ICbWriter writer, CbObjectId objectId)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.ObjectId, default, 12);
			objectId.CopyTo(buffer);
		}

		/// <summary>
		/// Writes an unnamed string value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteStringValue(this ICbWriter writer, string value) => WriteUtf8StringValue(writer, new Utf8String(value));

		/// <summary>
		/// Writes a named string value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteString(this ICbWriter writer, CbFieldName name, string? value)
		{
			if (value != null)
			{
				writer.WriteUtf8String(name, new Utf8String(value));
			}
		}

		/// <summary>
		/// Writes an unnamed string value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteUtf8StringValue(this ICbWriter writer, Utf8String value) => WriteUtf8String(writer, default, value);

		/// <summary>
		/// Writes a named string value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteUtf8String(this ICbWriter writer, CbFieldName name, Utf8String value)
		{
			Span<byte> buffer = WriteFieldWithLength(writer, CbFieldType.String, name, value.Length);
			value.Span.CopyTo(buffer);
		}

		/// <summary>
		/// Writes a named uuid/guid value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteUuid(this ICbWriter writer, CbFieldName name, Guid value)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.Uuid, name, 16);
			value.TryWriteBytes(buffer, true, out _);
		}

		/// <summary>
		/// Writes a unnamed uuid/guid value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">The uuid value</param>
		public static void WriteUuidValue(this ICbWriter writer, Guid value)
		{
			Span<byte> buffer = writer.WriteField(CbFieldType.Uuid, default, 16);
			value.TryWriteBytes(buffer, true, out _);
		}

		/// <summary>
		/// Writes an external binary value into the output stream
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="data">Data to reference</param>
		public static void WriteBinaryReference(this ICbWriter writer, CbFieldName name, ReadOnlyMemory<byte> data)
		{
			int lengthBytes = VarInt.MeasureUnsigned(data.Length);
			Span<byte> span = writer.WriteField(CbFieldType.Binary, name, lengthBytes);
			VarInt.WriteUnsigned(span, data.Length);
			writer.WriteReference(data);
		}

		/// <summary>
		/// Writes an unnamed binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinarySpanValue(this ICbWriter writer, ReadOnlySpan<byte> value) => WriteBinarySpan(writer, default, value);

		/// <summary>
		/// Writes a named binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinarySpan(this ICbWriter writer, CbFieldName name, ReadOnlySpan<byte> value)
		{
			Span<byte> buffer = WriteFieldWithLength(writer, CbFieldType.Binary, name, value.Length);
			value.CopyTo(buffer);
		}

		/// <summary>
		/// Writes an unnamed binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinaryValue(this ICbWriter writer, ReadOnlyMemory<byte> value) => writer.WriteBinarySpanValue(value.Span);

		/// <summary>
		/// Writes a named binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinary(this ICbWriter writer, CbFieldName name, ReadOnlyMemory<byte> value) => writer.WriteBinarySpan(name, value.Span);

		/// <summary>
		/// Writes an unnamed binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinaryArrayValue(this ICbWriter writer, byte[] value) => writer.WriteBinarySpanValue(value);

		/// <summary>
		/// Writes a named binary value
		/// </summary>
		/// <param name="writer">Writer for output data</param>
		/// <param name="name">Name of the field</param>
		/// <param name="value">Value to be written</param>
		public static void WriteBinaryArray(this ICbWriter writer, CbFieldName name, byte[] value) => writer.WriteBinarySpan(name, value);
	}
}