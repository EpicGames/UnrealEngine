// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.IO;
using System.IO.Hashing;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using Blake3;
using EpicGames.Compression;
using EpicGames.Core;

namespace EpicGames.Serialization
{
	/// <summary>
	/// Binary header for the Unreal compressed buffer format.
	/// </summary>
	public class CompressedBufferHeader
	{
		/// <summary>
		/// The type of compression for this buffer.
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1028:Enum Storage should be Int32", Justification = "Interop requires byte")]
		public enum CompressionMethod : byte
		{
			/// <summary>
			/// Header is followed by one uncompressed block.
			/// </summary>
			None = 0,
			/// <summary>
			/// Header is followed by an array of compressed block sizes then the compressed blocks.
			/// </summary>
			Oodle = 3,
			// Support for LZ4 is no longer required and has been removed.
			//LZ4 = 4,
		}

		/// <summary>
		/// A sequence of four bytes identifying a byte-stream as an Unreal compressed buffer.
		/// </summary>
		public const uint ExpectedMagic = 0xb7756362; // <dot>ucb
		/// <summary>
		/// The length of the binary header.
		/// </summary>
		public const uint HeaderLength = 64;

		/// <summary>
		/// A magic number to identify a compressed buffer. Should always equal <see cref="ExpectedMagic"/>.
		/// </summary>
		public uint Magic { get; set; }

		/// <summary>
		/// A CRC-32 used to check integrity of the buffer. Uses the polynomial 0x04c11db7.
		/// </summary>
		public uint Crc32 { get; set; }

		/// <summary>
		/// The method used to compress the buffer. Affects layout of data following the header. 
		/// </summary>
		public CompressionMethod Method { get; set; }
		/// <summary>
		/// The numerical compression level of the data. See <see cref="OodleCompressionLevel"/> for values for Oodle-compressed data.
		/// </summary>
		public byte CompressionLevel { get; set; }
		/// <summary>
		/// For Oodle compression, represents the Oodle algorithm used (Mermaid, Selkie etc), see <see cref="OodleCompressorType"/>.
		/// </summary>
		public byte CompressionMethodUsed { get; set; }

		/// <summary>
		/// The power of two size of every uncompressed block except the last. Size is 1 binary-left-shifted by <see cref="BlockSizeExponent" />.
		/// </summary>
		public byte BlockSizeExponent { get; set; }

		/// <summary>
		/// The number of blocks that follow the header.
		/// </summary>
		public uint BlockCount { get; set; }

		/// <summary>
		/// The total size of the uncompressed data.
		/// </summary>
		public ulong TotalRawSize { get; set; }

		/// <summary>
		/// The total size of the compressed data including the header.
		/// </summary>
		public ulong TotalCompressedSize { get; set; }

		/// <summary>
		/// The hash of the uncompressed data.
		/// </summary>
		public byte[] RawHash { get; set; } = [];

		/// <summary>
		/// Swap endianness of the header if the running system is not big-endian.
		/// </summary>
		public void ByteSwap()
		{
			Magic = BinaryPrimitives.ReverseEndianness(Magic);
			Crc32 = BinaryPrimitives.ReverseEndianness(Crc32);
			BlockCount = BinaryPrimitives.ReverseEndianness(BlockCount);
			TotalRawSize = BinaryPrimitives.ReverseEndianness(TotalRawSize);
			TotalCompressedSize = BinaryPrimitives.ReverseEndianness(TotalCompressedSize);
		}
	}

	/// <summary>
	/// Utilities for working with compressed buffers
	/// </summary>
	public static class CompressedBuffer
	{
		private static (CompressedBufferHeader, uint[]) ExtractHeader(BinaryReader br)
		{
			byte[] headerData = br.ReadBytes((int)CompressedBufferHeader.HeaderLength);

			using MemoryStream ms = new(headerData);
			using BinaryReader reader = new(ms);

			// the header is always stored big endian
			bool needsByteSwap = BitConverter.IsLittleEndian;

			CompressedBufferHeader header = new()
			{
				Magic = reader.ReadUInt32(),
				Crc32 = reader.ReadUInt32(),
				Method = (CompressedBufferHeader.CompressionMethod)reader.ReadByte(),
				CompressionLevel = reader.ReadByte(),
				CompressionMethodUsed = reader.ReadByte(),
				BlockSizeExponent = reader.ReadByte(),
				BlockCount = reader.ReadUInt32(),
				TotalRawSize = reader.ReadUInt64(),
				TotalCompressedSize = reader.ReadUInt64()
			};
			byte[] hash = reader.ReadBytes(32); // a full blake3 hash
			header.RawHash = hash;

			if (needsByteSwap)
			{
				header.ByteSwap();
			}

			if (header.Magic != CompressedBufferHeader.ExpectedMagic)
			{
				throw new InvalidMagicException(header.Magic, CompressedBufferHeader.ExpectedMagic);
			}

			// calculate the crc from the start of the method field (skipping magic which is a constant and the crc field itself)
			const int MethodOffset = sizeof(uint) + sizeof(uint);

			// none compressed objects have no extra blocks
			uint blocksByteUsed = header.Method != CompressedBufferHeader.CompressionMethod.None ? header.BlockCount * (uint)sizeof(uint) : 0;

			byte[] crcData = new byte[blocksByteUsed + headerData.Length];
			Array.Copy(headerData, crcData, headerData.Length);

			uint[] blocks = [];

			if (blocksByteUsed != 0)
			{
				byte[] blocksData = br.ReadBytes((int)blocksByteUsed);

				Array.Copy(blocksData, 0, crcData, headerData.Length, blocksData.Length);

				blocks = new uint[header.BlockCount];

				for (int i = 0; i < header.BlockCount; i++)
				{
					ReadOnlySpan<byte> memory = new(blocksData, i * sizeof(uint), sizeof(uint));
					uint compressedBlockSize = BinaryPrimitives.ReadUInt32BigEndian(memory);
					blocks[i] = compressedBlockSize;
				}
			}

			uint calculatedCrc = Crc32.HashToUInt32(crcData.AsSpan(MethodOffset, (int)(CompressedBufferHeader.HeaderLength - MethodOffset + blocksByteUsed)));

			if (header.Crc32 != calculatedCrc)
			{
				throw new InvalidHashException(header.Crc32, calculatedCrc);
			}

			return (header, blocks);
		}

		private static void WriteHeader(CompressedBufferHeader header, BinaryWriter writer)
		{
			// the header is always stored big endian
			bool needsByteSwap = BitConverter.IsLittleEndian;
			if (needsByteSwap)
			{
				header.ByteSwap();
			}

			writer.Write(header.Magic);
			writer.Write(header.Crc32);
			writer.Write((byte)header.Method);
			writer.Write((byte)header.CompressionLevel);
			writer.Write((byte)header.CompressionMethodUsed);
			writer.Write((byte)header.BlockSizeExponent);
			writer.Write(header.BlockCount);
			writer.Write(header.TotalRawSize);
			writer.Write(header.TotalCompressedSize);
			writer.Write(header.RawHash, 0, 20); // write the first 20 bytes as iohashes are 20 bytes
			for (int i = 0; i < 12; i++)
			{
				// the last 12 bytes should be 0 as they are reserved
				writer.Write((byte)0);
			}

			if (needsByteSwap)
			{
				header.ByteSwap();
			}
		}

		/// <summary>
		/// Decompress a compressed buffer into a stream
		/// </summary>
		/// <param name="sourceStream">Source stream containing a compressed buffer</param>
		/// <param name="streamSize">The length of the compressed buffer</param>
		/// <param name="targetStream">The stream to write the decompressed content to</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns></returns>
		/// <exception cref="Exception"></exception>
		public static async Task<CompressedBufferHeader> DecompressContentAsync(Stream sourceStream, ulong streamSize, Stream targetStream, CancellationToken cancellationToken = default)
		{
			using BinaryReader br = new(sourceStream);
			(CompressedBufferHeader header, uint[] compressedBlockSizes) = ExtractHeader(br);

			if (streamSize != header.TotalCompressedSize)
			{
				throw new Exception($"Expected stream to be {header.TotalCompressedSize} but it was {streamSize}");
			}

			ulong decompressedPayloadOffset = 0;

			bool willHaveBlocks = header.Method != CompressedBufferHeader.CompressionMethod.None;
			if (willHaveBlocks)
			{
				ulong blockSize = 1ul << header.BlockSizeExponent;

				foreach (uint compressedBlockSize in compressedBlockSizes)
				{
					ulong rawBlockSize = Math.Min(header.TotalRawSize - decompressedPayloadOffset, blockSize);
					byte[] compressedPayload = br.ReadBytes((int)compressedBlockSize);

					int writtenBytes;
					// if a block has the same raw and compressed size its uncompressed and we should not attempt to decompress it
					if (rawBlockSize == compressedBlockSize)
					{
						writtenBytes = (int)rawBlockSize;
						targetStream.Write(compressedPayload);
					}
					else
					{
						writtenBytes = DecompressPayload(compressedPayload, header, rawBlockSize, targetStream);
					}

					decompressedPayloadOffset += (uint)writtenBytes;
				}
			}
			else
			{
				await sourceStream.CopyToAsync(targetStream, cancellationToken);
			}

			return header;
		}

		private static int DecompressPayload(ReadOnlySpan<byte> compressedPayload, CompressedBufferHeader header, ulong rawBlockSize, Stream target)
		{
			switch (header.Method)
			{
				case CompressedBufferHeader.CompressionMethod.None:
					target.Write(compressedPayload);
					return compressedPayload.Length;
				case CompressedBufferHeader.CompressionMethod.Oodle:
					{
						byte[] result = new byte[rawBlockSize];
						long writtenBytes = Oodle.Decompress(compressedPayload, result);
						if (writtenBytes == 0)
						{
							throw new Exception("Failed to run oodle decompress");
						}
						target.Write(result);
						return (int)writtenBytes;
					}
				default:
					throw new NotImplementedException($"Method {header.Method} is not a supported value");
			}
		}

		/// <summary>
		/// Compress a stream into a compressed buffer stream.
		/// </summary>
		/// <param name="s">The stream to write the compressed content into.</param>
		/// <param name="method">Which Oodle compressor method to use.</param>
		/// <param name="compressionLevel">The Oodle compression level.</param>
		/// <param name="rawContents">The uncompressed content you wish to compress.</param>
		/// <returns>The iohash of the uncompressed content.</returns>
		public static IoHash CompressContent(Stream s, OodleCompressorType method, OodleCompressionLevel compressionLevel, byte[] rawContents)
		{
			const long DefaultBlockSize = 256 * 1024;
			long blockSize = DefaultBlockSize;
			long blockCount = (rawContents.LongLength + blockSize - 1) / blockSize;
			Span<byte> contentsSpan = new(rawContents);
			List<byte[]> blocks = [];

			for (int i = 0; i < blockCount; i++)
			{
				int rawBlockSize = Math.Min(rawContents.Length - (i * (int)blockSize), (int)blockSize);
				Span<byte> bufferToCompress = contentsSpan.Slice((int)(i * blockSize), rawBlockSize);

				blocks.Add(bufferToCompress.ToArray());
			}

			return CompressContent(s, method, compressionLevel, blocks, blockSize);
		}

		/// <summary>
		/// Compress a stream into a compressed buffer stream.
		/// </summary>
		/// <param name="s">The stream to write the compressed content into.</param>
		/// <param name="method">Which Oodle compressor method to use.</param>
		/// <param name="compressionLevel">The Oodle compression level.</param>
		/// <param name="sourceStream">The uncompressed content in a stream that you wish to compress.</param>
		/// <returns>The iohash of the uncompressed content.</returns>
		public static IoHash CompressContent(Stream s, OodleCompressorType method, OodleCompressionLevel compressionLevel, Stream sourceStream)
		{
			const long DefaultBlockSize = 256 * 1024;
			long blockSize = DefaultBlockSize;
			long blockCount = (sourceStream.Length + blockSize - 1) / blockSize;
			List<byte[]> blocks = [];

			for (int i = 0; i < blockCount; i++)
			{
				long rawBlockSize = Math.Min(sourceStream.Length - (i * blockSize), blockSize);
				byte[] bufferToCompress = new byte[(int)rawBlockSize];

				sourceStream.ReadFixedLengthBytes(bufferToCompress);
				blocks.Add(bufferToCompress);
			}

			return CompressContent(s, method, compressionLevel, blocks, blockSize);
		}

		/// <summary>
		/// Compress a stream into a compressed buffer stream
		/// </summary>
		/// <param name="s">The stream to write the compressed content into.</param>
		/// <param name="method">Which Oodle compressor method to use.</param>
		/// <param name="compressionLevel">The Oodle compression level.</param>
		/// <param name="blocks">A list of binary blocks, the first N-1 of which should be <paramref name="blockSize"/> bytes.
		/// The last block may be that size or less.</param>
		/// <param name="blockSize">The size of all binary blocks but the last, and the upper bound size for the last block.</param>
		/// <returns>The iohash of the uncompressed content.</returns>
		public static IoHash CompressContent(Stream s, OodleCompressorType method, OodleCompressionLevel compressionLevel, List<byte[]> blocks, long blockSize)
		{
			long blockCount = blocks.Count;

			byte blockSizeExponent = (byte)Math.Floor(Math.Log2(blockSize));

			List<byte[]> compressedBlocks = [];
			using Hasher hasher = Hasher.New();

			ulong uncompressedContentLength = (ulong)blocks.Sum(b => b.LongLength);

			ulong compressedContentLength = CompressedBufferHeader.HeaderLength;
			for (int i = 0; i < blockCount; i++)
			{
				int rawBlockSize = blocks[i].Length;
				byte[] bufferToCompress = blocks[i];
				hasher.UpdateWithJoin(new ReadOnlySpan<byte>(bufferToCompress, 0, rawBlockSize));
				int maxSize = Oodle.MaximumOutputSize(method, rawBlockSize);
				byte[] compressedBlock = new byte[maxSize];
				long encodedSize = Oodle.Compress(method, bufferToCompress, compressedBlock, compressionLevel);

				if (encodedSize == 0)
				{
					throw new Exception("Failed to compress content");
				}

				byte[] actualCompressedBlock = new byte[encodedSize];
				Array.Copy(compressedBlock, actualCompressedBlock, encodedSize);
				compressedBlocks.Add(actualCompressedBlock);
				compressedContentLength += (ulong)encodedSize;
			}
			compressedContentLength += (uint)(sizeof(uint) * blockCount);

			Hash blake3Hash = hasher.Finalize();
			byte[] hashData = blake3Hash.AsSpan().Slice(0, 20).ToArray();
			IoHash hash = new(hashData);

			CompressedBufferHeader header = new()
			{
				Magic = CompressedBufferHeader.ExpectedMagic,
				Crc32 = 0,
				Method = CompressedBufferHeader.CompressionMethod.Oodle,
				CompressionLevel = (byte)compressionLevel,
				CompressionMethodUsed = (byte)method,
				BlockSizeExponent = blockSizeExponent,
				BlockCount = (uint)blockCount,
				TotalRawSize = (ulong)uncompressedContentLength,
				TotalCompressedSize = (ulong)compressedContentLength,
				RawHash = hashData
			};

			byte[] headerAndBlocks = WriteHeaderToBuffer(header, [.. compressedBlocks.Select(b => (uint)b.Length)]);

			using BinaryWriter writer = new(s, Encoding.Default, leaveOpen: true);

			writer.Write(headerAndBlocks);

			for (int i = 0; i < blockCount; i++)
			{
				writer.Write(compressedBlocks[i]);
			}

			return hash;
		}

		private static byte[] WriteHeaderToBuffer(CompressedBufferHeader header, uint[] compressedBlockLengths)
		{
			uint blockCount = header.BlockCount;
			uint blocksByteUsed = blockCount * sizeof(uint);

			byte[] headerBuffer = new byte[CompressedBufferHeader.HeaderLength + blocksByteUsed];

			// write the compressed buffer, but with the wrong crc which we update and rewrite later
			{
				using MemoryStream ms = new(headerBuffer);
				using BinaryWriter writer = new(ms);

				WriteHeader(header, writer);

				for (int i = 0; i < blockCount; i++)
				{
					uint value = compressedBlockLengths[i];
					if (BitConverter.IsLittleEndian)
					{
						value = BinaryPrimitives.ReverseEndianness(value);
					}
					writer.Write(value);
				}
			}

			// calculate the crc from the start of the method field (skipping magic which is a constant and the crc field itself)
			const int MethodOffset = sizeof(uint) + sizeof(uint);
			uint calculatedCrc = Crc32.HashToUInt32(headerBuffer.AsSpan(MethodOffset, (int)(CompressedBufferHeader.HeaderLength - MethodOffset + blocksByteUsed)));
			header.Crc32 = calculatedCrc;

			// write the header again now that we have the crc
			{
				using MemoryStream ms = new(headerBuffer);
				using BinaryWriter writer = new(ms);
				WriteHeader(header, writer);
			}

			return headerBuffer;
		}
	}

	/// <summary>
	/// Thrown when the crc32 of the header does not align with the decompressed content, usually indicating that the full content was not present.
	/// </summary>
	/// <param name="headerCrc32"></param>
	/// <param name="calculatedCrc"></param>
	public class InvalidHashException(uint headerCrc32, uint calculatedCrc)
		: Exception($"Header specified crc \"{headerCrc32}\" but calculated hash was \"{calculatedCrc}\"");

	/// <summary>
	/// Thrown when a byte buffer is encountered that is not a compressed buffer
	/// </summary>
	/// <param name="headerMagic"></param>
	/// <param name="expectedMagic"></param>
	public class InvalidMagicException(uint headerMagic, uint expectedMagic)
		: Exception($"Header magic \"{headerMagic}\" was incorrect, expected to be {expectedMagic}");
}
