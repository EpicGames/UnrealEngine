// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicGames.Serialization.Converters
{
	/// <summary>
	/// Converter for array types
	/// </summary>
	/// <typeparam name="T"></typeparam>
	class CbArrayConverter<T> : CbConverter<T[]>
	{
		/// <inheritdoc/>
		public override T[] Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}

			List<T> list = [];
			foreach (CbField elementField in field)
			{
				list.Add(CbSerializer.Deserialize<T>(elementField));
			}
			return [.. list];
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, T[] list)
		{
			if (list == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.BeginArray();
				foreach (T element in list)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndArray();
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, T[] array)
		{
			if (array == null)
			{
				writer.WriteNull(name);
			}
			else
			{
				writer.BeginArray(name);
				foreach (T element in array)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndArray();
			}
		}
	}

	/// <summary>
	/// Factory for CbListConverter
	/// </summary>
	class CbArrayConverterFactory : CbConverterFactory
	{
		/// <inheritdoc/>
		public override CbConverter? CreateConverter(Type type)
		{
			if (type.IsArray)
			{
				Type elementType = type.GetElementType()!;
				Type converterType = typeof(CbArrayConverter<>).MakeGenericType(elementType);
				return (CbConverter)Activator.CreateInstance(converterType)!;
			}
			return null;
		}
	}
}
