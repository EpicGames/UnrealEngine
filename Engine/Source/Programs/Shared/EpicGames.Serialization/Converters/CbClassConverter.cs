// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Linq.Expressions;
using System.Reflection;
using System.Reflection.Emit;
using EpicGames.Core;

namespace EpicGames.Serialization.Converters
{
	class CbClassConverterMethods
	{
		class CbReflectedTypeInfo<T> where T : class
		{
			public static Utf8String[]? Names = null;
			public static PropertyInfo[]? Properties = null;

			public static bool MatchName(CbField field, int idx)
			{
				return field.Name == Names![idx];
			}
		}

		static readonly Utf8String s_discriminatorKey = new("_t");

		public Type ClassType { get; }
		public bool IsPolymorphic { get; }

		public DynamicMethod ReadMethod { get; }
		public DynamicMethod WriteMethod { get; }
		public DynamicMethod WriteNamedMethod { get; }

		public DynamicMethod WriteContentsMethod { get; }

		public DynamicMethod ReadConcreteMethod { get; }
		public DynamicMethod WriteConcreteContentsMethod { get; }

		static readonly Dictionary<Type, CbClassConverterMethods> s_typeToMethods = [];

		public static CbClassConverterMethods Create(Type classType)
		{
			CbClassConverterMethods? methods;
			if (!s_typeToMethods.TryGetValue(classType, out methods))
			{
				methods = new CbClassConverterMethods(classType);
				s_typeToMethods.Add(classType, methods);
				methods.GenerateBytecode();
			}
			return methods;
		}

		private CbClassConverterMethods(Type classType)
		{
			ClassType = classType;
			IsPolymorphic = (classType.GetCustomAttribute<CbPolymorphicAttribute>(true) != null);

			ReadConcreteMethod = new DynamicMethod($"ReadConcrete_{classType.Name}", classType, [typeof(CbField)]);
			WriteConcreteContentsMethod = new DynamicMethod($"WriteConcreteContents_{classType.Name}", null, [typeof(CbWriter), classType]);

			WriteMethod = new DynamicMethod($"Write_{classType.Name}", null, [typeof(CbWriter), classType]);
			WriteNamedMethod = new DynamicMethod($"WriteNamed_{classType.Name}", null, [typeof(CbWriter), typeof(CbFieldName), classType]);

			if (IsPolymorphic)
			{
				ReadMethod = new DynamicMethod($"Read_{classType.Name}", classType, [typeof(CbField)]);
				WriteContentsMethod = new DynamicMethod($"WriteContents_{classType.Name}", null, [typeof(CbWriter), classType]);
			}
			else
			{
				ReadMethod = ReadConcreteMethod;
				WriteContentsMethod = WriteConcreteContentsMethod;
			}
		}

		private void GenerateBytecode()
		{
			// Create the regular methods
			CreateConcreteObjectReader(ClassType, ReadConcreteMethod.GetILGenerator());
			CreateConcreteObjectContentsWriter(ClassType, WriteConcreteContentsMethod.GetILGenerator());

			CreateObjectWriter(WriteMethod.GetILGenerator(), WriteContentsMethod);
			CreateNamedObjectWriter(WriteNamedMethod.GetILGenerator(), WriteContentsMethod);

			// Create the extra polymorphic methods
			if (IsPolymorphic)
			{
				// Create the dispatch type
				Type dispatchType = typeof(CbPolymorphicDispatch<>).MakeGenericType(ClassType);

				// Create the read dispatch method
				{
					ILGenerator generator = ReadMethod.GetILGenerator();
					generator.Emit(OpCodes.Ldarg_0);
					generator.Emit(OpCodes.Call, dispatchType.GetMethod(nameof(CbPolymorphicDispatch<object>.Read))!);
					generator.Emit(OpCodes.Ret);
				}

				// Create the write dispatch method
				{
					ILGenerator generator = WriteContentsMethod.GetILGenerator();
					generator.Emit(OpCodes.Ldarg_0);
					generator.Emit(OpCodes.Ldarg_1);
					generator.Emit(OpCodes.Call, dispatchType.GetMethod(nameof(CbPolymorphicDispatch<object>.WriteContents))!);
					generator.Emit(OpCodes.Ret);
				}

				// Finally, update the dispatch type with all the methods. We should be safe on the recursive path now.
				PopulateDispatchType(ClassType, dispatchType);
			}
		}

		static void PopulateDispatchType(Type classType, Type dispatchType)
		{
			Dictionary<Utf8String, Type> discriminatorToKnownType = [];

			Type[] knownTypes = classType.Assembly.GetTypes();
			foreach (Type knownType in knownTypes)
			{
				if (knownType.IsClass && !knownType.IsAbstract)
				{
					for (Type? baseType = knownType; baseType != null; baseType = baseType.BaseType)
					{
						if (baseType == classType)
						{
							CbDiscriminatorAttribute discriminator = knownType.GetCustomAttribute<CbDiscriminatorAttribute>() ?? throw new NotSupportedException();
							discriminatorToKnownType[new Utf8String(discriminator.Name)] = knownType;
						}
					}
				}
			}

			// Populate the dictionary
			Dictionary<Utf8String, Func<CbField, object>> nameToReadFunc = (Dictionary<Utf8String, Func<CbField, object>>)dispatchType.GetField(nameof(CbPolymorphicDispatch<object>.NameToReadFunc))!.GetValue(null)!;
			Dictionary<Type, Action<CbWriter, object>> typeToWriteContentsFunc = (Dictionary<Type, Action<CbWriter, object>>)dispatchType.GetField(nameof(CbPolymorphicDispatch<object>.TypeToWriteContentsFunc))!.GetValue(null)!;

			foreach ((Utf8String name, Type knownType) in discriminatorToKnownType)
			{
				CbClassConverterMethods methods = Create(knownType);

				{
					DynamicMethod dynamicMethod = new("_", typeof(object), [typeof(CbField)]);
					ILGenerator generator = dynamicMethod.GetILGenerator();
					generator.Emit(OpCodes.Ldarg_0);
					generator.Emit(OpCodes.Call, methods.ReadConcreteMethod);
					generator.Emit(OpCodes.Castclass, typeof(object));
					generator.Emit(OpCodes.Ret);
					nameToReadFunc[name] = CreateDelegate<Func<CbField, object>>(dynamicMethod);
				}

				{
					DynamicMethod dynamicMethod = new("_", null, [typeof(CbWriter), typeof(object)]);
					ILGenerator generator = dynamicMethod.GetILGenerator();
					generator.Emit(OpCodes.Ldarg_0);
					generator.Emit(OpCodes.Ldarg_1);
					generator.Emit(OpCodes.Castclass, knownType);
					generator.Emit(OpCodes.Call, methods.WriteConcreteContentsMethod);
					generator.Emit(OpCodes.Ret);
					typeToWriteContentsFunc[knownType] = CreateDelegate<Action<CbWriter, object>>(dynamicMethod);
				}
			}
		}

		static void CreateObjectWriter(ILGenerator generator, DynamicMethod contentsWriter)
		{
			generator.Emit(OpCodes.Ldarg_1);

			Label skipLabel = generator.DefineLabel();
			generator.Emit(OpCodes.Brfalse, skipLabel);

			generator.Emit(OpCodes.Ldarg_0);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.BeginObject()), null);

			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_1);
			generator.EmitCall(OpCodes.Call, contentsWriter, null);

			generator.Emit(OpCodes.Ldarg_0);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.EndObject()), null);

			generator.MarkLabel(skipLabel);
			generator.Emit(OpCodes.Ret);
		}

		static void CreateNamedObjectWriter(ILGenerator generator, DynamicMethod contentsWriter)
		{
			generator.Emit(OpCodes.Ldarg_2);

			Label skipLabel = generator.DefineLabel();
			generator.Emit(OpCodes.Brfalse, skipLabel);

			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_1);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.BeginObject(default)), null);

			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_2);
			generator.EmitCall(OpCodes.Call, contentsWriter, null);

			generator.Emit(OpCodes.Ldarg_0);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.EndObject()), null);

			generator.MarkLabel(skipLabel);
			generator.Emit(OpCodes.Ret);
		}

		static void CreateConcreteObjectContentsWriter(Type type, ILGenerator generator)
		{
			// Find the reflected properties from this type
			(Utf8String Name, PropertyInfo Property)[] properties = GetProperties(type);

			// Create a static type with the required reflection data
			Type reflectedType = typeof(CbReflectedTypeInfo<>).MakeGenericType(type);
			FieldInfo namesField = reflectedType.GetField(nameof(CbReflectedTypeInfo<object>.Names))!;
			namesField.SetValue(null, properties.Select(x => x.Name).ToArray());

			// Write the discriminator
			CbDiscriminatorAttribute? discriminator = type.GetCustomAttribute<CbDiscriminatorAttribute>();
			if (discriminator != null)
			{
				FieldInfo discriminatorKeyField = GetFieldInfo(() => s_discriminatorKey);
				generator.Emit(OpCodes.Ldarg_0);
				generator.Emit(OpCodes.Ldsfld, discriminatorKeyField);
				generator.Emit(OpCodes.Ldstr, discriminator.Name);
				generator.Emit(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.WriteString(default, null!)));
			}

			// Write all the remaining properties
			for (int idx = 0; idx < properties.Length; idx++)
			{
				PropertyInfo property = properties[idx].Property;
				Type propertyType = property.PropertyType;

				// Get the field value
				generator.Emit(OpCodes.Ldarg_1);
				generator.EmitCall(OpCodes.Call, property.GetMethod!, null);

				Label skipLabel = generator.DefineLabel();

				MethodInfo writeMethod;
				if (s_typeToMethods.TryGetValue(propertyType, out CbClassConverterMethods? dynamicMethods))
				{
					generator.Emit(OpCodes.Dup);
					generator.Emit(OpCodes.Brfalse, skipLabel);
					writeMethod = dynamicMethods.WriteNamedMethod;
				}
				else
				{
					ICbConverterMethods methods = CbConverterMethods.Get(property);
					writeMethod = methods.WriteNamedMethod;
				}

				// Store the variable in a local
				LocalBuilder local = generator.DeclareLocal(propertyType);
				generator.Emit(OpCodes.Dup);
				generator.Emit(OpCodes.Stloc, local);

				// Call the writer
				generator.Emit(OpCodes.Ldarg_0);

				generator.Emit(OpCodes.Ldsfld, namesField);
				generator.Emit(OpCodes.Ldc_I4, idx);
				generator.Emit(OpCodes.Ldelem, typeof(CbFieldName));

				generator.Emit(OpCodes.Ldloc, local);
				generator.EmitCall(OpCodes.Call, writeMethod, null);

				// Remove the duplicated value from the top of the stack
				generator.MarkLabel(skipLabel);
				generator.Emit(OpCodes.Pop);
			}
			generator.Emit(OpCodes.Ret);
		}

		class CbPolymorphicDispatch<T>
		{
			public static Dictionary<Utf8String, Func<CbField, object>> NameToReadFunc = [];
			public static Dictionary<Type, Action<CbWriter, object>> TypeToWriteContentsFunc = [];

			public static object Read(CbField field)
			{
				Utf8String name = field.AsObject().Find(s_discriminatorKey).AsUtf8String();
				return NameToReadFunc[name](field);
			}

			public static void WriteContents(CbWriter writer, object value)
			{
				Type type = value!.GetType();
				TypeToWriteContentsFunc[type](writer, value);
			}
		}

		static T CreateDelegate<T>(DynamicMethod method) where T : Delegate
		{
			return (T)method.CreateDelegate(typeof(T));
		}

		static void CreateConcreteObjectReader(Type type, ILGenerator generator)
		{
			// Construct the object
			ConstructorInfo? constructor =
				type.GetConstructor(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance, null, Type.EmptyTypes, null) ?? throw new CbException($"Unable to find default constructor for {type}");

			// Find the reflected properties from this type
			(Utf8String Name, PropertyInfo Property)[] properties = GetProperties(type);

			// Create a static type with the required reflection data
			Type reflectedType = typeof(CbReflectedTypeInfo<>).MakeGenericType(type);
			FieldInfo namesField = reflectedType.GetField(nameof(CbReflectedTypeInfo<object>.Names))!;
			namesField.SetValue(null, properties.Select(x => x.Name).ToArray());
			MethodInfo matchNameMethod = reflectedType.GetMethod(nameof(CbReflectedTypeInfo<object>.MatchName))!;

			// NewObjectLocal = new Type()
			LocalBuilder newObjectLocal = generator.DeclareLocal(typeof(object));
			generator.Emit(OpCodes.Newobj, constructor);
			generator.Emit(OpCodes.Stloc, newObjectLocal);

			// Stack(0) = CbField.CreateIterator()
			generator.Emit(OpCodes.Ldarg_0);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbField>(x => x.CreateIterator()), null);

			// CbFieldIterator IteratorLocal = Stack(0)
			LocalBuilder iteratorLocal = generator.DeclareLocal(typeof(CbFieldIterator));
			generator.Emit(OpCodes.Dup);
			generator.Emit(OpCodes.Stloc, iteratorLocal);

			// if(!Stack.Pop().IsValid()) goto ReturnLabel
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbFieldIterator>(x => x.IsValid()), null);
			Label returnLabel = generator.DefineLabel();
			generator.Emit(OpCodes.Brfalse, returnLabel);

			// NamesLocal = CbReflectedTypeInfo<Type>.Names
			LocalBuilder namesLocal = generator.DeclareLocal(typeof(Utf8String[]));
			generator.Emit(OpCodes.Ldsfld, namesField);
			generator.Emit(OpCodes.Stloc, namesLocal);

			// IterationLoopLabel:
			Label iterationLoopLabel = generator.DefineLabel();
			generator.MarkLabel(iterationLoopLabel);

			// bool MatchLocal = false
			LocalBuilder matchLocal = generator.DeclareLocal(typeof(bool));
			generator.Emit(OpCodes.Ldc_I4_0);
			generator.Emit(OpCodes.Stloc, matchLocal);

			// Stack(0) = IteratorLocal.GetCurrent()
			generator.Emit(OpCodes.Ldloc, iteratorLocal);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbFieldIterator>(x => x.GetCurrent()), null);

			// Try to parse each of the properties in order. If fields are ordered correctly, we will parse the object in a single pass. Otherwise we can loop and start again.
			LocalBuilder fieldLocal = generator.DeclareLocal(typeof(CbField));
			for (int idx = 0; idx < properties.Length; idx++)
			{
				PropertyInfo property = properties[idx].Property;

				// Get the read method for this property type
				MethodInfo readMethod;
				if (s_typeToMethods.TryGetValue(property.PropertyType, out CbClassConverterMethods? dynamicMethods))
				{
					readMethod = dynamicMethods.ReadMethod;
				}
				else
				{
					readMethod = CbConverterMethods.Get(property).ReadMethod;
				}

				// if(!CbReflectedTypeInfo<Type>.MatchName(Stack(0), Idx)) goto SkipPropertyLabel
				Label skipPropertyLabel = generator.DefineLabel();
				generator.Emit(OpCodes.Dup); // Current CbField
				generator.Emit(OpCodes.Ldc_I4, idx);
				generator.Emit(OpCodes.Call, matchNameMethod);
				generator.Emit(OpCodes.Brfalse, skipPropertyLabel);

				// FieldLocal = Stack.Pop()
				generator.Emit(OpCodes.Stloc, fieldLocal);

				// Copy the collection over if necessary
				if (property.SetMethod != null)
				{
					// Property.SetMethod(NewObjectLocal, ReadMethod(FieldLocal))
					generator.Emit(OpCodes.Ldloc, newObjectLocal);
					generator.Emit(OpCodes.Ldloc, fieldLocal);
					generator.EmitCall(OpCodes.Call, readMethod, null);
					generator.EmitCall(OpCodes.Call, property.SetMethod!, null);
				}
				else if (TryGetCollectionCopyMethod(property.PropertyType, out MethodInfo? copyMethod))
				{
					// CopyMethod(ReadMethod(FieldLocal), Property.GetMethod(NewObjectLocal))
					generator.Emit(OpCodes.Ldloc, fieldLocal);
					generator.EmitCall(OpCodes.Call, readMethod, null);
					generator.Emit(OpCodes.Ldloc, newObjectLocal);
					generator.EmitCall(OpCodes.Call, property.GetMethod!, null);
					generator.EmitCall(OpCodes.Call, copyMethod, null);
				}
				else
				{
					throw new CbException($"Unable to write to property {property.Name}");
				}

				// if(!IteratorLocal.MoveNext()) goto ReturnLabel
				generator.Emit(OpCodes.Ldloc, iteratorLocal);
				generator.EmitCall(OpCodes.Call, GetMethodInfo<CbFieldIterator>(x => x.MoveNext()), null);
				generator.Emit(OpCodes.Brfalse, returnLabel);

				// MatchLocal = true
				generator.Emit(OpCodes.Ldc_I4_1);
				generator.Emit(OpCodes.Stloc, matchLocal);

				// Stack(0) = IteratorLocal.GetCurrent()
				generator.Emit(OpCodes.Ldloc, iteratorLocal);
				generator.EmitCall(OpCodes.Call, GetMethodInfo<CbFieldIterator>(x => x.GetCurrent()), null);

				// SkipPropertyLabel:
				generator.MarkLabel(skipPropertyLabel);
			}

			// Stack.Pop()
			generator.Emit(OpCodes.Pop); // Current CbField

			// if(MatchLocal) goto IterationLoopLabel
			generator.Emit(OpCodes.Ldloc, matchLocal);
			generator.Emit(OpCodes.Brtrue, iterationLoopLabel);

			// if(IteratorLocal.MoveNext()) goto IterationLoopLabel
			generator.Emit(OpCodes.Ldloc, iteratorLocal);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbFieldIterator>(x => x.MoveNext()), null);
			generator.Emit(OpCodes.Brtrue, iterationLoopLabel);

			// return NewObjectLocal
			generator.MarkLabel(returnLabel);
			generator.Emit(OpCodes.Ldloc, newObjectLocal);
			generator.Emit(OpCodes.Ret);
		}

		static bool TryGetCollectionCopyMethod(Type type, [NotNullWhen(true)] out MethodInfo? method)
		{
			foreach (Type interfaceType in type.GetInterfaces())
			{
				if (interfaceType.IsGenericType && interfaceType.GetGenericTypeDefinition() == typeof(ICollection<>))
				{
					MethodInfo genericMethod = typeof(CbClassConverterMethods).GetMethod(nameof(CopyCollection), BindingFlags.Static | BindingFlags.NonPublic)!;
					method = genericMethod.MakeGenericMethod(interfaceType, interfaceType.GetGenericArguments()[0]);
					return true;
				}
			}

			method = null;
			return false;
		}

		static void CopyCollection<TCollection, TElement>(TCollection source, TCollection target) where TCollection : ICollection<TElement>
		{
			foreach (TElement element in source)
			{
				target.Add(element);
			}
		}

		static (Utf8String, PropertyInfo)[] GetProperties(Type type)
		{
			List<(Utf8String, PropertyInfo)> propertyList = [];
			foreach (PropertyInfo property in type.GetProperties(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance))
			{
				CbIgnoreAttribute? ignoreAttribute = property.GetCustomAttribute<CbIgnoreAttribute>();
				if (ignoreAttribute == null)
				{
					CbFieldAttribute? attribute = property.GetCustomAttribute<CbFieldAttribute>();
					if (attribute != null || (property.GetGetMethod()?.IsPublic ?? false))
					{
						Utf8String name = new(attribute?.Name ?? property.Name);
						propertyList.Add((name, property));
					}
				}
			}
			if (propertyList.Count == 0 && type.GetCustomAttribute<CbObjectAttribute>() == null)
			{
				throw new CbEmptyClassException(type);
			}
			return [.. propertyList];
		}

		static FieldInfo GetFieldInfo<T>(Expression<Func<T>> expr)
		{
			return (FieldInfo)((MemberExpression)expr.Body).Member;
		}

		static MethodInfo GetMethodInfo(Expression<Action> expr)
		{
			return ((MethodCallExpression)expr.Body).Method;
		}

		static MethodInfo GetMethodInfo<T>(Expression<Action<T>> expr)
		{
			return ((MethodCallExpression)expr.Body).Method;
		}
	}

	class CbClassConverter<T> : CbConverter<T>, ICbConverterMethods where T : class
	{
		readonly CbClassConverterMethods _methods;

		readonly Func<CbField, T> _readFunc;
		readonly Action<CbWriter, T> _writeFunc;
		readonly Action<CbWriter, CbFieldName, T> _writeNamedFunc;

		public CbClassConverter()
		{
			_methods = CbClassConverterMethods.Create(typeof(T));

			_readFunc = CreateDelegate<Func<CbField, T>>(_methods.ReadMethod);
			_writeFunc = CreateDelegate<Action<CbWriter, T>>(_methods.WriteMethod);
			_writeNamedFunc = CreateDelegate<Action<CbWriter, CbFieldName, T>>(_methods.WriteNamedMethod);
		}

		public MethodInfo ReadMethod => _methods.ReadMethod;

		public MethodInfo WriteMethod => _methods.WriteMethod;

		public MethodInfo WriteNamedMethod => _methods.WriteNamedMethod;

		static TDelegate CreateDelegate<TDelegate>(DynamicMethod method) where TDelegate : Delegate => (TDelegate)method.CreateDelegate(typeof(TDelegate));

		public override T Read(CbField field) => _readFunc(field);

		public override void Write(CbWriter writer, T value) => _writeFunc(writer, value);

		public override void WriteNamed(CbWriter writer, CbFieldName name, T value) => _writeNamedFunc(writer, name, value);
	}

	class CbClassConverterFactory : CbConverterFactory
	{
		public override CbConverter? CreateConverter(Type type)
		{
			CbConverter? converter = null;
			if (type.IsClass)
			{
				Type converterType = typeof(CbClassConverter<>).MakeGenericType(type);
				try
				{
					converter = (CbConverter?)Activator.CreateInstance(converterType);
				}
				catch (TargetInvocationException ex) when (ex.InnerException is not null)
				{
					throw ex.InnerException;
				}
			}
			return converter;
		}
	}
}
