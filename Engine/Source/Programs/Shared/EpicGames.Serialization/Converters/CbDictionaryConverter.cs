// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace EpicGames.Serialization.Converters
{
	/// <summary>
	/// Converter for dictionary types
	/// </summary>
	class CbDictionaryConverter<TKey, TValue> : CbConverter<Dictionary<TKey, TValue>> where TKey : notnull
	{
		/// <inheritdoc/>
		public override Dictionary<TKey, TValue> Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}

			Dictionary<TKey, TValue> dictionary = [];
			foreach (CbField element in field)
			{
				IEnumerator<CbField> enumerator = element.AsArray().GetEnumerator();

				if (!enumerator.MoveNext())
				{
					throw new CbException("Missing key for dictionary entry");
				}
				TKey key = CbSerializer.Deserialize<TKey>(enumerator.Current);

				if (!enumerator.MoveNext())
				{
					throw new CbException("Missing value for dictionary entry");
				}
				TValue value = CbSerializer.Deserialize<TValue>(enumerator.Current);

				dictionary.Add(key, value);
			}
			return dictionary;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, Dictionary<TKey, TValue> value)
		{
			if (value == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.BeginUniformArray(CbFieldType.Array);
				foreach (KeyValuePair<TKey, TValue> pair in value)
				{
					writer.BeginArray();
					CbSerializer.Serialize(writer, pair.Key);
					CbSerializer.Serialize(writer, pair.Value);
					writer.EndArray();
				}
				writer.EndUniformArray();
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, Dictionary<TKey, TValue> value)
		{
			if (value == null)
			{
				writer.WriteNull(name);
			}
			else if (value.Count > 0)
			{
				writer.BeginUniformArray(name, CbFieldType.Array);
				foreach (KeyValuePair<TKey, TValue> pair in value)
				{
					writer.BeginArray();
					CbSerializer.Serialize(writer, pair.Key);
					CbSerializer.Serialize(writer, pair.Value);
					writer.EndArray();
				}
				writer.EndUniformArray();
			}
		}
	}

	/// <summary>
	/// Factory for CbDictionaryConverter
	/// </summary>
	class CbDictionaryConverterFactory : CbConverterFactory
	{
		/// <inheritdoc/>
		public override CbConverter? CreateConverter(Type type)
		{
			if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(Dictionary<,>))
			{
				Type? keyType = type.GetGenericArguments().FirstOrDefault();
				if (keyType == typeof(string))
				{
					// if the key type is a string serialize as a CbObject to more closely follow typical conventions
					Type converterType = typeof(CbStringDictionaryConverter<>).MakeGenericType(type.GenericTypeArguments[1..]);
					return (CbConverter)Activator.CreateInstance(converterType)!;
				}
				else
				{
					Type converterType = typeof(CbDictionaryConverter<,>).MakeGenericType(type.GenericTypeArguments);
					return (CbConverter)Activator.CreateInstance(converterType)!;
				}
			}
			return null;
		}
	}

	/// <summary>
	/// Converter for dictionary types with string keys, serialized as objects
	/// </summary>
	class CbStringDictionaryConverter<TValue> : CbConverter<Dictionary<string, TValue>>
	{
		/// <inheritdoc/>
		public override Dictionary<string, TValue> Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}

			Dictionary<string, TValue> dictionary = [];
			foreach (CbField element in field)
			{
				string key = element.Name.ToString();

				TValue value = CbSerializer.Deserialize<TValue>(element);

				dictionary.Add(key, value);
			}
			return dictionary;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, Dictionary<string, TValue> value)
		{
			if (value == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.BeginObject();
				foreach (KeyValuePair<string, TValue> pair in value)
				{
					CbSerializer.Serialize(writer, pair.Key, pair.Value);
				}
				writer.EndObject();
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, Dictionary<string, TValue> value)
		{
			if (value == null)
			{
				writer.WriteNull(name);
			}
			else if (value.Count > 0)
			{
				writer.BeginObject(name);
				foreach (KeyValuePair<string, TValue> pair in value)
				{
					CbSerializer.Serialize(writer, pair.Key, pair.Value);
				}
				writer.EndObject();
			}
		}
	}
}
