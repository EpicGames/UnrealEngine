// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq.Expressions;
using System.Reflection;
using System.Reflection.Emit;

namespace EpicGames.Serialization.Converters
{
	class CbEnumConverter<T> : CbConverter<T>, ICbConverterMethods where T : Enum
	{
		readonly Func<CbField, T> _readFunc;
		readonly Action<CbWriter, T> _writeFunc;
		readonly Action<CbWriter, CbFieldName, T> _writeNamedFunc;

		public CbEnumConverter()
		{
			Type type = typeof(T);

			ReadMethod = new DynamicMethod($"Read_{type.Name}", type, [typeof(CbField)]);
			CreateEnumReader(ReadMethod.GetILGenerator());
			_readFunc = (Func<CbField, T>)ReadMethod.CreateDelegate(typeof(Func<CbField, T>));

			WriteMethod = new DynamicMethod($"Write_{type.Name}", null, [typeof(CbWriter), type]);
			CreateEnumWriter(WriteMethod.GetILGenerator());
			_writeFunc = (Action<CbWriter, T>)WriteMethod.CreateDelegate(typeof(Action<CbWriter, T>));

			WriteNamedMethod = new DynamicMethod($"WriteNamed_{type.Name}", null, [typeof(CbWriter), typeof(CbFieldName), type]);
			CreateNamedEnumWriter(WriteNamedMethod.GetILGenerator());
			_writeNamedFunc = (Action<CbWriter, CbFieldName, T>)WriteNamedMethod.CreateDelegate(typeof(Action<CbWriter, CbFieldName, T>));
		}

		public DynamicMethod ReadMethod { get; }
		MethodInfo ICbConverterMethods.ReadMethod => ReadMethod;

		public DynamicMethod WriteMethod { get; }
		MethodInfo ICbConverterMethods.WriteMethod => WriteMethod;

		public DynamicMethod WriteNamedMethod { get; }
		MethodInfo ICbConverterMethods.WriteNamedMethod => WriteNamedMethod;

		public override T Read(CbField field) => _readFunc(field);

		public override void Write(CbWriter writer, T value) => _writeFunc(writer, value);

		public override void WriteNamed(CbWriter writer, CbFieldName name, T value) => _writeNamedFunc(writer, name, value);

		static void CreateEnumReader(ILGenerator generator)
		{
			generator.Emit(OpCodes.Ldarg_0);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbField>(x => x.AsInt32()), null);
			generator.Emit(OpCodes.Ret);
		}

		static void CreateEnumWriter(ILGenerator generator)
		{
			generator.Emit(OpCodes.Ldarg_1);

			Label skipLabel = generator.DefineLabel();
			generator.Emit(OpCodes.Brfalse, skipLabel);

			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_1);
			generator.Emit(OpCodes.Conv_I8);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.WriteIntegerValue(0L)), null);

			generator.MarkLabel(skipLabel);
			generator.Emit(OpCodes.Ret);
		}

		static void CreateNamedEnumWriter(ILGenerator generator)
		{
			generator.Emit(OpCodes.Ldarg_2);

			Label skipLabel = generator.DefineLabel();
			generator.Emit(OpCodes.Brfalse, skipLabel);

			generator.Emit(OpCodes.Ldarg_0);
			generator.Emit(OpCodes.Ldarg_1);
			generator.Emit(OpCodes.Ldarg_2);
			generator.Emit(OpCodes.Conv_I8);
			generator.EmitCall(OpCodes.Call, GetMethodInfo<CbWriter>(x => x.WriteInteger(default, 0L)), null);

			generator.MarkLabel(skipLabel);
			generator.Emit(OpCodes.Ret);
		}

		static MethodInfo GetMethodInfo<TArg>(Expression<Action<TArg>> expr)
		{
			return ((MethodCallExpression)expr.Body).Method;
		}
	}

	class CbEnumConverterFactory : CbConverterFactory
	{
		public override CbConverter? CreateConverter(Type type)
		{
			CbConverter? converter = null;
			if (type.IsEnum)
			{
				Type converterType = typeof(CbEnumConverter<>).MakeGenericType(type);
				converter = (CbConverter?)Activator.CreateInstance(converterType);
			}
			return converter;
		}
	}
}
