// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Serialization.Converters
{
	/// <summary>
	/// Converter for raw CbField types
	/// </summary>
	class CbFieldConverter : CbConverter<CbField>
	{
		/// <inheritdoc/>
		public override CbField Read(CbField field)
		{
			return field;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, CbField field)
		{
			writer.WriteFieldValue(field);
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, CbField value)
		{
			writer.WriteField(name, value);
		}
	}

	/// <summary>
	/// Converter for raw CbObject types
	/// </summary>
	class CbObjectConverter : CbConverter<CbObject>
	{
		/// <inheritdoc/>
		public override CbObject Read(CbField field)
		{
			return field.AsObject();
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, CbObject obj)
		{
			writer.WriteFieldValue(obj.AsField());
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, CbObject obj)
		{
			writer.WriteField(name, obj.AsField());
		}
	}
}
