// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace EpicGames.Serialization.Converters
{
	/// <summary>
	/// Converter for list types
	/// </summary>
	/// <typeparam name="T"></typeparam>
	class CbListConverter<T> : CbConverter<List<T>>
	{
		/// <inheritdoc/>
		public override List<T> Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}
			else
			{
				List<T> list = [];
				foreach (CbField elementField in field)
				{
					list.Add(CbSerializer.Deserialize<T>(elementField));
				}
				return list;
			}
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, List<T> list)
		{
			if (list == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.BeginArray();
				foreach (T element in list)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndArray();
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, List<T> list)
		{
			if (list == null)
			{
				writer.WriteNull(name);
			}
			else
			{
				writer.BeginArray(name);
				foreach (T element in list)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndArray();
			}
		}
	}

	/// <summary>
	/// Specialization that writes a list as a uniform array of objects - a more efficient representation if you know the elements are always of the same type
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class CbUniformObjectListConverter<T> : CbConverter<List<T>>
	{
		/// <inheritdoc/>
		public override List<T> Read(CbField field)
		{
			if (field.IsNull())
			{
				return null!;
			}
			else
			{
				List<T> list = [];
				foreach (CbField elementField in field)
				{
					list.Add(CbSerializer.Deserialize<T>(elementField));
				}
				return list;
			}
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, List<T> list)
		{
			if (list == null)
			{
				writer.WriteNullValue();
			}
			else
			{
				writer.BeginUniformArray(CbFieldType.Object);
				foreach (T element in list)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndUniformArray();
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, List<T> list)
		{
			if (list == null)
			{
				writer.WriteNull(name);
			}
			else
			{
				writer.BeginUniformArray(name, CbFieldType.Object);
				foreach (T element in list)
				{
					CbSerializer.Serialize<T>(writer, element);
				}
				writer.EndUniformArray();
			}
		}
	}

	/// <summary>
	/// Specialization for serializing string lists
	/// </summary>
	sealed class CbStringListConverter : CbConverter<List<Utf8String>>
	{
		/// <inheritdoc/>
		public override List<Utf8String> Read(CbField field)
		{
			List<Utf8String> list = [];
			foreach (CbField elementField in field)
			{
				list.Add(elementField.AsUtf8String());
			}
			return list;
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, List<Utf8String> list)
		{
			writer.BeginUniformArray(CbFieldType.String);
			foreach (Utf8String str in list)
			{
				writer.WriteUtf8StringValue(str);
			}
			writer.EndUniformArray();
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, List<Utf8String> list)
		{
			if (list.Count > 0)
			{
				writer.BeginUniformArray(name, CbFieldType.String);
				foreach (Utf8String str in list)
				{
					writer.WriteUtf8StringValue(str);
				}
				writer.EndUniformArray();
			}
		}
	}

	/// <summary>
	/// Factory for CbListConverter
	/// </summary>
	class CbListConverterFactory : CbConverterFactory
	{
		/// <inheritdoc/>
		public override CbConverter? CreateConverter(Type type)
		{
			if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(List<>))
			{
				if (type.GenericTypeArguments[0] == typeof(Utf8String))
				{
					return new CbStringListConverter();
				}
				else
				{
					Type converterType = typeof(CbListConverter<>).MakeGenericType(type.GenericTypeArguments);
					return (CbConverter)Activator.CreateInstance(converterType)!;
				}
			}
			return null;
		}
	}
}
