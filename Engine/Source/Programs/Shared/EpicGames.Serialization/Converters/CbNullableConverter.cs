// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.Serialization.Converters
{
	/// <summary>
	/// Converter for list types
	/// </summary>
	/// <typeparam name="T"></typeparam>
	class CbNullableConverter<T> : CbConverter<Nullable<T>> where T : struct
	{
		/// <inheritdoc/>
		public override T? Read(CbField field)
		{
			return CbSerializer.Deserialize<T>(field);
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, T? nullable)
		{
			if (nullable.HasValue)
			{
				CbSerializer.Serialize<T>(writer, nullable.Value);
			}
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, T? nullable)
		{
			if (nullable.HasValue)
			{
				CbSerializer.Serialize<T>(writer, name, nullable.Value);
			}
		}
	}

	/// <summary>
	/// Factory for CbNullableConverter
	/// </summary>
	class CbNullableConverterFactory : CbConverterFactory
	{
		/// <inheritdoc/>
		public override CbConverter? CreateConverter(Type type)
		{
			if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				Type converterType = typeof(CbNullableConverter<>).MakeGenericType(type.GenericTypeArguments);
				return (CbConverter)Activator.CreateInstance(converterType)!;
			}
			return null;
		}
	}
}
