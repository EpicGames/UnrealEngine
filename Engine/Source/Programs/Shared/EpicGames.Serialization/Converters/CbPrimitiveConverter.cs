// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq.Expressions;
using System.Reflection;

namespace EpicGames.Serialization.Converters
{
	class CbPrimitiveConverter<T>(Expression<Func<CbField, T>> read, Expression<Action<CbWriter, T>> write, Expression<Action<CbWriter, CbFieldName, T>> writeNamed) : CbConverter<T>, ICbConverterMethods
	{
		public MethodInfo ReadMethod { get; } = ((MethodCallExpression)read.Body).Method;
		public Func<CbField, T> ReadFunc { get; } = read.Compile();

		public MethodInfo WriteMethod { get; } = ((MethodCallExpression)write.Body).Method;
		public Action<CbWriter, T> WriteFunc { get; } = write.Compile();

		public MethodInfo WriteNamedMethod { get; } = ((MethodCallExpression)writeNamed.Body).Method;
		public Action<CbWriter, CbFieldName, T> WriteNamedFunc { get; } = writeNamed.Compile();

		public override T Read(CbField field) => ReadFunc(field);

		public override void Write(CbWriter writer, T value) => WriteFunc(writer, value);

		public override void WriteNamed(CbWriter writer, CbFieldName name, T value) => WriteNamedFunc(writer, name, value);
	}
}
