// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;

namespace EpicGames.Serialization.Converters
{
	class CbStringConverter<T>(TypeConverter typeConverter) : CbConverter<T>
	{
		public override T Read(CbField field)
		{
			return (T)typeConverter.ConvertFromInvariantString(field.AsString())!;
		}

		public override void WriteNamed(CbWriter writer, CbFieldName name, T value)
		{
			writer.WriteString(name, typeConverter.ConvertToInvariantString(value));
		}

		public override void Write(CbWriter writer, T value)
		{
			writer.WriteStringValue(typeConverter.ConvertToInvariantString(value)!);
		}
	}

	class CbStringConverterFactory : CbConverterFactory
	{
		public override CbConverter? CreateConverter(Type type)
		{
			TypeConverter? frameworkTypeConverter = TypeDescriptor.GetConverter(type);
			if (frameworkTypeConverter == null || !frameworkTypeConverter.CanConvertFrom(typeof(string)) || !frameworkTypeConverter.CanConvertTo(typeof(string)))
			{
				return null;
			}

			Type converterType = typeof(CbStringConverter<>).MakeGenericType(type);
			return (CbConverter)Activator.CreateInstance(converterType, frameworkTypeConverter)!;
		}
	}
}
