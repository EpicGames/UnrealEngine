<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <PackageReference Include="System.IO.Hashing" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\EpicGames.Oodle\EpicGames.Oodle.csproj" />
  </ItemGroup>
</Project>
