// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Net.Http;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.Serialization
{
	/// <summary>
	/// Extension methods for System.Net.Http
	/// </summary>
	public static class HttpExtensions
	{
		/// <summary>
		/// Read content using CbSerializer, make sure the type T is correctly annotated to deserialize Compact Binary
		/// </summary>
		/// <typeparam name="T">Type to deserialize</typeparam>
		/// <param name="content">The HTTP content you want to convert to type T read as byte content</param>
		/// <returns>Instance of type T</returns>
		public static async Task<T> ReadAsCompactBinaryAsync<T>(this HttpContent content)
		{
			await using Stream s = await content.ReadAsStreamAsync();
			byte[] b = await s.ReadAllBytesAsync();
			return CbSerializer.Deserialize<T>(b);
		}
	}
}
