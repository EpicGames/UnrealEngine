// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Slack
{
	/// <summary>
	/// Abstract base class for all BlockKit blocks to derive from.
	/// </summary>
	[JsonConverter(typeof(BlockConverter))]
	public abstract class Block
	{
		/// <summary>
		/// The block type
		/// </summary>
		[JsonPropertyName("type"), JsonPropertyOrder(0)]
		public string Type { get; }

		/// <summary>
		/// A string acting as a unique identifier for a block. If not specified, one will be generated. Maximum length for this field is 255 characters. 
		/// block_id should be unique for each message and each iteration of a message. If a message is updated, use a new block_id.
		/// </summary>
		[JsonPropertyName("block_id"), JsonPropertyOrder(30)]
		public string? BlockId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">Type of the block</param>
		protected Block(string type)
		{
			Type = type;
		}
	}

	/// <summary>
	/// Interface for a container of <see cref="Block"/> items
	/// </summary>
	public interface ISlackBlockContainer
	{
		/// <summary>
		/// List of blocks
		/// </summary>
		List<Block> Blocks { get; }
	}

	/// <summary>
	/// Polymorphic serializer for Block objects.
	/// </summary>
	class BlockConverter : JsonConverter<Block>
	{
		public override Block? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		public override void Write(Utf8JsonWriter writer, Block value, JsonSerializerOptions options)
		{
			JsonSerializer.Serialize(writer, (object)value, value.GetType(), options);
		}
	}
}
