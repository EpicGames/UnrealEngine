// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// A block that is used to hold interactive <see cref="Element"/> objects.
	/// </summary>
	public class ActionsBlock : Block, ISlackElementContainer
	{
		/// <summary>
		/// A collection of interactive elements.
		/// </summary>
		[JsonPropertyName("elements"), JsonPropertyOrder(1)]
		public List<Element> Elements { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public ActionsBlock() : base("actions")
		{
		}
	}

	/// <summary>
	/// Extension methods for <see cref="Block"/>
	/// </summary>
	public static class ActionsBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="ActionsBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">Block container</param>
		public static ActionsBlock AddActions(this ISlackBlockContainer container)
		{
			ActionsBlock block = new();
			container.Blocks.Add(block);
			return block;
		}

		/// <summary>
		/// Add an <see cref="ActionsBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">Block container</param>
		/// <param name="configure">Configuration function for the block</param>
		public static void AddActions(this ISlackBlockContainer container, Action<ActionsBlock> configure)
		{
			ActionsBlock block = new();
			configure(block);
			container.Blocks.Add(block);
		}
	}
}
