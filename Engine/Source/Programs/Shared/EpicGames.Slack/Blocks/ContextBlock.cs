// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// Displays message context, which can include both images and text.
	/// </summary>
	public class ContextBlock : Block
	{
		/// <summary>
		/// An array of image elements and text objects. Maximum number of items is 10.
		/// </summary>
		[JsonPropertyName("elements")]
		public List<Element> Elements { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public ContextBlock() : base("context")
		{
		}
	}

	/// <summary>
	/// Extension methods for <see cref="Block"/>
	/// </summary>
	public static class ContextBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="ContextBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">Block container</param>
		/// <param name="configure">Configuration callback for the block</param>
		public static void AddContext(this ISlackBlockContainer container, Action<ContextBlock> configure)
		{
			ContextBlock block = new();
			configure(block);
			container.Blocks.Add(block);
		}

		/// <summary>
		/// Add an <see cref="ContextBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">Block container</param>
		/// <param name="text">Configuration callback for the block</param>
		public static ContextBlock AddContext(this ISlackBlockContainer container, TextObject text)
		{
			ContextBlock block = new();
			block.Elements.Add(text);
			container.Blocks.Add(block);
			return block;
		}
	}
}
