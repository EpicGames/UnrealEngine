// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// A content divider, like an &lt;hr&gt;, to split up different blocks inside of a message. The divider block is nice and neat, requiring only a type.
	/// </summary>
	public class DividerBlock : Block
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public DividerBlock() : base("divider")
		{
		}
	}

	/// <summary>
	/// Extension methods for <see cref="Block"/>
	/// </summary>
	public static class DividerBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="DividerBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">Block container</param>
		public static void AddDivider(this ISlackBlockContainer container)
		{
			DividerBlock block = new();
			container.Blocks.Add(block);
		}
	}
}
