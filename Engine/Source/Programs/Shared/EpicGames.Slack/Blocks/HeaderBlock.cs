// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// A header is a plain-text block that displays in a larger, bold font. Use it to delineate between different groups of content in your app's surfaces.
	/// </summary>
	public class HeaderBlock : Block
	{
		/// <summary>
		/// The text for the block, in the form of a plain_text text object. Maximum length for the text in this field is 150 characters.
		/// </summary>
		[JsonPropertyName("text"), JsonPropertyOrder(1)]
		public PlainTextObject Text { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Any text to initiale the Section with</param>
		public HeaderBlock(PlainTextObject text) : base("header")
		{
			Text = text;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Text to display in the header</param>
		public HeaderBlock(string text) : this(new PlainTextObject(text))
		{
		}
	}

	/// <summary>
	/// Extension methods for <see cref="Block"/>
	/// </summary>
	public static class HeaderBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="HeaderBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">List of blocks</param>
		/// <param name="text">Text for the header</param>
		public static void AddHeader(this ISlackBlockContainer container, PlainTextObject text)
		{
			HeaderBlock block = new(text);
			container.Blocks.Add(block);
		}

		/// <summary>
		/// Add an <see cref="HeaderBlock"/> to the list of blocks
		/// </summary>
		/// <param name="container">List of blocks</param>
		/// <param name="text">Text for the header</param>
		/// <param name="emoji">Whether to enable emoji</param>
		public static void AddHeader(this ISlackBlockContainer container, string text, bool? emoji) => AddHeader(container, new PlainTextObject(text, emoji));
	}
}
