// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// A simple image block, designed to make those cat photos really pop.
	/// </summary>
	public class ImageBlock : Block
	{
		/// <summary>
		/// The URL of the image to be displayed.Maximum length for this field is 3000 characters.
		/// </summary>
		[JsonPropertyName("image_url"), JsonPropertyOrder(1)]
		public Uri ImageUrl { get; set; }

		/// <summary>
		/// A plain-text summary of the image.This should not contain any markup.Maximum length for this field is 2000 characters.
		/// </summary>
		[JsonPropertyName("alt_text"), JsonPropertyOrder(2)]
		public string AltText { get; set; }

		/// <summary>
		/// An optional title for the image in the form of a text object that can only be of type: plain_text. Maximum length for the text in this field is 2000 characters.
		/// </summary>
		[JsonPropertyName("title"), JsonPropertyOrder(3)]
		public PlainTextObject? Title { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ImageBlock(Uri imageUrl, string altText, PlainTextObject? title = null) : base("image")
		{
			ImageUrl = imageUrl;
			AltText = altText;
			Title = title;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="ImageBlock"/>
	/// </summary>
	public static class ImageBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="ImageBlock"/> to the list of blocks
		/// </summary>
		public static void AddImage(this ISlackBlockContainer container, Uri imageUrl, string altText, PlainTextObject? title = null)
		{
			ImageBlock block = new(imageUrl, altText, title);
			container.Blocks.Add(block);
		}
	}
}
