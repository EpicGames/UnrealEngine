// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// A block that collects information from users - it can hold a plain-text input element, a checkbox element, a radio button element, a select menu element, a multi-select menu element, or a datepicker.
	/// </summary>
	public class InputBlock : Block
	{
		/// <summary>
		/// A label that appears above an input element in the form of a text object that must have type of plain_text. Maximum length for the text in this field is 2000 characters.
		/// </summary>
		[JsonPropertyName("label"), JsonPropertyOrder(1)]
		public PlainTextObject Label { get; set; }

		/// <summary>
		/// A plain-text input element, a checkbox element, a radio button element, a select menu element, a multi-select menu element, or a datepicker.
		/// </summary>
		[JsonPropertyName("element"), JsonPropertyOrder(2)]
		public Element Element { get; set; }

		/// <summary>
		/// A boolean that indicates whether or not the use of elements in this block should dispatch a block_actions payload. Defaults to false.
		/// </summary>
		[JsonPropertyName("dispatch_action"), JsonPropertyOrder(3)]
		public bool? DispatchAction { get; set; }

		/// <summary>
		/// An optional hint that appears below an input element in a lighter grey. It must be a text object with a type of plain_text. Maximum length for the text in this field is 2000 characters.
		/// </summary>
		[JsonPropertyName("hint"), JsonPropertyOrder(4)]
		public PlainTextObject? Hint { get; set; }

		/// <summary>
		/// A boolean that indicates whether the input element may be empty when a user submits the modal. Defaults to false.
		/// </summary>
		[JsonPropertyName("optional"), JsonPropertyOrder(5)]
		public bool? Optional { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public InputBlock(PlainTextObject label, Element element) : base("input")
		{
			Label = label;
			Element = element;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="InputBlock"/>
	/// </summary>
	public static class InputBlockExtensions
	{
		/// <summary>
		/// Add an <see cref="ImageBlock"/> to the list of blocks
		/// </summary>
		public static InputBlock AddInput(this ISlackBlockContainer container, PlainTextObject label, Element element)
		{
			InputBlock block = new(label, element);
			container.Blocks.Add(block);
			return block;
		}
	}
}
