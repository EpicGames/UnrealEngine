// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// Represents a BlockKit Markdown block
	/// </summary>
	public class MarkdownBlock : Block
	{
		/// <summary>
		///The standard markdown-formatted text. The cumulative limit for all markdown blocks in a single payload is 12,000 characters.
		/// </summary>
		[JsonPropertyName("text"), JsonPropertyOrder(1)]
		public string Text { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Any text to initiale the Markdown with</param>
		public MarkdownBlock(string text) : base("markdown")
		{
			Text = text;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="MarkdownBlock"/>
	/// </summary>
	public static class MarkdownBlockExtensions
	{
		/// <summary>
		/// Adds a new <see cref="MarkdownBlock"/> with the given text element to the container.
		/// <param name="container">The block container to add the markdown to</param>
		/// <param name="text">The text content to populate the markdown with</param>
		/// </summary>
		public static void AddMarkdown(this ISlackBlockContainer container, string text)
		{
			MarkdownBlock block = new(text);
			container.Blocks.Add(block);
		}
	}
}
