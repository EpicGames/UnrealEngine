// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack.Blocks
{
	/// <summary>
	/// Represents a BlockKit Section block
	/// </summary>
	public class SectionBlock : Block
	{
		/// <summary>
		/// The text for the block, in the form of a text object. Maximum length for the text in this field is 3000 characters.
		/// </summary>
		[JsonPropertyName("text"), JsonPropertyOrder(1)]
		public TextObject? Text { get; set; }

		/// <summary>
		/// Required if no text is provided. An array of text objects. Any text objects included with fields will be rendered in a compact format that allows for 2 columns of side-by-side text. 
		/// Maximum number of items is 10. Maximum length for the text in each item is 2000 characters.
		/// </summary>
		[JsonPropertyName("fields"), JsonPropertyOrder(2)]
		public IReadOnlyList<TextObject>? Fields { get; set; }

		/// <summary>
		/// One of the available element objects.
		/// </summary>
		[JsonPropertyName("accessory"), JsonPropertyOrder(3)]
		public Element? Accessory { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Any text to initiale the Section with</param>
		/// <param name="accessory">Optional accessory element</param>
		public SectionBlock(TextObject text, Element? accessory = null) : base("section")
		{
			Text = text;
			Accessory = accessory;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fields">Any text to initiale the Section with</param>
		/// <param name="accessory">Optional accessory element</param>
		public SectionBlock(List<TextObject> fields, Element? accessory = null) : base("section")
		{
			Fields = fields;
			Accessory = accessory;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="SectionBlock"/>
	/// </summary>
	public static class SectionBlockExtensions
	{
		/// <summary>
		/// Adds a new <see cref="SectionBlock"/> with a single text element to the container.
		/// <param name="container">The block container to add the section to</param>
		/// <param name="text">The text content to populate the section with</param>
		/// </summary>
		public static void AddSection(this ISlackBlockContainer container, TextObject text)
		{
			SectionBlock block = new(text);
			container.Blocks.Add(block);
		}

		/// <summary>
		/// Adds a new <see cref="SectionBlock"/> with multiple field elements to the container
		/// <param name="container">The block container to add the section to</param>
		/// <param name="fields">The list of text fields to populate the section with</param>
		/// </summary>
		public static void AddSection(this ISlackBlockContainer container, List<TextObject> fields)
		{
			SectionBlock block = new(fields);
			container.Blocks.Add(block);
		}
	}
}
