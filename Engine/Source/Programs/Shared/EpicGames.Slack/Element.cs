// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Slack
{
	/// <summary>
	/// Abstract base class for all BlockKit elements to derive from.
	/// </summary>
	[JsonConverter(typeof(ElementConverter))]
	public abstract class Element
	{
		/// <summary>
		/// Type of the element
		/// </summary>
		[JsonPropertyName("type"), JsonPropertyOrder(0)]
		public string Type { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type"></param>
		protected Element(string type)
		{
			Type = type;
		}
	}

	/// <summary>
	/// Something that contains a list of elements. Used for extension methods.
	/// </summary>
	public interface ISlackElementContainer
	{
		/// <summary>
		/// List of elements in this container
		/// </summary>
		public List<Element> Elements { get; }
	}

	/// <summary>
	/// Polymorphic serializer for Block objects.
	/// </summary>
	class ElementConverter : JsonConverter<Element>
	{
		public override Element? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		public override void Write(Utf8JsonWriter writer, Element value, JsonSerializerOptions options)
		{
			JsonSerializer.Serialize(writer, (object)value, value.GetType(), options);
		}
	}
}
