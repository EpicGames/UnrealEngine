// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.Serialization;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// Determines the visual color scheme to use for the button.
	/// </summary>
	[JsonConverter(typeof(ButtonStyleJsonConverter))]
	public enum ButtonStyle
	{
		/// <summary>
		/// Used for affirmation/confirmation actions.
		/// </summary>
		Primary,

		/// <summary>
		/// Used when an action is destructive and cannot be undone.
		/// </summary>
		Danger,
	}

	/// <summary>
	/// Json serializer for <see cref="ButtonStyle"/>
	/// </summary>
	class ButtonStyleJsonConverter : JsonConverter<ButtonStyle>
	{
		public override ButtonStyle Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		public override void Write(Utf8JsonWriter writer, ButtonStyle value, JsonSerializerOptions options)
		{
			switch (value)
			{
				case ButtonStyle.Primary:
					writer.WriteStringValue("primary");
					break;
				case ButtonStyle.Danger:
					writer.WriteStringValue("danger");
					break;
				default:
					throw new InvalidDataContractException();
			}
		}
	}

	/// <summary>
	/// An button element that can be added to an <see cref="Blocks.ActionsBlock"/>.
	/// </summary>
	public class ButtonElement : Element
	{
		/// <summary>
		/// A text object that defines the button's text. Can only be of type: plain_text. text may truncate with ~30 characters. Maximum length for the text in this field is 75 characters.
		/// </summary>
		[JsonPropertyName("text"), JsonPropertyOrder(1)]
		public PlainTextObject Text { get; }

		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id"), JsonPropertyOrder(2)]
		public string? ActionId { get; set; }

		/// <summary>
		/// A URL to load in the user's browser when the button is clicked. Maximum length for this field is 3000 characters. If you're using url, you'll still receive an interaction payload and will need to send an acknowledgement response.
		/// </summary>
		[JsonPropertyName("url"), JsonPropertyOrder(3)]
		public Uri? Url { get; set; }

		/// <summary>
		/// The value to send along with the interaction payload. Maximum length for this field is 2000 characters.
		/// </summary>
		[JsonPropertyName("value"), JsonPropertyOrder(4)]
		public string? Value { get; set; }

		/// <summary>
		/// Decorates buttons with alternative visual color schemes. Use this option with restraint.
		/// </summary>
		[JsonPropertyName("style"), JsonPropertyOrder(5)]
		public ButtonStyle? Style { get; set; }

		/// <summary>
		/// A label for longer descriptive text about a button element. This label will be read out by screen readers instead of the button text object. Maximum length for this field is 75 characters.
		/// </summary>
		[JsonPropertyName("accessibility_label"), JsonPropertyOrder(6)]
		public string? AccessibilityLabel { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public ButtonElement(PlainTextObject text, Uri? url = null, string? actionId = null, string? value = null, ButtonStyle? style = null)
			: base("button")
		{
			Text = text;
			Url = url;
			Value = value;
			ActionId = actionId;
			Style = style;
		}
	}

	/// <summary>
	/// Extension methods for buttons
	/// </summary>
	public static class ButtonExtensions
	{
		/// <summary>
		/// Add a new button to the list of elements
		/// </summary>
		public static ISlackElementContainer AddButton(this ISlackElementContainer list, string text, Uri? url = null, string? actionId = null, string? value = null, ButtonStyle? style = null)
		{
			ButtonElement button = new(text, url, actionId, value, style);
			list.Elements.Add(button);
			return list;
		}
	}
}
