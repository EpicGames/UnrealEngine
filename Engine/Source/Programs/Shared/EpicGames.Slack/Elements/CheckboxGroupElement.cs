// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// A checkbox group that allows a user to choose multiple items from a list of possible options.
	/// </summary>
	public class CheckboxGroupElement : Element
	{
		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id")]
		public string ActionId { get; set; }

		/// <summary>
		/// An array of option objects. A maximum of 10 options are allowed.
		/// </summary>
		[JsonPropertyName("options")]
		public List<SlackOption> Options { get; } = [];

		/// <summary>
		/// An array of option objects that exactly matches one or more of the options within options. These options will be selected when the checkbox group initially loads.
		/// </summary>
		[JsonPropertyName("initial_options")]
		public List<SlackOption> InitialOptions { get; } = [];

		/// <summary>
		/// A confirm object that defines an optional confirmation dialog that appears after clicking one of the checkboxes in this element.
		/// </summary>
		[JsonPropertyName("confirm")]
		public SlackConfirm? Confirm { get; set; }

		/// <summary>
		/// Indicates whether the element will be set to auto focus within the view object. Only one element can be set to true. Defaults to false.
		/// </summary>
		[JsonPropertyName("focus_on_load")]
		public bool FocusOnLoad { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public CheckboxGroupElement(string actionId, List<SlackOption> options)
			: base("checkboxes")
		{
			ActionId = actionId;
			Options.AddRange(options);
		}
	}
}
