// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// This is like a cross between a button and a select menu - when a user clicks on this overflow button, they will be presented with a list of options to choose from. 
	/// Unlike the select menu, there is no typeahead field, and the button always appears with an ellipsis ("…") rather than customisable text.
	///
	/// As such, it is usually used if you want a more compact layout than a select menu, or to supply a list of less visually important actions after a row of buttons.
	/// You can also specify simple URL links as overflow menu options, instead of actions.
	/// </summary>
	public class OverflowMenuElement : Element
	{
		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id")]
		public string ActionId { get; set; }

		/// <summary>
		/// An array of up to five option objects to display in the menu.
		/// </summary>
		[JsonPropertyName("options")]
		public List<SlackOption> Options { get; } = [];

		/// <summary>
		/// A confirm object that defines an optional confirmation dialog that appears after clicking one of the checkboxes in this element.
		/// </summary>
		[JsonPropertyName("confirm")]
		public SlackConfirm? Confirm { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public OverflowMenuElement(string actionId, List<SlackOption> options, SlackConfirm? confirm = null)
			: base("overflow")
		{
			ActionId = actionId;
			Options.AddRange(options);
			Confirm = confirm;
		}
	}
}
