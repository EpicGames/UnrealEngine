// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// A plain-text input, similar to the HTML &lt;input&gt; tag, creates a field where a user can enter freeform data. It can appear as a single-line field or a larger text area using the multiline flag.
	/// </summary>
	public class PlainTextInputElement : Element
	{
		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id")]
		public string ActionId { get; set; }

		/// <summary>
		/// A plain_text only text object that defines the placeholder text shown in the plain-text input. Maximum length for the text in this field is 150 characters.
		/// </summary>
		[JsonPropertyName("placeholder")]
		public PlainTextObject? Placeholder { get; set; }

		/// <summary>
		/// The initial value in the plain-text input when it is loaded.
		/// </summary>
		[JsonPropertyName("initial_value")]
		public string? InitialValue { get; set; }

		/// <summary>
		/// Indicates whether the input will be a single line (false) or a larger textarea (true). Defaults to false.
		/// </summary>
		[JsonPropertyName("multiline")]
		public bool MultiLine { get; set; }

		/// <summary>
		/// The minimum length of input that the user must provide. If the user provides less, they will receive an error. Maximum value is 3000.
		/// </summary>
		[JsonPropertyName("min_length")]
		public int? MinLength { get; set; }

		/// <summary>
		/// The maximum length of input that the user can provide. If the user provides more, they will receive an error.
		/// </summary>
		[JsonPropertyName("max_length")]
		public int? MaxLength { get; set; }

		/// <summary>
		/// Indicates whether the element will be set to auto focus within the view object. Only one element can be set to true. Defaults to false.
		/// </summary>
		[JsonPropertyName("focus_on_load")]
		public bool FocusOnLoad { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public PlainTextInputElement(string actionId, PlainTextObject? placeholder = null, string? initialValue = null, bool multiLine = false)
			: base("plain_text_input")
		{
			ActionId = actionId;
			Placeholder = placeholder;
			InitialValue = initialValue;
			MultiLine = multiLine;
		}
	}
}
