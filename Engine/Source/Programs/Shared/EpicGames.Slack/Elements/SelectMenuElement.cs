// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// An element which lets users easily select a date from a calendar style UI.
	/// </summary>
	public abstract class SelectMenuElement : Element
	{
		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id")]
		public string ActionId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		protected SelectMenuElement(string type, string actionId) : base(type)
		{
			ActionId = actionId;
		}
	}

	/// <summary>
	/// This is the simplest form of select menu, with a static list of options passed in when defining the element.
	/// </summary>
	public class StaticSelectMenuElement : SelectMenuElement
	{
		/// <summary>
		/// A plain_text only text object that defines the placeholder text shown on the menu. Maximum length for the text in this field is 150 characters.
		/// </summary>
		[JsonPropertyName("placeholder")]
		public PlainTextObject Placeholder { get; set; }

		/// <summary>
		/// An array of option objects. A maximum of 10 options are allowed.
		/// </summary>
		[JsonPropertyName("options")]
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only")]
		public List<SlackOption>? Options { get; set; }

		/// <summary>
		/// An array of option objects. A maximum of 10 options are allowed.
		/// </summary>
		[JsonPropertyName("option_groups")]
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only")]
		public List<SlackOptionGroup>? OptionGroups { get; set; }

		/// <summary>
		/// An array of option objects that exactly matches one or more of the options within options. These options will be selected when the checkbox group initially loads.
		/// </summary>
		[JsonPropertyName("initial_option")]
		public SlackOption? InitialOption { get; }

		/// <summary>
		/// A confirm object that defines an optional confirmation dialog that appears after clicking one of the checkboxes in this element.
		/// </summary>
		[JsonPropertyName("confirm")]
		public SlackConfirm? Confirm { get; set; }

		/// <summary>
		/// Indicates whether the element will be set to auto focus within the view object. Only one element can be set to true. Defaults to false.
		/// </summary>
		[JsonPropertyName("focus_on_load")]
		public bool FocusOnLoad { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public StaticSelectMenuElement(string actionId, PlainTextObject placeholder, List<SlackOption> options)
			: base("static_select", actionId)
		{
			Placeholder = placeholder;
			Options = [.. options];
		}

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public StaticSelectMenuElement(string actionId, PlainTextObject placeholder, List<SlackOptionGroup> optionGroups)
			: base("static_select", actionId)
		{
			Placeholder = placeholder;
			OptionGroups = [.. optionGroups];
		}
	}
}
