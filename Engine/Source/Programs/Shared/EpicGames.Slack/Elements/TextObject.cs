// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// A text object
	/// </summary>
	public class TextObject : Element
	{
		/// <summary>
		/// Text to render
		/// </summary>
		[JsonPropertyName("text"), JsonPropertyOrder(1)]
		public string Text { get; set; }

		/// <summary>
		/// Whether to show emojis in the text
		/// </summary>
		[JsonPropertyName("emoji"), JsonPropertyOrder(2)]
		public bool? Emoji { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text"></param>
		public TextObject(string text) : this("mrkdwn", text, null)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		protected TextObject(string type, string text, bool? emoji) : base(type)
		{
			Text = text;
			Emoji = emoji;
		}

		/// <summary>
		/// Implicit conversion from a regular string
		/// </summary>
		public static implicit operator TextObject(string text) => new(text);
	}

	/// <summary>
	/// Plain text object
	/// </summary>
	public class PlainTextObject : TextObject
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public PlainTextObject(string text, bool? emoji = null) : base("plain_text", text, emoji)
		{
		}

		/// <summary>
		/// Implicit conversion from a regular string
		/// </summary>
		public static implicit operator PlainTextObject(string text) => new(text);
	}
}
