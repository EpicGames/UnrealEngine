// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;

namespace EpicGames.Slack.Elements
{
	/// <summary>
	/// An element which lets users easily select a date from a calendar style UI.
	/// </summary>
	public class TimePickerElement : Element
	{
		/// <summary>
		/// An identifier for this action. You can use this when you receive an interaction payload to identify the source of the action. 
		/// Should be unique among all other action_ids in the containing block. Maximum length for this field is 255 characters.
		/// </summary>
		[JsonPropertyName("action_id")]
		public string ActionId { get; set; }

		/// <summary>
		/// A plain_text only text object that defines the placeholder text shown on the datepicker. Maximum length for the text in this field is 150 characters.
		/// </summary>
		[JsonPropertyName("placeholder")]
		public PlainTextObject? Placeholder { get; set; }

		/// <summary>
		/// The initial time that is selected when the element is loaded. This should be in the format HH:mm, where HH is the 24-hour format of an hour (00 to 23) and mm is minutes with leading zeros (00 to 59), for example 22:25 for 10:25pm.
		/// </summary>
		[JsonPropertyName("initial_time")]
		public string? InitialTime { get; set; }

		/// <summary>
		/// A confirm object that defines an optional confirmation dialog that appears after clicking one of the checkboxes in this element.
		/// </summary>
		[JsonPropertyName("confirm")]
		public SlackConfirm? Confirm { get; set; }

		/// <summary>
		/// Indicates whether the element will be set to auto focus within the view object. Only one element can be set to true. Defaults to false.
		/// </summary>
		[JsonPropertyName("focus_on_load")]
		public bool FocusOnLoad { get; set; }

		/// <summary>
		/// Construct a new Button action element.
		/// </summary>
		public TimePickerElement(string actionId)
			: base("timepicker")
		{
			ActionId = actionId;
		}
	}
}
