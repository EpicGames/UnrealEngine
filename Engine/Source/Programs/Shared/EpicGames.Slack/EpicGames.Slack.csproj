<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <DebugType>pdbonly</DebugType>
    <RunAnalyzersDuringBuild>true</RunAnalyzersDuringBuild>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Http" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http.Polly" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging.Abstractions" Version="10.0.6" />
  </ItemGroup>
</Project>
