// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json.Serialization;

namespace EpicGames.Slack
{
	/// <summary>
	/// Abstract base class for all BlockKit blocks to derive from.
	/// </summary>
	public class SlackAttachment : ISlackBlockContainer
	{
		/// <summary>
		/// The color of the line down the left side of the attachment.
		/// </summary>
		[JsonPropertyName("color"), JsonPropertyOrder(1)]
		public string Color { get; set; } = "#ff0000";

		/// <summary>
		/// Used for the "toast" notifications sent by Slack. If not set, and the Text property of the
		/// base message isn't set, the notification will have a message saying it has no preview.
		/// </summary>
		[JsonPropertyName("fallback"), JsonPropertyOrder(2)]
		public string? FallbackText { get; set; }

		/// <summary>
		/// A collection of BlockKit blocks the attachment is composed from.
		/// </summary>
		[JsonPropertyName("blocks"), JsonPropertyOrder(3)]
		public List<Block> Blocks { get; } = [];

		/// <summary>
		/// Implicit conversion to a message of its own
		/// </summary>
		/// <param name="attachment"></param>
		public static implicit operator SlackMessage(SlackAttachment attachment)
		{
			SlackMessage message = new();
			message.Attachments.Add(attachment);
			return message;
		}
	}
}
