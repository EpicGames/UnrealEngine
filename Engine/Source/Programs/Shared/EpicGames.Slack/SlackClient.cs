// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Net;
using System.Net.Http;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Http;
using Microsoft.Extensions.Logging;
using Polly;
using Polly.Extensions.Http;

namespace EpicGames.Slack
{
	/// <summary>
	/// Exception thrown due to an error response from Slack
	/// </summary>
	public class SlackException : Exception
	{
		/// <summary>
		/// Slack error code
		/// </summary>
		public string Code { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackException(string code) : base(code)
		{
			Code = code;
		}
	}

	/// <summary>
	/// Specifies how to handle a Slack API error
	/// </summary>
	enum SlackErrorBehavior
	{
		/// <summary>Throw a SlackException (caller handles logging)</summary>
		Throw,

		/// <summary>Log at error level, don't throw</summary>
		LogError,

		/// <summary>Log at warning level, don't throw</summary>
		LogWarning,

		/// <summary>Don't log, don't throw</summary>
		Ignore
	}

	/// <summary>
	/// Response from posting a Slack message
	/// </summary>
	public class SlackMessageId
	{
		/// <summary>
		/// The channel id. Note that posting messages is more lenient than updating messages, and will accept a user ID as a recipient channel. The actual DM channel this resolves to will be returned in this parameter.
		/// </summary>
		public string Channel { get; set; }

		/// <summary>
		/// Thread containing the message
		/// </summary>
		public string? ThreadTs { get; set; }

		/// <summary>
		/// Timestamp of the message
		/// </summary>
		public string Ts { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackMessageId(string channel, string? threadTs, string ts)
		{
			Channel = channel;
			ThreadTs = threadTs;
			Ts = ts;
		}

		/// <inheritdoc/>
		public override string ToString() => (ThreadTs == null) ? $"{Channel}:{Ts}" : $"{Channel}:{ThreadTs}:{Ts}";
	}

	/// <summary>
	/// Wrapper around Slack client functionality
	/// </summary>
	public sealed class SlackClient : IDisposable
	{
		class SlackResponse
		{
			[JsonPropertyName("ok")]
			public bool Ok { get; set; }

			[JsonPropertyName("error")]
			public string? Error { get; set; }
		}

		readonly HttpClient _httpClient;
		readonly JsonSerializerOptions _serializerOptions;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="httpClient">Http client for connecting to slack. Should have the necessary authorization headers.</param>
		/// <param name="logger">Logger interface</param>
		public SlackClient(HttpClient httpClient, ILogger logger)
		{
			_httpClient = httpClient;
			_serializerOptions = new JsonSerializerOptions { DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull };
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_httpClient.Dispose();
		}

		/// <summary>
		/// Create a Slack client with a retry policy
		/// </summary>
		/// <param name="slackToken">Slack token to use</param>
		/// <param name="logger">Logger</param>
		[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope", Justification = "Polly handler is disposed by HttpClient")]
		public static SlackClient Create(string? slackToken, ILogger logger)
		{
			// Slack has tight rate limiting so respecting their returned retried intervals is useful to avoid TooManyRequests errors
			Polly.Retry.AsyncRetryPolicy<HttpResponseMessage> retryPolicy = HttpPolicyExtensions
				.HandleTransientHttpError()
				.OrResult(r => r.StatusCode == HttpStatusCode.TooManyRequests)
				.WaitAndRetryAsync(
					retryCount: 5,
					sleepDurationProvider: (retryAttempt, outcome, context) =>
					{
						if (outcome.Result?.Headers.RetryAfter?.Delta.HasValue == true)
						{
							return outcome.Result.Headers.RetryAfter.Delta.Value;
						}
						return TimeSpan.FromSeconds(15);
					},
					onRetryAsync: (outcome, timespan, retryCount, context) =>
					{
						logger.LogInformation("Slack HTTP request to {Uri} was rate limited. Retry {RetryCount}/5. Waiting {WaitTime}s...",
							outcome.Result.RequestMessage?.RequestUri, retryCount, (int)timespan.TotalSeconds);

						return Task.CompletedTask;
					}
				);
			PolicyHttpMessageHandler pollyHandler = new(retryPolicy) { InnerHandler = new HttpClientHandler() };
			HttpClient httpClient = new(pollyHandler, true); // SlackClient will dispose this
			httpClient.DefaultRequestHeaders.Add("Authorization", $"Bearer {slackToken ?? ""}");
			return new SlackClient(httpClient, logger);
		}

		static SlackErrorBehavior GetErrorBehavior<TResponse>(TResponse response) where TResponse : SlackResponse
		{
			if (response.Ok)
			{
				return SlackErrorBehavior.Ignore;
			}

			return response.Error switch
			{
				"already_reacted" => SlackErrorBehavior.Ignore,
				"no_reaction" => SlackErrorBehavior.Ignore,
				"already_in_channel" => SlackErrorBehavior.Ignore,
				"users_not_found" => SlackErrorBehavior.Ignore,
				_ => SlackErrorBehavior.Throw
			};
		}

		private Task<TResponse> SendRequestAsync<TResponse>(string requestUrl, object request, CancellationToken cancellationToken) where TResponse : SlackResponse
		{
			return SendRequestAsync<TResponse>(requestUrl, request, GetErrorBehavior, cancellationToken);
		}

		private async Task<TResponse> SendRequestAsync<TResponse>(string requestUrl, object request, Func<TResponse, SlackErrorBehavior> getErrorBehavior, CancellationToken cancellationToken) where TResponse : SlackResponse
		{
			using HttpRequestMessage sendMessageRequest = new(HttpMethod.Post, requestUrl);
			string requestJson = JsonSerializer.Serialize(request, _serializerOptions);
			using StringContent messageContent = new(requestJson, Encoding.UTF8, "application/json");
			sendMessageRequest.Content = messageContent;
			return await SendRequestAsync<TResponse>(sendMessageRequest, requestJson, getErrorBehavior, cancellationToken);
		}

		private Task<TResponse> SendRequestAsync<TResponse>(HttpRequestMessage request, string requestJson, CancellationToken cancellationToken) where TResponse : SlackResponse
		{
			return SendRequestAsync<TResponse>(request, requestJson, GetErrorBehavior, cancellationToken);
		}

		private async Task<TResponse> SendRequestAsync<TResponse>(HttpRequestMessage request, string requestJson, Func<TResponse, SlackErrorBehavior> getErrorBehavior, CancellationToken cancellationToken) where TResponse : SlackResponse
		{
			HttpResponseMessage response = await _httpClient.SendAsync(request, cancellationToken);
			byte[] responseBytes = await response.Content.ReadAsByteArrayAsync(cancellationToken);

			TResponse responseObject = JsonSerializer.Deserialize<TResponse>(responseBytes)!;
			SlackErrorBehavior behavior = getErrorBehavior(responseObject);

			switch (behavior)
			{
				case SlackErrorBehavior.Throw:
					throw new SlackException(responseObject.Error ?? "unknown");
				case SlackErrorBehavior.Ignore:
					break;
				case SlackErrorBehavior.LogError:
				case SlackErrorBehavior.LogWarning:
					LogLevel level = behavior == SlackErrorBehavior.LogError ? LogLevel.Error : LogLevel.Warning;
					_logger.Log(level, new SlackException(responseObject.Error ?? "unknown"), "Slack request to {Url} failed ({Error}). Request: {Request}. Response: {Response}",
						request.RequestUri, responseObject.Error, requestJson, Encoding.UTF8.GetString(responseBytes));
					break;
				default:
					throw new InvalidOperationException($"Unhandled error behavior: {behavior}");
			}

			return responseObject;
		}

		#region Chat

		const string PostMessageUrl = "https://slack.com/api/chat.postMessage";
		const string UpdateMessageUrl = "https://slack.com/api/chat.update";
		const string GetPermalinkUrl = "https://slack.com/api/chat.getPermalink";

		class PostMessageRequest
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("ts")]
			public string? Ts { get; set; }

			[JsonPropertyName("thread_ts")]
			public string? ThreadTs { get; set; }

			[JsonPropertyName("text")]
			public string? Text { get; set; }

			[JsonPropertyName("mrkdwn")]
			public bool? Markdown { get; set; }

			[JsonPropertyName("blocks")]
			public List<Block>? Blocks { get; set; }

			[JsonPropertyName("attachments")]
			public List<SlackAttachment>? Attachments { get; set; }

			[JsonPropertyName("reply_broadcast")]
			public bool? ReplyBroadcast { get; set; }

			[JsonPropertyName("unfurl_links")]
			public bool? UnfurlLinks { get; set; }

			[JsonPropertyName("unfurl_media")]
			public bool? UnfurlMedia { get; set; }
		}

		class PostMessageResponse : SlackResponse
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("ts")]
			public string? Ts { get; set; }
		}

		/// <summary>
		/// Posts a message to a recipient
		/// </summary>
		/// <param name="recipient">Recipient of the message. May be a channel or Slack user id.</param>
		/// <param name="message">New message to post</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Identifier for the message. Note that the returned channel value will respond to a concrete channel identifier if a user is specified as recipient, and must be used to update the message.</returns>
		public async Task<SlackMessageId> PostMessageAsync(string recipient, SlackMessage message, CancellationToken cancellationToken = default)
		{
			return await PostOrUpdateMessageAsync(recipient, null, null, message, false, cancellationToken);
		}

		/// <summary>
		/// Posts a message to a recipient
		/// </summary>
		/// <param name="threadId">Parent message to post to</param>
		/// <param name="message">New message to post</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Identifier for the message. Note that the returned channel value will respond to a concrete channel identifier if a user is specified as recipient, and must be used to update the message.</returns>
		public Task<SlackMessageId> PostMessageToThreadAsync(SlackMessageId threadId, SlackMessage message, CancellationToken cancellationToken = default)
		{
			return PostMessageToThreadAsync(threadId, message, false, cancellationToken);
		}

		/// <summary>
		/// Posts a message to a recipient
		/// </summary>
		/// <param name="threadId">Parent message to post to</param>
		/// <param name="message">New message to post</param>
		/// <param name="replyBroadcast">Whether to broadcast the message to the channel</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Identifier for the message. Note that the returned channel value will respond to a concrete channel identifier if a user is specified as recipient, and must be used to update the message.</returns>
		public async Task<SlackMessageId> PostMessageToThreadAsync(SlackMessageId threadId, SlackMessage message, bool replyBroadcast, CancellationToken cancellationToken = default)
		{
			return await PostOrUpdateMessageAsync(threadId.Channel, null, threadId.Ts, message, replyBroadcast, cancellationToken);
		}

		/// <summary>
		/// Updates an existing message
		/// </summary>
		/// <param name="id">The message id</param>
		/// <param name="message">New message to post</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task UpdateMessageAsync(SlackMessageId id, SlackMessage message, CancellationToken cancellationToken = default)
		{
			return UpdateMessageAsync(id, message, false, cancellationToken);
		}

		/// <summary>
		/// Updates an existing message
		/// </summary>
		/// <param name="id">The message id</param>
		/// <param name="message">New message to post</param>
		/// <param name="replyBroadcast">For threaded messages, whether to broadcast the message to the channel</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task UpdateMessageAsync(SlackMessageId id, SlackMessage message, bool replyBroadcast = false, CancellationToken cancellationToken = default)
		{
			await PostOrUpdateMessageAsync(id.Channel, id.Ts, id.ThreadTs, message, replyBroadcast, cancellationToken);
		}

		async Task<SlackMessageId> PostOrUpdateMessageAsync(string recipient, string? ts, string? threadTs, SlackMessage message, bool replyBroadcast, CancellationToken cancellationToken)
		{
			PostMessageRequest request = new();
			request.Channel = recipient;
			request.Ts = String.IsNullOrEmpty(ts) ? null : ts;
			request.ThreadTs = String.IsNullOrEmpty(threadTs) ? null : threadTs;
			request.Text = message.Text;
			request.Blocks = message.Blocks;
			request.Markdown = message.Markdown;

			if (replyBroadcast)
			{
				request.ReplyBroadcast = replyBroadcast;
			}
			if (message.Attachments.Count > 0)
			{
				request.Attachments = message.Attachments;
			}
			if (!message.UnfurlLinks)
			{
				request.UnfurlLinks = false;
			}
			if (!message.UnfurlMedia)
			{
				request.UnfurlMedia = false;
			}

			PostMessageResponse response = await SendRequestAsync<PostMessageResponse>(ts == null ? PostMessageUrl : UpdateMessageUrl, request, cancellationToken);
			if (!response.Ok || response.Ts == null)
			{
				throw new SlackException(response.Error ?? "unknown");
			}

			return new SlackMessageId(response.Channel ?? recipient, threadTs, response.Ts);
		}

		class GetPermalinkRequest
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("message_ts")]
			public string? MessageTs { get; set; }
		}

		class GetPermalinkResponse : SlackResponse
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("permalink")]
			public string? Permalink { get; set; }
		}

		/// <summary>
		/// Gets a permalink for a message
		/// </summary>
		/// <param name="id">Message identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Link to the message</returns>
		public async Task<string> GetPermalinkAsync(SlackMessageId id, CancellationToken cancellationToken = default)
		{
			string requestUrl = $"{GetPermalinkUrl}?channel={id.Channel}&message_ts={id.Ts}";
			using (HttpRequestMessage sendMessageRequest = new(HttpMethod.Get, requestUrl))
			{
				GetPermalinkResponse response = await SendRequestAsync<GetPermalinkResponse>(sendMessageRequest, "", cancellationToken);
				if (!response.Ok || response.Permalink == null)
				{
					throw new SlackException(response.Error ?? "unknown");
				}
				return response.Permalink;
			}
		}

		#endregion

		#region Reactions

		const string AddReactionUrl = "https://slack.com/api/reactions.add";
		const string RemoveReactionUrl = "https://slack.com/api/reactions.remove";

		class ReactionMessage
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("timestamp")]
			public string? Ts { get; set; }

			[JsonPropertyName("name")]
			public string? Name { get; set; }
		}

		static SlackErrorBehavior GetReactionErrorBehavior<TResponse>(TResponse response) where TResponse : SlackResponse
		{
			if (response.Error == "not_in_channel")
			{
				return SlackErrorBehavior.LogWarning;
			}

			return GetErrorBehavior(response);
		}

		/// <summary>
		/// Adds a reaction to a posted message
		/// </summary>
		/// <param name="messageId">Message to react to</param>
		/// <param name="name">Name of the reaction to post</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AddReactionAsync(SlackMessageId messageId, string name, CancellationToken cancellationToken = default)
		{
			ReactionMessage message = new();
			message.Channel = messageId.Channel;
			message.Ts = messageId.Ts;
			message.Name = name;

			await SendRequestAsync<SlackResponse>(AddReactionUrl, message, GetReactionErrorBehavior, cancellationToken);
		}

		/// <summary>
		/// Removes a reaction from a posted message
		/// </summary>
		/// <param name="messageId">Message to react to</param>
		/// <param name="name">Name of the reaction to post</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task RemoveReactionAsync(SlackMessageId messageId, string name, CancellationToken cancellationToken = default)
		{
			ReactionMessage message = new();
			message.Channel = messageId.Channel;
			message.Ts = messageId.Ts;
			message.Name = name;

			await SendRequestAsync<SlackResponse>(RemoveReactionUrl, message, GetReactionErrorBehavior, cancellationToken);
		}

		#endregion

		#region Conversations

		const string ConversationsInviteUrl = "https://slack.com/api/conversations.invite";
		const string ConversationsOpenUrl = "https://slack.com/api/conversations.open";

		class InviteMessage
		{
			[JsonPropertyName("channel")]
			public string? Channel { get; set; }

			[JsonPropertyName("users")]
			public string? Users { get; set; } // Comma separated list of ids
		}

		class OpenDirectMessage
		{
			[JsonPropertyName("users")]
			public string? Users { get; set; } // Comma separated list of ids
		}

		class GetChannelResponse : SlackResponse
		{
			[JsonPropertyName("channel")]
			public SlackChannelObject? Channel { get; set; }
		}

		class SlackChannelObject
		{
			[JsonPropertyName("id")]
			public string Id { get; set; } = String.Empty;
		}

		/// <summary>
		/// Invite a user to a channel
		/// </summary>
		/// <param name="channel">Channel identifier to invite the user to</param>
		/// <param name="userId">The user id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task InviteUserAsync(string channel, string userId, CancellationToken cancellationToken = default) => InviteUsersAsync(channel, [userId], cancellationToken);

		/// <summary>
		/// Invite a set of users to a channel
		/// </summary>
		/// <param name="channel">Channel identifier to invite the user to</param>
		/// <param name="userIds">The user id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task InviteUsersAsync(string channel, IEnumerable<string> userIds, CancellationToken cancellationToken = default)
		{
			InviteMessage message = new();
			message.Channel = channel;
			message.Users = String.Join(",", userIds);

			await SendRequestAsync<SlackResponse>(ConversationsInviteUrl, message, cancellationToken);
		}

		/// <summary>
		/// Attempt to invite a set of users to a channel, returning the error code on failure
		/// </summary>
		/// <param name="channel">Channel identifier to invite the user to</param>
		/// <param name="userIds">The user id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<string?> TryInviteUsersAsync(string channel, IEnumerable<string> userIds, CancellationToken cancellationToken = default)
		{
			InviteMessage message = new();
			message.Channel = channel;
			message.Users = String.Join(",", userIds);

			static SlackErrorBehavior IgnoreAllErrors(SlackResponse response) => SlackErrorBehavior.Ignore;
			SlackResponse response = await SendRequestAsync<SlackResponse>(ConversationsInviteUrl, message, IgnoreAllErrors, cancellationToken);
			return response.Ok ? null : response.Error;
		}

		const string AdminConversationsInviteUrl = "https://slack.com/api/admin.conversations.invite";

		/// <summary>
		/// Invite a set of users to a channel using admin powers.
		/// </summary>
		/// <param name="channel">Channel identifier to invite the user to</param>
		/// <param name="userIds">The user id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task AdminInviteUsersAsync(string channel, IEnumerable<string> userIds, CancellationToken cancellationToken = default)
		{
			object message = new { channel_id = channel, user_ids = String.Join(",", userIds) };

			await SendRequestAsync<SlackResponse>(AdminConversationsInviteUrl, message, cancellationToken);
		}

		/// <summary>
		/// Open a direct message with multiple users
		/// </summary>
		/// <param name="userIds">The list of user ids</param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task <string?> OpenDirectMessageAsync(IEnumerable<string> userIds, CancellationToken cancellationToken = default)
		{
			OpenDirectMessage message = new();
			message.Users = String.Join(",", userIds);

			GetChannelResponse response = await SendRequestAsync<GetChannelResponse>(ConversationsOpenUrl, message, cancellationToken);
			if (response.Channel is null)
			{
				return null;
			}

			return response.Channel.Id;
		}

		#endregion

		#region Users

		const string UsersInfoUrl = "https://slack.com/api/users.info";
		const string UsersLookupByEmailUrl = "https://slack.com/api/users.lookupByEmail";

		class UserResponse : SlackResponse
		{
			[JsonPropertyName("user")]
			public SlackUser? User { get; set; }
		}

		/// <summary>
		/// Gets a user's profile
		/// </summary>
		/// <param name="userId">The user id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>User profile</returns>
		public async Task<SlackUser> GetUserAsync(string userId, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage request = new (HttpMethod.Post, $"{UsersInfoUrl}?user={userId}");
			UserResponse response = await SendRequestAsync<UserResponse>(request, "", cancellationToken);
			if (!response.Ok || response.User == null)
			{
				throw new SlackException(response.Error ?? "unknown");
			}
			return response.User;
		}

		/// <summary>
		/// Finds a user by email address
		/// </summary>
		/// <param name="email">The user's email address</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>User profile</returns>
		public async Task<SlackUser?> FindUserByEmailAsync(string email, CancellationToken cancellationToken = default)
		{
			try
			{
				using HttpRequestMessage request = new (HttpMethod.Post, $"{UsersLookupByEmailUrl}?email={email}");
				UserResponse response = await SendRequestAsync<UserResponse>(request, "", cancellationToken);
				return response.User;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Failed to get Slack user email {Url} ({Error})", $"{UsersLookupByEmailUrl}?email={email}", ex.Message);
				return null;
			}
		}

		#endregion

		#region Views

		const string ViewsOpenUrl = "https://slack.com/api/views.open";

		class ViewsOpenRequest
		{
			[JsonPropertyName("trigger_id")]
			public string? TriggerId { get; set; }

			[JsonPropertyName("view")]
			public SlackView? View { get; set; }
		}

		/// <summary>
		/// Open a new modal view, in response to a trigger
		/// </summary>
		/// <param name="triggerId">The trigger id, as returned as part of an interaction payload</param>
		/// <param name="view">Definition for the view</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task OpenViewAsync(string triggerId, SlackView view, CancellationToken cancellationToken = default)
		{
			ViewsOpenRequest request = new();
			request.TriggerId = triggerId;
			request.View = view;
			await SendRequestAsync<PostMessageResponse>(ViewsOpenUrl, request, cancellationToken);
		}

		#endregion
	}
}
