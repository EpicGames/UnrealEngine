// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack
{
	/// <summary>
	/// An object that defines a dialog that provides a confirmation step to any interactive element. This dialog will ask the user to confirm their action by offering a confirm and deny buttons.
	/// </summary>
	public class SlackConfirm
	{
		/// <summary>
		/// A plain_text-only text object that defines the dialog's title. Maximum length for this field is 100 characters.
		/// </summary>
		[JsonPropertyName("title")]
		public PlainTextObject Title { get; set; }

		/// <summary>
		/// A text object that defines the explanatory text that appears in the confirm dialog. Maximum length for the text in this field is 300 characters.
		/// </summary>
		[JsonPropertyName("text")]
		public TextObject Text { get; set; }

		/// <summary>
		/// A plain_text-only text object to define the text of the button that confirms the action. Maximum length for the text in this field is 30 characters.
		/// </summary>
		[JsonPropertyName("confirm")]
		public PlainTextObject Confirm { get; set; }

		/// <summary>
		/// A plain_text-only text object to define the text of the button that cancels the action. Maximum length for the text in this field is 30 characters.
		/// </summary>
		[JsonPropertyName("deny")]
		public PlainTextObject Deny { get; set; }

		/// <summary>
		/// Defines the color scheme applied to the confirm button. A value of danger will display the button with a red background on desktop, or red text on mobile. A value of primary will display the button with a green background on desktop, or blue text on mobile. If this field is not provided, the default value will be primary.
		/// </summary>
		[JsonPropertyName("style")]
		public ButtonStyle Style { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackConfirm(string title, string text, string confirm, string deny, ButtonStyle style = ButtonStyle.Primary)
			: this(title, new PlainTextObject(text), confirm, deny, style)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackConfirm(string title, TextObject text, string confirm, string deny, ButtonStyle style = ButtonStyle.Primary)
			: this(new PlainTextObject(title), text, new PlainTextObject(confirm), new PlainTextObject(deny), style)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackConfirm(PlainTextObject title, TextObject text, PlainTextObject confirm, PlainTextObject deny, ButtonStyle style = ButtonStyle.Primary)
		{
			Title = title;
			Text = text;
			Confirm = confirm;
			Deny = deny;
			Style = style;
		}
	}
}
