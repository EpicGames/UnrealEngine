// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace EpicGames.Slack
{
	/// <summary>
	/// A slack message object
	/// </summary>
	public class SlackMessage : ISlackBlockContainer
	{
		/// <summary>
		/// The formatted text of the message to be published. If blocks are included, this will become the fallback text used in notifications.
		/// </summary>
		public string? Text { get; set; }

		/// <summary>
		/// Disable Slack markup parsing by setting to false. Enabled by default.
		/// </summary>
		public bool? Markdown { get; set; }

		/// <summary>
		/// A JSON-based array of structured blocks, presented as a URL-encoded string.
		/// </summary>
		public List<Block> Blocks { get; } = [];

		/// <summary>
		/// Attachments for the message
		/// </summary>
		public List<SlackAttachment> Attachments { get; } = [];

		/// <summary>
		/// Pass true to enable unfurling of primarily text-based content.
		/// </summary>
		public bool UnfurlLinks { get; set; } = true;

		/// <summary>
		/// Pass false to disable unfurling of media content.
		/// </summary>
		public bool UnfurlMedia { get; set; } = true;

		/// <summary>
		/// Implciit conversion operator from a Markdown string
		/// </summary>
		/// <param name="text"></param>
		public static implicit operator SlackMessage(string text)
		{
			SlackMessage message = new();
			message.Text = text;
			return message;
		}
	}
}
