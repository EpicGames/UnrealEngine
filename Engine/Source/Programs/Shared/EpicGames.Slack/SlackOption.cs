// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack
{
	/// <summary>
	/// An object that represents a single selectable item in a select menu, multi-select menu, checkbox group, radio button group, or overflow menu.
	/// </summary>
	public class SlackOption
	{
		/// <summary>
		/// A text object that defines the text shown in the option on the menu. Overflow, select, and multi-select menus can only use plain_text objects, while radio buttons and checkboxes can use mrkdwn text objects.
		/// Maximum length for the text in this field is 75 characters.
		/// </summary>
		[JsonPropertyName("text")]
		public TextObject Text { get; set; }

		/// <summary>
		/// A unique string value that will be passed to your app when this option is chosen. Maximum length for this field is 75 characters.
		/// </summary>
		[JsonPropertyName("value")]
		public string Value { get; set; }

		/// <summary>
		/// A plain_text only text object that defines a line of descriptive text shown below the text field beside the radio button. Maximum length for the text object within this field is 75 characters.
		/// </summary>
		[JsonPropertyName("description")]
		public PlainTextObject? Description { get; set; }

		/// <summary>
		/// A URL to load in the user's browser when the option is clicked. The url attribute is only available in overflow menus. Maximum length for this field is 3000 characters. If you're using url, you'll still receive an interaction payload and will need to send an acknowledgement response.
		/// </summary>
		[JsonPropertyName("url")]
		public Uri? Url { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackOption()
			: this("", "")
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackOption(TextObject text, string value)
		{
			Text = text;
			Value = value;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackOption(string text, string value)
			: this(new PlainTextObject(text), value)
		{
		}
	}

	/// <summary>
	/// Provides a way to group options in a select menu or multi-select menu.
	/// </summary>
	public class SlackOptionGroup
	{
		/// <summary>
		/// A plain_text only text object that defines the label shown above this group of options. Maximum length for the text in this field is 75 characters.
		/// </summary>
		public PlainTextObject Label { get; }

		/// <summary>
		/// An array of option objects that belong to this specific group. Maximum of 100 items.
		/// </summary>
		public List<SlackOption> Options { get; } = [];

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackOptionGroup(PlainTextObject label, List<SlackOption> options)
		{
			Label = label;
			Options.AddRange(options);
		}
	}
}
