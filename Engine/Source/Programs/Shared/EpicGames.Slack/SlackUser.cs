// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;

namespace EpicGames.Slack
{
	/// <summary>
	/// Information about a Slack user
	/// </summary>
	public class SlackUser
	{
		/// <summary>
		/// Identifier for this workspace user. It is unique to the workspace containing the user. Use this field together with team_id as a unique key when storing related data or when specifying the user in API requests. We recommend considering the format of the string to be an opaque value, and not to rely on a particular structure.
		/// </summary>
		[JsonPropertyName("id")]
		public string? Id { get; set; }

		/// <summary>
		/// The profile object contains the default fields of a user's workspace profile. A user's custom profile fields may be discovered using users.profile.get.
		/// </summary>
		[JsonPropertyName("profile")]
		public SlackUserProfile? Profile { get; set; }

		/// <summary>
		/// A human-readable string for the geographic timezone-related region this user has specified in their account.
		/// </summary>
		[JsonPropertyName("tz")]
		public string? Timezone { get; set; }

		/// <summary>
		/// Describes the commonly used name of the tz timezone.
		/// </summary>
		[JsonPropertyName("tz_label")]
		public string? TimezoneLabel { get; set; }

		/// <summary>
		/// Indicates the number of seconds to offset UTC time by for this user's tz.
		/// </summary>
		[JsonPropertyName("tz_offset")]
		public int TimezoneOffset { get; set; }
	}

	/// <summary>
	/// Profile information for a Slack user
	/// </summary>
	public class SlackUserProfile
	{
		/// <summary>
		/// The display name the user has chosen to identify themselves by in their workspace profile. Do not use this field as a unique identifier for a user, as it may change at any time. Instead, use id and team_id in concert.
		/// </summary>
		[JsonPropertyName("display_name")]
		public string DisplayName { get; set; } = "";

		/// <summary>
		/// A valid email address. It cannot have spaces, and it must have an @ and a domain. It cannot be in use by another member of the same team. Changing a user's email address will send an email to both the old and new addresses, and also post a slackbot to the user informing them of the change. This field can only be changed by admins for users on paid teams.
		/// </summary>
		[JsonPropertyName("email")]
		public string Email { get; set; } = "";

		/// <summary>
		/// The user's first name. The name slackbot cannot be used. Updating first_name will update the first name within real_name.
		/// </summary>
		[JsonPropertyName("first_name")]
		public string FirstName { get; set; } = "";

		/// <summary>
		/// The user's last name. The name slackbot cannot be used. Updating last_name will update the second name within real_name.
		/// </summary>
		[JsonPropertyName("last_name")]
		public string LastName { get; set; } = "";

		/// <summary>
		/// The user's first and last name. Updating this field will update first_name and last_name. If only one name is provided, the value of last_name will be cleared.
		/// </summary>
		[JsonPropertyName("real_name")]
		public string RealName { get; set; } = "";

		/// <summary>
		/// Whether the user has a custom profile image
		/// </summary>
		[JsonPropertyName("is_custom_image")]
		public bool IsCustomImage { get; set; }

		/// <summary>
		/// These various fields will contain https URLs that point to square ratio, web-viewable images (GIFs, JPEGs, or PNGs) that represent different sizes of a user's profile picture.
		/// </summary>
		[JsonPropertyName("image_24")]
		public string? Image24 { get; set; }

		/// <inheritdoc cref="Image24"/>
		[JsonPropertyName("image_32")]
		public string? Image32 { get; set; }

		/// <inheritdoc cref="Image24"/>
		[JsonPropertyName("image_48")]
		public string? Image48 { get; set; }

		/// <inheritdoc cref="Image24"/>
		[JsonPropertyName("image_72")]
		public string? Image72 { get; set; }
	}
}
