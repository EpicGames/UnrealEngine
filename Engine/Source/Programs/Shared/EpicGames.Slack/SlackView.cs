// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json.Serialization;
using EpicGames.Slack.Elements;

namespace EpicGames.Slack
{
	/// <summary>
	/// Describes a Slack view
	/// </summary>
	public class SlackView : ISlackBlockContainer
	{
		/// <summary>
		/// The type of view. Set to modal for modals and home for Home tabs.
		/// </summary>
		[JsonPropertyName("type")]
		public string Type { get; } = "modal";

		/// <summary>
		/// The title that appears in the top-left of the modal. Must be a plain_text text element with a max length of 24 characters.
		/// </summary>
		[JsonPropertyName("title")]
		public PlainTextObject Title { get; set; }

		/// <summary>
		/// An array of blocks that defines the content of the view. Max of 100 blocks.
		/// </summary>
		[JsonPropertyName("blocks")]
		public List<Block> Blocks { get; } = [];

		/// <summary>
		/// An optional plain_text element that defines the text displayed in the close button at the bottom-right of the view. Max length of 24 characters.
		/// </summary>
		[JsonPropertyName("close")]
		public PlainTextObject? Close { get; set; }

		/// <summary>
		/// An optional plain_text element that defines the text displayed in the submit button at the bottom-right of the view. submit is required when an input block is within the blocks array. Max length of 24 characters.
		/// </summary>
		[JsonPropertyName("submit")]
		public PlainTextObject? Submit { get; set; }

		/// <summary>
		/// An optional string that will be sent to your app in view_submission and block_actions events. Max length of 3000 characters.
		/// </summary>
		[JsonPropertyName("private_metadata")]
		public string? PrivateMetadata { get; set; }

		/// <summary>
		/// An identifier to recognize interactions and submissions of this particular view.Don't use this to store sensitive information (use private_metadata instead). Max length of 255 characters.
		/// </summary>
		[JsonPropertyName("callback_id")]
		public string? CallbackId { get; set; }

		/// <summary>
		/// When set to true, clicking on the close button will clear all views in a modal and close it. Defaults to false.
		/// </summary>
		[JsonPropertyName("clear_on_close")]
		public bool? ClearOnClose { get; set; }

		/// <summary>
		/// Indicates whether Slack will send your request URL a view_closed event when a user clicks the close button. Defaults to false.
		/// </summary>
		[JsonPropertyName("notify_on_close")]
		public bool? NotifyOnClose { get; set; }

		/// <summary>
		/// A custom identifier that must be unique for all views on a per-team basis.
		/// </summary>
		[JsonPropertyName("external_id")]
		public string? ExternalId { get; set; }

		/// <summary>
		/// When set to true, disables the submit button until the user has completed one or more inputs. This property is for configuration modals.
		/// </summary>
		[JsonPropertyName("submit_disabled")]
		public bool? SubmitDisabled { get; set; }

		/// <summary>
		/// State value when returned via a form notification
		/// </summary>
		[JsonPropertyName("state")]
		public SlackViewState? State { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="title"></param>
		public SlackView(PlainTextObject title)
		{
			Title = title;
		}
	}

	/// <summary>
	/// State object returned from a submission
	/// </summary>
	public class SlackViewState
	{
		/// <summary>
		/// Values for the form, by blockId, then by actionId
		/// </summary>
		[JsonPropertyName("values")]
#pragma warning disable CA2227 // Collection properties should be read only
		public Dictionary<string, Dictionary<string, SlackViewValue>> Values { get; set; } = [];
#pragma warning restore CA2227 // Collection properties should be read only

		/// <summary>
		/// Gets a value from the state
		/// </summary>
		public bool TryGetValue(string blockId, string actionId, [NotNullWhen(true)] out string? value)
		{
			SlackViewValue? viewValue;
			if (TryGetValue(blockId, actionId, out viewValue))
			{
				value = viewValue.Value ?? viewValue.SelectedOption?.Value;
				return value != null;
			}
			else
			{
				value = null;
				return false;
			}
		}

		/// <summary>
		/// Gets a value from the state
		/// </summary>
		public bool TryGetValue(string blockId, string actionId, [NotNullWhen(true)] out SlackViewValue? value)
		{
			Dictionary<string, SlackViewValue>? actions;
			if (Values.TryGetValue(blockId, out actions) && actions.TryGetValue(actionId, out SlackViewValue? viewValue))
			{
				value = viewValue;
				return true;
			}
			else
			{
				value = null;
				return false;
			}
		}
	}

	/// <summary>
	/// Value for a form item
	/// </summary>
	public class SlackViewValue
	{
		/// <summary>
		/// The item type
		/// </summary>
		[JsonPropertyName("type")]
		public string Type { get; set; } = String.Empty;

		/// <summary>
		/// Value for the widget (used for text fields)
		/// </summary>
		[JsonPropertyName("value")]
		public string? Value { get; set; }

		/// <summary>
		/// Current selected option (used for radio buttons)
		/// </summary>
		[JsonPropertyName("selected_option")]
		public SlackOption? SelectedOption { get; set; }
	}
}
