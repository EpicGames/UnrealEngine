<Project Sdk="Microsoft.NET.Sdk">
  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Tracing\EpicGames.Tracing.csproj" />
  </ItemGroup>

  <ItemGroup>
    <None Update="UnrealInsights\example_trace.decomp.utrace">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>
</Project>
