// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using EpicGames.Tracing.UnrealInsights;

namespace EpicGames.Tracing.Tests.UnrealInsights
{
	[TestClass]
	public class EventTest
	{
		[TestMethod]
		public void EnterScopeEventTimestampDeserialize()
		{
			{
				using MemoryStream ms = new([0x10, 0x05, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00]);
				using BinaryReader reader = new(ms);
				EnterScopeEventTimestamp @event = EnterScopeEventTimestamp.Deserialize(reader);
				Assert.AreEqual((ulong)0x05, @event.Timestamp);
			}

			{
				using MemoryStream ms = new([0x10, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0xFF]);
				using BinaryReader reader = new(ms);
				EnterScopeEventTimestamp @event = EnterScopeEventTimestamp.Deserialize(reader);
				Assert.AreEqual((ulong)0x00_FF_00_00_00_00_00_00, @event.Timestamp);
			}
		}
	}
}