// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Tracing.UnrealInsights;
using EpicGames.Tracing.UnrealInsights.Events;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Tracing.Tests.UnrealInsights
{
	[TestClass]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores")]
	public class EventTypeTest
	{
		private readonly string _traceThreadInfoHex = "00 00 4A 00 24 00 04 07 06 0A 00 00 04 00 02 08 04 00 04 00 02 08 08 00 04 00 02 08 0C 00 00 00 88 04 24 54 72 61 63 65 54 68 72 65 61 64 49 6E 66 6F 54 68 72 65 61 64 49 64 53 79 73 74 65 6D 49 64 53 6F 72 74 48 69 6E 74 4E 61 6D 65";
		private readonly string _traceNewTraceHex = "00 00 55 00 10 00 04 05 06 08 00 00 08 00 03 0A 08 00 08 00 03 0E 10 00 02 00 01 06 12 00 01 00 00 0B 24 54 72 61 63 65 4E 65 77 54 72 61 63 65 53 74 61 72 74 43 79 63 6C 65 43 79 63 6C 65 46 72 65 71 75 65 6E 63 79 45 6E 64 69 61 6E 50 6F 69 6E 74 65 72 53 69 7A 65";
		private readonly string _diagnosticsSession2Hex = "00 00 9A 00 22 00 08 07 0B 08 00 00 00 00 88 08 00 00 00 00 88 07 00 00 00 00 89 0B 00 00 00 00 89 06 00 00 00 00 89 0C 00 00 04 00 02 0A 04 00 01 00 00 11 05 00 01 00 00 0A 44 69 61 67 6E 6F 73 74 69 63 73 53 65 73 73 69 6F 6E 32 50 6C 61 74 66 6F 72 6D 41 70 70 4E 61 6D 65 43 6F 6D 6D 61 6E 64 4C 69 6E 65 42 72 61 6E 63 68 42 75 69 6C 64 56 65 72 73 69 6F 6E 43 68 61 6E 67 65 6C 69 73 74 43 6F 6E 66 69 67 75 72 61 74 69 6F 6E 54 79 70 65 54 61 72 67 65 74 54 79 70 65";
		private readonly string _loggingLogMessageHex = "00 00 40 00 1A 00 03 06 07 0A 00 00 08 00 03 08 08 00 08 00 03 05 10 00 00 00 80 0A 4C 6F 67 67 69 6E 67 4C 6F 67 4D 65 73 73 61 67 65 4C 6F 67 50 6F 69 6E 74 43 79 63 6C 65 46 6F 72 6D 61 74 41 72 67 73";

		[TestMethod]
		public void Deserialize_Trace_NewTrace()
		{
			DeserializeAndAssertEventType(_traceNewTraceHex, TraceNewTraceEvent.EventType, 16);
		}

		[TestMethod]
		public void Serialize_Trace_NewTrace()
		{
			SerializeAndAssertEventType(_traceNewTraceHex, TraceNewTraceEvent.EventType, 16);
		}

		[TestMethod]
		public void Deserialize_Trace_ThreadInfo()
		{
			DeserializeAndAssertEventType(_traceThreadInfoHex, TraceThreadInfoEvent.EventType, 36);
		}

		[TestMethod]
		public void Serialize_Trace_ThreadInfo()
		{
			SerializeAndAssertEventType(_traceThreadInfoHex, TraceThreadInfoEvent.EventType, 36);
		}

		[TestMethod]
		public void Deserialize_Diagnostics_Session2()
		{
			DeserializeAndAssertEventType(_diagnosticsSession2Hex, DiagnosticsSession2Event.EventType, 34);
		}

		[TestMethod]
		public void Serialize_Diagnostics_Session2()
		{
			SerializeAndAssertEventType(_diagnosticsSession2Hex, DiagnosticsSession2Event.EventType, 34);
		}

		[TestMethod]
		public void Deserialize_Logging_LogMessage()
		{
			DeserializeAndAssertEventType(_loggingLogMessageHex, LoggingLogMessage.EventType, 26);
		}

		[TestMethod]
		public void Serialize_Logging_LogMessage()
		{
			SerializeAndAssertEventType(_loggingLogMessageHex, LoggingLogMessage.EventType, 26);
		}

		private static EventType DeserializeAndAssertEventType(string hexData, EventType expectedType, ushort expectedUid)
		{
			byte[] block = GenericEventTest.HexStringToBytes(hexData);
			using MemoryStream ms = new(block);
			using BinaryReader reader = new(ms);

			TraceImportantEventHeader eventHeader = TraceImportantEventHeader.Deserialize(reader);
			Assert.AreEqual(PredefinedEventUid.NewEvent, eventHeader.Uid);

			(ushort newEventUid, EventType actualType) = EventType.Deserialize(reader);
			reader.EnsureEntireStreamIsConsumed();

			Assert.AreEqual(expectedUid, actualType.NewEventUid);
			Assert.AreEqual(expectedUid, newEventUid);
			Assert.AreEqual(expectedType.Name, actualType.Name);
			Assert.AreEqual(expectedType.Size, actualType.Size);
			Assert.AreEqual(expectedType.Flags, actualType.Flags);
			Assert.AreEqual(expectedType.Fields.Count, actualType.Fields.Count);
			for (int i = 0; i < expectedType.Fields.Count; i++)
			{

				Assert.AreEqual(expectedType.Fields[i].Offset, actualType.Fields[i].Offset);
				Assert.AreEqual(expectedType.Fields[i].Size, actualType.Fields[i].Size);
				Assert.AreEqual(expectedType.Fields[i].TypeInfo, actualType.Fields[i].TypeInfo);
				Assert.AreEqual(expectedType.Fields[i].Name, actualType.Fields[i].Name);
				Assert.AreEqual(expectedType.Fields[i].NameSize, actualType.Fields[i].NameSize);
			}

			return actualType;
		}

		private static void SerializeAndAssertEventType(string expectedHexData, EventType eventType, ushort uid)
		{
			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);
			byte[] expectedData = GenericEventTest.HexStringToBytes(expectedHexData);
			int expectedEventSize = expectedData.Length;
			Assert.AreEqual(expectedEventSize, eventType.Size);
			eventType.Serialize(uid, writer);

			GenericEventTest.AssertHexString(expectedHexData, ms.ToArray());
		}
	}
}