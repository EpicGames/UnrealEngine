// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Tracing.UnrealInsights;
using EpicGames.Tracing.UnrealInsights.Events;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace EpicGames.Tracing.Tests.UnrealInsights
{
	[TestClass]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores")]
	public class GenericEventTest
	{
		private readonly string _cpuProfilerEventSpecHex = "21 00 77 00 34 25 00 00 01 C1 0D 00 4B 32 4E 6F 64 65 5F 56 61 72 69 61 62 6C 65 47 65 74 20 2F 45 6E 67 69 6E 65 2F 45 6E 67 69 6E 65 53 6B 79 2F 42 50 5F 53 6B 79 5F 53 70 68 65 72 65 2E 42 50 5F 53 6B 79 5F 53 70 68 65 72 65 3A 55 73 65 72 43 6F 6E 73 74 72 75 63 74 69 6F 6E 53 63 72 69 70 74 2E 4B 32 4E 6F 64 65 5F 56 61 72 69 61 62 6C 65 47 65 74 5F 39 38 33 03";
		private readonly string _cpuProfilerEventBatchHex1 = "4F 00 02 20 1E 00 87 A7 EB AE 8A D1 0B 01 A7 02 02 04 FB 23 03 A4 0A 61 04 0B 05 F2 80 42 AA 1C EB 5C 06 C0 BC 03 17 07 02 47 08 C2 02 AB 15 09 AA 03 07 0A AE 01 05 0B D8 0D 05 0C B7 01 0D D8 03 00 03 0E A4 03 6F 0F 89 FF 03 10 F3 C4 04 11 60 06 A7 02 10 AB 19 12 B3 04 10 9F 11 13 0A 04 02 04 ED 03 10 91 11 14 80 01 02 97 02 10 A9 D6 0B 15 10 06 DD 03 10 CB 85 04 16 12 06 D3 01 10 89 16 17 0C 02 D6 02 07 18 DA 84 28 99 01 19 18 05 1A 0A A5 10 1B CD 01 1C 06 88 6A C5 49 1D B8 EB 24 A9 17 20 AF 01 10 CD F9 01 21 71 22 F2 AE 12 85 04 23 D1 0D 24 85 74 25 EA 58 91 0D 25 80 E6 02 A9 1C 25 88 CD 06 A7 27 25 D0 DA 09 8B 20 25 DE AD 0B AF 33 25 C0 A9 09 ED 23 25 C4 81 0A F5 2D 25 9A EC 0A F3 35 25 D4 CF 03 FF 19 25 BE 92 01 B1 0D 25 98 22 06";
		// Unused at the moment -- private readonly string CpuProfilerEventBatchHex2 = "4F 00 02 40 1E 00 E4 AD A2 B2 8A D1 0B 17 41 12 0B 41 12 0B 41 04 11 41 2C 0F 41 0E 17 41 14 13 41 10 11 41 26 0F 41 12 B1 01 41 28 0D 41 04 0F 41 18 09 41 04 0F 41 16 0B 41 16 19 41 24 0D 41 14 17 41 0E 0B 41 2A 0B 41 48 31 41 06 15 41 06 5B 41 14 0B 41 0C 25 41 14 25 41 14 25 41 10 15 41 12 2D 41 0E 15 41 04 25 41 10 19 41 0C 17 41 16 17 41 1E 17 41 14 0B 41 0E 2D 41 0E 45 41 04 15 41 14 17 41 14 13 41 18 19 41 06 0D 41 10 11 41 10 0B 41 0E 0B 41 0E 0D 41 10 0B 41 10 0F 41 18 0F 41 12 0B 41 18 19 41 0E 1F 41 16 0D 41 14 1F 41 16 33 41 10 0D 41 24 25 41 18 15 41 04 23 41 06 25 41 04 0B 41 14 0F 41 0E 15 41 12 77 41 12 0D 41 14 23 41 18 0F 41 16 3B 41 18 19 41 20 81 01 41 18 4D 41 16 2B 41 16 1F 41 1C 11 41 10 11 41 10 0B 41 0E 15 41 06";
		private readonly string _loggingLogMessageHex = "35 00 B4 FD CA E1 FB 7F 00 00 A4 AB DB 52 44 17 00 00 02 A2 03 00 02 C2 44 61 00 71 00 50 00 72 00 6F 00 66 00 2E 00 64 00 6C 00 6C 00 00 00 7E 00 00 00 06";
		private readonly string _memoryMemoryScopeHex1 = "25 00 2D 00 00 27 00 00 00";
		private readonly string _memoryMemoryScopeHex2 = "25 00 2E 00 00 27 00 00 00";
		private readonly string _memoryMemoryScopeHex3 = "25 00 3D 59 00 33 00 00 00";
		private readonly string _traceNewTraceHex = "10 00 13 00 D9 6E DA 52 44 17 00 00 80 96 98 00 00 00 00 00 4D 52 08";
		private readonly string _diagnosticsSession2Hex = "22 00 EA 00 DC 26 0A 01 03 04 01 A0 00 00 57 69 6E 36 34 01 81 01 00 55 6E 72 65 61 6C 45 64 69 74 6F 72 01 C2 0F 00 20 00 44 00 3A 00 5C 00 64 00 65 00 70 00 6F 00 74 00 5C 00 73 00 74 00 61 00 72 00 73 00 68 00 69 00 70 00 2D 00 6D 00 61 00 69 00 6E 00 5C 00 51 00 41 00 47 00 61 00 6D 00 65 00 5C 00 51 00 41 00 47 00 61 00 6D 00 65 00 2E 00 75 00 70 00 72 00 6F 00 6A 00 65 00 63 00 74 00 20 00 2D 00 74 00 72 00 61 00 63 00 65 00 3D 00 63 00 70 00 75 00 20 00 2D 00 67 00 61 00 6D 00 65 00 01 83 02 00 2B 00 2B 00 55 00 45 00 35 00 2B 00 4D 00 61 00 69 00 6E 00 01 84 05 00 2B 00 2B 00 55 00 45 00 35 00 2B 00 4D 00 61 00 69 00 6E 00 2D 00 43 00 4C 00 2D 00 31 00 37 00 34 00 34 00 32 00 35 00 32 00 34 00 03";
		private readonly string _traceThreadInfoHex1 = "24 00 1B 00 02 00 00 00 74 9D 02 00 FF FF FF FF 01 43 01 00 47 61 6D 65 54 68 72 65 61 64 03";
		private readonly string _traceThreadInfoHex2 = "24 00 16 00 03 00 00 00 00 00 00 00 FF FF FF 7F 01 A3 00 00 54 72 61 63 65 03";
		private readonly string _traceThreadInfoHex3 = "24 00 25 00 04 00 00 00 D8 BB 06 00 50 00 00 00 01 83 02 00 46 6F 72 65 67 72 6F 75 6E 64 20 57 6F 72 6B 65 72 20 23 30 03";

		[TestMethod]
		public void Deserialize_Memory_MemoryScope()
		{
			GenericEvent event1 = DeserializeAndAssertGenericEvent(_memoryMemoryScopeHex1, MemoryMemoryScopeEvent.EventType, false, 18, 45);
			Assert.AreEqual(0x27, event1.GetFields()[0].Int!.Value);

			GenericEvent event2 = DeserializeAndAssertGenericEvent(_memoryMemoryScopeHex2, MemoryMemoryScopeEvent.EventType, false, 18, 46);
			Assert.AreEqual(0x27, event2.GetFields()[0].Int!.Value);

			GenericEvent event3 = DeserializeAndAssertGenericEvent(_memoryMemoryScopeHex3, MemoryMemoryScopeEvent.EventType, false, 18, 5832765);
			Assert.AreEqual(0x33, event3.GetFields()[0].Int!.Value);
		}

		[TestMethod]
		public void Serialize_Memory_MemoryScope()
		{
			SerializeAndAssertGenericEvent(_memoryMemoryScopeHex1, MemoryMemoryScopeEvent.EventType, false, 18,
			[
				Field.FromInt(0x27),
			], 45);

			SerializeAndAssertGenericEvent(_memoryMemoryScopeHex2, MemoryMemoryScopeEvent.EventType, false, 18,
			[
				Field.FromInt(0x27),
			], 46);

			SerializeAndAssertGenericEvent(_memoryMemoryScopeHex3, MemoryMemoryScopeEvent.EventType, false, 18,
			[
				Field.FromInt(0x33),
			], 5832765);
		}

		[TestMethod]
		public void Deserialize_Trace_NewTrace()
		{
			GenericEvent @event = DeserializeAndAssertGenericEvent(_traceNewTraceHex, TraceNewTraceEvent.EventType, true, 16);
			Field[] fields = @event.GetFields();
			Assert.AreEqual(25582215261913, fields[0].Long!.Value);
			Assert.AreEqual(10000000, fields[1].Long!.Value);
			Assert.AreEqual(21069, fields[2].Int!.Value);
			Assert.AreEqual(8, fields[3].Int!.Value);
		}

		[TestMethod]
		public void Serialize_Trace_NewTrace_Generic()
		{
			SerializeAndAssertGenericEvent(_traceNewTraceHex, TraceNewTraceEvent.EventType, true, 16,
			[
				Field.FromLong(25582215261913),
				Field.FromLong(10000000),
				Field.FromInt(21069),
				Field.FromInt(8),
			]);
		}

		[TestMethod]
		public void Serialize_Trace_NewTrace()
		{
			TraceNewTraceEvent @event = new(25582215261913, 10000000, 21069, 8);
			SerializeAndAssertEvent(_traceNewTraceHex, @event, true, 16);
		}

		[TestMethod]
		public void Deserialize_Trace_ThreadInfo()
		{
			{
				GenericEvent @event = DeserializeAndAssertGenericEvent(_traceThreadInfoHex1, TraceThreadInfoEvent.EventType, true, 36);
				Field[] fields = @event.GetFields();
				Assert.AreEqual(2, fields[0].Int!.Value);
				Assert.AreEqual(171380, fields[1].Int!.Value);
				Assert.AreEqual(-1, fields[2].Int!.Value);
				Assert.AreEqual("GameThread", fields[3].String!);
			}

			{
				GenericEvent @event = DeserializeAndAssertGenericEvent(_traceThreadInfoHex2, TraceThreadInfoEvent.EventType, true, 36);
				Field[] fields = @event.GetFields();
				Assert.AreEqual(3, fields[0].Int!.Value);
				Assert.AreEqual(0, fields[1].Int!.Value);
				Assert.AreEqual(2147483647, fields[2].Int!.Value);
				Assert.AreEqual("Trace", fields[3].String!);
			}

			{
				GenericEvent @event = DeserializeAndAssertGenericEvent(_traceThreadInfoHex3, TraceThreadInfoEvent.EventType, true, 36);
				Field[] fields = @event.GetFields();
				Assert.AreEqual(4, fields[0].Int!.Value);
				Assert.AreEqual(441304, fields[1].Int!.Value);
				Assert.AreEqual(80, fields[2].Int!.Value);
				Assert.AreEqual("Foreground Worker #0", fields[3].String!);
			}
		}

		[TestMethod]
		public void Serialize_Trace_ThreadInfo()
		{
			{
				TraceThreadInfoEvent @event = new(2, 171380, -1, "GameThread");
				SerializeAndAssertEvent(_traceThreadInfoHex1, @event, true, 36);
			}

			{
				TraceThreadInfoEvent @event = new(3, 0, 2147483647, "Trace");
				SerializeAndAssertEvent(_traceThreadInfoHex2, @event, true, 36);
			}

			{
				TraceThreadInfoEvent @event = new(4, 441304, 80, "Foreground Worker #0");
				SerializeAndAssertEvent(_traceThreadInfoHex3, @event, true, 36);
			}
		}

		[TestMethod]
		public void Deserialize_Diagnostics_Session2()
		{
			GenericEvent @event = DeserializeAndAssertGenericEvent(_diagnosticsSession2Hex, DiagnosticsSession2Event.EventType, true, 34);
			Field[] fields = @event.GetFields();
			Assert.AreEqual("Win64", fields[0].String!);
			Assert.AreEqual("UnrealEditor", fields[1].String!);
			Assert.AreEqual(@" D:\depot\starship-main\QAGame\QAGame.uproject -trace=cpu -game", fields[2].String!);
			Assert.AreEqual("++UE5+Main", fields[3].String!);
			Assert.AreEqual("++UE5+Main-CL-17442524", fields[4].String!);
			Assert.AreEqual(17442524, fields[5].Int!.Value);
			Assert.AreEqual(3, fields[6].Int!.Value);
			Assert.AreEqual(4, fields[7].Int!.Value);
		}

		[TestMethod]
		public void Serialize_Diagnostics_Session2_Generic()
		{
			SerializeAndAssertGenericEvent(_diagnosticsSession2Hex, DiagnosticsSession2Event.EventType, true, 34,
			[
				Field.FromString("Win64"),
				Field.FromString("UnrealEditor"),
				Field.FromString(@" D:\depot\starship-main\QAGame\QAGame.uproject -trace=cpu -game"),
				Field.FromString("++UE5+Main"),
				Field.FromString("++UE5+Main-CL-17442524"),
				Field.FromInt(17442524),
				Field.FromInt(3),
				Field.FromInt(4)
			]);
		}

		[TestMethod]
		public void Serialize_Diagnostics_Session2()
		{
			DiagnosticsSession2Event @event = new("Win64", "UnrealEditor", @" D:\depot\starship-main\QAGame\QAGame.uproject -trace=cpu -game", "++UE5+Main", "++UE5+Main-CL-17442524", 17442524, 3, 4);
			SerializeAndAssertEvent(_diagnosticsSession2Hex, @event, true, 34);
		}

		[TestMethod]
		public void CpuProfiler_EventSpec_Deserialize()
		{
			GenericEvent @event = DeserializeAndAssertGenericEvent(_cpuProfilerEventSpecHex, CpuProfilerEventSpecEvent.EventType, true, 33);
			Assert.AreEqual(9524, @event.GetFields()[0].Int!.Value);
			Assert.AreEqual("K2Node_VariableGet /Engine/EngineSky/BP_Sky_Sphere.BP_Sky_Sphere:UserConstructionScript.K2Node_VariableGet_983", @event.GetFields()[1].String!);
		}

		[TestMethod]
		public void CpuProfiler_EventSpec_Serialize_Generic()
		{
			SerializeAndAssertGenericEvent(_cpuProfilerEventSpecHex, CpuProfilerEventSpecEvent.EventType, true, 33,
			[
				Field.FromInt(9524),
				Field.FromString("K2Node_VariableGet /Engine/EngineSky/BP_Sky_Sphere.BP_Sky_Sphere:UserConstructionScript.K2Node_VariableGet_983")
			]);
		}

		[TestMethod]
		public void CpuProfiler_EventSpec_Serialize()
		{
			CpuProfilerEventSpecEvent @event = new(9524, "K2Node_VariableGet /Engine/EngineSky/BP_Sky_Sphere.BP_Sky_Sphere:UserConstructionScript.K2Node_VariableGet_983");
			SerializeAndAssertEvent(_cpuProfilerEventSpecHex, @event, true, 33);
		}

		[TestMethod]
		public void CpuProfiler_EventBatch_Deserialize()
		{
			GenericEvent @event = DeserializeAndAssertGenericEvent(_cpuProfilerEventBatchHex1, CpuProfilerEventBatchEvent.EventType, false, 39);
			byte[] eventData = @event.GetFields()[0].GetArray()!;
			Assert.AreEqual(241, eventData.Length);

			CpuProfilerSerializer serializer = new(10000000);
			serializer.Read(eventData);

			byte[] eventDataSerialized = serializer.Write()[0];

			AssertHexString(eventData, eventDataSerialized);
		}

		[TestMethod]
		public void CpuProfiler_EventBatch_Serialize()
		{
			byte[] data = HexStringToBytes("87 A7 EB AE 8A D1 0B 01 A7 02 02 04 FB 23 03 A4 0A 61 04 0B 05 F2 80 42 AA 1C EB 5C 06 C0 BC 03 17 07 02 47 08 C2 02 AB 15 09 AA 03 07 0A AE 01 05 0B D8 0D 05 0C B7 01 0D D8 03 00 03 0E A4 03 6F 0F 89 FF 03 10 F3 C4 04 11 60 06 A7 02 10 AB 19 12 B3 04 10 9F 11 13 0A 04 02 04 ED 03 10 91 11 14 80 01 02 97 02 10 A9 D6 0B 15 10 06 DD 03 10 CB 85 04 16 12 06 D3 01 10 89 16 17 0C 02 D6 02 07 18 DA 84 28 99 01 19 18 05 1A 0A A5 10 1B CD 01 1C 06 88 6A C5 49 1D B8 EB 24 A9 17 20 AF 01 10 CD F9 01 21 71 22 F2 AE 12 85 04 23 D1 0D 24 85 74 25 EA 58 91 0D 25 80 E6 02 A9 1C 25 88 CD 06 A7 27 25 D0 DA 09 8B 20 25 DE AD 0B AF 33 25 C0 A9 09 ED 23 25 C4 81 0A F5 2D 25 9A EC 0A F3 35 25 D4 CF 03 FF 19 25 BE 92 01 B1 0D 25 98 22");
			CpuProfilerEventBatchEvent @event = new(data);
			SerializeAndAssertEvent(_cpuProfilerEventBatchHex1, @event, false, 39);
		}

		[TestMethod]
		public void CpuProfiler_EventBatch_EncodeDecodeTimestamp()
		{
			static void AssertTimestamp(ulong expectedTimestamp, bool expectedIsScopeEnter)
			{
				ulong temp = CpuProfilerSerializer.EncodeTimestamp(expectedTimestamp, expectedIsScopeEnter);
				(ulong actualTimestamp, bool actualIsScopeEnter) = CpuProfilerSerializer.DecodeTimestamp(temp);
				Assert.AreEqual(expectedTimestamp, actualTimestamp);
				Assert.AreEqual(expectedIsScopeEnter, actualIsScopeEnter);
			}

			AssertTimestamp(0, true);
			AssertTimestamp(0, false);
			AssertTimestamp(1, true);
			AssertTimestamp(1, false);
			AssertTimestamp(14, true);
			AssertTimestamp(14, false);
			AssertTimestamp(255, true);
			AssertTimestamp(255, false);
			AssertTimestamp(256, true);
			AssertTimestamp(256, false);
			AssertTimestamp(4828284612, true);
			AssertTimestamp(4828284612, false);
		}

		[TestMethod]
		public void Deserialize_Logging_LogMessage()
		{
			GenericEvent @event = DeserializeAndAssertGenericEvent(_loggingLogMessageHex, LoggingLogMessage.EventType, false, 26);
			Field[] fields = @event.GetFields();
			Assert.AreEqual(140719801695668, fields[0].Long!.Value);
			Assert.AreEqual(25582215343012, fields[1].Long!.Value);
			byte[] bytes = fields[2].GetArray()!;
			Assert.AreEqual(29, bytes.Length);
			Assert.AreEqual(0x02, bytes[0]);
			Assert.AreEqual(0xC2, bytes[1]);
			Assert.AreEqual(0x44, bytes[2]);
			Assert.AreEqual(0x7E, bytes[25]);
		}

		[TestMethod]
		public void SerializeEventWithAuxAndImportant()
		{
			ushort uid = 33;
			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);

			Field[] fields =
			[
				Field.FromInt(9524),
				Field.FromString("K2Node_VariableGet /Engine/EngineSky/BP_Sky_Sphere.BP_Sky_Sphere:UserConstructionScript.K2Node_VariableGet_983"),
			];
			GenericEvent @event = new(98989, fields, CpuProfilerEventSpecEvent.EventType);
			Assert.AreEqual(0x77, @event.Size);

			TraceImportantEventHeader eventHeader = new(uid, @event.Size);
			eventHeader.Serialize(writer);
			@event.Serialize(20, writer);

			AssertHexString(_cpuProfilerEventSpecHex, ms.ToArray());
		}

		[TestMethod]
		public void AuxHeaderDeserialize()
		{
			// Aux header from field "Name" in CpuProfiler.EventSpec
			using MemoryStream ms = new([0x01, 0xC1, 0x0D, 0x00]);
			using BinaryReader reader = new(ms);

			(ushort uid, int fieldIndex, int size) = GenericEvent.DeserializeAuxHeader(reader.ReadUInt32());
			Assert.AreEqual(PredefinedEventUid.AuxData, uid);
			Assert.AreEqual(1, fieldIndex);
			Assert.AreEqual(110, size);
		}

		[TestMethod]
		public void AuxHeaderSerialize()
		{
			uint header = GenericEvent.SerializeAuxHeader(1, 1, 110);
			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);
			writer.Write(header);
			byte[] data = ms.ToArray();
			Assert.AreEqual(0x01, data[0]);
			Assert.AreEqual(0xC1, data[1]);
			Assert.AreEqual(0x0D, data[2]);
			Assert.AreEqual(0x00, data[3]);

			(ushort uid, int fieldIndex, int size) = GenericEvent.DeserializeAuxHeader(header);
			Assert.AreEqual(PredefinedEventUid.AuxData, uid);
			Assert.AreEqual(1, fieldIndex);
			Assert.AreEqual(110, size);
		}

		internal static byte[] StringToByteArray(string hex)
		{
			int numChars = hex.Length;
			byte[] bytes = new byte[numChars / 2];
			for (int i = 0; i < numChars; i += 2)
			{
				bytes[i / 2] = Convert.ToByte(hex.Substring(i, 2), 16);
			}

			return bytes;
		}

		internal static byte[] HexStringToBytes(string hexData)
		{
			return StringToByteArray(hexData.Replace(" ", "", StringComparison.Ordinal));
		}

		public static string ToHexString(byte[] ba)
		{
			return BitConverter.ToString(ba).Replace("-", " ", StringComparison.Ordinal);
		}

		internal static void AssertHexString(string expected, string actual)
		{
			if (expected != actual)
			{
				Console.WriteLine("Expected:\n" + expected);
				Console.WriteLine("Actual:\n" + actual);
			}

			Assert.AreEqual(expected, actual);
		}

		internal static void AssertHexString(string expected, byte[] actual)
		{
			AssertHexString(expected, ToHexString(actual));
		}

		internal static void AssertHexString(byte[] expected, byte[] actual)
		{
			AssertHexString(ToHexString(expected), ToHexString(actual));
		}

		private static GenericEvent DeserializeAndAssertGenericEvent(string hexData, EventType eventType, bool isImportant, ushort expectedUid, uint? expectedSerial = 0)
		{
			byte[] block = HexStringToBytes(hexData);
			using MemoryStream ms = new(block);
			using BinaryReader reader = new(ms);

			ushort uid;
			if (isImportant)
			{
				TraceImportantEventHeader eventHeader = TraceImportantEventHeader.Deserialize(reader);
				Assert.AreEqual(expectedUid, eventHeader.Uid);
				Assert.AreEqual(block.Length - TraceImportantEventHeader.HeaderSize, eventHeader.EventSize);
				uid = eventHeader.Uid;
			}
			else
			{
				uid = reader.ReadPackedUid(out _);
				Assert.AreEqual(expectedUid, uid);
			}

			GenericEvent @event = GenericEvent.Deserialize(uid, reader, eventType);
			reader.EnsureEntireStreamIsConsumed();

			Assert.AreEqual(expectedSerial, @event.Serial);
			Assert.AreEqual(eventType.Fields.Count, @event.GetFields().Length);
			return @event;
		}

		private static void SerializeAndAssertGenericEvent(string expectedHexData, EventType eventType, bool isImportant, ushort uid, Field[] fields, uint serial = 0)
		{
			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);

			byte[] expectedData = HexStringToBytes(expectedHexData);
			GenericEvent @event = new(serial, fields, eventType);

			if (isImportant)
			{
				int expectedEventSize = expectedData.Length - TraceImportantEventHeader.HeaderSize;
				Assert.AreEqual(expectedEventSize, @event.Size);
				TraceImportantEventHeader eventHeader = new(uid, @event.Size);
				eventHeader.Serialize(writer);
			}
			else
			{
				writer.WritePackedUid(uid);
			}

			@event.Serialize(20, writer);

			AssertHexString(expectedHexData, ms.ToArray());
		}

		private static void SerializeAndAssertEvent(string expectedHexData, ITraceEvent @event, bool isImportant, ushort uid)
		{
			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);
			@event.Serialize(uid, writer);

			if (isImportant)
			{
				byte[] expectedData = HexStringToBytes(expectedHexData);
				int expectedEventSize = expectedData.Length - TraceImportantEventHeader.HeaderSize;
				Assert.AreEqual(expectedData.Length, @event.Size);
			}

			AssertHexString(expectedHexData, ms.ToArray());
		}
	}
}