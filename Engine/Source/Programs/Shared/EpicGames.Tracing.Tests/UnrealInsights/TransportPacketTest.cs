// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using EpicGames.Tracing.UnrealInsights;
using EpicGames.Tracing.UnrealInsights.Events;

namespace EpicGames.Tracing.Tests.UnrealInsights
{
	[TestClass]
	public class TransportPacketTest
	{
		[TestMethod]
		public void ShortByteHandling()
		{
			ushort val1 = 0x1234;
			Assert.AreEqual(0x34, BinaryReaderExtensions.GetLowByte(val1));
			Assert.AreEqual(0x12, BinaryReaderExtensions.GetHighByte(val1));

			ushort val2 = BinaryReaderExtensions.MakeShort(0x56, 0x78);
			Assert.AreEqual(0x56, BinaryReaderExtensions.GetLowByte(val2));
			Assert.AreEqual(0x78, BinaryReaderExtensions.GetHighByte(val2));

			ushort val3 = BinaryReaderExtensions.MakeShort(0x25, 0x00);
			Assert.AreEqual(0x25, val3);
		}

		[TestMethod]
		public void GetPackedUid()
		{
			static void AssertGetPackedUid(ushort expectedUid, bool expectedIsTwoByteUid, byte uidLow, byte uidHigh)
			{
				Assert.AreEqual(expectedUid, BinaryReaderExtensions.GetPackedUid(uidLow, uidHigh, out bool isTwoByteUid));
				Assert.AreEqual(expectedIsTwoByteUid, isTwoByteUid);
			}

			AssertGetPackedUid(0x05, false, 0x0A, 0x08);
			AssertGetPackedUid(0x13, false, 0x26, 0x00);
			AssertGetPackedUid(4, false, 0x08, 0x25);
			AssertGetPackedUid(4, false, 0x08, 0x29);
			AssertGetPackedUid(5, false, 0x0A, 0x08);
			AssertGetPackedUid(8, false, 0x10, 0xE5);
			AssertGetPackedUid(12, false, 0x18, 0xCA);
			AssertGetPackedUid(5, false, 0x0A, 0x0A);
			AssertGetPackedUid(12, false, 0x18, 0xF3);

			AssertGetPackedUid(17, true, 0x23, 0x00);
			AssertGetPackedUid(18, true, 0x25, 0x00);
			AssertGetPackedUid(39, true, 0x4F, 0x00);
			AssertGetPackedUid(41, true, 0x53, 0x00);
			AssertGetPackedUid(0x12, true, 0x25, 0x00);
			AssertGetPackedUid(0x13, true, 0x27, 0x00);
			AssertGetPackedUid(0x4513, true, 0x27, 0x45);
		}

		[TestMethod]
		public void WritePackedUid()
		{
			static void AssertWriteUid(ushort uid, bool expectTwoByteUid)
			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				writer.WritePackedUid(uid);
				byte[] data = ms.ToArray();
				Assert.AreEqual(expectTwoByteUid ? 2 : 1, data.Length);
				byte uidHigh = data.Length == 2 ? data[1] : (byte)0xFF;
				ushort deserializedUid = BinaryReaderExtensions.GetPackedUid(data[0], uidHigh, out bool isTwoByteUid);
				Assert.AreEqual(expectTwoByteUid, isTwoByteUid);
				Assert.AreEqual(uid, deserializedUid);
			}

			AssertWriteUid(0x00, false);
			AssertWriteUid(0x0A, false);
			AssertWriteUid(0x05, false);

			AssertWriteUid(17, true);
			AssertWriteUid(18, true);
			AssertWriteUid(126, true);

			Assert.ThrowsExactly<NotImplementedException>(() => AssertWriteUid(127, true));
			Assert.ThrowsExactly<NotImplementedException>(() => AssertWriteUid(128, true));
			Assert.ThrowsExactly<NotImplementedException>(() => AssertWriteUid(4362, true));
		}

		[TestMethod]
		public void Deserialize7Bit()
		{
			static void Assert7Bit(ulong expectedValue, string actualBytesHex)
			{
				byte[] actualBytes = GenericEventTest.StringToByteArray(actualBytesHex.Replace(" ", "", StringComparison.Ordinal));
				using MemoryStream ms = new(actualBytes);
				using BinaryReader reader = new(ms);
				Assert.AreEqual(expectedValue, TraceUtils.Read7BitUint(reader));
			}

			Assert7Bit(0, "00");
			Assert7Bit(1, "01");
			Assert7Bit(127, "7F");
			Assert7Bit(128, "80 01");
			Assert7Bit(129, "81 01");

			Assert7Bit(16383, "FF 7F");
			Assert7Bit(16384, "80 80 01");
			Assert7Bit(16385, "81 80 01");

			Assert7Bit(2097151, "FF FF 7F");
			Assert7Bit(2097152, "80 80 80 01");
			Assert7Bit(2097153, "81 80 80 01");

			Assert7Bit(268435455, "FF FF FF 7F");
			Assert7Bit(268435456, "80 80 80 80 01");
			Assert7Bit(268435457, "81 80 80 80 01");

			Assert7Bit(34359738367, "FF FF FF FF 7F");
			Assert7Bit(34359738368, "80 80 80 80 80 01");
			Assert7Bit(34359738369, "81 80 80 80 80 01");

			Assert7Bit(562949953421311, "FF FF FF FF FF FF 7F");
			Assert7Bit(562949953421312, "80 80 80 80 80 80 80 01");
			Assert7Bit(562949953421313, "81 80 80 80 80 80 80 01");
		}

		[TestMethod]
		public void Serialize7Bit()
		{
			static void Assert7Bit(ulong value, string expectedBytesHex)
			{
				byte[] expectedBytes = GenericEventTest.StringToByteArray(expectedBytesHex.Replace(" ", "", StringComparison.Ordinal));
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);

				int actualNumBytesWritten = TraceUtils.Write7BitUint(writer, value);
				byte[] actualBytes = ms.ToArray();

				Assert.AreEqual(expectedBytes.Length, actualNumBytesWritten);
				Assert.AreEqual(expectedBytes.Length, actualBytes.Length);
				GenericEventTest.AssertHexString(expectedBytesHex, actualBytes);
			}

			Assert7Bit(0, "00");
			Assert7Bit(1, "01");
			Assert7Bit(127, "7F");
			Assert7Bit(128, "80 01");
			Assert7Bit(129, "81 01");

			Assert7Bit(16383, "FF 7F");
			Assert7Bit(16384, "80 80 01");
			Assert7Bit(16385, "81 80 01");

			Assert7Bit(2097151, "FF FF 7F");
			Assert7Bit(2097152, "80 80 80 01");
			Assert7Bit(2097153, "81 80 80 01");

			Assert7Bit(268435455, "FF FF FF 7F");
			Assert7Bit(268435456, "80 80 80 80 01");
			Assert7Bit(268435457, "81 80 80 80 01");

			Assert7Bit(34359738367, "FF FF FF FF 7F");
			Assert7Bit(34359738368, "80 80 80 80 80 01");
			Assert7Bit(34359738369, "81 80 80 80 80 01");

			Assert7Bit(562949953421311, "FF FF FF FF FF FF 7F");
			Assert7Bit(562949953421312, "80 80 80 80 80 80 80 01");
			Assert7Bit(562949953421313, "81 80 80 80 80 80 80 01");
		}

		[TestMethod]
		public void SerializeDeserializeWithStubs()
		{
			StubTraceEvent event1 = new(1000, 20);
			StubTraceEvent event2 = new(2000, 30);
			ushort uid1 = 5555;
			ushort uid2 = 6666;

			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				TransportPacket packet = TransportPacket.Create(0, 0);
				packet.Serialize(writer, [(uid1, event1 as ITraceEvent), (uid2, event2 as ITraceEvent)]);

				ms.Position = 0;
				using BinaryReader reader = new(ms);

				TransportPacket deserializedPacket = TransportPacket.Deserialize(reader);
				Assert.AreEqual(0, deserializedPacket.ThreadIdAndMarkers);
				Assert.AreEqual(event1.Size + event2.Size, deserializedPacket.GetData().Length);
			}

			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				TransportPacket packet = TransportPacket.Create(0, 1);
				packet.Serialize(writer, []);

				ms.Position = 0;
				using BinaryReader reader = new(ms);

				TransportPacket deserializedPacket = TransportPacket.Deserialize(reader);
				Assert.AreEqual(1, deserializedPacket.ThreadIdAndMarkers);
				Assert.AreEqual(0, deserializedPacket.GetData().Length);
			}

			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				TransportPacket packet = TransportPacket.Create(0, 12345);
				packet.Serialize(writer, []);

				ms.Position = 0;
				using BinaryReader reader = new(ms);

				TransportPacket deserializedPacket = TransportPacket.Deserialize(reader);
				Assert.AreEqual(12345, deserializedPacket.ThreadIdAndMarkers);
				Assert.AreEqual(0, deserializedPacket.GetData().Length);
			}
		}

		[TestMethod]
		public void SerializeDeserialize()
		{
			TraceNewTraceEvent event1 = new(25582215261913, 10000000, 21069, 8);
			DiagnosticsSession2Event event2 = new("Win64", "UnrealEditor", @" D:\depot\starship-main\QAGame\QAGame.uproject -trace=cpu -game", "++UE5+Main", "++UE5+Main-CL-17442524", 17442524, 3, 4);
			CpuProfilerEventSpecEvent event3 = new(9524, "K2Node_VariableGet /Engine/EngineSky/BP_Sky_Sphere.BP_Sky_Sphere:UserConstructionScript.K2Node_VariableGet_983");

			ushort uid1 = 10;
			ushort uid2 = 20;
			ushort uid3 = 30;

			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				TransportPacket packet = TransportPacket.Create(0, 0);
				packet.Serialize(writer, [(uid1, event1 as ITraceEvent), (uid2, event2 as ITraceEvent)]);

				ms.Position = 0;
				using BinaryReader reader = new(ms);

				TransportPacket deserializedPacket = TransportPacket.Deserialize(reader);
				Assert.AreEqual(0, deserializedPacket.ThreadIdAndMarkers);
				Assert.AreEqual(event1.Size + event2.Size, deserializedPacket.GetData().Length);
				reader.EnsureEntireStreamIsConsumed();
			}

			{
				using MemoryStream ms = new();
				using BinaryWriter writer = new(ms);
				TransportPacket packet = TransportPacket.Create(0, 0);
				packet.Serialize(writer,
				[
					(uid1, event1.Type as ITraceEvent),
					(uid2, event2.Type as ITraceEvent),
					(uid3, event3.Type as ITraceEvent)
				]);

				ms.Position = 0;
				using BinaryReader reader = new(ms);

				TransportPacket deserializedPacket = TransportPacket.Deserialize(reader);
				Assert.AreEqual(0, deserializedPacket.ThreadIdAndMarkers);
				Assert.AreEqual(event1.Type.Size + event2.Type.Size + event3.Type.Size, deserializedPacket.GetData().Length);
				reader.EnsureEntireStreamIsConsumed();
			}
		}
	}
}