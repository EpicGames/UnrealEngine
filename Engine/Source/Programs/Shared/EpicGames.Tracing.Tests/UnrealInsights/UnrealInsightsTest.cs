// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using EpicGames.Tracing.UnrealInsights;
using EpicGames.Tracing.UnrealInsights.Events;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using ITraceEvent = EpicGames.Tracing.UnrealInsights.ITraceEvent;
using UnrealInsightsReader = EpicGames.Tracing.UnrealInsights.UnrealInsightsReader;
using UnrealInsightsWriter = EpicGames.Tracing.UnrealInsights.UnrealInsightsWriter;

namespace EpicGames.Tracing.Tests.UnrealInsights
{
	public class StubTraceEvent : ITraceEvent
	{
		readonly uint _val1;
		readonly byte _val2;

		public StubTraceEvent(uint val1, byte val2)
		{
			_val1 = val1;
			_val2 = val2;
		}

		public ushort Size => sizeof(uint) + sizeof(byte);
		public EventType Type => EventType.WellKnown(15, "StubEvent");

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			writer.Write(_val1);
			writer.Write(_val2);
		}

		public static StubTraceEvent Deserialize(BinaryReader reader)
		{
			return new StubTraceEvent(reader.ReadUInt32(), reader.ReadByte());
		}
	}

	[TestClass]
	public class TraceEventHeaderTest
	{
		[TestMethod]
		public void SerializeDeserialize()
		{
			uint test1 = 222;
			uint test2 = 333;
			ulong test3 = 444;
			ushort eventSize = 4 + 4 + 8;

			using MemoryStream ms = new();
			using BinaryWriter writer = new(ms);
			new TraceImportantEventHeader(1000, eventSize).Serialize(writer);
			writer.Write(test1);
			writer.Write(test2);
			writer.Write(test3);

			ms.Position = 0;
			using BinaryReader reader = new(ms);
			TraceImportantEventHeader eventHeader = TraceImportantEventHeader.Deserialize(reader);
			Assert.AreEqual(1000, eventHeader.Uid);
			Assert.AreEqual(eventSize, eventHeader.EventSize);
			Assert.AreEqual(test1, reader.ReadUInt32());
			Assert.AreEqual(test2, reader.ReadUInt32());
			Assert.AreEqual(test3, reader.ReadUInt64());
		}
	}

	[TestClass]
	public class UnrealInsightsTest
	{
		[TestMethod]
		public void WriteUtraceFile()
		{
			using MemoryStream ms = new();
			using BinaryWriter binaryWriter = new(ms);

			UnrealInsightsWriter writer = new();

			TraceNewTraceEvent newTraceEvent1 = new(1001, 2001, 31, 8);
			TraceNewTraceEvent newTraceEvent2 = new(1002, 2002, 32, 8);
			TraceNewTraceEvent newTraceEvent3 = new(1003, 2003, 33, 8);
			CpuProfilerEventSpecEvent cpu1 = new(400, "MyCpuEventSpecEvent1");

			writer.AddEvent(TransportPacket.ThreadIdImportants, newTraceEvent1);
			writer.AddEvent(TransportPacket.ThreadIdImportants, newTraceEvent2);
			writer.AddEvent(TransportPacket.ThreadIdImportants, newTraceEvent3);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpu1);
			writer.Write(binaryWriter);

			ms.Position = 0;

			UnrealInsightsReader reader = new();
			reader.Read(ms);
			reader.PrintEventSummary();
			Assert.AreEqual(4, reader.EventTypes.Count);

			Assert.AreEqual(0, reader.EventsPerThread[TransportPacket.ThreadIdEvents].Count);
			Assert.AreEqual(4, reader.EventsPerThread[TransportPacket.ThreadIdImportants].Count);

			{
				GenericEvent @event = (GenericEvent)reader.EventsPerThread[1][0];
				Field[] fields = @event.GetFields();
				Assert.AreEqual(1001, fields[0].Long!.Value);
				Assert.AreEqual(2001, fields[1].Long!.Value);
				Assert.AreEqual(31, fields[2].Int!.Value);
				Assert.AreEqual(8, fields[3].Int!.Value);
			}
			{
				GenericEvent @event = (GenericEvent)reader.EventsPerThread[1][1];
				Field[] fields = @event.GetFields();
				Assert.AreEqual(1002, fields[0].Long!.Value);
				Assert.AreEqual(2002, fields[1].Long!.Value);
				Assert.AreEqual(32, fields[2].Int!.Value);
				Assert.AreEqual(8, fields[3].Int!.Value);
			}
			{
				GenericEvent @event = (GenericEvent)reader.EventsPerThread[1][2];
				Field[] fields = @event.GetFields();
				Assert.AreEqual(1003, fields[0].Long!.Value);
				Assert.AreEqual(2003, fields[1].Long!.Value);
				Assert.AreEqual(33, fields[2].Int!.Value);
				Assert.AreEqual(8, fields[3].Int!.Value);
			}
			{
				GenericEvent @event = (GenericEvent)reader.EventsPerThread[1][3];
				Field[] fields = @event.GetFields();
				Assert.AreEqual(400, fields[0].Int!.Value);
				Assert.AreEqual("MyCpuEventSpecEvent1", fields[1].String!);
			}
		}

		[TestMethod]
		public void WriteUtraceExample()
		{
			using MemoryStream ms = new();
			using BinaryWriter binaryWriter = new(ms);
			UnrealInsightsWriter writer = new();

			byte[] cpuBatchData1 = GenericEventTest.HexStringToBytes("87 A7 EB AE 8A D1 0B 01 A7 02 02 04 FB 23 03 A4 0A 61 04 0B 05 F2 80 42 AA 1C EB 5C 06 C0 BC 03 17 07 02 47 08 C2 02 AB 15 09 AA 03 07 0A AE 01 05 0B D8 0D 05 0C B7 01 0D D8 03 00 03 0E A4 03 6F 0F 89 FF 03 10 F3 C4 04 11 60 06 A7 02 10 AB 19 12 B3 04 10 9F 11 13 0A 04 02 04 ED 03 10 91 11 14 80 01 02 97 02 10 A9 D6 0B 15 10 06 DD 03 10 CB 85 04 16 12 06 D3 01 10 89 16 17 0C 02 D6 02 07 18 DA 84 28 99 01 19 18 05 1A 0A A5 10 1B CD 01 1C 06 88 6A C5 49 1D B8 EB 24 A9 17 20 AF 01 10 CD F9 01 21 71 22 F2 AE 12 85 04 23 D1 0D 24 85 74 25 EA 58 91 0D 25 80 E6 02 A9 1C 25 88 CD 06 A7 27 25 D0 DA 09 8B 20 25 DE AD 0B AF 33 25 C0 A9 09 ED 23 25 C4 81 0A F5 2D 25 9A EC 0A F3 35 25 D4 CF 03 FF 19 25 BE 92 01 B1 0D 25 98 22");

			ulong cycleFrequency = 10000000;
			TraceNewTraceEvent newTrace1 = new(25582215261913, cycleFrequency, 21069, 8);
			TraceThreadInfoEvent threadInfo1 = new(2, 171380, -1, "GameThread");
			TraceThreadInfoEvent threadInfo2 = new(3, 0, 2147483647, "Trace");
			DiagnosticsSession2Event diagnosticsSession = new("Finally it works", "HELLO WORLD", "QAGame.uproject -trace=cpu -game", "++UE5+Main", "++UE5+Main-CL-17442524", 1000, 3, 4);
			CpuProfilerEventSpecEvent cpuEventSpec1 = new(1, "MyCpuEventSpecEvent1");
			CpuProfilerEventSpecEvent cpuEventSpec2 = new(2, "MyCpuEventSpecEvent2");
			CpuProfilerEventSpecEvent cpuEventSpec3 = new(3, "MyCpuEventSpecEvent3");
			CpuProfilerEventSpecEvent cpuEventSpec4 = new(4, "MyCpuEventSpecEvent4");
			CpuProfilerEventSpecEvent cpuEventSpec5 = new(5, "MyCpuEventSpecEvent5");
			CpuProfilerEventSpecEvent cpuEventSpec6 = new(6, "MyCpuEventSpecEvent6");
			CpuProfilerEventSpecEvent cpuEventSpec7 = new(7, "MyCpuEventSpecEvent7");
			CpuProfilerEventSpecEvent cpuEventSpec8 = new(8, "MyCpuEventSpecEvent8");
			CpuProfilerEventSpecEvent cpuEventSpecTesting = new(9, "TestingSome");
			CpuProfilerEventBatchEvent cpuBatch1 = new(cpuBatchData1);

			writer.AddEvent(TransportPacket.ThreadIdImportants, newTrace1);
			writer.AddEvent(TransportPacket.ThreadIdImportants, threadInfo1);
			writer.AddEvent(TransportPacket.ThreadIdImportants, threadInfo2);
			writer.AddEvent(TransportPacket.ThreadIdImportants, diagnosticsSession);

			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec1);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec2);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec3);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec4);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec5);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec6);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec7);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpec8);
			writer.AddEvent(TransportPacket.ThreadIdImportants, cpuEventSpecTesting);

			writer.AddEvent(2, cpuBatch1);

			CpuProfilerSerializer cpuSerializerThread3 = new(cycleFrequency);
			cpuSerializerThread3.ScopeEvents.Add(new CpuProfilerScopeEvent(25582215261913 + 400 * 1000, true, cpuEventSpecTesting.Id));
			cpuSerializerThread3.ScopeEvents.Add(new CpuProfilerScopeEvent(25582215261913 + (cycleFrequency * cycleFrequency), false, null));

			List<byte[]> eventBatchData = cpuSerializerThread3.Write();
			foreach (byte[] data in eventBatchData)
			{
				writer.AddEvent(3, new CpuProfilerEventBatchEvent(data));
			}
			writer.Write(binaryWriter);

			ms.Seek(0, SeekOrigin.Begin);

			UnrealInsightsReader reader = new();
			reader.Read(ms);
			reader.PrintEventSummary();

			Assert.AreEqual(0, reader.EventsPerThread[TransportPacket.ThreadIdEvents].Count);
		}

		[TestMethod]
		public void ReadUtraceFile()
		{
			string exampleDecompTrace = "UnrealInsights/example_trace.decomp.utrace";

			UnrealInsightsReader reader = new();
			using FileStream stream = File.Open(exampleDecompTrace, FileMode.Open);
			reader.Read(stream);
			reader.PrintEventSummary();

			Assert.AreEqual(6281, reader.NumTransportPacketsRead);
			Dictionary<ushort, List<ITraceEvent>> eventsPerUid = reader.GetEventsPerUid();

			void AssertEvent(ushort uid, string expectedName, int expectedCount)
			{
				Assert.IsTrue(reader.EventTypes.ContainsKey(uid));
				Assert.IsTrue(eventsPerUid.ContainsKey(uid));
				Assert.AreEqual(expectedName, reader.EventTypes[uid].Name);
				Assert.AreEqual(expectedCount, eventsPerUid[uid].Count);
			}

			AssertEvent(16, "$Trace.NewTrace", 1);
			AssertEvent(17, "$Trace.ThreadTiming", 71);
			AssertEvent(18, "Memory.MemoryScope", 16614);
			AssertEvent(19, "Logging.LogCategory", 741);
			AssertEvent(20, "Memory.MemoryScopeRealloc", 6262);
			AssertEvent(21, "Stats.Spec", 972);
			AssertEvent(22, "CsvProfiler.RegisterCategory", 51);
			AssertEvent(23, "CsvProfiler.DefineDeclaredStat", 159);
			AssertEvent(24, "Counters.Spec", 85);
			AssertEvent(25, "Logging.LogMessageSpec", 6);
			AssertEvent(26, "Logging.LogMessage", 11);
			AssertEvent(27, "CsvProfiler.DefineInlineStat", 1);
			AssertEvent(28, "LoadTime.ClassInfo", 1);
			AssertEvent(29, "SlateTrace.AddWidget", 1);
			AssertEvent(30, "SlateTrace.WidgetInfo", 1);
			AssertEvent(31, "Misc.BookmarkSpec", 2);
			AssertEvent(32, "Misc.Bookmark", 2);
			AssertEvent(33, "CpuProfiler.EventSpec", 13003);
			AssertEvent(34, "Diagnostics.Session2", 1);
			AssertEvent(35, "Trace.ChannelToggle", 32);
			AssertEvent(36, "$Trace.ThreadInfo", 100);
			AssertEvent(37, "Trace.ChannelAnnounce", 34);
			AssertEvent(38, "$Trace.ThreadGroupBegin", 12);
			AssertEvent(39, "CpuProfiler.EventBatch", 64354);
			AssertEvent(40, "$Trace.ThreadGroupEnd", 12);
			AssertEvent(41, "Cpu.CacheResourceShadersForRendering", 109);
			AssertEvent(42, "SourceFilters.FilterClass", 2);
			AssertEvent(43, "WorldSourceFilters.WorldInstance", 1);
			AssertEvent(44, "WorldSourceFilters.WorldOperation", 2);
			AssertEvent(45, "CpuProfiler.EndThread", 66);
		}
	}
}