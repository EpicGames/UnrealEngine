<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>
</Project>
