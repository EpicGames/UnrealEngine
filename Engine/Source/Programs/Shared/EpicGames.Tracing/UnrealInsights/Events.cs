// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Runtime.CompilerServices;
using EpicGames.Tracing.UnrealInsights.Events;

[assembly: InternalsVisibleTo("EpicGames.Tracing.Tests")]

namespace EpicGames.Tracing.UnrealInsights
{
	public static class BinaryReaderExtensions
	{
		public static byte[] ReadBytesStrict(this BinaryReader reader, int count)
		{
			byte[] result = reader.ReadBytes(count);
			if (result.Length != count)
			{
				throw new EndOfStreamException();
			}

			return result;
		}

		/// <summary>
		/// Check if bit is set for a given byte
		/// </summary>
		/// <param name="value">Value to check</param>
		/// <param name="pos">Pos 0 is least significant bit, pos 7 is most</param>
		/// <returns>True if set</returns>
		internal static bool IsBitSet(byte value, int pos)
		{
			return (value & (1 << pos)) != 0;
		}

		internal static bool IsTwoByteUid(byte uidLow)
		{
			return IsBitSet(uidLow, 0);
		}

		internal static byte GetHighByte(ushort a)
		{
			return (byte)(a >> 8);
		}

		internal static byte GetLowByte(ushort a)
		{
			return (byte)(a & 0xff);
		}

		internal static ushort GetHighWord(uint a)
		{
			return (ushort)(a >> 16);
		}

		internal static ushort GetLowWord(uint a)
		{
			return (ushort)(a & 0xffff);
		}

		internal static ushort MakeShort(byte lowByte, byte highByte)
		{
			return (ushort)((byte)(lowByte & 0xff) | (ushort)(highByte & 0xff) << 8);
		}

		internal static ushort GetPackedUid(byte uidLow, byte uidHigh, out bool isTwoByteUidArg)
		{
			// struct packed_uid
			// {
			// 	uint8   is_two_byte_uid : 1
			// 			uid_low         : 7
			// 	(uint8  uid_high) // if is_two_byte_uid == 1
			// }

			isTwoByteUidArg = IsTwoByteUid(uidLow);
			byte uidLowNoBit = (byte)(uidLow >> 1); // Strip the is_two_byte_uid bit

			return MakeShort(uidLowNoBit, isTwoByteUidArg ? uidHigh : (byte)0x00);
		}

		public static ushort ReadPackedUid(this BinaryReader reader, out bool isTwoByteUid)
		{
			byte uidLow = reader.ReadByte();
			byte uidHigh = reader.ReadByte();
			reader.BaseStream.Position -= 1; // UidLow is always consumed, but UidHigh maybe not.

			ushort uid = GetPackedUid(uidLow, uidHigh, out isTwoByteUid);
			if (isTwoByteUid)
			{
				reader.ReadByte();  // Consume UidHigh
			}

			return uid;
		}

		public static void EnsureEntireStreamIsConsumed(this BinaryReader reader)
		{
			bool isEntireStreamConsumed = reader.BaseStream.Position == reader.BaseStream.Length;
			if (!isEntireStreamConsumed)
			{
				throw new Exception($"Entire stream/buffer was not consumed. Pos={reader.BaseStream.Position} Len={reader.BaseStream.Length}");
			}
		}

		public static bool IsEntireStreamConsumed(this BinaryReader reader)
		{
			return reader.BaseStream.Position == reader.BaseStream.Length;
		}
	}

	public static class BinaryWriterExtensions
	{
		public static void WritePackedUid(this BinaryWriter writer, ushort uid)
		{
			if (uid < PredefinedEventUid.WellKnownNum)
			{
				byte uidLow = (byte)uid;
				uidLow = (byte)(uidLow << 1);
				// LSB is 0 after shifting, indicating a one-byte UID
				writer.Write(uidLow);
			}
			else if (uid < 127)
			{
				byte uidLow = BinaryReaderExtensions.GetLowByte(uid);
				byte uidHigh = BinaryReaderExtensions.GetHighByte(uid);
				uidLow = (byte)(uidLow << 1);
				uidLow = (byte)(uidLow | 1);
				writer.Write(uidLow);
				writer.Write(uidHigh);
			}
			else
			{
				throw new NotImplementedException("Handling of UIDs >= 127 not implemented");
			}
		}
	}

	public interface ITraceEvent
	{
		ushort Size { get; }
		EventType Type { get; }
		public void Serialize(ushort uid, BinaryWriter writer);
	}

	public class EnterScopeEvent : ITraceEvent
	{
		public ushort Size => 0;
		public EventType Type => EventType.WellKnown(PredefinedEventUid.EnterScope, "EnterScope");
		public void Serialize(ushort uid, BinaryWriter writer) { throw new NotImplementedException(); }
	}

	public class LeaveScopeEvent : ITraceEvent
	{
		public ushort Size => 0;
		public EventType Type => EventType.WellKnown(PredefinedEventUid.LeaveScope, "LeaveScope");
		public void Serialize(ushort uid, BinaryWriter writer) { throw new NotImplementedException(); }
	}

	public class EnterScopeEventTimestamp : ITraceEvent
	{
		public ushort Size => 7;
		public EventType Type => EventType.WellKnown(PredefinedEventUid.EnterScope_T, "EnterScopeTimestamp");
		public ulong Timestamp { get; }

		public EnterScopeEventTimestamp(ulong timestamp)
		{
			Timestamp = timestamp;
		}

		public void Serialize(ushort uid, BinaryWriter writer) { throw new NotImplementedException(); }

		public static EnterScopeEventTimestamp Deserialize(BinaryReader reader)
		{
			ulong value = reader.ReadUInt64();
			ushort uidFound = BinaryReaderExtensions.GetPackedUid((byte)(value & 0xFF), 0x00, out bool _);

			if (uidFound != PredefinedEventUid.EnterScope_T)
			{
				throw new ArgumentException($"Bad UID found when deserializing 0x{uidFound:X4}/{uidFound}");
			}

			ulong timestamp = value >> 8;
			return new EnterScopeEventTimestamp(timestamp);
		}
	}

	public class LeaveScopeEventTimestamp : ITraceEvent
	{
		public ushort Size => 7;
		public EventType Type => EventType.WellKnown(PredefinedEventUid.EnterScope_T, "LeaveScopeTimestamp");
		[System.Diagnostics.CodeAnalysis.SuppressMessage("CodeQuality", "IDE0052:Remove unread private members", Justification = "Serialize() unimplemented")]
		readonly ulong _timestamp;

		public LeaveScopeEventTimestamp(ulong timestamp)
		{
			_timestamp = timestamp;
		}

		public void Serialize(ushort uid, BinaryWriter writer) { throw new NotImplementedException(); }

		public static LeaveScopeEventTimestamp Deserialize(BinaryReader reader)
		{
			ulong value = reader.ReadUInt64();
			ushort uidFound = BinaryReaderExtensions.GetPackedUid((byte)(value & 0xFF), 0x00, out bool _);

			if (uidFound != PredefinedEventUid.LeaveScope_T)
			{
				throw new ArgumentException($"Bad UID found when deserializing 0x{uidFound:X4}/{uidFound}");
			}

			ulong timestamp = value >> 8;
			return new LeaveScopeEventTimestamp(timestamp);
		}
	}

	public class TraceImportantEventHeader
	{
		public ushort Uid { get; }
		public ushort EventSize { get; }

		public const ushort HeaderSize = sizeof(ushort) + sizeof(ushort);

		public TraceImportantEventHeader(ushort uid, ushort eventSize)
		{
#pragma warning disable CA1508 // Avoid dead conditional code
			Debug.Assert(HeaderSize == 4);
#pragma warning restore CA1508 // Avoid dead conditional code
			Uid = uid;
			EventSize = eventSize;
		}

		public void Serialize(BinaryWriter writer)
		{
			writer.Write(Uid);
			writer.Write(EventSize);
		}

		public static TraceImportantEventHeader Deserialize(BinaryReader reader)
		{
			ushort uid = reader.ReadUInt16();
			return new TraceImportantEventHeader(uid, reader.ReadUInt16());
		}
	}
}