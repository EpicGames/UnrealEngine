// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class CpuProfilerEventSpecEvent : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "CpuProfiler", "EventSpec", EventType.FlagImportant | EventType.FlagMaybeHasAux | EventType.FlagNoSync,
			[
				new EventTypeField(0, 4, EventTypeField.TypeInt32, "Id"),
				new EventTypeField(4, 0, EventTypeField.TypeAnsiString, "Name")
			]);

		public ushort Size => (ushort)(_genericEvent.Size + TraceImportantEventHeader.HeaderSize);
		public EventType Type => EventType;

		public uint Id { get; }
		readonly string _name;

		private readonly GenericEvent _genericEvent;

		public CpuProfilerEventSpecEvent(uint id, string name)
		{
			Id = id;
			_name = name;

			Field[] fields =
			[
				Field.FromInt((int) Id),
				Field.FromString(_name),
			];

			_genericEvent = new GenericEvent(0, fields, EventType);
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			new TraceImportantEventHeader(uid, _genericEvent.Size).Serialize(writer);
			_genericEvent.Serialize(uid, writer);
		}
	}

	public class CpuProfilerEventBatchEvent : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "CpuProfiler", "EventBatch", EventType.FlagMaybeHasAux | EventType.FlagNoSync,
			[new EventTypeField(0, 0, EventTypeField.TypeArray, "Data")]);

		public ushort Size => (ushort)(_genericEvent.Size + sizeof(ushort));
		public EventType Type => EventType;

		readonly GenericEvent _genericEvent;

		public CpuProfilerEventBatchEvent(byte[] data)
		{
			Field[] fields = [Field.FromArray(data)];
			_genericEvent = new GenericEvent(0, fields, EventType);
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			writer.WritePackedUid(uid);
			_genericEvent.Serialize(uid, writer);
		}
	}

	public class CpuProfilerScopeEvent
	{
		public ulong Timestamp { get; }
		public bool IsEnterScope { get; }
		public uint? SpecId { get; }

		public CpuProfilerScopeEvent(ulong timestamp, bool isEnterScope, uint? specId)
		{
			Timestamp = timestamp;
			IsEnterScope = isEnterScope;
			SpecId = specId;
		}

		public override string ToString()
		{
			string name = IsEnterScope ? "EnterScope" : "ExitScope";
			return $"{nameof(CpuProfilerScopeEvent)}({name} {nameof(Timestamp)}={Timestamp} {nameof(SpecId)}={SpecId})";
		}
	}

	/// <summary>
	/// Stateful serializer that can serialize/deserialize the data argument in CpuProfiler.EventBatch events
	/// </summary>
	public class CpuProfilerSerializer
	{
		private readonly ulong _cycleFrequency;
		public List<CpuProfilerScopeEvent> ScopeEvents { get; } = [];

		public CpuProfilerSerializer(ulong cycleFrequency)
		{
			_cycleFrequency = cycleFrequency;
		}

		public void Read(byte[] eventBatchData)
		{
			using MemoryStream ms = new(eventBatchData);
			using BinaryReader reader = new(ms);

			ulong absTimestamp = 0;
			ulong lastTimestamp = 0;

			while (reader.BaseStream.Position != reader.BaseStream.Length)
			{
				ulong value = TraceUtils.Read7BitUint(reader);
				(ulong timestamp, bool isEnterScope) = DecodeTimestamp(value);

				if (absTimestamp == 0)
				{
					absTimestamp = timestamp; // First timestamp is always absolute
				}
				else
				{
					timestamp = lastTimestamp + timestamp * _cycleFrequency; // Make timestamp absolute
				}
				lastTimestamp = timestamp;

				if (isEnterScope)
				{
					ulong specId = TraceUtils.Read7BitUint(reader);
					CpuProfilerScopeEvent scopeEvent = new(timestamp, true, (uint?)specId);
					ScopeEvents.Add(scopeEvent);
				}
				else
				{
					CpuProfilerScopeEvent scopeEvent = new(timestamp, false, null);
					ScopeEvents.Add(scopeEvent);
				}
			}
		}

		public List<byte[]> Write()
		{
			List<MemoryStream> streams = [];
			MemoryStream stream = new();
			BinaryWriter writer = new(stream);
			streams.Add(stream);

			ulong lastAbsTimestamp = 0;
			foreach (CpuProfilerScopeEvent scope in ScopeEvents)
			{
				ulong timestamp;
				if (lastAbsTimestamp == 0)
				{
					timestamp = scope.Timestamp;
				}
				else
				{
					timestamp = (scope.Timestamp - lastAbsTimestamp) / _cycleFrequency;
				}
				lastAbsTimestamp = scope.Timestamp;

				TraceUtils.Write7BitUint(writer, EncodeTimestamp(timestamp, scope.IsEnterScope));
				if (scope.IsEnterScope)
				{
					TraceUtils.Write7BitUint(writer, (ulong)scope.SpecId!);
				}

				// Split buffers after 250 bytes
				if (stream.Position > 250)
				{
					writer.Close();
					stream = new MemoryStream();
					writer = new BinaryWriter(stream);
					lastAbsTimestamp = 0;
				}
			}

			writer.Close();

			return [.. streams.Select(x => x.ToArray())];
		}

		internal static ulong EncodeTimestamp(ulong timestamp, bool isScopeEnter)
		{
			ulong value = (timestamp << 1) | (uint)(isScopeEnter ? 1 : 0);
			return value;
		}

		internal static (ulong Timestamp, bool IsScopeEnter) DecodeTimestamp(ulong value)
		{
			bool isScopeEnter = (value & 1) != 0;
			ulong timestamp = value >> 1; // Strip the IsScopeEnter bit
			return (timestamp, isScopeEnter);
		}
	}
}