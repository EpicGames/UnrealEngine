// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class DiagnosticsSession2Event : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "Diagnostics", "Session2", EventType.FlagImportant | EventType.FlagMaybeHasAux | EventType.FlagNoSync,
			[
				new EventTypeField(0, 0, EventTypeField.TypeAnsiString, "Platform"),
				new EventTypeField(0, 0, EventTypeField.TypeAnsiString, "AppName"),
				new EventTypeField(0, 0, EventTypeField.TypeWideString, "CommandLine"),
				new EventTypeField(0, 0, EventTypeField.TypeWideString, "Branch"),
				new EventTypeField(0, 0, EventTypeField.TypeWideString, "BuildVersion"),
				new EventTypeField(0, 4, EventTypeField.TypeInt32, "Changelist"),
				new EventTypeField(4, 1, EventTypeField.TypeInt8, "ConfigurationType"),
				new EventTypeField(5, 1, EventTypeField.TypeInt8, "TargetType"),
			]);

		public ushort Size => (ushort)(_genericEvent.Size + TraceImportantEventHeader.HeaderSize);
		public EventType Type => EventType;
		private readonly GenericEvent _genericEvent;

		public DiagnosticsSession2Event(string platform, string appName, string commandLine, string branch, string buildVersion, int changeList, int configurationType, int targetType)
		{
			Field[] fields =
			[
				Field.FromString(platform),
				Field.FromString(appName),
				Field.FromString(commandLine),
				Field.FromString(branch),
				Field.FromString(buildVersion),
				Field.FromInt(changeList),
				Field.FromInt(configurationType),
				Field.FromInt(targetType),
			];

			_genericEvent = new GenericEvent(0, fields, EventType);
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			new TraceImportantEventHeader(uid, _genericEvent.Size).Serialize(writer);
			_genericEvent.Serialize(uid, writer);
		}
	}
}