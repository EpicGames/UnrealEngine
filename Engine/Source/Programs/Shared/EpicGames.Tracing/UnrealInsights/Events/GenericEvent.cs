// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class Field
	{
		public bool? Bool { get; }
		public int? Int { get; }
		public long? Long { get; }
		public double? Float { get; }
		public string? String { get; }
		readonly byte[]? _array;
		public byte[]? GetArray()
		{
			return _array;
		}

		private Field(bool value) { Bool = value; }
		private Field(int value) { Int = value; }
		private Field(long value) { Long = value; }
		private Field(float value) { Float = value; }
		private Field(double value) { Float = value; }
		private Field(string value) { String = value; }
		private Field(byte[] value) { _array = value; }
		public static Field FromBool(bool value) => new(value);
		public static Field FromInt(int value) => new(value);
		public static Field FromLong(long value) => new(value);
		public static Field FromFloat(float value) => new(value);
		public static Field FromDouble(double value) => new(value);
		public static Field FromString(string value) => new(value);
		public static Field FromArray(byte[] value) => new(value);
	}

	/// <summary>
	/// Represents a generic event in the stream with fields stored dynamically
	/// </summary>
	public class GenericEvent : ITraceEvent
	{
		public uint Serial { get; }
		readonly Field[] _fields;
		public Field[] GetFields() => _fields;
		public EventType Type => _eventType;
		private readonly EventType _eventType;

		public GenericEvent(uint serial, Field[] fields, EventType eventType)
		{
			Serial = serial;
			_fields = fields;
			_eventType = eventType;
		}

		public ushort Size
		{
			get
			{
				ushort totalSize = 0;
				if (_eventType.HasSerial)
				{
					totalSize += 3; // 24-bit serial
				}

				totalSize += _eventType.GetEventSize();
				if (_eventType.MaybeHasAux())
				{
					for (int i = 0; i < _eventType.Fields.Count; i++)
					{
						EventTypeField fieldType = _eventType.Fields[i];
						if (!fieldType.IsAuxData())
						{
							continue;
						}

						ushort auxDataSize;
						if (fieldType.TypeInfo == EventTypeField.TypeAnsiString)
						{
							auxDataSize = (ushort)_fields[i].String!.Length;
						}
						else if (fieldType.TypeInfo == EventTypeField.TypeWideString)
						{
							auxDataSize = (ushort)(_fields[i].String!.Length * 2);
						}
						else
						{
							auxDataSize = (ushort)_fields[i].GetArray()!.Length;
						}

						totalSize += sizeof(uint); // AuxHeader
						totalSize += auxDataSize;
					}
					totalSize += sizeof(byte); // AuxTerminal UID
				}

				return totalSize;
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public void Serialize(ushort uid, BinaryWriter writer)
		{
			if (_eventType.HasSerial)
			{
				byte serialLow = (byte)(Serial & 0xFF);
				writer.Write(serialLow);
				writer.Write(BinaryReaderExtensions.GetHighWord(Serial));
			}

			for (int i = 0; i < _eventType.Fields.Count; i++)
			{
				EventTypeField fieldType = _eventType.Fields[i];

				if (fieldType.TypeInfo == EventTypeField.TypeInt8)
				{
					writer.Write((byte)_fields[i].Int!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt16)
				{
					writer.Write((ushort)_fields[i].Int!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt32)
				{
					writer.Write((uint)_fields[i].Int!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt64)
				{
					writer.Write((ulong)_fields[i].Long!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypePointer)
				{
					writer.Write((ulong)_fields[i].Long!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeFloat32)
				{
					writer.Write((float)_fields[i].Float!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeFloat64)
				{
					writer.Write((double)_fields[i].Float!);
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeAnsiString)
				{
				} // Write later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeWideString)
				{
				} // Write later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeArray)
				{
				} // Write later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeBool)
				{
					writer.Write(_fields[i].Bool!.Value ? (byte)1 : (byte)0);
				}
				else
				{
					throw new Exception($"Found unknown TypeInfo {fieldType.TypeInfo}");
				}
			}

			if (_eventType.MaybeHasAux())
			{
				for (int i = 0; i < _eventType.Fields.Count; i++)
				{
					EventTypeField fieldType = _eventType.Fields[i];

					if (fieldType.Size > 0)
					{
						continue; // Skip any non-aux fields
					}

					ushort auxUid = PredefinedEventUid.AuxData;
					if (!_eventType.IsImportant())
					{
						// Well-known UIDs are only shifted in non-important events
						auxUid = (ushort)(auxUid << 1);
					}

					byte[] data;
					if (fieldType.TypeInfo == EventTypeField.TypeAnsiString)
					{
						data = Encoding.ASCII.GetBytes(_fields[i].String!);
					}
					else if (fieldType.TypeInfo == EventTypeField.TypeWideString)
					{
						data = Encoding.Unicode.GetBytes(_fields[i].String!);
					}
					else
					{
						data = _fields[i].GetArray()!;
					}

					uint auxHeader = 0;
					auxHeader = SetBits(auxHeader, auxUid, 0, 8);
					auxHeader = SetBits(auxHeader, (uint)i, 8, 5);
					auxHeader = SetBits(auxHeader, (uint)data.Length, 13, 19);

					writer.Write(auxHeader);
					writer.Write(data);
				}

				ushort auxDataTerminalUid = PredefinedEventUid.AuxDataTerminal;

				if (!_eventType.IsImportant())
				{
					// Well-known UIDs are only shifted in non-important events
					auxDataTerminalUid = (ushort)(auxDataTerminalUid << 1);
				}
				writer.Write((byte)auxDataTerminalUid);
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public static GenericEvent Deserialize(ushort uid, BinaryReader reader, EventType eventType)
		{
			Field[] fields = new Field[eventType.Fields.Count];
			uint serial = 0;

			if (!eventType.IsNoSync() && !eventType.IsImportant())
			{
				// Read uint24 as serial
				byte serialLow = reader.ReadByte();
				ushort serialHigh = reader.ReadUInt16();
				serial = serialLow | ((uint)serialHigh << 16);
			}

			for (int i = 0; i < eventType.Fields.Count; i++)
			{
				EventTypeField fieldType = eventType.Fields[i];

				if (fieldType.TypeInfo == EventTypeField.TypeInt8)
				{
					fields[i] = Field.FromInt(reader.ReadByte());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt16)
				{
					fields[i] = Field.FromInt(reader.ReadUInt16());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt32)
				{
					fields[i] = Field.FromInt((int)reader.ReadUInt32());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeInt64)
				{
					fields[i] = Field.FromLong((long)reader.ReadUInt64());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypePointer)
				{
					fields[i] = Field.FromLong((long)reader.ReadUInt64());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeFloat32)
				{
					fields[i] = Field.FromFloat(reader.ReadSingle());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeFloat64)
				{
					fields[i] = Field.FromDouble(reader.ReadDouble());
				}
				else if (fieldType.TypeInfo == EventTypeField.TypeAnsiString)
				{
				} // Read later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeWideString)
				{
				} // Read later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeArray)
				{
				} // Read later as aux-data
				else if (fieldType.TypeInfo == EventTypeField.TypeBool)
				{
					fields[i] = Field.FromBool(reader.ReadByte() == 1);
				}
				else
				{
					throw new Exception($"Found unknown TypeInfo {fieldType.TypeInfo}");
				}
			}

			if (eventType.MaybeHasAux())
			{
				for (; ; )
				{
					ushort auxUid = reader.ReadByte();
					if (!eventType.IsImportant())
					{
						// Well-known UIDs are only shifted in non-important events
						auxUid = (ushort)(auxUid >> 1);
					}

					if (auxUid == PredefinedEventUid.AuxData)
					{
						reader.BaseStream.Position -= 1; // Include the UID for parsing rest of header as a single uint32
						uint auxHeader = reader.ReadUInt32();

						int fieldIndex = (int)ReadBits(auxHeader, 8, 5);
						uint size = ReadBits(auxHeader, 13, 19);

						byte[] auxData = reader.ReadBytesStrict((int)size);

						if (eventType.Fields[fieldIndex].TypeInfo == EventTypeField.TypeAnsiString)
						{
							fields[fieldIndex] = Field.FromString(Encoding.ASCII.GetString(auxData));
						}
						else if (eventType.Fields[fieldIndex].TypeInfo == EventTypeField.TypeWideString)
						{
							fields[fieldIndex] = Field.FromString(Encoding.Unicode.GetString(auxData));
						}
						else
						{
							fields[fieldIndex] = Field.FromArray(auxData);
						}
					}
					else if (auxUid == PredefinedEventUid.AuxDataTerminal)
					{
						break;
					}
					else
					{
						throw new Exception($"Invalid AuxUid found: {auxUid} / 0x{auxUid:X4}");
					}
				}
			}

			return new GenericEvent(serial, fields, eventType);
		}

		private static uint SetBits(uint word, uint value, int pos, int size)
		{
			uint mask = (((uint)1 << size) - 1) << pos;
			word &= ~mask;
			word |= (value << pos) & mask;
			return word;
		}

		private static uint ReadBits(uint word, int pos, int size)
		{
			uint mask = (((uint)1 << size) - 1) << pos;
			return (word & mask) >> pos;
		}

		internal static (ushort Uid, int FieldIndex, int Size) DeserializeAuxHeader(uint header)
		{
			ushort uid = (ushort)ReadBits(header, 0, 8);
			int fieldIndex = (int)ReadBits(header, 8, 5);
			int size = (int)ReadBits(header, 13, 19);
			return (uid, fieldIndex, size);
		}

		internal static uint SerializeAuxHeader(ushort uid, int fieldIndex, int size)
		{
			uint header = 0;
			header = SetBits(header, uid, 0, 8);
			header = SetBits(header, (uint)fieldIndex, 8, 5);
			header = SetBits(header, (uint)size, 13, 19);
			return header;
		}
	}
}