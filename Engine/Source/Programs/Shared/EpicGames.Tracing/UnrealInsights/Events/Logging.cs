// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class LoggingLogMessage : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "Logging", "LogMessage", EventType.FlagMaybeHasAux | EventType.FlagNoSync,
			[
				new EventTypeField(0, 8, EventTypeField.TypeInt64, "LogPoint"),
				new EventTypeField(8, 8, EventTypeField.TypeInt64, "Cycle"),
				new EventTypeField(16, 0, EventTypeField.TypeArray, "FormatArgs")
			]);

		public ushort Size => throw new NotImplementedException();
		public EventType Type => EventType;

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			throw new NotImplementedException();
		}
	}
}