// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class MemoryMemoryScopeEvent : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "Memory", "MemoryScope", EventType.FlagNone,
			[new EventTypeField(0, 4, EventTypeField.TypeInt32, "Id")]);

		public ushort Size => throw new NotImplementedException();
		public EventType Type => EventType;

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			throw new NotImplementedException();
		}
	}
}