// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	[System.Diagnostics.CodeAnalysis.SuppressMessage("CodeQuality", "IDE0052:Remove unread private members")]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Performance", "CA1823:Avoid unused private fields")]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles")]
	public class EventTypeField
	{
		public ushort Offset { get; private set; }
		public ushort Size { get; private set; }
		public byte TypeInfo { get; private set; }
		public byte NameSize { get; private set; }
		public string Name { get; private set; } = "<unknown>";
		public const ushort StructSize = 2 + 2 + 1 + 1;
		private static readonly byte Field_CategoryMask = ToOctalByte("300");
		private static readonly byte Field_Integer = ToOctalByte("000");
		private static readonly byte Field_Float = ToOctalByte("100");
		private static readonly byte Field_Array = ToOctalByte("200");

		private static readonly byte Field_Pow2SizeMask = ToOctalByte("003");
		private static readonly byte Field_8 = ToOctalByte("000");
		private static readonly byte Field_16 = ToOctalByte("001");
		private static readonly byte Field_32 = ToOctalByte("002");
		private static readonly byte Field_64 = ToOctalByte("003");
		private static readonly byte Field_Ptr = ToOctalByte("003"); // Assume 64-bit

		private static readonly byte Field_SpecialMask = ToOctalByte("030");
		private static readonly byte Field_Pod = ToOctalByte("000");
		private static readonly byte Field_String = ToOctalByte("010");

		public static readonly byte TypeBool = (byte)(Field_Pod | Field_Integer | Field_8);
		public static readonly byte TypeInt8 = (byte)(Field_Pod | Field_Integer | Field_8);
		public static readonly byte TypeInt16 = (byte)(Field_Pod | Field_Integer | Field_16);
		public static readonly byte TypeInt32 = (byte)(Field_Pod | Field_Integer | Field_32);
		public static readonly byte TypeInt64 = (byte)(Field_Pod | Field_Integer | Field_64);
		public static readonly byte TypePointer = (byte)(Field_Pod | Field_Integer | Field_Ptr);
		public static readonly byte TypeFloat32 = (byte)(Field_Pod | Field_Float | Field_32);
		public static readonly byte TypeFloat64 = (byte)(Field_Pod | Field_Float | Field_64);
		public static readonly byte TypeAnsiString = (byte)(Field_String | Field_Integer | Field_Array | Field_8);
		public static readonly byte TypeWideString = (byte)(Field_String | Field_Integer | Field_Array | Field_16);
		public static readonly byte TypeArray = (byte)(Field_Array);

		public static byte ToOctalByte(string Value)
		{
			return (byte)Convert.ToInt32(Value, 8);
		}

		public static string TypeInfoToString(byte ByteInfo)
		{
			if (ByteInfo == TypeInt8)
			{
				return "int8";
			}

			if (ByteInfo == TypeInt16)
			{
				return "int16";
			}

			if (ByteInfo == TypeInt32)
			{
				return "int32";
			}

			if (ByteInfo == TypeInt64)
			{
				return "int64";
			}

			if (ByteInfo == TypePointer)
			{
				return "ptr";
			}

			if (ByteInfo == TypeFloat32)
			{
				return "float32";
			}

			if (ByteInfo == TypeFloat64)
			{
				return "float64";
			}

			if (ByteInfo == TypeAnsiString)
			{
				return "ansi_str";
			}

			if (ByteInfo == TypeWideString)
			{
				return "wide_str";
			}

			if (ByteInfo == TypeArray)
			{
				return "array";
			}

			if (ByteInfo == TypeBool)
			{
				return "bool";
			}

			throw new Exception($"Unable to convert type info {ByteInfo} to string");
		}

		public EventTypeField(ushort Offset, ushort Size, byte TypeInfo, byte NameSize)
		{
			this.Offset = Offset;
			this.Size = Size;
			this.TypeInfo = TypeInfo;
			this.NameSize = NameSize;
			ValidateSize();
		}

		public EventTypeField(ushort Offset, ushort Size, byte TypeInfo, string Name)
		{
			this.Offset = Offset;
			this.Size = Size;
			this.TypeInfo = TypeInfo;
			SetName(Name);
			ValidateSize();
		}

		public bool IsArray()
		{
			return (TypeInfo & Field_Array) != 0;
		}

		public bool IsAuxData()
		{
			return IsArray();
		}

		public void SetName(string Name)
		{
			NameSize = (byte)Encoding.UTF8.GetBytes(Name).Length;
			this.Name = Name;
		}

		public void ValidateSize()
		{
			if (TypeInfo == TypeAnsiString || TypeInfo == TypeWideString || TypeInfo == TypeArray)
			{
				if (Size != 0)
				{
					// Array data is encoded in aux-data, following the event
					throw new ArgumentException("Array-based field must have a size of 0");
				}
			}
		}

		public override string ToString()
		{
			return $"Field({Name} Offset={Offset} Size={Size} Type={TypeInfoToString(TypeInfo)} TypeInfo={TypeInfo})";
		}
	}

	public class EventType : ITraceEvent
	{
		public ushort NewEventUid { get; set; }
		public string LoggerName { get; private set; }
		public string EventName { get; private set; }
		public byte Flags { get; private set; }
		public List<EventTypeField> Fields { get; private set; }

		public string Name => $"{LoggerName}.{EventName}";

		private int _eventSize;

		public const byte FlagNone = 0;
		public const byte FlagImportant = 1 << 0;
		public const byte FlagMaybeHasAux = 1 << 1;
		public const byte FlagNoSync = 1 << 2;

		public EventType(ushort newEventUid, string loggerName, string eventName, byte flags, List<EventTypeField> fields)
		{
			NewEventUid = newEventUid;
			LoggerName = loggerName;
			EventName = eventName;
			Flags = flags;
			Fields = fields;
			_eventSize += Fields.Sum(f => f.Size);
		}

		public EventType(string loggerName, string eventName, byte flags)
		{
			NewEventUid = 0;
			LoggerName = loggerName;
			EventName = eventName;
			Flags = flags;
			Fields = [];
		}

		public static EventType WellKnown(ushort uid, string name)
		{
			return new EventType(uid, "WellKnown", name, 0, []);
		}

		private static EventType Self()
		{
			return new EventType(0, "EventType", "Self", 0, []);
		}

		public void AddEventType(ushort offset, ushort size, byte typeInfo, string name)
		{
			EventTypeField field = new(offset, size, typeInfo, name);
			Fields.Add(field);
			_eventSize += field.Size;
		}

		public bool IsWellKnown()
		{
			return NewEventUid < PredefinedEventUid.WellKnownNum;
		}

		public bool IsImportant()
		{
			return (Flags & FlagImportant) != 0;
		}

		public bool MaybeHasAux()
		{
			return (Flags & FlagMaybeHasAux) != 0;
		}

		public bool IsNoSync()
		{
			return (Flags & FlagNoSync) != 0;
		}

		public bool HasSerial => !IsNoSync() && !IsImportant();

		public int NumAuxFields => Fields.FindAll(x => x.IsAuxData()).Count;

		/// <summary>
		/// Size of events as seen in the stream
		/// For size of this NewEvent event, see GetSize()
		/// </summary>
		/// <returns></returns>
		public ushort GetEventSize()
		{
			return (ushort)_eventSize;
		}

		public override string ToString()
		{
			return $"EventType({Name})";
		}

		public string ToStringDetailed()
		{
			string fieldText = String.Join(',', Fields.Select(x => x.ToString()));
			string flagText = "";
			if (IsImportant())
			{
				flagText += ",Important";
			}

			if (MaybeHasAux())
			{
				flagText += ",MaybeHasAux";
			}

			if (IsNoSync())
			{
				flagText += ",NoSync";
			}

			flagText = flagText.Trim(',');
			return $"EventType(Uid={NewEventUid} {LoggerName} {EventName} Flags={flagText} FlagRaw={Flags} Fields={fieldText})";
		}

		public ushort Size
		{
			get
			{
				byte[] loggerNameBytes = Encoding.UTF8.GetBytes(LoggerName);
				byte[] eventNameBytes = Encoding.UTF8.GetBytes(EventName);

				ushort fieldArraySize = (ushort)(Fields.Count * EventTypeField.StructSize);
				ushort namesSize = (ushort)(loggerNameBytes.Length + eventNameBytes.Length + Fields.Sum(x => x.NameSize));
				ushort newEventSize = (ushort)(2 + 1 + 1 + 1 + 1 + fieldArraySize + namesSize);

				newEventSize += TraceImportantEventHeader.HeaderSize;
				return newEventSize;
			}
		}

		public EventType Type => Self();

		public override int GetHashCode()
		{
			return HashCode.Combine(NewEventUid, LoggerName, EventName, Flags);
		}

		protected bool Equals(EventType other)
		{
			return NewEventUid == other.NewEventUid && LoggerName == other.LoggerName && EventName == other.EventName && Flags == other.Flags;
		}

		public override bool Equals(object? obj)
		{
			if (obj is null)
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			if (obj.GetType() != GetType())
			{
				return false;
			}

			return Equals((EventType)obj);
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			new TraceImportantEventHeader(PredefinedEventUid.NewEvent, (ushort)(Size - TraceImportantEventHeader.HeaderSize)).Serialize(writer);

			byte[] loggerNameBytes = Encoding.UTF8.GetBytes(LoggerName);
			byte[] eventNameBytes = Encoding.UTF8.GetBytes(EventName);

			writer.Write(uid); // UID of the new event to declare
			writer.Write((byte)Fields.Count);
			writer.Write(Flags);
			writer.Write((byte)loggerNameBytes.Length);
			writer.Write((byte)eventNameBytes.Length);

			foreach (EventTypeField field in Fields)
			{
				writer.Write(field.Offset);
				writer.Write(field.Size);
				writer.Write(field.TypeInfo);
				writer.Write(field.NameSize);
			}

			writer.Write(loggerNameBytes);
			writer.Write(eventNameBytes);

			foreach (EventTypeField field in Fields)
			{
				writer.Write(Encoding.UTF8.GetBytes(field.Name));
			}
		}

		public static (ushort, EventType) Deserialize(BinaryReader reader)
		{
			ushort newEventUid = reader.ReadUInt16();
			byte fieldCount = reader.ReadByte();
			byte flags = reader.ReadByte();
			byte loggerNameSize = reader.ReadByte();
			byte eventNameSize = reader.ReadByte();

			EventTypeField[] fields = new EventTypeField[fieldCount];

			for (int i = 0; i < fieldCount; i++)
			{
				ushort offset = reader.ReadUInt16();
				ushort size = reader.ReadUInt16();
				byte typeInfo = reader.ReadByte();
				byte nameSize = reader.ReadByte();
				fields[i] = new EventTypeField(offset, size, typeInfo, nameSize);
			}

			string loggerName = Encoding.UTF8.GetString(reader.ReadBytesStrict(loggerNameSize));
			string eventName = Encoding.UTF8.GetString(reader.ReadBytesStrict(eventNameSize));

			for (int i = 0; i < fieldCount; i++)
			{
				fields[i].SetName(Encoding.UTF8.GetString(reader.ReadBytesStrict(fields[i].NameSize)));
			}

			return (newEventUid, new EventType(newEventUid, loggerName, eventName, flags, [.. fields]));
		}
	}
}