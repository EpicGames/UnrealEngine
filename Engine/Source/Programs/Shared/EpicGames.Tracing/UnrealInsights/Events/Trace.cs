// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace EpicGames.Tracing.UnrealInsights.Events
{
	public class TraceNewTraceEvent : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "$Trace", "NewTrace", EventType.FlagImportant | EventType.FlagNoSync,
			[
				new EventTypeField(0, 8, EventTypeField.TypeInt64, "StartCycle"),
				new EventTypeField(8, 8, EventTypeField.TypeInt64, "CycleFrequency"),
				new EventTypeField(16, 2, EventTypeField.TypeInt16, "Endian"),
				new EventTypeField(18, 1, EventTypeField.TypeInt8, "PointerSize")
			]);

		public ulong StartCycle { get; private set; }
		public ulong CycleFrequency { get; private set; }
		public ushort Endian { get; private set; }
		public byte PointerSize { get; private set; }

		public ushort Size => (ushort)(EventType.GetEventSize() + TraceImportantEventHeader.HeaderSize);
		public EventType Type => EventType;

		public TraceNewTraceEvent(ulong startCycle, ulong cycleFrequency, ushort endian, byte pointerSize)
		{
			StartCycle = startCycle;
			CycleFrequency = cycleFrequency;
			Endian = endian;
			PointerSize = pointerSize;
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			new TraceImportantEventHeader(uid, EventType.GetEventSize()).Serialize(writer);
			writer.Write(StartCycle);
			writer.Write(CycleFrequency);
			writer.Write(Endian);
			writer.Write(PointerSize);
		}

		public static TraceNewTraceEvent Deserialize(BinaryReader reader)
		{
			ulong startCycle = reader.ReadUInt64();
			ulong cycleFrequency = reader.ReadUInt64();
			ushort endian = reader.ReadUInt16();
			byte pointerSize = reader.ReadByte();
			return new TraceNewTraceEvent(startCycle, cycleFrequency, endian, pointerSize);
		}
	}

	public class TraceThreadInfoEvent : ITraceEvent
	{
		public static readonly EventType EventType = new(0, "$Trace", "ThreadInfo", EventType.FlagImportant | EventType.FlagMaybeHasAux | EventType.FlagNoSync,
			[
				new EventTypeField(0, 4, EventTypeField.TypeInt32, "ThreadId"),
				new EventTypeField(4, 4, EventTypeField.TypeInt32, "SystemId"),
				new EventTypeField(8, 4, EventTypeField.TypeInt32, "SortHint"),
				new EventTypeField(12, 0, EventTypeField.TypeAnsiString, "Name")
			]);

		public ushort Size => (ushort)(_genericEvent.Size + TraceImportantEventHeader.HeaderSize);
		public EventType Type => EventType;
		private readonly GenericEvent _genericEvent;

		public TraceThreadInfoEvent(int threadId, int systemId, int sortHint, string name)
		{
			Field[] fields =
			[
				Field.FromInt((int) threadId),
				Field.FromInt((int) systemId),
				Field.FromInt((int) sortHint),
				Field.FromString(name),
			];

			_genericEvent = new GenericEvent(0, fields, EventType);
		}

		public void Serialize(ushort uid, BinaryWriter writer)
		{
			new TraceImportantEventHeader(uid, _genericEvent.Size).Serialize(writer);
			_genericEvent.Serialize(uid, writer);
		}
	}
}