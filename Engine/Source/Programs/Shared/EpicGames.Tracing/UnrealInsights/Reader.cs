// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Tracing.UnrealInsights.Events;

namespace EpicGames.Tracing.UnrealInsights
{
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1707:Identifiers should not contain underscores")]
	public static class PredefinedEventUid
	{
		public const ushort NewEvent = 0;
		public const ushort AuxData = 1;
		// public const ushort _AuxData_Unused = 2;
		public const ushort AuxDataTerminal = 3;
		public const ushort EnterScope = 4;
		public const ushort LeaveScope = 5;
		// public const ushort _Unused6 = 6;
		// public const ushort _Unused7 = 7;
		public const ushort EnterScope_T = 8;
		// public const ushort _EnterScope_T_Unused0 = 9;
		// public const ushort _EnterScope_T_Unused1 = 10;
		// public const ushort _EnterScope_T_Unused2 = 11;
		public const ushort LeaveScope_T = 12;
		// public const ushort _LeaveScope_T_Unused0 = 13;
		// public const ushort _LeaveScope_T_Unused1 = 14;
		// public const ushort _LeaveScope_T_Unused2 = 15;
		public const ushort WellKnownNum = 16;
	}

	public class UnrealInsightsReader
	{
		public Dictionary<ushort, EventType> EventTypes { get; } = [];
		public Dictionary<ushort, List<ITraceEvent>> EventsPerThread { get; } = [];

		internal int NumTransportPacketsRead { get; private set; } = 0;

		public UnrealInsightsReader()
		{
			EventTypes[PredefinedEventUid.EnterScope_T] = new EnterScopeEventTimestamp(0).Type;
			EventTypes[PredefinedEventUid.LeaveScope_T] = new LeaveScopeEventTimestamp(0).Type;
		}

		public void Read(Stream stream)
		{
			using BinaryReader reader = new(stream);
			StreamHeader.Deserialize(reader);

			Dictionary<ushort, MemoryStream> threadStreams = Demux(reader);
			reader.EnsureEntireStreamIsConsumed(); // There should be no more trailing bytes once all packets are consumed.

			void ReadThreadStream(ushort threadId)
			{
				MemoryStream threadStream = threadStreams[threadId];
				using BinaryReader threadReader = new(threadStream);
				EventsPerThread[threadId] = ReadEvents(threadId, threadReader);
				threadReader.EnsureEntireStreamIsConsumed();
				threadStream.Dispose();
				threadStreams.Remove(threadId);
			}

			// Read the NewEvents first to ensure event types are registered
			ReadThreadStream(TransportPacket.ThreadIdEvents);

			// Read the important events next
			ReadThreadStream(TransportPacket.ThreadIdImportants);

			foreach (ushort threadId in threadStreams.Keys)
			{
				ReadThreadStream(threadId);
			}
		}

		/// <summary>
		/// Demultiplex the stream into one continuous buffer per thread.
		/// This avoids dealing with blocks that span multiple transport packets.
		///
		/// It's a naive and simple enough for our parsing purposes but it definitely won't read larger trace files.
		/// </summary>
		/// <param name="reader">Reader to consume</param>
		/// <returns>A stream of data per thread</returns>
		public Dictionary<ushort, MemoryStream> Demux(BinaryReader reader)
		{
			Dictionary<ushort, MemoryStream> threadStreams = [];
			while (reader.BaseStream.Position < reader.BaseStream.Length)
			{
				TransportPacket packet = TransportPacket.Deserialize(reader);

				if (!threadStreams.TryGetValue(packet.GetThreadId(), out MemoryStream? threadStream))
				{
					threadStream = new MemoryStream(5 * 1024 * 1024);
					threadStreams[packet.GetThreadId()] = threadStream;
				}

				threadStream.Write(packet.GetData());
				NumTransportPacketsRead++;
			}
			reader.EnsureEntireStreamIsConsumed();

			// Reset stream positions so it can be read from the beginning once returned
			threadStreams.Values.ToList().ForEach(ms => ms.Position = 0);

			return threadStreams;
		}

		public List<ITraceEvent> ReadEvents(int threadId, BinaryReader reader)
		{
			List<ITraceEvent> eventsRead = [];
			while (reader.BaseStream.Position != reader.BaseStream.Length)
			{
				ushort uid;
				if (threadId == TransportPacket.ThreadIdEvents || threadId == TransportPacket.ThreadIdImportants)
				{
					uid = reader.ReadUInt16();
					reader.ReadUInt16(); // TODO: validate and respect the size
				}
				else
				{
					uid = reader.ReadPackedUid(out _);
				}

				if (uid >= PredefinedEventUid.WellKnownNum)
				{
					if (!EventTypes.TryGetValue(uid, out EventType? eventType))
					{
						throw new Exception($"No event type registered for UID {uid} / 0x{uid:X4}");
					}

					GenericEvent @event = GenericEvent.Deserialize(uid, reader, eventType);
					eventsRead.Add(@event);
				}
				else
				{
					switch (uid)
					{
						case PredefinedEventUid.NewEvent:
							if (threadId != TransportPacket.ThreadIdEvents)
							{
								throw new Exception("NewEvents are only allowed on thread ID " + TransportPacket.ThreadIdEvents);
							}

							(ushort newEventUid, EventType newEvent) = EventType.Deserialize(reader);
							EventTypes[newEventUid] = newEvent;
							break;

						case PredefinedEventUid.EnterScope:
							break;

						case PredefinedEventUid.LeaveScope:
							break;

						case PredefinedEventUid.EnterScope_T:
							reader.BaseStream.Position -= 1; // Reset so UID can be read inside event's deserialize method
							eventsRead.Add(EnterScopeEventTimestamp.Deserialize(reader));
							break;

						case PredefinedEventUid.LeaveScope_T:
							reader.BaseStream.Position -= 1; // Reset so UID can be read inside event's deserialize method
							eventsRead.Add(LeaveScopeEventTimestamp.Deserialize(reader));
							break;

						default:
							throw new Exception($"Cannot handle unknown UID {uid}/0x{uid:X4}");
					}
				}
			}

			return eventsRead;
		}

		public Dictionary<ushort, List<ITraceEvent>> GetEventsPerUid()
		{
			Dictionary<ushort, List<ITraceEvent>> eventsPerUid = [];

			foreach ((ushort _, List<ITraceEvent> threadEvents) in EventsPerThread)
			{
				foreach (ITraceEvent @event in threadEvents)
				{
					ushort uid = EventTypes.First(x => @event.Type.Name == x.Value.Name).Key;
					if (!eventsPerUid.TryGetValue(uid, out List<ITraceEvent>? events))
					{
						events = [];
						eventsPerUid[uid] = events;
					}
					events.Add(@event);
				}
			}

			return eventsPerUid;
		}

		public void PrintEventSummary()
		{
			foreach ((ushort uid, List<ITraceEvent> events) in GetEventsPerUid())
			{
				Console.WriteLine($"{uid,4} {EventTypes[uid].Name,-45} {events.Count}");
			}
		}
	}
}