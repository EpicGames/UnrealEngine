// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace EpicGames.Tracing.UnrealInsights
{
	public class StreamHeader
	{
		public const uint MagicTrc = 1414677317; // TRCE
		public const uint MagicTrc2 = 1414677298; // TRC2

		uint _magicFourCc;
		ushort _metadataSize;
		ushort _metadataField0;
		ushort _controlPort;

		byte _transportVersion;
		byte _protocolVersion;

		private StreamHeader()
		{
		}

		public static StreamHeader Default()
		{
			StreamHeader handshake = new();
			handshake._magicFourCc = MagicTrc;
			handshake._transportVersion = 3;
			handshake._protocolVersion = 5;
			return handshake;
		}

		public void Serialize(BinaryWriter writer)
		{
			if (_magicFourCc != MagicTrc && _magicFourCc != MagicTrc2)
			{
				throw new ArgumentException("Only support magic number TRCE and TRC2");
			}
			writer.Write(_magicFourCc);
			if (_magicFourCc == MagicTrc2)
			{
				if (_metadataSize != 4)
				{
					throw new ArgumentException("Only support metadata size of 4 bytes (got " + _metadataSize + ")");
				}
				writer.Write(_metadataSize);
				writer.Write(_metadataField0);
				writer.Write(_controlPort);
			}
			writer.Write(_transportVersion);
			writer.Write(_protocolVersion);
		}

		public static StreamHeader Deserialize(BinaryReader reader)
		{
			StreamHeader header = new();
			header._magicFourCc = reader.ReadUInt32();
			if (header._magicFourCc != MagicTrc && header._magicFourCc != MagicTrc2)
			{
				throw new ArgumentException("Only support magic number TRCE and TRC2");
			}

			if (header._magicFourCc == MagicTrc2)
			{
				header._metadataSize = reader.ReadUInt16();
				if (header._metadataSize != 4)
				{
					throw new ArgumentException("Only support metadata size of 4 bytes (got " + header._metadataSize + ")");
				}

				header._metadataField0 = reader.ReadUInt16();
				header._controlPort = reader.ReadUInt16();
			}

			header._transportVersion = reader.ReadByte();
			header._protocolVersion = reader.ReadByte();
			return header;
		}
	}
}