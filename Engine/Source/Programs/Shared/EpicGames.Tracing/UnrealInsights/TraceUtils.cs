// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace EpicGames.Tracing.UnrealInsights
{
	public static class TraceUtils
	{
		public static ulong Read7BitUint(BinaryReader reader)
		{
			ulong value = 0;
			ulong byteIndex = 0;
			bool hasMoreBytes;

			do
			{
				byte byteValue = reader.ReadByte();
				hasMoreBytes = (byteValue & 0x80) != 0;
				value |= (ulong)(byteValue & 0x7f) << (int)(byteIndex * 7);
				++byteIndex;
			} while (hasMoreBytes);

			return value;
		}

		public static int Write7BitUint(BinaryWriter writer, ulong value)
		{
			int numBytesWritten = 0;
			do
			{
				byte hasMoreBytesBit = (byte)(value > 0x7F ? 1 : 0);
				byte hasMoreBytes = (byte)(hasMoreBytesBit << 7);
				byte byteToWrite = (byte)((value & 0x7F) | hasMoreBytes);

				writer.Write(byteToWrite);
				value >>= 7;
				numBytesWritten++;

			} while (value > 0);

			return numBytesWritten;
		}
	}
}