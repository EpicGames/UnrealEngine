// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace EpicGames.Tracing.UnrealInsights
{
	// Handles FTidPacketBase, TTidPacket, TTidPacketEncoded
	public class TransportPacket
	{
		public const int ThreadIdEvents = 0;
		public const int ThreadIdInternal = 1;
		public const int ThreadIdImportants = ThreadIdInternal;

		private const ushort EncodedMarker = 0x8000;
		private const ushort PartialMarker = 0x4000;
		private const ushort ThreadIdMask = PartialMarker - 1;

		public ushort PacketSize { get; private set; }
		public ushort ThreadIdAndMarkers { get; private set; }
		public ushort DecodedSize { get; private set; }
		byte[] _data = [];
		public byte[] GetData()
		{
			return _data;
		}

		private TransportPacket()
		{
		}

		public bool IsEncoded()
		{
			return (ThreadIdAndMarkers & EncodedMarker) != 0;
		}

		public bool IsPartial()
		{
			return (ThreadIdAndMarkers & PartialMarker) != 0;
		}

		public ushort GetThreadId()
		{
			return (ushort)(ThreadIdAndMarkers & ThreadIdMask);
		}

		public static bool IsNormalThread(ushort threadId)
		{
			if (threadId == ThreadIdEvents)
			{
				return false;
			}

			if (threadId == ThreadIdInternal)
			{
				return false;
			}

			if (threadId == ThreadIdImportants)
			{
				return false;
			}

			return true;
		}

		public void Serialize(BinaryWriter writer, IEnumerable<(ushort, ITraceEvent)> events)
		{
			ushort totalSize = (ushort)events.Sum(x =>
			{
				ITraceEvent @event = x.Item2;
				return @event.Size;
			});
			totalSize += 4; // The two uint16 writes below are included

			writer.Write(totalSize);
			writer.Write(ThreadIdAndMarkers);

			foreach ((ushort uid, ITraceEvent @event) in events)
			{
				@event.Serialize(uid, writer);
			}
		}

		public override string ToString()
		{
			return $"TransportPacket(ThreadId={GetThreadId()} PacketSize={PacketSize} IsEncoded={IsEncoded()} IsPartial={IsPartial()})";
		}

		public static TransportPacket Create(ushort packetSize, ushort threadId)
		{
			TransportPacket packet = new();
			packet.PacketSize = packetSize;
			packet.ThreadIdAndMarkers = threadId;
			return packet;
		}

		public static TransportPacket Deserialize(BinaryReader reader)
		{
			TransportPacket packet = new();
			packet.PacketSize = reader.ReadUInt16();
			packet.ThreadIdAndMarkers = reader.ReadUInt16();

			int headerSize = sizeof(ushort) + sizeof(ushort); // PacketSize + ThreadId
			if (packet.IsEncoded())
			{
				packet.DecodedSize = reader.ReadUInt16();
				headerSize += sizeof(ushort); // sizeof(DecodedSize) 
			}

			int bytesToRead = packet.PacketSize - headerSize;
			packet._data = reader.ReadBytesStrict(bytesToRead);
			return packet;
		}
	}
}