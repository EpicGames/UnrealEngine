// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Tracing.UnrealInsights.Events;

namespace EpicGames.Tracing.UnrealInsights
{
	public class UnrealInsightsWriter
	{
		readonly Dictionary<EventType, ushort> _eventTypeToUids = [];
		readonly Dictionary<ushort, EventType> _uidToEventTypes = [];
		readonly Dictionary<ushort, List<ITraceEvent>> _threadToEvents = [];

		private ushort _uidCounter = PredefinedEventUid.WellKnownNum; // IDs below 16 are reserved for well-known events
		private readonly Lock _lock = new();

		public UnrealInsightsWriter()
		{
			_threadToEvents[TransportPacket.ThreadIdEvents] = [];

			EnterScopeEvent enterScopeEvent = new();
			RegisterEventType(PredefinedEventUid.EnterScope, enterScopeEvent.Type);

			LeaveScopeEvent leaveScopeEvent = new();
			RegisterEventType(PredefinedEventUid.LeaveScope, leaveScopeEvent.Type);

			EnterScopeEvent enterScopeTimestampEvent = new();
			RegisterEventType(PredefinedEventUid.EnterScope_T, enterScopeTimestampEvent.Type);

			LeaveScopeEventTimestamp leaveScopeTimestampEvent = new(0);
			RegisterEventType(PredefinedEventUid.LeaveScope_T, leaveScopeTimestampEvent.Type);
		}

		public void AddEvent(ushort threadId, ITraceEvent @event)
		{
			if (threadId == TransportPacket.ThreadIdEvents)
			{
				throw new ArgumentException("Cannot add events directly to new-event thread");
			}

			lock (_lock)
			{
				if (!_threadToEvents.TryGetValue(threadId, out List<ITraceEvent>? events))
				{
					events = [];
					_threadToEvents[threadId] = events;
				}

				if (!_eventTypeToUids.ContainsKey(@event.Type))
				{
					ushort newUid = _uidCounter++;
					RegisterEventType(newUid, @event.Type);
					_threadToEvents[TransportPacket.ThreadIdEvents].Add(@event);
				}

				events.Add(@event);
			}
		}

		private void RegisterEventType(ushort uid, EventType eventType)
		{
			_uidToEventTypes[uid] = eventType;
			_eventTypeToUids[eventType] = uid;
		}

		public void Write(BinaryWriter writer)
		{
			lock (_lock)
			{
				WriteHeader(writer);
				WriteThread(TransportPacket.ThreadIdImportants, writer);

				foreach (ushort threadId in _threadToEvents.Keys)
				{
					if (threadId == TransportPacket.ThreadIdEvents || threadId == TransportPacket.ThreadIdImportants)
					{
						continue;
					}

					WriteThread(threadId, writer);
				}
			}
		}

		private void WriteHeader(BinaryWriter writer)
		{
			StreamHeader.Default().Serialize(writer);
			TransportPacket transportPacket = TransportPacket.Create(0, 0);
			IEnumerable<(ushort Uid, ITraceEvent Event)> newEvents = _threadToEvents[TransportPacket.ThreadIdEvents].Select(e =>
			{
				ushort newEventUid = _eventTypeToUids[e.Type];
				ITraceEvent myEventType = e.Type;
				return (newEventUid, myEventType);
			});
			transportPacket.Serialize(writer, newEvents);
		}

		private void WriteThread(ushort threadId, BinaryWriter writer)
		{
			foreach (ITraceEvent @event in _threadToEvents[threadId])
			{
				ushort uid = _eventTypeToUids[@event.Type];
				TransportPacket transportPacket = TransportPacket.Create(0, threadId);
				transportPacket.Serialize(writer, [(uid, @event)]);
			}
		}
	}
}