<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
    <PackageReadmeFile>README.md</PackageReadmeFile>
  </PropertyGroup>

  <ItemGroup Condition="'$(DesignTimeBuild)' == 'true'">
    <Compile Include="../../../../Restricted/*/Source/Programs/Shared/EpicGames.UBA/**/*.cs" Link="Restricted/$([System.String]::Copy('%(RecursiveDir)').Replace('\', '/').Replace('Source/Programs/Shared/EpicGames.UBA/', ''))%(Filename)%(Extension)" />
  </ItemGroup>

  <ItemGroup Condition="'$(DesignTimeBuild)' != 'true'">
    <Compile Include="../../../../Restricted/*/Source/Programs/Shared/EpicGames.UBA/**/*.cs" />
  </ItemGroup>

  <ItemGroup>
    <None Include="docs\README.md" Pack="true" PackagePath="\" />
  </ItemGroup>

  <Import Project="Library.props" />

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Logging" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>
</Project>
