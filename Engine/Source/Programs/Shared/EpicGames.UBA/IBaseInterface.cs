// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.UBA
{
	/// <summary>
	/// Base interface for all classes that have unmanaged resources.
	/// </summary>
	public interface IBaseInterface : IDisposable
	{
		/// <summary>
		/// Returns the handle to an unmanaged object
		/// </summary>
		/// <returns>An unmanaged handle</returns>
		public abstract IntPtr GetHandle();

		static IBaseInterface()
		{
			Utils.IsAvailable();
		}
	}
}