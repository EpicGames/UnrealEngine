// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Struct containing results from artifact fetch
	/// </summary>
	/// <param name="Success">Is set to true if succeeded in fetching artifacts</param>
	/// <param name="LogLines">Contains log lines if any</param>
	public readonly record struct FetchFromCacheResult(bool Success, List<string> LogLines);

	/// <summary>
	/// Base interface for a cache client
	/// </summary>
	public interface ICacheClient : IBaseInterface
	{
		/// <summary>
		/// Connect to cache server
		/// </summary>
		/// <param name="host">Cache server address</param>
		/// <param name="port">Cache server port</param>
		/// <param name="desiredConnectionCount">Number of tcp connections we want</param>
		/// <returns>True if successful</returns>
		public abstract bool Connect(string host, int port, int desiredConnectionCount = 1);

		/// <summary>
		/// Disconnect from cache server
		/// </summary>
		public abstract void Disconnect();

		/// <summary>
		/// Traverse all path hashes that have been registered and add them to the cache server. Note, this should be done just before processing actions
		/// </summary>
		public abstract void PopulatePathHashes();

		/// <summary>
		/// Write to cache
		/// </summary>
		/// <param name="bucket">Bucket to store cache entry</param>
		/// <param name="processHandle">Process</param>
		/// <param name="inputs">Input files</param>
		/// <param name="inputsSize">Input files size</param>
		/// <param name="outputs">Output files</param>
		/// <param name="outputsSize">Output files size</param>
		/// <returns>True if successful</returns>
		public abstract bool WriteToCache(uint bucket, nint processHandle, byte[] inputs, uint inputsSize, byte[] outputs, uint outputsSize);

		/// <summary>
		/// Fetch from cache
		/// </summary>
		/// <param name="rootsHandle">handle for roots</param>
		/// <param name="bucket">Bucket to search for cache entry</param>
		/// <param name="info">Process start info</param>
		/// <returns>True if successful</returns>
		public abstract FetchFromCacheResult FetchFromCache(ulong rootsHandle, uint bucket, ProcessStartInfo info);

		/// <summary>
		/// Request the connected server to shutdown
		/// </summary>
		/// <param name="reason">Reason for shutdown</param>
		public abstract void RequestServerShutdown(string reason);

		/// <summary>
		/// Create a ICacheClient object
		/// </summary>
		/// <param name="session">The session</param>
		/// <param name="reportMissReason">Output reason for cache miss to log.</param>
		/// <param name="crypto">Enable crypto by using a 32 character crypto string (representing a 16 byte value)</param>
		/// <param name="hint">string that will show in cache server log</param>
		/// <returns>The ICacheClient</returns>
		public static ICacheClient CreateCacheClient(ISessionServer session, bool reportMissReason, string crypto = "", string hint = "")
		{
			return new CacheClientImpl(session, reportMissReason, crypto, hint);
		}
	}
}
