// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Base interface for uba config file
	/// </summary>
	public interface IConfig : IBaseInterface
	{
		/// <summary>
		/// Load a config file
		/// </summary>
		/// <param name="configFile">The name of the config file</param>
		/// <returns>The IConfig</returns>
		public static IConfig LoadConfig(string configFile)
		{
			return new ConfigImpl(configFile);
		}

		/// <summary>
		/// Load a config file
		/// </summary>
		/// <param name="table">The name of the table where the property is added</param>
		/// <param name="name">The name of the property</param>
		/// <param name="value">The value of the property</param>
		void AddValue(string table, string name, string value);
	}
}