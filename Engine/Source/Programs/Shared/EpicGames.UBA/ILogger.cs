// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// The verbosity of a log entry
	/// </summary>
	public enum LogEntryType
	{
		/// <summary>
		/// Error verbosity
		/// </summary>
		Error = 0,

		/// <summary>
		/// Warning verbosity
		/// </summary>
		Warning = 1,

		/// <summary>
		/// Info verbosity
		/// </summary>
		Info = 2,

		/// <summary>
		/// Info verbosity
		/// </summary>
		Detail = 3,

		/// <summary>
		/// Info verbosity
		/// </summary>
		Debug = 4,
	}

	/// <summary>
	/// Base interface for logging functionality
	/// </summary>
	public interface ILogger : IBaseInterface
	{
		/// <summary>
		/// Begin logging scope
		/// </summary>
		public abstract void BeginScope();

		/// <summary>
		/// End logging scope
		/// </summary>
		public abstract void EndScope();

		/// <summary>
		/// Log message
		/// </summary>
		/// <param name="type">entry verbosity</param>
		/// <param name="message">the message to log</param>
		public abstract void Log(LogEntryType type, string message);

		/// <summary>
		/// Create a ILogger object
		/// </summary>
		/// <param name="logger">The Microsoft.Extensions.Logging.ILogger to wrap</param>
		/// <param name="makeGlobal">Set this log as the one to be used by all sessions</param>
		/// <returns>The ILogger</returns>
		public static ILogger CreateLogger(Microsoft.Extensions.Logging.ILogger logger, bool makeGlobal)
		{
			try
			{
				return new LoggerImpl(logger, makeGlobal);
			}
			catch
			{
				return new NullLogger();
			}
		}

		/// <summary>
		/// Creates null logger
		/// </summary>
		public static ILogger CreateNullLogger() => new NullLogger();

		/// <summary>
		/// Get the global trace if there is one
		/// </summary>
		public static ILogger? GlobalLogger => LoggerImpl.GlobalLoggerImpl;
	}
}