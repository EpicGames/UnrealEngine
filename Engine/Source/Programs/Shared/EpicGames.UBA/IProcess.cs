// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Information needed to create a process
	/// </summary>
	public struct ProcessStartInfo
	{
		/// <summary>
		/// Common configs for processes to run
		/// </summary>
		public enum CommonProcessConfigs
		{
			/// <summary>
			/// MSVC based compiler
			/// </summary>
			CompileMsvc,

			/// <summary>
			/// Clang based compiler
			/// </summary>
			CompileClang,
		}

		/// <summary>
		/// The path to the application binary
		/// </summary>
		public string Application { get; set; }

		/// <summary>
		/// The working directory
		/// </summary>
		public string WorkingDirectory { get; set; }

		/// <summary>
		/// The command line arguments
		/// </summary>
		public string Arguments { get; set; }

		/// <summary>
		/// A text description of the process
		/// </summary>
		public string Description { get; set; }

		/// <summary>
		/// Which configuration to use
		/// </summary>
		public CommonProcessConfigs Configuration { get; set; }

		/// <summary>
		/// The process priority of the created process
		/// </summary>
		public ProcessPriorityClass Priority { get; set; }

		/// <summary>
		/// If input should be tracked
		/// </summary>
		public bool TrackInputs { get; set; }

		/// <summary>
		/// A path to a log file, or null for not log file
		/// </summary>
		public string? LogFile { get; set; }

		/// <summary>
		/// Arbitary user data to pass along with the process
		/// </summary>
		public object? UserData { get; set; }

		/// <summary>
		/// RootsHandle for process used to convert paths
		/// </summary>
		public ulong RootsHandle { get; set; }
	}

	/// <summary>
	/// Base interface for process start info
	/// </summary>
	internal interface IProcessStartInfo : IBaseInterface
	{
		/// <summary>
		/// Create a IProcessStartInfo object
		/// </summary>
		/// <param name="info">The start info for the process</param>
		/// <param name="useExitedCallback">Set to true if exit callback is used</param>
		/// <returns>The IProcessStartInfo</returns>
		public static IProcessStartInfo CreateProcessStartInfo(ProcessStartInfo info, bool useExitedCallback)
		{
			return new ProcessStartInfoImpl(info, useExitedCallback);
		}
	}

	/// <summary>
	/// Event args for exited event
	/// </summary>
	/// <param name="process">The process to pull data from</param>
	public class ExitedEventArgs(IProcess process) : EventArgs
	{
		/// <summary>
		/// Process exit code
		/// </summary>
		public int ExitCode { get; } = process.ExitCode;

		/// <summary>
		/// The remote host that ran the process, if run remotely
		/// </summary>
		public string? ExecutingHost { get; } = process.ExecutingHost;

		/// <summary>
		/// Captured output lines
		/// </summary>
		public List<string> LogLines { get; } = process.LogLines;

		/// <summary>
		/// Total time spent for the processor
		/// </summary>
		public TimeSpan TotalProcessorTime { get; } = process.TotalProcessorTime;

		/// <summary>
		/// Total wall time spent
		/// </summary>
		public TimeSpan TotalWallTime { get; } = process.TotalWallTime;

		/// <summary>
		/// Total wall time spent
		/// </summary>
		public object? UserData { get; } = process.UserData;
	}

	/// <summary>
	/// Interface for a process instance
	/// </summary>
	public interface IProcess : IBaseInterface
	{
		/// <summary>
		/// Delegate for Exited events
		/// </summary>
		/// <param name="sender">The sender object</param>
		/// <param name="e">The event args</param>
		public delegate void ExitedEventHandler(object sender, ExitedEventArgs e);

		/// <summary>
		/// Exited event handler
		/// </summary>
		public abstract event ExitedEventHandler? Exited;

		/// <summary>
		/// Process exit code
		/// </summary>
		public abstract int ExitCode { get; }

		/// <summary>
		/// The remote host that ran the process, if run remotely
		/// </summary>
		public abstract string? ExecutingHost { get; }

		/// <summary>
		/// Captured output lines
		/// </summary>
		public abstract List<string> LogLines { get; }

		/// <summary>
		/// Total time spent for the processor
		/// </summary>
		public abstract TimeSpan TotalProcessorTime { get; }

		/// <summary>
		/// Total wall time spent
		/// </summary>
		public abstract TimeSpan TotalWallTime { get; }

		/// <summary>
		/// Unique hash for this process (not stable between runs)
		/// </summary>
		public abstract ulong Hash { get; }

		/// <summary>
		/// Arbitary user data
		/// </summary>
		public abstract object? UserData { get; }

		/// <summary>
		/// Cancel the running process
		/// </summary>
		/// <param name="terminate">If the process should be force terminated</param>
		public abstract void Cancel(bool terminate);

		/// <summary>
		/// Create a IProcess object
		/// </summary>
		/// <param name="handle">unmanaged pointer to the process</param>
		/// <param name="info">the processes start info</param>
		/// <param name="exitedEventHandler">Optional callback when the process exits</param>
		/// <param name="userData">Arbitary user data</param>
		/// <returns>The IProcess</returns>
		internal static IProcess CreateProcess(IntPtr handle, IProcessStartInfo info, ExitedEventHandler? exitedEventHandler, object? userData)
		{
			return new ProcessImpl(handle, info, exitedEventHandler, userData);
		}
	}
}
