// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Type of execution used for process
	/// </summary>
	public enum ProcessExecutionType
	{
		/// <summary>
		/// Process executed locally without detouring
		/// </summary>
		Native,

		/// <summary>
		/// Process executed locally with detouring enabled
		/// </summary>
		Local,

		/// <summary>
		/// Process executed on a remote session
		/// </summary>
		Remote,

		/// <summary>
		/// Process was never executed and instead downloaded from cache
		/// </summary>
		Cache,

		/// <summary>
		/// Process was skipped and never executed
		/// </summary>
		Skip,
	}

	/// <summary>
	/// </summary>
	public interface IProcessFinishedInfo
	{
		/// <summary>
		/// Type of execution
		/// </summary>
		ProcessExecutionType ExecutionType { get; }

		/// <summary>
		/// Process exit code
		/// </summary>
		int ExitCode { get; }

		/// <summary>
		/// Captured output lines
		/// </summary>
		List<string> LogLines { get; }

		/// <summary>
		/// The remote host that ran the process, if run remotely
		/// </summary>
		string? ExecutingHost { get; }

		/// <summary>
		/// Total time spent for the processor
		/// </summary>
		TimeSpan TotalProcessorTime { get; }

		/// <summary>
		/// Total wall time spent
		/// </summary>
		TimeSpan TotalWallTime { get; }

		/// <summary>
		/// Peak memory used, requires a job object so will only be non-zero for Windows hosts.
		/// </summary>
		ulong PeakMemoryUsed { get; }

		/// <summary>
		/// UserData that was provided in EnqueueProcess
		/// </summary>
		object UserData { get; }

		/// <summary>
		/// Native uba handle to process.
		/// </summary>
		nint ProcessHandle { get; }
	}

	/// <summary>
	/// </summary>
	public enum ProcessFinishedResponse
	{
		/// <summary>
		/// None means that nothing should be done
		/// </summary>
		None,
		/// <summary>
		/// RerunLocal means that we want to re-run the process locally with detouring enabled
		/// </summary>
		RerunLocal,
		/// <summary>
		/// RerunNative means that we want to re-run the process locally without detouring
		/// </summary>
		RerunNative,
	}

	/// <summary>
	/// Base interface for uba config file
	/// </summary>
	public interface IScheduler : IBaseInterface
	{

		/// <summary>
		/// Start the scheduler. It will start processing enqueued processes straight away
		/// </summary>
		void Start();

		/// <summary>
		/// Queue process.
		/// </summary>
		uint EnqueueProcess(ProcessStartInfo info, double weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, int[]? dependencies, byte[]? knownInputs, uint knownInputsCount, uint cacheBucket, uint memoryGroup, ulong predictedMemoryUsage, object userData);

		/// <summary>
		/// Cancel all active processes and skip queued ones
		/// </summary>
		void Cancel();

		/// <summary>
		/// Returns true if no processes are running or queued
		/// </summary>
		bool IsEmpty { get; }

		/// <summary>
		/// Accumulated weight of all processes that are Queued and can run right now (dependencies are done)
		/// </summary>
		void GetProcessWeightThatCanRunRemotelyNow(out double totalWeight, out double crossArchitecture, out double crossPlatform);

		/// <summary>
		/// Set callback for when process has finished
		/// </summary>
		void SetProcessFinishedCallback(Func<IProcessFinishedInfo, ProcessFinishedResponse> processFinished);

		/// <summary>
		/// Allows uba to disable remote execution if running out of processes that can execute remotely
		/// </summary>
		void SetAllowDisableRemoteExecution();

		/// <summary>
		/// Create a scheduler
		/// </summary>
		/// <param name="session">Session</param>
		/// <param name="cacheClients">List of cache clients</param>
		/// <param name="config">Config for scheduler</param>
		/// <param name="maxLocalProcessors">Max number of local processes scheduler can run in parallel</param>
		/// <param name="forceRemote">Force all processes that can to run remote</param>
		public static IScheduler CreateScheduler(ISessionServer session, IEnumerable<ICacheClient> cacheClients, IConfig config, int maxLocalProcessors, bool forceRemote)
		{
			return new SchedulerImpl(session, cacheClients, config, maxLocalProcessors, forceRemote);
		}
	}
}