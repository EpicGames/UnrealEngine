// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Base interface for a server instance
	/// </summary>
	public interface IServer : IBaseInterface
	{
		/// <summary>
		/// Start the server
		/// </summary>
		/// <param name="ip">Ip address or host name</param>
		/// <param name="port">The port to use, -1 for default</param>
		/// <param name="crypto">Enable crypto by using a 32 character crypto string (representing a 16 byte value)</param>
		/// <returns></returns>
		public abstract bool StartServer(string ip = "", int port = -1, string crypto = "");

		/// <summary>
		/// Stop the server
		/// </summary>
		public abstract void StopServer();

		/// <summary>
		/// Add a named connection to the server
		/// </summary>
		/// <param name="name">The name of the connection</param>
		/// <returns>Success</returns>
		public abstract bool AddNamedConnection(string name);

		/// <summary>
		/// Adds a client that server will try to connect one or more tcp connections to
		/// </summary>
		/// <param name="ip">The ip of the listening client</param>
		/// <param name="port">The port of the listening client</param>
		/// <param name="crypto">Enable crypto by using a 32 character crypto string (representing a 16 byte value)</param>
		/// <returns>Success</returns>
		public abstract bool AddClient(string ip, int port, string crypto = "");

		/// <summary>
		/// Set config (.toml) that should be used by all clients.
		/// </summary>
		/// <param name="config"></param>
		public abstract void SetClientsConfig(IConfig config);

		/// <summary>
		/// Create a IServer object
		/// </summary>
		/// <param name="maxWorkers">Maximum number of workers</param>
		/// <param name="sendSize">Send size in bytes</param>
		/// <param name="logger">The logger</param>
		/// <param name="useQuic">Use Quic protocol instead of tcp for communication between host and helpers</param>
		/// <returns>The IServer</returns>
		public static IServer CreateServer(int maxWorkers, int sendSize, ILogger logger, bool useQuic)
		{
			return new ServerImpl(maxWorkers, sendSize, logger, useQuic);
		}

		/// <summary>
		/// Create a null server
		/// </summary>
		public static IServer CreateNullServer() => new NullServer();
	}
}
