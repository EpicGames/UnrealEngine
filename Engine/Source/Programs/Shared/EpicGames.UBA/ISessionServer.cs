// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Information needed to create a session server
	/// </summary>
	/// <param name="rootDirectory">Root directory to store content addressable data</param>
	/// <param name="traceOutputFile">Path to a trace file that records the build</param>
	/// <param name="disableCustomAllocator">If the custom allocator should be disabled</param>
	/// <param name="launchVisualizer">If the visualizer should be launched</param>
	/// <param name="resetCas">If the content addressable storage should be reset</param>
	/// <param name="writeToDisk">If intermediate/output files should be written to disk</param>
	/// <param name="detailedTrace">More detailed trace information</param>
	/// <param name="allowWaitOnMem">Wait for memory before starting new processes</param>
	/// <param name="allowKillOnMem">Kill processes when close to run out of memory</param>
	/// <param name="storeObjFilesCompressed">Store .obj files compressed on disk</param>
	public readonly struct SessionServerCreateInfo(string rootDirectory, string traceOutputFile, bool disableCustomAllocator, bool launchVisualizer, bool resetCas, bool writeToDisk, bool detailedTrace, bool allowWaitOnMem, bool allowKillOnMem, bool storeObjFilesCompressed)
	{
		/// <summary>
		/// Root directory to store content addressable data
		/// </summary>
		public string RootDirectory { get; } = rootDirectory;

		/// <summary>
		/// Path to a trace file that records the build
		/// </summary>
		public string TraceOutputFile { get; } = traceOutputFile;

		/// <summary>
		/// If the custom allocator should be disabled
		/// </summary>
		public bool DisableCustomAllocator { get; } = disableCustomAllocator;

		/// <summary>
		/// If the visualizer should be launched
		/// </summary>
		public bool LaunchVisualizer { get; } = launchVisualizer;

		/// <summary>
		/// If the content addressable storage should be reset
		/// </summary>
		public bool ResetCas { get; } = resetCas;

		/// <summary>
		/// If intermediate/output files should be written to disk
		/// </summary>
		public bool WriteToDisk { get; } = writeToDisk;

		/// <summary>
		/// More detailed trace information
		/// </summary>
		public bool DetailedTrace { get; } = detailedTrace;

		/// <summary>
		/// Wait for memory before starting new processes
		/// </summary>
		public bool AllowWaitOnMem { get; } = allowWaitOnMem;

		/// <summary>
		/// Kill processes when close to run out of memory
		/// </summary>
		public bool AllowKillOnMem { get; } = allowKillOnMem;

		/// <summary>
		/// Store .obj files compressed on disk
		/// </summary>
		public bool StoreObjFilesCompressed { get; } = storeObjFilesCompressed;
	}

	/// <summary>
	/// Base interface for session server create info
	/// </summary>
	public interface ISessionServerCreateInfo : IBaseInterface
	{
		/// <summary>
		/// Create a ISessionServerCreateInfo object
		/// </summary>
		/// <param name="storage">The storage server</param>
		/// <param name="client">The client</param>
		/// <param name="logger">The logger</param>
		/// <param name="info">The session create info</param>
		/// <returns>The ISessionServerCreateInfo</returns>
		public static ISessionServerCreateInfo CreateSessionServerCreateInfo(IStorageServer storage, IServer client, ILogger logger, SessionServerCreateInfo info)
		{
			return new SessionServerCreateInfoImpl(storage, client, logger, info);
		}
	}

	/// <summary>
	/// Event args for remote process slot available event
	/// </summary>
	/// <param name="isCrossArchitecture">Is true if the available remote process slot is on a cross architecture machine</param>
	public class RemoteProcessSlotAvailableEventArgs(bool isCrossArchitecture) : EventArgs
	{
		/// <summary>
		/// Is true if the available remote process slot is on a cross architecture machine
		/// </summary>
		public bool IsCrossArchitecture { get; } = isCrossArchitecture;
	}

	/// <summary>
	/// Event args for remote process returned event
	/// </summary>
	/// <param name="process">The process being returned</param>
	public class RemoteProcessReturnedEventArgs(IProcess process) : EventArgs
	{

		/// <summary>
		/// The remote process that was returned
		/// </summary>
		public IProcess Process { get; } = process;
	}

	/// <summary>
	/// Base interface for a session server instance
	/// </summary>
	public interface ISessionServer : IBaseInterface
	{
		/// <summary>
		/// Degeate for remote process slot available events
		/// </summary>
		/// <param name="sender">The sender object</param>
		/// <param name="e">The event args</param>
		public delegate void RemoteProcessSlotAvailableEventHandler(object sender, RemoteProcessSlotAvailableEventArgs e);

		/// <summary>
		/// Degeate for remote process returned events
		/// </summary>
		/// <param name="sender">The sender object</param>
		/// <param name="e">The event args</param>
		public delegate void RemoteProcessReturnedEventHandler(object sender, RemoteProcessReturnedEventArgs e);

		/// <summary>
		/// Remote process slot available event handler
		/// </summary>
		public abstract event RemoteProcessSlotAvailableEventHandler? RemoteProcessSlotAvailable;

		/// <summary>
		/// Remote process returned event handler
		/// </summary>
		public abstract event RemoteProcessReturnedEventHandler? RemoteProcessReturned;

		/// <summary>
		/// Add more info about session that show in visualizer
		/// </summary>
		public abstract void AddInfo(string info);

		/// <summary>
		/// Will tell all remote machines that they can disconnect once their active processes are done
		/// Will also stop listening for new remote machines
		/// </summary>
		public abstract void DisableRemoteExecution();

		/// <summary>
		/// Sets specific agent to finish current work and then disconnect
		/// </summary>
		public abstract bool DisableRemoteExecutionOnAgent(string agentName);

		/// <summary>
		/// Set max number of processes that can be executed remotely.
		/// Setting this can let the backend disconnect remote workers earlier
		/// </summary>
		public abstract void SetMaxRemoteProcessCount(uint count);

		/// <summary>
		/// Run a local process
		/// </summary>
		/// <param name="info">Process start info</param>
		/// <param name="async">If the process should be run async</param>
		/// <param name="exitedEventHandler">Optional callback when the process exits</param>
		/// <param name="enableDetour">Should be true unless process does not work being detoured (And in that case we need to manually register file system changes)</param>
		/// <returns>The process being run</returns>
		public abstract IProcess RunProcess(ProcessStartInfo info, bool async, IProcess.ExitedEventHandler? exitedEventHandler, bool enableDetour);

		/// <summary>
		/// Run a remote process
		/// </summary>
		/// <param name="info">Process start info</param>
		/// <param name="exitedEventHandler">Optional callback when the process exits</param>
		/// <param name="weight">Number of cores this process uses</param>
		/// <param name="knownInputs">Optionally contains input that we know process will need. Memory block containing zero-terminated strings with an extra termination in the end.</param>
		/// <param name="knownInputsCount">Number of strings in known inputs</param>
		/// <param name="canExecuteCrossArchitecture">True means that this process can run on a different architecture and expects a mapping to exist</param>
		/// <returns>The remote process being run</returns>
		public abstract IProcess RunProcessRemote(ProcessStartInfo info, IProcess.ExitedEventHandler? exitedEventHandler, double weight = 1.0, byte[]? knownInputs = null, uint knownInputsCount = 0, bool canExecuteCrossArchitecture = false);

		/// <summary>
		/// Register roots data and get handle back
		/// </summary>
		/// <param name="rootsData">Buffer containing roots used for cache and vfs. Format is multiples of byte/utf8string/utf8string. rootindex/vfspath/localpath</param>
		/// <param name="rootsDataSize">Size of buffer in bytes</param>
		/// <returns>The remote process being run</returns>
		public abstract ulong RegisterRoots(byte[] rootsData, uint rootsDataSize);

		/// <summary>
		/// Refresh cached information about directories
		/// </summary>
		/// <param name="directories">The directories to refresh</param>
		public abstract void RefreshDirectories(params string[] directories);

		/// <summary>
		/// Registers external files write to session caches
		/// </summary>
		/// <param name="files">The files to register</param>
		public abstract void RegisterNewFiles(params string[] files);

		/// <summary>
		/// Registers virtual files that are part of other files (mapped into sections of existing file)
		/// </summary>
		/// <param name="name">Name of the virtual file</param>
		/// <param name="sourceFile">Name of the source file</param>
		/// <param name="sourceOffset">Offset inside the source file</param>
		/// <param name="sourceSize">Size of the segment inside the source file</param>
		public abstract bool RegisterVirtualFile(string name, string sourceFile, ulong sourceOffset, ulong sourceSize);

		/// <summary>
		/// Registers the start of an external process
		/// </summary>
		/// <param name="description">The description of the process</param>
		/// <returns>The process id that should be sent into EndExternalProcess</returns>
		public abstract uint BeginExternalProcess(string description);

		/// <summary>
		/// Registers the end of an external process
		/// </summary>
		/// <param name="id">The id returned by BeginExternalProcess</param>
		/// <param name="exitCode">The exit code of the external process</param>
		public abstract void EndExternalProcess(uint id, uint exitCode);

		/// <summary>
		/// Writes external status to the uba trace stream which can then be visualized by ubavisualizer
		/// </summary>
		/// <param name="processesTotal">Total processes in session</param>
		/// <param name="processesDone">Processes done in session</param>
		/// <param name="errorCount">Number of errors in session</param>
		public abstract void UpdateProgress(uint processesTotal, uint processesDone, uint errorCount);

		/// <summary>
		/// Set a custom cas key for a process's tracked inputs
		/// </summary>
		/// <param name="file">The file to track</param>
		/// <param name="workingDirectory">The working directory</param>
		/// <param name="process">The process to get tracked inputs from</param>
		public abstract void SetCustomCasKeyFromTrackedInputs(string file, string workingDirectory, IProcess process);

		/// <summary>
		/// Cancel all processes
		/// </summary>
		public abstract void CancelAll();

		/// <summary>
		/// Print summary information to the logger
		/// </summary>
		public abstract void PrintSummary();

		/// <summary>
		/// Get the trace for this session
		/// </summary>
		public abstract ITrace GetTrace();

		/// <summary>
		/// Create a ISessionServer object
		/// </summary>
		/// <param name="info">The session server create info</param>
		/// <returns>The ISessionServer</returns>
		public static ISessionServer CreateSessionServer(ISessionServerCreateInfo info)
		{
			return new SessionServerImpl(info);
		}
	}
}
