// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UBA.Impl;

namespace EpicGames.UBA
{
	/// <summary>
	/// Information needed to create a storage server
	/// </summary>
	/// <param name="rootDirectory">The root directory for the storage</param>
	/// <param name="capacityBytes">The capacity of the storage in bytes</param>
	/// <param name="storeCompressed">If the storage should be stored as compressed</param>
	/// <param name="zone">The geographical zone this machine belongs to. Can be empty</param>
	public readonly struct StorageServerCreateInfo(string rootDirectory, ulong capacityBytes, bool storeCompressed, string zone)
	{
		/// <summary>
		/// The root directory for the storage
		/// </summary>
		public string RootDirectory { get; } = rootDirectory;

		/// <summary>
		/// The capacity of the storage in bytes
		/// </summary>
		public ulong CapacityBytes { get; } = capacityBytes;

		/// <summary>
		/// If the storage should be stored as compressed
		/// </summary>
		public bool StoreCompressed { get; } = storeCompressed;

		/// <summary>
		/// The geographical zone this machine belongs to. Can be empty
		/// </summary>
		public string Zone { get; } = zone;
	}

	/// <summary>
	/// Base interface for a storage server instance
	/// </summary>
	public interface IStorageServer : IBaseInterface
	{
		/// <summary>
		/// Save tge content addressable storage table
		/// </summary>
		public abstract void SaveCasTable();

		/// <summary>
		/// Register disallowed paths for clients to download
		/// </summary>
		public abstract void RegisterDisallowedPath(string path);

		/// <summary>
		/// Preload cas table (otherwise it will be loaded on demand which might stall)
		/// </summary>
		public abstract void PreloadCasTable();

		/// <summary>
		/// Create a IStorageServer object
		/// </summary>
		/// <param name="server">The server</param>
		/// <param name="logger">The logger</param>
		/// <param name="info">The storage create info</param>
		/// <returns>The IStorageServer</returns>
		public static IStorageServer CreateStorageServer(IServer server, ILogger logger, StorageServerCreateInfo info)
		{
			return new StorageServerImpl(server, logger, info);
		}
	}
}
