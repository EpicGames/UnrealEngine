// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.UBA
{
	/// <summary>
	/// Trace interface
	/// </summary>
	public interface ITrace : IBaseInterface
	{
		/// <summary>
		/// The path to the trace file on disk, if any.
		/// </summary>
		public abstract string? Path { get; }

		/// <summary>
		/// Begin task. Will create a bar in uba visualizer
		/// </summary>
		/// <param name="description">Description showing on bar in visualizer</param>
		/// <param name="details">Details showing when hovering over bar in visualizer</param>
		/// <returns>Task id</returns>
		public abstract uint TaskBegin(string description, string details);

		/// <summary>
		/// Add hint to task. will show with milliseconds since start or previous hint
		/// </summary>
		/// <param name="taskId">Task id returned by TaskBegin</param>
		/// <param name="hint">Hint text</param>
		public abstract void TaskHint(uint taskId, string hint);

		/// <summary>
		/// End task.
		/// </summary>
		/// <param name="taskId">Task id returned by TaskBegin</param>
		public abstract void TaskEnd(uint taskId);

		/// <summary>
		/// Writes external status to the uba trace stream which can then be visualized by ubavisualizer
		/// </summary>
		/// <param name="statusRow">Row of status text. Reuse one index to show one line in visualizer</param>
		/// <param name="statusColumn">The identation of status name that will be shown in visualizer</param>
		/// <param name="statusText">The status text that will be shown in visualizer</param>
		/// <param name="statusType">The status type</param>
		/// <param name="statusLink">Optional hyperlink that can be used to make text clickable in visualizer</param>
		public abstract void UpdateStatus(uint statusRow, uint statusColumn, string statusText, LogEntryType statusType, string? statusLink = null);

		/// <summary>
		/// Creates a new trace
		/// </summary>
		/// <param name="name">Name of trace</param>
		/// <param name="traceFile">File to write trace to</param>
		/// <param name="makeGlobal">Set this trace as the one to be used by all sessions</param>
		/// <param name="channel">Set which channel this trace should be visible to</param>
		public static ITrace Create(string? name = null, string? traceFile = null, bool makeGlobal = false, string? channel = "Default")
		{
			if (name != null)
			{
				try
				{
					return new TraceImpl(name, traceFile, makeGlobal, channel);
				}
				catch
				{
				}
			}

			return new TraceNull();
		}

		/// <summary>
		/// Get the global trace if there is one
		/// </summary>
		public static ITrace? GlobalTrace => TraceImpl.GlobalTraceImpl;
	}

	/// <summary>
	/// Helper type to track task in scope
	/// </summary>
	public readonly struct TraceTaskScope : IDisposable
	{
		private readonly ITrace _trace;
		private readonly uint _taskId;

		/// <summary>
		/// Will create a bar in uba visualizer
		/// </summary>
		/// <param name="trace">Trace used for this task scope</param>
		/// <param name="description">Description showing on bar in visualizer</param>
		/// <param name="details">Details showing when hovering over bar in visualizer</param>
		public TraceTaskScope(ITrace trace, string description, string details)
		{
			_trace = trace;
			_taskId = trace.TaskBegin(description, details);
		}

		/// <summary>
		/// End task
		/// </summary>
		public void Dispose()
		{
			_trace.TaskEnd(_taskId);
		}
	}
}
