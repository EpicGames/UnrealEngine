// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Threading.Tasks;
using EpicGames.Core;

namespace EpicGames.UBA.Impl
{
	internal class CacheClientImpl : ICacheClient
	{
		nint _handle = IntPtr.Zero;
		readonly ISessionServer _sessionServer;

		public delegate void ExitCallback(nint userData, nint handle);

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessStartInfo_Create2(string application, string arguments, string workingDir, string description, uint priorityClass, ulong rootsHandle, byte trackInputs, string logFile, ExitCallback? exit);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void ProcessStartInfo_Destroy(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint CacheClient_Create(nint session, byte reportMissReason, string crypto, string hint);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte CacheClient_Connect(nint cacheClient, string host, int port);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void CacheClient_Disconnect(nint cacheClient);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte CacheClient_WriteToCache2(nint cacheClient, uint bucket, nint info, byte[] inputs, uint inputsSize, byte[] outputs, uint outputsSize);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint CacheClient_FetchFromCache3(nint cacheClient, ulong rootsHandle, uint bucket, nint info);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint CacheResult_GetLogLine(nint result, uint index);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint CacheResult_GetLogLineType(nint result, uint index);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void CacheResult_Delete(nint result);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void CacheClient_RequestServerShutdown(nint cacheClient, string reason);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void CacheClient_Destroy(nint cacheClient);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte CacheClient_RegisterPathHash(nint cacheClient, string path, string hashString);

		#endregion

		#region IDisposable
		~CacheClientImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				if (_connectingTasks != null)
				{
					Task.WaitAll([.. _connectingTasks]);
				}
				CacheClient_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		public CacheClientImpl(ISessionServer server, bool reportMissReason, string crypto, string hint)
		{
			_sessionServer = server;
			_handle = CacheClient_Create(_sessionServer.GetHandle(), (byte)(reportMissReason ? 1 : 0), crypto, hint);
		}

		public bool Connect(string host, int port, int desiredConnectionCount)
		{
			if (CacheClient_Connect(_handle, host, port) == 0)
			{
				return false;
			}

			if (desiredConnectionCount > 1)
			{
				_connectingTasks = [];
				for (int i = 1; i < desiredConnectionCount; ++i)
				{
					_connectingTasks.Add(Task.Run(() =>
					{
						CacheClient_Connect(_handle, host, port);
					}));
				}
			}

			return true;
		}

		public void Disconnect()
		{
			if (_connectingTasks != null)
			{
				Task.WaitAll([.. _connectingTasks]);
			}

			CacheClient_Disconnect(_handle);
		}

		public void PopulatePathHashes()
		{
			foreach (KeyValuePair<DirectoryReference, string> item in Utils.PathHashes)
			{
				CacheClient_RegisterPathHash(_handle, item.Key.FullName, item.Value);
			}
		}

		public bool WriteToCache(uint bucket, nint processHandle, byte[] inputs, uint inputsSize, byte[] outputs, uint outputsSize)
		{
			return CacheClient_WriteToCache2(_handle, bucket, processHandle, inputs, inputsSize, outputs, outputsSize) != 0;
		}

		public FetchFromCacheResult FetchFromCache(ulong rootsHandle, uint bucket, ProcessStartInfo info)
		{
			nint si = ProcessStartInfo_Create2(info.Application, info.Arguments, info.WorkingDirectory, info.Description, (uint)info.Priority, rootsHandle, (byte)(info.TrackInputs ? 1 : 0), info.LogFile ?? String.Empty, null);
			nint result = CacheClient_FetchFromCache3(_handle, rootsHandle, bucket, si);
			ProcessStartInfo_Destroy(si);

			if (result == IntPtr.Zero)
			{
				return new FetchFromCacheResult(false, []);
			}

			string? line = Marshal.PtrToStringAuto(CacheResult_GetLogLine(result, 0));
			if (line == null)
			{
				return new FetchFromCacheResult(true, []);
			}

			List<string> logLines = [];
			while (line != null)
			{
				logLines.Add(line);
				line = Marshal.PtrToStringAuto(CacheResult_GetLogLine(result, (uint)logLines.Count));
			}

			return new FetchFromCacheResult(true, logLines);
		}

		public void RequestServerShutdown(string reason)
		{
			CacheClient_RequestServerShutdown(_handle, reason);
		}

		public nint GetHandle() => _handle;

		private List<Task>? _connectingTasks = null;
	}
}
