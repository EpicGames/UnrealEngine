// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;

namespace EpicGames.UBA.Impl
{
	internal class ConfigImpl : IConfig
	{
		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint Config_Load(string configFile);
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint Config_AddTable(nint config, string name);
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ConfigTable_AddValueString(nint table, string name, string value);
		#endregion

		readonly nint _handle = IntPtr.Zero;

		public ConfigImpl(string configFile)
		{
			_handle = Config_Load(configFile);
		}

		public void AddValue(string table, string name, string value)
		{
			nint tablePtr = Config_AddTable(_handle, table);
			ConfigTable_AddValueString(tablePtr, name, value);
		}

		#region IDisposable
		~ConfigImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (_handle != IntPtr.Zero)
			{
				//Config_Destroy(_handle); // instance acquired by Config_Load is owned by native side
				//_handle = IntPtr.Zero;
			}
		}
		#endregion

		public nint GetHandle() => _handle;
	}
}