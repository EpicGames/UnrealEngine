// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace EpicGames.UBA.Impl
{
	/// <summary>
	/// Logger implementation which logs to an ILogger
	/// </summary>
	internal class LoggerImpl : ILogger
	{
		delegate void BeginScopeCallback();
		delegate void EndScopeCallback();
		delegate void LogCallback(LogEntryType type, nint str, uint len);

		nint _handle = IntPtr.Zero;
		readonly Microsoft.Extensions.Logging.ILogger _logger;
		readonly BeginScopeCallback _beginScopeCallbackDelegate;
		readonly EndScopeCallback _endScopeCallbackDelegate;
		readonly LogCallback _logCallbackDelegate;
		readonly Lock _lock = new();

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint CreateCallbackLogWriter(BeginScopeCallback begin, EndScopeCallback end, LogCallback log);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void DestroyCallbackLogWriter(nint logger);
		#endregion

		public LoggerImpl(Microsoft.Extensions.Logging.ILogger logger, bool makeGlobal)
		{
			_logger = logger;
			_beginScopeCallbackDelegate = BeginScope;
			_endScopeCallbackDelegate = EndScope;
			_logCallbackDelegate = Log;
			_handle = CreateCallbackLogWriter(_beginScopeCallbackDelegate, _endScopeCallbackDelegate, _logCallbackDelegate);
			if (makeGlobal)
			{
				try
				{
					GlobalLoggerImpl = this;
				}
				catch (EntryPointNotFoundException)
				{
				}
			}
		}

		internal static LoggerImpl? GlobalLoggerImpl { set; get; }

		#region IDisposable
		~LoggerImpl() => Dispose(false);

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				DestroyCallbackLogWriter(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region ILogger
		/// <inheritdoc/>
		public nint GetHandle() => _handle;

		/// <inheritdoc/>
		public void BeginScope() => _lock.Enter();

		/// <inheritdoc/>
		public void EndScope() => _lock.Exit();

		/// <inheritdoc/>
		public void Log(LogEntryType type, string message)
		{
			LogLevel logLevel = LogEntryTypeToLogLevel(type);
			lock (_lock)
			{
				_logger.Log(logLevel, KnownLogEvents.UBA, "{Message}", message);
			}
		}
		#endregion

		void Log(LogEntryType type, nint ptr, uint len) => Log(type, Marshal.PtrToStringAuto(ptr, (int)len) ?? String.Empty);

		static LogLevel LogEntryTypeToLogLevel(LogEntryType type)
		{
			switch (type)
			{
				case LogEntryType.Error: return LogLevel.Error;
				case LogEntryType.Warning: return LogLevel.Warning;
				case LogEntryType.Info: return LogLevel.Information;
				case LogEntryType.Detail:
				case LogEntryType.Debug:
				default: return LogLevel.Debug;
			}
		}
	}
}