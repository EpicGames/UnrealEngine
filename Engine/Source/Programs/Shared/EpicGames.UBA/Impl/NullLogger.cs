// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;

namespace EpicGames.UBA.Impl
{
	/// <summary>
	/// Logger implementation which discards output
	/// </summary>
	internal class NullLogger : ILogger
	{
		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint LogWriter_GetNull();
		#endregion

		/// <inheritdoc/>
		public void BeginScope() { }

		/// <inheritdoc/>
		public void EndScope() { }

		/// <inheritdoc/>
		public void Log(LogEntryType type, string message) { }

		/// <inheritdoc/>
		public nint GetHandle() => LogWriter_GetNull();

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		protected virtual void Dispose(bool disposing) { }
	}
}
