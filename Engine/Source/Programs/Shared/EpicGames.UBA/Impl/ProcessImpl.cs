// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Threading;

namespace EpicGames.UBA.Impl
{
	internal class ProcessStartInfoImpl : IProcessStartInfo
	{
		nint _handle = IntPtr.Zero;
		public delegate void ExitCallback(nint userData, nint handle);
		public event ExitCallback? ExitCallbackDelegate;
		public ProcessImpl? Process { get; set; }

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessStartInfo_Create2(string application, string arguments, string workingDir, string description, uint priorityClass, ulong rootsHandle, byte trackInputs, string logFile, ExitCallback? exit);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void ProcessStartInfo_Destroy(nint server);
		#endregion

		public ProcessStartInfoImpl(ProcessStartInfo info, bool needCallback)
		{
			if (needCallback)
			{
				ExitCallbackDelegate = RaiseExited;
			}

			_handle = ProcessStartInfo_Create2(info.Application, info.Arguments, info.WorkingDirectory, info.Description, (uint)info.Priority, info.RootsHandle, (byte)(info.TrackInputs ? 1 : 0), info.LogFile ?? String.Empty, ExitCallbackDelegate);
		}

		void RaiseExited(nint userData, nint handle)
		{
			while (Process == null)
			{
				Thread.Sleep(1); // This should never happen in practice. If c# is garbage collecting while c++ do a full network turnaround with an executed action it could theoretically happen
			}

			Process.RaiseExited();
		}

		#region IDisposable
		~ProcessStartInfoImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				ProcessStartInfo_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region IProcessStartInfo
		public nint GetHandle() => _handle;
		#endregion
	}

	internal class ProcessImpl : IProcess
	{
		nint _handle = IntPtr.Zero;
		[System.Diagnostics.CodeAnalysis.SuppressMessage("CodeQuality", "IDE0052:Remove unread private members", Justification = "maintain unmanaged reference")]
		readonly IProcessStartInfo _processStartInfo;

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint ProcessHandle_GetExitCode(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessHandle_GetExecutingHost(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessHandle_GetLogLine(nint process, uint index);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetTotalProcessorTime(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetTotalWallTime(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetHash(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void ProcessHandle_Cancel(nint process, byte terminate);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void DestroyProcessHandle(nint process);
		#endregion

		public ProcessImpl(nint handle, IProcessStartInfo info, IProcess.ExitedEventHandler? exitedEventHandler, object? userData)
		{
			_handle = handle;
			_processStartInfo = info;
			UserData = userData;
			if (exitedEventHandler != null)
			{
				Exited += exitedEventHandler;
			}
			((ProcessStartInfoImpl)info).Process = this;
		}

		#region IDisposable
		~ProcessImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				DestroyProcessHandle(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region IServer
		public nint GetHandle() => _handle;
		public event IProcess.ExitedEventHandler? Exited;

		public int ExitCode => (int)ProcessHandle_GetExitCode(_handle);

		public string? ExecutingHost => Marshal.PtrToStringAuto(ProcessHandle_GetExecutingHost(_handle));

		public List<string> LogLines
		{
			get
			{
				List<string> logLines = [];
				string? line = Marshal.PtrToStringAuto(ProcessHandle_GetLogLine(_handle, 0));
				while (line != null)
				{
					logLines.Add(line);
					line = Marshal.PtrToStringAuto(ProcessHandle_GetLogLine(_handle, (uint)logLines.Count));
				}
				return logLines;
			}
		}

		public TimeSpan TotalProcessorTime => new((long)ProcessHandle_GetTotalProcessorTime(_handle));

		public TimeSpan TotalWallTime => new((long)ProcessHandle_GetTotalWallTime(_handle));

		public ulong Hash => ProcessHandle_GetHash(_handle);

		public object? UserData { get; }

		public void Cancel(bool terminate) => ProcessHandle_Cancel(_handle, (byte)(terminate ? 1 : 0));
		#endregion

		internal void RaiseExited()
		{
			Exited?.Invoke(this, new ExitedEventArgs(this));
		}
	}
}
