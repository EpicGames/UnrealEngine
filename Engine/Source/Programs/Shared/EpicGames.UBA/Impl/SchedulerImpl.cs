// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;

namespace EpicGames.UBA.Impl
{
	internal class ProcessFinishedInfoImpl : IProcessFinishedInfo
	{
		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint ProcessHandle_GetExitCode(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte ProcessHandle_GetExecutionType(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessHandle_GetExecutingHost(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessHandle_GetLogLine(nint process, uint index);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetTotalProcessorTime(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetTotalWallTime(nint process);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong ProcessHandle_GetPeakMemory(nint process);
		#endregion

		internal ProcessFinishedInfoImpl(object userData, nint processHandle)
		{
			_userData = userData;
			_processHandle = processHandle;
		}
		readonly object _userData;
		readonly nint _processHandle;

		public ProcessExecutionType ExecutionType => (ProcessExecutionType)ProcessHandle_GetExecutionType(_processHandle);
		public int ExitCode => (int)ProcessHandle_GetExitCode(_processHandle);
		public List<string> LogLines
		{
			get
			{
				List<string> logLines = [];
				string? line = Marshal.PtrToStringAuto(ProcessHandle_GetLogLine(_processHandle, 0));
				while (line != null)
				{
					logLines.Add(line);
					line = Marshal.PtrToStringAuto(ProcessHandle_GetLogLine(_processHandle, (uint)logLines.Count));
				}
				return logLines;
			}
		}
		public string? ExecutingHost => Marshal.PtrToStringAuto(ProcessHandle_GetExecutingHost(_processHandle));
		public TimeSpan TotalProcessorTime => new((long)ProcessHandle_GetTotalProcessorTime(_processHandle));
		public TimeSpan TotalWallTime => new((long)ProcessHandle_GetTotalWallTime(_processHandle));
		public ulong PeakMemoryUsed => ProcessHandle_GetPeakMemory(_processHandle);
		public object UserData => _userData;
		public nint ProcessHandle => _processHandle;
	}

	internal class SchedulerProcess
	{
		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint ProcessStartInfo_Create2(string application, string arguments, string workingDir, string description, uint priorityClass, ulong rootsHandle, byte trackInputs, string logFile, ExitCallback? exit);
		#endregion

		public SchedulerProcess(SchedulerImpl scheduler, ProcessStartInfo info, object userData)
		{
			_scheduler = scheduler;
			_userData = userData;
			ExitCallbackDelegate = RaiseExited;
			_startInfoHandle = ProcessStartInfo_Create2(info.Application, info.Arguments, info.WorkingDirectory, info.Description, (uint)info.Priority, info.RootsHandle, (byte)(info.TrackInputs ? 1 : 0), info.LogFile ?? String.Empty, ExitCallbackDelegate);
		}

		void RaiseExited(nint userData, nint processHandle, ref byte exitedResponse)
		{
			// This to prevent a (most likely impossible) race condition (Index is set after process is enqueued)
			while (_index == UInt32.MaxValue)
			{
				Thread.Sleep(1);
			}

			//ProcessImpl process = new(handle);
			ProcessFinishedInfoImpl info = new(_userData, processHandle);

			exitedResponse = (byte)_scheduler._processFinishedCallback!.Invoke(info);
		}

		public delegate void ExitCallback(nint userData, nint processHandle, ref byte exitedResponse);
		readonly SchedulerImpl _scheduler;
		readonly object _userData;
		event ExitCallback ExitCallbackDelegate;
		public nint _startInfoHandle;
		public uint _index = UInt32.MaxValue;
	}

	internal class SchedulerImpl : IScheduler
	{
		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint Scheduler_Create3(nint session, nint[] cacheClients, uint cacheClientCount, nint config);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Scheduler_Destroy(nint scheduler);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Scheduler_Start(nint scheduler);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Scheduler_Cancel(nint scheduler);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte Scheduler_IsEmpty(nint scheduler);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint Scheduler_EnqueueProcess4(nint scheduler, nint info, float weight, byte canDetour, byte canExecuteRemotely, byte canCrossArchitecture, byte canCrossPlatform, int[]? dependencies, int dependencyCount, byte[]? knownInputs, int knownInputsBytes, uint knownInputsCount, uint cacheBucket, uint memoryGroup, ulong predictedMemoryUsage);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern float Scheduler_GetProcessWeightThatCanRunRemotelyNow2(nint scheduler, out float totalWeight, out float crossArchitecture, out float crossPlatform);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern float Scheduler_SetAllowDisableRemoteExecution(nint scheduler, byte allow);
		#endregion

		nint _schedulerHandle = IntPtr.Zero;
		internal Func<IProcessFinishedInfo, ProcessFinishedResponse>? _processFinishedCallback;
		readonly List<SchedulerProcess> _processes = [];

		internal SchedulerImpl(ISessionServer server, IEnumerable<ICacheClient> cacheClients, IConfig config, int maxLocalProcessors, bool forceRemote)
		{
			nint[] cc = new nint[cacheClients.Count()];
			uint cacheClientCount = 0;
			foreach (ICacheClient client in cacheClients)
			{
				cc[cacheClientCount++] = client.GetHandle();
			}

			config.AddValue("Scheduler", "MaxLocalProcessors", maxLocalProcessors.ToString());
			if (forceRemote)
			{
				config.AddValue("Scheduler", "ForceRemote", "True");
			}
			_schedulerHandle = Scheduler_Create3(server.GetHandle(), cc, cacheClientCount, config.GetHandle());
		}

		public void Start()
		{
			Scheduler_Start(_schedulerHandle);
		}

		public uint EnqueueProcess(ProcessStartInfo info, double weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, int[]? dependencies, byte[]? knownInputs, uint knownInputsCount, uint cacheBucket, uint memoryGroup, ulong predictedMemoryUsage, object userData)
		{
			byte canExecuteRemotely_ = (byte)(canExecuteRemotely ? 1 : 0);
			byte canCrossArchitecture_ = (byte)(canCrossArchitecture ? 1 : 0);
			byte canCrossPlatform_ = (byte)(canCrossPlatform ? 1 : 0);
			SchedulerProcess process = new(this, info, userData);
			uint index = Scheduler_EnqueueProcess4(_schedulerHandle, process._startInfoHandle, (float)weight, (byte)(canDetour ? 1 : 0), canExecuteRemotely_, canCrossArchitecture_, canCrossPlatform_, dependencies, dependencies != null ? dependencies.Length : 0, knownInputs, knownInputs != null ? knownInputs.Length : 0, knownInputsCount, cacheBucket, memoryGroup, predictedMemoryUsage);
			process._index = index;
			_processes.Add(process);
			return index;
		}

		public void Cancel()
		{
			Scheduler_Cancel(_schedulerHandle);
		}

		public void SetProcessFinishedCallback(Func<IProcessFinishedInfo, ProcessFinishedResponse> processFinished)
		{
			_processFinishedCallback = processFinished;
		}

		public void SetAllowDisableRemoteExecution()
		{
			Scheduler_SetAllowDisableRemoteExecution(_schedulerHandle, 1);
		}

		public bool IsEmpty => Scheduler_IsEmpty(_schedulerHandle) != 0;

		public void GetProcessWeightThatCanRunRemotelyNow(out double totalWeight, out double crossArchitecture, out double crossPlatform)
		{
			float totalWeight_;
			float crossArchitecture_;
			float crossPlatform_;
			Scheduler_GetProcessWeightThatCanRunRemotelyNow2(_schedulerHandle, out totalWeight_, out crossArchitecture_, out crossPlatform_);
			totalWeight = totalWeight_;
			crossArchitecture = crossArchitecture_;
			crossPlatform = crossPlatform_;
		}

		public nint GetHandle() => _schedulerHandle;

		#region IDisposable
		~SchedulerImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_schedulerHandle != IntPtr.Zero)
			{
				Scheduler_Destroy(_schedulerHandle);
				_schedulerHandle = IntPtr.Zero;
			}
		}
		#endregion
	}
}