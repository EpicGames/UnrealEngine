// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;

namespace EpicGames.UBA.Impl
{
	internal class ServerImpl : IServer
	{
		nint _handle = IntPtr.Zero;
		readonly ILogger _logger;

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint NetworkServer_Create(nint logger, int workerCount, int sendSize, int receiveTimeoutSeconds, byte useQuic);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void NetworkServer_Destroy(nint server);

		// Server Imports
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte NetworkServer_StartListen(nint server, int port, string ip, string crypto);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte NetworkServer_AddClient(nint server, string ip, int port, string crypto);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte NetworkServer_AddNamedConnection(nint server, string name);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte NetworkServer_SetClientsConfig(nint server, nint config);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte NetworkServer_Stop(nint server);

		const int DEFAULT_PORT = 1345;
		#endregion

		public ServerImpl(int maxWorkers, int sendSize, ILogger logger, bool useQuic)
		{
			_logger = logger;
			int receiveTimeoutSeconds = 10 * 60; // 10 minutes. This should never happen
			_handle = NetworkServer_Create(_logger.GetHandle(), maxWorkers, sendSize, receiveTimeoutSeconds, (byte)(useQuic ? 1 : 0));
		}

		#region IDisposable
		~ServerImpl() => Dispose(false);

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				NetworkServer_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region IServer
		/// <inheritdoc/>
		public nint GetHandle() => _handle;

		/// <inheritdoc/>
		public bool StartServer(string ip = "", int port = -1, string crypto = "") => NetworkServer_StartListen(_handle, port > 0 ? port : DEFAULT_PORT, ip, crypto) != 0;

		/// <inheritdoc/>
		public void StopServer() => NetworkServer_Stop(_handle);

		/// <inheritdoc/>
		public bool AddClient(string ip, int port, string crypto = "") => NetworkServer_AddClient(_handle, ip, port, crypto) != 0;

		/// <inheritdoc/>
		public bool AddNamedConnection(string name) => NetworkServer_AddNamedConnection(_handle, name) != 0;

		/// <inheritdoc/>
		public void SetClientsConfig(IConfig config)
		{
			NetworkServer_SetClientsConfig(_handle, config.GetHandle());
		}

		#endregion
	}

	/// <summary>
	/// Null server
	/// </summary>
	internal class NullServer : IServer
	{
		/// <inheritdoc/>
		public bool StartServer(string ip = "", int port = -1, string crypto = "") => false;
		/// <inheritdoc/>
		public void StopServer() { }
		/// <inheritdoc/>
		public bool AddNamedConnection(string name) => false;
		/// <inheritdoc/>
		public bool AddClient(string ip, int port, string crypto = "") => false;
		/// <inheritdoc/>
		public void SetClientsConfig(IConfig config) { }
		/// <inheritdoc/>
		public nint GetHandle() => 0;
		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}
		/// <inheritdoc/>
		protected virtual void Dispose(bool disposing) { }
	}
}
