// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Runtime.InteropServices;
using EpicGames.Core;

namespace EpicGames.UBA.Impl
{
	internal class SessionServerCreateInfoImpl : ISessionServerCreateInfo
	{
		nint _handle = IntPtr.Zero;
		readonly IStorageServer _storage;
		readonly IServer _client;
		readonly ILogger _logger;

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint SessionServerCreateInfo_Create(nint storage, nint client, nint logger, string rootDir, string traceOutputFile,
			byte disableCustomAllocator, byte launchVisualizer, byte resetCas, byte writeToDisk, byte detailedTrace, byte allowWaitOnMem, byte allowKillOnMem, byte storeObjFilesCompressed);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServerCreateInfo_Destroy(nint server);
		#endregion

		public SessionServerCreateInfoImpl(IStorageServer storage, IServer client, ILogger logger, SessionServerCreateInfo info)
		{
			_storage = storage;
			_client = client;
			_logger = logger;
			_handle = SessionServerCreateInfo_Create(_storage.GetHandle(), _client.GetHandle(), _logger.GetHandle(), info.RootDirectory, info.TraceOutputFile, (byte)(info.DisableCustomAllocator ? 1 : 0), (byte)(info.LaunchVisualizer ? 1 : 0), (byte)(info.ResetCas ? 1 : 0), (byte)(info.WriteToDisk ? 1 : 0), (byte)(info.DetailedTrace ? 1 : 0), (byte)(info.AllowWaitOnMem ? 1 : 0), (byte)(info.AllowKillOnMem ? 1 : 0), (byte)(info.StoreObjFilesCompressed ? 1 : 0));
		}

		#region IDisposable
		~SessionServerCreateInfoImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				SessionServerCreateInfo_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region ISessionServerCreateInfo
		public nint GetHandle() => _handle;
		#endregion
	}

	internal class SessionServerImpl : ISessionServer
	{
		delegate void RemoteProcessSlotAvailableCallback(nint userData, byte isCrossArchitecture);
		delegate void RemoteProcessReturnedCallback(nint handle);

		nint _handle = IntPtr.Zero;
		readonly ITrace _trace;
		readonly ISessionServerCreateInfo _info;
		readonly RemoteProcessSlotAvailableCallback _remoteProcessSlotAvailableCallbackDelegate;
		readonly RemoteProcessReturnedCallback _remoteProcessReturnedCallbackDelegate;
		readonly ConcurrentDictionary<ulong, IProcess> _remoteProcesses = new();

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint SessionServer_Create(nint info, byte[] environment, uint environmentSize);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_SetRemoteProcessAvailable(nint server, RemoteProcessSlotAvailableCallback func, nint userData);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_SetRemoteProcessReturned(nint server, RemoteProcessReturnedCallback func);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_RefreshDirectory(nint server, string directory);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_RegisterNewFile(nint server, string filename);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte SessionServer_RegisterVirtualFile(nint server, string filename, string sourceFile, ulong sourceOffset, ulong sourceSize);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint SessionServer_BeginExternalProcess(nint server, string description);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_EndExternalProcess(nint server, uint id, uint exitCode);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_UpdateProgress(nint server, uint processesTotal, uint processesDone, uint errorCount);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint SessionServer_RunProcess(nint server, nint info, bool async, bool enableDetour);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint SessionServer_RunProcessRemote(nint server, nint info, float weight, byte[]? knownInputs, uint knownInputsCount, bool allowCrossArchitecture);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern ulong SessionServer_RegisterRoots(nint server, byte[] rootsData, uint rootsDataSize);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_SetMaxRemoteProcessCount(nint server, uint count);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_AddInfo(nint server, string info);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_DisableRemoteExecution(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern byte SessionServer_DisableRemoteExecutionOnAgent(nint server, string agentName);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_SetCustomCasKeyFromTrackedInputs(nint server, nint process, string filename, string workingdir);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_RegisterCrossArchitectureMapping(nint server, string from, string to);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_PrintSummary(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_CancelAll(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint SessionServer_GetTrace(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void SessionServer_Destroy(nint server);
		#endregion

		public SessionServerImpl(ISessionServerCreateInfo info)
		{
			_info = info;
			_remoteProcessSlotAvailableCallbackDelegate = RaiseRemoteProcessSlotAvailable;
			_remoteProcessReturnedCallbackDelegate = RaiseRemoteProcessReturned;

			// We need to manually transfer environment variables on non-windows platforms since they are not automatically propagated from c# to native.
			using MemoryStream environmentMemory = new();
			{
				if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					using (BinaryWriter writer = new(environmentMemory, System.Text.Encoding.UTF8, true))
					{
						foreach (DictionaryEntry de in Environment.GetEnvironmentVariables())
						{
							writer.Write($"{de.Key}={de.Value}");
						}
					}
				}
			}

			_handle = SessionServer_Create(_info.GetHandle(), environmentMemory.GetBuffer(), (uint)environmentMemory.Position);
			SessionServer_SetRemoteProcessAvailable(_handle, _remoteProcessSlotAvailableCallbackDelegate, 0);
			SessionServer_SetRemoteProcessReturned(_handle, _remoteProcessReturnedCallbackDelegate);

			if (ITrace.GlobalTrace != null)
			{
				_trace = ITrace.GlobalTrace;
			}
			else
			{
				_trace = new TraceImpl(SessionServer_GetTrace(_handle));
			}

			foreach (KeyValuePair<DirectoryReference, DirectoryReference> kv in Utils.CrossArchitecturePaths)
			{
				SessionServer_RegisterCrossArchitectureMapping(_handle, kv.Key.FullName, kv.Value.FullName);
			}
		}

		#region IDisposable
		~SessionServerImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (ITrace.GlobalTrace == null)
			{
				_trace.Dispose();
			}

			if (_handle != IntPtr.Zero)
			{
				SessionServer_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region ISessionServer
		public nint GetHandle() => _handle;

		public event ISessionServer.RemoteProcessSlotAvailableEventHandler? RemoteProcessSlotAvailable;

		public event ISessionServer.RemoteProcessReturnedEventHandler? RemoteProcessReturned;

		public IProcess RunProcess(ProcessStartInfo info, bool async, IProcess.ExitedEventHandler? exitedEventHandler, bool enableDetour)
		{
			IProcessStartInfo startInfo = IProcessStartInfo.CreateProcessStartInfo(info, exitedEventHandler != null);
			nint processPtr = SessionServer_RunProcess(_handle, startInfo.GetHandle(), async, enableDetour);
			IProcess process = IProcess.CreateProcess(processPtr, startInfo, exitedEventHandler, info.UserData);
			return process;
		}

		public IProcess RunProcessRemote(ProcessStartInfo info, IProcess.ExitedEventHandler? exitedEventHandler, double weight, byte[]? knownInputs, uint knownInputsCount, bool canExecuteCrossArchitecture)
		{
			IProcessStartInfo startInfo = IProcessStartInfo.CreateProcessStartInfo(info, exitedEventHandler != null);
			nint processPtr = SessionServer_RunProcessRemote(_handle, startInfo.GetHandle(), (float)weight, knownInputs, knownInputsCount, canExecuteCrossArchitecture);
			IProcess process = IProcess.CreateProcess(processPtr, startInfo, exitedEventHandler, info.UserData);
			_remoteProcesses.AddOrUpdate(process.Hash, process, (k, v) => process);
			return process;
		}

		public ulong RegisterRoots(byte[] rootsData, uint rootsDataSize) => SessionServer_RegisterRoots(_handle, rootsData, rootsDataSize);

		public void DisableRemoteExecution() => SessionServer_DisableRemoteExecution(_handle);

		public bool DisableRemoteExecutionOnAgent(string agentName) => SessionServer_DisableRemoteExecutionOnAgent(_handle, agentName) != 0;

		public void AddInfo(string info) => SessionServer_AddInfo(_handle, info);

		public void SetMaxRemoteProcessCount(uint count) => SessionServer_SetMaxRemoteProcessCount(_handle, count);

		public void RefreshDirectories(params string[] directories) => Array.ForEach(directories, (directory) => SessionServer_RefreshDirectory(_handle, directory));

		public void RegisterNewFiles(params string[] files) => Array.ForEach(files, (file) => SessionServer_RegisterNewFile(_handle, file));
		public bool RegisterVirtualFile(string name, string sourceFile, ulong sourceOffset, ulong sourceSize) => SessionServer_RegisterVirtualFile(_handle, name, sourceFile, sourceOffset, sourceSize) != 0;

		public uint BeginExternalProcess(string description) => SessionServer_BeginExternalProcess(_handle, description);

		public void EndExternalProcess(uint id, uint exitCode) => SessionServer_EndExternalProcess(_handle, id, exitCode);

		public void UpdateProgress(uint processesTotal, uint processesDone, uint errorCount) => SessionServer_UpdateProgress(_handle, processesTotal, processesDone, errorCount);

		public void SetCustomCasKeyFromTrackedInputs(string file, string workingDirectory, IProcess process) => SessionServer_SetCustomCasKeyFromTrackedInputs(_handle, process.GetHandle(), file, workingDirectory);

		public void PrintSummary() => SessionServer_PrintSummary(_handle);

		public void CancelAll() => SessionServer_CancelAll(_handle);

		public ITrace GetTrace() => _trace;

		#endregion

		void RaiseRemoteProcessSlotAvailable(nint userData, byte isCrossArchitecture)
		{
			RemoteProcessSlotAvailable?.Invoke(this, new RemoteProcessSlotAvailableEventArgs(isCrossArchitecture != 0));
		}

		void RaiseRemoteProcessReturned(nint handle)
		{
			IProcess? process = null;
			try
			{
				if (_remoteProcesses.Remove((ulong)handle.ToInt64(), out process))
				{
					RemoteProcessReturned?.Invoke(this, new RemoteProcessReturnedEventArgs(process));
				}
			}
			finally
			{
				process?.Dispose();
			}
		}
	}
}
