// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Runtime.InteropServices;

namespace EpicGames.UBA.Impl
{
	internal class StorageServerImpl : IStorageServer
	{
		nint _handle = IntPtr.Zero;
		readonly IServer _server;
		readonly ILogger _logger;

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint StorageServer_Create(nint server, string rootDir, ulong casCapacityBytes, byte storeCompressed, nint logger, string zone);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void StorageServer_Destroy(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void StorageServer_SaveCasTable(nint server);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void StorageServer_RegisterDisallowedPath(nint server, string path);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void StorageServer_PreloadCasTable(nint server);
		#endregion

		public StorageServerImpl(IServer server, ILogger logger, StorageServerCreateInfo info)
		{
			_server = server;
			_logger = logger;
			_handle = StorageServer_Create(_server.GetHandle(), info.RootDirectory, info.CapacityBytes, (byte)(info.StoreCompressed ? 1 : 0), _logger.GetHandle(), info.Zone);
			Utils.DisallowedPaths.ToList().ForEach(x => RegisterDisallowedPath(x.FullName));
		}

		#region IDisposable
		~StorageServerImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				StorageServer_Destroy(_handle);
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		#region IStorageServer
		public nint GetHandle() => _handle;

		public void SaveCasTable() => StorageServer_SaveCasTable(_handle);

		public void RegisterDisallowedPath(string path) => StorageServer_RegisterDisallowedPath(_handle, path);

		public void PreloadCasTable() => StorageServer_PreloadCasTable(_handle);
		#endregion
	}
}
