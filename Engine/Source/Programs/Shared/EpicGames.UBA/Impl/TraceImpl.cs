// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;

namespace EpicGames.UBA
{
	internal class TraceImpl : ITrace
	{
		nint _handle = IntPtr.Zero;
		readonly string? _traceFile;
		readonly bool _ownsHandle = false;

		#region DllImport
		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern uint Trace_TaskBegin(nint trace, string description, string details);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Trace_TaskHint(nint trace, uint taskId, string hint);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Trace_TaskEnd(nint trace, uint taskId, bool success);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Trace_UpdateStatus(nint trace, uint statusRow, uint statusColumn, string statusText, byte statusType, string? statusLink);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern nint Trace_Create(string? name, string? channel);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Trace_Destroy(nint trace, string? writeFile);

		[DllImport("UbaHost", CharSet = CharSet.Auto)]
		static extern void Trace_SetGlobal(nint trace);
		#endregion

		internal TraceImpl(nint handle)
		{
			_handle = handle;
		}

		internal TraceImpl(string? name, string? traceFile, bool makeGlobal, string? channel)
		{
			_traceFile = traceFile;
			_handle = Trace_Create(name, channel);
			_ownsHandle = true;
			if (makeGlobal)
			{
				try
				{
					Trace_SetGlobal(_handle);
					GlobalTraceImpl = this;
				}
				catch (EntryPointNotFoundException)
				{
				}
			}
		}

		internal static TraceImpl? GlobalTraceImpl { set; get; }

		#region IDisposable
		~TraceImpl() => Dispose(false);

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
			}

			if (_handle != IntPtr.Zero)
			{
				if (GlobalTraceImpl == this)
				{
					GlobalTraceImpl = null;
				}

				if (_ownsHandle)
				{
					Trace_Destroy(_handle, _traceFile);
				}
				_handle = IntPtr.Zero;
			}
		}
		#endregion

		public string? Path => _traceFile;

		public nint GetHandle() => _handle;

		public uint TaskBegin(string description, string details) => Trace_TaskBegin(_handle, description, details);

		public void TaskHint(uint taskId, string hint) => Trace_TaskHint(_handle, taskId, hint);

		public void TaskEnd(uint taskId) => Trace_TaskEnd(_handle, taskId, true);

		public void UpdateStatus(uint statusRow, uint statusColumn, string statusText, LogEntryType statusType, string? statusLink) => Trace_UpdateStatus(_handle, statusRow, statusColumn, statusText, (byte)statusType, statusLink);
	}

	internal class TraceNull : ITrace
	{
		public nint GetHandle() => 0;
		public void Dispose() { }
		public string? Path => null;
		public uint TaskBegin(string description, string details) => 0;
		public void TaskHint(uint taskId, string hint) { }
		public void TaskEnd(uint taskId) { }
		public void UpdateStatus(uint statusRow, uint statusColumn, string statusText, LogEntryType statusType, string? statusLink = null) { }
	}
}