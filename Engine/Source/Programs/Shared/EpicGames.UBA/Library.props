<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">

  <!-- Windows -->
  <!-- x64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaHost.dll')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaHost.dll" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\win-x64\native\UbaHost.dll</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaDetours.dll')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaDetours.dll" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\win-x64\native\UbaDetours.dll</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaWine.dll.so')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaWine.dll.so" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\win-x64\native\UbaWine.dll.so</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <!-- arm64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaHost.dll')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaHost.dll" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\win-arm64\native\UbaHost.dll</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaDetours.dll')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaDetours.dll" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\win-arm64\native\UbaDetours.dll</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>

  <!-- Linux -->
  <!-- x64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaHost.so')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaHost.so" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\linux-x64\native\libUbaHost.so</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaDetours.so')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaDetours.so" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\linux-x64\native\libUbaDetours.so</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\UbaStaticStub.bin')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\UbaStaticStub.bin" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\linux-x64\native\UbaStaticStub.bin</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <!-- arm64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaHost.so')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaHost.so" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\linux-arm64\native\libUbaHost.so</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaDetours.so')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaDetours.so" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\linux-arm64\native\libUbaDetours.so</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>

  <!-- Mac -->
  <!-- universal -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\osx-x64\native\libUbaHost.dylib</TargetPath>
    </ContentWithTargetPath>
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\osx-arm64\native\libUbaHost.dylib</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib')">
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\osx-x64\native\libUbaDetours.dylib</TargetPath>
    </ContentWithTargetPath>
    <ContentWithTargetPath Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib" UpToDateCheckInput="true">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
      <TargetPath>runtimes\osx-arm64\native\libUbaDetours.dylib</TargetPath>
    </ContentWithTargetPath>
  </ItemGroup>

  <!-- Pack -->
  <!-- x64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaHost.dll')">
    <None Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaHost.dll">
      <Link>runtimes\win-x64\native\UbaHost.dll</Link>
      <PackagePath>runtimes\win-x64\native\UbaHost.dll</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaDetours.dll')">
    <None Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\x64\UbaDetours.dll">
      <Link>runtimes\win-x64\native\UbaDetours.dll</Link>
      <PackagePath>runtimes\win-x64\native\UbaDetours.dll</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <!-- arm64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaHost.dll')">
    <None Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaHost.dll">
      <Link>runtimes\win-arm64\native\UbaHost.dll</Link>
      <PackagePath>runtimes\win-arm64\native\UbaHost.dll</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaDetours.dll')">
    <None Include="$(EngineDir)\Binaries\Win64\UnrealBuildAccelerator\arm64\UbaDetours.dll">
      <Link>runtimes\win-arm64\native\UbaDetours.dll</Link>
      <PackagePath>runtimes\win-arm64\native\UbaDetours.dll</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>

  <!-- Linux -->
  <!-- x64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaHost.so')">
    <None Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaHost.so">
      <Link>runtimes\linux-x64\native\libUbaHost.so</Link>
      <PackagePath>runtimes\linux-x64\native\libUbaHost.so</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaDetours.so')">
    <None Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\libUbaDetours.so">
      <Link>runtimes\linux-x64\native\libUbaDetours.so</Link>
      <PackagePath>runtimes\linux-x64\native\libUbaDetours.so</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\UbaStaticStub.bin')">
    <None Include="$(EngineDir)\Binaries\Linux\UnrealBuildAccelerator\UbaStaticStub.bin">
      <Link>runtimes\linux-x64\native\UbaStaticStub.bin</Link>
      <PackagePath>runtimes\linux-x64\native\UbaStaticStub.bin</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <!-- arm64 -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaHost.so')">
    <None Include="$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaHost.so">
      <Link>runtimes\linux-arm64\native\libUbaHost.so</Link>
      <PackagePath>runtimes\linux-arm64\native\libUbaHost.so</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaDetours.so')">
    <None Include="$(EngineDir)\Binaries\LinuxArm64\UnrealBuildAccelerator\libUbaDetours.so">
      <Link>runtimes\linux-arm64\native\libUbaDetours.so</Link>
      <PackagePath>runtimes\linux-arm64\native\libUbaDetours.so</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>

  <!-- Mac -->
  <!-- universal -->
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib')">
    <None Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib">
      <Link>runtimes\osx-x64\native\libUbaHost.dylib</Link>
      <PackagePath>runtimes\osx-x64\native\libUbaHost.dylib</PackagePath>
    </None>
    <None Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaHost.dylib">
      <Link>runtimes\osx-arm64\native\libUbaHost.dylib</Link>
      <PackagePath>runtimes\osx-arm64\native\libUbaHost.dylib</PackagePath>
    </None>
  </ItemGroup>
  <ItemGroup Condition="Exists('$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib')">
    <None Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib">
      <Link>runtimes\osx-x64\native\libUbaDetours.dylib</Link>
      <PackagePath>runtimes\osx-x64\native\libUbaDetours.dylib</PackagePath>
      <Pack>true</Pack>
    </None>
    <None Include="$(EngineDir)\Binaries\Mac\UnrealBuildAccelerator\libUbaDetours.dylib">
      <Link>runtimes\osx-arm64\native\libUbaDetours.dylib</Link>
      <PackagePath>runtimes\osx-arm64\native\libUbaDetours.dylib</PackagePath>
      <Pack>true</Pack>
    </None>
  </ItemGroup>

</Project>