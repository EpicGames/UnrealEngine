// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using EpicGames.Core;

namespace EpicGames.UBA
{
	/// <summary>
	/// Utils
	/// </summary>
	public static partial class Utils
	{
		/// <summary>
		/// Is UBA available?
		/// </summary>
		public static bool IsAvailable() => s_available.Value;
		static readonly Lazy<bool> s_available = new(() => File.Exists(GetLibraryPath()));

		/// <summary>
		/// Paths that are not allowed to be transferred over the network for UBA remote agents.
		/// </summary>
		/// <returns>IEnumerable of disallowed paths</returns>
		public static IEnumerable<DirectoryReference> DisallowedPaths => s_disallowedPaths.Order().Distinct();
		static readonly ConcurrentBag<DirectoryReference> s_disallowedPaths = [];

		/// <summary>
		/// Mapping of binary paths for cross architecture host binaries, to allow for using helpers of a different architecture.
		/// </summary>
		/// <returns>IEnumerable of binary mappings, where the key is the binary for the current host architecture</returns>
		public static IEnumerable<KeyValuePair<DirectoryReference, DirectoryReference>> CrossArchitecturePaths => s_crossArchitecturePaths.ToArray().OrderBy(x => x.Key);
		static readonly ConcurrentDictionary<DirectoryReference, DirectoryReference> s_crossArchitecturePaths = [];

		/// <summary>
		/// Mapping of a folder path to a single hash, to allow for hashing an entire folder so each individual file does not need to be processed.
		/// </summary>
		public static IEnumerable<KeyValuePair<DirectoryReference, string>> PathHashes => s_pathHashes.ToArray().OrderBy(x => x.Key);
		static readonly ConcurrentDictionary<DirectoryReference, string> s_pathHashes = [];

		/// <summary>
		/// Registers a path that is not allowed to be transferred over the network for UBA remote agents.
		/// </summary>
		/// <param name="paths">The paths to add to the disallowed list</param>
		public static void RegisterDisallowedPaths(params DirectoryReference[] paths)
		{
			DirectoryReference[] canonicalPaths = new DirectoryReference[paths.Length];
			for (int i = 0; i < paths.Length; ++i)
			{
				DirectoryReference path = paths[i];
				DirectoryReference canonicalPath = DirectoryReference.FindCorrectCase(path);
				canonicalPaths[i] = canonicalPath;
				if (!s_disallowedPaths.Contains(canonicalPath))
				{
					s_disallowedPaths.Add(canonicalPath);
				}
			}
			DisallowedPathRegistered?.Invoke(DisallowedPaths, new(canonicalPaths));
		}

		/// <summary>
		/// Registers a path mapping for cross architecture binaries
		/// </summary>
		/// <param name="path">host architecture path</param>
		/// <param name="crossPath">cross architecture path</param>
		public static void RegisterCrossArchitecturePath(DirectoryReference path, DirectoryReference crossPath)
		{
			DirectoryReference canonicalPath = DirectoryReference.FindCorrectCase(path);
			DirectoryReference canonicalCrossPath = DirectoryReference.FindCorrectCase(crossPath);
			if (s_crossArchitecturePaths.TryAdd(canonicalPath, canonicalCrossPath))
			{
				CrossArchitecturePathRegistered?.Invoke(CrossArchitecturePaths, new(canonicalPath, canonicalCrossPath));
			}
		}

		/// <summary>
		/// Registers a hash for a path
		/// </summary>
		/// <param name="path">The path to add</param>
		/// <param name="hash">The hash string</param>
		public static void RegisterPathHash(DirectoryReference path, string hash)
		{
			DirectoryReference canonicalPath = DirectoryReference.FindCorrectCase(path);
			if (s_pathHashes.TryAdd(canonicalPath, hash))
			{
				PathHashRegistered?.Invoke(PathHashes, new(canonicalPath, hash));
			}
		}

		/// <summary>
		/// Delegate for registering a remote disallowed path
		/// </summary>
		/// <param name="sender">collection that is being changed</param>
		/// <param name="e">event args containing which paths were added</param>
		public delegate void DisallowedPathRegisteredEventHandler(IEnumerable<DirectoryReference> sender, DisallowedPathRegisteredEventArgs e);

		/// <summary>
		/// Delegate for registering a cross architecture path
		/// </summary>
		/// <param name="sender">collection that is being changed</param>
		/// <param name="e">event args containing which path was added</param>
		public delegate void CrossArchitecturePathRegisteredEventHandler(IEnumerable<KeyValuePair<DirectoryReference, DirectoryReference>> sender, CrossArchitecturePathRegisteredEventArgs e);

		/// <summary>
		/// Delegate for registering a path hash
		/// </summary>
		/// <param name="sender">collection that is being changed</param>
		/// <param name="e">event args containing which path was added</param>
		public delegate void PathHashRegisteredEventHandler(IEnumerable<KeyValuePair<DirectoryReference, string>> sender, PathHashRegisteredEventArgs e);

		/// <summary>
		/// Remote disallowed path registered event handler
		/// </summary>
		public static event DisallowedPathRegisteredEventHandler? DisallowedPathRegistered;

		/// <summary>
		/// Cross architecture path registered event handler
		/// </summary>
		public static event CrossArchitecturePathRegisteredEventHandler? CrossArchitecturePathRegistered;

		/// <summary>
		/// Remote disallowed path registered event handler
		/// </summary>
		public static event PathHashRegisteredEventHandler? PathHashRegistered;

		/// <summary>
		/// Get the path to the p/invoke library that would be loaded 
		/// </summary>
		/// <returns>The path to the library</returns>
		/// <exception cref="PlatformNotSupportedException">If the operating system is not supported</exception>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase", Justification = "folder path is lowercase")]
		static string GetLibraryPath()
		{
			string arch = RuntimeInformation.ProcessArchitecture.ToString().ToLowerInvariant();
			string assemblyFolder = Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location)!;
			if (OperatingSystem.IsWindows())
			{
				return Path.Combine(assemblyFolder, "runtimes", $"win-{arch}", "native", "UbaHost.dll");
			}
			else if (OperatingSystem.IsLinux())
			{
				return Path.Combine(assemblyFolder, "runtimes", $"linux-{arch}", "native", "libUbaHost.so");
			}
			else if (OperatingSystem.IsMacOS())
			{
				return Path.Combine(assemblyFolder, "runtimes", $"osx-{arch}", "native", "libUbaHost.dylib");
			}
			throw new PlatformNotSupportedException();
		}
	}

	/// <summary>
	/// Event args for registering a remote disallowed path
	/// </summary>
	public sealed class DisallowedPathRegisteredEventArgs(params DirectoryReference[] paths) : EventArgs
	{
		/// <summary>
		/// The paths being registered
		/// </summary>
		public IEnumerable<DirectoryReference> Paths { get; } = paths;
	}

	/// <summary>
	/// Event args for registering a cross architecture path
	/// </summary>
	public sealed class CrossArchitecturePathRegisteredEventArgs(DirectoryReference path, DirectoryReference crossPath) : EventArgs
	{
		/// <summary>
		/// The host architecture path being registered
		/// </summary>
		public DirectoryReference Path { get; } = path;

		/// <summary>
		/// The cross architecture path being registered
		/// </summary>
		public DirectoryReference CrossPath { get; } = crossPath;
	}

	/// <summary>
	/// Event args for registering a path hash
	/// </summary>
	public sealed class PathHashRegisteredEventArgs(DirectoryReference path, string hash) : EventArgs
	{
		/// <summary>
		/// The path being registered
		/// </summary>
		public DirectoryReference Path { get; } = path;

		/// <summary>
		/// The hash for the path being registered
		/// </summary>
		public string Hash { get; } = hash;
	}
}
