# NOTE: Requires **VS2019 16.3** or later

# EditorConfig is awesome:http://EditorConfig.org

# Dotnet code style settings:
[*.cs]

# Sort using and Import directives with System.* appearing first
dotnet_sort_system_directives_first = true
dotnet_separate_import_directive_groups = false

# Avoid "this." and "Me." if not necessary
dotnet_style_qualification_for_field = false:error
dotnet_style_qualification_for_property = false:error
dotnet_style_qualification_for_method = false:error
dotnet_style_qualification_for_event = false:error

# Use language keywords instead of framework type names for type references
dotnet_style_predefined_type_for_locals_parameters_members = true:error
dotnet_style_predefined_type_for_member_access = false:error

# Suggest more modern language features when available
dotnet_style_object_initializer = false:suggestion
dotnet_style_collection_initializer = false:suggestion
dotnet_style_coalesce_expression = true:suggestion
dotnet_style_null_propagation = true:suggestion
dotnet_style_explicit_tuple_names = true:suggestion

# Whitespace options
dotnet_style_allow_multiple_blank_lines_experimental = false
dotnet_style_allow_statement_immediately_after_block_experimental = true

# Non-private static fields are PascalCase
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.symbols = non_private_static_fields
dotnet_naming_rule.non_private_static_fields_should_be_pascal_case.style = non_private_static_field_style

dotnet_naming_symbols.non_private_static_fields.applicable_kinds = field
dotnet_naming_symbols.non_private_static_fields.applicable_accessibilities = public, protected, internal, protected_internal, private_protected
dotnet_naming_symbols.non_private_static_fields.required_modifiers = static

dotnet_naming_style.non_private_static_field_style.capitalization = pascal_case

# Non-private readonly fields are PascalCase
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.symbols = non_private_readonly_fields
dotnet_naming_rule.non_private_readonly_fields_should_be_pascal_case.style = non_private_readonly_field_style

dotnet_naming_symbols.non_private_readonly_fields.applicable_kinds = field
dotnet_naming_symbols.non_private_readonly_fields.applicable_accessibilities = public, protected, internal, protected_internal, private_protected
dotnet_naming_symbols.non_private_readonly_fields.required_modifiers = readonly

dotnet_naming_style.non_private_readonly_field_style.capitalization = pascal_case

# Constants are PascalCase
dotnet_naming_rule.constants_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.constants_should_be_pascal_case.symbols = constants
dotnet_naming_rule.constants_should_be_pascal_case.style = constant_style

dotnet_naming_symbols.constants.applicable_kinds = field, local
dotnet_naming_symbols.constants.required_modifiers = const

dotnet_naming_style.constant_style.capitalization = pascal_case

# Static fields are camelCase and start with s_
dotnet_naming_rule.static_fields_should_be_camel_case.severity = suggestion
dotnet_naming_rule.static_fields_should_be_camel_case.symbols = static_fields
dotnet_naming_rule.static_fields_should_be_camel_case.style = static_field_style

dotnet_naming_symbols.static_fields.applicable_kinds = field
dotnet_naming_symbols.static_fields.required_modifiers = static

dotnet_naming_style.static_field_style.capitalization = camel_case
dotnet_naming_style.static_field_style.required_prefix = s_

# Instance fields are camelCase and start with _
dotnet_naming_rule.instance_fields_should_be_camel_case.severity = suggestion
dotnet_naming_rule.instance_fields_should_be_camel_case.symbols = instance_fields
dotnet_naming_rule.instance_fields_should_be_camel_case.style = instance_field_style

dotnet_naming_symbols.instance_fields.applicable_kinds = field

dotnet_naming_style.instance_field_style.capitalization = camel_case
dotnet_naming_style.instance_field_style.required_prefix = _

# Locals and parameters are camelCase
dotnet_naming_rule.locals_should_be_camel_case.severity = suggestion
dotnet_naming_rule.locals_should_be_camel_case.symbols = locals_and_parameters
dotnet_naming_rule.locals_should_be_camel_case.style = camel_case_style

dotnet_naming_symbols.locals_and_parameters.applicable_kinds = parameter, local

dotnet_naming_style.camel_case_style.capitalization = camel_case

# Local functions are PascalCase
dotnet_naming_rule.local_functions_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.local_functions_should_be_pascal_case.symbols = local_functions
dotnet_naming_rule.local_functions_should_be_pascal_case.style = local_function_style

dotnet_naming_symbols.local_functions.applicable_kinds = local_function

dotnet_naming_style.local_function_style.capitalization = pascal_case

# By default, name items with PascalCase
dotnet_naming_rule.members_should_be_pascal_case.severity = suggestion
dotnet_naming_rule.members_should_be_pascal_case.symbols = all_members
dotnet_naming_rule.members_should_be_pascal_case.style = pascal_case_style

dotnet_naming_symbols.all_members.applicable_kinds = *

dotnet_naming_style.pascal_case_style.capitalization = pascal_case

# IDE0005: Remove unnecessary import
dotnet_diagnostic.IDE0005.severity = none

# IDE0073: File header
dotnet_diagnostic.IDE0073.severity = warning
file_header_template = Copyright Epic Games, Inc. All Rights Reserved.

# IDE0035: Remove unreachable code
dotnet_diagnostic.IDE0035.severity = warning

# IDE0036: Order modifiers
dotnet_diagnostic.IDE0036.severity = warning

# IDE0043: Format string contains invalid placeholder
dotnet_diagnostic.IDE0043.severity = warning

# IDE0044: Make field readonly
dotnet_diagnostic.IDE0044.severity = warning

# IDE0011: Add braces
csharp_prefer_braces = when_multiline:warning
# NOTE: We need the below severity entry for Add Braces due to https://github.com/dotnet/roslyn/issues/44201
dotnet_diagnostic.IDE0011.severity = warning

# IDE0040: Add accessibility modifiers
dotnet_diagnostic.IDE0040.severity = warning

# CONSIDER: Are IDE0051 and IDE0052 too noisy to be warnings for IDE editing scenarios? Should they be made build-only warnings?
# IDE0051: Remove unused private member
dotnet_diagnostic.IDE0051.severity = warning

# IDE0052: Remove unread private member
dotnet_diagnostic.IDE0052.severity = warning

# IDE0059: Unnecessary assignment to a value
dotnet_diagnostic.IDE0059.severity = warning

# IDE0060: Remove unused parameter
dotnet_diagnostic.IDE0060.severity = warning

# CA1012: Abstract types should not have public constructors
dotnet_diagnostic.CA1012.severity = warning

# CA1822: Make member static
dotnet_diagnostic.CA1822.severity = warning

# IDE0005: Using directive is unnecessary
dotnet_diagnostic.IDE0005.severity = warning

# dotnet_style_allow_multiple_blank_lines_experimental
dotnet_diagnostic.IDE2000.severity = warning

# csharp_style_allow_embedded_statements_on_same_line_experimental
dotnet_diagnostic.IDE2001.severity = warning

# csharp_style_allow_blank_lines_between_consecutive_braces_experimental
dotnet_diagnostic.IDE2002.severity = warning

# dotnet_style_allow_statement_immediately_after_block_experimental
dotnet_diagnostic.IDE2003.severity = warning

# csharp_style_allow_blank_line_after_colon_in_constructor_initializer_experimental
dotnet_diagnostic.IDE2004.severity = warning

dotnet_style_require_accessibility_modifiers = false

# CSharp code style settings:
[*.cs]
csharp_prefer_simple_using_statement = true
csharp_style_prefer_range_operator = true
csharp_style_implicit_object_creation_when_type_is_apparent = true

# Newline settings
csharp_new_line_before_open_brace = all
csharp_new_line_before_else = true
csharp_new_line_before_catch = true
csharp_new_line_before_finally = true
csharp_new_line_before_members_in_object_initializers = true
csharp_new_line_before_members_in_anonymous_types = true
csharp_new_line_between_query_expression_clauses = true

# Indentation preferences
csharp_indent_block_contents = true
csharp_indent_braces = false
csharp_indent_case_contents = true
csharp_indent_case_contents_when_block = true
csharp_indent_switch_labels = true
csharp_indent_labels = flush_left

# Whitespace options
csharp_style_allow_embedded_statements_on_same_line_experimental = false
csharp_style_allow_blank_lines_between_consecutive_braces_experimental = false
csharp_style_allow_blank_line_after_colon_in_constructor_initializer_experimental = false

# Prefer "var" everywhere
dotnet_diagnostic.IDE0007.severity = error
csharp_style_var_for_built_in_types = false:error
csharp_style_var_when_type_is_apparent = false:error
csharp_style_var_elsewhere = false:error

# Prefer method-like constructs to have a block body
csharp_style_expression_bodied_methods = false:none
csharp_style_expression_bodied_constructors = false:none
csharp_style_expression_bodied_operators = false:none

# Prefer property-like constructs to have an expression-body
csharp_style_expression_bodied_properties = true:error
csharp_style_expression_bodied_indexers = true:error
csharp_style_expression_bodied_accessors = true:error

# Suggest more modern language features when available
csharp_style_pattern_matching_over_is_with_cast_check = true:error
csharp_style_pattern_matching_over_as_with_null_check = false:error
csharp_style_inlined_variable_declaration = true:suggestion
csharp_style_throw_expression = true:error
csharp_style_conditional_delegate_call = true:suggestion

# Spacing
csharp_space_after_cast = false
csharp_space_after_colon_in_inheritance_clause = true
csharp_space_after_comma = true
csharp_space_after_dot = false
csharp_space_after_keywords_in_control_flow_statements = true
csharp_space_after_semicolon_in_for_statement = true
csharp_space_around_binary_operators = before_and_after
csharp_space_around_declaration_statements = do_not_ignore
csharp_space_before_colon_in_inheritance_clause = true
csharp_space_before_comma = false
csharp_space_before_dot = false
csharp_space_before_open_square_brackets = false
csharp_space_before_semicolon_in_for_statement = false
csharp_space_between_empty_square_brackets = false
csharp_space_between_method_call_empty_parameter_list_parentheses = false
csharp_space_between_method_call_name_and_opening_parenthesis = false
csharp_space_between_method_call_parameter_list_parentheses = false
csharp_space_between_method_declaration_empty_parameter_list_parentheses = false
csharp_space_between_method_declaration_name_and_open_parenthesis = false
csharp_space_between_method_declaration_parameter_list_parentheses = false
csharp_space_between_parentheses = false
csharp_space_between_square_brackets = false

# Blocks are allowed
csharp_prefer_braces = true:silent
csharp_preserve_single_line_blocks = true
csharp_preserve_single_line_statements = true

# The follow are UHT specific variances

# CA1045: Do not pass types by reference
dotnet_diagnostic.CA1045.severity = none

# CA1508: Avoid dead conditional code
dotnet_diagnostic.CA1508.severity = none

# CA1501: 'UhtClassParser' has an object hierarchy '6' levels deep within the defining module. If possible, eliminate base classes within the hierarchy to decrease its hierarchy level below '6': 'UhtClassBaseParser, UhtClass, UhtStruct, UhtField, UhtObject, UhtType'.
dotnet_diagnostic.CA1501.severity = none

# IDE0079: Remove unnecessary supression - Different tool versions will complain about unused stuff.
dotnet_diagnostic.IDE0079.severity = none
