<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
    <EnforceCodeStyleInBuild>False</EnforceCodeStyleInBuild>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>

</Project>
