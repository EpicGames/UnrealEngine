// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EditorDataStorageUhtExtension
{
	[UnrealHeaderTool]
	sealed class Extension
	{
		private const string DynamicTemplateMetadata = "EditorDataStorage_DynamicColumnTemplate";

		[UhtCodeGeneratorInjector(
			UhtType = typeof(UhtScriptStruct),
			Location = UhtCodeGeneratorInjectionLocation.GeneratedMacro)]
		public static void InjectDynamicColumnTemplateStaticAttribute(StringBuilder builder, UhtType uhtType, int leadingTabs, string eolSequence)
		{
			if (uhtType.MetaData.ContainsKey(DynamicTemplateMetadata))
			{
				builder.Append('\t', leadingTabs).Append("struct EditorDataStorage_DynamicColumnTemplate{};").Append(eolSequence);
			}
		}
	}
}