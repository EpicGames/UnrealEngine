// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal readonly struct UhtCodeBlockComment : IDisposable
	{
		private const string HighlightPrefix = "**********";
		private const char HighlightChar = '*';
		private const int HighlightLength = 100;
		private readonly StringBuilder _builder;
		private readonly UhtType? _primaryType = null;
		private readonly UhtType? _secondaryType = null;
		private readonly string? _text = null;

		public UhtCodeBlockComment(StringBuilder builder, string text)
		{
			_builder = builder;
			_text = text;
			Begin();
		}

		public UhtCodeBlockComment(StringBuilder builder, UhtType primaryType, string? text = null)
		{
			_builder = builder;
			_primaryType = primaryType;
			_text = text;
			Begin();
		}

		public UhtCodeBlockComment(StringBuilder builder, UhtType primaryType, UhtType secondaryType, string? text = null)
		{
			_builder = builder;
			_primaryType = primaryType;
			_secondaryType = secondaryType;
			_text = text;
			Begin();
		}

		public void Dispose()
		{
			AppendText("End");
		}

		private void Begin()
		{
			_builder.Append("\r\n");
			AppendText("Begin");
		}

		private void AppendText(string startOrEnd)
		{
			int startLength = _builder.Length;
			_builder.Append("// ").Append(HighlightPrefix).Append(' ').Append(startOrEnd);
			if (_primaryType != null)
			{
				_builder.Append(' ').Append(_primaryType.EngineType).Append(' ').Append(_primaryType.SourceName);
			}
			if (_secondaryType != null)
			{
				_builder.Append(' ').Append(_secondaryType.EngineType).Append(' ').Append(_secondaryType.SourceName);
			}
			if (!String.IsNullOrEmpty(_text))
			{
				_builder.Append(' ').Append(_text);
			}
			_builder.Append(' ');
			int endLength = _builder.Length;
			int lineLength = endLength - startLength;
			if (lineLength < HighlightLength)
			{
				_builder.Append(HighlightChar, HighlightLength - lineLength);
			}
			_builder.Append("\r\n");
		}
	}
}
