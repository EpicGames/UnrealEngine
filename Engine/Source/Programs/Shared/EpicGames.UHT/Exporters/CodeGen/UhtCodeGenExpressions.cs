// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	/// <summary>
	/// Utility functions to construct structs which can be used in interpolated strings.
	/// It is recommended to import this type into a namespace with 'using static' so its
	/// methods can be used without the type name prefix for brevity.
	/// </summary>
	internal static class UhtExpressionFactory
	{
		static readonly ReadOnlyMemory<char> s_tabs = String.Empty.PadLeft(100, '\t').AsMemory();

		internal static TabsExpression Tabs(int count) => new(count, s_tabs);

		internal static SingletonRefExpression SingletonRef(IUhtObjectLinker provider, UhtObject? obj, UhtSingletonType type)
			=> new(provider, obj, type, false);

		internal static SingletonRefExpression ConstInitSingletonRef(IUhtObjectLinker provider, UhtObject? obj, bool explicitType = false)
			=> new(provider, obj, UhtSingletonType.ConstInit, explicitType);

		internal static UhtCppIdentifier VniModuleStatics(UhtModule module)
			=> new("Z_VerseRegistration_", module.Module.Name, "_Statics");

		// NOTE: This name is generated by V_CPP_MODULE_PACKAGEDESC_NAME macro
		internal static UhtCppIdentifier VniPackageDesc(UhtModule module)
			=> new("UHTAutoModule_", module.Module.Name, "_PackageDesc");
	}

	#region Expression structs
	internal readonly record struct TabsExpression(int Count, ReadOnlyMemory<char> Tabs) : ISpanFormattable
	{
		public string ToString(string? format, IFormatProvider? formatProvider) => $"{this}";

		public bool TryFormat(Span<char> destination, out int charsWritten, ReadOnlySpan<char> format, IFormatProvider? provider)
		{
			if (Count > Tabs.Length)
			{
				throw new UhtIceException($"Trying to indent something by more than {Tabs.Length}");
			}
			if (Count <= 0)
			{
				charsWritten = 0;
				return true;
			}
			if (Tabs.Span[0..Count].TryCopyTo(destination))
			{
				charsWritten = Count;
				return true;
			}
			charsWritten = 0;
			return false;
		}
	}

	/// <summary>
	/// Utility class for simple expressions such as variable names derived from a type name.
	/// Intended to be used as a return type from factory methods with a more explicit name, e.g. ModuleLinkableObjects
	/// </summary>
	/// <param name="Prefix">Optional prefix e.g. Z_SomeCategory_</param>
	/// <param name="Unique">The part of the string derived from the type/object</param>
	/// <param name="Suffix">Optional suffix if any, e.g. ()</param>
	internal readonly record struct SimpleExpression(string? Prefix, string Unique, string? Suffix = null) : ISpanFormattable
	{
		public string ToString(string? format, IFormatProvider? formatProvider) => $"{this}";

		public bool TryFormat(Span<char> destination, out int charsWritten, ReadOnlySpan<char> format, IFormatProvider? provider)
			=> destination.TryWrite($"{Prefix}{Unique}{Suffix}", out charsWritten);
	}

	internal readonly record struct SingletonRefExpression(IUhtObjectLinker Provider, UhtObject? Obj, UhtSingletonType Type, bool Explicit) : ISpanFormattable
	{
		public string ToString(string? format, IFormatProvider? formatProvider) => $"{this}";

		public bool TryFormat(Span<char> destination, out int charsWritten, ReadOnlySpan<char> format, IFormatProvider? provider) => (Obj, Type, Obj?.IsCrossModuleRef(Provider)) switch
		{
			(null, _, _) => destination.TryWrite($"nullptr", out charsWritten),
			(_, UhtSingletonType.ConstInit, false) => destination.TryWrite($"&{Provider.GetSingletonName(Obj, UhtSingletonType.ConstInit)}", out charsWritten),
			(_, UhtSingletonType.ConstInit, true) =>
				destination.TryWrite($"UHT_LINK_{Obj.Module.ShortName}({Provider.GetModuleIndex(Obj.Module)}, {Provider.GetObjectIndex(Obj)}, &{Provider.GetSingletonName(Obj, UhtSingletonType.ConstInit)})", out charsWritten),
			(_, _, _) => destination.TryWrite($"{Provider.GetSingletonName(Obj, Type)}", out charsWritten),
		};
	}
	#endregion
}
