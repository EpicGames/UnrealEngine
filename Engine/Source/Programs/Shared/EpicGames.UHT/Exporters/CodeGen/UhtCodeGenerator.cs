// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	/// <summary>
	/// Type of singleton to retrieve a name or declaration for
	/// </summary>
	public enum UhtSingletonType
	{
		/// <summary>
		/// Function that returns a fully constructed object - the distinction between this and Unregistered is only important for classes.
		/// </summary>
		Registered = 0,

		/// <summary>
		/// Function that returns a partially constructed object
		/// </summary>
		Unregistered = 1,

		/// <summary>
		/// Constinit variable (C++20 feature)
		/// </summary>
		ConstInit = 2,

		/// <summary>
		/// Struct containing initialization data for the object 
		/// </summary>
		Statics = 3,
	}

	/// <summary>
	/// Mirror of UhtSingletonType as flags for storing sets of singleton types
	/// </summary>
	[Flags]
	public enum UhtSingletonTypeFlag
	{
		/// <summary>
		/// Flag matching nothing
		/// </summary>
		None = 0,
		/// <summary>
		/// Flag for UhtSingletonType.Registered
		/// </summary>
		Registered = 1 << UhtSingletonType.Registered,
		/// <summary>
		/// Flag for UhtSingletonType.Unregistered
		/// </summary>
		Unregistered = 1 << UhtSingletonType.Unregistered,
		/// <summary>
		/// Flag for UhtSingletonType.ConstInit
		/// </summary>
		ConstInit = 1 << UhtSingletonType.ConstInit,
		/// <summary>
		/// Flag for UhtSingletonType.Statics
		/// </summary>
		Statics = 1 << UhtSingletonType.Statics,
	}

	/// <summary>
	/// Interface to provide singleton names during code generation. 
	/// </summary>
	public interface IUhtSingletonNameProvider
	{
		/// <summary>
		/// Return the singleton name for the given object
		/// </summary>
		/// <param name="obj">Object in question - may be a struct, function, class etc or a package</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>Singleton function name</returns>
		public string GetSingletonName(UhtObject? obj, UhtSingletonType type);
	}

	/// <summary>
	/// Interface to link objects together in generated code by initialization expressions such as
	/// address-of or initializing encoded pointers for runtime linking
	/// </summary>
	public interface IUhtObjectLinker : IUhtSingletonNameProvider
	{
		/// <summary>
		/// Used by code generation to determine how a reference should be linked, i.e. by direct pointer or by encoded pointe requiring runtime fixup.
		/// </summary>
		/// <param name="obj"></param>
		/// <returns>True if the object should be linked as a cross-module reference</returns>
		public bool IsCrossModuleRef(UhtObject obj);

		/// <summary>
		/// 
		/// </summary>
		/// <param name="obj"></param>
		/// <returns></returns>
		public int GetModuleIndex(UhtModule obj);

		/// <summary>
		/// 
		/// </summary>
		/// <param name="obj"></param>
		/// <returns></returns>
		public int GetObjectIndex(UhtObject obj);
	}

	[UnrealHeaderTool]
	sealed class UhtCodeGenerator : IUhtSingletonNameProvider
	{
		[UhtExporter(Name = "CodeGen", Description = "Standard UnrealEngine code generation", Options = UhtExporterOptions.Default,
			CppFilters = new string[] { "*.generated.cpp", "*.generated.*.cpp", "*.gen.cpp", "*.gen.*.cpp" },
			HeaderFilters = new string[] { "*.generated.h" })]
		public static void CodeGenerator(IUhtExportFactory factory)
		{
			UhtCodeGenerator generator = new(factory);
			generator.Generate();
		}

		public struct PackageInfo
		{
			public string StrippedName { get; set; }
		}
		public PackageInfo[] PackageInfos { get; set; }

		public struct HeaderInfo
		{
			public Task? Task { get; set; }
			public string IncludePath { get; set; }
			public string FileId { get; set; }
			public IoHash BodyHash { get; set; }
			public bool NeedsPushModelHeaders { get; set; }
			public bool NeedsFastArrayHeaders { get; set; }
			public bool NeedsVerseValue { get; set; }
			public bool NeedsVerseString { get; set; }
			public bool NeedsVerseCodeGen { get; set; }
			public bool NeedsVerseField { get; set; }
			public bool NeedsVersePointer { get; set; }
			public bool NeedsVerseTask { get; set; }
			public bool NeedsVerseCoroutine { get; set; }
		}
		public HeaderInfo[] HeaderInfos { get; set; }

		public struct ObjectInfo
		{
			// e.g. Z_Construct_UClass_UObject
			public string RegisteredSingletonName { get; set; }
			// e.g. COREUOBJECT_API* UClass* Z_Construct_UClass_UObject(ETypeConstructPhase);
			public string RegisteredExternalDecl { get; set; }
			// Fully qualified name of variable containing const-initialized object 
			// e.g. Z_ConstInit_UClass_UObject
			public string ConstInitSingletonName { get; set; }
			// Full declaration to import variable containing const-initialized object
			// e.g. extern UE_CONSTINIT_UOBJECT_DECL UClass Z_ConstInit_UClass_UObject
			// Note lack of API macro 
			public string ConstInitExternalDecl { get; set; }
			// Name of statics struct in generated cpp containing construction parameters
			// e.g. Z_Construct_UClass_UObject_Statics
			public string StaticsName { get; set; }
			public UhtClass? NativeInterface { get; set; }
			public UhtProperty? FastArrayProperty { get; set; }
			public IoHash Hash { get; set; }
		}
		public ObjectInfo[] ObjectInfos { get; set; }

		public readonly IUhtExportFactory Factory;
		public UhtSession Session => Factory.Session;
		public UhtScriptStruct? FastArraySerializer { get; set; } = null;

		private UhtCodeGenerator(IUhtExportFactory factory)
		{
			Factory = factory;
			HeaderInfos = new HeaderInfo[Factory.Session.HeaderFileTypeCount];
			ObjectInfos = new ObjectInfo[Factory.Session.ObjectTypeCount];
			PackageInfos = new PackageInfo[Factory.Session.PackageTypeCount];
		}

		private void Generate()
		{
			List<Task?> prereqs = [];

			FastArraySerializer = Session.FindType(null, UhtFindOptions.SourceName | UhtFindOptions.ScriptStruct, "FFastArraySerializer") as UhtScriptStruct;

			// Perform some startup initialization to compute things we need over and over again

			if (Session.GoWide)
			{
				Parallel.ForEach(Factory.Session.Modules, module =>
				{
					InitModuleInfo(module);
				});
			}
			else
			{
				foreach (UhtModule module in Factory.Session.Modules)
				{
					InitModuleInfo(module);
				}
			}

			// Generate the files for the header files
			foreach (UhtHeaderFile headerFile in Session.SortedHeaderFiles)
			{
				if (headerFile.ShouldExport)
				{
					prereqs.Clear();
					foreach (UhtHeaderFile referenced in headerFile.ReferencedHeadersNoLock)
					{
						if (headerFile != referenced)
						{
							prereqs.Add(HeaderInfos[referenced.HeaderFileTypeIndex].Task);
						}
					}

					HeaderInfos[headerFile.HeaderFileTypeIndex].Task = Factory.CreateTask(prereqs,
						factory =>
						{
							new UhtHeaderCodeGeneratorHFile(this, headerFile).Generate(factory);
							new UhtHeaderCodeGeneratorCppFile(this, headerFile).Generate(factory);
						});
				}
				else
				{
					// Still needs to write cpp file since UBT expects it to exist (there are very few of these)
					HeaderInfos[headerFile.HeaderFileTypeIndex].Task = Factory.CreateTask(prereqs,
						factory =>
						{
							new UhtHeaderCodeGeneratorHFile(this, headerFile).Generate(factory);
							new UhtHeaderCodeGeneratorCppFile(this, headerFile).Generate(factory);
						});
				}
			}

			// Generate the files for the modules
			List<Task?> generatedModules = new(Session.Modules.Count);
			foreach (UhtModule module in Session.Modules)
			{
				bool writeHeader = false;
				prereqs.Clear();
				foreach (UhtHeaderFile headerFile in module.Headers)
				{
					prereqs.Add(HeaderInfos[headerFile.HeaderFileTypeIndex].Task);
					if (!writeHeader)
					{
						foreach (UhtType type in headerFile.Children)
						{
							if (type is UhtClass classObj)
							{
								if (classObj.ClassType != UhtClassType.NativeInterface &&
									classObj.ClassFlags.HasExactFlags(EClassFlags.Native | EClassFlags.Intrinsic, EClassFlags.Native) &&
									!classObj.ClassExportFlags.HasAllFlags(UhtClassExportFlags.NoExport))
								{
									writeHeader = true;
									break;
								}
							}
						}
					}
				}

				generatedModules.Add(Factory.CreateTask(prereqs,
					factory =>
					{
						List<UhtHeaderFile> moduleSortedHeaders = GetSortedHeaderFiles(module);
						if (writeHeader)
						{
							new UhtPackageCodeGeneratorHFile(this, module).Generate(factory, moduleSortedHeaders);
						}
						new UhtPackageCodeGeneratorCppFile(this, module).Generate(factory, moduleSortedHeaders);
					}));
			}

			// Wait for all the packages to complete
			List<Task> moduleTasks = new(generatedModules.Count);
			foreach (Task? output in generatedModules)
			{
				if (output != null)
				{
					moduleTasks.Add(output);
				}
			}
			Task.WaitAll([.. moduleTasks]);
		}

		#region Utility functions
		/// <summary>
		/// Return the singleton name for an object
		/// </summary>
		/// <param name="obj">The type or package in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>Singleton name or "nullptr" if Object is null</returns>
		public string GetSingletonName(UhtObject? obj, UhtSingletonType type)
		{
			if (obj == null)
			{
				return "nullptr";
			}
			switch (type)
			{
				case UhtSingletonType.Statics:
					return ObjectInfos[obj.ObjectTypeIndex].StaticsName;
				case UhtSingletonType.Registered:
				case UhtSingletonType.Unregistered:
					return ObjectInfos[obj.ObjectTypeIndex].RegisteredSingletonName;
				case UhtSingletonType.ConstInit:
					return ObjectInfos[obj.ObjectTypeIndex].ConstInitSingletonName;
			}
			throw new UhtIceException($"Invalid UhtSingletonType value {type}");
		}

		/// <summary>
		/// Return the external declaration for an object
		/// </summary>
		/// <param name="obj">The object in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>External declaration</returns>
		public string GetExternalDecl(UhtObject obj, UhtSingletonType type)
		{
			return GetExternalDecl(obj.ObjectTypeIndex, type);
		}

		/// <summary>
		/// Return the external declaration for an object
		/// </summary>
		/// <param name="objectIndex">The object in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>External declaration</returns>
		public string GetExternalDecl(int objectIndex, UhtSingletonType type)
		{
			switch (type)
			{
				case UhtSingletonType.Registered:
				case UhtSingletonType.Unregistered:
					return ObjectInfos[objectIndex].RegisteredExternalDecl;
				case UhtSingletonType.ConstInit:
					return ObjectInfos[objectIndex].ConstInitExternalDecl;
				case UhtSingletonType.Statics:
					throw new UhtIceException($"Statics struct for {ObjectInfos[objectIndex].StaticsName} cannot be imported across translation units");
			}
			throw new UhtIceException($"Invalid UhtSingletonType value {type}");
		}
		#endregion

		#region Information initialization
		private void InitModuleInfo(UhtModule module)
		{
			StringBuilder builder = new();

			foreach (UhtPackage package in module.Packages)
			{
				InitPackageInfo(builder, package);
			}
			foreach (UhtHeaderFile headerFile in module.Headers)
			{
				InitHeaderInfo(builder, headerFile);
			}
		}

		private void InitPackageInfo(StringBuilder builder, UhtPackage package)
		{
			ref PackageInfo packageInfo = ref PackageInfos[package.PackageTypeIndex];
			packageInfo.StrippedName = package.SourceName.Replace('/', '_');

			// Construct the names used commonly during export
			ref ObjectInfo objectInfo = ref ObjectInfos[package.ObjectTypeIndex];
			builder.Clear();
			builder.Append("Z_Construct_UPackage_");
			builder.Append(packageInfo.StrippedName);
			objectInfo.RegisteredSingletonName = builder.ToString();
			objectInfo.StaticsName = $"{objectInfo.RegisteredSingletonName}_Statics";
			objectInfo.RegisteredExternalDecl = $"\tUPackage* {objectInfo.RegisteredSingletonName}(ETypeConstructPhase);\r\n";
			objectInfo.ConstInitSingletonName = $"Z_ConstInit_UPackage_{packageInfo.StrippedName}";
			objectInfo.ConstInitExternalDecl = $"\textern UE_CONSTINIT_UOBJECT_DECL UPackage {objectInfo.ConstInitSingletonName};\r\n";
		}

		private void InitHeaderInfo(StringBuilder builder, UhtHeaderFile headerFile)
		{
			ref HeaderInfo headerInfo = ref HeaderInfos[headerFile.HeaderFileTypeIndex];

			// Find the shortest matching relative path
			headerInfo.IncludePath = Factory.GetModuleShortestIncludePath(headerFile.Module, headerFile.FilePath);

			// Convert the file path to a C identifier
			string filePath = headerFile.FilePath;
			bool isRelative = !Path.IsPathRooted(filePath);
			if (!isRelative && Session.EngineDirectory != null)
			{
				string? directory = Path.GetDirectoryName(Session.EngineDirectory);
				if (!String.IsNullOrEmpty(directory))
				{
					filePath = Path.GetRelativePath(directory, filePath);
					isRelative = !Path.IsPathRooted(filePath);
				}
			}
			if (!isRelative && Session.ProjectDirectory != null)
			{
				string? directory = Path.GetDirectoryName(Session.ProjectDirectory);
				if (!String.IsNullOrEmpty(directory))
				{
					filePath = Path.GetRelativePath(directory, filePath);
					isRelative = !Path.IsPathRooted(filePath);
				}
			}
			filePath = filePath.Replace('\\', '/');
			if (isRelative)
			{
				while (filePath.StartsWith("../", StringComparison.Ordinal))
				{
					filePath = filePath[3..];
				}
			}

			char[] outFilePath = new char[filePath.Length + 4];
			outFilePath[0] = 'F';
			outFilePath[1] = 'I';
			outFilePath[2] = 'D';
			outFilePath[3] = '_';
			for (int index = 0; index < filePath.Length; ++index)
			{
				outFilePath[index + 4] = UhtFCString.IsAlnum(filePath[index]) ? filePath[index] : '_';
			}
			headerInfo.FileId = new string(outFilePath);

			foreach (UhtType headerFileChild in headerFile.Children)
			{
				if (headerFileChild is UhtObject obj)
				{
					UhtPackage? package = obj.Outer as UhtPackage;
					if (package == null)
					{
						throw new UhtIceException("Expected type defined in a header to have a package outer");
					}
					InitObjectInfo(builder, package, ref headerInfo, obj);
				}
			}
		}

		private void InitObjectInfo(StringBuilder builder, UhtPackage package, ref HeaderInfo headerInfo, UhtObject obj)
		{
			ref ObjectInfo objectInfo = ref ObjectInfos[obj.ObjectTypeIndex];

			builder.Clear();

			// Construct the names used commonly during export
			builder.Append("Z_Construct_U").Append(obj.EngineClassName).AppendOuterNames(obj);

			string api = obj.Module.Api;
			if (obj is UhtClass classObj)
			{
				if (classObj.ClassExportFlags.HasExactFlags(UhtClassExportFlags.HasReplciatedProperties, UhtClassExportFlags.SelfHasReplicatedProperties))
				{
					headerInfo.NeedsPushModelHeaders = true;
				}
				InitVerseInfo(ref headerInfo, classObj);
				if (classObj.ClassType == UhtClassType.NativeInterface)
				{
					if (classObj.AlternateObject != null)
					{
						ObjectInfos[classObj.AlternateObject.ObjectTypeIndex].NativeInterface = classObj;
					}
				}
			}
			else if (obj is UhtScriptStruct scriptStructObj)
			{
				InitVerseInfo(ref headerInfo, scriptStructObj);

				// Check to see if we are a FastArraySerializer and should try to deduce the FastArraySerializerItemType
				// To fulfill that requirement the struct should be derived from FFastArraySerializer and have a single replicated TArrayProperty
				if (scriptStructObj.IsChildOf(FastArraySerializer))
				{
					// If Super is a valid fastarray we mark this struct as a FastArrayProperty as well
					if (scriptStructObj.Super != null && ObjectInfos[scriptStructObj.Super.ObjectTypeIndex].FastArrayProperty != null)
					{
						objectInfo.FastArrayProperty = ObjectInfos[scriptStructObj.Super.ObjectTypeIndex].FastArrayProperty;
					}

					// A valid fastarray cannot have any additional replicated properties.
					foreach (UhtType child in scriptStructObj.Children)
					{
						if (child is UhtProperty property)
						{
							if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.RepSkip) && property is UhtArrayProperty)
							{
								if (objectInfo.FastArrayProperty != null)
								{
									objectInfo.FastArrayProperty = null;
									break;
								}
								objectInfo.FastArrayProperty = property;
							}
						}
					}
					if (objectInfo.FastArrayProperty != null)
					{
						headerInfo.NeedsFastArrayHeaders = true;
					}
				}
			}
			else if (obj is UhtEnum enumObj)
			{
				if (enumObj.IsVerseField)
				{
					headerInfo.NeedsVerseCodeGen = true;
				}
			}

			objectInfo.RegisteredSingletonName = builder.ToString();
			objectInfo.StaticsName = $"{objectInfo.RegisteredSingletonName}_Statics";
			objectInfo.RegisteredExternalDecl = $"\t{api}U{obj.EngineLinkClassName}* {objectInfo.RegisteredSingletonName}(ETypeConstructPhase);\r\n";
			if (obj is UhtPartial)
			{
				objectInfo.ConstInitSingletonName = $"{objectInfo.RegisteredSingletonName}_PartialParams";
				objectInfo.ConstInitExternalDecl = $"\textern UE_CONSTINIT_UOBJECT_DECL UE::CoreUObject::Private::FPartialClass {objectInfo.ConstInitSingletonName};\r\n";
			}
			else
			{
				objectInfo.ConstInitSingletonName = objectInfo.RegisteredSingletonName.Replace("Z_Construct_", "Z_ConstInit_", StringComparison.InvariantCultureIgnoreCase);
				objectInfo.ConstInitExternalDecl = $"\textern UE_CONSTINIT_UOBJECT_DECL TNoDestroy<U{obj.EngineClassName}> {objectInfo.ConstInitSingletonName};\r\n";
			}

			// Init the children
			foreach (UhtType child in obj.Children)
			{
				if (child is UhtObject childObject)
				{
					InitObjectInfo(builder, package, ref headerInfo, childObject);
				}
			}
		}

		/// <summary>
		/// Given a structure (class or script struct), initialize the header info based on the existence of verse elements
		/// </summary>
		/// <param name="headerInfo">Header information to be initialized</param>
		/// <param name="structObj">Structure in question</param>
		static void InitVerseInfo(ref HeaderInfo headerInfo, UhtStruct structObj)
		{
			if (structObj.IsVerseField)
			{
				headerInfo.NeedsVerseCodeGen = true;
			}
			foreach (UhtType childType in structObj.Children)
			{
				if (childType is UhtProperty childProperty)
				{
					if (childProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVal))
					{
						headerInfo.NeedsVerseField = true;
					}
					if (childProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TPtr))
					{
						headerInfo.NeedsVersePointer = true;
					}
					if (childProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseTask))
					{
						headerInfo.NeedsVerseTask = true;
					}
					if (childProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseCall))
					{
						headerInfo.NeedsVerseCoroutine = true;
					}
					if (!headerInfo.NeedsVerseValue && childProperty is UhtVerseValueProperty)
					{
						headerInfo.NeedsVerseValue = true;
					}
					if (!headerInfo.NeedsVerseString && childProperty is UhtVerseStringProperty)
					{
						headerInfo.NeedsVerseString = true;
					}
				}
			}
		}
		#endregion

		#region Utility functions
		/// <summary>
		/// Return a module's sorted header file list of all header files that or referenced or have declarations.
		/// </summary>
		/// <param name="module">The module in question</param>
		/// <returns>Sorted list of the header files</returns>
		private static List<UhtHeaderFile> GetSortedHeaderFiles(UhtModule module)
		{
			List<UhtHeaderFile> sortedHeaders = new(module.Headers.Count);
			foreach (UhtHeaderFile headerFile in module.Headers)
			{
				if (headerFile.ShouldExport)
				{
					sortedHeaders.Add(headerFile);
				}
			}
			sortedHeaders.Sort((lhs, rhs) => { return StringComparerUE.OrdinalIgnoreCase.Compare(lhs.FilePath, rhs.FilePath); });
			return sortedHeaders;
		}
		#endregion
	}

	static class UhtCodeGeneratorExtensions
	{
		/// <summary>
		/// Convenience function so allow use of ?. operator on objects to determine if they are cross-module references according to the given linker.
		/// </summary>
		/// <param name="obj"></param>
		/// <param name="linker"></param>
		/// <returns></returns>
		public static bool IsCrossModuleRef(this UhtObject obj, IUhtObjectLinker linker) => linker.IsCrossModuleRef(obj);
	}
}
