// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	class UhtHeaderCodeGenerator : UhtPackageCodeGenerator
	{
		#region Define block macro names
		public const string CallbackWrappersMacroSuffix = "CALLBACK_WRAPPERS";
		public const string SparseDataMacroSuffix = "SPARSE_DATA";
		public const string SparseDataPropertyAccessorsMacroSuffix = "SPARSE_DATA_PROPERTY_ACCESSORS";
		public const string RpcWrappersMacroSuffix = "RPC_WRAPPERS";
		public const string RpcWrappersNoPureDeclsMacroSuffix = "RPC_WRAPPERS_NO_PURE_DECLS";
		public const string VerseNativeCallableWrappersMacroSuffix = "VERSE_NATIVE_CALLABLE_WRAPPERS";
		public const string AccessorsMacroSuffix = "ACCESSORS";
		public const string ArchiveSerializerMacroSuffix = "ARCHIVESERIALIZER";
		public const string StandardConstructorsMacroSuffix = "STANDARD_CONSTRUCTORS";
		public const string EnchancedConstructorsMacroSuffix = "ENHANCED_CONSTRUCTORS";
		public const string GeneratedBodyMacroSuffix = "GENERATED_BODY";
		public const string GeneratedBodyLegacyMacroSuffix = "GENERATED_BODY_LEGACY";
		public const string GeneratedUInterfaceBodyMacroSuffix = "GENERATED_UINTERFACE_BODY()";
		public const string FieldNotifyMacroSuffix = "FIELDNOTIFY";
		public const string InClassMacroSuffix = "INCLASS";
		public const string InClassNoPureDeclsMacroSuffix = "INCLASS_NO_PURE_DECLS";
		public const string InClassIInterfaceMacroSuffix = "INCLASS_IINTERFACE";
		public const string InClassIInterfaceNoPureDeclsMacroSuffix = "INCLASS_IINTERFACE_NO_PURE_DECLS";
		public const string PrologMacroSuffix = "PROLOG";
		public const string AutoGettersSettersMacroSuffix = "AUTOGETTERSETTER_DECLS";
		public const string VInterfacesMacroSuffix = "VINTERFACES";
		#endregion

		public readonly UhtHeaderFile HeaderFile;
		public string FileId => HeaderInfos[HeaderFile.HeaderFileTypeIndex].FileId;

		public UhtHeaderCodeGenerator(UhtCodeGenerator codeGenerator, UhtHeaderFile headerFile)
			: base(codeGenerator, headerFile.Module)
		{
			HeaderFile = headerFile;
		}

		#region Event parameter
		protected static StringBuilder AppendEventParameter(StringBuilder builder, UhtFunction function, string strippedFunctionName, UhtPropertyTextType textType, bool outputConstructor, int tabs, string endl)
		{
			if (!WillExportEventParameters(function))
			{
				return builder;
			}

			string eventParameterStructName = GetEventStructParametersName(function.Outer, strippedFunctionName);

			builder.AppendTabs(tabs).Append("struct ").Append(eventParameterStructName).Append(endl);
			builder.AppendTabs(tabs).Append('{').Append(endl);

			++tabs;
			foreach (UhtType functionChild in function.Children)
			{
				if (functionChild is UhtProperty property)
				{
					bool emitConst = property.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm) && property is UhtObjectProperty;

					//@TODO: UCREMOVAL: This is awful code duplication to avoid a double-const
					{
						//@TODO: bEmitConst will only be true if we have an object, so checking interface here doesn't do anything.
						// export 'const' for parameters
						bool isConstParam = property is UhtInterfaceProperty && !property.PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm); //@TODO: This should be const once that flag exists
						bool isOnConstClass = false;
						if (property is UhtObjectProperty objectProperty)
						{
							isOnConstClass = objectProperty.Class.ClassFlags.HasAnyFlags(EClassFlags.Const);
						}

						if (isConstParam || isOnConstClass)
						{
							emitConst = false; // ExportCppDeclaration will do it for us
						}
					}

					builder.AppendTabs(tabs);
					if (emitConst)
					{
						builder.Append("const ");
					}

					builder.AppendFullDecl(property, textType, false);
					builder.Append(';').Append(endl);
				}
			}

			if (outputConstructor)
			{
				UhtProperty? returnProperty = function.ReturnProperty;
				if (returnProperty != null && returnProperty.PropertyCaps.HasAnyFlags(UhtPropertyCaps.RequiresNullConstructorArg))
				{
					builder.Append(endl);
					builder.AppendTabs(tabs).Append("/** Constructor, initializes return property only **/").Append(endl);
					builder.AppendTabs(tabs).Append(eventParameterStructName).Append("()").Append(endl);
					builder.AppendTabs(tabs + 1).Append(": ").Append(returnProperty.SourceName).Append('(').AppendNullConstructorArg(returnProperty, true).Append(')').Append(endl);
					builder.AppendTabs(tabs).Append('{').Append(endl);
					builder.AppendTabs(tabs).Append('}').Append(endl);
				}
			}
			--tabs;

			builder.AppendTabs(tabs).Append("};").Append(endl);

			return builder;
		}

		protected static bool WillExportEventParameters(UhtFunction function)
		{
			return function.Children.Count > 0;
		}

		protected static string GetEventStructParametersName(UhtType? outer, string functionName)
		{
			string outerName = String.Empty;
			if (outer == null)
			{
				throw new UhtIceException("Outer type not set on delegate function");
			}
			else if (outer is UhtClass classObj)
			{
				outerName = classObj.EngineName;
			}
			else if (outer is UhtPartial partial)
			{
				// For functions in partials, use the extended class name for parameter struct naming
				outerName = partial.OwnerClass!.EngineName;
			}
			else if (outer is UhtPackage packageObj)
			{
				string packageName = packageObj.EngineName;
				outerName = packageName.Replace('/', '_');
			}

			string result = $"{outerName}_event{functionName}_Parms";
			if (UhtFCString.IsDigit(result[0]))
			{
				result = "_" + result;
			}
			return result;
		}
		#endregion

		#region Function helper methods
		protected StringBuilder AppendNativeFunctionHeader(StringBuilder builder, UhtFunction function, UhtPropertyTextType textType, bool isDeclaration,
			string? alternateFunctionName, string? extraParam, UhtFunctionExportFlags extraExportFlags, int tabs, string endl)
		{
			UhtClass? outerClass = function.Outer as UhtClass;
			UhtFunctionExportFlags exportFlags = function.FunctionExportFlags | extraExportFlags;
			bool isDelegate = function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate);
			bool isInterface = !isDelegate && (outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.Interface));
			bool isK2Override = function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent);

			builder.AppendTabs(tabs);

			if (isDeclaration)
			{
				// If the function was marked as 'RequiredAPI', then add the *_API macro prefix.  Note that if the class itself
				// was marked 'RequiredAPI', this is not needed as C++ will exports all methods automatically.
				if (textType != UhtPropertyTextType.EventFunctionArgOrRetVal &&
					!(outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.RequiredAPI)) &&
					exportFlags.HasAnyFlags(UhtFunctionExportFlags.RequiredAPI) && (!isInterface || textType == UhtPropertyTextType.InterfaceFunctionArgOrRetVal))
				{
					builder.Append(Module.Api);
				}

				if (textType == UhtPropertyTextType.InterfaceFunctionArgOrRetVal)
				{
					builder.Append("static ");
				}
				else
				{
					bool isVirtual = false;
					if (outerClass != null && outerClass.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.Final))
					{
						//
					}
					else if (isK2Override)
					{
						isVirtual = true;
					}
					// if the owning class is an interface class
					else if (isInterface)
					{
						isVirtual = true;
					}
					// this is not an event, the function is not a static function and the function is not marked final
					else if (textType != UhtPropertyTextType.EventFunctionArgOrRetVal &&
						!function.FunctionFlags.HasAnyFlags(EFunctionFlags.Static) &&
						!exportFlags.HasAnyFlags(UhtFunctionExportFlags.Final))
					{
						isVirtual = true;
					}

					if (isVirtual)
					{
						builder.Append("virtual ");
					}
					else if (exportFlags.HasAnyFlags(UhtFunctionExportFlags.Inline))
					{
						builder.Append("inline ");
					}
				}
			}

			UhtProperty? returnProperty = function.ReturnProperty;
			if (returnProperty != null)
			{
				if (returnProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
				{
					builder.Append("const ");
				}
				builder.AppendPropertyText(returnProperty, textType);
			}
			else
			{
				builder.Append("void");
			}

			builder.Append(' ');
			if (!isDeclaration && outerClass != null)
			{
				builder.AppendClassSourceNameOrInterfaceName(outerClass).Append("::");
			}

			if (alternateFunctionName != null)
			{
				builder.Append(alternateFunctionName);
			}
			else
			{
				switch (textType)
				{
					case UhtPropertyTextType.InterfaceFunctionArgOrRetVal:
						builder.Append("Execute_").Append(function.SourceName);
						break;
					case UhtPropertyTextType.EventFunctionArgOrRetVal:
						builder.Append(function.MarshalAndCallName);
						break;
					case UhtPropertyTextType.ClassFunctionArgOrRetVal:
						builder.Append(function.CppImplName);
						break;
					default:
						throw new UhtIceException("Unexpected type text");
				}
			}

			AppendParameters(builder, function, textType, extraParam, false);

			if (textType != UhtPropertyTextType.InterfaceFunctionArgOrRetVal)
			{
				if (!isDelegate && function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const))
				{
					builder.Append(" const");
				}

				if (isInterface && isDeclaration)
				{
					// all methods in interface classes are pure virtuals
					if (isK2Override)
					{
						// For BlueprintNativeEvent methods we emit a stub implementation. This allows Blueprints that implement the interface class to be nativized.
						builder.Append(" {");
						if (returnProperty != null)
						{
							if (returnProperty is UhtByteProperty byteProperty && byteProperty.Enum != null)
							{
								builder.Append(" return TEnumAsByte<").Append(byteProperty.Enum.CppType).Append(">(").AppendNullConstructorArg(returnProperty, false).Append("); ");
							}
							else if (returnProperty is UhtEnumProperty enumProperty && enumProperty.Enum.CppForm != UhtEnumCppForm.EnumClass)
							{
								builder.Append(" return TEnumAsByte<").Append(enumProperty.Enum.CppType).Append(">(").AppendNullConstructorArg(returnProperty, false).Append("); ");
							}
							else
							{
								builder.Append(" return ").AppendNullConstructorArg(returnProperty, false).Append("; ");
							}
						}
						builder.Append('}');
					}
					else
					{
						builder.Append("=0");
					}
				}
			}
			builder.Append(endl);

			return builder;
		}

		// Find the UFUNCTION associated with this event so that we can check
		// if there is a BP implementation of it
		protected static StringBuilder AppendFindUFunction(StringBuilder builder, UhtClass classObj, UhtFunction function, int tabs, string endl = "\r\n")
		{
			builder
				.AppendTabs(tabs)
				.Append("UFunction* Func = FindFunctionChecked(")
				.Append("NAME_")
				.Append(classObj.SourceName)
				.Append('_')
				.Append(function.EngineName)
				.Append(");")
				.Append(endl);

			return builder;
		}

		protected static StringBuilder AppendEventFunctionPrologue(StringBuilder builder, UhtFunction function, string functionName, int tabs, string endl, bool addEventParameterStruct, bool bAddFunctionScopeBracket = true)
		{
			if (bAddFunctionScopeBracket)
			{
				builder.AppendTabs(tabs).Append('{').Append(endl);
			}

			if (function.Children.Count == 0)
			{
				return builder;
			}

			++tabs;

			if (addEventParameterStruct)
			{
				AppendEventParameter(builder, function, functionName, UhtPropertyTextType.EventParameterMember, true, tabs, "\r\n");
			}

			string eventParameterStructName = GetEventStructParametersName(function.Outer, functionName);

			builder.AppendTabs(tabs).Append(eventParameterStructName).Append(" Parms;").Append(endl);

			// Declare a parameter struct for this event/delegate and assign the struct members using the values passed into the event/delegate call.
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					if (property.IsStaticArray)
					{
						builder.AppendTabs(tabs).Append("FMemory::Memcpy(Parm.")
							.Append(property.SourceName).Append(',')
							.Append(property.SourceName).Append(",sizeof(Parms.")
							.Append(property.SourceName).Append(");").Append(endl);
					}
					else
					{
						builder.AppendTabs(tabs).Append("Parms.").Append(property.SourceName).Append('=').Append(property.SourceName);
						if (property is UhtBoolProperty)
						{
							builder.Append(" ? true : false");
						}
						builder.Append(';').Append(endl);
					}
				}
			}
			return builder;
		}

		protected static StringBuilder AppendEventFunctionEpilogue(StringBuilder builder, UhtFunction function, int tabs, string endl, bool bAddFunctionScopeBracket = true)
		{
			++tabs;

			// Out parm copying.
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					if (property.PropertyFlags.HasExactFlags(EPropertyFlags.ConstParm | EPropertyFlags.OutParm, EPropertyFlags.OutParm))
					{
						if (property.IsStaticArray)
						{
							builder
								.AppendTabs(tabs)
								.Append("FMemory::Memcpy(&")
								.Append(property.SourceName)
								.Append(",&Parms.")
								.Append(property.SourceName)
								.Append(",sizeof(")
								.Append(property.SourceName)
								.Append("));").Append(endl);
						}
						else
						{
							builder
								.AppendTabs(tabs)
								.Append(property.SourceName)
								.Append("=Parms.")
								.Append(property.SourceName)
								.Append(';').Append(endl);
						}
					}
				}
			}

			// Return value.
			UhtProperty? returnProperty = function.ReturnProperty;
			if (returnProperty != null)
			{
				// Make sure uint32 -> bool is supported
				if (returnProperty is UhtBoolProperty)
				{
					builder
						.AppendTabs(tabs)
						.Append("return !!Parms.")
						.Append(returnProperty.SourceName)
						.Append(';').Append(endl);
				}
				else
				{
					builder
						.AppendTabs(tabs)
						.Append("return Parms.")
						.Append(returnProperty.SourceName)
						.Append(';').Append(endl);
				}
			}

			--tabs;

			if (bAddFunctionScopeBracket)
			{
				builder.AppendTabs(tabs).Append('}').Append(endl);
			}

			return builder;
		}

		protected static StringBuilder AppendParameters(StringBuilder builder, UhtFunction function, UhtPropertyTextType textType, string? extraParameter, bool skipParameterName)
		{
			bool needsSeparator = false;

			builder.Append('(');

			if (extraParameter != null)
			{
				builder.Append(extraParameter);
				needsSeparator = true;
			}

			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					if (needsSeparator)
					{
						builder.Append(", ");
					}
					else
					{
						needsSeparator = true;
					}
					builder.AppendFullDecl(property, textType, skipParameterName);
				}
			}

			builder.Append(')');
			return builder;
		}

		protected static bool IsRpcFunction(UhtFunction function)
		{
			return
				function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native) &&
				!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CustomThunk);
		}

		protected static bool IsCallbackFunction(UhtFunction function)
		{
			return function.FunctionFlags.HasAnyFlags(EFunctionFlags.Event) && function.SuperFunction == null;
		}

		/// <summary>
		/// Determines whether the glue version of the specified native function should be exported.
		/// </summary>
		/// <param name="function">The function to check</param>
		/// <returns>True if the glue version of the function should be exported.</returns>
		protected static bool ShouldExportFunction(UhtFunction function)
		{
			// Always create a stub for verse native callables
			if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable))
			{
				return true;
			}

			// export any script stubs for native functions declared in interface classes
			bool isBlueprintNativeEvent = function.FunctionFlags.HasAllFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.Native);
			if (!isBlueprintNativeEvent)
			{
				if (function.Outer != null)
				{
					if (function.Outer is UhtClass classObj)
					{
						if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
						{
							return true;
						}
					}
				}
			}

			// always export if the function is static
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Static))
			{
				return true;
			}

			// don't export the function if this is not the original declaration and there is
			// at least one parent version of the function that is declared native
			for (UhtFunction? parentFunction = function.SuperFunction; parentFunction != null; parentFunction = parentFunction.SuperFunction)
			{
				if (parentFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Type of constructor on the class regardless of explicit or generated.
		/// </summary>
		protected enum ConstructorType
		{
			ObjectInitializer,
			Default,
			ForbiddenDefault,
		}

		/// <summary>
		/// Return the type of constructor the class will have regardless of if one has been explicitly 
		/// declared or will be generated.
		/// </summary>
		/// <param name="classObj">Class in question</param>
		/// <returns>Constructor type</returns>
		protected static ConstructorType GetConstructorType(UhtClass classObj)
		{
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasConstructor))
			{

				// Assume super class has OI constructor, this may not always be true but we should always be able to check this.
				// In any case, it will default to old behavior before we even checked this.
				bool superClassObjectInitializerConstructorDeclared = true;
				UhtClass? superClass = classObj.SuperClass;
				if (superClass != null)
				{
					if (!superClass.HeaderFile.IsNoExportTypes)
					{
						superClassObjectInitializerConstructorDeclared = GetConstructorType(superClass) == ConstructorType.ObjectInitializer;
					}
				}

				if (superClassObjectInitializerConstructorDeclared)
				{
					return ConstructorType.ObjectInitializer;
				}
				else
				{
					return ConstructorType.Default;
				}
			}
			else
			{
				if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasObjectInitializerConstructor))
				{
					return ConstructorType.ObjectInitializer;
				}
				else if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDefaultConstructor))
				{
					return ConstructorType.Default;
				}
				else
				{
					return ConstructorType.ForbiddenDefault;
				}
			}
		}

		/// <summary>
		/// A list of sparse data structs that should export accessors for this class.
		/// This list excludes anything that has already been exported by the super class.
		/// </summary>
		/// <param name="classObj">Class in question</param>
		/// <returns>Enumeration of structs</returns>
		protected static IEnumerable<UhtScriptStruct> GetSparseDataStructsToExport(UhtClass classObj)
		{
			string[]? sparseDataTypes = classObj.MetaData.GetStringArray(UhtNames.SparseClassDataTypes);
			if (sparseDataTypes != null)
			{
				HashSet<string> baseSparseDataTypes = new(classObj.SuperClass?.MetaData.GetStringArray(UhtNames.SparseClassDataTypes) ?? [], StringComparer.OrdinalIgnoreCase);
				foreach (string sparseDataType in sparseDataTypes)
				{
					UhtScriptStruct? sparseScriptStruct = classObj.FindType(UhtFindOptions.EngineName | UhtFindOptions.ScriptStruct | UhtFindOptions.NoSelf, sparseDataType) as UhtScriptStruct;
					while (sparseScriptStruct != null && !baseSparseDataTypes.Contains(sparseScriptStruct.EngineName))
					{
						yield return sparseScriptStruct;
						sparseScriptStruct = sparseScriptStruct.SuperScriptStruct;
					}
				}
			}
		}
		#endregion

		#region Field notify support
		protected static bool NeedFieldNotifyCodeGen(UhtClass classObj)
		{
			return
				!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasCustomFieldNotify) &&
				classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasFieldNotify);
		}

		protected static string GetNotifyTypeName(UhtType notifyType)
		{
			if (notifyType is UhtProperty property)
			{
				return property.SourceName;
			}
			else if (notifyType is UhtFunction function)
			{
				return function.CppImplName;
			}
			else
			{
				throw new UhtIceException("Unexpected type in notification code generation");
			}
		}

		protected static UhtUsedDefineScopes<UhtType> GetFieldNotifyTypes(UhtClass classObj)
		{
			UhtUsedDefineScopes<UhtType> notifyTypes = new(classObj.Children.Where(x =>
			{
				if (x is UhtProperty property)
				{
					return property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.FieldNotify);
				}
				else if (x is UhtFunction function)
				{
					return function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.FieldNotify);
				}
				else
				{
					return false;
				}
			}));

			// We want properties followed by functions
			notifyTypes.Instances.SortBy(x => (x is UhtProperty ? 0 : 1) * (int)UhtDefineScope.ScopeCount + x.DefineScope);
			return notifyTypes;
		}
		#endregion

		#region AutoGettersSetters support
		protected static UhtUsedDefineScopes<UhtProperty> GetAutoGetterSetterProperties(UhtClass classObj)
		{
			UhtUsedDefineScopes<UhtProperty> properties = new(classObj.Properties.Where(
				x => x.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecifiedAuto | UhtPropertyExportFlags.SetterSpecifiedAuto
				)));
			return properties;
		}
		#endregion
	}

	internal static class UhtHaederCodeGeneratorStringBuilderExtensions
	{
		public static int GetMacroLineNumber(this UhtType type)
		{
			return type switch
			{
				UhtClass classObj => classObj.GeneratedBodyLineNumber,
				UhtScriptStruct scriptStruct => scriptStruct.MacroDeclaredLineNumber,
				UhtPartial partial => partial.MacroDeclaredLineNumber,
				UhtFunction function => function.MacroLineNumber,
				_ => throw new UhtException(type, "Can not use given type to create a macro"),
			};
		}

		public static string GetScopeSuffix(this UhtDefineScope definedScope)
		{
			return definedScope switch
			{
				UhtDefineScope.None => "",
				UhtDefineScope.EditorOnlyData => "_EOD",
				UhtDefineScope.Editor => "_EDITOR",
				UhtDefineScope.Tests => "_TESTS",
				UhtDefineScope.VerseVM => "_VM",
				UhtDefineScope.VerseBPVM => "_BPVM",
				_ => $"_{(int)definedScope}",
			};
		}

		public static StringBuilder AppendMacroName(this StringBuilder builder, string fileId, int lineNumber, string macroSuffix)
		{
			builder.Append(fileId).Append('_').Append(lineNumber).Append('_').Append(macroSuffix);
			return builder;
		}

		public static StringBuilder AppendMacroName(this StringBuilder builder, UhtHeaderCodeGenerator generator, int lineNumber, string macroSuffix)
		{
			return builder.AppendMacroName(generator.FileId, lineNumber, macroSuffix);
		}

		public static StringBuilder AppendMacroName(this StringBuilder builder, UhtHeaderCodeGenerator generator, UhtType type, string macroSuffix)
		{
			return builder.AppendMacroName(generator, type.GetMacroLineNumber(), macroSuffix);
		}

		/// <summary>
		/// Append the macro name including an optional suffix to uniquely identify a define scope
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="fileId"></param>
		/// <param name="lineNumber"></param>
		/// <param name="macroSuffix"></param>
		/// <param name="defineScope"></param>
		/// <param name="includeSuffix"></param>
		/// <returns></returns>
		public static StringBuilder AppendMacroName(this StringBuilder builder, string fileId, int lineNumber, string macroSuffix, UhtDefineScope defineScope, bool includeSuffix)
		{
			builder.AppendMacroName(fileId, lineNumber, macroSuffix);
			if (includeSuffix && defineScope != UhtDefineScope.None)
			{
				builder.Append(defineScope.GetScopeSuffix());
			}
			return builder;
		}

		/// <summary>
		/// Append just the overall return type of the native callable verse function
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="function">Function in question</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendVerseNativeCallableReturnType(this StringBuilder builder, UhtFunction function)
		{
			UhtProperty? returnProperty = function.ReturnProperty;
			bool isIncompleteReturn = function.Session.IsIncompleteReturn(returnProperty);
			bool isSuspends = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);
			bool isIfNoRuntimeError = returnProperty != null && !isIncompleteReturn;

			if (isSuspends)
			{
				builder.Append("TVerseTask<");
			}
			else if (isIfNoRuntimeError)
			{
				builder.Append("TIfNoRuntimeError<");
			}

			if (returnProperty == null)
			{
				builder.Append("void");
			}
			else if (isIncompleteReturn)
			{
				builder.Append("EVerseNativeCallResult");
			}
			else
			{
				builder.AppendTokens(returnProperty.TypeTokens.AllTokens);
			}

			if (isSuspends || isIfNoRuntimeError)
			{
				builder.Append("> ");
			}
			else
			{
				builder.Append(' ');
			}

			return builder;
		}

		/// <summary>
		/// Append the complete signature of the native callable verse function
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="classObj">If supplied, used to append the ClassSourceName:: part of the declaration</param>
		/// <param name="function">Function in question</param>
		/// <param name="classNameSuffix">Appended to the end of the class name.  Used to generate the Proxy implementations</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendVerseNativeCallableSignature(this StringBuilder builder, UhtClass? classObj, UhtFunction function, string classNameSuffix)
		{
			builder.AppendVerseNativeCallableReturnType(function);

			if (classObj != null)
			{
				builder.AppendClassSourceNameOrInterfaceName(classObj).Append(classNameSuffix).Append("::");
			}
			builder.Append(function.SourceName).Append("(V_EXEC_CONTEXT_PARAM");
			foreach (UhtType type in function.ParameterProperties.Span)
			{
				if (type is UhtProperty property)
				{
					builder.Append(", ").AppendTokens(property.TypeTokens.AllTokens).Append(' ').Append(property.SourceName);
				}
			}
			builder.Append(')');
			return builder;
		}

		public static StringBuilder AppendVerseNativeCallableTypeDef(this StringBuilder builder, UhtFunction function)
		{
			UhtProperty? returnProperty = function.ReturnProperty;
			bool isIncompleteReturn = function.Session.IsIncompleteReturn(returnProperty);
			bool isSuspends = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);
			if (!isIncompleteReturn && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides) && returnProperty is UhtOptionalProperty optionalReturn)
			{
				returnProperty = optionalReturn.ValueProperty;
			}

			builder.Append("TVerseFunction<");
			if (isSuspends)
			{
				builder.Append("FVerseResult(TVerseCall<");
			}
			if (returnProperty == null)
			{
				builder.Append("void");
			}
			else if (isIncompleteReturn)
			{
				builder.Append("EVerseNativeCallResult");
			}
			else
			{
				builder.AppendTokens(returnProperty.TypeTokens.FullDeclarationTokens);
			}

			builder.Append(isSuspends ? '>' : '(').AppendVerseFunctionArgs(function, false, true, property => builder.AppendTokens(property.TypeTokens.FullDeclarationTokens), () => builder.Append("FDecidesContext")).Append(")>");
			return builder;
		}

		public static StringBuilder AppendVerseFunctionArgs(this StringBuilder builder, UhtFunction function, bool startWithSeparator, bool decidesBeforeReturn, Action<UhtProperty> propertyAction, Action decidesAction)
		{
			bool insertSeparator = startWithSeparator;
			if (function.Session.IsIncompleteReturn(function.ReturnProperty))
			{
				if (decidesBeforeReturn && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides))
				{
					if (insertSeparator)
					{
						builder.Append(", ");
					}
					insertSeparator = true;
					decidesAction();
				}
				{
					if (function.ParameterProperties.Span[^1] is UhtProperty property)
					{
						if (insertSeparator)
						{
							builder.Append(", ");
						}
						insertSeparator = true;
						propertyAction(property);
					}
				}
				if (!decidesBeforeReturn && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides))
				{
					if (insertSeparator)
					{
						builder.Append(", ");
					}
					insertSeparator = true;
					decidesAction();
				}
				foreach (UhtType type in function.ParameterProperties.Span[..^1])
				{
					if (type is UhtProperty property)
					{
						if (insertSeparator)
						{
							builder.Append(", ");
						}
						insertSeparator = true;
						propertyAction(property);
					}
				}
			}
			else
			{
				if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides))
				{
					if (insertSeparator)
					{
						builder.Append(", ");
					}
					insertSeparator = true;
					decidesAction();
				}

				foreach (UhtType type in function.ParameterProperties.Span)
				{
					if (type is UhtProperty property)
					{
						if (insertSeparator)
						{
							builder.Append(", ");
						}
						insertSeparator = true;
						propertyAction(property);
					}
				}
			}
			return builder;
		}
	}
}
