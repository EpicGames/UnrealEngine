// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	using static UhtExpressionFactory;

	/// <summary>
	/// Collection of all registrations for a specific package
	/// </summary>
	internal sealed class UhtRegistrations
	{
		public UhtUsedDefineScopes<UhtEnum> Enumerations { get; } = new();
		public UhtUsedDefineScopes<UhtScriptStruct> ScriptStructs { get; } = new();
		public UhtUsedDefineScopes<UhtScriptStruct> RigVMStructs { get; } = new();
		public UhtUsedDefineScopes<UhtClass> Classes { get; } = new();
		public UhtUsedDefineScopes<UhtPartial> Partials { get; } = new();
	}

	internal sealed class UhtHeaderCodeGeneratorCppFile : UhtHeaderCodeGenerator
	{
		/// <summary>
		/// Construct an instance of this generator object
		/// </summary>
		/// <param name="codeGenerator">The base code generator</param>
		/// <param name="headerFile">Header file being generated</param>
		public UhtHeaderCodeGeneratorCppFile(UhtCodeGenerator codeGenerator, UhtHeaderFile headerFile)
			: base(codeGenerator, headerFile)
		{
		}

#pragma warning disable CA1505 // warning CA1505: 'Generate' has a maintainability index of '8'. Rewrite or refactor the code to increase its maintainability index (MI) above '9'. 

		/// <summary>
		/// For a given UE header file, generated the generated H file
		/// </summary>
		/// <param name="factory">Requesting factory</param>
		public void Generate(IUhtExportFactory factory)
		{
			ref UhtCodeGenerator.HeaderInfo headerInfo = ref HeaderInfos[HeaderFile.HeaderFileTypeIndex];
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
				StringBuilder builder = borrower.StringBuilder;

				builder.Append(HeaderCopyright);
				builder.Append(RequiredCPPIncludes);
				builder.Append("#include \"").Append(headerInfo.IncludePath).Append("\"\r\n");

				bool addedStructuredArchiveFromArchiveHeader = false;
				bool addedArchiveUObjectFromStructuredArchiveHeader = false;
				bool addedCoreNetHeader = false;
				bool addedRigVMHeaders = false;
				HashSet<UhtHeaderFile> addedIncludes = [];
				List<string> includesToAdd = [];
				addedIncludes.Add(HeaderFile);

				if (headerInfo.NeedsFastArrayHeaders)
				{
					includesToAdd.Add("Net/Serialization/FastArraySerializerImplementation.h");
				}
				if (headerInfo.NeedsVerseCodeGen)
				{
					includesToAdd.Add("VerseVM/VVMVerseNativeTypeDesc.h");
					includesToAdd.Add("VerseVM/VVMUECodeGen.h");
					includesToAdd.Add("VerseInteropUtils.h");
				}

				if (HeaderFile.Children.Any(x => x is UhtScriptStruct structObj && structObj.IsVerseField))
				{
					includesToAdd.Add("VerseVM/VVMVerseStruct.h");
				}
				if (HeaderFile.Children.Any(x => x is UhtClass classObj && classObj.IsVerseField))
				{
					includesToAdd.Add("VerseVM/VVMVerseClass.h");
					includesToAdd.Add("VerseVM/VVMVerseFunction.h");
				}
				if (HeaderFile.Children.Any(x => x is UhtEnum enumObj && enumObj.IsVerseField))
				{
					includesToAdd.Add("VerseVM/VVMVerseEnum.h");
				}
				if (HeaderFile.Children.Any(x => x is UhtStruct structObj && x.Children.Any(y => y is UhtVerseValueProperty)))
				{
					includesToAdd.Add("VerseVM/VBPVMDynamicProperty.h");  // Only needed when WITH_VERSE_BPVM
					includesToAdd.Add("UObject/VerseValueProperty.h");
				}
				if (HeaderFile.Children.Any(x => x is UhtStruct structObj && x.Children.Any(y => y is UhtVerseClassProperty)))
				{
					includesToAdd.Add("UObject/VerseClassProperty.h");
				}

				// Collect unique partial class names from all partials in this file
				Dictionary<string, UhtClass> partialClassNames = [];

				foreach (UhtType type in HeaderFile.Children)
				{
					if (type is UhtPartial partial)
					{
						if (partialClassNames.Count == 0)
						{
							includesToAdd.Add("UObject/UObjectPartials.h");
						}
						partialClassNames.TryAdd(partial.OwnerClassName, partial.OwnerClass!);
					}

					if (type is UhtStruct structObj)
					{
						// Functions
						foreach (UhtFunction function in structObj.Functions)
						{
							if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CppStatic) && function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate))
							{
								if (!addedCoreNetHeader)
								{
									includesToAdd.Add("UObject/CoreNet.h");
									addedCoreNetHeader = true;
								}
							}

							bool requireIncludeForClasses = IsRpcFunction(function) && ShouldExportFunction(function);

							foreach (UhtProperty property in function.Properties)
							{
								AddIncludeForProperty(property, requireIncludeForClasses, addedIncludes, includesToAdd);
							}

							foreach (UhtType parameter in function.ParameterProperties.Span)
							{
								if (parameter is UhtProperty property
										&& property.NeedsGCBarrierWhenPassedToFunction(function)
										&& property is UhtObjectProperty objectProperty)
								{
									UhtClass uhtClass = objectProperty.Class;
									if (!uhtClass.HeaderFile.IsNoExportTypes && addedIncludes.Add(uhtClass.HeaderFile))
									{
										includesToAdd.Add(HeaderInfos[uhtClass.HeaderFile.HeaderFileTypeIndex].IncludePath);
									}
								}
							}
						}

						// Properties
						foreach (UhtProperty property in structObj.Properties)
						{
							AddIncludeForProperty(property, false, addedIncludes, includesToAdd);
						}
					}

					if (type is UhtScriptStruct scriptStruct)
					{
						if (!addedRigVMHeaders && scriptStruct.RigVMStructInfo is not null)
						{
							includesToAdd.Add("RigVMCore/RigVMFunction.h");
							includesToAdd.Add("RigVMCore/RigVMRegistry.h");
							addedRigVMHeaders = true;
						}
					}

					if (type is UhtClass classObj)
					{
						if (classObj.ClassWithin != Session.UObject && !classObj.ClassWithin.HeaderFile.IsNoExportTypes)
						{
							if (addedIncludes.Add(classObj.ClassWithin.HeaderFile))
							{
								includesToAdd.Add(HeaderInfos[classObj.ClassWithin.HeaderFile.HeaderFileTypeIndex].IncludePath);
							}
						}

						switch (classObj.SerializerArchiveType)
						{
							case UhtSerializerArchiveType.None:
								break;

							case UhtSerializerArchiveType.Archive:
								if (!addedArchiveUObjectFromStructuredArchiveHeader)
								{
									includesToAdd.Add("Serialization/ArchiveUObjectFromStructuredArchive.h");
									addedArchiveUObjectFromStructuredArchiveHeader = true;
								}
								break;

							case UhtSerializerArchiveType.StructuredArchiveRecord:
								if (!addedStructuredArchiveFromArchiveHeader)
								{
									includesToAdd.Add("Serialization/StructuredArchive.h");
									addedStructuredArchiveFromArchiveHeader = true;
								}
								break;
						}
					}
					else
					{
						if (!type.HeaderFile.IsNoExportTypes && addedIncludes.Add(type.HeaderFile))
						{
							includesToAdd.Add(HeaderInfos[type.HeaderFile.HeaderFileTypeIndex].IncludePath);
						}
					}
				}

				includesToAdd.Sort(StringComparerUE.OrdinalIgnoreCase);
				foreach (string include in includesToAdd)
				{
					builder.Append("#include \"").Append(include).Append("\"\r\n");
				}

				builder.Append("\r\n");
				builder.Append(DisableDeprecationWarnings).Append("\r\n");

				if (!Session.IsUsingMultipleCompiledInObjectFormats)
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						builder.Append("static_assert(UE_WITH_CONSTINIT_UOBJECT, \"This generated code can only be compiled with UE_WITH_CONSTINIT_UOBJECT\");");
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						builder.Append("static_assert(!UE_WITH_CONSTINIT_UOBJECT, \"This generated code can only be compiled with !UE_WITH_CONSTINIT_UOBJECT\");");
					}
				}

				builder.Append("\r\n");
				string cleanFileName = HeaderFile.FileNameWithoutExtension.Replace('.', '_');
				builder.Append("void EmptyLinkFunctionForGeneratedCode").Append(cleanFileName).Append("() {}\r\n");

				if (!HeaderFile.References.Declaration.IsEmpty)
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtCodeBlockComment blockComment = new(builder, "Forward Declarations");
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

						foreach (UhtPackage package in HeaderFile.Module.Packages.OrderBy(x => x.SourceName))
						{
							builder.Append(GetExternalDecl(package, UhtSingletonType.ConstInit).AsSpan().TrimStart());
						}
						IEnumerable<UhtObject> decls = HeaderFile.References.Declaration.SortReferences(UhtSingletonTypeFlag.Registered | UhtSingletonTypeFlag.Unregistered | UhtSingletonTypeFlag.ConstInit);
						foreach (UhtObject obj in decls)
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.ConstInit).AsSpan().TrimStart());
						}
					}
				}

				if (!HeaderFile.References.ExternalObjects.IsEmpty)
				{
					using UhtCodeBlockComment blockComment = new(builder, "Cross Module References");
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.AppendLine("// Cross-module objects can only be linked directly in monolithic builds");
						builder.AppendLine("#if IS_MONOLITHIC");
						UhtSingletonTypeFlag filter = UhtSingletonTypeFlag.Registered | UhtSingletonTypeFlag.Unregistered | UhtSingletonTypeFlag.ConstInit;
						foreach (UhtObject obj in HeaderFile.References.ExternalObjects.SortReferences(filter))
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.ConstInit).AsSpan().TrimStart());
						}
						builder.AppendLine("#endif // IS_MONOLITHIC");
					}

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

						foreach (UhtObject obj in HeaderFile.References.ExternalObjects.SortReferences(UhtSingletonTypeFlag.Registered))
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.Registered).AsSpan().TrimStart());
						}

						foreach (UhtObject obj in HeaderFile.References.ExternalObjects.SortReferences(UhtSingletonTypeFlag.Unregistered))
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.Unregistered).AsSpan().TrimStart());
						}
					}
				}

				if (!HeaderFile.References.InternalObjects.IsEmpty)
				{
					using UhtCodeBlockComment blockComment = new(builder, "Same Module References");
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						UhtSingletonTypeFlag filter = UhtSingletonTypeFlag.Registered | UhtSingletonTypeFlag.Unregistered | UhtSingletonTypeFlag.ConstInit;
						foreach (UhtObject obj in HeaderFile.References.InternalObjects.SortReferences(filter))
						{
							if (obj is UhtPartial)
							{
								continue;
							}
							builder.Append(GetExternalDecl(obj, UhtSingletonType.ConstInit).AsSpan().TrimStart());
						}
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

						foreach (UhtObject obj in HeaderFile.References.InternalObjects.SortReferences(UhtSingletonTypeFlag.Registered))
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.Registered).AsSpan().TrimStart());
						}

						foreach (UhtObject obj in HeaderFile.References.InternalObjects.SortReferences(UhtSingletonTypeFlag.Unregistered))
						{
							builder.Append(GetExternalDecl(obj, UhtSingletonType.Unregistered).AsSpan().TrimStart());
						}
					}
				}

				if (headerInfo.NeedsVerseCodeGen)
				{
					// Verse registration
					if (Module.Module.HasVerse)
					{
						builder.AppendLine($$"""
							namespace {{VniModuleStatics(Module)}}
							{
								extern const FVniPackageDesc {{VniPackageDesc(Module)}};
							}
							""");
					}
				}

				int generatedBodyStart = builder.Length;

				builder.AppendLinkingMacros(Session, HeaderFile.References.Modules);
				builder.AppendLine($$"""
					#define UHT_STRUCT_BASE(INIT) UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>(UE::Private::AsStructBaseChain(INIT))
					""");

				Dictionary<UhtPackage, UhtRegistrations> packageRegistrations = [];
				foreach (UhtField field in HeaderFile.References.ExportTypes)
				{
					if (field is UhtEnum enumObj)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, enumObj.DefineScope);
						AppendEnum(builder, enumObj);
						GetRegistrations(packageRegistrations, field).Enumerations.Add(enumObj);
					}
					else if (field is UhtScriptStruct scriptStruct)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, scriptStruct.DefineScope);
						AppendScriptStruct(builder, scriptStruct);
						if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
						{
							GetRegistrations(packageRegistrations, field).ScriptStructs.Add(scriptStruct);
							if (scriptStruct.RigVMStructInfo != null && scriptStruct.RigVMStructInfo.Methods.Count > 0)
							{
								GetRegistrations(packageRegistrations, field).RigVMStructs.Add(scriptStruct);
							}
						}
					}
					else if (field is UhtPartial partial)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, partial.DefineScope);
						AppendPartial(builder, partial);
						GetRegistrations(packageRegistrations, field).Partials.Add(partial);
					}
					else if (field is UhtFunction function)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, function.DefineScope);
						AppendFunction(builder, function, false, false);
					}
					else if (field is UhtClass classObj)
					{
						if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Intrinsic))
						{
							// Collect the functions to be exported
							UhtUsedDefineScopes<UhtFunction> functions = new(classObj.Functions);
							functions.Instances.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.EngineName, y.EngineName));

							// Output any functions
							foreach (UhtFunction classFunction in functions.Instances)
							{
								AppendClassFunction(builder, classObj, classFunction);
							}

							using UhtCodeBlockComment blockComment = new(builder, field);
							using UhtMacroBlockEmitter macroBlockEmitter = new(builder, classObj.DefineScope);
							AppendClass(builder, classObj, functions);
							GetRegistrations(packageRegistrations, field).Classes.Add(classObj);

							if (classObj.ClassType == UhtClassType.Interface && classObj.IsVerseField)
							{
								foreach (UhtClass baseClass in classObj.FlattenedVerseInterfaces)
								{
									AppendNativeInterfaceVerseProxyFunctions(builder, classObj, baseClass.AlternateObject as UhtClass);
								}
							}
						}
					}
				}

				foreach (UhtPackage package in Module.Packages)
				{
					if (!packageRegistrations.TryGetValue(package, out UhtRegistrations? registrations))
					{
						continue;
					}

					string name = $"Z_CompiledInDeferFile_{headerInfo.FileId}_{PackageInfos[package.PackageTypeIndex].StrippedName}";

					IoHash combinedHash = new(UInt64.MaxValue, UInt64.MaxValue, UInt32.MaxValue);

					using UhtCodeBlockComment blockComment = new(builder, "Registration");
					using UhtStaticsEmitter statics = new(builder, $"{name}_Statics");
					builder.Append($"struct UHT_STATICS\r\n");
					builder.Append("{\r\n");

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

						// Registration for RigVM functions
						builder.AppendInstances(registrations.RigVMStructs, 
							builder => builder.Append("\tstatic inline constinit FRigVMCompiledInStruct RigVMStructs[] = {\r\n"),
							(builder, scriptStruct) => builder.Append($"\t\t{{ .Struct = {ConstInitSingletonRef(this, scriptStruct)}, ")
								 .Append(".Functions = MakeArrayView(").AppendSingletonName(this, scriptStruct, UhtSingletonType.Statics).Append("::RigVMFunctions), ")
								.Append(" },\r\n"),
							builder => builder.Append("\t};\r\n"));
					}

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						// Non-constinit version
						builder.AppendInstances(registrations.Enumerations, 
							builder => builder.Append("\tstatic constexpr FEnumRegisterCompiledInInfo EnumInfo[] = {\r\n"),
							(builder, enumObj) =>
							{
								IoHash hash = ObjectInfos[enumObj.ObjectTypeIndex].Hash;
								builder
									.Append($"\t\t{{ {GetSingletonName(enumObj, UhtSingletonType.Registered)}, ")
									.Append($"TEXT(\"{enumObj.EngineName}\"), ")
									.Append($"&ZRIE_{enumObj.SourceName}, ")
									.Append($"CONSTRUCT_RELOAD_VERSION_INFO(FEnumReloadVersionInfo, {(uint)hash}U) }},\r\n");
								combinedHash = IoHash.Combine(combinedHash, hash);
							},
							builder => builder.Append("\t};\r\n"));

						builder.AppendInstances(registrations.ScriptStructs, 
							builder => builder.Append("\tstatic constexpr FStructRegisterCompiledInInfo ScriptStructInfo[] = {\r\n"),
							(builder, scriptStruct) =>
							{
								IoHash hash = ObjectInfos[scriptStruct.ObjectTypeIndex].Hash;
								builder
									.Append($"\t\t{{ {GetSingletonName(scriptStruct, UhtSingletonType.Registered)}, ")
									.Append($"{GetSingletonName(scriptStruct, UhtSingletonType.Statics)}::NewStructOps, ")
									.Append($"TEXT(\"{scriptStruct.EngineName}\"),")
									.Append($"&Z_Registration_Info_UScriptStruct_{scriptStruct.SourceName}, ")
									.Append($"CONSTRUCT_RELOAD_VERSION_INFO(FStructReloadVersionInfo, sizeof({scriptStruct.FullyQualifiedSourceName}), {(uint)hash}U) }},\r\n");
								combinedHash = IoHash.Combine(combinedHash, hash);
							},
							builder => builder.Append("\t};\r\n"));

						builder.AppendInstances(registrations.Classes, 
							builder => builder.Append("\tstatic constexpr FClassRegisterCompiledInInfo ClassInfo[] = {\r\n"),
							(builder, classObj) =>
							{
								IoHash hash = ObjectInfos[classObj.ObjectTypeIndex].Hash;
								builder
									.Append($"\t\t{{ {GetSingletonName(classObj, UhtSingletonType.Registered)}, ")
									.Append($"TEXT(\"{(classObj.IsVerseField ? classObj.EngineName : classObj.SourceName)}\"), ")
									.Append($"&Z_Registration_Info_UClass_{classObj.SourceName}, ")
									.Append($"CONSTRUCT_RELOAD_VERSION_INFO(FClassReloadVersionInfo, sizeof({classObj.FullyQualifiedSourceName}), {(uint)hash}U) }},\r\n");
								combinedHash = IoHash.Combine(combinedHash, hash);
							},
							builder => builder.Append("\t};\r\n"));

						builder.AppendInstances(registrations.Partials,
							builder => builder.Append("\tstatic constexpr FPartialRegisterCompiledInInfo PartialInfo[] = {\r\n"),
							(builder, partial) =>
							{
								IoHash hash = ObjectInfos[partial.ObjectTypeIndex].Hash;
								builder.Append("\t\t{ ")
									.Append($"[](ETypeConstructPhase) -> UE::CoreUObject::Private::FPartialClass* {{ return &{GetSingletonName(partial, UhtSingletonType.Registered)}_PartialParams; }},")
									.Append($"TEXT(\"{partial.EngineName}\"), ")
									.Append($"&Z_Registration_Info_Partial_{partial.SourceName}, ")
									.Append($"CONSTRUCT_RELOAD_VERSION_INFO(FPartialReloadVersionInfo, sizeof({partial.FullyQualifiedSourceName}), {(uint)hash}U)")
									.Append(" },\r\n");
								combinedHash = IoHash.Combine(combinedHash, hash);
							},
							builder => builder.Append("\t};\r\n"));
					}

					builder.Append($"}}; // UHT_STATICS \r\n");

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						if (!registrations.RigVMStructs.IsEmpty)
						{
							builder.Append($"static FRegisterRigVMStructs RigVM_{name}{{\r\n")
								.AppendArrayView(registrations.RigVMStructs, UhtNames.StaticsRigVMStructsId, 1, "\r\n")
								.Append("};\r\n");
						}
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("static FRegisterCompiledInInfo ")
							.Append(name)
							.Append($"_{combinedHash}{{\r\n")
							.Append("\tTEXT(\"")
							.Append(package.EngineName)
							.Append("\"),\r\n")
							.AppendArrayPtrAndCountLine(registrations.Classes, UhtNames.StaticsClassInfoId, 1, ",\r\n")
							.AppendArrayPtrAndCountLine(registrations.ScriptStructs, UhtNames.StaticsScriptStructInfoId, 1, ",\r\n")
							.AppendArrayPtrAndCountLine(registrations.Enumerations, UhtNames.StaticsEnumInfoId, 1, ",\r\n")
							.AppendArrayPtrAndCountLine(registrations.Partials, UhtNames.StaticsPartialInfoId, 1, ",\r\n")
							.Append("};\r\n");
					}
				}

				builder.AppendLinkingMacrosUndef(Session, HeaderFile.References.Modules);
				builder.AppendLine($$"""
					#undef UHT_STRUCT_BASE
					""");

				if (Session.IncludeDebugOutput)
				{
					builder.Append("#if 0\r\n");
					IReadOnlyList<string> sorted = HeaderFile.References.Declaration.GetSortedReferences(GetExternalDecl);
					foreach (string declaration in sorted)
					{
						builder.Append(declaration);
					}
					builder.Append("#endif\r\n");
				}

				int generatedBodyEnd = builder.Length;

				builder.Append("\r\n");
				builder.Append(EnableDeprecationWarnings).Append("\r\n");

				{
					using UhtRentedPoolBuffer<char> borrowBuffer = builder.RentPoolBuffer();
					string cppFilePath = factory.MakePath(HeaderFile, ".gen.cpp");
					StringView generatedBody = new(borrowBuffer.Buffer.Memory);
					if (SaveExportedHeaders)
					{
						factory.CommitOutput(cppFilePath, generatedBody);
					}

					// Save the hash of the generated body 
					HeaderInfos[HeaderFile.HeaderFileTypeIndex].BodyHash = IoHash.Compute(generatedBody.Span[generatedBodyStart..generatedBodyEnd]);
				}
			}
		}
#pragma warning restore CA1505 // warning CA1505: 'Generate' has a maintainability index of '8'. Rewrite or refactor the code to increase its maintainability index (MI) above '9'. 

		private void AddIncludeForType(UhtProperty uhtProperty, bool requireIncludeForClasses, HashSet<UhtHeaderFile> addedIncludes, IList<string> includesToAdd)
		{
			if (uhtProperty is UhtStructProperty structProperty)
			{
				UhtScriptStruct scriptStruct = structProperty.ScriptStruct;
				if (!scriptStruct.HeaderFile.IsNoExportTypes && addedIncludes.Add(scriptStruct.HeaderFile))
				{
					includesToAdd.Add(HeaderInfos[scriptStruct.HeaderFile.HeaderFileTypeIndex].IncludePath);
				}
			}
			else if (requireIncludeForClasses && uhtProperty is UhtClassProperty classProperty)
			{
				UhtClass uhtClass = classProperty.Class;
				if (!uhtClass.HeaderFile.IsNoExportTypes && addedIncludes.Add(uhtClass.HeaderFile))
				{
					includesToAdd.Add(HeaderInfos[uhtClass.HeaderFile.HeaderFileTypeIndex].IncludePath);
				}
			}
		}

		private void AddIncludeForProperty(UhtProperty property, bool requireIncludeForClasses, HashSet<UhtHeaderFile> addedIncludes, IList<string> includesToAdd)
		{
			AddIncludeForType(property, requireIncludeForClasses, addedIncludes, includesToAdd);

			if (property is UhtContainerBaseProperty containerProperty)
			{
				AddIncludeForType(containerProperty.ValueProperty, false, addedIncludes, includesToAdd);
			}

			if (property is UhtMapProperty mapProperty)
			{
				AddIncludeForType(mapProperty.KeyProperty, false, addedIncludes, includesToAdd);
			}
		}

		private static string GetUEnumEnumUnderlyingType(UhtEnumUnderlyingType underlyingType)
		{
			switch (underlyingType)
			{
				case UhtEnumUnderlyingType.Int8:
					return "UEnum::EUnderlyingType::int8";
				case UhtEnumUnderlyingType.Int16:
					return "UEnum::EUnderlyingType::int16";
				case UhtEnumUnderlyingType.Int32:
				case UhtEnumUnderlyingType.Int:
				case UhtEnumUnderlyingType.Unspecified:
					return "UEnum::EUnderlyingType::int32";
				case UhtEnumUnderlyingType.Int64:
					return "UEnum::EUnderlyingType::int64";
				case UhtEnumUnderlyingType.Uint8:
					return "UEnum::EUnderlyingType::uint8";
				case UhtEnumUnderlyingType.Uint16:
					return "UEnum::EUnderlyingType::uint16";
				case UhtEnumUnderlyingType.Uint32:
					return "UEnum::EUnderlyingType::uint32";
				case UhtEnumUnderlyingType.Uint64:
					return "UEnum::EUnderlyingType::uint64";
				default:
					throw new UhtIceException("1PH Unexpected underlying enum type: " + underlyingType.ToString());
			}
		}

		private StringBuilder AppendEnum(StringBuilder builder, UhtEnum enumObj)
		{
			const string ObjectFlags = "RF_Public|RF_Transient|RF_MarkAsNative";
			string singletonName = GetSingletonName(enumObj, UhtSingletonType.Registered);
			string registrationName = $"ZRIE_{enumObj.SourceName}";

			using UhtStaticsEmitter statics = new(builder, this, enumObj);

			string enumDisplayNameFn = enumObj.MetaData.GetValueOrDefault(UhtNames.EnumDisplayNameFn);
			if (enumDisplayNameFn.Length == 0)
			{
				enumDisplayNameFn = "nullptr";
			}

			// If we don't have a zero 0 then we emit a static assert to verify we have one
			if (!enumObj.IsValidEnumValue(0) && enumObj.MetaData.ContainsKey(UhtNames.BlueprintType))
			{
				bool hasUnparsedValue = enumObj.EnumValues.Exists(x => x.Value == -1);
				if (hasUnparsedValue)
				{
					builder.Append("static_assert(");
					bool doneFirst = false;
					foreach (UhtEnumValue value in enumObj.EnumValues)
					{
						if (value.Value == -1)
						{
							if (doneFirst)
							{
								builder.Append("||");
							}
							doneFirst = true;
							builder.Append("!int64(").Append(value.Name).Append(')');
						}
					}
					builder.Append(", \"'").Append(enumObj.SourceName).Append("' does not have a 0 entry!(This is a problem when the enum is initialized by default)\");\r\n");
				}
			}
			if (Module.Module.AlwaysExportEnums)
			{
				builder.Append("template<> ").Append(Module.NonAttributedApi).Append("UEnum* StaticEnum<").Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType).Append(">()\r\n");
			}
			else
			{
				builder.Append("template<> ").Append("UEnum* StaticEnum<").Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType).Append(">()\r\n");
			}
			builder.Append("{\r\n");
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("\treturn ").AppendConstInitSingletonRef(this, enumObj).Append(";\r\n");
			}
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("\treturn ").Append(singletonName).Append("(ETypeConstructPhase::Outer);\r\n");
			}
			builder.Append("}\r\n");

			if (enumObj.IsVerseField)
			{
				builder.Append("V_DEFINE_IMPORTED_ENUM(").Append(Module.Api.TrimEnd()).Append(", \"").AppendVerseUEPackageName(enumObj).Append("\", \"").Append(enumObj.GetVerseScopeAndName(UhtVerseNameMode.Default)).Append("\", ").Append(enumObj.SourceName).Append(");\r\n");
			}

			// Everything from this point on will be part of the definition hash
			int hashCodeBlockStart = builder.Length;

			// Statics declaration
			string? enumeratorNamesArray = null;
			string? enumeratorValuesArray = null;
			string enumFlags = enumObj.EnumFlags.HasAnyFlags(EEnumFlags.Flags) ? "EEnumFlags::Flags" : "EEnumFlags::None";
			bool hasEnumValues = enumObj.EnumValues.Count != 0 || enumObj.GeneratedMaxName is not null;
			{
				builder.Append("struct UHT_STATICS\r\n");
				builder.Append("{\r\n");
				builder.AppendConditionalMetaData(enumObj, UhtNames.TypeId, null, null, 1);

				// Enumerators
				if (hasEnumValues)
				{
					enumeratorNamesArray = "EnumeratorNamesUTF8";
					enumeratorValuesArray = "EnumeratorValues";

					if (enumObj.GeneratedMaxValue is not null)
					{
						builder.Append("\tstatic inline constexpr int64 GeneratedMaxValue = ").Append(enumObj.GeneratedMaxValue).Append(";\r\n");
					}
					else if (enumObj.GeneratedMaxName is not null)
					{
						// Use helper function to generate max value if UHT couldn't parse all values
						builder.Append("\tstatic inline constexpr int64 GeneratedMaxValue = UEnum::CalculateMaxEnumValue({\r\n")
							.AppendEach(enumObj.EnumValues, (builder, value) => builder.Append($"\t\t(int64){enumObj.Namespace.FullSourceName}{value.Name},\r\n"))
							.Append("\t\t(int64)0\r\n") // Emit 0 to handle empty enums and avoid skipping the last comma in the loop above 
							.Append($"\t}}, {enumFlags});\r\n");
					}

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic constexpr const UTF8CHAR* EnumeratorNamesUTF8[]= {\r\n");
						int enumIndex = 0;
						foreach (UhtEnumValue value in enumObj.EnumValues)
						{
							if (!enumObj.MetaData.TryGetValue("OverrideName", enumIndex, out string? keyName))
							{
								keyName = value.Name.ToString();
							}
							builder.Append("\t\tUTF8TEXT(").AppendUTF8LiteralString(keyName).Append("),\r\n");
							++enumIndex;
						}
						if (enumObj.GeneratedMaxName is not null)
						{
							builder.Append("\t\tUTF8TEXT(").AppendUTF8LiteralString(enumObj.GeneratedMaxName).Append("),\r\n");
						}
						builder.Append("\t};\r\n");

						builder.Append("\tstatic inline UE_CONSTINIT_UOBJECT_DECL int64 EnumeratorValues[]= {\r\n")
							.AppendEach(enumObj.EnumValues, (builder, value) => builder.Append($"\t\t(int64){enumObj.Namespace.FullSourceName}{value.Name},\r\n"));
						if (enumObj.GeneratedMaxName is not null)
						{
							builder.Append("\t\tGeneratedMaxValue,\r\n");
						}
						builder.Append("\t};\r\n");
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic constexpr UECodeGen_Private::FEnumeratorParam Enumerators[] = {\r\n");
						int enumIndex = 0;
						foreach (UhtEnumValue value in enumObj.EnumValues)
						{
							if (!enumObj.MetaData.TryGetValue("OverrideName", enumIndex, out string? keyName))
							{
								keyName = value.Name.ToString();
							}
							builder.Append("\t\t{ ").AppendUTF8LiteralString(keyName).Append(", (int64)").Append(enumObj.Namespace.FullSourceName).Append(value.Name).Append(" },\r\n");
							++enumIndex;
						}
						if (enumObj.GeneratedMaxName is not null)
						{
							builder.Append("\t\t{ ").AppendUTF8LiteralString(enumObj.GeneratedMaxName).Append(", GeneratedMaxValue").Append(" },\r\n");
						}
						builder.Append("\t};\r\n");
					}
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					builder.Append("\tstatic const UECodeGen_Private::").AppendEnumParamsType(enumObj).Append(" EnumParams;\r\n");
				}

				if (enumObj.IsVerseField)
				{
					AppendVniTypeDesc(builder, enumObj);
				}

				builder.Append($"}}; // struct UHT_STATICS \r\n");
			}

			// Statics definition
			{
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append($"UE_CONSTINIT_UOBJECT_DECL TNoDestroy<U{enumObj.EngineClassName}> {GetSingletonName(enumObj, UhtSingletonType.ConstInit)}{{NoDestroyConstEval,\r\n");
					builder.AppendConstInitObjectParams(enumObj, this, 1);
					builder.Append("\tUE::CodeGen::ConstInit::FUFieldParams{},\r\n");
					builder.Append("\tUE::CodeGen::ConstInit::FEnumParams{\r\n");
					builder.Append("\t\t.CppTypeStaticUTF8 = UTF8TEXT(").AppendUTF8LiteralString(enumObj.CppType).Append("),\r\n");
					builder.Append(enumeratorNamesArray is null ? null : $"\t\t.StaticNamesUTF8 = MakeArrayView(UHT_STATICS::{enumeratorNamesArray}),\r\n");
					builder.Append(enumeratorValuesArray is null ? null : $"\t\t.EnumValues = MakeArrayView(UHT_STATICS::{enumeratorValuesArray}),\r\n");
					builder.Append($"\t\t.CppForm = (uint8)UEnum::ECppForm::{enumObj.CppForm.ToString()},\r\n");
					builder.Append($"\t\t.UnderlyingType = (uint8){GetUEnumEnumUnderlyingType(enumObj.UnderlyingType)},\r\n");
					builder.Append($"\t\t.EnumFlags = {(enumObj.EnumFlags.HasAnyFlags(EEnumFlags.Flags) ? "EEnumFlags::Flags" : "EEnumFlags::None")},\r\n");
					builder.Append($"\t\t.DisplayNameFn = {enumDisplayNameFn},\r\n");
					builder.Append("\t\tIF_WITH_METADATA(.MetaData = ").AppendMetaDataArrayView(enumObj.MetaData, UhtNames.StaticsTypeId, 0, ",)\r\n");
					builder.Append("\t},");
					if (enumObj.IsVerseField)
					{
						builder.Append("\tUE::CodeGen::ConstInit::FVerseEnumParams{\r\n")
							.Append("\t\t.QualifiedName = UTF8TEXT(\"").AppendVerseScopeAndName(enumObj, UhtVerseNameMode.Qualified).Append("\"),\r\n")
							.Append("\t\t.NativeTypeDesc = &UHT_STATICS::NativeTypeDesc,\r\n")
							.Append("\t},\r\n");
					}
					builder.Append("};\r\n");
				}
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("const UECodeGen_Private::").AppendEnumParamsType(enumObj).Append(" UHT_STATICS::EnumParams = {\r\n");
					if (enumObj.IsVerseField)
					{
						builder.Append("\t{\r\n");
					}
					builder.Append("\t(FTypeConstructFunc*)").Append(GetSingletonName(enumObj.Package, UhtSingletonType.Registered)).Append(",\r\n");
					builder.Append('\t').Append(enumDisplayNameFn).Append(",\r\n");
					builder.Append('\t').AppendUTF8LiteralString(enumObj.SourceName).Append(",\r\n");
					builder.Append('\t').AppendUTF8LiteralString(enumObj.CppType).Append(",\r\n");
					if (hasEnumValues)
					{
						builder.Append("\tUHT_STATICS::Enumerators,\r\n");
						builder.Append('\t').Append(ObjectFlags).Append(",\r\n");
						builder.Append("\tUE_ARRAY_COUNT(UHT_STATICS::Enumerators),\r\n");
					}
					else
					{
						builder.Append('\t').Append("nullptr,\r\n");
						builder.Append('\t').Append(ObjectFlags).Append(",\r\n");
						builder.Append("\t0,\r\n");
					}
					builder.Append('\t').Append(enumObj.EnumFlags.HasAnyFlags(EEnumFlags.Flags) ? "EEnumFlags::Flags" : "EEnumFlags::None").Append(",\r\n");
					builder.Append("\t(uint8)UEnum::ECppForm::").Append(enumObj.CppForm.ToString()).Append(",\r\n");
					builder.Append("\t(uint8)").Append(GetUEnumEnumUnderlyingType(enumObj.UnderlyingType)).Append(",\r\n");
					builder.Append('\t').AppendMetaDataParams(enumObj, UhtNames.StaticsTypeId).Append("\r\n");
					if (enumObj.IsVerseField)
					{
						builder.Append("\t},\r\n");
						builder.Append("\t\"").AppendVerseScopeAndName(enumObj, UhtVerseNameMode.Qualified).Append("\",\r\n");
						builder.Append("\t&UHT_STATICS::NativeTypeDesc,\r\n");
					}
					builder.Append("};\r\n");
				}
			}

			// Registration singleton
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("static FEnumRegistrationInfo ").Append(registrationName).Append(";\r\n");
				builder.Append("UEnum* ").Append(singletonName).Append("(ETypeConstructPhase Phase)\r\n");
				builder.Append("{\r\n");

				builder.Append("\tif (Phase == ETypeConstructPhase::Outer)\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\tif (!").Append(registrationName).Append(".OuterSingleton)\r\n");
				builder.Append("\t\t{\r\n");
				builder.Append("\t\t\t").Append(registrationName).Append(".OuterSingleton = GetStaticEnum(").Append(singletonName).Append(", (UObject*)")
					.Append(GetSingletonName(enumObj.Package, UhtSingletonType.Registered)).Append("(ETypeConstructPhase::Outer), TEXT(\"").Append(enumObj.SourceName).Append("\"));\r\n");
				builder.Append("\t\t}\r\n");
				builder.Append("\t\treturn ").Append(registrationName).Append(".OuterSingleton;\r\n");
				builder.Append("\t}\r\n");

				builder.Append("\tif (!").Append(registrationName).Append(".InnerSingleton)\r\n");
				builder.Append("\t{\r\n");
				if (enumObj.IsVerseField)
				{
					builder.Append("\t\tVerse::CodeGen::Private::ConstructUVerseEnum(").Append(registrationName).Append(".InnerSingleton, UHT_STATICS::EnumParams);\r\n");
				}
				else
				{
					builder.Append("\t\tUECodeGen_Private::ConstructUEnum(").Append(registrationName).Append(".InnerSingleton, UHT_STATICS::EnumParams);\r\n");
				}
				builder.Append("\t}\r\n");
				builder.Append("\treturn ").Append(registrationName).Append(".InnerSingleton;\r\n");
				builder.Append("}\r\n");
			}

			{
				using UhtRentedPoolBuffer<char> borrowBuffer = builder.RentPoolBuffer(hashCodeBlockStart, builder.Length - hashCodeBlockStart);
				ObjectInfos[enumObj.ObjectTypeIndex].Hash = IoHash.Compute<char>(borrowBuffer.Buffer.Memory.Span);
			}
			return builder;
		}

		private StringBuilder AppendScriptStruct(StringBuilder builder, UhtScriptStruct scriptStruct)
		{
			string singletonName = GetSingletonName(scriptStruct, UhtSingletonType.Registered);
			string constinitName = GetSingletonName(scriptStruct, UhtSingletonType.ConstInit);
			string registrationName = $"Z_Registration_Info_UScriptStruct_{scriptStruct.SourceName}";
			List<UhtScriptStruct> noExportStructs = FindNoExportStructs(scriptStruct);

			using UhtStaticsEmitter statics = new(builder, this, scriptStruct);

			// Everything from this point on will be part of the definition hash
			int hashCodeBlockStart = builder.Length;

			// Collect the properties
			UhtPropertyMemberContextImpl context = new(this, scriptStruct);
			UhtUsedDefineScopes<UhtProperty> properties = new(scriptStruct.Properties);

			// Declare the statics structure
			{
				builder.Append("struct UHT_STATICS\r\n");
				builder.Append("{\r\n");

				foreach (UhtScriptStruct noExportStruct in noExportStructs)
				{
					AppendMirrorsForNoexportStruct(builder, noExportStruct, 1);
					builder.Append("\tstatic_assert(sizeof(").Append(noExportStruct.NamespaceExportName).Append(noExportStruct.SourceName).Append(") < MAX_uint16);\r\n");
					builder.Append("\tstatic_assert(alignof(").Append(noExportStruct.NamespaceExportName).Append(noExportStruct.SourceName).Append(") < MAX_uint8);\r\n");
				}

				// Declare a function to access the size/alignment for NoExport cases as well, consteval should prevent this function having code in the binary
				builder.Append("\tstatic inline consteval int32 GetStructSize() { return DataSizeOf<").Append(scriptStruct.NamespaceExportName).Append(scriptStruct.SourceName).Append(">(); }\r\n");
				builder.Append("\tstatic inline consteval int16 GetStructAlignment() { return alignof(").Append(scriptStruct.NamespaceExportName).Append(scriptStruct.SourceName).Append("); }\r\n");

				// Meta data
				builder.AppendConditionalMetaData(scriptStruct, UhtNames.TypeId, context, properties, 1);

				// Properties
				AppendPropertiesDecl(builder, context, properties, 1);

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic UE_CONSTINIT_UOBJECT_DECL inline UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain> StructBases[] = {\r\n");
					if (scriptStruct.SuperScriptStruct is not null)
					{
						List<UhtScriptStruct> supers = [];
						for (UhtScriptStruct? super = scriptStruct.SuperScriptStruct; super is not null; super = super.SuperScriptStruct)
						{
							supers.Add(super);
						}
						foreach (UhtScriptStruct super in supers.AsEnumerable().Reverse())
						{
							builder.AppendLine($"\t\tUHT_STRUCT_BASE({SingletonRef(this, super, UhtSingletonType.ConstInit)}), ");
						}
					}
					builder.AppendLine($"\t\tUHT_STRUCT_BASE({SingletonRef(this, scriptStruct, UhtSingletonType.ConstInit)}), ");
					builder.Append("\t};\r\n");
				}

				if (scriptStruct.RigVMStructInfo != null && scriptStruct.RigVMStructInfo.Methods.Count > 0)
				{
					// Arrays of parameters per method - note that predicates and other functions read different lists
					foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (methodInfo.IsPredicate)
						{
							if (methodInfo.Parameters.Count == 0)
							{
								continue;
							}

							builder.Append($"\tstatic inline constexpr FRigVMCompiledInFunctionArgument RigVMFunctionParameters_{methodInfo.Name}[] = {{\r\n");
							foreach (UhtRigVMParameter parameter in methodInfo.Parameters)
							{
								builder.Append($"\t\t{{ .Name = TEXT(\"{parameter.NameOriginal()}\"), ")
									.Append($".Type = TEXT(\"{parameter.TypeOriginal()}\"), ")
									.Append($".Direction = ERigVMFunctionArgumentDirection::Input }},\r\n");
							}
							builder.Append($"\t\t{{ .Name = TEXT(\"Return\"),")
								.Append($".Type = TEXT(\"{methodInfo.ReturnType}\"),")
								.Append($".Direction = ERigVMFunctionArgumentDirection::Output, }}\r\n");
							builder.Append("\t};\r\n");
						}
						else
						{
							if (scriptStruct.RigVMStructInfo.Members.Count == 0)
							{
								continue;
							}

							builder.Append($"\tstatic inline constexpr FRigVMCompiledInFunctionArgument RigVMFunctionParameters_{methodInfo.Name}[] = {{\r\n");
							foreach (UhtRigVMParameter parameter in scriptStruct.RigVMStructInfo.Members)
							{
								builder.Append($"\t\t{{ .Name = TEXT(\"{parameter.NameOriginal()}\"), ")
									.Append($".Type = TEXT(\"{parameter.TypeOriginal()}\"), ")
									.Append($".Direction = ERigVMFunctionArgumentDirection::Input }},\r\n");
							}
							builder.Append("\t};\r\n");
						}
					}

					builder.Append("\tstatic inline constexpr FRigVMCompiledInFunction RigVMFunctions[] = {\r\n");
					foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (methodInfo.IsPredicate)
						{
							builder.Append($"\t\t{{ .MethodName = TEXT(\"{methodInfo.Name}\"), .Function = nullptr, \r\n");
						}
						else
						{
							builder.Append($"\t\t{{ .MethodName = TEXT(\"{scriptStruct::SourceName}::{methodInfo.Name}\"), \r\n")
								.Append($"\t\t\t.Function = &{scriptStruct.SourceName}::RigVM{methodInfo.Name}, \r\n");
						}

						if ((methodInfo.IsPredicate ? methodInfo.Parameters.Count : scriptStruct.RigVMStructInfo.Members.Count) > 0)
						{
							builder.Append($"\t\t\t.Parameters = MakeArrayView(RigVMFunctionParameters_{methodInfo.Name}),\r\n");
						}
						builder.Append("\t\t},\r\n");
					}
					builder.Append("\t};\r\n");
				}

				// New struct ops
				if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append($"\tstatic inline UE_CONSTINIT_UOBJECT_DECL UScriptStruct::TCppStructOps<{scriptStruct.Namespace.FullSourceName}{scriptStruct.SourceName}> StructOps").Append("{};\r\n");
					}

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic void* NewStructOps()\r\n");
						builder.Append("\t{\r\n");
						builder.Append($"\t\treturn (UScriptStruct::ICppStructOps*)new UScriptStruct::TCppStructOps<{scriptStruct.Namespace.FullSourceName}{scriptStruct.SourceName}>();\r\n");
						builder.Append("\t}\r\n");
					}
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic const UECodeGen_Private::").AppendStructParamsType(scriptStruct).Append(" StructParams;\r\n");
				}

				if (scriptStruct.IsVerseField)
				{
					AppendVniTypeDesc(builder, scriptStruct);
				}
				builder.Append($"}}; // struct UHT_STATICS\r\n");
			}

			if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
			{
				// Inject static assert to verify that we do not add vtable
				if (scriptStruct.SuperScriptStruct != null)
				{
					builder.Append("static_assert(std::is_polymorphic<")
						.Append(scriptStruct.FullyQualifiedSourceName)
						.Append(">() == std::is_polymorphic<")
						.Append(scriptStruct.SuperScriptStruct.FullyQualifiedSourceName)
						.Append(">(), \"USTRUCT ")
						.Append(scriptStruct.FullyQualifiedSourceName)
						.Append(" cannot be polymorphic unless super ")
						.Append(scriptStruct.SuperScriptStruct.FullyQualifiedSourceName)
						.Append(" is polymorphic\");\r\n");
				}

				// Outer singleton
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append(Module.Api).Append("UScriptStruct* ").Append(GetSingletonName(scriptStruct, UhtSingletonType.Registered)).Append("(ETypeConstructPhase)\r\n");
					builder.Append("{\r\n");
					builder.Append("\treturn &").Append(constinitName).Append(";\r\n");
					builder.Append("}\r\n");
				}

				// Inject implementation needed to support auto bindings of fast arrays
				if (ObjectInfos[scriptStruct.ObjectTypeIndex].FastArrayProperty != null)
				{
					// The preprocessor conditional is written here instead of in FastArraySerializerImplementation.h
					// since it may evaluate differently in different modules, triggering warnings in IncludeTool.
					builder.Append("#if defined(UE_NET_HAS_IRIS_FASTARRAY_BINDING) && UE_NET_HAS_IRIS_FASTARRAY_BINDING\r\n");
					builder.Append("UE_NET_IMPLEMENT_FASTARRAY(").Append(scriptStruct.SourceName).Append(");\r\n");
					builder.Append("#else\r\n");
					builder.Append("UE_NET_IMPLEMENT_FASTARRAY_STUB(").Append(scriptStruct.SourceName).Append(");\r\n");
					builder.Append("#endif\r\n");
				}
			}

			// Populate the elements of the static structure
			{
				AppendPropertiesDefs(builder, context, properties, 0);

				// NOTE: This is temporary while the new VM doesn't match the value in the old VM
				string? verseDefaultName = scriptStruct.IsVerseField ? scriptStruct.GetVerseScopeAndName(UhtVerseNameMode.Default) : null;
				string? verseQualifiedName = scriptStruct.IsVerseField ? scriptStruct.GetVerseScopeAndName(UhtVerseNameMode.Qualified) : null;
				uint verseClassFlags = 0;
				if (scriptStruct.ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.IsVersePersonaConstructible))
				{
					verseClassFlags |= (uint)EVerseClassFlags.PersonaConstructible;
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append($"UE_CONSTINIT_UOBJECT_DECL TNoDestroy<U{scriptStruct.EngineClassName}> {constinitName}{{NoDestroyConstEval,\r\n");
					builder.AppendConstInitObjectParams(scriptStruct, this, 1);
					builder.Append("\tUE::CodeGen::ConstInit::FUFieldParams{},\r\n");
					builder.Append("\tUE::CodeGen::ConstInit::FStructParams{\r\n")
						.AppendLine($"\t\t.Super = {ConstInitSingletonRef(this, scriptStruct.SuperScriptStruct)}, ")
						.Append("\t\t.BaseChain = MakeArrayView(UHT_STATICS::StructBases),\r\n")
						.Append("\t\t.ChildProperties =  UHT_STATICS::GetFirstProperty(),\r\n")
						.Append("\t\t.PropertiesSize = UHT_STATICS::GetStructSize(),\r\n")
						.Append("\t\t.MinAlignment = UHT_STATICS::GetStructAlignment(),\r\n")
						.Append("\t\tIF_WITH_METADATA(.MetaData = ").AppendMetaDataArrayView(scriptStruct.MetaData, UhtNames.StaticsTypeId, 0, ",)\r\n")
						.Append("\t},\r\n");
					builder.Append("\tUE::CodeGen::ConstInit::FScriptStructParams{\r\n")
						.Append($"\t\t.StructFlags = EStructFlags(0x{(uint)(scriptStruct.ScriptStructFlags & ~EStructFlags.ComputedFlags):X8}),\r\n")
						.Append(scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native)
							? "\t\t.StructOps = &UHT_STATICS::StructOps,\r\n"
							: null)
						.Append("\t},\r\n");
					if (scriptStruct.IsVerseField)
					{
						uint guidA = Crc.StriHash(scriptStruct.EngineName);
						uint guidB = Crc.StriHash(scriptStruct.Package.EngineName);
						builder.Append("\tUE::CodeGen::ConstInit::FVerseStructParams{\r\n")
							.Append($"\t\t.GuidA = {guidA},\r\n")
							.Append($"\t\t.GuidB = {guidB},\r\n")
							.Append("\t\t.QualifiedName = ");
						if (verseDefaultName!.Equals(verseQualifiedName, StringComparison.Ordinal))
						{
							builder.Append($"UTF8TEXT(\"{verseQualifiedName}\"),\r\n");
						}
						else
						{
							builder.Append($"IF_WITH_VERSE_VM(UTF8TEXT(\"{verseDefaultName}\"), UTF8TEXT(\"{verseQualifiedName}\")),\r\n");
						}
						builder.Append("\t\t.NativeTypeDesc = &UHT_STATICS::NativeTypeDesc,\r\n");
						builder.Append($"\t\t.VerseClassFlags = {verseClassFlags},\r\n");
						builder.Append("},\r\n");
					}
					builder.Append("};\r\n");
				}
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("const UECodeGen_Private::").AppendStructParamsType(scriptStruct).Append(" UHT_STATICS::StructParams = {\r\n");
					if (scriptStruct.IsVerseField)
					{
						builder.Append("\t{\r\n");
					}
					builder.Append("\t(FTypeConstructFunc*)").Append(GetSingletonName(scriptStruct.Package, UhtSingletonType.Registered)).Append(",\r\n");
					builder.Append('\t').Append(GetSingletonName(scriptStruct.SuperScriptStruct, UhtSingletonType.Registered)).Append(",\r\n");
					if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
					{
						builder.Append("\t&NewStructOps,\r\n");
					}
					else
					{
						builder.Append("\tnullptr,\r\n");
					}
					builder.Append('\t').AppendUTF8LiteralString(scriptStruct.EngineName).Append(",\r\n");
					builder.AppendArrayPtrLine(properties, UhtNames.StaticsPropPointersId, 1, ",\r\n");
					builder.AppendArrayCountLine(properties, UhtNames.StaticsPropPointersId, 1, ",\r\n");
					builder.Append("\tDataSizeOf<").Append(scriptStruct.NamespaceExportName).Append(scriptStruct.SourceName).Append(">(),\r\n");
					builder.Append("\talignof(").Append(scriptStruct.NamespaceExportName).Append(scriptStruct.SourceName).Append("),\r\n");
					builder.Append("\tRF_Public|RF_Transient|RF_MarkAsNative,\r\n");
					builder.Append($"\tEStructFlags(0x{(uint)(scriptStruct.ScriptStructFlags & ~EStructFlags.ComputedFlags):X8}),\r\n");
					builder.Append('\t').AppendMetaDataParams(scriptStruct, UhtNames.StaticsTypeId).Append("\r\n");
					if (scriptStruct.IsVerseField)
					{
						builder.Append("\t},\r\n");

						if (verseDefaultName!.Equals(verseQualifiedName, StringComparison.Ordinal))
						{
							builder.Append($"\t\"{verseQualifiedName}\",\r\n");
						}
						else
						{
							builder.Append("\t#if WITH_VERSE_VM\r\n");
							builder.Append($"\t\"{verseDefaultName}\",\r\n");
							builder.Append("\t#else\r\n");
							builder.Append($"\t\"{verseQualifiedName}\",\r\n");
							builder.Append("\t#endif\r\n");
						}
						builder.Append("\t&UHT_STATICS::NativeTypeDesc,\r\n");
						builder.Append($"\t{verseClassFlags},\r\n");
					}
					builder.Append("};\r\n");
				}
			}

			// Generate the registration function
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("static FStructRegistrationInfo ").Append(registrationName).Append(";\r\n");
				builder.Append($"U{scriptStruct.EngineLinkClassName}* {singletonName}(ETypeConstructPhase Phase)\r\n");
				builder.Append("{\r\n");
				string innerSingletonName;
				if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
				{
					builder.Append("\tif (Phase == ETypeConstructPhase::Outer)\r\n");
					builder.Append("\t{\r\n");
					builder.Append("\t\tif (!").Append(registrationName).Append(".OuterSingleton)\r\n");
					builder.Append("\t\t{\r\n");
					builder.Append("\t\t\t")
						.Append(registrationName)
						.Append(".OuterSingleton = GetStaticStruct(")
						.Append(singletonName)
						.Append(", (UObject*)")
						.Append(GetSingletonName(scriptStruct.Package, UhtSingletonType.Registered))
						.Append("(ETypeConstructPhase::Outer), TEXT(\"")
						.Append(scriptStruct.EngineName)
						.Append("\"));\r\n");

					// if this struct has RigVM methods - we need to register the method to our central
					// registry on construction of the static struct
					if (scriptStruct.RigVMStructInfo != null && scriptStruct.RigVMStructInfo.Methods.Count > 0)
					{
						builder.Append($"\t\t\tFRigVMRegistry::Get().RegisterCompiledInStruct({registrationName}.OuterSingleton, MakeArrayView(UHT_STATICS::RigVMFunctions));\r\n");
					}

					builder.Append("\t\t}\r\n");
					builder.Append("\t\treturn ").Append(registrationName).Append(".OuterSingleton;\r\n");
					builder.Append("\t}\r\n");
					innerSingletonName = $"{registrationName}.InnerSingleton";
				}
				else
				{
					builder.Append("\tstatic UScriptStruct* ReturnStruct = nullptr;\r\n");
					innerSingletonName = "ReturnStruct";
				}
				builder.Append("\tif (!").Append(innerSingletonName).Append(")\r\n");
				builder.Append("\t{\r\n");
				if (scriptStruct.IsVerseField)
				{
					builder.Append("\t\tVerse::CodeGen::Private::ConstructUVerseStruct(").Append(innerSingletonName).Append(", UHT_STATICS::StructParams);\r\n");
				}
				else
				{
					builder.Append("\t\tUECodeGen_Private::ConstructUScriptStruct(").Append(innerSingletonName).Append(", UHT_STATICS::StructParams);\r\n");
				}
				builder.Append("\t}\r\n");
				builder.Append($"\treturn CastChecked<U{scriptStruct.EngineClassName}>({innerSingletonName});\r\n");
				builder.Append("}\r\n");
			}

			using (UhtRentedPoolBuffer<char> borrowBuffer = builder.RentPoolBuffer(hashCodeBlockStart, builder.Length - hashCodeBlockStart))
			{
				ObjectInfos[scriptStruct.ObjectTypeIndex].Hash = IoHash.Compute<char>(borrowBuffer.Buffer.Memory.Span);
			}

			// if this struct has RigVM methods we need to implement both the 
			// virtual function as well as the stub method here.
			// The static method is implemented by the user using a macro.
			if (scriptStruct.RigVMStructInfo != null)
			{
				string constPrefix = "";
				if (!scriptStruct.RigVMStructInfo.HasAnyExecuteContextMember || scriptStruct.RigVMStructInfo.IsPure)
				{
					constPrefix = "const ";
				}

				foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
				{
					if (methodInfo.IsPredicate)
					{
						continue;
					}

					builder
						.Append(methodInfo.ReturnType)
						.Append(' ')
						.Append(scriptStruct.Namespace.FullSourceName)
						.Append(scriptStruct.SourceName)
						.Append("::")
						.Append(methodInfo.Name)
						.Append("()\r\n");
					builder.Append("{\r\n");
					if (String.IsNullOrEmpty(scriptStruct.RigVMStructInfo.ExecuteContextMember))
					{
						builder.Append('\t').Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append(" TemporaryExecuteContext;\r\n");
					}
					else
					{
						builder.Append('\t').Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append("& TemporaryExecuteContext = ").Append(scriptStruct.RigVMStructInfo.ExecuteContextMember).Append(";\r\n");
					}

					builder.Append("\tTemporaryExecuteContext.Initialize();\r\n");
					builder.Append('\t');
					if (methodInfo.ReturnType != "void")
					{
						builder.Append("return ");
					}
					builder
						.Append(methodInfo.Name)
						.Append("(TemporaryExecuteContext);\r\n")
						.Append("}\r\n");

					builder
						.Append(methodInfo.ReturnType)
						.Append(' ')
						.Append(scriptStruct.Namespace.FullSourceName)
						.Append(scriptStruct.SourceName)
						.Append("::")
						.Append(methodInfo.Name)
						.Append('(')
						.Append(constPrefix)
						.Append(scriptStruct.RigVMStructInfo.ExecuteContextType)
						.Append("& InExecuteContext)\r\n");
					builder.Append("{\r\n");

					foreach (UhtRigVMParameter parameter in scriptStruct.RigVMStructInfo.Members)
					{
						if (!parameter.RequiresCast())
						{
							continue;
						}
						builder.Append('\t').Append(parameter.CastType).Append(' ').Append(parameter.CastName).Append('(').Append(parameter.Name).Append(");\r\n");
					}

					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append('\t').Append(predicateInfo.Name).Append("Struct ").Append(predicateInfo.Name).Append("Predicate; \r\n");
						}
					}

					builder.Append('\t').Append(methodInfo.ReturnPrefix()).Append("Static").Append(methodInfo.Name).Append("(\r\n");
					builder.Append("\t\tInExecuteContext");
					builder.AppendParameterNames(scriptStruct.RigVMStructInfo.Members, true, ",\r\n\t\t", true);

					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append(", \r\n\t\t").Append(predicateInfo.Name).Append("Predicate");
						}
					}

					builder.Append("\r\n");
					builder.Append("\t);\r\n");
					builder.Append("}\r\n");
				}
			}
			return builder;
		}

		private StringBuilder AppendPartial(StringBuilder builder, UhtPartial partial)
		{
			// Generate functions for this partial first
			// Functions must be generated before the function pointer array
			foreach (UhtFunction function in partial.Functions)
			{
				AppendClassFunction(builder, partial.OwnerClass!, function);
			}

			string registrationName = $"Z_Registration_Info_Partial_{partial.SourceName}";

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				builder.Append("static FPartialRegistrationInfo ").Append(registrationName).Append(";\r\n");
			}

			// Collect the properties
			UhtPropertyMemberContextImpl context = new(this, partial);
			UhtUsedDefineScopes<UhtProperty> properties = new(partial.Properties);

			using UhtStaticsEmitter statics = new(builder, this, partial);// staticsName);

			builder.Append($"struct UHT_STATICS\r\n");
			builder.Append("{\r\n");

			// Meta data
			// No need adding this since the partial "disappears" in runtime so there is way to access the information here
			builder.AppendConditionalMetaData(partial, UhtNames.TypeId, context, properties, 1);

			// Declare properties
			// ROBM: NOTE: This should already create property linked lists in the constinit case, but the head of the list is usually a field in a UStruct
			AppendPropertiesDecl(builder, context, properties, 1);

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit) && partial.Functions.Any())
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("\tstatic consteval UFunction* GetFirstFunction()\r\n");
				builder.Append("\t{\r\n");
				builder.AppendScopeLink(partial.FirstFunction, context, 2, "return ", ";\r\n");
				builder.Append("\t}\r\n");
			}

			builder.Append("};\r\n\r\n");

			// Define properties
			AppendPropertiesDefs(builder, context, properties, 0);
			builder.Append("\r\n");


			// Wrappers
			builder.Append($"static void Construct_{partial.SourceName}(void* Memory) {{ new(Memory) {partial.Namespace.FullSourceName}{partial.SourceName}(); }}\r\n");
			builder.Append($"static void Destruct_{partial.SourceName}(void* Memory) {{ (({partial.Namespace.FullSourceName}{partial.SourceName}*)Memory)->~{partial.SourceName}(); }}\r\n");
			if (partial.HasBeginDestroy)
			{
				// BeginDestroy wrapper (only if partial defines BeginDestroy method)
				// For now, we'll always generate it and have it check for the method
				builder.Append($"static void BeginDestroy_{partial.SourceName}(void* Memory) {{ (({partial.Namespace.FullSourceName}{partial.SourceName}*)Memory)->BeginDestroy(); }}\r\n");
			}

			// Define static PartialOffset member
			builder.Append($"int32 {partial.Namespace.FullSourceName}{partial.SourceName}::PartialOffset;\r\n\r\n");

			// Forward declare the partial class for partial function compilation
			if (partial.Functions.Any())
			{
				builder.Append($"class {partial.OwnerClass!.Namespace.FullSourceName}{partial.OwnerClass.SourceName};\r\n\r\n");
			}

			// Generate function arrays for partial functions (after functions are generated)
			if (!Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit) && partial.Functions.Any())
			{
				bool allIsEditorOnly = partial.Functions.All(f => f.FunctionFlags.HasAnyFlags(EFunctionFlags.EditorOnly));
				using UhtConditionalMacroBlock editorBlock = new(builder, UhtNames.WithEditorOnlyData, allIsEditorOnly);

				// Function constructor pointers (for creating UFunctions)
				builder.Append($"static UFunction* (*PartialFunctionConstructors_{partial.SourceName}[])(ETypeConstructPhase) = {{\r\n");
				foreach (UhtFunction function in partial.Functions)
				{
					builder.Append($"\t&{GetSingletonName(function, UhtSingletonType.Registered)},\r\n");
				}
				builder.Append("};\r\n");

				// Native function bindings (for registering exec functions)
				builder.Append($"static constexpr UE::CodeGen::FClassNativeFunction PartialNativeFunctions_{partial.SourceName}[] = {{\r\n");
				foreach (UhtFunction function in partial.Functions)
				{
					builder.Append($"\t{{ .NameUTF8 = UTF8TEXT(\"{function.EngineName}\"), .Pointer = &{partial.SourceName}::{function.UnMarshalAndCallName} }},\r\n");
				}
				builder.Append("};\r\n\r\n");
			}

			// ROBM: There should be an if/else constinit case here, and the constinit case should just store the head of the linked list of properties etc
			// See UClass/UStruct versions 
			// There should be a global var like `Z_ConstInit_UPartial_{BaseName}_{PartialName}` here that can be linked in the package code generator 
			builder.Append($"UE_CONSTINIT_UOBJECT_DECL UE::CoreUObject::Private::FPartialClass {GetSingletonName(partial, UhtSingletonType.Registered)}_PartialParams\r\n");
			builder.Append("{\r\n");
			builder.Append($"\t.Name = TEXT(\"{partial.SourceName}\"),\r\n");
			builder.Append($"\t.Size = sizeof({partial.Namespace.FullSourceName}{partial.SourceName}),\r\n");
			builder.Append($"\t.Alignment = alignof({partial.Namespace.FullSourceName}{partial.SourceName}),\r\n");

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append($"\t.Class = &{GetSingletonName(partial.OwnerClass, UhtSingletonType.ConstInit)},\r\n");

				if (partial.Properties.Any())
				{
					builder.Append($"\t.FirstProperty = UHT_STATICS::GetFirstProperty(),\r\n");
				}
				if (partial.Functions.Any())
				{
					builder.Append($"\t.FirstChild = UHT_STATICS::GetFirstFunction(),\r\n");
				}
			}

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append($"\t.Class = {GetSingletonName(partial.OwnerClass, UhtSingletonType.Registered)},\r\n");
				if (partial.Properties.Any())
				{
					bool allIsEditorOnly = partial.Properties.All(p => p.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly));
					using UhtConditionalMacroBlock editorBlock = new(builder, UhtNames.WithEditorOnlyData, allIsEditorOnly);
					builder.Append($"\t.Properties = UHT_STATICS::PropPointers,\r\n");
					builder.Append($"\t.NumProperties = UE_ARRAY_COUNT(UHT_STATICS::PropPointers),\r\n");
				}
				if (partial.Functions.Any())
				{
					bool allIsEditorOnly = partial.Functions.All(f => f.FunctionFlags.HasAnyFlags(EFunctionFlags.EditorOnly));
					using UhtConditionalMacroBlock editorBlock = new(builder, UhtNames.WithEditorOnlyData, allIsEditorOnly);
					builder.Append($"\t.FunctionConstructors = PartialFunctionConstructors_{partial.SourceName},\r\n");
					builder.Append($"\t.NativeFunctions = PartialNativeFunctions_{partial.SourceName},\r\n");
					builder.Append($"\t.NumFunctions = {partial.Functions.Count()},\r\n");
				}
			}

			builder.Append($"\t.PartialOffsetPtr = &{partial.Namespace.FullSourceName}{partial.SourceName}::PartialOffset,\r\n");
			builder.Append($"\t.Constructor = &Construct_{partial.SourceName},\r\n");
			builder.Append($"\t.BeginDestroy = {(partial.HasBeginDestroy ? $"BeginDestroy_{partial.SourceName}" : "nullptr")},\r\n");
			builder.Append($"\t.Destructor = &Destruct_{partial.SourceName},\r\n");
			builder.Append($"\t.GetLifetimeReplicatedProps = nullptr, // TODO: Detect if method exists\r\n");
			builder.Append("};\r\n");

			return builder;
		}

		private StringBuilder AppendFunction(StringBuilder builder, UhtFunction function, bool isNoExport, bool isVerseNativeCallable)
		{
			string singletonName = GetSingletonName(function, UhtSingletonType.Registered);
			string constinitName = GetSingletonName(function, UhtSingletonType.ConstInit);
			bool paramsInStatic = isNoExport || !IsCallbackFunction(function);
			bool isNet = function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest | EFunctionFlags.NetResponse);

			string fullSourceName = "";
			string callName = "";

			// Get the outer class - either directly or from the partial's partial class
			if (function.Outer is UhtClass outerClass)
			{
				fullSourceName = outerClass.Namespace.FullSourceName;
				callName = outerClass.NativeFunctionCallName;
			}
			else if (function.Outer is UhtPartial partial)
			{
				fullSourceName = partial.SourceName;
			}

			using UhtStaticsEmitter statics = new(builder, this, function);

			string strippedFunctionName = function.StrippedFunctionName;
			string eventParameters = GetEventStructParametersName(function.Outer, strippedFunctionName);

			// Everything from this point on will be part of the definition hash
			int hashCodeBlockStart = builder.Length;

			// Collect the properties
			UhtPropertyMemberContextImpl context = new(this, function, "", eventParameters);
			UhtUsedDefineScopes<UhtProperty> properties = new(function.Properties);

			// Statics declaration
			{
				builder.Append("struct UHT_STATICS\r\n");
				builder.Append("{\r\n");

				if (paramsInStatic)
				{
					List<UhtScriptStruct> noExportStructs = FindNoExportStructs(function);
					foreach (UhtScriptStruct noExportStruct in noExportStructs)
					{
						AppendMirrorsForNoexportStruct(builder, noExportStruct, 1);
					}
					AppendEventParameter(builder, function, strippedFunctionName, UhtPropertyTextType.EventParameterFunctionMember, false, 1, "\r\n");
				}

				builder.AppendConditionalMetaData(function, UhtNames.TypeId, context, properties, 1);

				AppendPropertiesDecl(builder, context, properties, 1);

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					if (function.FunctionFlags.HasExactFlags(EFunctionFlags.Native | EFunctionFlags.NetRequest, EFunctionFlags.Native)
						&& function.FunctionExportFlags.HasAllFlags(UhtFunctionExportFlags.CustomThunk))
					{
						// Add an accessor for the probably-private static memeber function ptr in the static struct which can be declared as a friend
						builder.Append($"\tUE_CONSTEVAL static FNativeFuncPtr GetCustomThunk_{function.SourceName}()\r\n")
							.Append("\t{\r\n")
							.Append($"\t\treturn &{(function.Outer as UhtClass)!.NativeFunctionCallName}::exec{function.SourceName};\r\n")
							.Append("\t}\r\n");
					}
				}
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic const UECodeGen_Private::").AppendFunctionParamsType(function).Append(" FuncParams;\r\n");
				}

				builder.Append("};\r\n");
			}

			// Statics definition
			{
				AppendPropertiesDefs(builder, context, properties, 0);

				string functionEngineName = isVerseNativeCallable ? function.GetEncodedVerseScopeAndName(UhtVerseNameMode.Default) : function.EngineName;

				string? sizeOfStatic = null;
				string? dataSizeOfStatic = null;
				string? alignOfStatic = null;
				if (function.Children.Count > 0)
				{
					UhtFunction tempFunction = function;
					while (tempFunction.SuperFunction != null)
					{
						tempFunction = tempFunction.SuperFunction;
					}

					string eventStructName = GetEventStructParametersName(tempFunction.Outer, tempFunction.StrippedFunctionName);
					UhtCppIdentifier identifier = paramsInStatic ? new UhtCppIdentifier(eventStructName).MakeStatics() : new UhtCppIdentifier(eventStructName);
					dataSizeOfStatic = $"DataSizeOf<{identifier}>()";
					sizeOfStatic = $"sizeof({identifier})";
					alignOfStatic = $"alignof({identifier})";
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append($"UE_CONSTINIT_UOBJECT_DECL TNoDestroy<U{function.EngineClassName}> {constinitName}{{NoDestroyConstEval,\r\n")
						.AppendConstInitObjectParams(function, this, 1);
					builder.Append("\tUE::CodeGen::ConstInit::FUFieldParams{ \r\n")
						.AppendScopeLink(function.NextFunction, context, 2, ".Next = ", ",\r\n")
						.Append("\t},\r\n");
					builder.Append("\tUE::CodeGen::ConstInit::FStructParams{\r\n")
						.AppendLine($"\t\t.Super = {ConstInitSingletonRef(this, function.SuperFunction)}, ")
						.Append("\t\t.ChildProperties = UHT_STATICS::GetFirstProperty(),\r\n")
						.Append(dataSizeOfStatic is null ? null : $"\t\t.PropertiesSize = {dataSizeOfStatic},\r\n")
						.Append(alignOfStatic is null ? null : $"\t\t.MinAlignment = {alignOfStatic},\r\n")
						.Append("\t\tIF_WITH_METADATA(.MetaData = ").AppendMetaDataArrayView(function.MetaData, UhtNames.StaticsTypeId, 0, ",)\r\n")
						.Append("\t},\r\n");
					builder.Append("\tUE::CodeGen::ConstInit::FFunctionParams{\r\n")
						.Append((function.FunctionFlags & EFunctionFlags.Native, function.FunctionFlags & EFunctionFlags.NetRequest, function.FunctionExportFlags & UhtFunctionExportFlags.CustomThunk) switch
						{
							// Native non-service with custom thunk
							(EFunctionFlags.Native, EFunctionFlags.None, UhtFunctionExportFlags.CustomThunk)
								=> $"\t\t.NativeFunction = UHT_STATICS::GetCustomThunk_{function.SourceName}(),\r\n",
							// Native without custom thunk
							(EFunctionFlags.Native, EFunctionFlags.None, _)
								=> $"\t\t.NativeFunction = &{fullSourceName}{callName}::exec{function.SourceName},\r\n",
							// Non-native function 
							// Can't use cross-dll assignment in modular builds here 
							(EFunctionFlags.None, _, _) => ".NativeFunction = UE_IF(IS_MONOLITHIC, &UObject::ProcessInternal, nullptr),\r\n",
							// No thunk for whatever reason - e.g. net service function
							_ => $"",
						})
						.Append($"\t\t.FunctionFlags = (EFunctionFlags)0x{(uint)function.FunctionFlags:X8}, \r\n")
						.Append(isNet ? $"\t\t.RPCId = {function.RPCId}, \r\n" : null)
						.Append(isNet ? $"\t\t.RPCResponseId = {function.RPCResponseId}, \r\n" : null)
						.Append("\t},\r\n");
					if (function.FunctionType == UhtFunctionType.SparseDelegate)
					{
						builder.Append("\tUE::CodeGen::ConstInit::FSparseDelegateParams{\r\n")
						.Append("\t\tUTF8TEXT(").AppendUTF8LiteralString(function.SparseOwningClassName).Append("), \r\n")
						.Append("\t\tUTF8TEXT(").AppendUTF8LiteralString(function.SparseDelegateName).Append("), \r\n")
						.Append("\t},\r\n");
					}
					if (function.IsVerseField)
					{
						builder.Append("\tUE::CodeGen::ConstInit::FVerseFunctionParams{\r\n")
							.Append("\t\t.AlternateNameUTF8 = IF_WITH_VERSE_VM(")
							.Append("UTF8TEXT(").AppendUTF8LiteralString(function.GetVerseScopeAndName(UhtVerseNameMode.Default)).Append("), ")
							.Append("UTF8TEXT(").AppendUTF8LiteralString(function.GetEncodedVerseScopeAndName(UhtVerseNameMode.Default)).Append(") ")
							.Append("), \r\n")
							.Append("\t},\r\n");
					}
					builder.Append(" };\r\n");
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("const UECodeGen_Private::").AppendFunctionParamsType(function).Append(' ')
						.Append("UHT_STATICS::FuncParams = { { ")
						.Append("(FTypeConstructFunc*)").Append(GetFunctionOuterFunc(function)).Append(", ")
						.Append(GetSingletonName(function.SuperFunction, UhtSingletonType.Registered)).Append(", ")
						.AppendUTF8LiteralString(functionEngineName).Append(", ")
						.AppendArrayPtrLine(properties, UhtNames.StaticsPropPointersId, 0, ", ")
						.AppendArrayCountLine(properties, UhtNames.StaticsPropPointersId, 0, ", ")
						.Append(dataSizeOfStatic ?? "0").Append(", ")
						.Append("RF_Public|RF_Transient|RF_MarkAsNative, ")
						.Append($"(EFunctionFlags)0x{(uint)function.FunctionFlags:X8}, ")
						.Append(isNet ? function.RPCId : 0).Append(", ")
						.Append(isNet ? function.RPCResponseId : 0).Append(", ")
						.AppendMetaDataParams(function, UhtNames.StaticsTypeId)
						.Append("}, ");

					switch (function.FunctionType)
					{
						case UhtFunctionType.Function:
							if (function.IsVerseField)
							{
								builder.Append("IF_WITH_VERSE_VM(");
								builder.AppendUTF8LiteralString(function.GetVerseScopeAndName(UhtVerseNameMode.Default)).Append(", ");
								builder.AppendUTF8LiteralString(function.GetEncodedVerseScopeAndName(UhtVerseNameMode.Default)).Append("), ");
							}
							break;

						case UhtFunctionType.Delegate:
							break;

						case UhtFunctionType.SparseDelegate:
							builder
								.AppendUTF8LiteralString(function.SparseOwningClassName).Append(", ")
								.AppendUTF8LiteralString(function.SparseDelegateName).Append(", ");
							break;
					}

					builder.Append(" };\r\n");
				}

				if (sizeOfStatic is not null)
				{
					builder.Append("static_assert(").Append(sizeOfStatic).Append(" < MAX_uint16);\r\n");
				}
			}

			// Registration function
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("UFunction* ").Append(singletonName).Append("(ETypeConstructPhase Phase)\r\n");
				builder.Append("{\r\n");
				builder.Append("\tstatic UFunction* ReturnFunction = nullptr;\r\n");
				builder.Append("\tif (!ReturnFunction)\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\t").AppendFunctionConstructorName(function).Append("(&ReturnFunction, UHT_STATICS::FuncParams);\r\n");
				builder.Append("\t}\r\n");
				builder.Append("\treturn ReturnFunction;\r\n");
				builder.Append("}\r\n");
			}

			using (UhtRentedPoolBuffer<char> borrowBuffer = builder.RentPoolBuffer(hashCodeBlockStart, builder.Length - hashCodeBlockStart))
			{
				ObjectInfos[function.ObjectTypeIndex].Hash = IoHash.Compute<char>(borrowBuffer.Buffer.Memory.Span);
			}
			return builder;
		}

		private string GetFunctionOuterFunc(UhtFunction function)
		{
			if (function.Outer == null)
			{
				return "nullptr";
			}
			else if (function.Outer is UhtPackage package)
			{
				return GetSingletonName(package, UhtSingletonType.Registered);
			}
			else if (function.Outer is UhtPartial partial)
			{
				return GetSingletonName(partial.OwnerClass, UhtSingletonType.Registered);
			}
			else
			{
				return GetSingletonName((UhtObject)function.Outer, UhtSingletonType.Registered);
			}
		}

		private static StringBuilder AppendMirrorsForNoexportStruct(StringBuilder builder, UhtScriptStruct noExportStruct, int tabs)
		{
			builder.AppendTabs(tabs).Append("struct ").Append(noExportStruct.SourceName);
			if (noExportStruct.SuperScriptStruct != null)
			{
				builder.Append(" : public ").Append(noExportStruct.SuperScriptStruct.SourceName);
			}
			builder.Append("\r\n");
			builder.AppendTabs(tabs).Append("{\r\n");

			// Export the struct's CPP properties
			AppendExportProperties(builder, noExportStruct, tabs + 1);

			builder.AppendTabs(tabs).Append("};\r\n");
			builder.Append("\r\n");
			return builder;
		}

		private static StringBuilder AppendExportProperties(StringBuilder builder, UhtScriptStruct scriptStruct, int tabs)
		{
			using (UhtMacroBlockEmitter emitter = new(builder, UhtDefineScope.None))
			{
				foreach (UhtProperty property in scriptStruct.Properties)
				{
					emitter.Set(property.DefineScope);
					builder.AppendTabs(tabs).AppendFullDecl(property, UhtPropertyTextType.ExportMember, false).Append(";\r\n");
				}
			}
			return builder;
		}

		private StringBuilder AppendPropertiesDecl(StringBuilder builder, UhtPropertyMemberContextImpl context, UhtUsedDefineScopes<UhtProperty> properties, int tabs)
		{
			using UhtCodeBlockComment block = new(builder, context.OuterStruct, "constinit property declarations");
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock macroBlock = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				if (properties.IsEmpty)
				{
					builder.AppendTabs(tabs).Append("static consteval FProperty* GetFirstProperty() { return nullptr; }\r\n");
				}
				else
				{
					builder.AppendInstances(properties, (builder, property) =>
					{
						UhtCppIdentifier identifier = UhtNames.GetPropertyIdentifier(property);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#if WITH_VERSE_VM");
							context.IsLegacy = false;
							builder.AppendConstInitDecl(property, context, identifier.AppendSuffix("_Legacy"), tabs);
							context.IsLegacy = true;
							UhtProperty.AppendConstInitDecl(builder, property, identifier, tabs, "VRestValueProperty");
							builder.AppendLine("#else");
						}
						builder.AppendConstInitDecl(property, context, identifier, tabs);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#endif");
						}
					});

					// declare consteval function to provide first property for constructor unconditionally - body will be empty if there are no properties
					builder.AppendTabs(tabs).Append("static consteval FProperty* GetFirstProperty();\r\n");

					// for properties which have more than one 'next' property depending on preprocessor definitions, define a function to return the correct property
					builder.AppendInstances(properties, (builder, property) =>
					{
						property.AppendGetNextProperty(builder, tabs);
					});
				}
			}
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock macroBlock = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.AppendInstances(properties,
					builder => { },
					(builder, property) =>
					{
						UhtCppIdentifier identifier = UhtNames.GetPropertyIdentifier(property);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#if WITH_VERSE_VM");
							context.IsLegacy = false;
							builder.AppendParamsDecl(property, context, identifier.AppendSuffix("_Legacy"), tabs);
							context.IsLegacy = true;
							UhtProperty.AppendParamsDecl(builder, identifier, tabs, "FVerseValuePropertyParams");
							builder.AppendLine("#else");
						}
						builder.AppendParamsDecl(property, context, identifier, tabs);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#endif");
						}
					},
					builder => builder.AppendTabs(tabs).Append("static const UECodeGen_Private::FPropertyParamsBase* const PropPointers[];\r\n")
					);
			}

			return builder;
		}

		private StringBuilder AppendPropertiesDefs(StringBuilder builder, UhtPropertyMemberContextImpl context, UhtUsedDefineScopes<UhtProperty> properties, int tabs)
		{
			if (properties.IsEmpty)
			{
				return builder;
			}

			using UhtCodeBlockComment block = new(builder, context.OuterStruct, "Property Definitions");
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock macroBlock = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.AppendInstances(properties, (builder, property) =>
				{
					UhtCppIdentifier identifier = UhtNames.GetPropertyIdentifier(property);
					if (property.CodeGenWrapInRestValue)
					{
						builder.AppendLine("#if WITH_VERSE_VM");
						context.IsLegacy = false;
						builder.AppendConstInitDef(property, context, identifier.AppendSuffix("_Legacy"), (builder) => property.AppendConstInitPtr(builder, identifier, tabs, ""), "0", tabs);
						context.IsLegacy = true;
						UhtProperty.AppendConstInitDefStart(builder, property, context, identifier, null, null, tabs, "VRestValueProperty");
						context.IsLegacy = false;
						property.AppendConstInitPtr(builder, identifier.AppendSuffix("_Legacy"), 0, ", ");
						context.IsLegacy = true;
						UhtProperty.AppendConstInitDefEnd(builder, property, context);
						builder.AppendLine("#else");
					}
					builder.AppendConstInitDef(property, context, identifier, null, null, tabs);
					if (property.CodeGenWrapInRestValue)
					{
						builder.AppendLine("#endif");
					}
				});

				builder
					.AppendTabs(tabs).Append($"consteval FProperty* UHT_STATICS::GetFirstProperty() {{\r\n")
					.AppendScopeLink(context.OuterStruct.FirstProperty, tabs + 1, "return ", ";\r\n")
					.Append("}\r\n");
			}
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock macroBlock = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.AppendInstances(properties, 
					(builder, property) =>
					{
						UhtCppIdentifier identifier = UhtNames.GetPropertyIdentifier(property);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#if WITH_VERSE_VM");
							context.IsLegacy = false;
							builder.AppendParamsDef(property, context, identifier.AppendSuffix("_Legacy"), "0", tabs);
							context.IsLegacy = true;
							UhtProperty.AppendParamsDefStart(builder, property, context, identifier, null, tabs, "FVerseValuePropertyParams", "UECodeGen_Private::EPropertyGenFlags::VerseLegacyValue", true);
							UhtProperty.AppendParamsDefEnd(builder, property, context, identifier);
							builder.AppendLine("#else");
						}
						builder.AppendParamsDef(property, context, identifier, null, tabs);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#endif");
						}
					});

				builder.AppendInstances(properties, 
					builder => builder.AppendTabs(tabs).Append("const UECodeGen_Private::FPropertyParamsBase* const UHT_STATICS::PropPointers[] = {\r\n"),
					(builder, property) =>
					{
						UhtCppIdentifier identifier = UhtNames.GetPropertyIdentifier(property);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#if WITH_VERSE_VM");
							context.IsLegacy = false;
							builder.AppendParamsPtr(property, context, identifier.AppendSuffix("_Legacy"), tabs + 1);
							context.IsLegacy = true;
							builder.AppendParamsPtr(property, context, identifier, tabs + 1);
							builder.AppendLine("#else");
						}
						builder.AppendParamsPtr(property, context, UhtNames.GetPropertyIdentifier(property), tabs + 1);
						if (property.CodeGenWrapInRestValue)
						{
							builder.AppendLine("#endif");
						}
					},
					builder =>
					{
						builder.AppendTabs(tabs).Append("};\r\n");
						builder.AppendTabs(tabs).Append("static_assert(UE_ARRAY_COUNT(UHT_STATICS::PropPointers) < 2048);\r\n");
					});
			}
			return builder;
		}

		private StringBuilder AppendClassFunction(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			bool isNotDelegate = !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate);
			bool isRpc = IsRpcFunction(function) && ShouldExportFunction(function);
			bool isCallback = IsCallbackFunction(function);
			bool isVerseNativeCallable = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable);
			bool isVerseCallable = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable);

			if (isNotDelegate || isRpc || isCallback)
			{
				using UhtCodeBlockComment blockComment = new(builder, classObj, function);
				using UhtMacroBlockEmitter blockEmitter = new(builder, function.DefineScope);
				if (isCallback)
				{
					AppendEventParameter(builder, function, function.StrippedFunctionName, UhtPropertyTextType.EventParameterMember, true, 0, "\r\n");
					AppendClassFunctionCallback(builder, classObj, function);
					if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
					{
						AppendInterfaceCallFunction(builder, classObj, function);
					}
				}
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
				{
					AppendFunction(builder, function, classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport), false);
					if (isVerseNativeCallable)
					{
						AppendVerseNativeCallableFunctionStub(builder, classObj, function);
						AppendVerseNativeCallableFunction(builder, classObj, function, "");
						if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
						{
							AppendVerseNativeCallableFunction(builder, classObj, function, UhtNames.VerseProxySuffix);
						}
					}
					if (isVerseCallable)
					{
						AppendVerseCallableFunction(builder, classObj, function);
					}
				}
				if (isRpc)
				{
					if (function.Outer is UhtPartial partial)
					{
						builder.Append($"DEFINE_FUNCTION({partial.SourceName}::{function.UnMarshalAndCallName})\r\n");
						builder.Append("{\r\n");
						builder.Append($"\ttypedef {classObj.Namespace.FullSourceName}{classObj.SourceName} ThisClass;\r\n");
						AppendFunctionThunk(builder, classObj, function);
						builder.Append("}\r\n");
					}
					else
					{
						builder.Append($"DEFINE_FUNCTION({classObj.Namespace.FullSourceName}{classObj.NativeFunctionCallName}::{function.UnMarshalAndCallName})\r\n");
						builder.Append("{\r\n");
						AppendFunctionThunk(builder, classObj, function);
						builder.Append("}\r\n");
					}
				}
			}
			return builder;
		}

		private static StringBuilder AppendNativeInterfaceVerseProxyFunctions(StringBuilder builder, UhtClass classObj, UhtClass? funcClassObj)
		{
			if (funcClassObj != null)
			{
				UhtUsedDefineScopes<UhtFunction> verseNativeCallableFunctions = new(funcClassObj.Functions.Where(x => x.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable)));
				builder.Append("\r\n");
				builder.Append("// Verse native callable thunks for ").Append(classObj.SourceName).Append("\r\n");
				AppendVerseNativeCallableProxyFunctions(builder, classObj, verseNativeCallableFunctions);
			}
			return builder;
		}

		private static StringBuilder AppendVerseNativeCallableProxyFunctions(StringBuilder builder, UhtClass classObj, UhtUsedDefineScopes<UhtFunction> verseNativeCallableFunctions)
		{
			return builder.AppendInstances(verseNativeCallableFunctions, 
				(builder, function) =>
				{
					AppendVerseNativeCallableFunction(builder, classObj, function, UhtNames.VerseProxySuffix);
				});
		}

		private static StringBuilder AppendVerseNativeCallableFunctionStub(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			UhtProperty? returnProperty = function.ReturnProperty;
			bool isSuspends = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);

			if (returnProperty != null)
			{
				builder.AppendTokens(returnProperty.TypeTokens.AllTokens).Append(' ');
			}
			else
			{
				builder.Append(isSuspends ? "TVerseTask<void> " : "void ");
			}

			builder.Append(classObj.Namespace.FullSourceName).AppendClassSourceNameOrInterfaceName(classObj).Append("::").Append(function.SourceName).Append('(');
			bool needsComma = false;
			foreach (UhtType type in function.ParameterProperties.Span)
			{
				if (type is UhtProperty property)
				{
					if (needsComma)
					{
						builder.Append(", ");
					}
					needsComma = true;
					builder.AppendTokens(property.TypeTokens.AllTokens).Append(' ').Append(property.SourceName);
				}
			}
			builder.Append(")\r\n");
			builder.Append("{\r\n");
			if (returnProperty != null || isSuspends)
			{
				builder.Append("\treturn ");
			}
			else
			{
				builder.Append('\t');
			}
			builder.Append(function.SourceName).Append("(verse::FExecutionContext::GetActiveContext()");
			foreach (UhtType type in function.ParameterProperties.Span)
			{
				if (type is UhtProperty property)
				{
					builder.Append(", std::forward<decltype(").Append(property.SourceName).Append(")>(").Append(property.SourceName).Append(')');
				}
			}
			if (returnProperty != null)
			{
				if (function.Session.IsIncompleteReturn(returnProperty))
				{
					builder.Append(");\r\n");
				}
				else
				{
					builder.Append(").GetValue();\r\n");
				}
			}
			else
			{
				builder.Append(");\r\n");
			}
			builder.Append("}\r\n");
			return builder;
		}

		private static StringBuilder AppendVerseNativeCallableFunction(StringBuilder builder, UhtClass classObj, UhtFunction function, string classNameSuffix)
		{
			UhtProperty? returnProperty = function.ReturnProperty;
			bool isSuspends = returnProperty != null || function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);

			builder.AppendVerseNativeCallableSignature(classObj, function, classNameSuffix).Append("\r\n");
			builder.Append("{\r\n");

			// Result value declaration
			if (isSuspends)
			{
				builder.Append('\t').AppendVerseNativeCallableReturnType(function).Append("Result;\r\n");
			}

			// The bodies must differ based on the VM
			builder.Append("#if WITH_VERSE_VM\r\n");
			AppendVerseNativeCallableFunctionBody(builder, classObj, function, returnProperty, true);
			builder.Append("#else\r\n");
			AppendVerseNativeCallableFunctionBody(builder, classObj, function, returnProperty, false);
			builder.Append("#endif\r\n");

			if (isSuspends)
			{
				builder.Append("\treturn Result;\r\n");
			}
			builder.Append("}\r\n");
			return builder;
		}

		private static StringBuilder AppendVerseNativeCallableFunctionBody(StringBuilder builder, UhtClass classObj, UhtFunction function, UhtProperty? returnProperty, bool isVerseVM)
		{
			builder.Append(isVerseVM ? "\tAutoRTFM::Open([&] AUTORTFM_DISABLE {\r\n" : "\t{\r\n");

			// Callee type declaration
			builder.Append("\t\tusing CalleeType = ").AppendVerseNativeCallableTypeDef(function).Append(";\r\n");

			// Callee paths
			string verseScopeAndName = function.GetVerseScopeAndName(UhtVerseNameMode.Default);
			if (isVerseVM)
			{
				builder.Append("\t\tconst char* CalleePath = \"").Append(verseScopeAndName).Append("\";\r\n");
			}
			else
			{
				builder.Append("\t\tconst TCHAR* CalleePath = TEXT(\"").AppendEncodedVerseName(verseScopeAndName).Append("\");\r\n");
			}

			// Callee declaration
			if (classObj.ClassType == UhtClassType.Interface)
			{
				builder.Append("\t\tPRAGMA_DISABLE_DEPRECATION_WARNINGS\r\n");
				builder.Append("\t\tCalleeType Callee{_V_EXEC_CONTEXT_PARAM_NAME, _getUObject(), CalleePath};\r\n");
				builder.Append("\t\tPRAGMA_ENABLE_DEPRECATION_WARNINGS\r\n");
			}
			else if (classObj.ClassType == UhtClassType.VModule)
			{
				builder.Append("\t\tCalleeType Callee{_V_EXEC_CONTEXT_PARAM_NAME, V_SAFE_STATIC_CONTEXT(), CalleePath};\r\n");
			}
			else
			{
				builder.Append("\t\tCalleeType Callee{_V_EXEC_CONTEXT_PARAM_NAME, this, CalleePath};\r\n");
			}

			// Invocation
			builder.Append("\t\t");
			if (returnProperty != null || function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends))
			{
				builder.Append("Result = ");
			}
			builder.Append("Callee(_V_EXEC_CONTEXT_PARAM_NAME");
			builder.AppendVerseFunctionArgs(function, true, false, property => builder.Append(property.SourceName), () => builder.Append("FDecidesContext(FDecidesContext::EDefaultContruct::UnsafeDoNotUse)"));
			builder.Append(");\r\n");

			builder.Append(isVerseVM ? "\t});\r\n" : "\t}\r\n");
			return builder;
		}

		private static StringBuilder AppendVerseCallableFunction(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			UhtProperty? returnProperty = function.ReturnProperty;

			bool isCoroutine = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);
			bool isNoRollback = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNoRollback);

			// Write _exec_ wrapper that checks for native method implementation
			if (isNoRollback)
			{
				builder.Append($"V_DEFINE_EXEC_NOROLLBACK({classObj.Namespace.FullSourceName}{classObj.SourceName}, {function.VerseName})\r\n");
			}
			else if (isCoroutine)
			{
				builder.Append($"V_DEFINE_EXEC_SUSPENDS({classObj.Namespace.FullSourceName}{classObj.SourceName}, {function.VerseName})\r\n");
			}
			else
			{
				builder.Append($"V_DEFINE_EXEC({classObj.Namespace.FullSourceName}{classObj.SourceName}, {function.VerseName})\r\n");
			}

			builder.Append("{\r\n");
			builder.Append($"\tV_CALL_IMPL_NO_PRED({function.VerseName});\r\n");
			builder.Append("}\r\n");

			builder.Append($"V_DEFINE_IMPL_NO_PRED({classObj.Namespace.FullSourceName}{classObj.SourceName}, {function.VerseName})\r\n");
			builder.Append("{\r\n");
			// The bodies must differ based on the VM
			builder.Append("#if WITH_VERSE_VM\r\n");
			AppendVerseCallableFunctionBody(builder, classObj, function, returnProperty, true);
			builder.Append("#else\r\n");
			AppendVerseCallableFunctionBody(builder, classObj, function, returnProperty, false);
			builder.Append("#endif\r\n");
			builder.Append("}\r\n");
			return builder;
		}

		private static StringBuilder AppendVerseCallableFunctionBody(StringBuilder builder, UhtClass classObj, UhtFunction function, UhtProperty? returnProperty, bool isVerseVM)
		{
			bool isModule = classObj.ClassType == UhtClassType.VModule;
			bool isDecides = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides);
			bool isCoroutine = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends);
			bool isExtensionMethod = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseExtensionMethod);
			bool isIncompleteReturn = function.Session.IsIncompleteReturn(returnProperty);
			bool isAlwaysOpen = function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseRTFMAlwaysOpen);

			// V_MARSHALLING_...
			{
				builder.Append("\tV_MARSHALLING_PARAM_BEGIN\r\n");
				if (isVerseVM || !isCoroutine)
				{
					ReadOnlySpan<UhtType> arguments = function.ParameterProperties.Span;
					if (arguments.Length != 1)
					{
						builder.Append("\tV_MARSHAL_TUPLE_BEGIN\r\n");
					}
					foreach (UhtType argumentType in arguments)
					{
						UhtProperty argument = (UhtProperty)argumentType;
						builder.Append('\t').AppendVerseArgumentUnmarshal(argument, isVerseVM).Append("\r\n");
					}
					if (arguments.Length != 1)
					{
						builder.Append("\tV_MARSHAL_TUPLE_END\r\n");
					}
				}
				builder.Append("\tV_MARSHALLING_END\r\n");
			}

			// V_NATIVE_BEGIN
			{
				builder.Append("\tV_NATIVE_BEGIN(");
				builder.Append(classObj.SourceName);
				builder.Append(", ");
				builder.Append(isAlwaysOpen ? "true" : "false");
				builder.Append(", V_COMMA_SEPARATED(");
				if (!isVerseVM && isCoroutine)
				{
					builder.Append("FVerseResult");
				}
				else if (returnProperty == null)
				{
					builder.Append("EVerseTrue");
				}
				else
				{
					UhtProperty adjustedReturnProperty = returnProperty;
					if (isVerseVM && adjustedReturnProperty is UhtOptionalProperty optionalProperty)
					{
						adjustedReturnProperty = optionalProperty.ValueProperty;
					}
					builder.AppendTokens(adjustedReturnProperty.TypeTokens.FullDeclarationTokens);
				}
				builder.Append("))\r\n");
			}

			// Coroutine FTask
			if (!isVerseVM && isCoroutine)
			{
				ReadOnlySpan<UhtType> arguments = function.ParameterProperties.Span;
				builder.Append("\tstruct FTask : verse::task\r\n");
				builder.Append("\t{\r\n");
				if (!isModule)
				{
					builder.Append("\t\tV_THIS_CLASS* _Self;\r\n");
				}
				if (isModule && isExtensionMethod && arguments.Length > 0)
				{
					UhtProperty thisProperty = (UhtProperty)arguments[0];
					arguments = arguments[1..];
					builder.Append("\t\t").AppendTokens(thisProperty.TypeTokens.FullDeclarationTokens).Append($" {thisProperty.SourceName};\r\n");
				}
				builder.Append("\t\tstruct\r\n");
				builder.Append("\t\t{\r\n");
				foreach (UhtType argument in arguments)
				{
					UhtProperty property = (UhtProperty)argument;
					builder.Append("\t\t\t").AppendTokens(property.TypeTokens.FullDeclarationTokens).Append($" {property.SourceName};\r\n");
				}
				builder.Append("\t\t};\r\n");
				if (returnProperty != null)
				{
					builder.Append("\t\t").AppendTokens(returnProperty.TypeTokens.FullDeclarationTokens).Append(" _RetVal;\r\n");
				}
				builder.Append("\t};\r\n");
			}

			if (isVerseVM && !isCoroutine && !isDecides && (isIncompleteReturn || returnProperty == null))
			{
				builder.Append("\t__NativeReturnValue.Emplace();");
			}

			// Invocation
			{
				builder.Append('\t');
				if (isCoroutine)
				{
					if (!isVerseVM)
					{
						builder.Append("*__NativeReturnValue = ");
					}
					else
					{
						builder.Append("__ControlFlow = ");
					}
				}
				else if (!isIncompleteReturn && (isDecides || returnProperty != null))
				{
					if (!isVerseVM)
					{
						builder.Append('*');
					}
					builder.Append("__NativeReturnValue = ");
				}
				if (isModule)
				{
					builder.Append($"{classObj.Namespace.FullSourceName}");
				}
				else
				{
					if (!isVerseVM && isCoroutine)
					{
						builder.Append("((FTask*)V_THIS)->_Self->");
					}
					else
					{
						builder.Append("V_THIS->");
					}
				}

				builder.Append(function.SourceName).Append('(');

				bool needsComma = false;
				if (isCoroutine)
				{
					if (!isVerseVM)
					{
						builder.Append("(FTask*)V_THIS");
					}
					else
					{
						builder.Append("{__Context}");
					}
					needsComma = true;
				}

				foreach (UhtType argumentType in function.ParameterProperties.Span)
				{
					if (argumentType is UhtProperty argument)
					{
						if (needsComma)
						{
							builder.Append(", ");
						}
						if (!isVerseVM && isCoroutine)
						{
							builder.Append("((FTask*)V_THIS)->");
						}
						builder.Append(argument.SourceName); // InteropBlock << GetRemappedSymbolString(Param->GetName());
						needsComma = true;
					}
				}
				if (!isCoroutine && isIncompleteReturn)
				{
					if (needsComma)
					{
						builder.Append(", ");
					}
					if (!isVerseVM)
					{
						builder.Append("*__NativeReturnValue");
					}
					else
					{
						if (isDecides)
						{
							builder.Append("__NativeReturnValue");
						}
						else
						{
							builder.Append("*__NativeReturnValue");
						}
					}
				}
				builder.Append(");\r\n");
			}

			if (!isVerseVM && isCoroutine)
			{
				builder.Append("\t((FTask*)V_THIS)->_bEverSuspended = true;\r\n");
			}
			builder.Append("\tV_NATIVE_END(").Append(isAlwaysOpen ? "true" : "false").Append(")\r\n");
			builder.Append("\tV_REPORT_FUNCTION_CALL(").AppendUTF8LiteralString(function.GetVerseName(UhtVerseNameMode.Default)).Append(", ").AppendUTF8LiteralString(function.GetVerseScope(UhtVerseNameMode.Default)).Append(");\r\n");
			return builder;
		}

		private StringBuilder AppendClassFunctionCallback(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			// Net response functions don't go into the VM
			if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport) || function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetResponse))
			{
				return builder;

			}

			bool isInterfaceClass = classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface);
			using UhtMacroBlockEmitter blockEmitter = new(builder, function.DefineScope);

			if (!isInterfaceClass)
			{
				// Do not make this a static const.  It causes issues with live coding
				builder.Append("static FName NAME_").Append(classObj.SourceName).Append('_').Append(function.EngineName).Append(" = FName(TEXT(\"").Append(function.EngineName).Append("\"));\r\n");
			}

			AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.EventFunctionArgOrRetVal, false, null, null, UhtFunctionExportFlags.None, 0, "\r\n");

			if (isInterfaceClass)
			{
				builder.Append("{\r\n");

				// assert if this is ever called directly
				builder
					.Append("\tcheck(0 && \"Do not directly call Event functions in Interfaces. Call Execute_")
					.Append(function.EngineName)
					.Append(" instead.\");\r\n");

				// satisfy compiler if it's expecting a return value
				if (function.ReturnProperty != null)
				{
					string eventParmStructName = GetEventStructParametersName(classObj, function.EngineName);
					builder.Append('\t').Append(eventParmStructName).Append(" Parms;\r\n");
					builder.Append("\treturn Parms.ReturnValue;\r\n");
				}
				builder.Append("}\r\n");
			}
			else
			{
				bool isBlueprintEvent = function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent);
				bool isNetEvent = function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net);
				bool isNetRemoteEvent = isNetEvent && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetClient | EFunctionFlags.NetServer | EFunctionFlags.NetMulticast);
				bool isCallInEditor = function.MetaData.ContainsKey(UhtNames.CallInEditor);
				bool isEditorFunction = function.FunctionFlags.HasAnyFlags(EFunctionFlags.EditorOnly);
				bool hasValidCppImpl = (function.StrippedFunctionName != function.CppImplName);

				// This is a small optimization that we can do for BlueprintNativeEvents.
				// If there is a native "_Implementation" of the current function, then we can do a small
				// optimization where we only call "ProcessEvent" if there is actually a BP script override
				// of the native implementation. This saves us the cost of unnecessarily copying the function
				// params to the BPVM if we don't have to. We can only do this optimization
				// if the implementation function name is not the same as the actual C++ function name, as that
				// would just call the function recursively. We cannot do this optimization for networked events
				// because ProcessEvent does some important replication behavior which we do not want to lose.
				bool doNativeImplOptimization =
					isBlueprintEvent &&
					!isNetEvent &&
					!isCallInEditor &&
					!isEditorFunction &&
					hasValidCppImpl;

				if (!doNativeImplOptimization)
				{
					AppendEventFunctionPrologue(builder, function, function.EngineName, 0, "\r\n", false);

					AppendFindUFunction(builder, classObj, function, 1, "\r\n");
				}
				else
				{
					builder.Append("{\r\n");

					AppendFindUFunction(builder, classObj, function, 1, "\r\n");

					builder
						.Append("\tif (!Func->GetOwnerClass()->HasAnyClassFlags(CLASS_Native))\r\n")
						.Append("\t{\r\n");

					AppendEventFunctionPrologue(builder, function, function.EngineName, /*tabs*/ 1, "\r\n", /*addEventParameterStruct=*/false,/*addEventParameterStruct*/ false);
				}

				// For remote net functions add them to our tracking stack
				if (isNetRemoteEvent)
				{
					builder.Append("\tUE::Net::Private::FScopedRemoteRPCMode CallingRemoteRPC(Func, UE::Net::Private::ERemoteFunctionMode::Sending);\r\n");
				}

				// Cast away const just in case, because ProcessEvent isn't const
				builder.Append('\t');
				if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const))
				{
					builder.AppendTabs(1).Append("const_cast<").Append(classObj.SourceName).Append("*>(this)->");
				}

				builder
					.Append("ProcessEvent(Func,")
					.Append(function.Children.Count > 0 ? "&Parms" : "NULL")
					.Append(");\r\n");

				// Call into the native implementation of the function if there is one
				// We don't want to do this all the time, like for BlueprintInternalUseOnly functions
				// which will not have a "_Implementation" appended to their name
				if (doNativeImplOptimization)
				{
					AppendEventFunctionEpilogue(builder, function, /*tabs*/ 1, "\r\n", /*bAddFunctionScopeBracket=*/true);

					builder
						.AppendTabs(1)
						.Append("else\r\n")
						.AppendTabs(1)
						.Append("{\r\n")
						.AppendTabs(2)
						.Append(function.HasReturnProperty ? "return " : "");

					// Cast away const just in case, because ProcessEvent isn't const
					if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const))
					{
						builder.Append("const_cast<").Append(classObj.SourceName).Append("*>(this)->");
					}

					// Begin the native function call...
					builder
						.Append(function.CppImplName)
						.Append('(');

					// For every param in this function call, pass it to our native C++ function
					int numParams = function.ParameterProperties.Length;
					ReadOnlySpan<UhtType> paramSpan = function.ParameterProperties.Span;
					for (int i = 0; i < numParams; i++)
					{
						UhtType parameter = paramSpan[i];

						if (parameter is UhtProperty property)
						{
							builder.Append(property.SourceName);
						}

						// Add a "," between function params as long as it isn't the last one
						if ((i + 1) != numParams)
						{
							builder.Append(", ");
						}
					}

					// ...close the function call
					builder
						.Append(");\r\n")
						.AppendTabs(1)
						.Append("}\r\n}\r\n");
				}
				else
				{
					AppendEventFunctionEpilogue(builder, function, /*tabs*/ 0, "\r\n", /*bAddFunctionScopeBracket=*/true);
				}
			}

			return builder;
		}

		private StringBuilder AppendInterfaceCallFunction(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			builder.Append("static FName NAME_").Append(function.Outer?.SourceName).Append('_').Append(function.SourceName).Append(" = FName(TEXT(\"").Append(function.EngineName).Append("\"));\r\n");
			string extraParameter = function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const) ? "const UObject* O" : "UObject* O";
			AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.InterfaceFunctionArgOrRetVal, false, null, extraParameter, UhtFunctionExportFlags.None, 0, "\r\n");
			builder.Append("{\r\n");
			builder.Append("\tcheck(O != NULL);\r\n");
			builder.Append("\tcheck(O->GetClass()->ImplementsInterface(").Append(classObj.SourceName).Append("::StaticClass()));\r\n");
			if (function.Children.Count > 0)
			{
				builder.Append('\t').Append(GetEventStructParametersName(classObj, function.StrippedFunctionName)).Append(" Parms;\r\n");
			}
			builder.Append("\tUFunction* const Func = O->FindFunction(NAME_").Append(function.Outer?.SourceName).Append('_').Append(function.SourceName).Append(");\r\n");
			builder.Append("\tif (Func)\r\n");
			builder.Append("\t{\r\n");
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					builder.Append("\t\tParms.").Append(property.SourceName).Append("=std::move(").Append(property.SourceName).Append(");\r\n");
				}
			}
			builder
				.Append("\t\t")
				.Append(function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const) ? "const_cast<UObject*>(O)" : "O")
				.Append("->ProcessEvent(Func, ")
				.Append(function.Children.Count > 0 ? "&Parms" : "NULL")
				.Append(");\r\n");
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					if (property.PropertyFlags.HasExactFlags(EPropertyFlags.OutParm | EPropertyFlags.ConstParm, EPropertyFlags.OutParm))
					{
						builder.Append("\t\t").Append(property.SourceName).Append("=std::move(Parms.").Append(property.SourceName).Append(");\r\n");
					}
				}
			}
			builder.Append("\t}\r\n");

			// else clause to call back into native if it's a BlueprintNativeEvent
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
			{
				builder
					.Append("\telse if (auto I = (")
					.Append(function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const) ? "const I" : "I")
					.Append(classObj.SourceName[1..])
					.Append("*)(O->GetNativeInterfaceAddress(")
					.Append(classObj.SourceName)
					.Append("::StaticClass())))\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\t");
				if (function.HasReturnProperty)
				{
					builder.Append("Parms.ReturnValue = ");
				}
				builder.Append("I->").Append(function.SourceName).Append("_Implementation(");

				bool first = true;
				foreach (UhtType parameter in function.ParameterProperties.Span)
				{
					if (parameter is UhtProperty property)
					{
						if (!first)
						{
							builder.Append(',');
						}
						first = false;
						builder.Append(property.SourceName);
					}
				}
				builder.Append(");\r\n");
				builder.Append("\t}\r\n");
			}

			if (function.HasReturnProperty)
			{
				builder.Append("\treturn Parms.ReturnValue;\r\n");
			}
			builder.Append("}\r\n");
			return builder;
		}

		private StringBuilder AppendClass(StringBuilder builder, UhtClass classObj, UhtUsedDefineScopes<UhtFunction> functions)
		{
			// Add the auto getters/setters
			AppendAutoGettersSetters(builder, classObj);

			// Add the accessors
			AppendPropertyAccessors(builder, classObj);

			// Add sparse accessors
			AppendSparseAccessors(builder, classObj);

			AppendNativeGeneratedInitCode(builder, classObj, functions);

			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasConstructor))
			{
				if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.UsesGeneratedBodyLegacy))
				{
					switch (GetConstructorType(classObj))
					{
						case ConstructorType.ObjectInitializer:
							builder.Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").Append(classObj.SourceName).Append("(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}\r\n");
							break;

						case ConstructorType.Default:
							builder.Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").Append(classObj.SourceName).Append("() {}\r\n");
							break;
					}
				}
			}

			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasCustomVTableHelperConstructor))
			{
				builder.Append($"DEFINE_VTABLE_PTR_HELPER_CTOR_NS({classObj.Namespace.FullSourceName}, {classObj.SourceName});\r\n");
			}

			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDestructor) && !classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
			{
				builder.Append($"{classObj.Namespace.FullSourceName}{classObj.SourceName}::~{classObj.SourceName}() {{}}\r\n");
			}

			AppendFieldNotify(builder, classObj);

			// Only write out adapters if the user has provided one or the other of the Serialize overloads
			if (classObj.SerializerArchiveType != UhtSerializerArchiveType.None && classObj.SerializerArchiveType != UhtSerializerArchiveType.All)
			{
				AppendSerializer(builder, classObj, UhtSerializerArchiveType.Archive, "IMPLEMENT_FARCHIVE_SERIALIZER");
				AppendSerializer(builder, classObj, UhtSerializerArchiveType.StructuredArchiveRecord, "IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER");
			}
			return builder;
		}

		private static StringBuilder AppendFieldNotify(StringBuilder builder, UhtClass classObj)
		{
			if (!NeedFieldNotifyCodeGen(classObj))
			{
				return builder;
			}

			UhtUsedDefineScopes<UhtType> notifyTypes = GetFieldNotifyTypes(classObj);

			//UE_FIELD_NOTIFICATION_DECLARE_FIELD
			builder.AppendInstances(notifyTypes, 
				(builder, notifyType) =>
				{
					builder.Append($"\tUE_FIELD_NOTIFICATION_IMPLEMENT_FIELD({classObj.SourceName}, {GetNotifyTypeName(notifyType)})\r\n");
				});

			//UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD
			builder.AppendInstances(notifyTypes,
				builder => builder.Append($"\tUE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN({classObj.SourceName})\r\n"),
				(builder, notifyType) =>
				{
					builder.Append($"\tUE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD({classObj.SourceName}, {GetNotifyTypeName(notifyType)})\r\n");
				},
				builder => builder.Append($"\tUE_FIELD_NOTIFICATION_IMPLEMENTATION_END({classObj.SourceName});\r\n"));

			return builder;
		}

		private static StringBuilder AppendAutoGettersSetters(StringBuilder builder, UhtClass classObj)
		{
			UhtUsedDefineScopes<UhtProperty> autoGetterSetterProperties = GetAutoGetterSetterProperties(classObj);
			if (autoGetterSetterProperties.IsEmpty)
			{
				return builder;
			}

			return builder.AppendInstances(autoGetterSetterProperties, 
				(builder, property) =>
				{
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecifiedAuto))
					{
						string getterCallText = property.Getter ?? "Get" + property.SourceName;
						builder.AppendPropertyText(property, UhtPropertyTextType.GetterRetVal).Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").Append(getterCallText).Append("() const\r\n");
						builder.Append("{\r\n");
						builder.Append("\treturn ").Append(property.SourceName).Append(";\r\n");
						builder.Append("}\r\n");
					}
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterSpecifiedAuto))
					{
						string setterCallText = property.Setter ?? "Set" + property.SourceName;
						builder.Append("void ").Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").Append(setterCallText).Append('(').AppendPropertyText(property, UhtPropertyTextType.SetterParameterArgType).Append("InValue").Append(")\r\n");
						builder.Append("{\r\n");
						// @todo: setter defn
						builder.Append("}\r\n");
					}
				});
		}

		private static StringBuilder AppendPropertyAccessors(StringBuilder builder, UhtClass classObj)
		{
			foreach (UhtType type in classObj.Children)
			{
				if (type is UhtProperty property)
				{
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterFound))
					{
						builder.Append("void ").Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").AppendPropertyGetterWrapperName(property).Append("(const void* Object, void* OutValue)\r\n");
						builder.Append("{\r\n");
						using (UhtMacroBlockEmitter blockEmitter = new(builder, property.DefineScope))
						{
							builder.Append("\tconst ").Append(classObj.SourceName).Append("* Obj = (const ").Append(classObj.SourceName).Append("*)Object;\r\n");
							if (property.IsStaticArray)
							{
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("* Source = (")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("*)Obj->")
									.Append(property.Getter!)
									.Append("();\r\n");
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("* Result = (")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("*)OutValue;\r\n");
								builder
									.Append("\tCopyAssignItems(Result, Source, ")
									.Append(property.ArrayDimensions)
									.Append(");\r\n");
							}
							else if (property is UhtByteProperty byteProperty && byteProperty.Enum != null)
							{
								// If someone passed in a TEnumAsByte instead of the actual enum value, the cast in the else clause would cause an issue.
								// Since this is known to be a TEnumAsByte, we just fetch the first byte.  *HOWEVER* on MSB machines where 
								// the actual enum value is passed in this will fail and return zero if the native size of the enum > 1 byte.
								builder
									.Append('\t')
									.Append("uint8")
									.Append("& Result = *(")
									.Append("uint8")
									.Append("*)OutValue;\r\n");
								builder
									.Append("\tResult = (")
									.Append("uint8")
									.Append(")Obj->")
									.Append(property.Getter!)
									.Append("();\r\n");
							}
							else
							{
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("& Result = *(")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("*)OutValue;\r\n");
								builder
									.Append("\tResult = (")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append(")Obj->")
									.Append(property.Getter!)
									.Append("();\r\n");
							}
						}
						builder.Append("}\r\n");
					}
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterFound))
					{
						builder.Append("void ").Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append("::").AppendPropertySetterWrapperName(property).Append("(void* Object, const void* InValue)\r\n");
						builder.Append("{\r\n");
						using (UhtMacroBlockEmitter blockEmitter = new(builder, property.DefineScope))
						{
							builder.Append('\t').Append(classObj.SourceName).Append("* Obj = (").Append(classObj.SourceName).Append("*)Object;\r\n");
							if (property.IsStaticArray)
							{
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("* Value = (")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("*)InValue;\r\n");
							}
							else if (property is UhtByteProperty byteProperty && byteProperty.Enum != null)
							{
								// If someone passed in a TEnumAsByte instead of the actual enum value, the cast in the else clause would cause an issue.
								// Since this is known to be a TEnumAsByte, we just fetch the first byte.  *HOWEVER* on MSB machines where 
								// the actual enum value is passed in this will fail and return zero if the native size of the enum > 1 byte.
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append(" Value = (")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append(")*(uint8*)InValue;\r\n");
							}
							else
							{
								builder
									.Append('\t')
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("& Value = *(")
									.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg)
									.Append("*)InValue;\r\n");
							}
							builder
								.Append("\tObj->")
								.Append(property.Setter!)
								.Append("(Value);\r\n");
						}
						builder.Append("}\r\n");
					}
				}
			}
			return builder;
		}

		private static StringBuilder AppendSparseAccessors(StringBuilder builder, UhtClass classObj)
		{
			foreach (UhtScriptStruct sparseScriptStruct in GetSparseDataStructsToExport(classObj))
			{
				string sparseDataType = sparseScriptStruct.EngineName;

				builder.Append('F').Append(sparseDataType).Append("* ").Append(classObj.SourceName).Append("::Get").Append(sparseDataType).Append("() const \r\n");
				builder.Append("{\r\n");
				builder.Append("\treturn static_cast<F").Append(sparseDataType).Append("*>(GetClass()->GetOrCreateSparseClassData());\r\n");
				builder.Append("}\r\n");

				builder.Append('F').Append(sparseDataType).Append("* ").Append(classObj.SourceName).Append("::GetMutable").Append(sparseDataType).Append("() const \r\n");
				builder.Append("{\r\n");
				builder.Append("\treturn static_cast<F").Append(sparseDataType).Append("*>(GetClass()->GetOrCreateSparseClassData());\r\n");
				builder.Append("}\r\n");

				builder.Append("const F").Append(sparseDataType).Append("* ").Append(classObj.SourceName).Append("::Get").Append(sparseDataType).Append("(EGetSparseClassDataMethod GetMethod) const\r\n");
				builder.Append("{\r\n");
				builder.Append("\treturn static_cast<const F").Append(sparseDataType).Append("*>(GetClass()->GetSparseClassData(GetMethod));\r\n");
				builder.Append("}\r\n");

				builder.Append("UScriptStruct* ").Append(classObj.SourceName).Append("::StaticGet").Append(sparseDataType).Append("ScriptStruct()\r\n");
				builder.Append("{\r\n");
				builder.Append("\treturn F").Append(sparseDataType).Append("::StaticStruct();\r\n");
				builder.Append("}\r\n");
			}
			return builder;
		}

		private static StringBuilder AppendSerializer(StringBuilder builder, UhtClass classObj, UhtSerializerArchiveType serializerType, string macroText)
		{
			if (!classObj.SerializerArchiveType.HasAnyFlags(serializerType))
			{
				builder.AppendScoped(classObj.SerializerDefineScope,
					builder => builder.Append(macroText).Append('(').Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append(")\r\n"));
			}
			return builder;
		}

#pragma warning disable CA1505 //  'AppendNativeGeneratedInitCode' has a maintainability index of '5'. Rewrite or refactor the code to increase its maintainability index (MI) above '9'. 

		private StringBuilder AppendNativeGeneratedInitCode(StringBuilder builder, UhtClass classObj, UhtUsedDefineScopes<UhtFunction> functions)
		{
			string singletonName = GetSingletonName(classObj, UhtSingletonType.Registered);
			string constinitName = GetSingletonName(classObj, UhtSingletonType.ConstInit);
			string registrationName = $"Z_Registration_Info_UClass_{classObj.SourceName}";
			string[]? sparseDataTypes = classObj.MetaData.GetStringArray(UhtNames.SparseClassDataTypes);

			using UhtStaticsEmitter statics = new(builder, this, classObj);

			UhtPropertyMemberContextImpl context = new(this, classObj);

			bool hasInterfaces = classObj.Bases.Any(x => x is UhtClass baseClass && baseClass.ClassFlags.HasAnyFlags(EClassFlags.Interface)) || classObj.FlattenedVerseInterfaces.Count > 0;


			// Everything from this point on will be part of the definition hash
			int hashCodeBlockStart = builder.Length;

			// Collect the properties
			UhtUsedDefineScopes<UhtProperty> properties = new(classObj.Properties);
			UhtUsedDefineScopes<UhtFunction> nativeFunctions = new();
			bool hasNativeFunctions = false;
			bool hasVerseCallableFunctions = false;

			// Declare the statics object
			{
				builder.Append("struct UHT_STATICS\r\n");
				builder.Append("{\r\n");

				builder.AppendConditionalMetaData(classObj, UhtNames.TypeId, context, properties, 1);

				AppendPropertiesDecl(builder, context, properties, 1);

				if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
				{
					// Filter to only native non-net functions for exporting
					nativeFunctions.AddRange(functions.Instances.Where(x => x.FunctionFlags.HasExactFlags(EFunctionFlags.Native | EFunctionFlags.NetRequest, EFunctionFlags.Native)));
					hasNativeFunctions = !nativeFunctions.IsEmpty;
					hasVerseCallableFunctions = nativeFunctions.Instances.Any(x => x.IsVerseField && x.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable));
					if (hasNativeFunctions)
					{
						using UhtMacroBlockEmitter blockEmitter = new(builder, nativeFunctions.SoleScope);
						builder.Append("\tstatic constexpr UE::CodeGen::FClassNativeFunction Funcs[] = {\r\n");

						foreach (UhtFunction function in nativeFunctions.Instances)
						{
							blockEmitter.Set(function.DefineScope);
							builder
								.Append("\t\t{ .NameUTF8 = UTF8TEXT(")
								.AppendUTF8LiteralString(function.EngineName)
								.Append("), .Pointer = &")
								.AppendClassSourceNameOrInterfaceName(classObj)
								.Append($"::exec{function.EngineName} }},\r\n");
						}

						if (hasVerseCallableFunctions)
						{
							builder.Append("#if WITH_VERSE_BPVM\r\n");
							foreach (UhtFunction function in nativeFunctions.Instances)
							{
								if (function.IsVerseField && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable))
								{
									blockEmitter.Set(function.DefineScope);
									builder
										.Append("\t\t{ UTF8TEXT(")
										.AppendUTF8LiteralString(function.GetEncodedVerseScopeAndName(UhtVerseNameMode.Default))
										.Append("), &")
										.AppendClassSourceNameOrInterfaceName(classObj)
										.Append($"::_exec_{function.SourceName}___ }},\r\n");
								}
							}
							builder.Append("#endif\r\n");
						}

						// This will close the block if we have one that isn't editor only
						blockEmitter.Set(nativeFunctions.SoleScope);

						builder.Append("\t};\r\n");
					}

					if (hasVerseCallableFunctions)
					{
						builder.Append("#if WITH_VERSE_VM\r\n");
						using UhtMacroBlockEmitter blockEmitter = new(builder, nativeFunctions.SoleScope);
						builder.Append("\tstatic constexpr FVerseCallableThunk VerseFuncs[] = {\r\n");

						foreach (UhtFunction function in nativeFunctions.Instances)
						{
							if (function.IsVerseField && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable))
							{
								blockEmitter.Set(function.DefineScope);
								builder
									.Append("\t\t{ ")
									.AppendUTF8LiteralString(function.GetVerseScopeAndName(UhtVerseNameMode.Default))
									.Append(", &")
									.AppendClassSourceNameOrInterfaceName(classObj)
									.Append($"::_exec_{function.SourceName}___ }},\r\n");
							}
						}

						// This will close the block if we have one that isn't editor only
						blockEmitter.Set(nativeFunctions.SoleScope);

						builder.Append("\t};\r\n");
						builder.Append("#endif\r\n");
					}
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic FTypeConstructFunc* DependentSingletons[];\r\n");
				}

				// Functions
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic consteval UFunction* GetFirstFunction()\r\n");
					builder.Append("\t{\r\n");
					builder.AppendScopeLink(classObj.FirstFunction, context, 2, "return ", ";\r\n");
					builder.Append("\t}\r\n");
				}
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.AppendInstances(functions, 
						builder =>
						{
							builder.Append("\tstatic constexpr FClassFunctionLinkInfo ").Append("FuncInfo[] = {\r\n");
						},
						(builder, function) =>
						{
							builder
								.Append("\t\t{ &")
								.Append(GetSingletonName(function, UhtSingletonType.Registered))
								.Append(", ")
								.AppendUTF8LiteralString(function.EngineName)
								.Append(" },")
								.AppendObjectHash(classObj, context, function)
								.Append("\r\n");
						},
						builder =>
						{
							builder.Append("\t};\r\n");
							builder.Append("\tstatic_assert(UE_ARRAY_COUNT(").Append("FuncInfo) < 2048);\r\n");
						});
				}

				if (hasInterfaces)
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic UE::CodeGen::ConstInit::FClassImplementedInterface Interfaces[];\r\n");
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic const UECodeGen_Private::FImplementedInterfaceParams InterfaceParams[];\r\n");
					}
				}

				builder.Append("\tstatic constexpr FCppClassTypeInfoStatic StaticCppClassTypeInfo = {\r\n");
				builder.Append($"\t\tTCppClassTypeTraits<{classObj.Namespace.FullSourceName}{classObj.NativeFunctionCallName}>::IsAbstract,\r\n");
				builder.Append("\t};\r\n");

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic UE_CONSTINIT_UOBJECT_DECL inline UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain> StructBases[] = {\r\n");
					if (classObj.SuperClass is not null)
					{
						List<UhtClass> supers = [];
						for (UhtClass? super = classObj.SuperClass; super is not null; super = super.SuperClass)
						{
							supers.Add(super);
						}
						foreach (UhtClass super in supers.AsEnumerable().Reverse())
						{
							builder.AppendLine($"\t\tUHT_STRUCT_BASE({ConstInitSingletonRef(this, super)}), ");
						}
					}
					builder.AppendLine($"\t\tUHT_STRUCT_BASE({ConstInitSingletonRef(this, classObj)}),");
					builder.Append("\t};\r\n");
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("\tstatic const UECodeGen_Private::").AppendClassParamsType(classObj).Append(" ClassParams;\r\n");
				}

				if (classObj.IsVerseField)
				{
					AppendVniTypeDesc(builder, classObj);
				}

				builder.Append($"}}; // struct UHT_STATICS\r\n");
			}

			// Define the statics object
			{
				AppendPropertiesDefs(builder, context, properties, 0);

				// Dependent singletons
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("FTypeConstructFunc* UHT_STATICS::DependentSingletons[] = {\r\n");
					if (classObj.SuperClass != null && classObj.SuperClass != classObj)
					{
						builder.Append("\t(FTypeConstructFunc*)").Append(GetSingletonName(classObj.SuperClass, UhtSingletonType.Registered)).Append(",\r\n");
					}
					builder.Append("\t(FTypeConstructFunc*)").Append(GetSingletonName(classObj.Package, UhtSingletonType.Registered)).Append(",\r\n");
					builder.Append("};\r\n");
					builder.Append("static_assert(UE_ARRAY_COUNT(UHT_STATICS::DependentSingletons) < 16);\r\n");
				}

				// Implemented interfaces
				if (hasInterfaces)
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("UE::CodeGen::ConstInit::FClassImplementedInterface UHT_STATICS::Interfaces[] = {\r\n");
						void AppendConstInitInterfaceParam(UhtClass interfaceObj)
						{
							if (interfaceObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
							{
								string isVerseDirectInterface = interfaceObj.IsVerseField && classObj.VerseInterfaces.Contains(interfaceObj) ? "true" : "false";
								builder.Append("\t{ ")
									.Append($".Class = {ConstInitSingletonRef(this, interfaceObj.AlternateObject)}, ")
									.Append(interfaceObj.IsVerseField
										? ".PointerOffset= 0, .bImplementedByK2 = true, "
										: $".PointerOffset = (int32)VTABLE_OFFSET({classObj.FullyQualifiedSourceName}, {interfaceObj.FullyQualifiedSourceName}), .bImplementedByK2 = false, ")
									.Append($".bVerseDirectInterface = {isVerseDirectInterface}, ")
									.Append("},")
									.AppendObjectHash(classObj, context, interfaceObj.AlternateObject)
									.Append("\r\n");
							}
						}
						foreach (UhtStruct structObj in classObj.Bases)
						{
							if (structObj is UhtClass interfaceObj)
							{
								AppendConstInitInterfaceParam(interfaceObj);
							}
						}
						foreach (UhtClass interfaceObj in classObj.FlattenedVerseInterfaces)
						{
							AppendConstInitInterfaceParam(interfaceObj);
						}
						builder.Append("};\r\n");
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("const UECodeGen_Private::FImplementedInterfaceParams UHT_STATICS::InterfaceParams[] = {\r\n");
						foreach (UhtStruct structObj in classObj.Bases)
						{
							if (structObj is UhtClass interfaceObj)
							{
								AppendImplementedInterfaceParam(builder, context, classObj, interfaceObj, UhtSingletonType.Unregistered);
							}
						}
						foreach (UhtClass interfaceObj in classObj.FlattenedVerseInterfaces)
						{
							AppendImplementedInterfaceParam(builder, context, classObj, interfaceObj, UhtSingletonType.Unregistered);
						}
						builder.Append("};\r\n");
					}
				}

				EClassFlags classFlags = classObj.ClassFlags & EClassFlags.SaveInCompiledInClasses;
				// Propagate class flags without allowing removal for UClass::ClassFlags
				for (UhtClass? superClass = classObj.SuperClass; superClass is not null; superClass = superClass.SuperClass)
				{
					classFlags |= superClass.ClassFlags & EClassFlags.SaveInCompiledInClasses & EClassFlags.Inherit;
				}
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					// constinit class definition
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append($"UE_CONSTINIT_UOBJECT_DECL TNoDestroy<U{classObj.EngineClassName}> {constinitName}{{NoDestroyConstEval,\r\n")
						.AppendConstInitObjectParams(classObj, this, 1, "|RF_Standalone")
						.Append("\tUE::CodeGen::ConstInit::FUFieldParams{},\r\n")
						.Append("\tUE::CodeGen::ConstInit::FStructParams{\r\n")
						.AppendLine($"\t\t.Super = {ConstInitSingletonRef(this, classObj.Super)}, ")
						.Append("\t\t.FirstChild = UHT_STATICS::GetFirstFunction(),\r\n")
						.Append("\t\t.BaseChain = MakeArrayView(UHT_STATICS::StructBases),\r\n")
						.Append("\t\t.ChildProperties = UHT_STATICS::GetFirstProperty(),\r\n")
						.Append($"\t\t.PropertiesSize = DataSizeOf<{classObj.Namespace.FullSourceName}{classObj.SourceName}>(),\r\n")
						.Append($"\t\t.MinAlignment = alignof({classObj.Namespace.FullSourceName}{classObj.SourceName}),\r\n")
						.Append("\t\tIF_WITH_METADATA(.MetaData = ").AppendMetaDataArrayView(classObj.MetaData, UhtNames.StaticsTypeId, 0, ",)\r\n")
						.Append("\t},\r\n")
						.Append("\tUE::CodeGen::ConstInit::FClassParams{\r\n")
						.Append($"\t\t.ConfigName = {classObj.Namespace.FullSourceName}{classObj.SourceName}::StaticConfigName(),\r\n")
						.Append($"\t\t.ClassConstructor = InternalConstructor<{classObj.Namespace.FullSourceName}{classObj.SourceName}>,\r\n")
						.Append($"\t\t.ClassVTableHelperCtorCaller = InternalVTableHelperCtorCaller<{classObj.Namespace.FullSourceName}{classObj.SourceName}>,\r\n")
						.Append($"\t\t.CppClassStaticFunctions = FUObjectCppClassStaticFunctions::ForClass<{classObj.Namespace.FullSourceName}{classObj.SourceName}>(),\r\n")
						.Append($"\t\t.ClassFlags = (EClassFlags)0x{(uint)classFlags:X8}u,\r\n")
						.Append($"\t\t.ClassCastFlags = {classObj.Namespace.FullSourceName}{classObj.SourceName}::StaticAllClassCastFlags(),\r\n") // Propagate cast flags at compile time
						.Append("\t\t.CppClassTypeInfo = &UHT_STATICS::StaticCppClassTypeInfo,\r\n");
					if (hasNativeFunctions)
					{
						builder.Append("\t\t.NativeFunctions = ").AppendArrayView(nativeFunctions, UhtNames.StaticsFuncsId, 0, ",\r\n");
					}
					if (classObj.ClassWithin is not null && classObj != Session.UObject)
					{
						builder.AppendLine($"\t\t.ClassWithin = {ConstInitSingletonRef(this, classObj.ClassWithin)}, ");
					}
					if (hasInterfaces)
					{
						builder.Append("\t\t.Interfaces = MakeArrayView(UHT_STATICS::Interfaces),\r\n");
					}
					if (classObj.SparseClassDataStruct is not null)
					{
						builder.Append($"\t\t.SparseClassDataStruct = {ConstInitSingletonRef(this, classObj.SparseClassDataStruct)}\r\n");
					}
					builder.Append("\t},\r\n");
					if (classObj.IsVerseField)
					{
						EVerseClassFlags verseClassFlags = classObj.ClassType == UhtClassType.VModule ? EVerseClassFlags.Module : EVerseClassFlags.None;
						builder.Append("\tUE::CodeGen::ConstInit::FVerseClassParams{\r\n")
							.Append("\t\t.PackageRelativeVersePath = UTF8TEXT(\"").AppendVerseScopeAndName(classObj, UhtVerseNameMode.PackageRelative).Append("\"),\r\n")
							.Append("\t\t.MangledPackageVersePath = UTF8TEXT(\"").Append(VerseNameMangling.MangleCasedName(classObj.Module.Module.VersePath).Result).Append("\"),\r\n")
							.Append("\t\t.NativeTypeDesc = &UHT_STATICS::NativeTypeDesc,\r\n");
						if (hasVerseCallableFunctions)
						{
							builder.Append("\t\tIF_WITH_VERSE_VM((.VerseCallableThunks = ").AppendArrayView(UhtNames.StaticsVerseFuncsId, 0, ",)").Append(",)\r\n");
						}
						builder
							.Append($"\t\t.VerseClassFlags = 0x{(uint)verseClassFlags:X8}u,\r\n")
							.Append("\t},\r\n");
					}
					builder.Append("};\r\n");
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					// Class parameters
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append("const UECodeGen_Private::").AppendClassParamsType(classObj).Append($" UHT_STATICS::ClassParams = {{\r\n");
					if (classObj.IsVerseField)
					{
						builder.Append("\t{\r\n");
					}
					builder.Append("\t&").Append(GetSingletonName(classObj, UhtSingletonType.Registered)).Append(",\r\n");
					if (classObj.Config.Length > 0)
					{
						builder.Append('\t').AppendUTF8LiteralString(classObj.Config).Append(",\r\n");
					}
					else
					{
						builder.Append("\tnullptr,\r\n");
					}
					builder.Append("\t&StaticCppClassTypeInfo,\r\n");
					builder.Append("\tDependentSingletons,\r\n");
					builder.AppendArrayPtrLine(functions, UhtNames.FuncInfoId, 1, ",\r\n");
					builder.AppendArrayPtrLine(properties, UhtNames.StaticsPropPointersId, 1, ",\r\n");
					builder.Append('\t').Append(hasInterfaces ? "InterfaceParams" : "nullptr").Append(",\r\n");
					builder.Append("\tUE_ARRAY_COUNT(DependentSingletons),\r\n");
					builder.AppendArrayCountLine(functions, UhtNames.FuncInfoId, 1, ",\r\n");
					builder.AppendArrayCountLine(properties, UhtNames.StaticsPropPointersId, 1, ",\r\n");
					builder.Append('\t').Append(hasInterfaces ? "UE_ARRAY_COUNT(InterfaceParams)" : "0").Append(",\r\n");
					builder.Append($"\t0x{(uint)classFlags:X8}u,\r\n");
					builder.Append('\t').AppendMetaDataParams(classObj, UhtNames.StaticsTypeId).Append("\r\n");
					if (classObj.IsVerseField)
					{
						EVerseClassFlags verseClassFlags = classObj.ClassType == UhtClassType.VModule ? EVerseClassFlags.Module : EVerseClassFlags.None;
						builder.Append("\t},\r\n");
						if (classObj.ClassType == UhtClassType.Class || classObj.ClassType == UhtClassType.VModule || classObj.ClassType == UhtClassType.Interface)
						{
							builder.Append("\t\"").AppendVerseScopeAndName(classObj, UhtVerseNameMode.PackageRelative).Append("\",\r\n");
						}
						else
						{
							builder.Append("\t\"\",\r\n");
						}
						builder.Append("\t\"").Append(VerseNameMangling.MangleCasedName(classObj.Module.Module.VersePath).Result).Append("\",\r\n");
						builder.Append("\t&UHT_STATICS::NativeTypeDesc,\r\n");
						builder.Append($"\t0x{(uint)verseClassFlags:X8}u,\r\n");
					}
					builder.Append("};\r\n");
				}
			}

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
			{
				using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

				string registerNativesFuncName = "nullptr";
				if (hasNativeFunctions)
				{
					registerNativesFuncName = $"{classObj.SourceName}_StaticRegisterNatives{classObj.SourceName}";
					builder.Append($"static void {registerNativesFuncName}()\r\n");
					builder.Append("{\r\n");
					builder.Append($"\tUClass* Class = {classObj.Namespace.FullSourceName}{classObj.SourceName}::StaticClass();\r\n");
					builder.Append($"\tFNativeFunctionRegistrar::RegisterFunctions(Class, ").AppendArrayView(nativeFunctions, UhtNames.StaticsFuncsId, 2, ");\r\n");
					if (hasVerseCallableFunctions)
					{
						builder.Append("#if WITH_VERSE_VM\r\n");
						builder.Append("\tVerse::CodeGen::Private::RegisterVerseCallableThunks(Class, UHT_STATICS::VerseFuncs, UE_ARRAY_COUNT(UHT_STATICS::VerseFuncs));\r\n");
						builder.Append("#endif\r\n");
					}
					builder.Append("}\r\n"); // Close StaticRegisterNatives function
				}


				builder.Append("FClassRegistrationInfo ").Append(registrationName).Append(";\r\n");
				builder.Append("UClass* ").Append(singletonName).Append("(ETypeConstructPhase Phase)\r\n");
				builder.Append("{\r\n");
				builder.Append("\tif (Phase == ETypeConstructPhase::Inner)\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\tusing TClass = ").Append(classObj.Namespace.FullSourceName).Append(classObj.SourceName).Append(";\r\n");
				builder.Append("\t\tif (!").Append(registrationName).Append(".InnerSingleton)\r\n");
				builder.Append("\t\t{\r\n");
				builder.Append("\t\t\t").Append(classObj.IsVerseField ? "Verse::CodeGen::Private::ConstructUVerseClassNoInit" : "GetPrivateStaticClassBody").Append("(\r\n");
				builder.Append("\t\t\t\tTClass::StaticPackage(),\r\n");
				builder.Append("\t\t\t\tTEXT(\"").Append(classObj.EngineName).Append("\"),\r\n");
				builder.Append("\t\t\t\t").Append(registrationName).Append(".InnerSingleton,\r\n");
				builder.Append("\t\t\t\t").Append(registerNativesFuncName).Append(",\r\n");
				builder.Append("\t\t\t\tDataSizeOf<TClass>(),\r\n");
				builder.Append("\t\t\t\talignof(TClass),\r\n");
				builder.Append("\t\t\t\tTClass::StaticClassFlags,\r\n");
				builder.Append("\t\t\t\tTClass::StaticClassCastFlags(),\r\n");
				builder.Append("\t\t\t\tTClass::StaticConfigName(),\r\n");
				builder.Append("\t\t\t\t(UClass::ClassConstructorType)InternalConstructor<TClass>,\r\n");
				builder.Append("\t\t\t\t(UClass::ClassVTableHelperCtorCallerType)InternalVTableHelperCtorCaller<TClass>,\r\n");
				builder.Append("\t\t\t\tUOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(TClass),\r\n");
				builder.Append("\t\t\t\t&TClass::Super::StaticClass,\r\n");
				builder.Append("\t\t\t\t&TClass::WithinClass::StaticClass\r\n");
				builder.Append("\t\t\t);\r\n");
				builder.Append("\t\t}\r\n");
				builder.Append("\t\treturn ").Append(registrationName).Append(".InnerSingleton;\r\n");
				builder.Append("\t}\r\n");
				builder.Append("\tif (!").Append(registrationName).Append(".OuterSingleton)\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\t")
					.Append(classObj.IsVerseField ? "Verse::CodeGen::Private::ConstructUVerseClass(" : "UECodeGen_Private::ConstructUClass(")
					.Append(registrationName).Append(".OuterSingleton, UHT_STATICS::ClassParams);\r\n");
				if (sparseDataTypes != null)
				{
					foreach (string sparseClass in sparseDataTypes)
					{
						builder.Append("\t\t").Append(registrationName).Append(".OuterSingleton->SetSparseClassDataStruct(").Append(classObj.SourceName).Append("::StaticGet").Append(sparseClass).Append("ScriptStruct());\r\n");
					}
				}
				builder.Append("\t}\r\n");
				builder.Append("\treturn ").Append(registrationName).Append(".OuterSingleton;\r\n");
				builder.Append("}\r\n");
			}
			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				builder.Append("UClass* ").Append(singletonName).Append("(ETypeConstructPhase Phase)\r\n");
				builder.Append("{\r\n");
				builder.Append("\treturn &").Append(GetSingletonName(classObj, UhtSingletonType.ConstInit)).Append(";\r\n");
				builder.Append("}\r\n");
			}

			// At this point, we can compute the hash... HOWEVER, in the old UHT extra data is appended to the hash block that isn't emitted to the actual output
			using (BorrowStringBuilder hashBorrower = new(StringBuilderCache.Small))
			{
				StringBuilder hashBuilder = hashBorrower.StringBuilder;
				hashBuilder.Append(builder, hashCodeBlockStart, builder.Length - hashCodeBlockStart);

				int saveLength = hashBuilder.Length;

				// Append base class' hash at the end of the generated code, this will force update derived classes
				// when base class changes during hot-reload.
				IoHash baseClassHash = IoHash.Zero;
				if (classObj.SuperClass != null && !classObj.SuperClass.ClassFlags.HasAnyFlags(EClassFlags.Intrinsic))
				{
					baseClassHash = ObjectInfos[classObj.SuperClass.ObjectTypeIndex].Hash;
				}
				hashBuilder.Append($"\r\n// {baseClassHash}\r\n");

				// Append info for the sparse class data struct onto the text to be hashed
				if (sparseDataTypes != null)
				{
					foreach (string sparseDataType in sparseDataTypes)
					{
						UhtType? type = Session.FindType(classObj, UhtFindOptions.ScriptStruct | UhtFindOptions.EngineName, sparseDataType);
						if (type != null)
						{
							hashBuilder.Append(type.EngineName).Append("\r\n");
							for (UhtScriptStruct? sparseStruct = type as UhtScriptStruct; sparseStruct != null; sparseStruct = sparseStruct.SuperScriptStruct)
							{
								foreach (UhtProperty property in sparseStruct.Properties)
								{
									hashBuilder.AppendPropertyText(property, UhtPropertyTextType.SparseShort).Append(' ').Append(property.SourceName).Append("\r\n");
								}
							}
						}
					}
				}

				if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
				{
					builder.Append("/* friend declarations for pasting into noexport class ").Append(classObj.SourceName).Append("\r\n");
					builder.Append("friend struct UHT_STATICS;\r\n");
					builder.Append("*/\r\n");
				}

				if (Session.IncludeDebugOutput)
				{
					using UhtRentedPoolBuffer<char> borrowBuffer = hashBuilder.RentPoolBuffer(saveLength, hashBuilder.Length - saveLength);
					builder.Append("#if 0\r\n");
					builder.Append(borrowBuffer.Buffer.Memory);
					builder.Append("#endif\r\n");
				}

				// Calculate generated class initialization code hash so that we know when it changes after hot-reload
				{
					using UhtRentedPoolBuffer<char> borrowBuffer = hashBuilder.RentPoolBuffer();
					ObjectInfos[classObj.ObjectTypeIndex].Hash = IoHash.Compute<char>(borrowBuffer.Buffer.Memory.Span);
				}
			}

			if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.SelfHasReplicatedProperties))
			{
				builder.Append("#if VALIDATE_CLASS_REPS\r\n");
				builder.Append("void ").Append(classObj.SourceName).Append("::ValidateGeneratedRepEnums(const TArray<struct FRepRecord>& ClassReps) const\r\n");
				builder.Append("{\r\n");

				foreach (UhtProperty property in classObj.Properties)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
					{
						// Do not make this a static const.  It causes issues with live coding
						builder.Append("\tstatic FName Name_").Append(property.SourceName).Append("(TEXT(\"").Append(property.SourceName).Append("\"));\r\n");
					}
				}
				builder.Append("\tconst bool bIsValid = true");
				foreach (UhtProperty property in classObj.Properties)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
					{
						if (!property.IsStaticArray)
						{
							builder.Append("\r\n\t\t&& Name_").Append(property.SourceName).Append(" == ClassReps[(int32)ENetFields_Private::").Append(property.SourceName).Append("].Property->GetFName()");
						}
						else
						{
							builder.Append("\r\n\t\t&& Name_").Append(property.SourceName).Append(" == ClassReps[(int32)ENetFields_Private::").Append(property.SourceName).Append("_STATIC_ARRAY].Property->GetFName()");
						}
					}
				}
				builder.Append(";\r\n");
				builder.Append("\tcheckf(bIsValid, TEXT(\"UHT Generated Rep Indices do not match runtime populated Rep Indices for properties in ").Append(classObj.SourceName).Append("\"));\r\n");
				builder.Append("}\r\n");
				builder.Append("#endif\r\n");
			}
			return builder;
		}
#pragma warning restore CA1505 //  'AppendNativeGeneratedInitCode' has a maintainability index of '5'. Rewrite or refactor the code to increase its maintainability index (MI) above '9'. 

		private static StringBuilder AppendImplementedInterfaceParam(StringBuilder builder, UhtPropertyMemberContextImpl context, UhtClass classObj, UhtClass interfaceObj, UhtSingletonType singletonType)
		{
			if (interfaceObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
			{
				builder
					.Append("\t{ ")
					.AppendSingletonRef(context, interfaceObj.AlternateObject, singletonType);
				if (interfaceObj.IsVerseField)
				{
					builder
						.Append(", 0, true");
					if (classObj.VerseInterfaces.Contains(interfaceObj))
					{
						builder.Append(", true");
					}
				}
				else
				{
					builder
						.Append(", (int32)VTABLE_OFFSET(")
						.Append(classObj.FullyQualifiedSourceName)
						.Append(", ")
						.Append(interfaceObj.FullyQualifiedSourceName)
						.Append("), false");
				}
				builder
					.Append(" }, ")
					.AppendObjectHash(classObj, context, interfaceObj.AlternateObject)
					.Append("\r\n");
			}
			return builder;
		}

		private static StringBuilder AppendFunctionThunk(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{

			// TEMPORARY: Verse suspends methods can't be called but must exist.  So generate an error
			if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends))
			{
				builder.Append("\tcheckf(false, TEXT(\"Verse coroutines can not be invoked from their UFunction\"));\r\n");
				return builder;
			}

			// Export the GET macro for the parameters
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					builder.Append('\t').AppendFunctionThunkParameterGet(property).Append(";\r\n");
				}
			}

			builder.Append("\tP_FINISH;\r\n");
			builder.Append("\tP_NATIVE_BEGIN;\r\n");

			// Call the validate function if there is one
			if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CppStatic) && function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate))
			{
				builder.Append("\tif (!P_THIS->").Append(function.CppValidationImplName).Append('(').AppendFunctionThunkParameterNames(function).Append("))\r\n");
				builder.Append("\t{\r\n");
				builder.Append("\t\tRPC_ValidateFailed(TEXT(\"").Append(function.CppValidationImplName).Append("\"));\r\n");
				builder.Append("\t\treturn;\r\n");   // If we got here, the validation function check failed
				builder.Append("\t}\r\n");
			}

			// Write out the return value
			builder.Append('\t');
			UhtProperty? returnProperty = function.ReturnProperty;
			if (returnProperty != null)
			{
				builder.Append("*(").AppendFunctionThunkReturn(returnProperty).Append("*)Z_Param__Result=");
			}

			// Export the call to the C++ version
			if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CppStatic))
			{
				if (classObj.ClassType == UhtClassType.VModule)
				{
					builder.Append($"{classObj.Namespace.FullSourceName}{function.CppImplName}(").AppendFunctionThunkParameterNames(function).Append(");\r\n");
				}
				else
				{
					builder.Append($"{classObj.SourceName}::{function.CppImplName}(").AppendFunctionThunkParameterNames(function).Append(");\r\n");
				}
			}
			else if (function.Outer is UhtPartial partial)
			{
				// Function is in an partial - call through GetPartial<>()
				builder.Append("P_THIS->GetPartial<").Append(partial.SourceName).Append(">().").Append(function.CppImplName).Append('(').AppendFunctionThunkParameterNames(function).Append(");\r\n");
			}
			else
			{
				builder.Append("P_THIS->").Append(function.CppImplName).Append('(').AppendFunctionThunkParameterNames(function).Append(");\r\n");
			}
			builder.Append("\tP_NATIVE_END;\r\n");
			return builder;
		}

		private void AppendVniTypeDesc(StringBuilder builder, UhtField fieldObj)
		{
			UHTManifest.Module module = fieldObj.Package.Module.Module;
			builder.AppendLine($$"""
					static constexpr FVniTypeDesc NativeTypeDesc
					{
						.UEPackageName = TEXT("{{fieldObj.Package.EngineName}}"),
						.UEName = TEXT("{{fieldObj.EngineName}}"),
						.VersePackageDesc = &{{VniModuleStatics(Module)}}::{{VniPackageDesc(Module)}},
						.VersePackageName = TEXT("{{module.VersePackageName}}"),
						.VerseModulePath = TEXT("{{module.VersePath}}"),
						.VerseScopeName = TEXT("{{fieldObj.VerseName}}"),
					};
				""");
		}

		private static void FindNoExportStructsRecursive(List<UhtScriptStruct> outScriptStructs, UhtStruct structObj)
		{
			for (UhtStruct? current = structObj; current != null; current = current.SuperStruct)
			{
				// Is isn't true for noexport structs
				if (current is UhtScriptStruct scriptStruct)
				{
					if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
					{
						break;
					}

					// these are a special cases that already exists and if wrong if exported naively
					if (!scriptStruct.IsAlwaysAccessible)
					{
						outScriptStructs.Remove(scriptStruct);
						outScriptStructs.Add(scriptStruct);
					}
				}

				foreach (UhtType type in current.Children)
				{
					if (type is UhtProperty property)
					{
						foreach (UhtType referenceType in property.EnumerateReferencedTypes())
						{
							if (referenceType is UhtScriptStruct propertyScriptStruct)
							{
								FindNoExportStructsRecursive(outScriptStructs, propertyScriptStruct);
							}
						}
					}
				}
			}
		}

		private static List<UhtScriptStruct> FindNoExportStructs(UhtStruct structObj)
		{
			List<UhtScriptStruct> outScriptStructs = [];
			FindNoExportStructsRecursive(outScriptStructs, structObj);
			outScriptStructs.Reverse();
			return outScriptStructs;
		}

		private static UhtRegistrations GetRegistrations(Dictionary<UhtPackage, UhtRegistrations> packageRegistrations, UhtField fieldObj)
		{
			UhtPackage package = fieldObj.Package;
			if (packageRegistrations.TryGetValue(package, out UhtRegistrations? registrations))
			{
				return registrations;
			}
			registrations = new();
			packageRegistrations.Add(package, registrations);
			return registrations;
		}

		private sealed class UhtPropertyMemberContextImpl : IUhtPropertyMemberContext
		{
			private readonly UhtHeaderCodeGenerator _codeGenerator;
			private readonly UhtStruct _outerStruct;
			private readonly UhtCppIdentifier _outerIdentifier;

			public UhtPropertyMemberContextImpl(UhtHeaderCodeGenerator codeGenerator, UhtStruct outerStruct, string outerStructNamespaceName, string outerStructSourceName)
			{
				_codeGenerator = codeGenerator;
				_outerStruct = outerStruct;
				_outerIdentifier = new(outerStructNamespaceName, null, outerStructSourceName, null);
			}

			public UhtPropertyMemberContextImpl(UhtHeaderCodeGenerator codeGenerator, UhtStruct outerStruct)
			{
				_codeGenerator = codeGenerator;
				_outerStruct = outerStruct;
				string outerStructNamespaceName = _outerStruct is UhtScriptStruct scriptStruct ? scriptStruct.NamespaceExportName : (_outerStruct is UhtPartial partial ? partial.NamespaceExportName : _outerStruct.Namespace.FullSourceName);
				string outerStructSourceName = _outerStruct.SourceName;
				_outerIdentifier = new(outerStructNamespaceName, null, outerStructSourceName, null);
			}

			public UhtStruct OuterStruct => _outerStruct;

			public UhtCppIdentifier OuterIdentifier => _outerIdentifier;

			public bool IsLegacy { get; set; } = true;

			public string GetSingletonName(UhtObject? obj, UhtSingletonType type)
			{
				return _codeGenerator.GetSingletonName(obj, type);
			}

			public bool IsCrossModuleRef(UhtObject obj) => _codeGenerator.IsCrossModuleRef(obj);

			public int GetModuleIndex(UhtModule obj) => _codeGenerator.GetModuleIndex(obj);

			public int GetObjectIndex(UhtObject obj) => _codeGenerator.GetObjectIndex(obj);

			public IoHash GetTypeHash(UhtObject obj)
			{
				return _codeGenerator.ObjectInfos[obj.ObjectTypeIndex].Hash;
			}
		}
	}

	/// <summary>
	/// Collection of string builder extensions used to generate the cpp files for individual headers.
	/// </summary>
	public static class UhtHeaderCodeGeneratorCppFileStringBuilderExtensions
	{
		/// <summary>
		/// Appends a structure initialization expression for a constant-initialized UObject, followed
		/// by a comma and newline for use in a larger aggregate initialization expression
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="uhtObject"></param>
		/// <param name="linker"></param>
		/// <param name="tabs"></param>
		/// <param name="additionalObjectFlags"></param>
		/// <returns></returns>
		public static StringBuilder AppendConstInitObjectParams(this StringBuilder builder, UhtObject uhtObject, IUhtObjectLinker linker, int tabs, string? additionalObjectFlags = null)
		{
			const string ObjectFlags = "RF_Public|RF_Transient|RF_MarkAsNative";
			if (uhtObject == uhtObject.Session.UClass)
			{
				builder.AppendTabs(tabs).AppendLine("UE::CodeGen::ConstInit::EClass::Class, ");
			}
			else if (uhtObject == uhtObject.Session.UObject)
			{
				builder.AppendTabs(tabs).AppendLine("UE::CodeGen::ConstInit::EObject::Object, ");
			}
			builder.AppendTabs(tabs).Append("UE::CodeGen::ConstInit::FObjectParams{\r\n");
			builder.AppendTabs(tabs + 1).Append($".Flags = {ObjectFlags}{additionalObjectFlags},\r\n");
			// Can't take a pointer to Class to initialize itself because of TNoDestroy, null class signals to take the address of Class in the constructor 
			if (uhtObject != uhtObject.EngineClass)
			{
				builder.AppendLine($"{Tabs(tabs + 1)}.Class = {SingletonRef(linker, uhtObject.EngineClass, UhtSingletonType.ConstInit)}, ");
			}
			UhtObject? outer = uhtObject.Outer as UhtObject;
			if (outer is UhtPartial partial)
			{
				outer = partial.OwnerClass;
			}
			builder.AppendTabs(tabs + 1).Append($".NameUTF8 = UTF8TEXT(").AppendUTF8LiteralString(uhtObject.EngineName).Append("),\r\n");
			builder.AppendTabs(tabs + 1).AppendLine($".Outer = {ConstInitSingletonRef(linker, outer)},");
			builder.AppendTabs(tabs).Append("},\r\n");
			return builder;
		}

		/// <summary>
		/// Append the parameter names for a function
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="function">Function in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterNames(this StringBuilder builder, UhtFunction function)
		{
			bool first = true;
			foreach (UhtType parameter in function.ParameterProperties.Span)
			{
				if (parameter is UhtProperty property)
				{
					if (first)
					{
						first = false;
					}
					else
					{
						builder.Append(',');
					}

					bool needsGCBarrier = property.NeedsGCBarrierWhenPassedToFunction(function);
					if (needsGCBarrier)
					{
						builder.Append("P_ARG_GC_BARRIER(");
					}
					builder.AppendFunctionThunkParameterArg(property);
					if (needsGCBarrier)
					{
						builder.Append(')');
					}
				}
			}
			return builder;
		}

		/// <summary>
		/// Append the name of the function params type
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="function">Function in question</param>
		/// <returns></returns>
		public static StringBuilder AppendFunctionParamsType(this StringBuilder builder, UhtFunction function)
		{
			switch (function.FunctionType)
			{
				case UhtFunctionType.Function:
					builder.Append(function.IsVerseField ? "FVerseFunctionParams" : "FFunctionParams");
					break;
				case UhtFunctionType.Delegate:
					builder.Append("FDelegateFunctionParams");
					break;
				case UhtFunctionType.SparseDelegate:
					builder.Append("FSparseDelegateFunctionParams");
					break;
			}
			return builder;
		}

		/// <summary>
		/// Append the name of the function constructor
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="function">Function in question</param>
		/// <returns></returns>
		public static StringBuilder AppendFunctionConstructorName(this StringBuilder builder, UhtFunction function)
		{
			switch (function.FunctionType)
			{
				case UhtFunctionType.Function:
					builder.Append(function.IsVerseField ? "Verse::CodeGen::Private::ConstructUVerseFunction" : "UECodeGen_Private::ConstructUFunction");
					break;
				case UhtFunctionType.Delegate:
					builder.Append("UECodeGen_Private::ConstructUDelegateFunction");
					break;
				case UhtFunctionType.SparseDelegate:
					builder.Append("UECodeGen_Private::ConstructUSparseDelegateFunction");
					break;
			}
			return builder;
		}

		/// <summary>
		/// Append the name of the class params
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="classObj">Class in question</param>
		/// <returns></returns>
		public static StringBuilder AppendClassParamsType(this StringBuilder builder, UhtClass classObj)
		{
			builder.Append(classObj.IsVerseField ? "FVerseClassParams" : "FClassParams");
			return builder;
		}

		/// <summary>
		/// Append the name of the struct params
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="structObj">Struct in question</param>
		/// <returns></returns>
		public static StringBuilder AppendStructParamsType(this StringBuilder builder, UhtStruct structObj)
		{
			builder.Append(structObj.IsVerseField ? "FVerseStructParams" : "FStructParams");
			return builder;
		}

		/// <summary>
		/// Append the name of the enum params
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="enumObj">enum in question</param>
		/// <returns></returns>
		public static StringBuilder AppendEnumParamsType(this StringBuilder builder, UhtEnum enumObj)
		{
			builder.Append(enumObj.IsVerseField ? "FVerseEnumParams" : "FEnumParams");
			return builder;
		}
	}
}
