// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal sealed class UhtHeaderCodeGeneratorHFile
		: UhtHeaderCodeGenerator
	{
		public static string RigVMExecuteContextParamName = "ExecuteContext";
		public static string RigVMExecuteContextDeclaration = "FRigVMExtendedExecuteContext& RigVMExecuteContext";

		/// <summary>
		/// Construct an instance of this generator object
		/// </summary>
		/// <param name="codeGenerator">The base code generator</param>
		/// <param name="headerFile">Header file being generated</param>
		public UhtHeaderCodeGeneratorHFile(UhtCodeGenerator codeGenerator, UhtHeaderFile headerFile)
			: base(codeGenerator, headerFile)
		{
		}

		/// <summary>
		/// For a given UE header file, generated the generated H file
		/// </summary>
		/// <param name="factory">Requesting factory</param>
		public void Generate(IUhtExportFactory factory)
		{
			ref UhtCodeGenerator.HeaderInfo headerInfo = ref HeaderInfos[HeaderFile.HeaderFileTypeIndex];
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
				StringBuilder builder = borrower.StringBuilder;

				builder.Append(HeaderCopyright);
				builder.Append("// IWYU pragma: private, include \"").Append(HeaderFile.IncludeFilePath).Append("\"\r\n");

				string strippedName = Path.GetFileNameWithoutExtension(HeaderFile.FilePath);
				string defineName = $"{Module.ShortName.ToUpperInvariant()}_{strippedName}_generated_h".Replace('.', '_');

				builder.Append("\r\n");
				builder.Append("#ifdef ").Append(defineName).Append("\r\n");
				builder.Append("#error \"").Append(strippedName).Append(".generated.h already included, missing '#pragma once' in ").Append(strippedName).Append(".h\"\r\n");
				builder.Append("#endif\r\n");
				builder.Append("#define ").Append(defineName).Append("\r\n");

				// Attempt to limit the headers included. This is needed in the lower level engine code
				// to get around circular header include issues.
				builder.Append("\r\n");

				builder.Append("#include \"UObject/ObjectMacros.h\"\r\n");

				if (HeaderFile.References.ExportTypes.Any(x => x is UhtEnum or UhtClass or UhtScriptStruct))
				{
					// We need to specialize StaticClass/StaticStruct/StaticEnum template 
					builder.Append("#include \"UObject/ReflectedTypeAccessors.h\"\r\n");
				}
				if (HeaderFile.References.ExportTypes.Any(x => x is UhtEnum))
				{
					// We need to specialize TIsUEnumClass
					builder.Append("#include \"Templates/IsUEnumClass.h\"\r\n");
				}

				if (Module.Module.MinimizeGeneratedIncludes || (HeaderFile.References.ExportTypes.Count > 0 && HeaderFile.References.ExportTypes.Find(x => x is not UhtEnum) == null))
				{
					builder.Append("#include \"Templates/NoDestroy.h\"\r\n");
				}
				else
				{
					builder.Append("#include \"UObject/ScriptMacros.h\"\r\n");
				}

				if (headerInfo.NeedsPushModelHeaders)
				{
					builder.Append("#include \"Net/Core/PushModel/PushModelMacros.h\"\r\n");
				}
				if (headerInfo.NeedsVerseString)
				{
					builder.Append("#include \"VerseVM/VVMNativeString.h\"\r\n");
				}
				if (headerInfo.NeedsVerseValue)
				{
					builder.Append("#include \"VerseValue.h\"\r\n");
				}
				if (headerInfo.NeedsVerseCodeGen)
				{
					builder.Append("#include \"VerseInteropMacros.h\"\r\n");
				}
				if (headerInfo.NeedsVerseField)
				{
					builder.Append("#include \"VerseField.h\"\r\n");
				}
				if (headerInfo.NeedsVersePointer)
				{
					builder.Append("#include \"VersePointer.h\"\r\n");
				}
				if (headerInfo.NeedsVerseTask)
				{
					builder.Append("#include \"VerseTask.h\"\r\n");
				}
				if (headerInfo.NeedsVerseCoroutine)
				{
					builder.Append("#include \"VerseCoroutine.h\"\r\n");
				}

				builder.Append("\r\n");
				builder.Append(DisableDeprecationWarnings).Append("\r\n");

				StringComparerUE compareUE = StringComparerUE.OrdinalIgnoreCase;
				if (HeaderFile.References.ForwardDeclarations.Count > 0)
				{
					UhtField[] sorted = [.. HeaderFile.References.ForwardDeclarations];
					Array.Sort(sorted, (x, y) =>
					{

						int compare;
						compare = compareUE.Compare(x.Namespace.FullSourceName, y.Namespace.FullSourceName);
						if (compare != 0)
						{
							return compare;
						}

						compare = compareUE.Compare(x.EngineClassName, y.EngineClassName);
						if (compare != 0)
						{
							return compare;
						}

						return compareUE.Compare(x.SourceName, y.SourceName);
					});

					for (int index = 0; index < sorted.Length;)
					{
						UhtNamespace namespaceObj = sorted[index].Namespace;
						namespaceObj.AppendMultipleLines(builder, (builder) =>
						{
							for (; index < sorted.Length && sorted[index].Namespace == namespaceObj; index++)
							{
								builder.AppendForwardDeclaration(sorted[index]).Append("\r\n");
							}
						});
					}
				}
				if (HeaderFile.References.ForwardDeclarationStrings.Count > 0)
				{
					string[] sorted = [.. HeaderFile.References.ForwardDeclarationStrings];
					Array.Sort(sorted, StringComparerUE.OrdinalIgnoreCase);
					builder.Append("\r\n");
					foreach (string forwardDeclaration in sorted)
					{
						builder.Append(forwardDeclaration).Append("\r\n");
					}
				}

				foreach (UhtField field in HeaderFile.References.ExportTypes)
				{
					if (field is UhtEnum enumObj)
					{
					}
					else if (field is UhtScriptStruct scriptStruct)
					{
						if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
						{
							using UhtCodeBlockComment blockComment = new(builder, field);
							using UhtMacroBlockEmitter macroBlockEmitter = new(builder, scriptStruct.DefineScope);
							AppendScriptStruct(builder, scriptStruct);
						}
					}
					else if (field is UhtPartial partial)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, partial.DefineScope);
						Appendpartial(builder, partial);
					}
					else if (field is UhtFunction function)
					{
					}
					else if (field is UhtClass classObj)
					{
						if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Intrinsic))
						{
							using UhtCodeBlockComment blockComment = new(builder, field);
							using UhtMacroBlockEmitter macroBlockEmitter = new(builder, classObj.DefineScope);
							AppendClass(builder, classObj);
						}
					}
				}

				foreach (UhtField field in HeaderFile.References.ExportTypes)
				{
					if (field is UhtClass classObj)
					{
						if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Intrinsic) && classObj.ClassType == UhtClassType.Interface && classObj.IsVerseField)
						{
							using UhtCodeBlockComment blockComment = new(builder, field, "Verse Interface Proxy");
							using UhtMacroBlockEmitter macroBlockEmitter = new(builder, classObj.DefineScope);
							AppendNativeInterfaceVerseProxy(builder, classObj);
						}
					}
				}

				builder.Append("\r\n");
				builder.Append("#undef CURRENT_FILE_ID\r\n");
				builder.Append("#define CURRENT_FILE_ID ").Append(headerInfo.FileId).Append("\r\n");

				foreach (UhtField field in HeaderFile.References.ExportTypes)
				{
					if (field is UhtEnum enumObject)
					{
						using UhtCodeBlockComment blockComment = new(builder, field);
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, field.DefineScope);
						AppendEnum(builder, enumObject);
					}
				}

				builder.Append("\r\n");
				builder.Append(EnableDeprecationWarnings).Append("\r\n");

				if (SaveExportedHeaders)
				{
					string headerFilePath = factory.MakePath(HeaderFile, ".generated.h");
					factory.CommitOutput(headerFilePath, builder);
				}
			}
		}

		private static StringBuilder AppendRigVmStructInfoDefine(StringBuilder builder, UhtScriptStruct scriptStruct)
		{
			if (scriptStruct.RigVMStructInfo != null)
			{
				builder.Append("\r\n");
				foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
				{
					if (methodInfo.IsPredicate)
					{
						continue;
					}

					builder.Append("#define ").Append(scriptStruct.SourceName).Append('_').Append(methodInfo.Name).Append("() \\\r\n");
					builder.Append('\t').Append(methodInfo.ReturnType).Append(' ').Append(scriptStruct.SourceName).Append("::Static").Append(methodInfo.Name).Append("( \\\r\n");
					builder.Append("\t\t");
					if (!scriptStruct.RigVMStructInfo.HasAnyExecuteContextMember || scriptStruct.RigVMStructInfo.IsPure)
					{
						builder.Append("const ");
					}
					builder.Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append("& ").Append(RigVMExecuteContextParamName);
					builder.AppendParameterDecls(scriptStruct.RigVMStructInfo.Members, true, ", \\\r\n\t\t", true, false);
					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append(", \\\r\n\t\t").Append(predicateInfo.Name).Append("Struct& ").Append(predicateInfo.Name);
						}
					}
					builder.Append(" \\\r\n");
					builder.Append("\t)\r\n");
				}
				builder.Append("\r\n");
			}

			return builder;
		}

		private static StringBuilder AppendRigVmStructInfoMethods(StringBuilder builder, UhtScriptStruct scriptStruct)
		{
			// if we have RigVM methods on this struct we need to 
			// declare the static method as well as the stub method
			if (scriptStruct.RigVMStructInfo != null)
			{
				string constPrefix = "";
				if (!scriptStruct.RigVMStructInfo.HasAnyExecuteContextMember || scriptStruct.RigVMStructInfo.IsPure)
				{
					constPrefix = "const ";
				}

				builder.Append("\tvirtual const UScriptStruct* GetExecuteContextStruct() const override { return ");
				builder.Append(scriptStruct.RigVMStructInfo.ExecuteContextType);
				builder.Append("::StaticStruct(); } \\\r\n");

				foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
				{
					if (methodInfo.IsPredicate)
					{
						builder.Append('\t').Append("struct ").Append(methodInfo.Name).Append("Struct \\\r\n");
						builder.Append("\t{ \\\r\n");
						builder.Append("\t\t").Append(methodInfo.Name).Append("Struct(){  } \\\r\n");
						builder.Append("\t\t").Append(methodInfo.Name).Append("Struct(FRigVMExtendedExecuteContext& InContext, FRigVMPredicateBranch InBranch){ Context = &InContext; Branch = InBranch; } \\\r\n");
						builder.Append("\t\t").Append(methodInfo.ReturnType).Append(" Execute(\\\r\n\t\t\t");
						builder.AppendParameterDecls(methodInfo.Parameters, false, ", \\\r\n\t\t\t", true, false);
						builder.Append("\t\t)  \\\r\n");
						builder.Append("\t\t{  \\\r\n");
						builder.Append("\t\t\tif (Branch.IsValid())  \\\r\n");
						builder.Append("\t\t\t{  \\\r\n");
						int parameterIndex = 0;
						foreach (UhtRigVMParameter parameter in methodInfo.Parameters)
						{
							string baseType = parameter.TypeOriginal().ToString();
							if (baseType.StartsWith("const", StringComparison.Ordinal))
							{
								baseType = baseType[5..].Trim();
							}

							if (baseType.EndsWith('&'))
							{
								baseType = baseType[..^1].Trim();
							}
							builder.Append("\t\t\t\t*(").Append(baseType).Append("*) Branch.MemoryHandles[")
								.Append(parameterIndex++).Append("].GetOutputData() = ")
								.Append(parameter.Name).Append(";  \\\r\n");

						}
						builder.Append("\t\t\t\tBranch.Execute(*Context);  \\\r\n");
						builder.Append("\t\t\t\treturn *(").Append(methodInfo.ReturnType).Append("*)Branch.MemoryHandles[").Append(methodInfo.Parameters.Count).Append("].GetInputData();  \\\r\n");
						builder.Append("\t\t\t}  \\\r\n");
						builder.Append("\t\t\treturn ").Append(methodInfo.Name).Append('(');
						builder.AppendParameterNames(methodInfo.Parameters, false, ", ", false, false);
						builder.Append("); \\\r\n");
						builder.Append("\t\t}  \\\r\n");
						builder.Append("\tprivate: \\\r\n");
						builder.Append("\t\tFRigVMExtendedExecuteContext* Context; \\\r\n");
						builder.Append("\t\tFRigVMPredicateBranch Branch; \\\r\n");
						builder.Append("\t};  \\\r\n");
					}
				}

				foreach (UhtRigVMMethodInfo methodInfo in scriptStruct.RigVMStructInfo.Methods)
				{
					if (methodInfo.IsPredicate)
					{
						continue;
					}

					builder.Append('\t').Append(methodInfo.DllStorage).Append(methodInfo.ReturnType).Append(' ').Append(methodInfo.Name).Append('(').Append(constPrefix).Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append("& InExecuteContext); \\\r\n");
					builder.Append("\tstatic ").Append(methodInfo.ReturnType).Append(' ').Append(methodInfo.DllStorage).Append("Static").Append(methodInfo.Name).Append("( \\\r\n");
					builder.Append("\t\t");
					if (!scriptStruct.RigVMStructInfo.HasAnyExecuteContextMember || scriptStruct.RigVMStructInfo.IsPure)
					{
						builder.Append("const ");
					}
					builder.Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append("& ").Append(RigVMExecuteContextParamName);
					builder.AppendParameterDecls(scriptStruct.RigVMStructInfo.Members, true, ", \\\r\n\t\t", true, false);

					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append(", \\\r\n\t\t").Append(predicateInfo.Name).Append("Struct& ").Append(predicateInfo.Name);
						}
					}

					builder.Append(" \\\r\n");
					builder.Append("\t); \\\r\n");

					builder.Append("\tFORCEINLINE_DEBUGGABLE static ").Append(methodInfo.ReturnType).Append(" RigVM").Append(methodInfo.Name).Append("( \\\r\n");
					builder.Append("\t\t");
					builder.Append(RigVMExecuteContextDeclaration).Append(", \\\r\n");
					builder.Append("\t\tFRigVMMemoryHandleArray RigVMMemoryHandles, \\\r\n");
					builder.Append("\t\tFRigVMPredicateBranchArray RigVMBranches \\\r\n");
					builder.Append("\t) \\\r\n");
					builder.Append("\t{ \\\r\n");

					if (scriptStruct.RigVMStructInfo.Members.Count > 0)
					{
						int operandIndex = 0;
						foreach (UhtRigVMParameter parameter in scriptStruct.RigVMStructInfo.Members)
						{
							string paramTypeOriginal = parameter.TypeOriginal(true);
							string paramNameOriginal = parameter.NameOriginal(false);
							string additionalParameters = String.Empty;
							if (parameter.IsLazy || (!parameter.Input && !parameter.Output && !parameter.Singleton))
							{
								additionalParameters = "RigVMExecuteContext.GetSlice().GetIndex()";
							}

							string getDataMethod = "GetInputData";
							if (parameter.IsLazy)
							{
								getDataMethod = $"GetDataLazily<{parameter.TypeOriginal(false, false)}>";
							}
							else if (parameter.Output)
							{
								getDataMethod = "GetOutputData";
							}
							else if (!parameter.Input && !parameter.Output && !parameter.Singleton)
							{
								getDataMethod = "GetPrivateData";
							}

							if (parameter.IsArray)
							{
								string extendedType = parameter.ExtendedType();

								if (!parameter.IsLazy)
								{
									builder
										.Append("\t\tTArray")
										.Append(extendedType)
										.Append("& ")
										.Append(paramNameOriginal)
										.Append(" = ")
										.Append("*(TArray")
										.Append(extendedType)
										.Append("*)");
								}
								else
								{
									builder
										.Append("\t\tconst ")
										.Append(paramTypeOriginal)
										.Append("& ")
										.Append(paramNameOriginal)
										.Append(" = ");
								}

								builder
									.Append("RigVMMemoryHandles[")
									.Append(operandIndex)
									.Append("].")
									.Append(getDataMethod)
									.Append('(')
									.Append(additionalParameters)
									.Append("); \\\r\n");
								operandIndex++;
							}
							else
							{
								string variableType = parameter.TypeVariableRef(true);
								string extractedType = parameter.TypeOriginal();

								if (parameter.IsEnumAsByte)
								{
									extractedType = parameter.TypeOriginal(true);
								}

								builder
									.Append("\t\t")
									.Append(variableType)
									.Append(' ')
									.Append(paramNameOriginal)
									.Append(" = ");

								if (!parameter.IsLazy)
								{
									builder
										.Append("*(")
										.Append(extractedType)
										.Append("*)");
								}

								builder
									.Append("RigVMMemoryHandles[")
									.Append(operandIndex)
									.Append("].")
									.Append(getDataMethod)
									.Append('(')
									.Append(additionalParameters)
									.Append("); \\\r\n");
								operandIndex++;
							}
						}
						builder.Append("\t\t \\\r\n");
					}

					int predicateId = 0;
					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append("\t\t").Append(predicateInfo.Name).Append("Struct ").Append(predicateInfo.Name).Append("Predicate(RigVMExecuteContext, RigVMBranches[").Append(predicateId++).Append("]); \\\r\n");
						}
					}

					builder.Append("\t\t").Append(methodInfo.ReturnPrefix()).Append("Static").Append(methodInfo.Name).Append("( \\\r\n");
					builder.Append("\t\t\tRigVMExecuteContext.GetPublicData<").Append(scriptStruct.RigVMStructInfo.ExecuteContextType).Append(">()");
					builder.AppendParameterNames(scriptStruct.RigVMStructInfo.Members, true, ", \\\r\n\t\t\t", false);
					foreach (UhtRigVMMethodInfo predicateInfo in scriptStruct.RigVMStructInfo.Methods)
					{
						if (predicateInfo.IsPredicate)
						{
							builder.Append(", \\\r\n\t\t\t").Append(predicateInfo.Name).Append("Predicate");
						}
					}
					builder.Append(" \\\r\n");
					builder.Append("\t\t); \\\r\n");
					builder.Append("\t} \\\r\n");
				}
			}

			return builder;
		}

		private StringBuilder AppendScriptStruct(StringBuilder builder, UhtScriptStruct scriptStruct)
		{
			string singleton = GetSingletonName(scriptStruct, UhtSingletonType.Registered);
			builder.Append("struct Z_Construct_UScriptStruct_").Append(scriptStruct.SourceName).Append("_Statics;\r\n");
			builder.Append(Module.Api).Append("UScriptStruct* ").Append(singleton).Append("(ETypeConstructPhase);\r\n");
			builder.Append("\r\n");

			AppendRigVmStructInfoDefine(builder, scriptStruct);

			if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
			{
				using (UhtMacroCreator macro = new(builder, this, scriptStruct, GeneratedBodyMacroSuffix))
				{
					builder.Append("\tfriend struct ::Z_Construct_UScriptStruct_").Append(scriptStruct.SourceName).Append("_Statics; \\\r\n");
					builder.Append('\t');
					builder.Append("UE_NODEBUG ");
					builder.Append("static UScriptStruct* StaticStruct() { return ").Append(singleton).Append("(ETypeConstructPhase::Inner); } \\\r\n");

					AppendRigVmStructInfoMethods(builder, scriptStruct);

					if (scriptStruct.SuperScriptStruct != null)
					{
						builder.Append("\ttypedef ").Append(scriptStruct.SuperScriptStruct.SourceName).Append(" Super; \\\r\n");
					}
					UhtProperty? fastArrayProperty = ObjectInfos[scriptStruct.ObjectTypeIndex].FastArrayProperty;
					if (fastArrayProperty != null)
					{
						builder
							.Append("\tUE_NET_DECLARE_FASTARRAY(")
							.Append(scriptStruct.SourceName)
							.Append(", ")
							.Append(fastArrayProperty.SourceName)
							.Append(", ");
						if (!scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.RequiredAPI) &&
							(Module.Module.AlwaysExportStructs || scriptStruct.ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.MinimalAPI)))
						{
							builder.Append(Module.Api);
						}
						builder.Append("); \\\r\n");
					}

					if (scriptStruct.MetaData.ContainsKey(UhtNames.StateStreamStaticState) || scriptStruct.MetaData.ContainsKey(UhtNames.StateStreamDynamicState))
					{
						if (scriptStruct.Properties.Count() > 64)
						{
							throw new UhtIceException("StateStream states cannot support more than 64 properties.");
						}

						string modifiedFlagsType = scriptStruct.Properties.Count() > 32 ? "uint64" : "uint32";

						builder.Append("\tenum class Fields : ").Append(modifiedFlagsType).Append(" \\\r\n");
						builder.Append("\t{ \\\r\n");
						ulong enumValue = 1;
						foreach (UhtProperty property in scriptStruct.Properties)
						{
							builder.Append("\t\tField_").Append(property.SourceName).Append(" = ").Append(enumValue).Append(", \\\r\n");
							enumValue <<= 1;
						}
						builder.Append("\t}; \\\r\n");

						builder.Append('\t').Append(modifiedFlagsType).Append(" ModifiedFlags = 0; \\\r\n");

						builder.Append('\t').Append(scriptStruct.SourceName).Append("() = default; \\\r\n");

						builder.Append('\t').Append(scriptStruct.SourceName).Append("(FStateStreamCopyContext& Context, const ").Append(scriptStruct.SourceName).Append("& Other) \\\r\n");
						bool isFirst = true;

						builder.Append(isFirst ? "\t:\t" : "\t,\t").Append("ModifiedFlags(Other.ModifiedFlags) \\\r\n");
						isFirst = false;

						bool hasHandles = false;

						foreach (UhtProperty property in scriptStruct.Properties)
						{
							builder.Append(isFirst ? "\t:\t" : "\t,\t").Append(property.SourceName).Append('(');
							if (property is UhtStructProperty structProperty)
							{
								if (structProperty.ScriptStruct.MetaData.ContainsKey(UhtNames.StateStreamHandle))
								{
									hasHandles = true;
									builder.Append("Context, ");
								}
							}
							builder.Append("Other.").Append(property.SourceName);
							builder.Append(") \\\r\n");
							isFirst = false;
						}

						builder.Append("\t{} \\\r\n");

						foreach (UhtProperty property in scriptStruct.Properties)
						{
							string propName = property.SourceName;
							bool isBoolWithB = (property is UhtBoolProperty && propName.StartsWith('b'));
							string cleanPropName = isBoolWithB ? propName[1..] : propName;
							builder.Append("\tinline bool ").Append(cleanPropName).Append("Modified() const { return !!(ModifiedFlags & (").Append(modifiedFlagsType).Append(")Fields::Field_").Append(propName).Append("); } \\\r\n");

							builder.Append("\tinline void Set").Append(cleanPropName).Append("(const ");
							property.AppendText(builder, UhtPropertyTextType.Generic);
							builder.Append("& In) { ").Append(propName).Append(" = In; ModifiedFlags |= (").Append(modifiedFlagsType).Append(")Fields::Field_").Append(propName).Append("; } \\\r\n");

							if (isBoolWithB)
							{
								builder.Append("\tinline bool Get").Append(cleanPropName).Append("() const { return ").Append(propName).Append("; } \\\r\n");
							}
							else
							{
								builder.Append("\tinline const ");
								property.AppendText(builder, UhtPropertyTextType.Generic);
								builder.Append("& Get").Append(propName).Append("() const { return ").Append(propName).Append("; } \\\r\n");
							}
						}

						int dependencyCount = 0;
						builder.Append("\tuint32 GetDependencies(FStateStreamHandle** OutHandles, uint32 Capacity) \\\r\n");
						builder.Append("\t{ \\\r\n");
						if (hasHandles)
						{
							foreach (UhtProperty property in scriptStruct.Properties)
							{
								if (property is UhtStructProperty structProperty)
								{
									if (structProperty.ScriptStruct.MetaData.ContainsKey(UhtNames.StateStreamHandle))
									{
										builder.Append("\t\tOutHandles[").Append(dependencyCount++).Append("] = &").Append(property.SourceName).Append("; \\\r\n");
									}
								}
							}
						}
						builder.Append("\t\treturn ").Append(dependencyCount).Append("; \\\r\n");
						builder.Append("\t} \\\r\n");

						builder.Append("\tvoid Apply(FStateStreamCopyContext& Context, const ").Append(scriptStruct.SourceName).Append("& Other) \\\r\n");
						builder.Append("\t{ \\\r\n");
						foreach (UhtProperty property in scriptStruct.Properties)
						{
							string propName = property.SourceName;
							bool isBoolWithB = (property is UhtBoolProperty && propName.StartsWith('b'));
							string cleanPropName = isBoolWithB ? propName[1..] : propName;

							builder.Append("\t\tif (Other.").Append(cleanPropName).Append("Modified()");
							builder.Append(" && !StateStreamEquals(").Append(propName).Append(", Other.").Append(propName);
							builder.Append(")) \\\r\n");
							builder.Append("\t\t{ \\\r\n");

							bool assignHandled = false;
							if (property is UhtStructProperty structProperty)
							{
								if (structProperty.ScriptStruct.MetaData.ContainsKey(UhtNames.StateStreamHandle))
								{
									builder.Append("\t\t\t").Append(propName).Append(".Apply(Context, Other.").Append(propName).Append("); \\\r\n");
									assignHandled = true;
								}
							}
							if (!assignHandled)
							{
								builder.Append("\t\t\t").Append(propName).Append(" = Other.").Append(propName).Append("; \\\r\n");
							}
							builder.Append("\t\t\tModifiedFlags |= (").Append(modifiedFlagsType).Append(")Fields::Field_").Append(propName).Append("; \\\r\n");
							builder.Append("\t\t} \\\r\n");
						}
						builder.Append("\t} \\\r\n");

						builder.Append("\tvoid Interpolate(FStateStreamInterpolateContext& Context, const ").Append(scriptStruct.SourceName).Append("& From, const ").Append(scriptStruct.SourceName).Append("& To) \\\r\n");
						builder.Append("\t{ \\\r\n");
						builder.Append("\t\tModifiedFlags = 0; \\\r\n");
						foreach (UhtProperty property in scriptStruct.Properties)
						{
							string propName = property.SourceName;
							bool isBool = property is UhtBoolProperty;
							bool isBoolWithB = isBool && propName.StartsWith('b');
							string cleanPropName = isBoolWithB ? propName[1..] : propName;

							builder.Append("\t\tif (To.").Append(cleanPropName).Append("Modified()) \\\r\n");
							builder.Append("\t\t{ \\\r\n");
							builder.Append("\t\t\tModifiedFlags |= (").Append(modifiedFlagsType).Append(")Fields::Field_").Append(propName).Append("; \\\r\n");

							bool assignHandled = false;
							if (property is UhtStructProperty structProperty)
							{
								if (structProperty.ScriptStruct.MetaData.ContainsKey(UhtNames.StateStreamHandle))
								{
									builder.Append("\t\t\t").Append(propName).Append(".Apply(Context, To.").Append(propName).Append("); \\\r\n");
									assignHandled = true;
								}
							}
							if (!assignHandled)
							{
								if (isBool)
								{
									builder.Append("\t\t\t").Append(propName).Append(" = To.").Append(propName).Append(";\\\r\n");
								}
								else
								{
									builder.Append("\t\t\tStateStreamInterpolate(Context, ").Append(propName).Append(", From.").Append(propName).Append(", To.").Append(propName).Append("); \\\r\n");
								}
							}
							builder.Append("\t\t} \\\r\n");
						}
						builder.Append("\t} \\\r\n");
					}
					else if (scriptStruct.MetaData.ContainsKey(UhtNames.StateStreamHandle))
					{
						ReadOnlySpan<char> shortName = scriptStruct.SourceName.AsSpan(1, scriptStruct.SourceName.Length - 7);
						builder.Append("\tDECLARE_STATESTREAM_HANDLE(").Append(shortName).Append(") \\\r\n");
					}

					Session.Tables!.CodeGeneratorInjectorTable.Inject(builder, scriptStruct, UhtCodeGeneratorInjectionLocation.GeneratedMacro);
				}

				// For back compat we still need to forward declare the struct here
				AppendForwardDeclarationWithNamespace(builder, scriptStruct);

				// Verse declarations
				if (scriptStruct.IsVerseField)
				{
					if (!scriptStruct.FieldExportFlags.HasAnyFlags(UhtFieldExportFlags.NoVerseAlias))
					{
						builder.Append("namespace verse { using ").Append(scriptStruct.VerseName).Append(" = ").Append(scriptStruct.SourceName).Append("; }\r\n");
					}
					builder
						.Append("V_DECLARE_STRUCT_RUNTIME_TRAITS(, ")
						.Append(scriptStruct.SourceName).Append(", \"")
						.AppendVerseUEPackageName(scriptStruct).Append("\", \"")
						.AppendVerseScopeAndName(scriptStruct, UhtVerseNameMode.Default).Append("\")\r\n");
				}

				Session.Tables!.CodeGeneratorInjectorTable.Inject(builder, scriptStruct, UhtCodeGeneratorInjectionLocation.Header);
			}
			return builder;
		}

		private StringBuilder Appendpartial(StringBuilder builder, UhtPartial partial)
		{
			using (UhtMacroCreator macro = new(builder, this, partial, GeneratedBodyMacroSuffix))
			{
				builder.Append('\t').Append(Module.Api).Append("static int32 PartialOffset; \\\r\n");
				builder.Append('\t').Append(partial.OwnerClassName).Append("& GetOwner() {");
				builder.Append(" return *reinterpret_cast<").Append(partial.OwnerClassName).Append("*>((uint8*)this - PartialOffset); } \\\r\n");
				builder.Append("\tconst ").Append(partial.OwnerClassName).Append("& GetOwner() const {");
				builder.Append(" return *reinterpret_cast<const ").Append(partial.OwnerClassName).Append("*>((const uint8*)this - PartialOffset); } \\\r\n");
				if (partial.Functions.Any())
				{
					builder.Append("\t \\\r\n");
					foreach (UhtFunction function in partial.Functions)
					{
						if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
						{
							builder.Append("\tDECLARE_FUNCTION(").Append(function.UnMarshalAndCallName).Append("); \\\r\n");
						}
					}
				}
			}

			return builder;
		}

		private StringBuilder AppendEnum(StringBuilder builder, UhtEnum enumObj)
		{
			// Export FOREACH macro
			builder.Append("#define FOREACH_ENUM_").Append(enumObj.EngineName.ToUpperInvariant()).Append("(op) ");
			bool hasExistingMax = EnumHasExistingMax(enumObj);
			long maxEnumValue = hasExistingMax ? GetMaxEnumValue(enumObj) : 0;
			foreach (UhtEnumValue enumValue in enumObj.EnumValues)
			{
				if (!hasExistingMax || enumValue.Value != maxEnumValue)
				{
					builder.Append("\\\r\n\top(").Append(enumValue.Name).Append(") ");
				}
			}
			builder.Append("\r\n");

			if (enumObj.CppForm == UhtEnumCppForm.EnumClass)
			{
				builder.Append("\r\n");
				enumObj.Namespace.AppendSingleLine(builder, builder =>
				{
					builder.Append("enum class ").Append(enumObj.CppType);
					if (enumObj.UnderlyingType != UhtEnumUnderlyingType.Unspecified)
					{
#pragma warning disable CA1308 // Normalize strings to uppercase
						builder.Append(" : ").Append(enumObj.UnderlyingType.ToString().ToLowerInvariant());
#pragma warning restore CA1308 // Normalize strings to uppercase
					}
					builder.Append(';');
				});

				// Add TIsUEnumClass typetraits
				builder.Append("template<> struct TIsUEnumClass<").Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType).Append("> { enum { Value = true }; };\r\n");
			}
			else if (enumObj.CppForm == UhtEnumCppForm.Regular && enumObj.UnderlyingType != UhtEnumUnderlyingType.Unspecified)
			{
				builder.Append("\r\n");
				enumObj.Namespace.AppendSingleLine(builder, builder =>
				{
#pragma warning disable CA1308 // Normalize strings to uppercase
					builder.Append("enum ").Append(enumObj.CppType).Append(" : ").Append(enumObj.UnderlyingType.ToString().ToLowerInvariant()).Append(';');
#pragma warning restore CA1308 // Normalize strings to uppercase
				});
			}
			else if (enumObj.CppForm == UhtEnumCppForm.Namespaced && enumObj.UnderlyingType != UhtEnumUnderlyingType.Unspecified)
			{
				string[] splitName = enumObj.CppType.Split("::");
				builder.Append("\r\n");
				builder.Append("namespace ").Append(enumObj.Namespace.FullSourceName).Append(splitName[0]).Append(" { enum ").Append(splitName[1]);
#pragma warning disable CA1308 // Normalize strings to uppercase
				builder.Append(" : ").Append(enumObj.UnderlyingType.ToString().ToLowerInvariant());
#pragma warning restore CA1308 // Normalize strings to uppercase
				builder.Append("; }\r\n");
			}

			if (enumObj.CppForm == UhtEnumCppForm.EnumClass || enumObj.UnderlyingType != UhtEnumUnderlyingType.Unspecified)
			{
				// Forward declare the StaticEnum<> specialization for enum classes
				if (Module.Module.AlwaysExportEnums || enumObj.EnumExportFlags.HasAnyFlags(UhtEnumExportFlags.MinimalAPI))
				{
					builder.Append("template<> UE_NODEBUG ").Append(Module.NonAttributedApi).Append("UEnum* StaticEnum<").Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType).Append(">();\r\n");
				}
				else
				{
					builder.Append("template<> UE_NODEBUG ").Append("UEnum* StaticEnum<").Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType).Append(">();\r\n");
				}
			}

			// Verse declarations
			if (enumObj.IsVerseField)
			{
				builder.Append("V_DECLARE_IMPORTED_ENUM(").Append(Module.Api.TrimEnd()).Append(", ").Append(enumObj.SourceName).Append(");\r\n");
			}
			return builder;
		}

		private StringBuilder AppendClass(StringBuilder builder, UhtClass classObj)
		{
			string api = classObj.ClassFlags.HasAnyFlags(EClassFlags.MinimalAPI) ? Module.Api : "NO_API ";
			bool usesLegacy = classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.UsesGeneratedBodyLegacy);
			UhtClass? nativeInterface = ObjectInfos[classObj.ObjectTypeIndex].NativeInterface;
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface) && nativeInterface == null)
			{
				throw new UhtIceException("Interfaces must have an associated native interface");
			}

			if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
				// Predeclare class object outside class body macro
				builder.Append(GetExternalDecl(classObj.ObjectTypeIndex, UhtSingletonType.ConstInit));
			}

			// With interfaces, there are cases where we use the native interface class export flags to check for legacy macro
			bool alternateUsesLegacy = (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface) ? nativeInterface! : classObj).ClassExportFlags.HasAnyFlags(UhtClassExportFlags.UsesGeneratedBodyLegacy);

			// Collect sparse data information
			IEnumerable<UhtScriptStruct> sparseScriptStructs = GetSparseDataStructsToExport(classObj);
			UhtUsedDefineScopes<UhtProperty> sparseProperties = new(EnumerateSparseDataStructProperties(sparseScriptStructs));

			// Write the spare declarations
			AppendSparseDeclarations(builder, classObj, sparseScriptStructs, sparseProperties);

			// Collect the rpc functions in reversed order
			UhtUsedDefineScopes<UhtFunction> rpcFunctions = new(classObj.Functions.Where(x => IsRpcFunction(x)));
			rpcFunctions.Instances.Reverse();

			// Output the RPC methods
			AppendRpcFunctions(builder, classObj, alternateUsesLegacy, rpcFunctions);

			// Collect the verse native callable functions
			UhtUsedDefineScopes<UhtFunction> verseNativeCallableFunctions = new(classObj.Functions.Where(x => x.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable)));

			// Output the verse native callback functions
			AppendVerseNativeCallableFunctions(builder, classObj, verseNativeCallableFunctions);

			// Output property accessors
			IEnumerable<UhtProperty> getterSetterProperties = classObj.Properties.Where(x => x.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterFound | UhtPropertyExportFlags.SetterFound));
			AppendPropertyAccessors(builder, classObj, getterSetterProperties);

			// Collect the callback function and sort by name to make the order stable
			List<UhtFunction> callbackFunctions = [.. classObj.Functions.Where(x => x.FunctionFlags.HasAnyFlags(EFunctionFlags.Event) && x.SuperFunction == null)];
			callbackFunctions.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.EngineName, y.EngineName));
			bool hasCallbacks = callbackFunctions.Count > 0;

			// Collect the auto getter setter properties
			UhtUsedDefineScopes<UhtProperty> autoGetterSetterProperties = GetAutoGetterSetterProperties(classObj);

			// Generate the RPC wrappers for the callbacks
			AppendCallbackRpcWrapperDecls(builder, classObj, callbackFunctions);

			// Only write out adapters if the user has provided one or the other of the Serialize overloads
			if (classObj.SerializerArchiveType != UhtSerializerArchiveType.None && classObj.SerializerArchiveType != UhtSerializerArchiveType.All)
			{
				AppendSerializer(builder, classObj, api, UhtSerializerArchiveType.Archive, "DECLARE_FARCHIVE_SERIALIZER");
				AppendSerializer(builder, classObj, api, UhtSerializerArchiveType.StructuredArchiveRecord, "DECLARE_FSTRUCTUREDARCHIVE_SERIALIZER");
			}

			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
			{
				builder.Append($"struct {GetSingletonName(classObj, UhtSingletonType.Statics)};\r\n");
			}
			// For functions with custom thunks, make the functions _Statics struct will need to be a friend of the class
			// We need to forward declare it outside of the class body to properly handle classes in namespaces
			foreach (UhtType child in classObj.Children)
			{
				if (child is UhtFunction function && function.FunctionExportFlags.HasAllFlags(UhtFunctionExportFlags.CustomThunk))
				{
					builder.Append($"\tstruct {GetSingletonName(function, UhtSingletonType.Statics)}; \\\r\n");
				}
			}
			// Declaration of global function fetching UClass of type
			builder.Append(classObj.Module.Api).Append("UClass* ").Append(ObjectInfos[classObj.ObjectTypeIndex].RegisteredSingletonName).Append("(ETypeConstructPhase);\r\n\r\n");

			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
			{
				if (usesLegacy)
				{
					AppendStandardConstructors(builder, classObj, api);
				}
				else
				{
					AppendEnhancedConstructors(builder, classObj, api);
				}

				using (UhtMacroCreator macro = new(builder, this, classObj, GeneratedUInterfaceBodyMacroSuffix))
				{
					AppendCommonGeneratedBody(builder, classObj);
				}

				if (usesLegacy)
				{
					using UhtMacroCreator macro = new(builder, this, classObj, GeneratedBodyLegacyMacroSuffix);
					builder.Append('\t');
					AppendGeneratedMacroDeprecationWarning(builder, "GENERATED_UINTERFACE_BODY");
					builder.Append('\t').Append(DisableDeprecationWarnings).Append(" \\\r\n");
					builder.Append('\t').AppendMacroName(this, classObj, GeneratedUInterfaceBodyMacroSuffix).Append(" \\\r\n");
					builder.Append('\t').AppendMacroName(this, classObj, StandardConstructorsMacroSuffix).Append(" \\\r\n");
					builder.Append('\t').Append(EnableDeprecationWarnings).Append(" \\\r\n");
				}
				else
				{
					using UhtMacroCreator macro = new(builder, this, classObj, GeneratedBodyMacroSuffix);
					builder.Append('\t').Append(DisableDeprecationWarnings).Append(" \\\r\n");
					builder.Append('\t').AppendMacroName(this, classObj, GeneratedUInterfaceBodyMacroSuffix).Append(" \\\r\n");
					builder.Append('\t').AppendMacroName(this, classObj, EnchancedConstructorsMacroSuffix).Append(" \\\r\n");
					AppendAccessSpecifier(builder, classObj);
					builder.Append(" \\\r\n");
					builder.Append('\t').Append(EnableDeprecationWarnings).Append(" \\\r\n");
				}

				if (alternateUsesLegacy)
				{
					using UhtMacroCreator macro = new(builder, this, classObj, InClassIInterfaceMacroSuffix);
					AppendInClassIInterface(builder, classObj, callbackFunctions, api);
				}
				else
				{
					using UhtMacroCreator macro = new(builder, this, classObj, InClassIInterfaceNoPureDeclsMacroSuffix);
					AppendInClassIInterface(builder, classObj, callbackFunctions, api);
				}

				AppendProlog(builder, classObj);
				AppendGeneratedBodyMacroBlock(builder, classObj, nativeInterface!, alternateUsesLegacy, rpcFunctions, verseNativeCallableFunctions,
					autoGetterSetterProperties, sparseScriptStructs.Any(), sparseProperties, getterSetterProperties, hasCallbacks, null);

				// For interfaces, the VINTERFACES macro generates nothing
				if (classObj.VerseInterfacesLine != 0)
				{
					using UhtMacroCreator macro = new(builder, this, classObj.VerseInterfacesLine, VInterfacesMacroSuffix);
				}
			}
			else
			{
				if (usesLegacy)
				{
					using (UhtMacroCreator macro = new(builder, this, classObj, InClassMacroSuffix))
					{
						AppendClassGeneratedBody(builder, classObj, api);
					}
					AppendStandardConstructors(builder, classObj, api);
				}
				else
				{
					using (UhtMacroCreator macro = new(builder, this, classObj, InClassNoPureDeclsMacroSuffix))
					{
						AppendClassGeneratedBody(builder, classObj, api);
					}
					AppendEnhancedConstructors(builder, classObj, api);
				}

				AppendFieldNotify(builder, classObj);
				AppendAutoGettersSetters(builder, classObj, api, autoGetterSetterProperties);
				AppendProlog(builder, classObj);
				AppendGeneratedBodyMacroBlock(builder, classObj, classObj, usesLegacy, rpcFunctions, verseNativeCallableFunctions,
					autoGetterSetterProperties, sparseScriptStructs.Any(), sparseProperties, getterSetterProperties, hasCallbacks,
					usesLegacy ? "GENERATED_UCLASS_BODY" : null);
			}

			// For back compat we still need to forward declare the class here
			AppendForwardDeclarationWithNamespace(builder, classObj);

			// Verse declarations
			if (classObj.IsVerseField)
			{
				if (!classObj.FieldExportFlags.HasAnyFlags(UhtFieldExportFlags.NoVerseAlias))
				{
					if (classObj.ClassType == UhtClassType.Class)
					{
						builder.Append("namespace verse { using ").Append(classObj.VerseName).Append(" = ").AppendClassSourceNameOrInterfaceProxyName(classObj).Append("; }\r\n");
					}
				}
			}

			Session.Tables!.CodeGeneratorInjectorTable.Inject(builder, classObj, UhtCodeGeneratorInjectionLocation.Header);
			builder.Append("\r\n");
			return builder;
		}

		private static StringBuilder AppendNativeInterfaceVerseProxy(StringBuilder builder, UhtClass classObj)
		{
			builder.Append("class ").AppendClassSourceNameOrInterfaceProxyName(classObj).Append("\r\n");
			builder.Append("{\r\n");
			builder.Append("public:\r\n");
			builder.Append("\tusing StaticClassType = ").Append(classObj.SourceName).Append(";\r\n");
			foreach (UhtClass baseClass in classObj.FlattenedVerseInterfaces)
			{
				AppendNativeInterfaceVerseProxyFunctions(builder, baseClass.AlternateObject as UhtClass);
			}
			AppendNativeInterfaceVerseProxyFunctions(builder, classObj);
			builder.Append('\t').AppendClassSourceNameOrInterfaceProxyName(classObj).Append("(const TNonNullPtr<UObject>& InSelf) : Self(InSelf) {}\r\n");
			builder.Append("\t[[deprecated(\"Do not call _getUObject().\")]] UObject* _getUObject() const { return Self; }\r\n");
			builder.Append('\t').AppendClassSourceNameOrInterfaceProxyName(classObj).Append("* operator->() { return this; }\r\n");
			builder.Append("private:\r\n");
			builder.Append("\tTNonNullPtr<UObject> Self;\r\n");
			builder.Append("};\r\n");
			foreach (UhtClass baseClass in classObj.FlattenedVerseInterfaces)
			{
				builder
					.Append("template<> struct TIsSubinterfaceOf<")
					.AppendClassSourceNameOrInterfaceProxyName(baseClass)
					.Append(", ")
					.AppendClassSourceNameOrInterfaceProxyName(classObj)
					.Append("> { enum { Value = true }; };\r\n");
			}

			if (!classObj.FieldExportFlags.HasAnyFlags(UhtFieldExportFlags.NoVerseAlias))
			{
				builder.Append("namespace verse { using ").Append(classObj.VerseName).Append(" = ").AppendClassSourceNameOrInterfaceProxyName(classObj).Append("; }\r\n");
			}
			return builder;
		}

		private static StringBuilder AppendNativeInterfaceVerseProxyFunctions(StringBuilder builder, UhtClass? classObj)
		{
			if (classObj != null)
			{
				UhtUsedDefineScopes<UhtFunction> verseNativeCallableFunctions = new(classObj.Functions.Where(x => x.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable)));
				builder.Append("\t// Verse native callable thunks for ").Append(classObj.SourceName).Append("\r\n");
				AppendVerseNativeCallableProxyFunctions(builder, classObj, verseNativeCallableFunctions);
				builder.Append("\r\n");
			}
			return builder;
		}

		private StringBuilder AppendProlog(StringBuilder builder, UhtClass classObj/*, List<UhtFunction> classbackFunctions*/)
		{
			using (UhtMacroCreator macro = new(builder, this, classObj.PrologLineNumber, PrologMacroSuffix))
			{
				//if (callbackFunctions.Count > 0)
				//{
				//	builder.Append('\t').AppendMacroName(this, classObj, EventParamsMacroSuffix).Append(" \\\r\n");
				//}
			}
			return builder;
		}

		private StringBuilder AppendFieldNotify(StringBuilder builder, UhtClass classObj)
		{
			if (!NeedFieldNotifyCodeGen(classObj))
			{
				return builder;
			}

			UhtUsedDefineScopes<UhtType> notifyTypes = GetFieldNotifyTypes(classObj);
			return builder.AppendSingleMacro(notifyTypes, this, classObj, FieldNotifyMacroSuffix,
				(builder, filteredTypes) =>
				{
					builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BEGIN(").Append(Module.Api).Append(") \\\r\n");

					// UE_FIELD_NOTIFICATION_DECLARE_FIELD
					foreach (UhtType notifyType in filteredTypes)
					{
						builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_FIELD(").Append(GetNotifyTypeName(notifyType)).Append(',').Append(Module.Api).Append(") \\\r\n");
					}

					// UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD
					bool isFirst = true;
					foreach (UhtType notifyType in filteredTypes)
					{
						if (isFirst)
						{
							isFirst = false;
							builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_BEGIN(").Append(GetNotifyTypeName(notifyType)).Append(") \\\r\n");
						}
						else
						{
							builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD(").Append(GetNotifyTypeName(notifyType)).Append(") \\\r\n");
						}
					}
					builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_END() \\\r\n");
					builder.Append("\tUE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_END(); \\\r\n");
				});
		}

		private StringBuilder AppendAutoGettersSetters(StringBuilder builder, UhtClass classObj, string api, UhtUsedDefineScopes<UhtProperty> autoGetterSetterProperties)
		{
			if (autoGetterSetterProperties.IsEmpty)
			{
				return builder;
			}

			return builder.AppendMultiScopedMacros(autoGetterSetterProperties, this, classObj, AutoGettersSettersMacroSuffix,
				(builder, properties) =>
				{
					foreach (UhtProperty property in properties)
					{
						if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecifiedAuto))
						{
							string getterCallText = property.Getter ?? "Get" + property.SourceName;
							builder.Append('\t').Append(api).AppendPropertyText(property, UhtPropertyTextType.GetterRetVal).Append(getterCallText).Append("() const; \\\r\n");
						}
						if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterSpecifiedAuto))
						{
							string setterCallText = property.Setter ?? "Set" + property.SourceName;
							builder.Append('\t').Append(api).Append("void ").Append(setterCallText).Append('(').AppendPropertyText(property, UhtPropertyTextType.SetterParameterArgType).Append("InValue").Append("); \\\r\n");
						}
					}
				});
		}

		private static IEnumerable<UhtProperty> EnumerateSparseDataStructProperties(IEnumerable<UhtScriptStruct> sparseScriptStructs)
		{
			foreach (UhtScriptStruct sparseScriptStruct in sparseScriptStructs)
			{
				foreach (UhtProperty property in sparseScriptStruct.Properties)
				{
					if (!property.MetaData.ContainsKey(UhtNames.NoGetter))
					{
						yield return property;
					}
				}
			}
		}

		private StringBuilder AppendSparseDeclarations(StringBuilder builder, UhtClass classObj, IEnumerable<UhtScriptStruct> sparseScriptStructs, UhtUsedDefineScopes<UhtProperty> sparseProperties)
		{
			if (!sparseScriptStructs.Any())
			{
				return builder;
			}

			AppendSparseStructDeclarations(builder, classObj, sparseScriptStructs);

			builder.AppendMultiScopedMacros(sparseProperties, this, classObj, SparseDataPropertyAccessorsMacroSuffix,
				(builder, properties) =>
				{
					foreach (UhtProperty property in properties)
					{
						string propertyName = property.SourceName;
						ReadOnlySpan<char> cleanPropertyName = propertyName.AsSpan();
						if (property is UhtBoolProperty && propertyName.StartsWith('b'))
						{
							cleanPropertyName = cleanPropertyName[1..];
						}

						if (property.MetaData.ContainsKey(UhtNames.GetByRef))
						{
							builder.Append("const ").AppendSparse(property).Append("& Get").Append(cleanPropertyName).Append("() const");
						}
						else
						{
							builder.AppendSparse(property).Append(" Get").Append(cleanPropertyName).Append("() const");
						}
						builder.Append(" { return Get").Append(property.Outer?.EngineName).Append("(EGetSparseClassDataMethod::ArchetypeIfNull)->").Append(propertyName).Append("; } \\\r\n");
					}
				});
			return builder;
		}

		private StringBuilder AppendSparseStructDeclarations(StringBuilder builder, UhtClass classObj, IEnumerable<UhtScriptStruct> sparseScriptStructs)
		{
			using (UhtMacroCreator macro = new(builder, this, classObj, SparseDataMacroSuffix))
			{
				string api = classObj.ClassFlags.HasAnyFlags(EClassFlags.MinimalAPI) ? Module.Api : "";

				foreach (UhtScriptStruct sparseScriptStruct in sparseScriptStructs)
				{
					string sparseDataType = sparseScriptStruct.EngineName;

					builder.Append("UE_DEPRECATED(5.6, \"Use (preferably) Get").Append(sparseDataType).Append("(EGetSparseClassDataMethod::ArchetypeIfNull), or GetMutable").Append(sparseDataType).Append("() instead if you need to mutate the sparse class data (best avoided at runtime)\")\\\r\n");
					builder.Append(api).Append('F').Append(sparseDataType).Append("* Get").Append(sparseDataType).Append("() const; \\\r\n");

					builder.Append(api).Append('F').Append(sparseDataType).Append("* GetMutable").Append(sparseDataType).Append("() const; \\\r\n");
					builder.Append(api).Append("const F").Append(sparseDataType).Append("* Get").Append(sparseDataType).Append("(EGetSparseClassDataMethod GetMethod) const; \\\r\n");
					builder.Append(api).Append("static UScriptStruct* StaticGet").Append(sparseDataType).Append("ScriptStruct(); \\\r\n");
				}
			}

			return builder;
		}

		private StringBuilder AppendRpcFunctions(StringBuilder builder, UhtClass classObj, bool usesLegacy, UhtUsedDefineScopes<UhtFunction> functions)
		{
			builder.AppendMultiScopedMacros(functions, this, classObj, usesLegacy ? RpcWrappersMacroSuffix : RpcWrappersNoPureDeclsMacroSuffix,
				(builder, functions) =>
				{
					if (usesLegacy)
					{
						AppendAutogeneratedBlueprintFunctionDeclarations(builder, classObj, functions);
						AppendRpcWrappers(builder, functions);
					}
					else
					{
						if (classObj.GeneratedCodeVersion <= EGeneratedCodeVersion.V1)
						{
							AppendAutogeneratedBlueprintFunctionDeclarationsOnlyNotDeclared(builder, classObj, functions);
						}
						AppendRpcWrappers(builder, functions);
					}
				});
			return builder;
		}

		private StringBuilder AppendVerseNativeCallableFunctions(StringBuilder builder, UhtClass classObj, UhtUsedDefineScopes<UhtFunction> functions)
		{
			builder.AppendMultiScopedMacros(functions, this, classObj, VerseNativeCallableWrappersMacroSuffix,
				(builder, functions) =>
				{
					foreach (UhtFunction function in functions)
					{
						builder.Append('\t').AppendVerseNativeCallableSignature(null, function, "").Append("; \\\r\n");
					}
				});
			return builder;
		}

		private static StringBuilder AppendVerseNativeCallableProxyFunctions(StringBuilder builder, UhtClass classObj, UhtUsedDefineScopes<UhtFunction> functions)
		{
			if (classObj.SuperClass != null && classObj.SuperClass.IsVerseField && classObj.SuperClass.ClassType == UhtClassType.Interface)
			{
				AppendVerseNativeCallableProxyFunctions(builder, classObj.SuperClass, new(classObj.SuperClass.Functions.Where(x => x.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable))));
			}

			builder.AppendMultiScoped(functions, 
				(builder, functions) =>
				{
					foreach (UhtFunction function in functions)
					{
						builder.Append('\t').AppendVerseNativeCallableSignature(null, function, "").Append(";\r\n");
					}
				});
			return builder;
		}

		private StringBuilder AppendPropertyAccessors(StringBuilder builder, UhtClass classObj, IEnumerable<UhtProperty> getterSetterProperties)
		{
			getterSetterProperties = [.. getterSetterProperties];
			if (getterSetterProperties.Any())
			{
				using UhtMacroCreator macro = new(builder, this, classObj, AccessorsMacroSuffix);
				foreach (UhtProperty property in getterSetterProperties)
				{
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterFound))
					{
						builder.Append("static void ").AppendPropertyGetterWrapperName(property).Append("(const void* Object, void* OutValue); \\\r\n");
					}
					if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterFound))
					{
						builder.Append("static void ").AppendPropertySetterWrapperName(property).Append("(void* Object, const void* InValue); \\\r\n");
					}
				}
			}
			return builder;
		}

		private StringBuilder AppendAutogeneratedBlueprintFunctionDeclarations(StringBuilder builder, UhtClass classObj, IEnumerable<UhtFunction> functions)
		{
			foreach (UhtFunction function in functions)
			{
				if (function.CppImplName == function.SourceName)
				{
					continue;
				}
				if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ValidationImplFound))
				{
					AppendNetValidateDeclaration(builder, classObj, function);
				}
				if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ImplFound))
				{
					AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.ClassFunctionArgOrRetVal, true, null, null, UhtFunctionExportFlags.None, 1, "; \\\r\n");
				}
			}
			return builder;
		}

		private StringBuilder AppendAutogeneratedBlueprintFunctionDeclarationsOnlyNotDeclared(StringBuilder builder, UhtClass classObj, IEnumerable<UhtFunction> functions)
		{
			foreach (UhtFunction function in functions)
			{
				if (function.CppImplName == function.SourceName)
				{
					continue;
				}
				if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ValidationImplFound))
				{
					AppendNetValidateDeclaration(builder, classObj, function);
				}
				if (!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ImplFound))
				{
					AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.ClassFunctionArgOrRetVal, true, null, null, UhtFunctionExportFlags.None, 1, "; \\\r\n");
				}
			}
			return builder;
		}

		private StringBuilder AppendNetValidateDeclaration(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate))
			{
				builder.Append('\t');

				if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.RequiredAPI) && function.FunctionFlags.HasAnyFlags(EFunctionFlags.RequiredAPI))
				{
					builder.Append(Module.Api);
				}

				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.Static) &&
					!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Final) &&
					!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.Final))
				{
					builder.Append("virtual");
				}
				builder.Append(" bool ").Append(function.CppValidationImplName);
				AppendParameters(builder, function, UhtPropertyTextType.ClassFunctionArgOrRetVal, null, true);
				builder.Append("; \\\r\n");
			}
			return builder;
		}

		private static StringBuilder AppendRpcWrappers(StringBuilder builder, IEnumerable<UhtFunction> functions)
		{
			foreach (UhtFunction function in functions)
			{
				if (ShouldExportFunction(function))
				{
					builder.Append("\tDECLARE_FUNCTION(").Append(function.UnMarshalAndCallName).Append("); \\\r\n");
				}
				if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable))
				{
					if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNoRollback))
					{
						builder.Append("\tV_DECLARE_EXEC_NOROLLBACK(").Append(function.VerseName).Append("); \\\r\n");
					}
					else if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends))
					{
						builder.Append("\tV_DECLARE_EXEC_SUSPENDS(").Append(function.VerseName).Append("); \\\r\n");
					}
					else
					{
						builder.Append("\tV_DECLARE_EXEC(").Append(function.VerseName).Append("); \\\r\n");
					}
					builder.Append("\tV_DECLARE_IMPL_NO_PRED(").Append(function.VerseName).Append("); \\\r\n");
				}
			}
			return builder;
		}

		private StringBuilder AppendCallbackRpcWrapperDecls(StringBuilder builder, UhtClass classObj, List<UhtFunction> callbackFunctions)
		{
			if (callbackFunctions.Count > 0)
			{
				using UhtMacroCreator macro = new(builder, this, classObj, CallbackWrappersMacroSuffix);
				foreach (UhtFunction function in callbackFunctions)
				{
					if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetResponse) &&
						function.EngineName != function.MarshalAndCallName)
					{
						AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.EventFunctionArgOrRetVal, true, null, null, UhtFunctionExportFlags.None, 1, " \\\r\n");
						builder.Append(" \\\r\n");
					}
				}
			}
			return builder;
		}

		private StringBuilder AppendSerializer(StringBuilder builder, UhtClass classObj, string api, UhtSerializerArchiveType type, string declare)
		{
			if (!classObj.SerializerArchiveType.HasAnyFlags(type))
			{
				builder.AppendScopedMacro(classObj.SerializerDefineScope, this, classObj, ArchiveSerializerMacroSuffix, false,
					builder => builder.Append('\t').Append(declare).Append('(').Append(classObj.SourceName).Append(", ").Append(api[0..^1]).Append(") \\\r\n"));
			}
			return builder;
		}

		/// <summary>
		/// Generates standard constructor declarations
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private StringBuilder AppendStandardConstructors(StringBuilder builder, UhtClass classObj, string api)
		{
			using (UhtMacroCreator macro = new(builder, this, classObj, StandardConstructorsMacroSuffix))
			{
				if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasCustomConstructor))
				{
					builder.Append("\t/** Standard constructor, called after all reflected properties have been initialized */ \\\r\n");
					builder.Append('\t').Append(api).Append(classObj.SourceName).Append("(const FObjectInitializer& ObjectInitializer");
					if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDefaultConstructor))
					{
						builder.Append(" = FObjectInitializer::Get()");
					}
					builder.Append("); \\\r\n");
				}
				if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
				{
					builder.Append("\tDEFINE_ABSTRACT_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
				}
				else
				{
					builder.Append("\tDEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
				}

				AppendVTableHelperCtorAndCaller(builder, classObj, api);
				AppendCopyConstructorDefinition(builder, classObj);
				AppendDestructorDefinition(builder, classObj, api);
			}
			return builder;
		}

		/// <summary>
		/// Generates enhanced constructor declaration.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private StringBuilder AppendEnhancedConstructors(StringBuilder builder, UhtClass classObj, string api)
		{
			using (UhtMacroCreator macro = new(builder, this, classObj, EnchancedConstructorsMacroSuffix))
			{
				AppendConstructorDefinition(builder, classObj, api);
				AppendVTableHelperCtorAndCaller(builder, classObj, api);
				AppendDefaultConstructorCallDefinition(builder, classObj);
				AppendDestructorDefinition(builder, classObj, api);
			}
			return builder;
		}

		/// <summary>
		/// Generates vtable helper caller and eventual constructor body.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendVTableHelperCtorAndCaller(StringBuilder builder, UhtClass classObj, string api)
		{
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasCustomVTableHelperConstructor))
			{
				builder.Append("\tDECLARE_VTABLE_PTR_HELPER_CTOR(").Append(api[0..^1]).Append(", ").Append(classObj.SourceName).Append("); \\\r\n");
			}
			builder.Append("\tDEFINE_VTABLE_PTR_HELPER_CTOR_CALLER(").Append(classObj.SourceName).Append("); \\\r\n");
			return builder;
		}

		/// <summary>
		/// Generates private copy-constructor declaration.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendCopyConstructorDefinition(StringBuilder builder, UhtClass classObj)
		{
			builder.Append("\t/** Deleted move- and copy-constructors, should never be used */ \\\r\n");
			builder.Append('\t').Append(classObj.SourceName).Append('(').Append(classObj.SourceName).Append("&&) = delete; \\\r\n");
			builder.Append('\t').Append(classObj.SourceName).Append("(const ").Append(classObj.SourceName).Append("&) = delete; \\\r\n");
			return builder;
		}

		/// <summary>
		/// Generates a destructor
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendDestructorDefinition(StringBuilder builder, UhtClass classObj, string api)
		{
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDestructor))
			{
				builder.Append('\t');

				if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
				{
					builder.Append(api);
				}

				if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.Final))
				{
					builder.Append("virtual ");
				}

				builder.Append('~').Append(classObj.SourceName);

				if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
				{
					builder.Append("() = default; \\\r\n");
				}
				else
				{
					builder.Append("(); \\\r\n");
				}
			}
			return builder;
		}

		/// <summary>
		/// Generates private copy-constructor declaration.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendConstructorDefinition(StringBuilder builder, UhtClass classObj, string api)
		{
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasConstructor))
			{
				builder.Append("\t/** Standard constructor, called after all reflected properties have been initialized */ \\\r\n");
				switch (GetConstructorType(classObj))
				{
					case ConstructorType.ObjectInitializer:
						builder.Append('\t').Append(api).Append(classObj.SourceName).Append("(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get()); \\\r\n");
						break;

					case ConstructorType.Default:
						builder.Append('\t').Append(api).Append(classObj.SourceName).Append("(); \\\r\n");
						break;
				}
			}
			AppendCopyConstructorDefinition(builder, classObj);
			return builder;
		}

		/// <summary>
		/// Generates constructor call definition
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendDefaultConstructorCallDefinition(StringBuilder builder, UhtClass classObj)
		{
			switch (GetConstructorType(classObj))
			{
				case ConstructorType.ObjectInitializer:
					if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
					{
						builder.Append("\tDEFINE_ABSTRACT_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
					}
					else
					{
						builder.Append("\tDEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
					}
					break;

				case ConstructorType.Default:
					if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
					{
						builder.Append("\tDEFINE_ABSTRACT_DEFAULT_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
					}
					else
					{
						builder.Append("\tDEFINE_DEFAULT_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
					}
					break;

				case ConstructorType.ForbiddenDefault:
					builder.Append("\tDEFINE_FORBIDDEN_DEFAULT_CONSTRUCTOR_CALL(").Append(classObj.SourceName).Append(") \\\r\n");
					break;

				default:
					throw new UhtIceException("Unexpected constructor type");
			}
			return builder;
		}

		/// <summary>
		/// Generates generated body code for classes
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <param name="api">API text to be used</param>
		/// <returns>Output builder</returns>
		private StringBuilder AppendClassGeneratedBody(StringBuilder builder, UhtClass classObj, string api)
		{
			AppendCommonGeneratedBody(builder, classObj);

			// export the class's config name
			UhtClass? superClass = classObj.SuperClass;
			if (classObj.Config.Length > 0 && (superClass == null || classObj.Config != superClass.Config))
			{
				builder.Append("\tstatic constexpr const TCHAR* StaticConfigName() {return TEXT(\"").Append(classObj.Config).Append("\");} \\\r\n \\\r\n");
			}

			// export implementation of _getUObject for classes that implement interfaces
			foreach (UhtStruct baseStruct in classObj.Bases)
			{
				if (baseStruct is UhtClass baseClass)
				{
					if (baseClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
					{
						builder.Append("\t[[deprecated(\"Do not call _getUObject(), use Cast.\")]] virtual UObject* _getUObject() const override { return const_cast<").Append(classObj.SourceName).Append("*>(this); } \\\r\n");
						break;
					}
				}
			}

			if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.SelfHasReplicatedProperties))
			{
				AppendReplicatedMacroData(builder, classObj, api);
			}

			return builder;
		}

		/// <summary>
		/// Generates standard generated body code for interfaces and non-interfaces
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class being exported</param>
		/// <returns>Output builder</returns>
		private StringBuilder AppendCommonGeneratedBody(StringBuilder builder, UhtClass classObj)
		{
			UhtCodeGenerator.ObjectInfo info = ObjectInfos[classObj.ObjectTypeIndex];

			builder.Append("private: \\\r\n");
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
			{
				builder.Append($"\tfriend struct ::{info.RegisteredSingletonName}_Statics; \\\r\n");
			}

			// For functions with custom thunks, make the functions _Statics struct a friend so it can take the address
			// of the thunk even if it's private
			foreach (UhtType child in classObj.Children)
			{
				if (child is UhtFunction function && function.FunctionExportFlags.HasAllFlags(UhtFunctionExportFlags.CustomThunk))
				{
					builder.Append($"\tfriend struct ::{GetSingletonName(function, UhtSingletonType.Statics)}; \\\r\n");
				}
			}

			builder.Append("\tfriend ").Append(classObj.Module.Api).Append("UClass* ::").Append(info.RegisteredSingletonName).Append("(ETypeConstructPhase); \\\r\n");

			builder.Append("public: \\\r\n");

			UhtClass? superClass = classObj.SuperClass;
			bool castedClass = classObj.ClassCastFlags.HasAnyFlags(EClassCastFlags.AllFlags) && superClass != null && classObj.ClassCastFlags != superClass.ClassCastFlags;

			builder
				.Append("\tDECLARE_CLASS2(")
				.Append(classObj)
				.Append(", ")
				.Append(superClass != null ? superClass.SourceName : classObj.SourceName) // UObject passes itself as super to the macro
				.Append(", COMPILED_IN_FLAGS(")
				.Append(classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract) ? "CLASS_Abstract" : "0");

			AppendClassFlags(builder, classObj);
			builder.Append("), ");
			if (castedClass)
			{
				builder
					.Append("CASTCLASS_")
					.Append(classObj.ClassCastFlags.ToString());
			}
			else
			{
				builder.Append("CASTCLASS_None");
			}
			builder.Append(", TEXT(\"").Append(classObj.Package.SourceName).Append("\"), ").Append(info.RegisteredSingletonName).Append(") \\\r\n");

			if (classObj != Session.UObject)
			{
				builder.Append("\tDECLARE_SERIALIZER(").Append(classObj.SourceName).Append(") \\\r\n");
			}

			// Add the serialization function declaration if we generated one
			if (classObj.SerializerArchiveType != UhtSerializerArchiveType.None && classObj.SerializerArchiveType != UhtSerializerArchiveType.All)
			{
				builder.Append('\t').AppendMacroName(this, classObj, ArchiveSerializerMacroSuffix).Append(" \\\r\n");
			}

			if (superClass != null && classObj.ClassWithin != superClass.ClassWithin)
			{
				if (classObj.ClassWithin == Session.UPackage)
				{
					builder.Append("\tDECLARE_WITHIN_UPACKAGE() \\\r\n");

				}
				else
				{
					builder.Append("\tDECLARE_WITHIN(").Append(classObj.ClassWithin.SourceName).Append(") \\\r\n");
				}
			}

			// Only generate GetPartial<>() if parent is not partial (i.e., this is the first partial class in the hierarchy)
			if (classObj.ClassFlags.HasFlag(EClassFlags.Partial) && (classObj.SuperClass == null || !classObj.SuperClass.ClassFlags.HasFlag(EClassFlags.Partial)))
			{
				builder.Append("\ttemplate<typename T> T& GetPartial() { return *reinterpret_cast<T*>((uint8*)this + T::PartialOffset); } \\\r\n");
				builder.Append("\ttemplate<typename T> const T& GetPartial() const { return *reinterpret_cast<const T*>((const uint8*)this + T::PartialOffset); } \\\r\n");
			}

			return builder;
		}

		/// <summary>
		/// Appends the class flags in the form of CLASS_Something|CLASS_Something which represents all class flags that are set 
		/// for the specified class which need to be exported as part of the DECLARE_CLASS macro
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="classObj">Class in question</param>
		/// <returns>Output builder</returns>
		private static StringBuilder AppendClassFlags(StringBuilder builder, UhtClass classObj)
		{
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Transient))
			{
				builder.Append(" | CLASS_Transient");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Optional))
			{
				builder.Append(" | CLASS_Optional");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Partial))
			{
				builder.Append(" | CLASS_Partial");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.DefaultConfig))
			{
				builder.Append(" | CLASS_DefaultConfig");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.GlobalUserConfig))
			{
				builder.Append(" | CLASS_GlobalUserConfig");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.ProjectUserConfig))
			{
				builder.Append(" | CLASS_ProjectUserConfig");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.PerPlatformConfig))
			{
				builder.Append(" | CLASS_PerPlatformConfig");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Config))
			{
				builder.Append(" | CLASS_Config");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
			{
				builder.Append(" | CLASS_Interface");
			}
			if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Deprecated))
			{
				builder.Append(" | CLASS_Deprecated");
			}
			return builder;
		}

		/// <summary>
		/// Appends preprocessor string to emit GENERATED_U*_BODY() macro is deprecated.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="macroName">Name of the macro to deprecate</param>
		/// <returns>Output builder</returns>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Referencing code commented out")]
		private static StringBuilder AppendGeneratedMacroDeprecationWarning(StringBuilder builder, string macroName)
		{
			// Deprecation warning is disabled right now. After people get familiar with the new macro it should be re-enabled.
			//Builder.Append("EMIT_DEPRECATED_WARNING_MESSAGE(\"").Append(MacroName).Append("() macro is deprecated. Please use GENERATED_BODY() macro instead.\") \\\r\n");
			return builder;
		}

		private static StringBuilder AppendAccessSpecifier(StringBuilder builder, UhtClass classObj)
		{
			switch (classObj.GeneratedBodyAccessSpecifier)
			{
				case UhtAccessSpecifier.Public:
					builder.Append("public:");
					break;
				case UhtAccessSpecifier.Private:
					builder.Append("private:");
					break;
				case UhtAccessSpecifier.Protected:
					builder.Append("protected:");
					break;
				default:
					builder.Append("static_assert(false, \"Unknown access specifier for GENERATED_BODY() macro in class ").Append(classObj.EngineName).Append(".\");");
					break;
			}
			return builder;
		}

		private StringBuilder AppendInClassIInterface(StringBuilder builder, UhtClass classObj, List<UhtFunction> callbackFunctions, string api)
		{
			string interfaceSourceName = classObj.GetNativeInterfaceName();

			builder.Append("protected: \\\r\n");
			builder.Append("\tvirtual ~").Append(interfaceSourceName).Append("() {} \\\r\n");
			builder.Append("public: \\\r\n");
			builder.Append("\ttypedef ").Append(classObj.SourceName).Append(" UClassType; \\\r\n");
			builder.Append("\ttypedef ").Append(interfaceSourceName).Append(" ThisClass; \\\r\n");

			AppendInterfaceCallFunctions(builder, callbackFunctions);

			// we'll need a way to get to the UObject portion of a native interface, so that we can safely pass native interfaces
			// to script VM functions
			if (classObj.IsVerseField || (classObj.SuperClass != null && classObj.SuperClass == Session.UInterface))
			{
				// Note: This used to be declared as a pure virtual function, but it was changed here in order to allow the Blueprint nativization process
				// to detect C++ interface classes that explicitly declare pure virtual functions via type traits. This code will no longer trigger that check.
				builder.Append("\t[[deprecated(\"Do not call _getUObject(), use Cast.\")]] virtual UObject* _getUObject() const { return nullptr; } \\\r\n");
			}

			if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.SelfHasReplicatedProperties))
			{
				AppendReplicatedMacroData(builder, classObj, api);
			}

			return builder;
		}

		private static StringBuilder AppendReplicatedMacroData(StringBuilder builder, UhtClass classObj, string api)
		{
			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasGetLifetimeReplicatedProps))
			{
				// Default version autogenerates declarations.
				if (classObj.GeneratedCodeVersion == EGeneratedCodeVersion.V1)
				{
					builder.Append('\t').Append(api).Append("void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override; \\\r\n");
				}
			}

			AppendNetData(builder, classObj, api);

			// If this class has replicated properties and it owns the first one, that means
			// it's the base most replicated class. In that case, go ahead and add our interface macro.
			if (classObj.ClassExportFlags.HasExactFlags(UhtClassExportFlags.HasReplciatedProperties, UhtClassExportFlags.SelfHasReplicatedProperties))
			{
				// Make sure the client hasn't implemented it themselves
				bool alreadyDefinedPushModel = false;
				UhtClass? checkForReplicatedBaseClass = classObj;
				while (!alreadyDefinedPushModel && checkForReplicatedBaseClass != null)
				{
					alreadyDefinedPushModel = checkForReplicatedBaseClass.TryGetDeclaration("REPLICATED_BASE_CLASS", out _);
					checkForReplicatedBaseClass = checkForReplicatedBaseClass.SuperClass;
				}

				if (!alreadyDefinedPushModel)
				{
					builder.Append("private: \\\r\n");
					builder.Append("\tREPLICATED_BASE_CLASS(").Append(classObj.SourceName).Append(") \\\r\n");
					builder.Append("public: \\\r\n");
				}
			}
			return builder;
		}

		private static StringBuilder AppendNetData(StringBuilder builder, UhtClass classObj, string api)
		{
			bool hasArray = false;
			foreach (UhtProperty property in classObj.EnumerateReplicatedProperties(false))
			{
				if (property.IsStaticArray)
				{
					if (!hasArray)
					{
						hasArray = true;
						builder.Append("\tenum class EArrayDims_Private : uint16 \\\r\n");
						builder.Append("\t{ \\\r\n");
					}
					builder.Append("\t\t").Append(property.SourceName).Append('=').Append(property.ArrayDimensions).Append(", \\\r\n");
				}
			}

			if (hasArray)
			{
				builder.Append("\t}; \\\r\n");
			}

			builder.Append("\tenum class ENetFields_Private : uint16 \\\r\n");
			builder.Append("\t{ \\\r\n");
			builder.Append("\t\tNETFIELD_REP_START=(uint16)((int32)Super::ENetFields_Private::NETFIELD_REP_END + (int32)1), \\\r\n");

			bool isFirst = true;
			UhtProperty? lastProperty = null;
			foreach (UhtProperty property in classObj.EnumerateReplicatedProperties(false))
			{
				lastProperty = property;
				if (!property.IsStaticArray)
				{
					if (isFirst)
					{
						builder.Append("\t\t").Append(property.SourceName).Append("=NETFIELD_REP_START, \\\r\n");
						isFirst = false;
					}
					else
					{
						builder.Append("\t\t").Append(property.SourceName).Append(", \\\r\n");
					}
				}
				else
				{
					if (isFirst)
					{
						builder.Append("\t\t").Append(property.SourceName).Append("_STATIC_ARRAY=NETFIELD_REP_START, \\\r\n");
						isFirst = false;
					}
					else
					{
						builder.Append("\t\t").Append(property.SourceName).Append("_STATIC_ARRAY, \\\r\n");
					}

					builder
						.Append("\t\t")
						.Append(property.SourceName)
						.Append("_STATIC_ARRAY_END=((uint16)")
						.Append(property.SourceName)
						.Append("_STATIC_ARRAY + (uint16)EArrayDims_Private::")
						.Append(property.SourceName)
						.Append(" - (uint16)1), \\\r\n");
				}
			}

			if (lastProperty != null)
			{
				builder.Append("\t\tNETFIELD_REP_END=").Append(lastProperty.SourceName);
				if (lastProperty.IsStaticArray)
				{
					builder.Append("_STATIC_ARRAY_END");
				}
			}
			builder.Append("\t}; \\\r\n");

			builder.Append('\t').Append("DECLARE_VALIDATE_GENERATED_REP_ENUMS(").Append(api[0..^1]).Append(") \\\r\n");

			return builder;
		}

		private StringBuilder AppendInterfaceCallFunctions(StringBuilder builder, List<UhtFunction> callbackFunctions)
		{
			const string ExtraArg = "UObject* O";
			const string ConstExtraArg = "const UObject* O";

			foreach (UhtFunction function in callbackFunctions)
			{
				AppendNativeFunctionHeader(builder, function, UhtPropertyTextType.InterfaceFunctionArgOrRetVal, true, null,
					function.FunctionFlags.HasAnyFlags(EFunctionFlags.Const) ? ConstExtraArg : ExtraArg, UhtFunctionExportFlags.None,
					1, "; \\\r\n");
			}
			return builder;
		}

		private StringBuilder AppendGeneratedBodyMacroBlock(StringBuilder builder, UhtClass classObj, UhtClass bodyClassObj, bool isLegacy,
			UhtUsedDefineScopes<UhtFunction> rpcFunctions, UhtUsedDefineScopes<UhtFunction> verseNativeCallableFunctions,
			UhtUsedDefineScopes<UhtProperty> autoGetterSetterProperties, bool hasSparseStructs,
			UhtUsedDefineScopes<UhtProperty> sparseProperties, IEnumerable<UhtProperty> getterSetterProperties, bool hasCallbacks, string? deprecatedMacroName)
		{
			bool isInterface = classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface);
			using (UhtMacroCreator macro = new(builder, this, bodyClassObj, isLegacy ? GeneratedBodyLegacyMacroSuffix : GeneratedBodyMacroSuffix))
			{
				if (deprecatedMacroName != null)
				{
					AppendGeneratedMacroDeprecationWarning(builder, deprecatedMacroName);
				}
				builder.Append(DisableDeprecationWarnings).Append(" \\\r\n");
				builder.Append("public: \\\r\n");
				if (hasSparseStructs)
				{
					builder.Append('\t').AppendMacroName(this, classObj, SparseDataMacroSuffix).Append(" \\\r\n");
					builder.AppendMultiMacroRefs(sparseProperties, this, classObj, SparseDataPropertyAccessorsMacroSuffix);
				}
				builder.AppendMultiMacroRefs(rpcFunctions, this, classObj, isLegacy ? RpcWrappersMacroSuffix : RpcWrappersNoPureDeclsMacroSuffix);
				builder.AppendMultiMacroRefs(verseNativeCallableFunctions, this, classObj, VerseNativeCallableWrappersMacroSuffix);
				if (getterSetterProperties.Any())
				{
					builder.Append('\t').AppendMacroName(this, classObj, AccessorsMacroSuffix).Append(" \\\r\n");
				}
				builder.AppendMultiMacroRefs(autoGetterSetterProperties, this, classObj, AutoGettersSettersMacroSuffix);
				if (hasCallbacks)
				{
					builder.Append('\t').AppendMacroName(this, classObj, CallbackWrappersMacroSuffix).Append(" \\\r\n");
				}
				if (isInterface)
				{
					if (isLegacy)
					{
						builder.Append('\t').AppendMacroName(this, classObj, InClassIInterfaceMacroSuffix).Append(" \\\r\n");
					}
					else
					{
						builder.Append('\t').AppendMacroName(this, classObj, InClassIInterfaceNoPureDeclsMacroSuffix).Append(" \\\r\n");
					}
				}
				else
				{
					if (isLegacy)
					{
						builder.Append('\t').AppendMacroName(this, classObj, InClassMacroSuffix).Append(" \\\r\n");
					}
					else
					{
						builder.Append('\t').AppendMacroName(this, classObj, InClassNoPureDeclsMacroSuffix).Append(" \\\r\n");
					}
				}
				if (!isInterface)
				{
					if (isLegacy)
					{
						builder.Append('\t').AppendMacroName(this, classObj, StandardConstructorsMacroSuffix).Append(" \\\r\n");
					}
					else
					{
						builder.Append('\t').AppendMacroName(this, classObj, EnchancedConstructorsMacroSuffix).Append(" \\\r\n");
					}
					if (NeedFieldNotifyCodeGen(classObj))
					{
						builder.Append('\t').AppendMacroName(this, classObj, FieldNotifyMacroSuffix).Append(" \\\r\n");
					}
				}
				if (isLegacy)
				{
					builder.Append("public: \\\r\n");
				}
				else
				{
					AppendAccessSpecifier(builder, classObj);
					builder.Append(" \\\r\n");
				}
				builder.Append(EnableDeprecationWarnings).Append(" \\\r\n");
				Session.Tables!.CodeGeneratorInjectorTable.Inject(builder, classObj, UhtCodeGeneratorInjectionLocation.GeneratedMacro);
			}
			return builder;
		}

		private static StringBuilder AppendForwardDeclarationWithNamespace(StringBuilder builder, UhtField field)
		{
			return field.Namespace.AppendSingleLine(builder, (builder) => builder.AppendForwardDeclaration(field));
		}

		#region Enum helper methods
		private static bool IsFullEnumName(string inEnumName)
		{
			return inEnumName.Contains("::", StringComparison.Ordinal);
		}

		private static StringView GenerateEnumPrefix(UhtEnum enumObj)
		{
			StringView prefix = new();
			if (enumObj.EnumValues.Count > 0)
			{
				prefix = enumObj.EnumValues[0].Name;

				// For each item in the enumeration, trim the prefix as much as necessary to keep it a prefix.
				// This ensures that once all items have been processed, a common prefix will have been constructed.
				// This will be the longest common prefix since as little as possible is trimmed at each step.
				for (int nameIdx = 1; nameIdx < enumObj.EnumValues.Count; ++nameIdx)
				{
					StringView enumItemName = enumObj.EnumValues[nameIdx].Name;

					// Find the length of the longest common prefix of Prefix and EnumItemName.
					int prefixIdx = 0;
					while (prefixIdx < prefix.Length && prefixIdx < enumItemName.Length && prefix.Span[prefixIdx] == enumItemName.Span[prefixIdx])
					{
						prefixIdx++;
					}

					// Trim the prefix to the length of the common prefix.
					prefix = new StringView(prefix, 0, prefixIdx);
				}

				// Find the index of the rightmost underscore in the prefix.
				int underscoreIdx = prefix.Span.LastIndexOf('_');

				// If an underscore was found, trim the prefix so only the part before the rightmost underscore is included.
				if (underscoreIdx > 0)
				{
					prefix = new StringView(prefix, 0, underscoreIdx);
				}
				else
				{
					// no underscores in the common prefix - this probably indicates that the names
					// for this enum are not using Epic's notation, so just empty the prefix so that
					// the max item will use the full name of the enum
					prefix = new StringView();
				}
			}

			// If no common prefix was found, or if the enum does not contain any entries,
			// use the name of the enumeration instead.
			if (prefix.Length == 0)
			{
				prefix = enumObj.EngineName;
			}
			return prefix;
		}

		private static string GenerateFullEnumName(UhtEnum enumObj, string inEnumName)
		{
			if (enumObj.CppForm == UhtEnumCppForm.Regular || IsFullEnumName(inEnumName))
			{
				return inEnumName;
			}
			return $"{enumObj.EngineName}::{inEnumName}";
		}

		private static bool EnumHasExistingMax(UhtEnum enumObj)
		{
			if (enumObj.GetIndexByName(GenerateFullEnumName(enumObj, "MAX")) != -1)
			{
				return true;
			}

			string maxEnumItem = GenerateFullEnumName(enumObj, GenerateEnumPrefix(enumObj).ToString() + "_MAX");
			if (enumObj.GetIndexByName(maxEnumItem) != -1)
			{
				return true;
			}
			return false;
		}

		private static long GetMaxEnumValue(UhtEnum enumObj)
		{
			if (enumObj.EnumValues.Count == 0)
			{
				return 0;
			}

			long maxValue = enumObj.EnumValues[0].Value;
			for (int i = 1; i < enumObj.EnumValues.Count; ++i)
			{
				long currentValue = enumObj.EnumValues[i].Value;
				if (currentValue > maxValue)
				{
					maxValue = currentValue;
				}
			}

			return maxValue;
		}
		#endregion
	}

	static class UhtClassExtensions
	{
		public static IEnumerable<UhtProperty> EnumerateReplicatedProperties(this UhtClass classObj, bool includeSuper)
		{
			if (includeSuper && classObj.SuperClass != null)
			{
				foreach (UhtProperty property in classObj.SuperClass.EnumerateReplicatedProperties(true))
				{
					yield return property;
				}
			}

			foreach (UhtProperty property in classObj.Properties)
			{
				if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
				{
					yield return property;
				}
			}
		}
	}
}
