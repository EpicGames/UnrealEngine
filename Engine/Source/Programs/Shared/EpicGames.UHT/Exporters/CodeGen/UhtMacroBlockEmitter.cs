// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal struct UhtMacroBlockEmitter : IDisposable
	{
		private readonly StringBuilder _builder;
		private UhtDefineScope _current = UhtDefineScope.None;
		private bool _open = false;

		public UhtMacroBlockEmitter(StringBuilder builder, UhtDefineScope initialState)
		{
			_builder = builder;
			Set(initialState);
		}

		/// <summary>
		/// Change the define scope to the new provided one using #endif and #if 
		/// </summary>
		/// <param name="defineScope"></param>
		public void Set(UhtDefineScope defineScope)
		{
			if (defineScope == UhtDefineScope.Invalid)
			{
				defineScope = UhtDefineScope.None;
			}
			if (_current == defineScope)
			{
				return;
			}
			_builder.AppendEndIfPreprocessor(_current);
			if (defineScope != UhtDefineScope.None)
			{
				_builder.AppendIfPreprocessor(defineScope);
				_open = true;
			}
			else
			{
				_open = false;
			}
			_current = defineScope;
		}

		/// <summary>
		/// Change the define scope to the new provided.
		/// None -> Scope X uses #if
		/// Scope X -> None uses #else
		/// Scope X -> Scope Y uses #elif
		/// </summary>
		/// <param name="defineScope"></param>
		public void SetElse(UhtDefineScope defineScope)
		{
			if (defineScope == UhtDefineScope.Invalid)
			{
				defineScope = UhtDefineScope.None;
			}
			if (_current == defineScope)
			{
				return;
			}
			if (_current == UhtDefineScope.None)
			{
				_builder.AppendIfPreprocessor(defineScope);
			}
			else if (defineScope == UhtDefineScope.None)
			{
				_builder.AppendElsePreprocessor(_current);
			}
			else
			{
				_builder.AppendElseIfPreprocessor(defineScope);
			}

			_current = defineScope;
			_open = true;
		}

		public void Dispose()
		{
			if (_open)
			{
				_builder.AppendLine("#endif");
				_open = false;
			}
		}
	}

	/// <summary>
	/// Utility class to wrap blocks of generated code in #if/#endif for a literal string conditionally.
	/// i.e. used to conditionally wrap code in #if UE_WITH_CONSTINIT_UOBJECT depending on the value of 
	/// Session.IsUsingMultipleCompiledInObjectFormats
	/// </summary>
	internal struct UhtConditionalMacroBlock : IDisposable
	{
		private readonly StringBuilder _builder;
		private readonly string _macroName;
		private bool _enabled;

		public UhtConditionalMacroBlock(StringBuilder builder, string macroName, bool isEnabled)
		{
			_builder = builder;
			_macroName = macroName;
			_enabled = isEnabled;
			if (isEnabled)
			{
				_builder.Append($"#if {_macroName}\r\n");
			}
		}

		public void Dispose()
		{
			if (_enabled)
			{
				_builder.Append($"#endif // {_macroName}\r\n");
				_enabled = false;
			}
		}
	}
}
