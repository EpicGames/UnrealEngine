// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal readonly struct UhtMacroCreator : IDisposable
	{
		private readonly StringBuilder _builder;
		private readonly int _startingLength;

		public UhtMacroCreator(StringBuilder builder, UhtHeaderCodeGenerator generator, int lineNumber, string macroSuffix)
		{
			builder.Append("#define ").AppendMacroName(generator.FileId, lineNumber, macroSuffix).Append(" \\\r\n");
			_builder = builder;
			_startingLength = builder.Length;
		}

		public UhtMacroCreator(StringBuilder builder, UhtHeaderCodeGenerator generator, UhtType type, string macroSuffix)
		{
			builder.Append("#define ").AppendMacroName(generator.FileId, type.GetMacroLineNumber(), macroSuffix).Append(" \\\r\n");
			_builder = builder;
			_startingLength = builder.Length;
		}

		public UhtMacroCreator(StringBuilder builder, UhtHeaderCodeGenerator generator, UhtType type, string macroSuffix, UhtDefineScope defineScope = UhtDefineScope.None, bool includeSuffix = true)
		{
			builder.Append("#define ").AppendMacroName(generator.FileId, type.GetMacroLineNumber(), macroSuffix, defineScope, includeSuffix).Append(" \\\r\n");
			_builder = builder;
			_startingLength = builder.Length;
		}

		public void Dispose()
		{
			int finalLength = _builder.Length;
			if (finalLength < 4 ||
				_builder[finalLength - 4] != ' ' ||
				_builder[finalLength - 3] != '\\' ||
				_builder[finalLength - 2] != '\r' ||
				_builder[finalLength - 1] != '\n')
			{
				throw new UhtException("Macro line must end in ' \\\\\\r\\n'");
			}

			_builder.Length -= 4;
			if (finalLength == _startingLength)
			{
				_builder.Append("\r\n");
			}
			else
			{
				_builder.Append("\r\n\r\n\r\n");
			}
		}
	}
}
