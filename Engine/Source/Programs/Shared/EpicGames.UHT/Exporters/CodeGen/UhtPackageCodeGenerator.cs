// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	class UhtPackageCodeGenerator : IUhtObjectLinker
	{
		public static string HeaderCopyright =
			"// Copyright Epic Games, Inc. All Rights Reserved.\r\n" +
			"/*===========================================================================\r\n" +
			"\tGenerated code exported from UnrealHeaderTool.\r\n" +
			"\tDO NOT modify this manually! Edit the corresponding .h files instead!\r\n" +
			"===========================================================================*/\r\n" +
			"\r\n";

		public static string RequiredCPPIncludes = "#include \"UObject/GeneratedCppIncludes.h\"\r\n";

		public static string EnableDeprecationWarnings = "PRAGMA_ENABLE_DEPRECATION_WARNINGS";
		public static string DisableDeprecationWarnings = "PRAGMA_DISABLE_DEPRECATION_WARNINGS";

		public readonly UhtCodeGenerator CodeGenerator;
		public readonly UhtModule Module;
		public bool SaveExportedHeaders => Module.Module.SaveExportedHeaders;

		public Utils.UhtSession Session => CodeGenerator.Session;
		public UhtCodeGenerator.PackageInfo[] PackageInfos => CodeGenerator.PackageInfos;
		public UhtCodeGenerator.HeaderInfo[] HeaderInfos => CodeGenerator.HeaderInfos;
		public UhtCodeGenerator.ObjectInfo[] ObjectInfos => CodeGenerator.ObjectInfos;

		public UhtPackageCodeGenerator(UhtCodeGenerator codeGenerator, UhtModule module)
		{
			CodeGenerator = codeGenerator;
			Module = module;
		}

		#region Utility functions

		/// <summary>
		/// Return the singleton name for an object
		/// </summary>
		/// <param name="obj">The object in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>Singleton name of "nullptr" if Object is null</returns>
		public string GetSingletonName(UhtObject? obj, UhtSingletonType type)
		{
			return CodeGenerator.GetSingletonName(obj, type);
		}

		/// <inheritdoc/>
		public bool IsCrossModuleRef(UhtObject obj) => obj.Module != Module;

		/// <inheritdoc/>
		public int GetModuleIndex(UhtModule otherModule)
		{
			if (Module.Linking == null)
			{
				throw new UhtIceException("Cannot link modules before link indices have been assigned by UhtSession");
			}
			if (Module.Linking.ReferencedModules.TryGetValue(otherModule, out int value))
			{
				return value;
			}
			else
			{
				throw new UhtIceException($"Module {otherModule.ShortName} was not discovered by reference collection from module {Module.ShortName}, so objects in it cannot be linked");
			}
		}

		/// <inheritdoc/>
		public int GetObjectIndex(UhtObject obj)
		{
			if (obj.Module.Linking == null)
			{
				throw new UhtIceException("Cannot link modules before link indices have been assigned by UhtSession");
			}
			if (obj.Module.Linking.LinkableObjects.TryGetValue(obj, out int value))
			{
				return value;
			}
			else
			{
				throw new UhtIceException($"Object {obj.FullName} is not linkable from module {obj.Module.ShortName}");
			}
		}

		/// <summary>
		/// Return the external declaration for an object
		/// </summary>
		/// <param name="obj">The object in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>External declaration</returns>
		public string GetExternalDecl(UhtObject obj, UhtSingletonType type)
		{
			return CodeGenerator.GetExternalDecl(obj, type);
		}

		/// <summary>
		/// Return the external declaration for an object
		/// </summary>
		/// <param name="objectIndex">The object in question.</param>
		/// <param name="type">Type of declaration to get</param>
		/// <returns>External declaration</returns>
		public string GetExternalDecl(int objectIndex, UhtSingletonType type)
		{
			return CodeGenerator.GetExternalDecl(objectIndex, type);
		}

		/// <summary>
		/// Test to see if the given field is a delegate function
		/// </summary>
		/// <param name="field">Field to be tested</param>
		/// <returns>True if the field is a delegate function</returns>
		public static bool IsDelegateFunction(UhtField field)
		{
			if (field is UhtFunction function)
			{
				return function.FunctionType.IsDelegate();
			}
			return false;
		}
		#endregion
	}

	/// <summary>
	/// Helper formatting methods
	/// </summary>
	public static class UhtPackageCodeGeneratorExtensions
	{
		/// <summary>
		/// Append an expression to reference a singleton. i.e. the name of a factory function, or an address-of expression for a constinit variable ref. 
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="provider">Interface through which to access singleton names</param>
		/// <param name="obj">Singleton to insert a reference to, or null</param>
		/// <param name="singletonType">Type of singleton</param>
		/// <returns></returns>
		internal static StringBuilder AppendSingletonRef(this StringBuilder builder, IUhtSingletonNameProvider provider, UhtObject? obj, UhtSingletonType singletonType)
		{
			if (obj == null)
			{
				return builder.Append("nullptr");
			}
			return builder.Append(singletonType == UhtSingletonType.ConstInit ? '&' : null)
				.Append(provider.GetSingletonName(obj, singletonType));
		}

		/// <summary>
		/// Append nullptr or an address-of expression for the constinit singleton for the given object
		/// </summary>
		/// <param name="builder">Stringbuillder to append to</param>
		/// <param name="provider">Interface to access singleton names</param>
		/// <param name="obj">Object to append the singleton for, e.g. a package or class</param>
		/// <returns></returns>
		internal static StringBuilder AppendConstInitSingletonRef(this StringBuilder builder, IUhtSingletonNameProvider provider, UhtObject? obj)
		{
			return builder.AppendSingletonRef(provider, obj, UhtSingletonType.ConstInit);
		}

		internal static StringBuilder AppendLinkingMacros(this StringBuilder builder, UhtSession session, IEnumerable<UhtModule> modules)
		{
			if (modules.Any() && session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", session.IsUsingMultipleCompiledInObjectFormats);
				UhtModule[] modulesSorted = [.. modules.OrderBy(x => x.ShortName)];

				builder.AppendLine("// Monolithic linking macros");
				builder.AppendLine("#if IS_MONOLITHIC");
				foreach (UhtModule otherModule in modulesSorted)
				{
					builder.AppendLine($"#define UHT_LINK_{otherModule.ShortName}(MODULE_INDEX, OBJECT_INDEX, POINTER) POINTER");
				}

				builder.AppendLine("// Modular linking macros");
				builder.AppendLine("#else // IS_MONOLITHIC");
				foreach (UhtModule otherModule in modulesSorted)
				{
					builder.AppendLine($"#define UHT_LINK_{otherModule.ShortName}(MODULE_INDEX, OBJECT_INDEX, POINTER) UE::CodeGen::ConstInit::FCompiledInObjectReference{{ MODULE_INDEX, OBJECT_INDEX }}");
				}
				builder.AppendLine("#endif // IS_MONOLITHIC");
			}
			return builder;
		}

		internal static StringBuilder AppendLinkingMacrosUndef(this StringBuilder builder, UhtSession session, IEnumerable<UhtModule> modules)
		{
			if (modules.Any() && session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
			{
				using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", session.IsUsingMultipleCompiledInObjectFormats);
				UhtModule[] modulesSorted = [.. modules.OrderBy(x => x.ShortName)];
				foreach (UhtModule otherModule in modulesSorted)
				{
					builder.AppendLine($"#undef UHT_LINK_{otherModule.ShortName}");
				}
			}
			return builder;
		}
	}
}
