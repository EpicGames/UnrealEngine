// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	using static UhtExpressionFactory;

	internal sealed class UhtPackageCodeGeneratorCppFile : UhtPackageCodeGenerator
	{
		/// <summary>
		/// Construct an instance of this generator object
		/// </summary>
		/// <param name="codeGenerator">The base code generator</param>
		/// <param name="module">Module being generated</param>
		public UhtPackageCodeGeneratorCppFile(UhtCodeGenerator codeGenerator, UhtModule module)
			: base(codeGenerator, module)
		{
		}

		/// <summary>
		/// For a given UE header file, generated the generated cpp file
		/// </summary>
		/// <param name="factory">Requesting factory</param>
		/// <param name="packageSortedHeaders">Sorted list of headers by name of all headers in the package</param>
		public void Generate(IUhtExportFactory factory, List<UhtHeaderFile> packageSortedHeaders)
		{
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
				StringBuilder builder = borrower.StringBuilder;

				// Collect information from all of the headers
				List<UhtField> singletons = [];
				StringBuilder declarations = new();
				IoHash bodiesHash = IoHash.Zero;
				foreach (UhtHeaderFile headerFile in packageSortedHeaders)
				{
					ref UhtCodeGenerator.HeaderInfo headerInfo = ref HeaderInfos[headerFile.HeaderFileTypeIndex];
					IReadOnlyList<string> sorted = headerFile.References.Declaration.GetSortedReferences(
						(objectIndex, type) => type switch
						{
							UhtSingletonType.Registered or UhtSingletonType.Unregistered => GetExternalDecl(objectIndex, type),
							_ => null
						});
					foreach (string declaration in sorted)
					{
						declarations.Append(declaration);
					}

					singletons.AddRange(headerFile.References.Singletons);

					IoHash bodyHash = HeaderInfos[headerFile.HeaderFileTypeIndex].BodyHash;
					if (bodiesHash == IoHash.Zero)
					{
						// Don't combine in the first case because it keeps GUID backwards compatibility
						bodiesHash = bodyHash;
					}
					else
					{
						bodiesHash = IoHash.Combine(bodyHash, bodiesHash);
					}
				}

				// No need to generate output if we have no declarations
				if (declarations.Length == 0)
				{
					if (SaveExportedHeaders)
					{
						// We need to create the directory, otherwise UBT will think that this module has not been properly updated and won't write a Timestamp file
						System.IO.Directory.CreateDirectory(Module.Module.OutputDirectory);
					}
					return;
				}
				IoHash declarationsHash = IoHash.Compute(declarations.ToString());

				singletons.Sort((lhs, rhs) =>
				{
					bool lhsIsDel = IsDelegateFunction(lhs);
					bool rhsIsDel = IsDelegateFunction(rhs);
					if (lhsIsDel != rhsIsDel)
					{
						return !lhsIsDel ? -1 : 1;
					}
					return StringComparerUE.OrdinalIgnoreCase.Compare(
						ObjectInfos[lhs.ObjectTypeIndex].RegisteredSingletonName,
						ObjectInfos[rhs.ObjectTypeIndex].RegisteredSingletonName);
				});

				builder.Append(HeaderCopyright);
				builder.Append(RequiredCPPIncludes);
				if (Module.Module.HasVerse)
				{
					builder.Append("#include \"VerseInteropMacros.h\"\r\n");
				}
				builder.Append(DisableDeprecationWarnings).Append("\r\n");
				builder.Append("void EmptyLinkFunctionForGeneratedCode").Append(Module.ShortName).Append("_init() {}\r\n");

				if (!Session.IsUsingMultipleCompiledInObjectFormats)
				{
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						builder.AppendLine("static_assert(UE_WITH_CONSTINIT_UOBJECT, \"This generated code can only be compiled with UE_WITH_CONSTINIT_OBJECT\");");
					}
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						builder.AppendLine("static_assert(!UE_WITH_CONSTINIT_UOBJECT, \"This generated code can only be compiled with !UE_WITH_CONSTINIT_OBJECT\");");
					}
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					builder.Append(GetExternalDecl(Session.UPackage, UhtSingletonType.ConstInit));
				}

				if (Session.IncludeDebugOutput)
				{
					builder.Append("#if 0\r\n");
					foreach (UhtHeaderFile headerFile in packageSortedHeaders)
					{
						builder.Append('\t').Append(headerFile.FilePath).Append("\r\n");
					}
					builder.Append(declarations);
					builder.Append("#endif\r\n");
				}

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					foreach (UhtObject obj in singletons)
					{
						builder.Append(GetExternalDecl(obj, UhtSingletonType.Registered));
					}
				}

				List<UhtObject> linkableObjectsOrdered = [.. Module.Linking!.LinkableObjects
					.Select((int order, UhtObject obj) (p) => (p.Value, p.Key))
					.OrderBy(p => p.order)
					.Select(p => p.obj)];

				List<UhtPartial> partials = Module.Packages.SelectMany(p => p.Children.OfType<UhtPartial>().Where(o => o.AlternateObject is null)).ToList();

				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
					foreach (UhtObject obj in singletons)
					{
						builder.Append(GetExternalDecl(obj, UhtSingletonType.ConstInit));
					}
					foreach ((UhtObject obj, int index) in Module.Linking!.LinkableObjects)
					{
						builder.Append(GetExternalDecl(obj, UhtSingletonType.ConstInit));
					}

					foreach (UhtPartial partial in partials)
					{
						builder.Append(GetExternalDecl(partial, UhtSingletonType.ConstInit));
					}

					builder.AppendLine($$"""
						extern {{Module.Api}}UObject* const {{ModuleLinkableObjects(Module)}}[{{linkableObjectsOrdered.Count}}] = 
						{
						""");
					foreach (UhtObject obj in linkableObjectsOrdered)
					{
						using UhtMacroBlockEmitter macroBlockEmitter = new(builder, obj.DefineScope);
						builder.AppendLine($"\t{SingletonRef(this, obj, UhtSingletonType.ConstInit)}, ");
						if (obj.DefineScope != UhtDefineScope.None)
						{
							macroBlockEmitter.SetElse(UhtDefineScope.None);
							builder.AppendLine("\tnullptr, ");
						}
					}
					builder.AppendLine($$"""
						};	

						""");

					if (partials.Count != 0)
					{
						builder.AppendLine($$"""
							extern UE::CoreUObject::Private::FPartialClass* const Z_Partials_{{Module.ShortName}}[{{partials.Count}}] = 
							{
							""");
						foreach (UhtPartial partial in partials)
						{
							using UhtMacroBlockEmitter macroBlockEmitter = new(builder, partial.DefineScope);
							builder.AppendLine($"\t{SingletonRef(this, partial, UhtSingletonType.ConstInit)}, ");
							if (partial.DefineScope != UhtDefineScope.None)
							{
								macroBlockEmitter.SetElse(UhtDefineScope.None);
								builder.AppendLine("\tnullptr, ");
							}
						}
						builder.AppendLine($$"""
							};	

							""");
					}

					List<UhtModule> orderedLinkedModules = [.. Module.Linking!.ReferencedModules
						.Select((int Index, UhtModule M) (p) => (p.Value, p.Key))
						.OrderBy(p => p.Index)
						.Select(p => p.M)
						];
					if (orderedLinkedModules.Count > 0)
					{
						builder.AppendLine("// Linkable object arrays from other modules");
						builder.AppendLine("#if !IS_MONOLITHIC");
						foreach (UhtModule otherModule in orderedLinkedModules)
						{
							builder.AppendLine($"extern {otherModule.Api}UObject* const {ModuleLinkableObjects(otherModule)}[{otherModule.Linking!.LinkableObjects.Count}];");
						}

						builder.AppendLine($$"""
							const TConstArrayView<UObject* const> {{ModuleLinkedModules(Module)}}[{{orderedLinkedModules.Count}}] = 
							{
							""");
						foreach (UhtModule module in orderedLinkedModules)
						{
							builder.AppendLine($"\tMakeArrayView({ModuleLinkableObjects(module)}, {module.Linking!.LinkableObjects.Count}), ");
						}
						builder.AppendLine($$"""
							};	

							""");
						builder.AppendLine("#endif // !IS_MONOLITHIC");
					}
				}

				builder.AppendLinkingMacros(Session, Module.Linking!.ReferencedModules.Keys);

				foreach (UhtPackage package in Module.Packages)
				{
					string strippedName = PackageInfos[package.PackageTypeIndex].StrippedName;
					string singletonName = GetSingletonName(package, UhtSingletonType.Registered);
					UhtUsedDefineScopes<UhtField> packageSingletons = new([.. singletons.Where(x => x.Package == package)]);
					EPackageFlags flags = package.PackageFlags & (EPackageFlags.ClientOptional | EPackageFlags.ServerSideOnly | EPackageFlags.EditorOnly | EPackageFlags.Developer | EPackageFlags.UncookedOnly);

					// Entire loop body is #if/#else/#endif
					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
					{
						using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append($"constinit UPackage {GetSingletonName(package, UhtSingletonType.ConstInit)}( \r\n")
							.Append("\tUE::CodeGen::ConstInit::FPackageParams{\r\n")
							.Append("\t\t.Object = UE::CodeGen::ConstInit::FObjectParams{\r\n")
							.Append("\t\t\t.Flags = RF_Public,\r\n")
							.Append($"\t\t\t.Class = {ConstInitSingletonRef(this, Session.UPackage)},\r\n")
							.Append("\t\t\t.NameUTF8 = UTF8TEXT(").AppendUTF8LiteralString(package.SourceName).Append("),\r\n")
							.Append("\t\t\t.Outer = nullptr,\r\n")
							.Append("\t\t},\r\n")
							.Append($"\t\t.Flags = EPackageFlags(PKG_CompiledIn | 0x{(uint)flags:X8}),\r\n")
							.Append($"\t\tIF_WITH_METADATA(.MetaData = ").AppendMetaDataArrayView(package.MetaData, UhtNames.TypeId, 0, ",)\r\n")
							.Append($"\t\t.BodiesHash = 0x{(uint)bodiesHash:X8},\r\n")
							.Append($"\t\t.DeclarationsHash = 0x{(uint)declarationsHash:X8},\r\n")
							.Append("});\r\n")
							.Append("\r\n");
					}

					if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
					{
						using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);
						builder.Append("\tstatic FPackageRegistrationInfo Z_Registration_Info_UPackage_").Append(strippedName).Append(";\r\n")
							.Append($"\tFORCENOINLINE UPackage* {singletonName}(ETypeConstructPhase)\r\n")
							.Append("\t{\r\n")
							.Append($"\t\tif (!Z_Registration_Info_UPackage_{strippedName}.OuterSingleton)\r\n")
							.Append("\t\t{\r\n");

						builder.AppendInstances(packageSingletons, 
							builder => builder.Append("\t\tstatic FTypeConstructFunc* SingletonFuncArray[] = {\r\n"),
							(builder, fieldObj) => builder.Append($"\t\t\t(FTypeConstructFunc*)").Append(ObjectInfos[fieldObj.ObjectTypeIndex].RegisteredSingletonName).Append(",\r\n"),
							builder => builder.Append("\t\t};\r\n"));

						builder.Append("\t\tstatic const UECodeGen_Private::FPackageParams PackageParams = {\r\n");
						builder.Append("\t\t\t").AppendUTF8LiteralString(package.SourceName).Append(",\r\n");
						builder.AppendArrayPtrLine(packageSingletons, UhtNames.SingletonFuncArrayId, 3, ",\r\n");
						builder.AppendArrayCountLine(packageSingletons, UhtNames.SingletonFuncArrayId, 3, ",\r\n");
						builder.Append("\t\t\tPKG_CompiledIn | ").Append($"0x{(uint)flags:X8}").Append(",\r\n");
						builder.Append("\t\t\t").Append($"0x{(uint)bodiesHash:X8}").Append(",\r\n");
						builder.Append("\t\t\t").Append($"0x{(uint)declarationsHash:X8}").Append(",\r\n");
						builder.Append("\t\t\t").AppendMetaDataParams(package, UhtNames.TypeId).Append("\r\n");
						builder.Append("\t\t};\r\n");
						builder.Append("\t\tUECodeGen_Private::ConstructUPackage(Z_Registration_Info_UPackage_").Append(strippedName).Append(".OuterSingleton, PackageParams);\r\n");
						builder.Append("\t}\r\n");
						builder.Append("\treturn Z_Registration_Info_UPackage_").Append(strippedName).Append(".OuterSingleton;\r\n");
						builder.Append("}\r\n");

						// Do not change the Z_CompiledInDeferPackage_UPackage_ without changing LC_SymbolPatterns
						builder.Append("static FRegisterCompiledInInfo Z_CompiledInDeferPackage_UPackage_").Append(strippedName).Append('(').Append(singletonName)
							.Append(", TEXT(\"").Append(package.SourceName).Append("\"), Z_Registration_Info_UPackage_").Append(strippedName).Append(", CONSTRUCT_RELOAD_VERSION_INFO(FPackageReloadVersionInfo, ")
							.Append($"0x{(uint)bodiesHash:X8}, 0x{(uint)declarationsHash:X8}").Append("));\r\n");
					}
				}

				// In constinit format, emit one registration struct per module, not per-package
				if (Session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", Session.IsUsingMultipleCompiledInObjectFormats);

					builder.AppendLine($"static UPackage* const Z_Packages_{Module.ShortName}[] = {{");
					foreach (UhtPackage package in Module.Packages)
					{
						builder.AppendLine($"\t{SingletonRef(this, package, UhtSingletonType.ConstInit)}, ");
					}
					builder.AppendLine($"}};");

					// UhtSession orders types according to UhtObject.LinkOrder to guarantee this ordering
					(int classesStart, int classesEnd) = GetTypeRange<UhtClass>(linkableObjectsOrdered.AsReadOnly(), 0);
					(int structsStart, int structsEnd) = GetTypeRange<UhtScriptStruct>(linkableObjectsOrdered.AsReadOnly(), classesEnd);
					(int enumsStart, int enumsEnd) = GetTypeRange<UhtEnum>(linkableObjectsOrdered.AsReadOnly(), structsEnd);
					int othersStart = enumsEnd;
					builder.AppendLine($$"""
						static FRegisterCompiledInObjects Z_Register_Module{{Module.ShortName}}{ {
						#if !IS_MONOLITHIC
							.LinkedModules = {{ModuleLinkedModules(Module)}},
							.LinkableObjects = MakeConstArrayView({{ModuleLinkableObjects(Module)}}),
						#endif // !IS_MONOLITHIC
							.Packages = MakeConstArrayView(Z_Packages_{{Module.ShortName}}),
							.Classes = MakeConstArrayView({{ModuleLinkableObjects(Module)}}).Mid({{classesStart}}, {{classesEnd - classesStart}}),
							.Structs = MakeConstArrayView({{ModuleLinkableObjects(Module)}}).Mid({{structsStart}}, {{structsEnd - structsStart}}),
							.Enums = MakeConstArrayView({{ModuleLinkableObjects(Module)}}).Mid({{enumsStart}}, {{enumsEnd - enumsStart}}),
							.Others = MakeConstArrayView({{ModuleLinkableObjects(Module)}}).RightChop({{othersStart}}),
						""");
					if (partials.Count > 0)
					{
						builder.Append($"\t.Partials = MakeConstArrayView(Z_Partials_{Module.ShortName}),\r\n");
					}
					builder.AppendLine("} };");
				}

				// Verse registration
				if (Module.Module.HasVerse)
				{
					builder.Append($"namespace {VniModuleStatics(Module)}\r\n");
					builder.Append("{\r\n");
					if (Module.Module.VerseDependencies.Count > 0)
					{
						builder.Append("\tconst FVniPackageName Dependencies[] = {\r\n");
						foreach (string dependency in Module.Module.VerseDependencies)
						{
							builder.Append($"\t\t{GetVniPackageName(dependency)},\r\n");
						}
						builder.Append("\t};\r\n");
					}
					builder.Append($"\tconst FVniPackageName Name = {GetVniPackageName(Module.Module.VersePackageName)};\r\n");
					builder.Append("\tV_DEFINE_CPP_MODULE_REGISTRAR(\r\n");
					builder.Append("\t\tUHT,\r\n");
					builder.Append($"\t\t{Module.Module.Name},\r\n");
					builder.Append("\t\tName,\r\n");
					builder.Append($"\t\tTEXT(\"{Module.Module.VersePath}\"),\r\n");
					builder.Append($"\t\tEVerseScope::{Module.Module.VerseScope},\r\n");
					builder.Append($"\t\tTEXT(\"{Module.Module.VerseDirectoryPath}\"),\r\n");
					builder.Append($"\t\t").Append(Module.Module.VerseDependencies.Count > 0 ? "Dependencies" : "nullptr").Append(",\r\n");
					builder.Append($"\t\t{Module.Module.VerseDependencies.Count},\r\n");
					builder.Append($"\t\tnullptr);\r\n");
					builder.Append("}\r\n");
				}

				builder.AppendLinkingMacrosUndef(Session, Module.Linking!.ReferencedModules.Keys);

				builder.Append(EnableDeprecationWarnings).Append("\r\n");

				if (SaveExportedHeaders)
				{
					string cppFilePath = factory.MakePath(Module, ".init.gen.cpp");
					factory.CommitOutput(cppFilePath, builder);
				}
			}
		}

		private static string GetVniPackageName(ReadOnlySpan<char> packageName)
		{
			int slashIndex = packageName.IndexOf('/');
			if (slashIndex == -1)
			{
				throw new UhtIceException("Verse package name expects a string containing a \"/\"");
			}
			ReadOnlySpan<char> mountPointName = packageName[..slashIndex];
			ReadOnlySpan<char> cppModuleName = packageName[(slashIndex + 1)..];
			return $"{{ TEXT(\"{mountPointName}\"), TEXT(\"{cppModuleName}\") }}";
		}

		private static (int Start, int End) GetTypeRange<T>(ReadOnlyCollection<UhtObject> objects, int start)
			where T : UhtObject
		{
			if (objects.Count > start && objects[start] is T)
			{
				for (int i = start + 1; i < objects.Count; ++i)
				{
					if (objects[i] is not T)
					{
						return (start, i);
					}
				}
				return (start, objects.Count);
			}
			return (start, start);
		}

		private static SimpleExpression ModuleLinkableObjects(UhtModule module) => new("Z_LinkableObjects_", module.ShortName);
		private static SimpleExpression ModuleLinkedModules(UhtModule module) => module.Linking!.ReferencedModules.Count == 0
			? new SimpleExpression("", "{}")
			: new SimpleExpression("Z_LinkedModules_", module.ShortName);
	}
}
