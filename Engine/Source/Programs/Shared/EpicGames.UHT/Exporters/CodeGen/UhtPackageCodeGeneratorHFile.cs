// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal sealed class UhtPackageCodeGeneratorHFile : UhtPackageCodeGenerator
	{
		/// <summary>
		/// Construct an instance of this generator object
		/// </summary>
		/// <param name="codeGenerator">The base code generator</param>
		/// <param name="module">Module being generated</param>
		public UhtPackageCodeGeneratorHFile(UhtCodeGenerator codeGenerator, UhtModule module)
			: base(codeGenerator, module)
		{
		}

		/// <summary>
		/// For a given UE header file, generated the generated H file
		/// </summary>
		/// <param name="factory">Requesting factory</param>
		/// <param name="moduleSortedHeaders">Sorted list of headers by name of all headers in the module</param>
		public void Generate(IUhtExportFactory factory, List<UhtHeaderFile> moduleSortedHeaders)
		{
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
				StringBuilder builder = borrower.StringBuilder;

				builder.Append(HeaderCopyright);
				builder.Append("#pragma once\r\n");
				builder.Append("\r\n");
				builder.Append("\r\n");

				List<UhtHeaderFile> headerFiles = new(moduleSortedHeaders.Count + Module.Headers.Count);
				headerFiles.AddRange(moduleSortedHeaders);

				foreach (UhtHeaderFile headerFile in Module.Headers)
				{
					if (headerFile.HeaderFileType == UhtHeaderFileType.Classes)
					{
						headerFiles.Add(headerFile);
					}
				}

				List<UhtHeaderFile> sortedHeaderFiles = [.. headerFiles.Distinct()];
				sortedHeaderFiles.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.FilePath, y.FilePath));

				foreach (UhtHeaderFile headerFile in sortedHeaderFiles)
				{
					if (headerFile.HeaderFileType == UhtHeaderFileType.Classes)
					{
						builder.Append("#include \"").Append(HeaderInfos[headerFile.HeaderFileTypeIndex].IncludePath).Append("\"\r\n");
					}
				}

				builder.Append("\r\n");

				if (SaveExportedHeaders)
				{
					string headerFilePath = factory.MakePath(Module, "Classes.h");
					factory.CommitOutput(headerFilePath, builder);
				}
			}
		}
	}
}
