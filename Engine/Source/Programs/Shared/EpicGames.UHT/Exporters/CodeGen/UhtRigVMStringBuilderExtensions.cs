// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Exporters.CodeGen
{
	internal static class UhtRigVMStringBuilderExtensions
	{
		public static StringBuilder AppendArgumentsName(this StringBuilder builder, UhtStruct structObj, UhtRigVMMethodInfo methodInfo)
		{
			return builder.Append("Arguments_").Append(structObj.SourceName).Append('_').Append(methodInfo.Name);
		}

		public static StringBuilder AppendParameterNames(this StringBuilder builder, IEnumerable<UhtRigVMParameter> parameters,
			bool leadingSeparator = false, string separator = ", ", bool castName = false, bool includeEditorOnly = false)
		{
			bool emitSeparator = leadingSeparator;
			foreach (UhtRigVMParameter parameter in parameters)
			{
				if (includeEditorOnly && !parameter.EditorOnly)
				{
					continue;
				}
				if (emitSeparator)
				{
					builder.Append(separator);
				}
				emitSeparator = true;
				builder.Append(parameter.NameOriginal(castName));
			}
			return builder;
		}

		public static StringBuilder AppendTypeConstRef(this StringBuilder builder, UhtRigVMParameter parameter, bool castType = false)
		{
			builder.Append("const ");
			string paramterType = parameter.TypeOriginal(castType);
			ReadOnlySpan<char> parameterTypeSpan = paramterType.AsSpan();
			if (parameterTypeSpan.EndsWith("&", StringComparison.Ordinal))
			{
				parameterTypeSpan = parameterTypeSpan[0..^1];
			}
			builder.Append(parameterTypeSpan);

			if (parameterTypeSpan.Length > 0)
			{
				int namespaceEndIndex = parameterTypeSpan.LastIndexOf(':');
				int firstTypeCharIndex = namespaceEndIndex >= 0 ? (namespaceEndIndex + 1) : 0;
				if (parameterTypeSpan[firstTypeCharIndex] == 'T' || parameterTypeSpan[firstTypeCharIndex] == 'F')
				{
					builder.Append('&');
				}
			}
			return builder;
		}

		public static StringBuilder AppendTypeRef(this StringBuilder builder, UhtRigVMParameter parameter, bool castType = false)
		{
			string paramterType = parameter.TypeOriginal(castType);
			ReadOnlySpan<char> parameterTypeSpan = paramterType.AsSpan();
			if (parameterTypeSpan.EndsWith("&", StringComparison.Ordinal))
			{
				builder.Append(parameterTypeSpan);
			}
			else
			{
				builder.Append(parameterTypeSpan).Append('&');
			}
			return builder;
		}

		public static StringBuilder AppendTypeNoRef(this StringBuilder builder, UhtRigVMParameter parameter, bool castType = false)
		{
			string paramterType = parameter.TypeOriginal(castType);
			ReadOnlySpan<char> parameterTypeSpan = paramterType.AsSpan();
			if (parameterTypeSpan.EndsWith("&", StringComparison.Ordinal))
			{
				parameterTypeSpan = parameterTypeSpan[0..^1];
			}
			return builder.Append(parameterTypeSpan);
		}

		public static StringBuilder AppendTypeVariableRef(this StringBuilder builder, UhtRigVMParameter parameter, bool castType = false)
		{
			if (parameter.IsConst())
			{
				builder.AppendTypeConstRef(parameter, castType);
			}
			else
			{
				builder.AppendTypeRef(parameter, castType);
			}
			return builder;
		}

		public static StringBuilder AppendParameterDecls(this StringBuilder builder, IEnumerable<UhtRigVMParameter> parameters,
			bool leadingSeparator = false, string separator = ", ", bool castType = false, bool castName = false, bool includeEditorOnly = false)
		{
			bool emitSeparator = leadingSeparator;
			foreach (UhtRigVMParameter parameter in parameters)
			{
				if (includeEditorOnly && !parameter.EditorOnly)
				{
					continue;
				}
				if (emitSeparator)
				{
					builder.Append(separator);
				}
				emitSeparator = true;
				builder
					.AppendTypeVariableRef(parameter, castType)
					.Append(' ')
					.Append(castName ? parameter.CastName : parameter.Name);
			}
			return builder;
		}
	}
}
