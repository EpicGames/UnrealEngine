// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Exporters.CodeGen
{
	class UhtStaticsEmitter : IDisposable
	{
		private readonly StringBuilder _builder;

		public UhtStaticsEmitter(StringBuilder builder, IUhtSingletonNameProvider provider, UhtObject obj)
		{
			_builder = builder;
			EmitMacro(provider.GetSingletonName(obj, UhtSingletonType.Statics));
		}

		public UhtStaticsEmitter(StringBuilder builder, string definition)
		{
			_builder = builder;
			EmitMacro(definition);
		}

		public void Dispose()
		{
			_builder.AppendLine("#undef UHT_STATICS");
		}

		private void EmitMacro(string definition)
		{
			_builder.AppendLine($$"""
				#ifdef UHT_STATICS
				#error UHT_STATICS already defined
				#endif
				#define UHT_STATICS {{definition}}
				""");
		}
	}
}
