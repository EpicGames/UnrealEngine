// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.CodeGen
{

	/// <summary>
	/// Helper class for collection where types exist in different define scope blocks
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtUsedDefineScopes<T> where T : UhtType
	{
		private bool _first = true;
		private UhtDefineScope _soleScope = UhtDefineScope.Invalid;
		private UhtDefineScope _allScopes = UhtDefineScope.None;
		private uint _present = 0;

		/// <summary>
		/// Collection of instances
		/// </summary>
		public List<T> Instances { get; } = [];

		/// <summary>
		/// If true, there are no instances
		/// </summary>
		public bool IsEmpty => Instances.Count == 0;

		/// <summary>
		/// If true, at least one instance had no scope
		/// </summary>
		public bool HasNoneScope => HasScope(UhtDefineScope.None);

		/// <summary>
		/// If all types share the same scope, then the sole scope is that scope.  The value will be Invalid if the
		/// instances have different scopes.
		/// </summary>
		public UhtDefineScope SoleScope => _soleScope;

		/// <summary>
		/// Collection of all scopes found in the types.  If 
		/// </summary>
		public UhtDefineScope AllScopes => _allScopes;

		/// <summary>
		/// If any instance has no scope, then None is returned.  Otherwise AllScopes is returned.
		/// </summary>
		public UhtDefineScope NoneAwareScopes => HasNoneScope ? UhtDefineScope.None : AllScopes;

		/// <summary>
		/// Constructor with no initial instances
		/// </summary>
		public UhtUsedDefineScopes()
		{
		}

		/// <summary>
		/// Constructor with initial range of types
		/// </summary>
		/// <param name="instances">Instances to initially add</param>
		public UhtUsedDefineScopes(IEnumerable<T> instances)
		{
			AddRange(instances);
		}

		/// <summary>
		/// Add an instance to the collection
		/// </summary>
		/// <param name="instance">Instance to be added</param>
		public void Add(T instance)
		{
			UhtDefineScope defineScope = instance.DefineScope;
			if (_first)
			{
				_soleScope = defineScope;
				_first = false;
			}
			else if (_soleScope != UhtDefineScope.Invalid && _soleScope != defineScope)
			{
				_soleScope = UhtDefineScope.Invalid;
			}
			_allScopes |= defineScope;
			_present |= ScopeToMask(defineScope);
			Instances.Add(instance);
		}

		/// <summary>
		/// Add a range of instances to the collection
		/// </summary>
		/// <param name="instances">Collection of instances</param>
		public void AddRange(IEnumerable<T> instances)
		{
			foreach (T instance in instances)
			{
				Add(instance);
			}
		}

		/// <summary>
		/// Check to see if the given scope has instances
		/// </summary>
		/// <param name="defineScope">Scope to test</param>
		/// <returns>True if the scope has elements</returns>
		public bool HasScope(UhtDefineScope defineScope)
		{
			return (_present & ScopeToMask(defineScope)) != 0;
		}

		/// <summary>
		/// Update the list to by ordered by the define scope
		/// </summary>
		public void OrderByDefineScope()
		{
			Instances.SortBy(x => x.DefineScope);
		}

		/// <summary>
		/// Enumerate all of the used defined scopes
		/// </summary>
		/// <returns>Enumeration</returns>
		public IEnumerable<UhtDefineScope> EnumerateDefinedScopes()
		{
			ulong present = _present;
			for (int index = 0; present != 0; ++index, present >>= 1)
			{
				if ((present & 1) != 0)
				{
					yield return (UhtDefineScope)index;
				}
			}
		}

		/// <summary>
		/// Enumerate all of the used defined scopes
		/// </summary>
		/// <returns>Enumeration</returns>
		public IEnumerable<UhtDefineScope> EnumerateDefinedScopesNoneAtEnd()
		{
			ulong present = _present >> 1;
			for (int index = 1; present != 0; ++index, present >>= 1)
			{
				if ((present & 1) != 0)
				{
					yield return (UhtDefineScope)index;
				}
			}

			// At this time, we alway return None
			yield return UhtDefineScope.None;
		}

		private static uint ScopeToMask(UhtDefineScope defineScope)
		{
			return (uint)1 << (int)defineScope;
		}
	}

	/// <summary>
	/// Helper class to build linked lists of objects which are defined in different scopes.
	/// e.g. properties of a class, where some properties are only present when WITH_EDITORONLY_DATA is defined
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtDefineScopeListBuilder<T> : IDisposable
		where T : UhtType
	{
		/// <summary>
		/// Initialize the builder with an empty 'head' link.
		/// </summary>
		public UhtDefineScopeListBuilder(UhtDefineScope scope)
		{
			Head = new UhtDefineScopeLink<T>(scope);
			_pending.Add(Head);
		}

		/// <summary>
		/// Link pointing to the first object in the list for each relevant define scope.
		/// </summary>
		public UhtDefineScopeLink<T> Head { get; }

		/// <summary>
		/// Add the given object as the next object after any preceding object which does not yet have a following object
		/// in the scope(s) the new object is defined in.
		/// </summary>
		/// <param name="inNext">Object to append to the list</param>
		public UhtDefineScopeLink<T> Add(T inNext)
		{
			foreach (UhtDefineScopeLink<T> link in _pending)
			{
				link.Add(inNext);
			}
			_pending.RemoveAll(x => x.IsComplete);
			UhtDefineScopeLink<T> newLink = new(inNext.DefineScope);
			_pending.Add(newLink);
			return newLink;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Explicitly terminate each pending link with null
		/// </summary>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				foreach (UhtDefineScopeLink<T> link in _pending)
				{
					link.Terminate();
				}
			}
		}

		private readonly List<UhtDefineScopeLink<T>> _pending = [];
	}

	/// <summary>
	/// This class can be used as a field in UhtTypes to perform as an intrusive linked list 'next' pointer to 
	/// another object of the same type, resolving to a different object depending on which preprocessor definitions
	/// are set.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtDefineScopeLink<T> where T : UhtType
	{
		/// <summary>
		/// Initialize this link with the declaration scope of the object holding the link.
		/// </summary>
		/// <param name="thisScope"></param>
		public UhtDefineScopeLink(UhtDefineScope thisScope)
		{
			_thisScope = thisScope;
			_completionMask = ToCompletionMask(thisScope);
			// A next node in scope 'None' always completes this link regardless of _thisScope
			_completionMask |= ToCompletionMask(UhtDefineScope.None);
		}

		/// <summary>
		/// Try to add a new element following this one in the linked list. 
		/// If there is already a next element for every combination of scopes for which this and the candidate are defined, 
		/// the new element will not be added.
		/// If, after adding the new element, there is a next element for every combination of scopes in which this is defined,
		/// this link will return true from IsComplete.
		/// </summary>
		/// <param name="inNext"></param>
		public void Add(T inNext)
		{
			uint addedMask = ToAddedMask(inNext.DefineScope);
			if ((addedMask & _addedScopes) == 0)
			{
				_addedScopes |= addedMask;
				_next.Add(inNext);

				uint completionCandidateMask = ToCompletionMask(inNext.DefineScope);
				if ((completionCandidateMask & _completionMask) != 0)
				{
					IsComplete = true;
				}
			}
		}

		/// <summary>
		/// Must be called on an incomplete link. Terminates the link explicitly with nullptr to handle the case where there 
		/// is a next link for some scopes, but none for all scopes the link is defined in.
		/// </summary>
		public void Terminate()
		{
			if (IsComplete)
			{
				throw new InvalidOperationException("Attemping to terminate an already-complete link. Such links should be removed from the pending list by the builder.");
			}
			_next.Add(null);
		}

		/// <summary>
		/// Returns true if there is a next element for every combination of scopes which this is defined.
		/// </summary>
		public bool IsComplete { get; private set; }

		/// <summary>
		/// Returns the scope of the first object in the list, or UhtDefineScope.None.
		/// </summary>
		public UhtDefineScope InitialScope => _next.First()?.DefineScope ?? UhtDefineScope.None;

		/// <summary>
		/// Returns all objects which follow this object in some relevant scope.
		/// Only the last element may be null.
		/// </summary>
		public IReadOnlyList<T?> Next => _next;

		// Ordered list of next objects with differing scopes
		private readonly List<T?> _next = [];

		// The scope in which this link is defined
		private readonly UhtDefineScope _thisScope;

		// When any the bits in this are matched by the bits in the next added item, this link is complete
		private readonly uint _completionMask;

		// Tracks which scopes have been added with Add
		private uint _addedScopes = 0;

		// One of two masks used by this class, this is used for _completionMask to track when this link
		// has a next node for every situation in which it is defined
		private static uint ToCompletionMask(UhtDefineScope scope)
		{
			// Shift other scopes by 1 so that 'None' is assigned a bit as well
			return scope == UhtDefineScope.None ? 1 : (uint)scope << 1;
		}

		// The second mask used by this class, this is used to track when a next node is already present for
		// a given scope, so a candidate does not need to be added
		private uint ToAddedMask(UhtDefineScope scope)
		{
			// Add the scope if this link to the mask because so that e.g.
			// thisScope == UhtDefineScope.EditorOnly
			// inNext.DefineScope = UhtDefineScope.VerseVM
			// 	if the next link to be added is UhtDefineScope.EditorOnly|UhtDefineScope.VerseVM, we do not need to add it
			//	because under the preprocessor state UhtDefineScope.EditorOnly|UhtDefineScope.VerseVM, a next node is defined already.
			return (uint)1 << (int)(scope | _thisScope);
		}
	}

	/// <summary>
	/// Helper extensions when working with defined scope collections.
	/// 
	/// This class helps with producing consistent and correct code that works with the DefineScope element in UhtType.
	/// 
	/// In generated header files, it supports two flavors of macro definitions used to generate the body macro.
	/// 
	/// The multi macro style will generate one macro for each unique combination of DefineScope found.  Only the instances
	/// that match the DefineScope will be placed in that macro.  The macros will be generated in such a way that they 
	/// will always be present and each much be included in the generated body macro.  This style is used to generate
	/// declarations as needed.
	/// 
	/// The single macro style will generate a single macro, but that macro will appear multiple times depending on
	/// each combination of the DefineScope combinations required.  Each macro will have a complete set of instances
	/// filtered by the DefineScope.  This style is used to populate such things as enum definitions needed by the engine.
	/// 
	/// In generated cpp files, there is support for enumerating through all the instances and emitting the appropriate
	/// #if block to include instances based on their DefineScope.
	/// </summary>
	internal static class UhtUsedDefineScopesExtensions
	{

		/// <summary>
		/// Append a macro scoped macro
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="defineScope">Specified scope</param>
		/// <param name="appendAction">Action to invoke to append an instance</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendScoped(this StringBuilder builder, UhtDefineScope defineScope, Action<StringBuilder> appendAction)
		{
			using UhtMacroBlockEmitter blockEmitter = new(builder, defineScope);
			appendAction(builder);
			return builder;
		}

		/// <summary>
		/// Append multiple  for the given collection of scopes
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="appendAction">Action to invoke to append an instance</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendMultiScoped<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, Action<StringBuilder, IEnumerable<T>> appendAction) where T : UhtType
		{
			foreach (UhtDefineScope defineScope in instances.EnumerateDefinedScopes())
			{
				AppendScoped(builder, defineScope, builder => appendAction(builder, instances.Instances.Where(x => x.DefineScope == defineScope)));
			}
			return builder;
		}

		/// <summary>
		/// Append a macro scoped macro
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="defineScope">Specified scope</param>
		/// <param name="generator">Header code generator</param>
		/// <param name="outerType">Output type owning the instances</param>
		/// <param name="macroSuffix">Macro being created</param>
		/// <param name="includeSuffix">If true, include such things as _EOD onto the macro name</param>
		/// <param name="appendAction">Action to invoke to append an instance</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendScopedMacro(this StringBuilder builder, UhtDefineScope defineScope,
			UhtHeaderCodeGenerator generator, UhtType outerType, string macroSuffix, bool includeSuffix, Action<StringBuilder> appendAction)
		{
			using (UhtMacroBlockEmitter blockEmitter = new(builder, defineScope))
			{
				using (UhtMacroCreator macro = new(builder, generator, outerType, macroSuffix, defineScope, includeSuffix))
				{
					appendAction(builder);
				}

				// We can skip writing the macros if there are no properties to declare, as the 'if' and 'else' would be the same
				if (defineScope != UhtDefineScope.None)
				{
					// Trim the extra newlines added after the macro generator
					if (builder.Length > 4 &&
						builder[^4] == '\r' &&
						builder[^3] == '\n' &&
						builder[^2] == '\r' &&
						builder[^1] == '\n')
					{
						builder.Length -= 4;
					}

					builder.AppendElsePreprocessor(defineScope);
					using UhtMacroCreator macro = new(builder, generator, outerType, macroSuffix, defineScope, includeSuffix); // Empty macro
				}
			}

			if (defineScope != UhtDefineScope.None)
			{
				builder.Append("\r\n\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Append multi macros for the given collection of scopes
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="generator">Header code generator</param>
		/// <param name="outerType">Output type owning the instances</param>
		/// <param name="macroSuffix">Macro being created</param>
		/// <param name="appendAction">Action to invoke to append an instance</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendMultiScopedMacros<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, 
			UhtHeaderCodeGenerator generator, UhtType outerType, string macroSuffix, Action<StringBuilder, IEnumerable<T>> appendAction) where T : UhtType
		{
			foreach (UhtDefineScope defineScope in instances.EnumerateDefinedScopes())
			{
				AppendScopedMacro(builder, defineScope, generator, outerType, macroSuffix, true,
					builder => appendAction(builder, instances.Instances.Where(x => x.DefineScope == defineScope)));
			}
			return builder;
		}

		/// <summary>
		/// Append the macro definitions requested
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="generator">Header code generator</param>
		/// <param name="outerType">Output type owning the instances</param>
		/// <param name="macroSuffix">Macro being created</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendMultiMacroRefs<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances,
			UhtHeaderCodeGenerator generator, UhtType outerType, string macroSuffix) where T : UhtType
		{
			foreach (UhtDefineScope defineScope in instances.EnumerateDefinedScopes())
			{
				builder.Append('\t').AppendMacroName(generator.FileId, outerType.GetMacroLineNumber(), macroSuffix, defineScope, true).Append(" \\\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Append a single macro, where the macro -same- definition can exist inside of define scopes where
		/// the instances contained are all instances that would be satisfied by a scope.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="generator">Header code generator</param>
		/// <param name="outerType">Output type owning the instances</param>
		/// <param name="macroSuffix">Macro being created</param>
		/// <param name="appendAction">Action to invoke to append an instance</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendSingleMacro<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, UhtHeaderCodeGenerator generator, 
			UhtType outerType, string macroSuffix, Action<StringBuilder, IEnumerable<T>> appendAction) where T : UhtType
		{
			if (instances.SoleScope == UhtDefineScope.None)
			{
				using UhtMacroCreator macro = new(builder, generator, outerType, macroSuffix);
				appendAction(builder, instances.Instances);
			}
			else
			{
				bool first = true;
				foreach (UhtDefineScope defineScope in instances.EnumerateDefinedScopesNoneAtEnd())
				{
					if (first)
					{
						builder.AppendIfPreprocessor(defineScope);
						first = false;
					}
					else if (defineScope != UhtDefineScope.None)
					{
						builder.AppendElseIfPreprocessor(defineScope);
					}
					else
					{
						builder.Append("#else\r\n");
					}
					using (UhtMacroCreator macro = new(builder, generator, outerType, macroSuffix))
					{
						appendAction(builder, instances.Instances.Where(x => (x.DefineScope & ~defineScope) == 0));
					}
					if (builder.Length > 4 &&
						builder[^4] == '\r' &&
						builder[^3] == '\n' &&
						builder[^2] == '\r' &&
						builder[^1] == '\n')
					{
						builder.Length -= 4;
					}
				}
				builder.Append("#endif\r\n\r\n\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Invoke the append action if any types are present.  If all types are from the same define scope, then
		/// it will be wrapped with and #if block.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="appendAction">Action to invoke</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendIfInstances<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, Action<StringBuilder> appendAction) where T : UhtType
		{
			if (!instances.IsEmpty)
			{
				using UhtMacroBlockEmitter blockEmitter = new(builder, instances.NoneAwareScopes);
				appendAction(builder);
			}
			return builder;
		}

		/// <summary>
		/// Append each instance to the builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="appendAction">Action to invoke</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendInstances<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, Action<StringBuilder, T> appendAction) where T : UhtType
		{
			return builder.AppendInstances(instances, null, appendAction, null);
		}

		/// <summary>
		/// Append each instance to the builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Defined scope instances</param>
		/// <param name="preambleAction">Action to invoke prior to first instance</param>
		/// <param name="appendAction">Action to invoke</param>
		/// <param name="postambleAction">Action to invoke following all instances</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendInstances<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, 
			Action<StringBuilder>? preambleAction, Action<StringBuilder, T> appendAction, Action<StringBuilder>? postambleAction) where T : UhtType
		{
			if (!instances.IsEmpty)
			{
				using UhtMacroBlockEmitter blockEmitter = new(builder, instances.SoleScope);
				preambleAction?.Invoke(builder);
				foreach (T instance in instances.Instances)
				{
					blockEmitter.Set(instance.DefineScope);
					appendAction(builder, instance);
				}

				// Make sure the postable run with the proper initial scope
				blockEmitter.Set(instances.SoleScope);
				postambleAction?.Invoke(builder);
			}
			return builder;
		}

		/// <summary>
		/// Append an array view construction expression for the given instances
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Collected instances</param>
		/// <param name="identifier">Name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayView<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, UhtCppIdentifier identifier, int tabs, string endl) where T : UhtType
		{
			UhtDefineScope noneAwareScopes = instances.NoneAwareScopes;

			// We have no instances in the collection, the list will always be empty
			if (instances.IsEmpty)
			{
				builder.AppendTabs(tabs).Append($"{{}}{endl}");
			}

			// If we have any instances that have no scope, then we can always just reference the array
			else if (noneAwareScopes == UhtDefineScope.None)
			{
				builder.AppendTabs(tabs).Append($"MakeConstArrayView({identifier}){endl}");
			}

			// If the only scope we have is WITH_EDITORONLY_DATA, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.EditorOnlyData)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITORONLY_DATA(MakeConstArrayView({identifier}), {{}}){endl}");
			}

			// If the only scope we have is WITH_EDITOR, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.Editor)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITOR(MakeConstArrayView({identifier}), {{}}){endl}");
			}

			// Otherwise we have many different scopes but no none scope, must generate #if block
			else
			{
				builder.Append("\r\n");
				builder.AppendIfPreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"MakeConstArrayView({identifier}){endl}");
				builder.AppendElsePreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"{{}}{endl}");
				builder.AppendEndIfPreprocessor(instances.AllScopes);
			}
			return builder;
		}

		/// <summary>
		/// Append the given array list and count as arguments to a structure constructor
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Collected instances</param>
		/// <param name="identifier">Name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayPtrAndCountLine<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, UhtCppIdentifier identifier, int tabs, string endl) where T : UhtType
		{
			UhtDefineScope noneAwareScopes = instances.NoneAwareScopes;

			// We have no instances in the collection, the list will always be empty
			if (instances.IsEmpty)
			{
				builder.AppendTabs(tabs).Append($"nullptr, 0{endl}");
			}

			// If we have any instances that have no scope, then we can always just reference the array
			else if (noneAwareScopes == UhtDefineScope.None)
			{
				builder.AppendTabs(tabs).Append($"{identifier}, UE_ARRAY_COUNT({identifier}){endl}");
			}

			// If the only scope we have is WITH_EDITORONLY_DATA, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.EditorOnlyData)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITORONLY_DATA({identifier}, nullptr), IF_WITH_EDITORONLY_DATA(UE_ARRAY_COUNT({identifier}), 0){endl}");
			}

			// If the only scope we have is WITH_EDITOR, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.Editor)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITOR({identifier}, nullptr), IF_WITH_EDITOR(UE_ARRAY_COUNT({identifier}), 0){endl}");
			}

			// Otherwise we have many different scopes but no none scope, must generate #if block
			else
			{
				builder.AppendIfPreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"{identifier}, UE_ARRAY_COUNT({identifier}){endl}");
				builder.AppendElsePreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"nullptr, 0{endl}");
				builder.AppendEndIfPreprocessor(instances.AllScopes);
			}
			return builder;
		}

		/// <summary>
		/// Append the given array list as arguments to a structure constructor
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Collected instances</param>
		/// <param name="identifier">Name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayPtrLine<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, UhtCppIdentifier identifier, int tabs, string endl) where T : UhtType
		{
			UhtDefineScope noneAwareScopes = instances.NoneAwareScopes;

			// We have no instances in the collection, the list will always be empty
			if (instances.IsEmpty)
			{
				builder.AppendTabs(tabs).Append($"nullptr{endl}");
			}

			// If we have any instances that have no scope, then we can always just reference the array
			else if (noneAwareScopes == UhtDefineScope.None)
			{
				builder.AppendTabs(tabs).Append($"{identifier}{endl}");
			}

			// If the only scope we have is WITH_EDITORONLY_DATA, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.EditorOnlyData)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITORONLY_DATA({identifier}, nullptr){endl}");
			}

			// If the only scope we have is WITH_EDITOR, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.Editor)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITOR({identifier}, nullptr){endl}");
			}

			// Otherwise we have many different scopes but no none scope, must generate #if block
			else
			{
				builder.AppendIfPreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"{identifier}{endl}");
				builder.AppendElsePreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"nullptr{endl}");
				builder.AppendEndIfPreprocessor(instances.AllScopes);
			}
			return builder;
		}

		/// <summary>
		/// Append the given array count as arguments to a structure constructor
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="instances">Collected instances</param>
		/// <param name="identifier">Name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayCountLine<T>(this StringBuilder builder, UhtUsedDefineScopes<T> instances, UhtCppIdentifier identifier, int tabs, string endl) where T : UhtType
		{
			UhtDefineScope noneAwareScopes = instances.NoneAwareScopes;

			// We have no instances in the collection, the list will always be empty
			if (instances.IsEmpty)
			{
				builder.AppendTabs(tabs).Append($"0{endl}");
			}

			// If we have any instances that have no scope, then we can always just reference the array
			else if (noneAwareScopes == UhtDefineScope.None)
			{
				builder.AppendTabs(tabs).Append($"UE_ARRAY_COUNT({identifier}){endl}");
			}

			// If the only scope we have is WITH_EDITORONLY_DATA, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.EditorOnlyData)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITORONLY_DATA(UE_ARRAY_COUNT({identifier}), 0){endl}");
			}

			// If the only scope we have is WITH_EDITOR, then we can use the existing macros
			else if (noneAwareScopes == UhtDefineScope.Editor)
			{
				builder.AppendTabs(tabs).Append($"IF_WITH_EDITOR(UE_ARRAY_COUNT({identifier}), 0){endl}");
			}

			// Otherwise we have many different scopes but no none scope, must generate #if block
			else
			{
				builder.AppendIfPreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"UE_ARRAY_COUNT({identifier}){endl}");
				builder.AppendElsePreprocessor(instances.AllScopes);
				builder.AppendTabs(tabs).Append($"0{endl}");
				builder.AppendEndIfPreprocessor(instances.AllScopes);
			}
			return builder;
		}

		/// <summary>
		/// Append a link which resolves to a different object under different preprocessor definitions
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="link"></param>
		/// <param name="tabs"></param>
		/// <param name="prefix"></param>
		/// <param name="endl"></param>
		/// <returns></returns>
		public static StringBuilder AppendScopeLink(this StringBuilder builder, UhtDefineScopeLink<UhtProperty>? link, int tabs, string? prefix, string? endl)
		{
			// No link or link contains nothing
			if (link is null)
			{
				return builder.AppendTabs(tabs).Append(prefix).Append("nullptr").Append(endl);
			}

			using (UhtMacroBlockEmitter blockEmitter = new(builder, link.InitialScope))
			{
				foreach (UhtProperty? next in link.Next)
				{
					blockEmitter.Set(next?.DefineScope ?? UhtDefineScope.None);
					builder.AppendTabs(tabs).Append(prefix);
					builder.AppendConstInitPtr(next, UhtNames.GetPropertyIdentifier(next), 0, endl);
				}
			}
			return builder;
		}

		/// <summary>
		/// Append a link which resolves to a different object under different preprocessor definitions
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="link"></param>
		/// <param name="context"></param>
		/// <param name="tabs"></param>
		/// <param name="prefix"></param>
		/// <param name="endl"></param>
		/// <returns></returns>
		public static StringBuilder AppendScopeLink(this StringBuilder builder, UhtDefineScopeLink<UhtFunction>? link, IUhtPropertyMemberContext context, int tabs, string? prefix, string? endl)
		{
			// No link or link contains nothing
			if (link is null)
			{
				return builder.AppendTabs(tabs).Append(prefix).Append("nullptr").Append(endl);
			}

			using (UhtMacroBlockEmitter blockEmitter = new(builder, link.InitialScope))
			{
				foreach (UhtFunction? next in link.Next)
				{
					blockEmitter.SetElse(next?.DefineScope ?? UhtDefineScope.None);
					builder.AppendTabs(tabs).Append(prefix);
					builder.AppendConstInitSingletonRef(context, next).Append(endl);
				}
			}
			return builder;
		}

		/// <summary>
		/// Start an #if block with the given scope
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="defineScope">Scope</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendIfPreprocessor(this StringBuilder builder, UhtDefineScope defineScope)
		{
			if (defineScope != UhtDefineScope.None && defineScope != UhtDefineScope.Invalid)
			{
				builder.Append("#if ").AppendScopeExpression(defineScope).Append("\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Start an #else block with the given scope
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="defineScope">Scope</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendElsePreprocessor(this StringBuilder builder, UhtDefineScope defineScope)
		{
			if (defineScope != UhtDefineScope.None && defineScope != UhtDefineScope.Invalid)
			{
				builder.Append("#else // ").AppendScopeExpression(defineScope).Append("\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Start an #elif block with the given scope
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="defineScope">Scope</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendElseIfPreprocessor(this StringBuilder builder, UhtDefineScope defineScope)
		{
			if (defineScope != UhtDefineScope.None && defineScope != UhtDefineScope.Invalid)
			{
				builder.Append("#elif ").AppendScopeExpression(defineScope).Append("\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Start an #endif block with the given scope
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="defineScope">Scope</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendEndIfPreprocessor(this StringBuilder builder, UhtDefineScope defineScope)
		{
			if (defineScope != UhtDefineScope.None && defineScope != UhtDefineScope.Invalid)
			{
				builder.Append("#endif // ").AppendScopeExpression(defineScope).Append("\r\n");
			}
			return builder;
		}

		/// <summary>
		/// Append scope expression (i.e. WITH_X &amp;&amp; WITH_Y &amp;&amp; ...)
		/// </summary>
		/// <param name="builder">String builder</param>
		/// <param name="defineScope">Scope</param>
		/// <returns>String builder</returns>
		public static StringBuilder AppendScopeExpression(this StringBuilder builder, UhtDefineScope defineScope)
		{
			if (defineScope != UhtDefineScope.None && defineScope != UhtDefineScope.Invalid)
			{
				int scopes = (int)defineScope;
				bool needAnd = false;
				for (int mask = 1; mask <= scopes; mask <<= 1)
				{
					if ((mask & scopes) != 0)
					{
						if (needAnd)
						{
							builder.Append(" && ");
						}
						builder.Append(((UhtDefineScope)(mask & scopes)).GetCompilerDirective().GetCompilerDirectiveText());
						needAnd = true;
					}
				}
			}
			return builder;
		}
	}
}
