// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Exporters.Json
{
	[UnrealHeaderTool]
	sealed class UhtJsonExporter
	{

		[UhtExporter(Name = "Json", Description = "Json description of packages", Options = UhtExporterOptions.None)]
		public static void JsonExporter(IUhtExportFactory factory)
		{
			new UhtJsonExporter(factory).Export();
		}

		public readonly IUhtExportFactory Factory;
		public UhtSession Session => Factory.Session;

		private UhtJsonExporter(IUhtExportFactory factory)
		{
			Factory = factory;
		}

		private void Export()
		{
			// Generate the files for the packages
			List<Task?> generatedPackages = new(Session.PackageTypeCount);
			JsonSerializerOptions options = new() { WriteIndented = true, DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull };
			foreach (UhtModule module in Session.Modules)
			{
				generatedPackages.Add(Factory.CreateTask(
					factory =>
					{
						string jsonPath = factory.MakePath(module, ".json");
						factory.CommitOutput(jsonPath, JsonSerializer.Serialize(module, options));
					}));
			}

			// Wait for all the packages to complete
			List<Task> packageTasks = new(Session.PackageTypeCount);
			foreach (Task? output in generatedPackages)
			{
				if (output != null)
				{
					packageTasks.Add(output);
				}
			}
			Task.WaitAll([.. packageTasks]);
		}
	}
}
