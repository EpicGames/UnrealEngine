// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;

namespace EpicGames.UHT.Exporters.Stats
{
	[UnrealHeaderTool]
	internal sealed class UhtStatsExporter
	{
		[UhtExporter(Name = "Stats", Description = "Type Stats", Options = UhtExporterOptions.None)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static void StatsExporter(IUhtExportFactory factory)
		{
			SortedDictionary<string, int> countByType = [];
			foreach (UhtModule module in factory.Session.Modules)
			{
				foreach (UhtPackage package in module.Packages)
				{
					Collect(countByType, package);
				}
			}

			ILogger logger = factory.Session.Logger;
			logger.LogInformation("Counts by type:");

			foreach (KeyValuePair<string, int> kvp in countByType)
			{
				logger.LogInformation("{Key} {Value}", kvp.Key, kvp.Value);
			}
			logger.LogInformation("");
		}

		private static void Collect(SortedDictionary<string, int> countByType, UhtType type)
		{
			if (countByType.TryGetValue(type.EngineClassName, out int count))
			{
				countByType[type.EngineClassName] = count + 1;
			}
			else
			{
				countByType[type.EngineClassName] = 1;
			}

			foreach (UhtType child in type.Children)
			{
				Collect(countByType, child);
			}
		}
	}
}
