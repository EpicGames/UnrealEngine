// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// UCLASS parser
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtClassParser
	{

		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UCLASSKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUClass(topScope, ref token);
		}

		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult VMODULEKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUClass(topScope, ref token);
		}

		[UhtKeyword(Extends = UhtTableNames.Class)]
		[UhtKeyword(Extends = UhtTableNames.Class, Keyword = "GENERATED_BODY")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult GENERATED_UCLASS_BODYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtClass classObj = (UhtClass)topScope.ScopeType;

			if (token.IsValue("GENERATED_BODY"))
			{
				classObj.HasGeneratedBody = true;
				classObj.GeneratedBodyAccessSpecifier = topScope.AccessSpecifier;
			}
			else
			{
				classObj.ClassExportFlags |= UhtClassExportFlags.UsesGeneratedBodyLegacy;
				topScope.AccessSpecifier = UhtAccessSpecifier.Public;
			}

			UhtParserHelpers.ParseCompileVersionDeclaration(topScope.TokenReader, topScope.Session.Config!, classObj);

			classObj.GeneratedBodyLineNumber = topScope.TokenReader.InputLine;

			// C++ UHT TODO - In the C++ version, we don't skip any trailing ';'.  ParseStatement will generate an error
			//topScope.TokenReader.Optional(';');
			return UhtParseResult.Handled;
		}
		#endregion

		private static UhtParseResult ParseUClass(UhtParsingScope parentScope, ref UhtToken token)
		{
			UhtClass classObj = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, token.InputLine);
			if (token.IsIdentifier("VMODULE"))
			{
				classObj.ClassType = UhtClassType.VModule;
			}

			{
				using UhtParsingScope topScope = new(parentScope, classObj, parentScope.Session.GetKeywordTable(UhtTableNames.Class), UhtAccessSpecifier.Private);
				const string ScopeName = "class";

				{
					using UhtMessageContext tokenContext = new(ScopeName);
					// Parse the specifiers
					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, classObj.MetaData);
					UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, ScopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.Class));
					specifiers.ParseSpecifiers();
					classObj.PrologLineNumber = topScope.TokenReader.InputLine;
					classObj.ClassFlags |= EClassFlags.Native;
					if (classObj.ClassType == UhtClassType.VModule)
					{
						classObj.ClassFlags |= EClassFlags.Hidden;
					}

					topScope.AddFormattedCommentsAsTooltipMetaData();

					// Consume the class specifier
					topScope.TokenReader.OptionalAttributes(false);
					topScope.TokenReader.Require("class");
					topScope.TokenReader.OptionalAttributes(true);

					// Read the class name and possible API macro name
					topScope.TokenReader.TryOptionalAPIMacro(out UhtToken apiMacroToken);
					classObj.SourceName = topScope.TokenReader.GetIdentifier().Value.ToString();

					// Update the context for better error messages
					tokenContext.Reset($"class '{classObj.SourceName}'");

					// Split the source name into the different parts
					UhtEngineNameParts nameParts = UhtUtilities.GetEngineNameParts(classObj.SourceName);
					classObj.EngineName = nameParts.EngineName.ToString();

					// Check for an empty engine name
					if (classObj.EngineName.Length == 0)
					{
						topScope.TokenReader.LogError($"When compiling class definition for '{classObj.SourceName}', attempting to strip prefix results in an empty name. Did you leave off a prefix?");
					}

					// Skip optional final keyword
					if (topScope.TokenReader.TryOptional("final"))
					{
						classObj.ClassExportFlags |= UhtClassExportFlags.Final;
					}

					// Parse the inheritance
					UhtParserHelpers.ParseInheritance(parentScope.HeaderParser, topScope.Session.Config!, out UhtInheritance inheritance);
					classObj.SuperIdentifier = inheritance.SuperIdentifier;
					classObj.BaseIdentifiers = inheritance.BaseIdentifiers;
					classObj.VerseInterfaceIdentifiers = inheritance.VerseInterfaceIdentifiers;
					classObj.VerseInterfacesLine = inheritance.VerseInterfacesLine;

					if (apiMacroToken)
					{
						classObj.ClassFlags |= EClassFlags.RequiredAPI;
					}

					specifiers.ParseDeferred();

					// Handle verse default settings
					if (classObj.IsVerseField)
					{
						classObj.MetaData.Add(UhtNames.IsBlueprintBase, "false");
						classObj.MetaData.Add(UhtNames.DisplayName, classObj.VerseName!);
						classObj.ClassFlags |= EClassFlags.EditInlineNew | EClassFlags.HasInstancedReference;
					}

					UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
					classObj.DefineScope |= compilerDirective.GetDefaultDefineScopes();

					classObj.Outer?.AddChild(classObj);

					topScope.AddModuleRelativePathToMetaData();

					topScope.HeaderParser.ParseStatements('{', '}', true);

					topScope.TokenReader.Require(';');

					if (classObj.GeneratedBodyLineNumber == -1)
					{
						topScope.TokenReader.LogError("Expected a GENERATED_BODY() at the start of the class");
					}
				}
			}

			return UhtParseResult.Handled;
		}
	}
}
