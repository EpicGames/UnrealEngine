// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// UENUM parser
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtEnumParser
	{
		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UENUMKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUEnum(topScope, token);
		}
		#endregion

		private static UhtParseResult ParseUEnum(UhtParsingScope parentScope, UhtToken keywordToken)
		{
			UhtEnum enumObject = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, keywordToken.InputLine);
			{
				using UhtParsingScope topScope = new(parentScope, enumObject, parentScope.Session.GetKeywordTable(UhtTableNames.Enum), UhtAccessSpecifier.Public);
				const string ScopeName = "UENUM";

				{
					using UhtMessageContext tokenContext = new(ScopeName);

					// Parse the specifiers
					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, enumObject.MetaData);
					UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, ScopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.Enum));
					specifiers.ParseSpecifiers();

					// Read the name and the CPP type
					if (topScope.TokenReader.TryOptional("namespace"))
					{
						enumObject.CppForm = UhtEnumCppForm.Namespaced;
					}
					else
					{
						topScope.TokenReader.OptionalAttributes(false);
						if (topScope.TokenReader.TryOptional("enum"))
						{
							enumObject.CppForm = topScope.TokenReader.TryOptional("class") || topScope.TokenReader.TryOptional("struct") ? UhtEnumCppForm.EnumClass : UhtEnumCppForm.Regular;
						}
						else
						{
							throw new UhtTokenException(topScope.TokenReader, topScope.TokenReader.PeekToken(), null);
						}
					}

					topScope.TokenReader.OptionalAttributes(false);

					UhtToken enumToken = topScope.TokenReader.GetIdentifier("enumeration name");

					enumObject.SourceName = enumToken.Value.ToString();

					specifiers.ParseFieldMetaData();
					specifiers.ParseDeferred();

					enumObject.Outer?.AddChild(enumObject);

					UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
					enumObject.DefineScope |= compilerDirective.GetDefaultDefineScopes();

					// Read base for enum class
					if (enumObject.CppForm == UhtEnumCppForm.EnumClass)
					{
						ParseUnderlyingType(topScope, enumObject);

						if (enumObject.UnderlyingType != UhtEnumUnderlyingType.Uint8 && enumObject.MetaData.ContainsKey("BlueprintType"))
						{
							topScope.TokenReader.LogError("Invalid BlueprintType enum base - currently only uint8 supported");
						}
					}
					else
					{
						if (enumObject.CppForm == UhtEnumCppForm.Regular)
						{
							ParseUnderlyingType(topScope, enumObject);
						}
						if ((enumObject.EnumFlags & EEnumFlags.Flags) != 0)
						{
							topScope.TokenReader.LogError("The 'Flags' specifier can only be used on enum classes");
						}
					}

					//EnumDef.GetDefinitionRange().Start = &Input[InputPos];

					// Get the opening brace
					topScope.TokenReader.Require('{');

					switch (enumObject.CppForm)
					{
						case UhtEnumCppForm.Namespaced:
							// Now handle the inner true enum portion
							topScope.TokenReader.OptionalAttributes(false);
							topScope.TokenReader.Require("enum");
							topScope.TokenReader.OptionalAttributes(true);

							UhtToken innerEnumToken = topScope.TokenReader.GetIdentifier("enumeration type name");

							ParseUnderlyingType(topScope, enumObject);

							topScope.TokenReader.Require('{');
							enumObject.CppType = $"{enumObject.SourceName}::{innerEnumToken.Value}";
							break;

						case UhtEnumCppForm.EnumClass:
						case UhtEnumCppForm.Regular:
							enumObject.CppType = enumObject.SourceName;
							break;
					}

					if (enumObject.CppForm != UhtEnumCppForm.EnumClass && enumObject.UnderlyingType == UhtEnumUnderlyingType.Unspecified)
					{
						string logMessage = $"Underlying type must be specified.";
						switch (topScope.Session.Config!.EnumUnderlyingTypeNotSet.GetBehavior(enumObject.Module))
						{
							case UhtIssueBehavior.AllowSilently:
								break;

							case UhtIssueBehavior.AllowAndLog:
								topScope.TokenReader.LogTrace(enumToken.InputLine, logMessage);
								break;

							default:
								topScope.TokenReader.LogError(enumToken.InputLine, logMessage);
								break;
						}
					}

					tokenContext.Reset($"UENUM {enumObject.SourceName}");

					topScope.AddModuleRelativePathToMetaData();
					topScope.AddFormattedCommentsAsTooltipMetaData();

					// Parse all enum tags
					bool hasUnparsedValue = false;
					long currentEnumValue = 0;

					topScope.TokenReader.RequireList('}', ',', true, () =>
					{
						UhtToken tagToken = topScope.TokenReader.GetIdentifier();
						if (tagToken.IsValue("true", true) || tagToken.IsValue("false", true))
						{
							// C++ UHT compatibility - TODO
							topScope.TokenReader.LogError("Enumerations can't have any elements named 'true' or 'false' regardless of case");
						}

						StringView fullEnumName;
						switch (enumObject.CppForm)
						{
							case UhtEnumCppForm.Namespaced:
							case UhtEnumCppForm.EnumClass:
								fullEnumName = new StringView($"{enumObject.SourceName}::{tagToken.Value}");
								break;

							case UhtEnumCppForm.Regular:
								fullEnumName = tagToken.Value;
								break;

							default:
								throw new UhtIceException("Unexpected EEnumCppForm value");
						}

						// Save the new tag with a default value.  This will be replaced later
						//COMPATIBILITY-TODO: If a enum value has a comment and a tooltip, it will generate an error.
						// This is the reverse of all other parsing.  This code should be modified to process the comment
						// after the UMETA.
						int enumIndex = enumObject.AddEnumValue(tagToken.Value.ToString(), 0);
						topScope.AddFormattedCommentsAsTooltipMetaData(enumIndex);

						// Skip any deprecation
						topScope.TokenReader.OptionalAttributes(false);

						// Try to read an optional explicit enum value specification
						if (topScope.TokenReader.TryOptional('='))
						{
							bool parsedValue = topScope.TokenReader.TryOptionalConstLong(out long scatchValue);
							if (parsedValue)
							{
								currentEnumValue = scatchValue;
							}
							else
							{
								// We didn't parse a literal, so set an invalid value
								currentEnumValue = -1;
								hasUnparsedValue = true;
							}

							// Skip tokens until we encounter a comma, a closing brace or a UMETA declaration
							// There are tokens after the initializer so it's not a standalone literal,
							// so set it to an invalid value.
							int skippedCount = topScope.TokenReader.ConsumeUntil([",", "}", "UMETA"]);
							if (skippedCount == 0 && !parsedValue)
							{
								throw new UhtTokenException(topScope.TokenReader, topScope.TokenReader.PeekToken(), "enumerator initializer");
							}
							if (skippedCount > 0)
							{
								currentEnumValue = -1;
								hasUnparsedValue = true;
							}
						}

						// Save the value and auto increment
						UhtEnumValue value = enumObject.EnumValues[enumIndex];
						value.Value = currentEnumValue;
						enumObject.EnumValues[enumIndex] = value;
						if (currentEnumValue != -1)
						{
							++currentEnumValue;
						}

						enumObject.MetaData.Add(UhtNames.Name, enumIndex, enumObject.EnumValues[enumIndex].Name.ToString());

						// check for metadata on this enum value
						specifierContext.MetaNameIndex = enumIndex;
						specifiers.ParseFieldMetaData();
					});

					// Trailing brace and semicolon for the enum
					topScope.TokenReader.Require(';');
					if (enumObject.CppForm == UhtEnumCppForm.Namespaced)
					{
						topScope.TokenReader.Require('}');
					}

					if (!hasUnparsedValue && !enumObject.IsValidEnumValue(0) && enumObject.MetaData.ContainsKey(UhtNames.BlueprintType))
					{
						enumObject.LogWarning($"'{enumObject.SourceName}' does not have a 0 entry! (This is a problem when the enum is initialized by default)");
					}
				}

				return UhtParseResult.Handled;
			}
		}

		private static void ParseUnderlyingType(UhtParsingScope topScope, UhtEnum enumObject)
		{
			if (topScope.TokenReader.TryOptional(':'))
			{
				UhtToken enumType = topScope.TokenReader.GetIdentifier("enumeration base");

				if (!System.Enum.TryParse<UhtEnumUnderlyingType>(enumType.Value.ToString(), true, out UhtEnumUnderlyingType underlyingType) || underlyingType == UhtEnumUnderlyingType.Unspecified)
				{
					topScope.TokenReader.LogError(enumType.InputLine, $"Unsupported enum underlying base type '{enumType.Value}'");
				}
				enumObject.UnderlyingType = underlyingType;
			}
			else
			{
				enumObject.UnderlyingType = UhtEnumUnderlyingType.Unspecified;
			}
		}
	}
}
