// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/** 
	 *	AdvancedDisplay can be used in two ways:
	 *	1. 'AdvancedDisplay = "3"' - the number tells how many parameters (from beginning) should NOT BE marked
	 *	2. 'AdvancedDisplay = "AttachPointName, Location, LocationType"' - list the parameters, that should BE marked
	 */
	struct UhtAdvancedDisplayParameterHandler
	{
		private readonly UhtMetaData _metaData;
		private readonly string[]? _parameterNames;
		private readonly int _numberLeaveUnmarked;
		private readonly bool _bUseNumber;
		private int _alreadyLeft;

		public UhtAdvancedDisplayParameterHandler(UhtMetaData metaData)
		{
			_metaData = metaData;
			_parameterNames = null;
			_numberLeaveUnmarked = -1;
			_alreadyLeft = 0;
			_bUseNumber = false;

			if (_metaData.TryGetValue(UhtNames.AdvancedDisplay, out string? foundString))
			{
				_parameterNames = foundString.ToString().Split(',', StringSplitOptions.RemoveEmptyEntries);
				for (int index = 0, endIndex = _parameterNames.Length; index < endIndex; ++index)
				{
					_parameterNames[index] = _parameterNames[index].Trim();
				}
				if (_parameterNames.Length == 1)
				{
					_bUseNumber = Int32.TryParse(_parameterNames[0], out _numberLeaveUnmarked);
				}
			}
		}

		/** 
		 * return if given parameter should be marked as Advance View, 
		 * the function should be called only once for any parameter
		 */
		public bool ShouldMarkParameter(StringView parameterName)
		{
			if (_bUseNumber)
			{
				if (_numberLeaveUnmarked < 0)
				{
					return false;
				}
				if (_alreadyLeft < _numberLeaveUnmarked)
				{
					++_alreadyLeft;
					return false;
				}
				return true;
			}

			if (_parameterNames == null)
			{
				return false;
			}

			foreach (string element in _parameterNames)
			{
				if (parameterName.Span.Equals(element, StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}
			return false;
		}

		/** return if more parameters can be marked */
		public readonly bool CanMarkMore()
		{
			return _bUseNumber ? _numberLeaveUnmarked > 0 : (_parameterNames != null && _parameterNames.Length > 0);
		}
	}

	/// <summary>
	/// UFUNCTION parser
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtFunctionParser
	{
		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[UhtKeyword(Extends = UhtTableNames.Class)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UDELEGATEKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUDelegate(topScope, token, true);
		}

		[UhtKeyword(Extends = UhtTableNames.Class)]
		[UhtKeyword(Extends = UhtTableNames.Interface)]
		[UhtKeyword(Extends = UhtTableNames.NativeInterface)]
		[UhtKeyword(Extends = UhtTableNames.VModule)]
		[UhtKeyword(Extends = UhtTableNames.Partial)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UFUNCTIONKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUFunction(topScope, token);
		}

		[UhtKeywordCatchAll(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static UhtParseResult ParseCatchAllKeyword(UhtParsingScope topScope, ref UhtToken token)
		{
			if (topScope.HeaderParser.NamespaceDepth == 1 && IsValidateDelegateDeclaration(token))
			{
				return ParseUDelegate(topScope, token, false);
			}
			return UhtParseResult.Unhandled;
		}
		#endregion

		private static UhtParseResult ParseUDelegate(UhtParsingScope parentScope, UhtToken token, bool hasSpecifiers)
		{
			UhtFunction function = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, token.InputLine);

			{
				using UhtParsingScope topScope = new(parentScope, function, parentScope.Session.GetKeywordTable(UhtTableNames.Function), UhtAccessSpecifier.Public);
				const string ScopeName = "delegate declaration";

				{
					using UhtMessageContext tokenContext = new(ScopeName);
					topScope.AddModuleRelativePathToMetaData();

					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, function.MetaData);
					UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, ScopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.Function));

					UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
					if ((compilerDirective & UhtCompilerDirective.WithEditor) != 0)
					{
						function.FunctionFlags |= EFunctionFlags.EditorOnly;
						function.DefineScope |= UhtDefineScope.Editor;
					}
					function.DefineScope |= (compilerDirective & ~(UhtCompilerDirective.WithEditorOnlyData | UhtCompilerDirective.WithEditor)).GetDefaultDefineScopes();

					// If this is a UDELEGATE, parse the specifiers
					StringView delegateMacro = new();
					if (hasSpecifiers)
					{
						specifiers.ParseSpecifiers();
						specifiers.ParseDeferred();
						FinalizeFunctionSpecifiers(function);

						UhtToken macroToken = topScope.TokenReader.GetToken();
						if (!IsValidateDelegateDeclaration(macroToken))
						{
							throw new UhtTokenException(topScope.TokenReader, macroToken, "delegate macro");
						}
						delegateMacro = macroToken.Value;
					}
					else
					{
						delegateMacro = token.Value;
					}

					// Break the delegate declaration macro down into parts
					bool hasReturnValue = delegateMacro.Span.Contains("_RetVal".AsSpan(), StringComparison.Ordinal);
					bool declaredConst = delegateMacro.Span.Contains("_Const".AsSpan(), StringComparison.Ordinal);
					bool isMulticast = delegateMacro.Span.Contains("_MULTICAST".AsSpan(), StringComparison.Ordinal);
					bool isSparse = delegateMacro.Span.Contains("_SPARSE".AsSpan(), StringComparison.Ordinal);

					// Determine the parameter count
					int foundParamIndex = topScope.Session.Config!.FindDelegateParameterCount(delegateMacro);

					// Try reconstructing the string to make sure it matches our expectations
					string expectedOriginalString = String.Format("DECLARE_DYNAMIC{0}{1}_DELEGATE{2}{3}{4}",
						isMulticast ? "_MULTICAST" : "",
						isSparse ? "_SPARSE" : "",
						hasReturnValue ? "_RetVal" : "",
						topScope.Session.Config!.GetDelegateParameterCountString(foundParamIndex),
						declaredConst ? "_Const" : "");
					if (delegateMacro != expectedOriginalString)
					{
						throw new UhtException(topScope.TokenReader, $"Unable to parse delegate declaration; expected '{expectedOriginalString}' but found '{delegateMacro}'.");
					}

					// Multi-cast delegate function signatures are not allowed to have a return value
					if (hasReturnValue && isMulticast)
					{
						throw new UhtException(topScope.TokenReader, "Multi-cast delegates function signatures must not return a value");
					}

					// Delegate signature
					function.FunctionType = isSparse ? UhtFunctionType.SparseDelegate : UhtFunctionType.Delegate;
					function.FunctionFlags |= EFunctionFlags.Public | EFunctionFlags.Delegate;
					if (isMulticast)
					{
						function.FunctionFlags |= EFunctionFlags.MulticastDelegate;
					}

					// Now parse the macro body
					topScope.TokenReader.Require('(');

					// Parse the return type
					UhtProperty? returnValueProperty = null;
					if (hasReturnValue)
					{
						UhtPropertyParser.Parse(topScope, EPropertyFlags.None,
							function.GetPropertyParseOptions(true), UhtPropertyCategory.Return,
							(UhtParsingScope topScope, UhtProperty property, ref UhtToken nameToken, UhtPropertyMacroType propertyMacroType) =>
							{
								property.PropertyFlags |= EPropertyFlags.Parm | EPropertyFlags.OutParm | EPropertyFlags.ReturnParm;
								returnValueProperty = property;
							});
						topScope.TokenReader.Require(',');
					}

					// Skip white spaces to get InputPos exactly on beginning of function name.
					topScope.TokenReader.SkipWhitespaceAndComments();

					// Get the delegate name
					UhtToken funcNameToken = topScope.TokenReader.GetIdentifier("name");
					function.SourceName = funcNameToken.Value.ToString();

					// If this is a delegate function then go ahead and mangle the name so we don't collide with
					// actual functions or properties
					{
						//@TODO: UCREMOVAL: Eventually this mangling shouldn't occur

						// Remove the leading F
						if (function.SourceName[0] != 'F')
						{
							topScope.TokenReader.LogError("Delegate type declarations must start with F");
						}
						function.StrippedFunctionName = function.SourceName[1..];
						function.EngineName = $"{function.StrippedFunctionName}{UhtFunction.GeneratedDelegateSignatureSuffix}";
					}

					SetFunctionNames(function);
					AddFunction(function);

					// determine whether this function should be 'const'
					if (declaredConst)
					{
						function.FunctionFlags |= EFunctionFlags.Const;
						function.FunctionExportFlags |= UhtFunctionExportFlags.DeclaredConst;
					}

					if (isSparse)
					{
						topScope.TokenReader.Require(',');
						UhtToken name = topScope.TokenReader.GetIdentifier("OwningClass specifier");
						UhtEngineNameParts parts = UhtUtilities.GetEngineNameParts(name.Value);
						function.SparseOwningClassName = parts.EngineName.ToString();
						topScope.TokenReader.Require(',');
						function.SparseDelegateName = topScope.TokenReader.GetIdentifier("delegate name").Value.ToString();
					}

					// Get parameter list.
					if (foundParamIndex >= 0)
					{
						topScope.TokenReader.Require(',');

						ParseParameterList(topScope, function.GetPropertyParseOptions(false));
					}
					else
					{
						// Require the closing paren even with no parameter list
						topScope.TokenReader.Require(')');
					}

					// Add back in the return value
					if (returnValueProperty != null)
					{
						topScope.ScopeType.AddChild(returnValueProperty);
					}

					// Verify the number of parameters (FoundParamIndex = -1 means zero parameters, 0 means one, ...)
					int expectedProperties = foundParamIndex + 1 + (hasReturnValue ? 1 : 0);
					int propertiesCount = function.Properties.Count();
					if (propertiesCount != expectedProperties)
					{
						throw new UhtException(topScope.TokenReader, $"Expected {expectedProperties} parameters but found {propertiesCount} parameters");
					}

					// The macro line must be set here
					function.MacroLineNumber = topScope.TokenReader.InputLine;

					// Try parsing metadata for the function
					specifiers.ParseFieldMetaData();

					topScope.AddFormattedCommentsAsTooltipMetaData();

					// Consume a semicolon, it's not required for the delegate macro since it contains one internally
					topScope.TokenReader.Require(';');
				}
				return UhtParseResult.Handled;
			}
		}

		private static UhtParseResult ParseUFunction(UhtParsingScope parentScope, UhtToken token)
		{
			UhtFunction function = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, token.InputLine);

			{
				using UhtParsingScope topScope = new(parentScope, function, parentScope.Session.GetKeywordTable(UhtTableNames.Function), UhtAccessSpecifier.Public);
				UhtParsingScope outerClassScope = topScope.CurrentClassScope;

				// Get the outer class - either directly or from partial's owner class
				UhtClass? outerClass = outerClassScope.ScopeType as UhtClass;
				UhtPartial? outerPartial = null;
				if (outerClass == null && outerClassScope.ScopeType is UhtPartial partial)
				{
					outerPartial = partial;
					outerClass = partial.OwnerClass;
				}

				// If we're not in a class or partial, that's an error
				if (outerClass == null && outerPartial == null)
				{
					throw new UhtException(function, "UFUNCTIONs must be declared in a class or partial");
				}

				string scopeName = "function";

				{
					using UhtMessageContext tokenContext = new(scopeName);
					topScope.AddModuleRelativePathToMetaData();

					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, function.MetaData);
					UhtSpecifierParser specifierParser = UhtSpecifierParser.GetThreadInstance(specifierContext, scopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.Function));
					specifierParser.ParseSpecifiers();

					// Only check native class flag if we have a resolved outer class (not an partial during parsing)
					if (outerClass != null && !outerClass.ClassFlags.HasAnyFlags(EClassFlags.Native))
					{
						throw new UhtException(function, "Should only be here for native classes!");
					}

					function.MacroLineNumber = topScope.TokenReader.InputLine;
					function.FunctionFlags |= EFunctionFlags.Native;

					bool automaticallyFinal = true;
					switch (outerClassScope.AccessSpecifier)
					{
						case UhtAccessSpecifier.Public:
							function.FunctionFlags |= EFunctionFlags.Public;
							break;

						case UhtAccessSpecifier.Protected:
							function.FunctionFlags |= EFunctionFlags.Protected;
							break;

						case UhtAccessSpecifier.Private:
							function.FunctionFlags |= EFunctionFlags.Private | EFunctionFlags.Final;

							// This is automatically final as well, but in a different way and for a different reason
							automaticallyFinal = false;
							break;
					}

					topScope.TokenReader.OptionalAttributes(false);

					if (topScope.TokenReader.TryOptional("static"))
					{
						function.FunctionFlags |= EFunctionFlags.Static;
						function.FunctionExportFlags |= UhtFunctionExportFlags.CppStatic;
					}

					if (function.MetaData.ContainsKey(UhtNames.CppFromBpEvent))
					{
						function.FunctionFlags |= EFunctionFlags.Event;
					}

					UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
					if ((compilerDirective & UhtCompilerDirective.WithEditor) != 0)
					{
						function.FunctionFlags |= EFunctionFlags.EditorOnly;
						function.DefineScope |= UhtDefineScope.Editor;
					}
					else if ((compilerDirective & UhtCompilerDirective.WithEditorOnlyData) != 0)
					{
						function.FunctionFlags |= EFunctionFlags.EditorOnly;
						function.DefineScope |= UhtDefineScope.EditorOnlyData;
					}
					function.DefineScope |= (compilerDirective & ~(UhtCompilerDirective.WithEditorOnlyData | UhtCompilerDirective.WithEditor)).GetDefaultDefineScopes();

					specifierParser.ParseDeferred();
					FinalizeFunctionSpecifiers(function);

					if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CustomThunk) && !function.MetaData.ContainsKey(UhtNames.CustomThunk))
					{
						function.MetaData.Add(UhtNames.CustomThunk, true);
					}

					if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
					{
						// Network replicated functions are always events, and are only final if sealed
						scopeName = "event";
						tokenContext.Reset(scopeName);
						automaticallyFinal = false;
					}

					if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
					{
						scopeName = function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native) ? "BlueprintNativeEvent" : "BlueprintImplementableEvent";
						tokenContext.Reset(scopeName);
						automaticallyFinal = false;
					}

					// Record the tokens so we can detect this function as a declaration later (i.e. RPC)
					{
						using UhtTokenRecorder tokenRecorder = new(parentScope, function);

						if (topScope.TokenReader.TryOptional("virtual"))
						{
							function.FunctionExportFlags |= UhtFunctionExportFlags.Virtual;
						}

						bool internalOnly = function.MetaData.GetBoolean(UhtNames.BlueprintInternalUseOnly);

						// Peek ahead to look for a CORE_API style DLL import/export token if present
						if (topScope.TokenReader.TryOptionalAPIMacro(out UhtToken apiMacroToken))
						{
							//@TODO: Validate the module name for RequiredAPIMacroIfPresent
							function.FunctionFlags |= EFunctionFlags.RequiredAPI;
							function.FunctionExportFlags |= UhtFunctionExportFlags.RequiredAPI;
						}

						// Look for static again, in case there was an ENGINE_API token first
						if (apiMacroToken && topScope.TokenReader.TryOptional("static"))
						{
							topScope.TokenReader.LogError($"Unexpected API macro '{apiMacroToken.Value}'. Did you mean to put '{apiMacroToken.Value}' after the static keyword?");
						}

						// Look for virtual again, in case there was an ENGINE_API token first
						if (topScope.TokenReader.TryOptional("virtual"))
						{
							function.FunctionExportFlags |= UhtFunctionExportFlags.Virtual;
						}

						// If virtual, remove the implicit final, the user can still specifying an explicit final at the end of the declaration
						if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Virtual))
						{
							automaticallyFinal = false;
						}

						// Handle the initial implicit/explicit final
						// A user can still specify an explicit final after the parameter list as well.
						if (automaticallyFinal || function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.SealedEvent))
						{
							function.FunctionFlags |= EFunctionFlags.Final;
							function.FunctionExportFlags |= UhtFunctionExportFlags.Final | UhtFunctionExportFlags.AutoFinal;
						}

						// Get return type.  C++ style functions always have a return value type, even if it's void
						UhtToken funcNameToken = new();
						UhtProperty? returnValueProperty = null;
						bool firstArgIsReturnValue = false;
						if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseCallable))
						{
							if (topScope.TokenReader.TryOptional("FVerseResult"))
							{
								firstArgIsReturnValue = true;
								function.FunctionExportFlags |= UhtFunctionExportFlags.VerseSuspends;
								funcNameToken = topScope.TokenReader.GetIdentifier();
							}
						}
						if (!firstArgIsReturnValue)
						{
							UhtPropertyParser.Parse(topScope, EPropertyFlags.None,
								function.GetPropertyParseOptions(true), UhtPropertyCategory.Return,
								(UhtParsingScope topScope, UhtProperty property, ref UhtToken nameToken, UhtPropertyMacroType propertyMacroType) =>
								{
									property.PropertyFlags |= EPropertyFlags.Parm | EPropertyFlags.OutParm | EPropertyFlags.ReturnParm;
									funcNameToken = nameToken;
									if (property is not UhtVoidProperty)
									{
										returnValueProperty = property;
									}
									if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseTask))
									{
										function.FunctionExportFlags |= UhtFunctionExportFlags.VerseSuspends;
									}
								});
						}

						if (funcNameToken.Value.Length == 0)
						{
							throw new UhtException(topScope.TokenReader, "expected return value and function name");
						}

						// Get function or operator name.
						function.SourceName = funcNameToken.Value.ToString();

						// If the function is a verse function, then we need to apply the source name to the verse name
						if (function.IsVerseField && String.IsNullOrEmpty(function.VerseName))
						{
							function.VerseName = function.SourceName;
						}

						scopeName = $"{scopeName} '{function.SourceName}'";
						tokenContext.Reset(scopeName);

						topScope.TokenReader.Require('(');

						SetFunctionNames(function);
						AddFunction(function);

						// Get parameter list.
						ParseParameterList(topScope, function.GetPropertyParseOptions(false));

						// If we are parsing a verse callable coroutine, then treat the first argument as the return value
						if (firstArgIsReturnValue && function.Children.Count > 1)
						{
							UhtProperty firstProperty = (UhtProperty)function.Children[0];
							function.Children.RemoveAt(0);
							if (!firstProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseCall))
							{
								firstProperty.LogError("The first argument for a verse_callable coroutine must be TVerseCall<ReturnType>");
							}
							else if (firstProperty is not UhtVoidProperty)
							{
								firstProperty.PropertyFlags |= EPropertyFlags.Parm | EPropertyFlags.OutParm | EPropertyFlags.ReturnParm;
								returnValueProperty = firstProperty;
							}
						}

						// Add back in the return value
						if (returnValueProperty != null)
						{
							topScope.ScopeType.AddChild(returnValueProperty);
						}

						// determine whether this function should be 'const'
						if (topScope.TokenReader.TryOptional("const"))
						{
							if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
							{
								// @TODO: UCREMOVAL Reconsider?
								//Throwf(TEXT("'const' may only be used for native functions"));
							}

							function.FunctionFlags |= EFunctionFlags.Const;
							function.FunctionExportFlags |= UhtFunctionExportFlags.DeclaredConst;
						}

						// Try parsing metadata for the function
						specifierParser.ParseFieldMetaData();

						// COMPATIBILITY-TODO - Try to pull any comment following the declaration
						topScope.TokenReader.PeekToken();
						topScope.TokenReader.CommitPendingComments();

						topScope.AddFormattedCommentsAsTooltipMetaData();

						// 'final' and 'override' can appear in any order before an optional '= 0' pure virtual specifier
						bool foundFinal = topScope.TokenReader.TryOptional("final");
						bool foundOverride = topScope.TokenReader.TryOptional("override");
						if (!foundFinal && foundOverride)
						{
							foundFinal = topScope.TokenReader.TryOptional("final");
						}

						// Handle C++ style functions being declared as abstract
						if (topScope.TokenReader.TryOptional('='))
						{
							bool gotZero = topScope.TokenReader.TryOptionalConstInt(out int zeroValue);
							gotZero = gotZero && (zeroValue == 0);
							if (!gotZero || zeroValue != 0)
							{
								throw new UhtException(topScope.TokenReader, "Expected 0 to indicate function is abstract");
							}
						}

						// Look for the final keyword to indicate this function is sealed
						if (foundFinal)
						{
							// This is a final (prebinding, non-overridable) function
							function.FunctionFlags |= EFunctionFlags.Final;
							function.FunctionExportFlags |= UhtFunctionExportFlags.Final;
						}

						// Optionally consume a semicolon
						// This is optional to allow inline function definitions
						if (topScope.TokenReader.TryOptional(';'))
						{
							// Do nothing (consume it)
						}
						else if (topScope.TokenReader.TryPeekOptional('{'))
						{
							// Skip inline function bodies
							topScope.SkipDeclaration(new());
						}
					}
				}
				return UhtParseResult.Handled;
			}
		}

		private static void FinalizeFunctionSpecifiers(UhtFunction function)
		{
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				// Network replicated functions are always events
				function.FunctionFlags |= EFunctionFlags.Event;
			}
		}

		internal static bool IsValidateDelegateDeclaration(UhtToken token)
		{
			return (token.IsIdentifier() && token.Value.Span.StartsWith("DECLARE_DYNAMIC_", StringComparison.Ordinal));
		}

		private static void ParseParameterList(UhtParsingScope topScope, UhtPropertyParseOptions options)
		{
			UhtFunction function = (UhtFunction)topScope.ScopeType;

			bool isNetFunc = function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net);
			UhtPropertyCategory propertyCategory = isNetFunc ? UhtPropertyCategory.ReplicatedParameter : UhtPropertyCategory.RegularParameter;
			EPropertyFlags disallowFlags = ~(EPropertyFlags.ParmFlags | EPropertyFlags.AutoWeak | EPropertyFlags.RepSkip | EPropertyFlags.UObjectWrapper | EPropertyFlags.NativeAccessSpecifiers);

			UhtAdvancedDisplayParameterHandler advancedDisplay = new(topScope.ScopeType.MetaData);

			topScope.TokenReader.RequireList(')', ',', false, () =>
			{
				UhtPropertyParser.Parse(topScope, disallowFlags, options, propertyCategory,
					(UhtParsingScope topScope, UhtProperty property, ref UhtToken nameToken, UhtPropertyMacroType propertyMacroType) =>
					{
						property.PropertyFlags |= EPropertyFlags.Parm;
						if (advancedDisplay.CanMarkMore() && advancedDisplay.ShouldMarkParameter(property.EngineName))
						{
							property.PropertyFlags |= EPropertyFlags.AdvancedDisplay;
						}

						// Default value.
						if (topScope.TokenReader.TryOptional('='))
						{
							List<UhtToken> defaultValueTokens = [];
							int parenthesisNestCount = 0;
							while (!topScope.TokenReader.IsEOF)
							{
								UhtToken token = topScope.TokenReader.PeekToken();
								if (token.IsSymbol(','))
								{
									if (parenthesisNestCount == 0)
									{
										break;
									}
									defaultValueTokens.Add(token);
									topScope.TokenReader.ConsumeToken();
								}
								else if (token.IsSymbol(')'))
								{
									if (parenthesisNestCount == 0)
									{
										break;
									}
									defaultValueTokens.Add(token);
									topScope.TokenReader.ConsumeToken();
									--parenthesisNestCount;
								}
								else if (token.IsSymbol('('))
								{
									++parenthesisNestCount;
									defaultValueTokens.Add(token);
									topScope.TokenReader.ConsumeToken();
								}
								else
								{
									defaultValueTokens.Add(token);
									topScope.TokenReader.ConsumeToken();
								}
							}

							if (defaultValueTokens.Count > 0)
							{
								property.DefaultValueTokens = defaultValueTokens;
							}
						}
					});
			});
		}

		private static void AddFunction(UhtFunction function)
		{
			function.Outer?.AddChild(function);
		}

		private static void SetFunctionNames(UhtFunction function)
		{
			// The source name won't have the suffix applied to delegate names, however, the engine name will
			// We use the engine name because we need to detect the suffix for delegates
			string functionName = function.EngineName;
			if (functionName.EndsWith(UhtFunction.GeneratedDelegateSignatureSuffix, StringComparison.Ordinal))
			{
				functionName = functionName[..^UhtFunction.GeneratedDelegateSignatureSuffix.Length];
			}

			function.UnMarshalAndCallName = "exec" + functionName;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				function.MarshalAndCallName = functionName;
				if (function.FunctionFlags.HasAllFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.Native))
				{
					function.CppImplName = function.EngineName + "_Implementation";
				}
			}
			else if (function.FunctionFlags.HasAllFlags(EFunctionFlags.Native | EFunctionFlags.Net))
			{
				function.MarshalAndCallName = functionName;
				if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetResponse))
				{
					// Response function implemented by programmer and called directly from thunk
					function.CppImplName = function.EngineName;
				}
				else
				{
					if (function.CppImplName.Length == 0)
					{
						function.CppImplName = function.EngineName + "_Implementation";
					}
					else if (function.CppImplName == functionName)
					{
						function.LogError("Native implementation function must be different than original function name.");
					}

					if (function.CppValidationImplName.Length == 0 && function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate))
					{
						function.CppValidationImplName = function.EngineName + "_Validate";
					}
					else if (function.CppValidationImplName == functionName)
					{
						function.LogError("Validation function must be different than original function name.");
					}
				}
			}
			else if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
			{
				function.MarshalAndCallName = "delegate" + functionName;
			}

			if (function.CppImplName.Length == 0)
			{
				function.CppImplName = functionName;
			}

			if (function.MarshalAndCallName.Length == 0)
			{
				function.MarshalAndCallName = "event" + functionName;
			}
		}
	}
}
