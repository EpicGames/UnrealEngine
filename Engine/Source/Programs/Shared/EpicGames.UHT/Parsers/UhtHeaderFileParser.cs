// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Keyword parse results
	/// </summary>
	public enum UhtParseResult
	{

		/// <summary>
		/// Keyword was handled
		/// </summary>
		Handled,

		/// <summary>
		/// Keyword wasn't handled (more attempts will be made to match)
		/// </summary>
		Unhandled,

		/// <summary>
		/// Keyword is invalid
		/// </summary>
		Invalid,
	}

	/// <summary>
	/// Compiler directives
	/// </summary>
	[Flags]
	[SuppressMessage("Usage", "CA2217:Do not mark enums with FlagsAttribute")]
	public enum UhtCompilerDirective
	{
		/// <summary>
		/// No compile directives
		/// </summary>
		None = 0,

		/// <summary>
		/// This indicates we are in a "#if CPP" block
		/// </summary>
		CPPBlock = 1 << 0,

		/// <summary>
		/// This indicates we are in a "#if !CPP" block
		/// </summary>
		NotCPPBlock = 1 << 1,

		/// <summary>
		/// This indicates we are in a "#if 0" block
		/// </summary>
		ZeroBlock = 1 << 2,

		/// <summary>
		/// This indicates we are in a "#if 1" block
		/// </summary>
		OneBlock = 1 << 3,

		/// <summary>
		/// This indicates we are in a "#if WITH_EDITOR" block
		/// </summary>
		WithEditor = 1 << 4,

		/// <summary>
		/// This indicates we are in a "#if WITH_EDITORONLY_DATA" block
		/// </summary>
		WithEditorOnlyData = 1 << 5,

		/// <summary>
		/// This indicates we are in a "#if WITH_HOT_RELOAD" block
		/// </summary>
		WithHotReload = 1 << 6,

		/// <summary>
		/// This indicates we are in a "#if WITH_ENGINE" block
		/// </summary>
		WithEngine = 1 << 7,

		/// <summary>
		/// This indicates we are in a "#if WITH_COREUOBJECT" block
		/// </summary>
		WithCoreUObject = 1 << 8,

		/// <summary>
		/// This indicates we are in a "#if WITH_VERSE_VM" block
		/// </summary>
		WithVerseVM = 1 << 9,

		/// <summary>
		/// This directive is unrecognized and does not change the code generation at all
		/// </summary>
		Unrecognized = 1 << 10,

		/// <summary>
		/// This indicates we are in a "#if WITH_TESTS" block
		/// </summary>
		WithTests = 1 << 11,

		/// <summary>
		/// This indicates we are in a "#if WITH_VERSE_BPVM" block
		/// </summary>
		WithVerseBPVM = 1 << 12,

		/// <summary>
		/// The following flags are always ignored when keywords test for allowed conditional blocks
		/// </summary>
		AllowedCheckIgnoredFlags = CPPBlock | NotCPPBlock | ZeroBlock | OneBlock | WithHotReload,

		/// <summary>
		/// Default compiler directives to be allowed
		/// </summary>
		DefaultAllowedCheck = WithEditor | WithEditorOnlyData | WithVerseVM | WithTests | WithVerseBPVM,

		/// <summary>
		/// All flags are allowed
		/// </summary>
		SilenceAllowedCheck = ~None,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtCompilerDirectiveExtensions
	{
		private static readonly Lazy<List<string>> s_names = new(() =>
		{
			List<string> outList = [];
			FieldInfo[] fields = typeof(UhtCompilerDirective).GetFields();
			for (int bit = 0; bit < 32; ++bit)
			{
				bool found = false;
				uint mask = (uint)1 << bit;
				foreach (FieldInfo field in fields)
				{
					if (field.IsSpecialName)
					{
						continue;
					}
					object? value = field.GetValue(null);
					if (value != null)
					{
						if (mask == (uint)value)
						{
							outList.Add(GetCompilerDirectiveText((UhtCompilerDirective)value));
							found = true;
							break;
						}
					}
				}
				if (!found)
				{
					outList.Add($"0x{mask:X8}");
				}
			}
			return outList;
		});

		/// <summary>
		/// Return the text associated with the given compiler directive
		/// </summary>
		/// <param name="compilerDirective">Directive in question</param>
		/// <returns>String representation</returns>
		public static string GetCompilerDirectiveText(this UhtCompilerDirective compilerDirective)
		{
			switch (compilerDirective)
			{
				case UhtCompilerDirective.CPPBlock: return "CPP";
				case UhtCompilerDirective.NotCPPBlock: return "!CPP";
				case UhtCompilerDirective.ZeroBlock: return "0";
				case UhtCompilerDirective.OneBlock: return "1";
				case UhtCompilerDirective.WithHotReload: return UhtNames.WithHotReload;
				case UhtCompilerDirective.WithEditor: return UhtNames.WithEditor;
				case UhtCompilerDirective.WithEditorOnlyData: return UhtNames.WithEditorOnlyData;
				case UhtCompilerDirective.WithEngine: return UhtNames.WithEngine;
				case UhtCompilerDirective.WithCoreUObject: return UhtNames.WithCoreUObject;
				case UhtCompilerDirective.WithVerseVM: return UhtNames.WithVerseVM;
				case UhtCompilerDirective.WithVerseBPVM: return UhtNames.WithVerseBPVM;
				case UhtCompilerDirective.WithTests: return UhtNames.WithTests;
				default: return "<unrecognized>";
			}
		}

		/// <summary>
		/// Return a define scope based on the given compiler directives
		/// </summary>
		/// <param name="compilerDirectives">Directives in question</param>
		/// <returns>Matching define scope</returns>
		public static UhtDefineScope GetDefaultDefineScopes(this UhtCompilerDirective compilerDirectives)
		{
			UhtDefineScope defineScope = UhtDefineScope.None;
			if (compilerDirectives.HasAnyFlags(UhtCompilerDirective.WithTests))
			{
				defineScope |= UhtDefineScope.Tests;
			}
			if (compilerDirectives.HasAnyFlags(UhtCompilerDirective.WithVerseBPVM))
			{
				defineScope |= UhtDefineScope.VerseBPVM;
			}
			if (compilerDirectives.HasAnyFlags(UhtCompilerDirective.WithVerseVM))
			{
				defineScope |= UhtDefineScope.VerseVM;
			}
			if (compilerDirectives.HasAnyFlags(UhtCompilerDirective.WithEditorOnlyData))
			{
				defineScope |= UhtDefineScope.EditorOnlyData;
			}
			return defineScope;
		}

		/// <summary>
		/// Return a string list of the given compiler directives
		/// </summary>
		/// <param name="inFlags"></param>
		/// <returns></returns>
		public static List<string> ToStringList(this UhtCompilerDirective inFlags)
		{
			List<string> names = s_names.Value;
			ulong intFlags = (ulong)inFlags;
			List<string> outList = [];
			for (int bit = 0; bit < 32; ++bit)
			{
				ulong mask = (ulong)1 << bit;
				if (mask > intFlags)
				{
					break;
				}
				if ((mask & intFlags) != 0)
				{
					outList.Add(names[bit]);
				}
			}
			return outList;
		}

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtCompilerDirective inFlags, UhtCompilerDirective testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtCompilerDirective inFlags, UhtCompilerDirective testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtCompilerDirective inFlags, UhtCompilerDirective testFlags, UhtCompilerDirective matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Specifiers for public, private, and protected
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtAccessSpecifierKeywords
	{
		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.ClassBase, Keyword = "public", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[UhtKeyword(Extends = UhtTableNames.ScriptStruct, Keyword = "public", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult PublicKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return SetAccessSpecifier(topScope, UhtAccessSpecifier.Public);
		}

		[UhtKeyword(Extends = UhtTableNames.ClassBase, Keyword = "protected", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[UhtKeyword(Extends = UhtTableNames.ScriptStruct, Keyword = "protected", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult ProtectedKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return SetAccessSpecifier(topScope, UhtAccessSpecifier.Protected);
		}

		[UhtKeyword(Extends = UhtTableNames.ClassBase, Keyword = "private", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[UhtKeyword(Extends = UhtTableNames.ScriptStruct, Keyword = "private", AllowedCompilerDirectives = UhtCompilerDirective.SilenceAllowedCheck)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult PrivateKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return SetAccessSpecifier(topScope, UhtAccessSpecifier.Private);
		}
		#endregion

		private static UhtParseResult SetAccessSpecifier(UhtParsingScope topScope, UhtAccessSpecifier accessSpecifier)
		{
			topScope.AccessSpecifier = accessSpecifier;
			topScope.TokenReader.Require(':');
			return UhtParseResult.Handled;
		}
	}

	/// <summary>
	/// Header file parser
	/// </summary>
	public class UhtHeaderFileParser : IUhtTokenPreprocessor
	{
		private struct CompilerDirective
		{
			public UhtCompilerDirective _element;
			public UhtCompilerDirective _composite;
		}

		/// <summary>
		/// Header file being parsed
		/// </summary>
		public UhtHeaderFile HeaderFile { get; }

		/// <summary>
		/// Module containing the header file
		/// </summary>
		public UhtModule Module => HeaderFile.Module;

		/// <summary>
		/// Token reader for the header
		/// </summary>
		public IUhtTokenReader TokenReader { get; }

		/// <summary>
		/// If true, the inclusion of the generated header file was seen
		/// </summary>
		public bool SpottedAutogeneratedHeaderInclude { get; set; } = false;

		/// <summary>
		/// If set, the preprocessor is run in a C++ UHT compatibility mode where only a subset
		/// of #if class of preprocessor statements are allowed.
		/// </summary>
		public string? RestrictedPreprocessorContext { get; set; } = null;

		/// <summary>
		/// Depth of the namespaces
		/// </summary>
		public int NamespaceDepth => _namespaces.Count;

		/// <summary>
		/// Stack of current #if states
		/// </summary>
		private readonly List<CompilerDirective> _compilerDirectives = [];

		/// <summary>
		/// Stack of current #if states saved as part of the preprocessor state
		/// </summary>
		private readonly List<CompilerDirective> _savedCompilerDirectives = [];

		/// <summary>
		/// Current top of the parsing scopes.  Classes, structures and functions all allocate scopes.
		/// </summary>
		private UhtParsingScope? _topScope = null;

		/// <summary>
		/// Current namespaces being parsed
		/// </summary>
		private readonly List<UhtNamespaceElement> _namespaces = [];

		/// <summary>
		/// Lookup cache for any namespaces defined in this header.
		/// </summary>
		private readonly Dictionary<string, UhtNamespace> _namespaceLookupCache = [];

		/// <summary>
		/// Parse the given header file
		/// </summary>
		/// <param name="headerFile">Header file to parse</param>
		/// <returns>Parser</returns>
		public static UhtHeaderFileParser Parse(UhtHeaderFile headerFile)
		{
			UhtHeaderFileParser headerParser = new(headerFile);
			using UhtParsingScope topScope = new(headerParser, headerFile.Session.GetKeywordTable(UhtTableNames.Global));
			headerParser.ParseStatements();

			if (!headerParser.SpottedAutogeneratedHeaderInclude && headerParser.HeaderFile.Data.Length > 0)
			{
				bool noExportClassesOnly = true;
				bool missingGeneratedHeader = false;
				foreach (UhtType type in headerParser.HeaderFile.Children)
				{
					if (type is UhtClass classObj)
					{
						if (classObj.ClassType != UhtClassType.NativeInterface && !classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
						{
							noExportClassesOnly = false;
							break;
						}
					}
					else if (!headerParser.HeaderFile.IsNoExportTypes)
					{
						missingGeneratedHeader = true;
					}
				}

				string logMessage = $"The given include must appear at the top of the header following all other includes: '#include \"{headerParser.HeaderFile.GeneratedHeaderFileName}\"'";
				if (!noExportClassesOnly)
				{
					headerParser.HeaderFile.LogError(logMessage);
				}

				if (missingGeneratedHeader)
				{
					switch (headerFile.Session.Config!.MissingGeneratedHeaderIncludeBehavior.GetBehavior(headerFile.Module))
					{
						case UhtIssueBehavior.AllowSilently:
							break;

						case UhtIssueBehavior.AllowAndLog:
							headerParser.HeaderFile.LogTrace(logMessage);
							break;

						default:
							headerParser.HeaderFile.LogError(logMessage);
							break;
					}
				}
			}
			return headerParser;
		}

		private UhtHeaderFileParser(UhtHeaderFile headerFile)
		{
			UhtNamespace globalNamespace = headerFile.Session.GlobalNamespace;
			TokenReader = new UhtTokenBufferReader(headerFile, headerFile.Data.Memory);
			HeaderFile = headerFile;
			TokenReader.TokenPreprocessor = this;
			_namespaces.Add(new("", globalNamespace));
			_namespaceLookupCache.Add("", globalNamespace);
		}

		/// <summary>
		/// Push a new scope
		/// </summary>
		/// <param name="scope">Scope to push</param>
		/// <exception cref="UhtIceException">Throw if the new scope isn't parented by the current scope</exception>
		public void PushScope(UhtParsingScope scope)
		{
			if (scope.ParentScope != _topScope)
			{
				throw new UhtIceException("Pushing a new scope whose parent isn't the current top scope.");
			}
			_topScope = scope;
		}

		/// <summary>
		/// Pop the given scope
		/// </summary>
		/// <param name="scope">Scope to be popped</param>
		/// <exception cref="UhtIceException">Thrown if the given scope isn't the top scope</exception>
		public void PopScope(UhtParsingScope scope)
		{
			if (scope != _topScope)
			{
				throw new UhtIceException("Attempt to pop a scope that isn't the top scope");
			}
			_topScope = scope.ParentScope;
		}

		/// <summary>
		/// Add a namespace to the token list
		/// </summary>
		/// <param name="token">Namespace being added</param>
		public void PushNamespace(UhtToken token)
		{
			_namespaces.Add(new(token.Value, null));
		}

		/// <summary>
		/// Remove the given number of namespaces
		/// </summary>
		/// <param name="count"></param>
		public void PopNamespaces(int count)
		{
			if (count > _namespaces.Count)
			{
				throw new UhtException(TokenReader, "Mismatch when looking for closing brace");
			}
			_namespaces.RemoveRange(_namespaces.Count - count, count);
		}

		/// <summary>
		/// Returns the currently active namespace
		/// </summary>
		/// <returns></returns>
		public UhtNamespace GetNamespace()
		{
			if (_topScope == null)
			{
				throw new UhtIceException("Expected a top scope");
			}
			return _topScope.Session.GetNamespace(_namespaces, _namespaceLookupCache);
		}

		/// <summary>
		/// Return the current compiler directive
		/// </summary>
		/// <returns>Enumeration flags for all active compiler directives</returns>
		public UhtCompilerDirective GetCurrentCompositeCompilerDirective()
		{
			return _compilerDirectives.Count > 0 ? _compilerDirectives[^1]._composite : UhtCompilerDirective.None;
		}

		/// <summary>
		/// Get the current compiler directive without any parent scopes merged in
		/// </summary>
		/// <returns>Current compiler directive</returns>
		public UhtCompilerDirective GetCurrentNonCompositeCompilerDirective()
		{
			return _compilerDirectives.Count > 0 ? _compilerDirectives[^1]._element : UhtCompilerDirective.None;
		}

		#region ITokenPreprocessor implementation
		/// <inheritdoc/>
		public bool ParsePreprocessorDirective(ref UhtToken token, bool isBeingIncluded, out bool clearComments, out bool illegalContentsCheck)
		{
			clearComments = true;
			if (ParseDirectiveInternal(isBeingIncluded))
			{
				clearComments = ClearCommentsCompilerDirective();
			}
			illegalContentsCheck = !GetCurrentNonCompositeCompilerDirective().HasAnyFlags(UhtCompilerDirective.ZeroBlock | UhtCompilerDirective.WithEditorOnlyData);
			return IncludeCurrentCompilerDirective();
		}

		/// <inheritdoc/>
		public void SaveState()
		{
			_savedCompilerDirectives.Clear();
			_savedCompilerDirectives.AddRange(_compilerDirectives);
		}

		/// <inheritdoc/>
		public void RestoreState()
		{
			_compilerDirectives.Clear();
			_compilerDirectives.AddRange(_savedCompilerDirectives);
		}
		#endregion

		#region Statement parsing

		/// <summary>
		/// Parse all statements in the header file
		/// </summary>
		public void ParseStatements()
		{
			ParseStatements((char)0, (char)0, true);
		}

		/// <summary>
		/// Parse the statements between the given symbols
		/// </summary>
		/// <param name="initiator">Starting symbol</param>
		/// <param name="terminator">Ending symbol</param>
		/// <param name="logUnhandledKeywords">If true, log any unhandled keywords</param>
		public void ParseStatements(char initiator, char terminator, bool logUnhandledKeywords)
		{
			if (_topScope == null)
			{
				return;
			}

			if (initiator != 0)
			{
				TokenReader.Require(initiator);
			}
			while (true)
			{
				UhtToken token = TokenReader.GetToken();
				if (token.TokenType.IsEndType())
				{
					if (_topScope != null && _topScope.ParentScope == null)
					{
						CheckEof(ref token);
					}
					return;
				}
				else if (terminator != 0 && token.IsSymbol(terminator))
				{
					return;
				}

				if (_topScope != null)
				{
					ParseStatement(_topScope, ref token, logUnhandledKeywords);
					TokenReader.ClearComments();
				}
			}
		}

		/// <summary>
		/// Parse a statement
		/// </summary>
		/// <param name="topScope">Current top scope</param>
		/// <param name="token">Token starting the statement</param>
		/// <param name="logUnhandledKeywords">If true, log unhandled keywords</param>
		/// <returns>Always returns true ATM</returns>
		public static bool ParseStatement(UhtParsingScope topScope, ref UhtToken token, bool logUnhandledKeywords)
		{
			UhtParseResult parseResult = UhtParseResult.Unhandled;

			switch (token.TokenType)
			{
				case UhtTokenType.Identifier:
					parseResult = DispatchKeyword(topScope, ref token);
					break;

				case UhtTokenType.Symbol:
					// C++ UHT TODO - Generate errors when extra ';' are found.  
					// UPROPERTY has code to remove extra ';'
					if (token.IsSymbol(';'))
					{
						UhtToken nextToken = topScope.TokenReader.PeekToken();
						if (nextToken.TokenType.IsEndType())
						{
							topScope.TokenReader.LogError("Extra ';' before end of file");
						}
						else
						{
							topScope.TokenReader.LogError($"Extra ';' before '{nextToken}'");
						}
						return true;
					}
					parseResult = DispatchKeyword(topScope, ref token);
					break;
			}

			if (parseResult == UhtParseResult.Unhandled)
			{
				parseResult = DispatchCatchAll(topScope, ref token);
			}

			if (parseResult == UhtParseResult.Unhandled)
			{
				parseResult = ProbablyAnUnknownObjectLikeMacro(topScope.TokenReader, ref token);
			}

			if (parseResult == UhtParseResult.Unhandled && logUnhandledKeywords)
			{
				topScope.Session.LogUnhandledKeywordError(topScope.TokenReader, token);
			}

			if (parseResult == UhtParseResult.Unhandled || parseResult == UhtParseResult.Invalid)
			{
				if (topScope.ScopeType is UhtClass classObj)
				{
					using UhtTokenRecorder recorder = new(topScope, ref token);
					topScope.SkipDeclaration(token);
					if (recorder.Stop())
					{
						if (classObj.Declarations != null)
						{
							UhtDeclaration declaration = classObj.Declarations[classObj.Declarations.Count - 1];
							if (topScope.HeaderParser.CheckForConstructor(classObj, declaration))
							{
							}
							else if (topScope.HeaderParser.CheckForDestructor(classObj, declaration))
							{
							}
							else if (classObj.ClassType == UhtClassType.Class)
							{
								if (topScope.HeaderParser.CheckForSerialize(classObj, declaration))
								{
								}
							}
						}
					}
				}
				else if (topScope.ScopeType is UhtPartial partial)
				{
					using UhtTokenRecorder recorder = new(topScope, ref token);
					topScope.SkipDeclaration(token);
					if (recorder.Stop())
					{
						if (partial.Declarations != null)
						{
							UhtDeclaration declaration = partial.Declarations[partial.Declarations.Count - 1];
							if (topScope.HeaderParser.CheckForBeginDestroy(partial, declaration))
							{
							}
							// Add AddReferencedObjects, PostLoad, GetLifetimeReplicatedProps and friends later for feature complete
						}
					}
				}
				else
				{
					topScope.SkipDeclaration(token);
				}
			}
			return true;
		}

		private static UhtParseResult DispatchKeyword(UhtParsingScope topScope, ref UhtToken token)
		{
			UhtParseResult parseResult = UhtParseResult.Unhandled;
			for (UhtParsingScope? currentScope = topScope; currentScope != null && parseResult == UhtParseResult.Unhandled; currentScope = currentScope.ParentScope)
			{
				if (currentScope.ScopeKeywordTable.TryGetValue(token.Value, out UhtKeyword keywordInfo))
				{
					if (keywordInfo.AllScopes || topScope == currentScope)
					{
						UhtCompilerDirective currentDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective() & ~UhtCompilerDirective.AllowedCheckIgnoredFlags;
						if (currentDirective.HasAnyFlags(~keywordInfo.AllowedCompilerDirectives))
						{
							List<string> strings = keywordInfo.AllowedCompilerDirectives.ToStringList();
							string directives = UhtUtilities.MergeTypeNames(strings, "or", false);
							topScope.TokenReader.LogError($"'{token.Value}' must not be inside preprocessor blocks, except for {directives}");
						}
						parseResult = keywordInfo.Delegate(topScope, currentScope, ref token);
					}
				}
			}
			return parseResult;
		}

		private static UhtParseResult DispatchCatchAll(UhtParsingScope topScope, ref UhtToken token)
		{
			for (UhtParsingScope? currentScope = topScope; currentScope != null; currentScope = currentScope.ParentScope)
			{
				foreach (UhtKeywordCatchAllDelegate catchAll in currentScope.ScopeKeywordTable.CatchAlls)
				{
					UhtParseResult parseResult = catchAll(topScope, ref token);
					if (parseResult != UhtParseResult.Unhandled)
					{
						return parseResult;
					}
				}
			}
			return UhtParseResult.Unhandled;
		}

		/// <summary>
		/// Tests if an identifier looks like a macro which doesn't have a following open parenthesis.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="token">The current token that initiated the process</param>
		/// <returns>Result if matching the token</returns>
		private static UhtParseResult ProbablyAnUnknownObjectLikeMacro(IUhtTokenReader tokenReader, ref UhtToken token)
		{
			// Non-identifiers are not macros
			if (!token.IsIdentifier())
			{
				return UhtParseResult.Unhandled;
			}

			// Macros must start with a capitalized alphanumeric character or underscore
			char firstChar = token.Value.Span[0];
			if (firstChar != '_' && (firstChar < 'A' || firstChar > 'Z'))
			{
				return UhtParseResult.Unhandled;
			}

			// We'll guess about it being a macro based on it being fully-capitalized with at least one underscore.
			int underscoreCount = 0;
			foreach (char ch in token.Value.Span[1..])
			{
				if (ch == '_')
				{
					++underscoreCount;
				}
				else if ((ch < 'A' || ch > 'Z') && (ch < '0' || ch > '9'))
				{
					return UhtParseResult.Unhandled;
				}
			}

			// We look for at least one underscore as a convenient way of allowing many known macros
			// like FORCEINLINE and CONSTEXPR, and non-macros like FPOV and TCHAR.
			if (underscoreCount == 0)
			{
				return UhtParseResult.Unhandled;
			}

			// Identifiers which end in _API are known
			if (token.Value.Span.Length > 4 && token.Value.Span.EndsWith("_API", StringComparison.Ordinal))
			{
				return UhtParseResult.Unhandled;
			}

			// Ignore certain known macros or identifiers that look like macros.
			if (token.IsValue("FORCEINLINE_DEBUGGABLE") ||
				token.IsValue("FORCEINLINE_STATS") ||
				token.IsValue("SIZE_T"))
			{
				return UhtParseResult.Unhandled;
			}

			// Check if there's an open parenthesis following the token.
			return tokenReader.PeekToken().IsSymbol('(') ? UhtParseResult.Unhandled : UhtParseResult.Handled;
		}

		private void CheckEof(ref UhtToken token)
		{
			if (_compilerDirectives.Count > 0)
			{
				throw new UhtException(TokenReader, token.InputLine, "Missing #endif");
			}
		}
		#endregion

		#region Internals
		/// <summary>
		/// Parse a preprocessor directive.
		/// </summary>
		/// <param name="isBeingIncluded">If true, then this directive is in an active block</param>
		/// <returns>True if we should check to see if tokenizer should clear comments</returns>
		private bool ParseDirectiveInternal(bool isBeingIncluded)
		{
			bool checkClearComments = false;

			// Collect all the lines of the preprocessor statement including any continuations.
			// We assume that the vast majority of lines will not be continuations.  So avoid using the
			// string builder as much as possible.
			int startingLine = TokenReader.InputLine;
			StringViewBuilder builder = new();
			while (true)
			{
				UhtToken lineToken = TokenReader.GetLine();
				if (lineToken.TokenType != UhtTokenType.Line)
				{
					break;
				}
				if (lineToken.Value.Span.Length > 0 && lineToken.Value.Span[^1] == '\\')
				{
					builder.Append(new StringView(lineToken.Value, 0, lineToken.Value.Span.Length - 1));
				}
				else
				{
					builder.Append(lineToken.Value);
					break;
				}
			}
			StringView line = builder.ToStringView();

			// Create a token reader we will use to decode
			UhtTokenBufferReader lineTokenReader = new(HeaderFile, line.Memory);
			lineTokenReader.InputLine = startingLine;

			if (!lineTokenReader.TryOptionalIdentifier(out UhtToken directive))
			{
				if (isBeingIncluded)
				{
					throw new UhtException(TokenReader, directive.InputLine, "Missing compiler directive after '#'");
				}
				return checkClearComments;
			}

			if (directive.IsValue("error"))
			{
				CheckRestrictedMode();
				if (isBeingIncluded)
				{
					throw new UhtException(TokenReader, directive.InputLine, "#error directive encountered");
				}
			}
			else if (directive.IsValue("pragma"))
			{
				CheckRestrictedMode();
				// Ignore all pragmas
			}
			else if (directive.IsValue("linenumber"))
			{
				CheckRestrictedMode();
				if (!lineTokenReader.TryOptionalConstInt(out int newInputLine))
				{
					throw new UhtException(TokenReader, directive.InputLine, "Missing line number in line number directive");
				}
				TokenReader.InputLine = newInputLine;
			}
			else if (directive.IsValue("include"))
			{
				CheckRestrictedMode();
				if (isBeingIncluded)
				{
					if (SpottedAutogeneratedHeaderInclude)
					{
						HeaderFile.LogError("#include found after .generated.h file - the .generated.h file should always be the last #include in a header");
					}

					StringView includeNameString = new();

					UhtToken includeName = lineTokenReader.GetToken();
					if (includeName.IsConstString())
					{
						includeNameString = includeName.GetUnescapedString(HeaderFile);
					}
					else if (includeName.IsSymbol('<'))
					{
						includeNameString = new StringView(lineTokenReader.GetRawString('>', UhtRawStringOptions.DontConsumeTerminator).Memory.Trim());
					}

					if (includeNameString.Length > 0)
					{
						if (HeaderFile.GeneratedHeaderFileName.AsSpan().Equals(includeNameString.Span, StringComparison.OrdinalIgnoreCase))
						{
							SpottedAutogeneratedHeaderInclude = true;
						}
						if (!includeNameString.Span.Contains(".generated.h", StringComparison.Ordinal))
						{
							HeaderFile.AddReferencedHeader(includeNameString.ToString(), UhtHeaderReferenceType.Include);
						}
					}
				}
			}
			else if (directive.IsValue("if"))
			{
				checkClearComments = true;
				PushCompilerDirective(ParserConditional(lineTokenReader));
				if (IsRestrictedDirective(GetCurrentNonCompositeCompilerDirective()))
				{
					CheckRestrictedMode();
				}
			}
			else if (directive.IsValue("ifdef") || directive.IsValue("ifndef"))
			{
				checkClearComments = true;
				PushCompilerDirective(UhtCompilerDirective.Unrecognized);
			}
			else if (directive.IsValue("elif"))
			{
				checkClearComments = true;
				UhtCompilerDirective oldCompilerDirective = PopCompilerDirective(directive);
				UhtCompilerDirective newCompilerDirective = ParserConditional(lineTokenReader);
				if (SupportsElif(oldCompilerDirective) != SupportsElif(newCompilerDirective))
				{
					throw new UhtException(TokenReader, directive.InputLine,
						$"Mixing {oldCompilerDirective.GetCompilerDirectiveText()} with {newCompilerDirective.GetCompilerDirectiveText()} in an #elif preprocessor block is not supported");
				}
				PushCompilerDirective(newCompilerDirective);
				if (IsRestrictedDirective(GetCurrentNonCompositeCompilerDirective()))
				{
					CheckRestrictedMode();
				}
			}
			else if (directive.IsValue("else"))
			{
				checkClearComments = true;
				UhtCompilerDirective oldCompilerDirective = PopCompilerDirective(directive);
				switch (oldCompilerDirective)
				{
					case UhtCompilerDirective.ZeroBlock:
						PushCompilerDirective(UhtCompilerDirective.OneBlock);
						break;
					case UhtCompilerDirective.OneBlock:
						PushCompilerDirective(UhtCompilerDirective.ZeroBlock);
						break;
					case UhtCompilerDirective.NotCPPBlock:
						PushCompilerDirective(UhtCompilerDirective.CPPBlock);
						break;
					case UhtCompilerDirective.CPPBlock:
						PushCompilerDirective(UhtCompilerDirective.NotCPPBlock);
						break;
					case UhtCompilerDirective.WithEngine:
						PushCompilerDirective(UhtCompilerDirective.Unrecognized);
						break;
					case UhtCompilerDirective.WithCoreUObject:
						PushCompilerDirective(UhtCompilerDirective.Unrecognized);
						break;
					case UhtCompilerDirective.WithHotReload:
						throw new UhtException(TokenReader, directive.InputLine, "Can not use WITH_HOT_RELOAD with an #else clause");
					default:
						PushCompilerDirective(oldCompilerDirective);
						break;
				}
				if (IsRestrictedDirective(GetCurrentNonCompositeCompilerDirective()))
				{
					CheckRestrictedMode();
				}
			}
			else if (directive.IsValue("endif"))
			{
				PopCompilerDirective(directive);
			}
			else if (directive.IsValue("define"))
			{
				CheckRestrictedMode();
			}
			else if (directive.IsValue("undef"))
			{
				CheckRestrictedMode();
			}
			else
			{
				if (isBeingIncluded)
				{
					throw new UhtException(TokenReader, directive.InputLine, $"Unrecognized compiler directive {directive.Value}");
				}
			}
			return checkClearComments;
		}

		private void CheckRestrictedMode()
		{
			if (RestrictedPreprocessorContext != null)
			{
				TokenReader.LogError($"Preprocessor statement not allowed while {RestrictedPreprocessorContext}");
			}
		}

		private UhtCompilerDirective ParserConditional(UhtTokenBufferReader lineTokenReader)
		{
			// Get any possible ! and the identifier
			UhtToken define = lineTokenReader.GetToken();
			bool notPresent = define.IsSymbol('!');
			if (notPresent)
			{
				define = lineTokenReader.GetToken();
			}

			//COMPATIBILITY-TODO
			// UModel.h contains a compound #if where the leading one is !CPP.
			// Checking for this being the only token causes that to fail
#if COMPATIBILITY_DISABLE
			// Make sure there is nothing left
			UhtToken end = LineTokenReader.GetToken();
			if (!end.TokenType.IsEndType())
			{
				return UhtCompilerDirective.Unrecognized;
			}
#endif

			switch (define.TokenType)
			{
				case UhtTokenType.DecimalConst:
					if (define.IsValue("0"))
					{
						return UhtCompilerDirective.ZeroBlock;
					}
					else if (define.IsValue("1"))
					{
						return UhtCompilerDirective.OneBlock;
					}
					break;

				case UhtTokenType.Identifier:
					if (define.IsValue(UhtNames.WithEditorOnlyData))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithEditorOnlyData;
					}
					else if (define.IsValue(UhtNames.WithEditor))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithEditor;
					}
					else if (define.IsValue(UhtNames.WithHotReload))
					{
						return UhtCompilerDirective.WithHotReload;
					}
					else if (define.IsValue(UhtNames.WithEngine))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithEngine;
					}
					else if (define.IsValue(UhtNames.WithCoreUObject))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithCoreUObject;
					}
					else if (define.IsValue(UhtNames.WithVerseVM))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithVerseVM;
					}
					else if (define.IsValue(UhtNames.WithVerseBPVM))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithVerseBPVM;
					}
					else if (define.IsValue(UhtNames.WithTests))
					{
						return notPresent ? UhtCompilerDirective.Unrecognized : UhtCompilerDirective.WithTests;
					}
					else if (define.IsValue("CPP"))
					{
						return notPresent ? UhtCompilerDirective.NotCPPBlock : UhtCompilerDirective.CPPBlock;
					}
					else if (define.ValueStartsWith("UE_VERSION"))
					{
						return ParserEngineVersionConditional(define, notPresent, lineTokenReader);
					}
					break;

				case UhtTokenType.EndOfFile:
				case UhtTokenType.EndOfDefault:
				case UhtTokenType.EndOfType:
				case UhtTokenType.EndOfDeclaration:
					throw new UhtException(TokenReader, define.InputLine, "#if with no expression");
			}

			return UhtCompilerDirective.Unrecognized;
		}

		private UhtCompilerDirective ParserEngineVersionConditional(UhtToken define, bool notPresent, UhtTokenBufferReader lineTokenReader)
		{
			lineTokenReader.Require('(');
			UhtToken majorToken = lineTokenReader.GetToken();
			lineTokenReader.Require(',');
			UhtToken minorToken = lineTokenReader.GetToken();
			lineTokenReader.Require(',');
			UhtToken patchToken = lineTokenReader.GetToken();
			lineTokenReader.Require(')');

			EngineVersion engineVersion = HeaderFile.Session.EngineVersion;

			majorToken.GetConstInt(out int major);
			minorToken.GetConstInt(out int minor);
			patchToken.GetConstInt(out int patch);
			EngineVersion defineVersion = new(major, minor, patch);

			bool? conditionPasses = null;
			if (define.IsValue("UE_VERSION_NEWER_THAN_OR_EQUAL"))
			{
				conditionPasses = engineVersion.CompareTo(defineVersion) >= 0;
			}
			else if (define.IsValue("UE_VERSION_NEWER_THAN"))
			{
				conditionPasses = engineVersion.CompareTo(defineVersion) > 0;
			}
			else if (define.IsValue("UE_VERSION_OLDER_THAN"))
			{
				conditionPasses = engineVersion.CompareTo(defineVersion) < 0;
			}

			if (conditionPasses.HasValue)
			{
				if (notPresent)
				{
					conditionPasses = !conditionPasses.Value;
				}
				return conditionPasses.Value ? UhtCompilerDirective.OneBlock : UhtCompilerDirective.ZeroBlock;
			}
			else
			{
				throw new UhtException(String.Format("Unknown engine version preprocessor conditional: {0}", define.Value));
			}
		}

		/// <summary>
		/// Add a new compiler directive to the stack
		/// </summary>
		/// <param name="compilerDirective">Directive to be added</param>
		private void PushCompilerDirective(UhtCompilerDirective compilerDirective)
		{
			CompilerDirective newCompileDirective = new();
			newCompileDirective._element = compilerDirective;
			newCompileDirective._composite = GetCurrentCompositeCompilerDirective() | compilerDirective;
			_compilerDirectives.Add(newCompileDirective);
		}

		/// <summary>
		/// Remove the top level compiler directive from the stack
		/// </summary>
		private UhtCompilerDirective PopCompilerDirective(UhtToken token)
		{
			if (_compilerDirectives.Count == 0)
			{
				throw new UhtException(TokenReader, token.InputLine, $"Unmatched '#{token.Value}'");
			}
			UhtCompilerDirective compilerDirective = _compilerDirectives[^1]._element;
			_compilerDirectives.RemoveAt(_compilerDirectives.Count - 1);
			return compilerDirective;
		}

		private bool IncludeCurrentCompilerDirective()
		{
			if (_compilerDirectives.Count == 0)
			{
				return true;
			}
			return !GetCurrentCompositeCompilerDirective().HasAnyFlags(UhtCompilerDirective.CPPBlock | UhtCompilerDirective.ZeroBlock | UhtCompilerDirective.Unrecognized);
		}

		/// <summary>
		/// The old UHT would preprocess the file and eliminate any #if blocks that were not required for 
		/// any contextual information.  This results in comments before the #if block being considered 
		/// for the next definition.  This routine classifies each #if block type into if comments should
		/// be purged after the directive.
		/// </summary>
		/// <returns></returns>
		/// <exception cref="UhtIceException"></exception>
		private bool ClearCommentsCompilerDirective()
		{
			if (_compilerDirectives.Count == 0)
			{
				return true;
			}
			UhtCompilerDirective compilerDirective = _compilerDirectives[^1]._element;
			switch (compilerDirective)
			{
				case UhtCompilerDirective.CPPBlock:
				case UhtCompilerDirective.NotCPPBlock:
				case UhtCompilerDirective.ZeroBlock:
				case UhtCompilerDirective.OneBlock:
				case UhtCompilerDirective.Unrecognized:
					return false;

				case UhtCompilerDirective.WithEditor:
				case UhtCompilerDirective.WithEditorOnlyData:
				case UhtCompilerDirective.WithEngine:
				case UhtCompilerDirective.WithCoreUObject:
				case UhtCompilerDirective.WithHotReload:
				case UhtCompilerDirective.WithVerseVM:
				case UhtCompilerDirective.WithVerseBPVM:
				case UhtCompilerDirective.WithTests:
					return true;

				default:
					throw new UhtIceException("Unknown compiler directive flag");
			}
		}

		private static bool IsRestrictedDirective(UhtCompilerDirective compilerDirective)
		{
			switch (compilerDirective)
			{
				case UhtCompilerDirective.CPPBlock:
				case UhtCompilerDirective.NotCPPBlock:
				case UhtCompilerDirective.ZeroBlock:
				case UhtCompilerDirective.OneBlock:
				case UhtCompilerDirective.Unrecognized:
					return false;

				case UhtCompilerDirective.WithEditor:
				case UhtCompilerDirective.WithEditorOnlyData:
				case UhtCompilerDirective.WithEngine:
				case UhtCompilerDirective.WithCoreUObject:
				case UhtCompilerDirective.WithHotReload:
				case UhtCompilerDirective.WithVerseVM:
				case UhtCompilerDirective.WithVerseBPVM:
				case UhtCompilerDirective.WithTests:
					return true;

				default:
					throw new UhtIceException("Unknown compiler directive flag");
			}
		}

		private static bool SupportsElif(UhtCompilerDirective compilerDirective)
		{
			return
				compilerDirective == UhtCompilerDirective.WithEditor ||
				compilerDirective == UhtCompilerDirective.WithEditorOnlyData ||
				compilerDirective == UhtCompilerDirective.WithHotReload ||
				compilerDirective == UhtCompilerDirective.WithEngine ||
				compilerDirective == UhtCompilerDirective.WithCoreUObject ||
				compilerDirective == UhtCompilerDirective.WithVerseVM ||
				compilerDirective == UhtCompilerDirective.WithVerseBPVM ||
				compilerDirective == UhtCompilerDirective.WithTests;
		}

		private static void SkipVirtualAndAPI(IUhtTokenReader replayReader)
		{
			while (true)
			{
				UhtToken peekToken = replayReader.PeekToken();
				if (!peekToken.IsValue("virtual") && !peekToken.Value.Span.EndsWith("_API", StringComparison.Ordinal))
				{
					break;
				}
				replayReader.ConsumeToken();
			}
		}

		private bool CheckForConstructor(UhtClass classObj, UhtDeclaration declaration)
		{
			using UhtTokenReplayReaderBorrower borrowedReader = new(HeaderFile, HeaderFile.Data.Memory, declaration.Tokens, UhtTokenType.EndOfDeclaration);
			IUhtTokenReader replayReader = borrowedReader.Reader;

			// Allow explicit constructors
			{
				bool foundExplicit = replayReader.TryOptional("explicit");
				if (replayReader.PeekToken().Value.Span.EndsWith("_API", StringComparison.Ordinal))
				{
					replayReader.ConsumeToken();
					if (!foundExplicit)
					{
						replayReader.TryOptional("explicit");
					}
				}
			}

			if (!replayReader.TryOptional(classObj.SourceName) ||
				!replayReader.TryOptional('('))
			{
				return false;
			}

			bool oiCtor = false;
			bool vtCtor = false;

			if (!classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDefaultConstructor) && replayReader.TryOptional(')'))
			{
				classObj.ClassExportFlags |= UhtClassExportFlags.HasDefaultConstructor;
			}
			else if (!classObj.ClassExportFlags.HasAllFlags(UhtClassExportFlags.HasObjectInitializerConstructor | UhtClassExportFlags.HasCustomVTableHelperConstructor))
			{
				bool isConst = false;
				bool isRef = false;
				int parenthesesNestingLevel = 1;

				while (parenthesesNestingLevel != 0)
				{
					UhtToken token = replayReader.GetToken();
					if (!token)
					{
						break;
					}

					// Template instantiation or additional parameter excludes ObjectInitializer constructor.
					if (token.IsValue(',') || token.IsValue('<'))
					{
						oiCtor = false;
						vtCtor = false;
						break;
					}

					if (token.IsValue('('))
					{
						parenthesesNestingLevel++;
						continue;
					}

					if (token.IsValue(')'))
					{
						parenthesesNestingLevel--;
						continue;
					}

					if (token.IsValue("const"))
					{
						isConst = true;
						continue;
					}

					if (token.IsValue('&'))
					{
						isRef = true;
						continue;
					}

					// FPostConstructInitializeProperties is deprecated, but left here, so it won't break legacy code.
					if (token.IsValue("FObjectInitializer") || token.IsValue("FPostConstructInitializeProperties"))
					{
						oiCtor = true;
					}

					if (token.IsValue("FVTableHelper"))
					{
						vtCtor = true;
					}
				}

				// Parse until finish.
				if (parenthesesNestingLevel != 0)
				{
					replayReader.SkipBrackets('(', ')', parenthesesNestingLevel);
				}

				if (oiCtor && isRef && isConst)
				{
					classObj.ClassExportFlags |= UhtClassExportFlags.HasObjectInitializerConstructor;
					classObj.MetaData.Add(UhtNames.ObjectInitializerConstructorDeclared, "");
				}
				if (vtCtor && isRef)
				{
					classObj.ClassExportFlags |= UhtClassExportFlags.HasCustomVTableHelperConstructor;
				}
			}

			if (!vtCtor)
			{
				classObj.ClassExportFlags |= UhtClassExportFlags.HasConstructor;
			}

			return false;
		}

		private bool CheckForDestructor(UhtClass classObj, UhtDeclaration declaration)
		{
			if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasDestructor))
			{
				return false;
			}

			using UhtTokenReplayReaderBorrower borrowedReader = new(HeaderFile, HeaderFile.Data.Memory, declaration.Tokens, UhtTokenType.EndOfDeclaration);
			IUhtTokenReader replayReader = borrowedReader.Reader;

			SkipVirtualAndAPI(replayReader);

			if (replayReader.TryOptional('~') && replayReader.TryOptional(classObj.SourceName))
			{
				classObj.ClassExportFlags |= UhtClassExportFlags.HasDestructor;
				return true;
			}
			return false;
		}

		bool CheckForSerialize(UhtClass classObj, UhtDeclaration declaration)
		{
			using UhtTokenReplayReaderBorrower borrowedReader = new(HeaderFile, HeaderFile.Data.Memory, declaration.Tokens, UhtTokenType.EndOfDeclaration);
			IUhtTokenReader replayReader = borrowedReader.Reader;

			SkipVirtualAndAPI(replayReader);

			if (!replayReader.TryOptional("void") ||
				!replayReader.TryOptional("Serialize") ||
				!replayReader.TryOptional('('))
			{
				return false;
			}

			replayReader.Optional("class");

			UhtToken token = replayReader.GetToken();

			UhtSerializerArchiveType archiveType = UhtSerializerArchiveType.None;
			if (token.IsValue("FArchive"))
			{
				if (replayReader.TryOptional('&'))
				{
					// Allow the declaration to not define a name for the archive parameter
					if (!replayReader.PeekToken().IsValue(')'))
					{
						replayReader.SkipOne();
					}
					if (replayReader.TryOptional(')'))
					{
						archiveType = UhtSerializerArchiveType.Archive;
					}
				}
			}
			else if (token.IsValue("FStructuredArchive"))
			{
				if (replayReader.TryOptional("::") &&
					replayReader.TryOptional("FRecord"))
				{
					// Allow the declaration to not define a name for the archive parameter
					if (!replayReader.PeekToken().IsValue(')'))
					{
						replayReader.SkipOne();
					}
					if (replayReader.TryOptional(')'))
					{
						archiveType = UhtSerializerArchiveType.StructuredArchiveRecord;
					}
				}
			}
			else if (token.IsValue("FStructuredArchiveRecord"))
			{
				// Allow the declaration to not define a name for the archive parameter
				if (!replayReader.PeekToken().IsValue(')'))
				{
					replayReader.SkipOne();
				}
				if (replayReader.TryOptional(')'))
				{
					archiveType = UhtSerializerArchiveType.StructuredArchiveRecord;
				}
			}

			if (archiveType != UhtSerializerArchiveType.None)
			{
				// Found what we want!
				if (declaration.CompilerDirectives == UhtCompilerDirective.None || declaration.CompilerDirectives == UhtCompilerDirective.WithEditorOnlyData)
				{
					classObj.SerializerArchiveType |= archiveType;
					classObj.SerializerDefineScope = declaration.CompilerDirectives == UhtCompilerDirective.None ? UhtDefineScope.None : UhtDefineScope.EditorOnlyData;
				}
				else
				{
					classObj.LogError("Serialize functions must not be inside preprocessor blocks, except for WITH_EDITORONLY_DATA");
				}
				return true;
			}

			return false;
		}

		private bool CheckForBeginDestroy(UhtPartial partial, UhtDeclaration declaration)
		{
			using UhtTokenReplayReaderBorrower borrowedReader = new(HeaderFile, HeaderFile.Data.Memory, declaration.Tokens, UhtTokenType.EndOfDeclaration);
			IUhtTokenReader replayReader = borrowedReader.Reader;

			if (!replayReader.TryOptional("void") ||
				!replayReader.TryOptional("BeginDestroy") ||
				!replayReader.TryOptional('('))
			{
				return false;
			}

			partial.HasBeginDestroy = true;

			return true;
		}

		#endregion
	}
}
