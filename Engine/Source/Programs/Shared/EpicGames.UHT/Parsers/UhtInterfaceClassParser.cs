// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Interface class parser
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtInterfaceClassParser
	{

		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UINTERFACEKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUInterface(topScope, ref token);
		}

		[UhtKeyword(Extends = UhtTableNames.Interface)]
		[UhtKeyword(Extends = UhtTableNames.Interface, Keyword = "GENERATED_BODY")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult GENERATED_UINTERFACE_BODYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtClass classObj = (UhtClass)topScope.ScopeType;

			UhtParserHelpers.ParseCompileVersionDeclaration(topScope.TokenReader, topScope.Session.Config!, classObj);

			classObj.GeneratedBodyAccessSpecifier = topScope.AccessSpecifier;
			classObj.GeneratedBodyLineNumber = topScope.TokenReader.InputLine;

			if (token.IsValue("GENERATED_UINTERFACE_BODY"))
			{
				topScope.AccessSpecifier = UhtAccessSpecifier.Public;
				classObj.ClassExportFlags |= UhtClassExportFlags.UsesGeneratedBodyLegacy;
			}
			return UhtParseResult.Handled;
		}
		#endregion

		private static UhtParseResult ParseUInterface(UhtParsingScope parentScope, ref UhtToken token)
		{
			UhtClass classObj = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, token.InputLine);
			classObj.ClassType = UhtClassType.Interface;

			{
				using UhtParsingScope topScope = new(parentScope, classObj, parentScope.Session.GetKeywordTable(UhtTableNames.Interface), UhtAccessSpecifier.Private);
				const string ScopeName = "interface";

				{
					using UhtMessageContext tokenContext = new(ScopeName);

					// Parse the specifiers
					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, classObj.MetaData);
					UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, ScopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.Interface));
					specifiers.ParseSpecifiers();
					classObj.PrologLineNumber = topScope.TokenReader.InputLine;
					classObj.ClassFlags |= EClassFlags.Native | EClassFlags.Interface | EClassFlags.Abstract;
					classObj.InternalObjectFlags |= EInternalObjectFlags.Native;

					// Consume the class specifier
					topScope.TokenReader.Require("class");

					// Get the optional API macro
					topScope.TokenReader.TryOptionalAPIMacro(out UhtToken apiMacroToken);

					// Get the name of the interface
					classObj.SourceName = topScope.TokenReader.GetIdentifier().Value.ToString();

					// Parse the inheritance
					UhtParserHelpers.ParseInheritance(parentScope.HeaderParser, topScope.Session.Config!, out UhtInheritance inheritance);
					classObj.SuperIdentifier = inheritance.SuperIdentifier;
					classObj.VerseInterfaceIdentifiers = inheritance.VerseInterfaceIdentifiers;
					classObj.VerseInterfacesLine = inheritance.VerseInterfacesLine;
					if (inheritance.BaseIdentifiers.Count > 0)
					{
						topScope.TokenReader.LogError($"Interfaces can not contain any extra base classes");
					}

					// Split the source name into the different parts
					UhtEngineNameParts nameParts = UhtUtilities.GetEngineNameParts(classObj.SourceName);
					classObj.EngineName = nameParts.EngineName.ToString();

					// Interfaces must start with a valid prefix
					if (nameParts.Prefix != "U")
					{
						topScope.TokenReader.LogError($"Interface name '{classObj.SourceName}' is invalid, the first class should be identified as 'U{nameParts.EngineName}'");
					}

					// Check for an empty engine name
					if (classObj.EngineName.Length == 0)
					{
						topScope.TokenReader.LogError($"When compiling class definition for '{classObj.SourceName}', attempting to strip prefix results in an empty name. Did you leave off a prefix?");
					}

					if (apiMacroToken)
					{
						classObj.ClassFlags |= EClassFlags.RequiredAPI;
					}

					specifiers.ParseDeferred();

					// Handle verse default settings
					if (classObj.IsVerseField)
					{
						classObj.ClassFlags |= EClassFlags.EditInlineNew | EClassFlags.HasInstancedReference;
					}

					classObj.Outer?.AddChild(classObj);

					topScope.AddModuleRelativePathToMetaData();

					topScope.HeaderParser.ParseStatements('{', '}', true);

					topScope.TokenReader.Require(';');

					if (classObj.GeneratedBodyLineNumber == -1)
					{
						topScope.TokenReader.LogError("Expected a GENERATED_BODY() at the start of the interface");
					}
				}
			}

			return UhtParseResult.Handled;
		}
	}
}
