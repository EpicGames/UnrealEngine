// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// namespace parser
	/// </summary>
	[UnrealHeaderTool]
	sealed class UhtNamespaceParser
	{
		[UhtKeyword(Extends = UhtTableNames.Global, Keyword = "namespace", DisableUsageError = true)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult NamespaceKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return topScope.Module.Module.AllowUETypesInNamespaces ? ParseNamespace(topScope) : UhtParseResult.Unhandled;
		}

		private static int ParseNamespaceIdentifiers(UhtParsingScope topScope)
		{
			// Parse the namespace identifiers
			int namespaceCount = 0;
			if (topScope.TokenReader.TryOptionalIdentifier(out UhtToken token))
			{
				namespaceCount++;
				topScope.HeaderParser.PushNamespace(token);
				while (topScope.TokenReader.TryOptional("::"))
				{
					namespaceCount++;
					topScope.TokenReader.Optional("inline");
					topScope.HeaderParser.PushNamespace(topScope.TokenReader.GetIdentifier());
				}
			}
			return namespaceCount;
		}

		private static UhtParseResult ParseNamespace(UhtParsingScope parentScope)
		{
			using UhtParsingScope topScope = new(parentScope, parentScope.ScopeType, parentScope.Session.GetKeywordTable(UhtTableNames.Global), UhtAccessSpecifier.Public);
			const string ScopeName = "namespace";
			using UhtMessageContext tokenContext = new(ScopeName);

			// Handle any attributes
			topScope.TokenReader.OptionalAttributes(false);

			// Parse the namespace identifiers
			int namespaceCount = ParseNamespaceIdentifiers(topScope);

			// if this is a namespace id = ... statement
			if (topScope.TokenReader.TryOptional('='))
			{
				while (true)
				{
					UhtToken skipped = topScope.TokenReader.GetToken();
					if (skipped.IsSymbol(';'))
					{
						break;
					}
				}
			}
			else
			{
				// Parse the namespace body
				topScope.HeaderParser.ParseStatements('{', '}', true);

				// TODO: The parse statement code expects to have any trailing ';' removed.  Remove any here
				topScope.TokenReader.Optional(';');
			}

			// Remove the namespaces from the active namespace
			topScope.HeaderParser.PopNamespaces(namespaceCount);
			return UhtParseResult.Handled;
		}
	}
}
