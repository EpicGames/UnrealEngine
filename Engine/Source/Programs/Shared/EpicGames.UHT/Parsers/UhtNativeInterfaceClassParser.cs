// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Parser object for native interfaces
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtNativeInterfaceClassParser
	{

		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global, Keyword = "class", DisableUsageError = true)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult ClassKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			{
				using UhtTokenSaveState saveState = new(topScope.TokenReader);
				UhtToken sourceName = new();
				UhtToken[] superName = [];
				if (TryParseIInterface(topScope, out sourceName, out superName))
				{
					saveState.AbandonState();
					ParseIInterface(topScope, ref token, sourceName, superName);
					return UhtParseResult.Handled;
				}
			}
			return UhtParseResult.Unhandled;
		}

		[UhtKeyword(Extends = UhtTableNames.NativeInterface)]
		[UhtKeyword(Extends = UhtTableNames.NativeInterface, Keyword = "GENERATED_BODY")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult GENERATED_IINTERFACE_BODYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtClass classObj = (UhtClass)topScope.ScopeType;

			UhtParserHelpers.ParseCompileVersionDeclaration(topScope.TokenReader, topScope.Session.Config!, classObj);

			classObj.GeneratedBodyAccessSpecifier = topScope.AccessSpecifier;
			classObj.GeneratedBodyLineNumber = topScope.TokenReader.InputLine;
			classObj.HasGeneratedBody = true;

			if (token.IsValue("GENERATED_IINTERFACE_BODY"))
			{
				classObj.ClassExportFlags |= UhtClassExportFlags.UsesGeneratedBodyLegacy;
				topScope.AccessSpecifier = UhtAccessSpecifier.Public;
			}
			return UhtParseResult.Handled;
		}
		#endregion

		private static bool TryParseIInterface(UhtParsingScope parentScope, out UhtToken sourceName, out UhtToken[] superName)
		{
			IUhtTokenReader tokenReader = parentScope.TokenReader;

			// Get the optional API macro
			tokenReader.TryOptionalAPIMacro(out UhtToken _);

			// Get the name of the interface
			sourceName = tokenReader.GetIdentifier();

			// Old UHT would still parse the inheritance
			superName = [];
			if (tokenReader.TryOptional(':'))
			{
				if (!tokenReader.TryOptional("public"))
				{
					return false;
				}
				superName = tokenReader.GetCppIdentifier(UhtCppIdentifierOptions.None).ToArray();
			}

			// Only process classes starting with 'I'
			if (sourceName.Value.Span[0] != 'I')
			{
				return false;
			}

			// If we end with a ';', then this is a forward declaration
			if (tokenReader.TryOptional(';'))
			{
				return false;
			}

			// If we don't have a '{', then this is something else
			if (!tokenReader.TryPeekOptional('{'))
			{
				return false;
			}
			return true;
		}

		private static void ParseIInterface(UhtParsingScope parentScope, ref UhtToken token, UhtToken sourceName, UhtToken[] superName)
		{
			IUhtTokenReader tokenReader = parentScope.TokenReader;

			UhtClass classObj = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, token.InputLine);
			classObj.ClassType = UhtClassType.NativeInterface;
			classObj.SourceName = sourceName.Value.ToString();
			classObj.ClassFlags |= EClassFlags.Native | EClassFlags.Interface;

			// Split the source name into the different parts
			UhtEngineNameParts nameParts = UhtUtilities.GetEngineNameParts(classObj.SourceName);
			classObj.EngineName = nameParts.EngineName.ToString();

			// Check for an empty engine name
			if (classObj.EngineName.Length == 0)
			{
				tokenReader.LogError($"When compiling class definition for '{classObj.SourceName}', attempting to strip prefix results in an empty name. Did you leave off a prefix?");
			}

			classObj.Outer?.AddChild(classObj);
			classObj.SuperIdentifier = superName;

			//TODO - C++ UHT compatibility - When we know for sure that we have a native interface, then we should error out.  Due to the lack of a symbol table,
			// we can't do this 100% reliably.  However, if we find a U class, we can assume we have a native interface.
			bool logUnhandledKeywords = false;
			string interfaceName = "U" + classObj.EngineName;
			foreach (UhtType outerChild in parentScope.HeaderFile.Children)
			{
				if (outerChild.SourceName == interfaceName && outerChild is UhtClass outerChildClass && outerChildClass.ClassType == UhtClassType.Interface)
				{
					logUnhandledKeywords = true;
					break;
				}
			}

			{
				using UhtParsingScope topScope = new(parentScope, classObj, parentScope.Session.GetKeywordTable(UhtTableNames.NativeInterface), UhtAccessSpecifier.Private);
				using UhtMessageContext tokenContext = new("native interface");
				topScope.HeaderParser.ParseStatements('{', '}', logUnhandledKeywords);
				tokenReader.Require(';');
			}
			return;
		}
	}
}
