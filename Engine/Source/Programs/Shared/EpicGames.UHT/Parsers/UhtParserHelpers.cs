// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Collection of parsed inheritance data
	/// </summary>
	public struct UhtInheritance
	{

		/// <summary>
		/// Super class identifier
		/// </summary>
		public UhtToken[] SuperIdentifier { get; set; } = [];

		/// <summary>
		/// Collection of other base identifiers
		/// </summary>
		public List<UhtToken[]> BaseIdentifiers { get; } = [];

		/// <summary>
		/// Collection of verse interface identifiers
		/// </summary>
		public List<UhtToken[]> VerseInterfaceIdentifiers { get; } = [];

		/// <summary>
		/// Line number that the VIDENTIFIER macro appeared
		/// </summary>
		public int VerseInterfacesLine { get; set; } = 0;

		/// <summary>
		/// Constructor
		/// </summary>
		public UhtInheritance()
		{
		}
	}

	/// <summary>
	/// Collection of helper methods
	/// </summary>
	public static class UhtParserHelpers
	{

		/// <summary>
		/// Parse the inheritance 
		/// </summary>
		/// <param name="headerFileParser">Header file being parsed</param>
		/// <param name="config">Configuration</param>
		/// <param name="inheritance">Collection of inheritance information</param>
		public static void ParseInheritance(UhtHeaderFileParser headerFileParser, IUhtConfig config, out UhtInheritance inheritance)
		{
			// TODO: C++ UHT doesn't allow preprocessor statements inside of inheritance lists
			string? restrictedPreprocessorContext = headerFileParser.RestrictedPreprocessorContext;
			headerFileParser.RestrictedPreprocessorContext = "parsing inheritance list";

			try
			{
				UhtInheritance scratch = new();
				IUhtTokenReader tokenReader = headerFileParser.TokenReader;
				tokenReader.Optional(':', () =>
				{
					tokenReader
						.Require("public", "public access modifier")
						.RequireCppIdentifier(UhtCppIdentifierOptions.None, identifier =>
						{
							identifier.RedirectTypeIdentifier(config);
							scratch.SuperIdentifier = identifier.ToArray();
						})
						.While(',', () =>
						{
							tokenReader
								.Require("public", "public interface access specifier")
								.RequireCppIdentifier(UhtCppIdentifierOptions.AllowTemplates, identifier =>
								{
									scratch.BaseIdentifiers.Add(identifier.ToArray());
								});
						})
						.Optional("VINTERFACES", () =>
						{
							tokenReader.Require('(');
							tokenReader.RequireList(')', ',', false, () =>
							{
								tokenReader.RequireCppIdentifier(UhtCppIdentifierOptions.None, identifier =>
								{
									scratch.VerseInterfaceIdentifiers.Add(identifier.ToArray());
								});
							});
							scratch.VerseInterfacesLine = tokenReader.InputLine;
						});
				});
				inheritance = scratch;
			}
			finally
			{
				headerFileParser.RestrictedPreprocessorContext = restrictedPreprocessorContext;
			}
		}

		/// <summary>
		/// Parse compiler version declaration
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="config">Configuration</param>
		/// <param name="structObj">Struct being parsed</param>
		public static void ParseCompileVersionDeclaration(IUhtTokenReader tokenReader, IUhtConfig config, UhtStruct structObj)
		{

			// Fetch the default generation code version. If supplied, then package code version overrides the default.
			EGeneratedCodeVersion version = structObj.Module.Module.GeneratedCodeVersion;
			if (version == EGeneratedCodeVersion.None)
			{
				version = config.DefaultGeneratedCodeVersion;
			}

			// Fetch the code version from header file
			tokenReader
				.Require('(')
				.OptionalIdentifier((ref UhtToken identifier) =>
				{
					if (!Enum.TryParse(identifier.Value.ToString(), true, out version))
					{
						version = EGeneratedCodeVersion.None;
					}
				})
				.Require(')');

			// Save the results
			structObj.GeneratedCodeVersion = version;
		}
	}
}
