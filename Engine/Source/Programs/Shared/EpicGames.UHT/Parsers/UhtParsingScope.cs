// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Nested structure of scopes being parsed
	/// </summary>
	public class UhtParsingScope : IDisposable
	{

		/// <summary>
		/// Header file parser
		/// </summary>
		public UhtHeaderFileParser HeaderParser { get; }

		/// <summary>
		/// Header file being parsed
		/// </summary>
		public UhtHeaderFile HeaderFile => HeaderParser.HeaderFile;

		/// <summary>
		/// Module owning the header file
		/// </summary>
		public UhtModule Module => HeaderFile.Module;

		/// <summary>
		/// Token reader
		/// </summary>
		public IUhtTokenReader TokenReader { get; }

		/// <summary>
		/// Parent scope
		/// </summary>
		public UhtParsingScope? ParentScope { get; }

		/// <summary>
		/// Type being parsed.
		/// </summary>
		public UhtType ScopeType { get; }

		/// <summary>
		/// Keyword table for the scope
		/// </summary>
		public UhtKeywordTable ScopeKeywordTable { get; }

		/// <summary>
		/// Current access specifier
		/// </summary>
		public UhtAccessSpecifier AccessSpecifier { get; set; } = UhtAccessSpecifier.Public;

		/// <summary>
		/// Current session
		/// </summary>
		public UhtSession Session => HeaderFile.Session;

		/// <summary>
		/// Return the current class scope being compiled
		/// </summary>
		public UhtParsingScope CurrentClassScope
		{
			get
			{
				UhtParsingScope? currentScope = this;
				while (currentScope != null)
				{
					if (currentScope.ScopeType is UhtClass or UhtPartial)
					{
						return currentScope;
					}
					currentScope = currentScope.ParentScope;
				}
				throw new UhtIceException("Attempt to fetch the current class when a class isn't currently being parsed");
			}
		}

		/// <summary>
		/// Construct a root/global scope
		/// </summary>
		/// <param name="headerParser">Header parser</param>
		/// <param name="keywordTable">Keyword table</param>
		public UhtParsingScope(UhtHeaderFileParser headerParser, UhtKeywordTable keywordTable)
		{
			HeaderParser = headerParser;
			TokenReader = headerParser.TokenReader;
			ParentScope = null;
			ScopeType = headerParser.Module.ScriptPackage; // The default package for parsing will be the standard UE package
			ScopeKeywordTable = keywordTable;
			HeaderParser.PushScope(this);
		}

		/// <summary>
		/// Construct a scope for a type
		/// </summary>
		/// <param name="parentScope">Parent scope</param>
		/// <param name="scopeType">Type being parsed</param>
		/// <param name="keywordTable">Keyword table</param>
		/// <param name="accessSpecifier">Current access specifier</param>
		public UhtParsingScope(UhtParsingScope parentScope, UhtType scopeType, UhtKeywordTable keywordTable, UhtAccessSpecifier accessSpecifier)
		{
			HeaderParser = parentScope.HeaderParser;
			TokenReader = parentScope.TokenReader;
			ParentScope = parentScope;
			ScopeType = scopeType;
			ScopeKeywordTable = keywordTable;
			AccessSpecifier = accessSpecifier;
			HeaderParser.PushScope(this);
		}

		/// <summary>
		/// Dispose the scope
		/// </summary>
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Virtual method for disposing the object
		/// </summary>
		/// <param name="disposing">If true, we are disposing</param>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				HeaderParser.PopScope(this);
			}
		}

		/// <summary>
		/// Add the module's relative path to the type's meta data
		/// </summary>
		public void AddModuleRelativePathToMetaData()
		{
			AddModuleRelativePathToMetaData(ScopeType.MetaData, ScopeType.HeaderFile);
		}

		/// <summary>
		/// Add the module's relative path to the meta data
		/// </summary>
		/// <param name="metaData">The meta data to add the information to</param>
		/// <param name="headerFile">The header file currently being parsed</param>
		public static void AddModuleRelativePathToMetaData(UhtMetaData metaData, UhtHeaderFile headerFile)
		{
			metaData.Add(UhtNames.ModuleRelativePath, headerFile.ModuleRelativeFilePath);
		}

		/// <summary>
		/// Format the current token reader comments and add it as meta data
		/// </summary>
		/// <param name="metaNameIndex">Index for the meta data key.  This is used for enum values</param>
		public void AddFormattedCommentsAsTooltipMetaData(int metaNameIndex = UhtMetaData.IndexNone)
		{
			AddFormattedCommentsAsTooltipMetaData(ScopeType, metaNameIndex);
		}

		/// <summary>
		/// Format the current token reader comments and add it as meta data
		/// </summary>
		/// <param name="type">The type to add the meta data to</param>
		/// <param name="metaNameIndex">Index for the meta data key.  This is used for enum values</param>
		public void AddFormattedCommentsAsTooltipMetaData(UhtType type, int metaNameIndex = UhtMetaData.IndexNone)
		{

			// Don't add a tooltip if one already exists.
			if (type.MetaData.ContainsKey(UhtNames.ToolTip, metaNameIndex))
			{
				return;
			}

			// Fetch the comments
			ReadOnlySpan<StringView> comments = TokenReader.Comments;

			// If we don't have any comments, just return
			if (comments.Length == 0)
			{
				return;
			}

			// Set the comment as just a simple concatenation of all the strings
			string mergedString = String.Empty;
			if (comments.Length == 1)
			{
				mergedString = comments[0].ToString();
				type.MetaData.Add(UhtNames.Comment, metaNameIndex, mergedString);
			}
			else
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
				StringBuilder builder = borrower.StringBuilder;
				foreach (StringView comment in comments)
				{
					builder.Append(comment);
				}
				mergedString = builder.ToString();
				type.MetaData.Add(UhtNames.Comment, metaNameIndex, mergedString);
			}

			// Format the tooltip and set the metadata
			string toolTip = FormatCommentForToolTip(mergedString);
			if (!String.IsNullOrEmpty(toolTip))
			{
				type.MetaData.Add(UhtNames.ToolTip, metaNameIndex, toolTip);

				//COMPATIBILITY-TODO - Old UHT would only clear the comments if there was some form of a tooltip
				TokenReader.ClearComments();
			}

			//COMPATIBILITY-TODO
			// Clear the comments since they have been consumed
			//TokenReader.ClearComments();
		}

		// We consider any alpha/digit or code point > 0xFF as a valid comment char
		/// <summary>
		/// Given a list of comments, check to see if any have alpha, numeric, or unicode code points with a value larger than 0xFF.
		/// </summary>
		/// <param name="comments">Comments to search</param>
		/// <returns>True is a character in question was found</returns>
		private static bool HasValidCommentChar(ReadOnlySpan<char> comments)
		{
			foreach (char c in comments)
			{
				if (UhtFCString.IsAlnum(c) || c > 0xFF)
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Convert the given list of comments to a tooltip.  Each string view is a comment where the // style comments also includes the trailing \r\n.
		/// 
		/// The following style comments are supported:
		/// 
		/// /* */ - C Style
		/// /** */ - C Style JavaDocs
		/// /*~ */ - C Style but ignore
		/// //\r\n - C++ Style
		/// ///\r\n - C++ Style JavaDocs
		/// //~\r\n - C++ Style bug ignore
		///
		/// As per TokenReader, there will only be one C style comment ever present, and it will be the first one.  When a C style comment is parsed, any prior comments
		/// are cleared.  However, if a C++ style comment follows a C style comment (regardless of any intermediate blank lines), then both blocks of comments will be present.
		/// If any blank lines are encountered between blocks of C++ style comments, then any prior comments are cleared.
		/// </summary>
		/// <param name="comments">Comments to be parsed</param>
		/// <returns>The generated tooltip</returns>
		private static string FormatCommentForToolTip(string comments)
		{
			if (!HasValidCommentChar(comments))
			{
				return String.Empty;
			}

			// Use the scratch characters to store the string as we process it in MANY passes
			char[] scratchChars = ArrayPool<char>.Shared.Rent(comments.Length);
			comments.CopyTo(0, scratchChars, 0, comments.Length);

			// Remove ignore comments and strip the comment markers:
			// These are all issues with how the old UHT worked.
			// 1) Must be done in order
			// 2) Block comment markers are removed first so that '///**' is process by removing '/**' first and then '//' second
			// 3) We only remove block comments if we find the start of a comment.  This means that if there is a '*/' in a line comment, it won't be removed.
			// 4) We must check to see if we have cppStyle prior to removing block style comments. 
			int commentsLength = RemoveIgnoreComments(scratchChars, comments.Length);
			ReadOnlySpan<char> span = scratchChars.AsSpan(0, commentsLength);
			bool javaDocStyle = span.Contains("/**", StringComparison.Ordinal);
			bool cStyle = javaDocStyle || span.Contains("/*", StringComparison.Ordinal);
			bool cppStyle = span.StartsWith("//", StringComparison.Ordinal);
			commentsLength = javaDocStyle || cStyle ? RemoveBlockCommentMarkers(scratchChars, commentsLength, javaDocStyle) : commentsLength;
			commentsLength = cppStyle ? RemoveLineCommentMarkers(scratchChars, commentsLength) : commentsLength;

			//wx widgets has a hard coded tab size of 8
			{
				const int SpacesPerTab = 8;

				// If we have any tab characters, then we need to convert them to spaces
				span = scratchChars.AsSpan(0, commentsLength);
				int tabIndex = span.IndexOf('\t');
				if (tabIndex != -1)
				{
					using BorrowStringBuilder tabsBorrower = new(StringBuilderCache.Small);
					StringBuilder tabsBuilder = tabsBorrower.StringBuilder;
					UhtFCString.TabsToSpaces(span, SpacesPerTab, true, tabIndex, tabsBuilder);
					commentsLength = tabsBuilder.Length;
					if (commentsLength > scratchChars.Length)
					{
						ArrayPool<char>.Shared.Return(scratchChars);
						scratchChars = ArrayPool<char>.Shared.Rent(commentsLength);
					}
					tabsBuilder.CopyTo(0, scratchChars, 0, commentsLength);
				}
			}

			static bool IsAllSameChar(ReadOnlySpan<char> line, int startIndex, char testChar)
			{
				for (int index = startIndex, end = line.Length; index < end; ++index)
				{
					if (line[index] != testChar)
					{
						return false;
					}
				}
				return true;
			}

			static bool IsWhitespaceOrLineSeparator(ReadOnlySpan<char> line)
			{
				// Skip any leading spaces
				int index = 0;
				int endPos = line.Length;
				for (; index < endPos && UhtFCString.IsWhitespace(line[index]); ++index)
				{
				}
				if (index == endPos)
				{
					return true;
				}

				// Check for the same character
				return IsAllSameChar(line, index, '-') || IsAllSameChar(line, index, '=') || IsAllSameChar(line, index, '*');
			}

			// Loop while we have data
			span = scratchChars.AsSpan(0, commentsLength);
			bool firstLine = true;
			int maxNumWhitespaceToRemove = 0;
			int lastNonWhitespaceLength = 0;
			int outEndPos = 0;
			while (span.Length > 0)
			{

				// Extract the next line to process
				int eolIndex = span.IndexOf('\n');
				ReadOnlySpan<char> line = eolIndex != -1 ? span[..eolIndex] : span;
				span = eolIndex != -1 ? span[(eolIndex + 1)..] : new ReadOnlySpan<char>();
				line = line.TrimEnd();

				// Remove leading "*" and "* " in javadoc comments.
				if (javaDocStyle || cStyle)
				{
					// Find first non-whitespace character
					int pos = 0;
					while (pos < line.Length && UhtFCString.IsWhitespace(line[pos]))
					{
						++pos;
					}

					// Is it a *?
					if (pos < line.Length && line[pos] == '*')
					{
						// Eat next space as well
						if (pos + 1 < line.Length && UhtFCString.IsWhitespace(line[pos + 1]))
						{
							++pos;
						}

						line = line[(pos + 1)..];
					}
				}

				// Test to see if this is whitespace or line separator.  If also first line, then just skip
				bool isWhitespaceOrLineSeparator = IsWhitespaceOrLineSeparator(line);
				if (firstLine && isWhitespaceOrLineSeparator)
				{
					continue;
				}

				// Figure out how much whitespace is on the first line
				if (firstLine)
				{
					for (; maxNumWhitespaceToRemove < line.Length; maxNumWhitespaceToRemove++)
					{
						if (!UhtFCString.IsWhitespace(line[maxNumWhitespaceToRemove]))
						{
							break;
						}
					}
					line = line[maxNumWhitespaceToRemove..];
				}
				else
				{

					// Trim any leading whitespace
					for (int i = 0; i < maxNumWhitespaceToRemove && line.Length > 0; i++)
					{
						if (!UhtFCString.IsWhitespace(line[0]))
						{
							break;
						}
						line = line[1..];
					}

					scratchChars[outEndPos++] = '\n';
				}

				if (line.Length > 0 && !IsAllSameChar(line, 0, '='))
				{
					for (int i = 0; i < line.Length; i++)
					{
						scratchChars[outEndPos++] = line[i];
					}
				}

				if (!isWhitespaceOrLineSeparator)
				{
					lastNonWhitespaceLength = outEndPos;
				}
				firstLine = false;
			}

			outEndPos = lastNonWhitespaceLength;

			//@TODO: UCREMOVAL: Really want to trim an arbitrary number of newlines above and below, but keep multiple newlines internally
			// Make sure it doesn't start with a newline
			int outStartPos = 0;
			if (outStartPos < outEndPos && scratchChars[outStartPos] == '\n')
			{
				outStartPos++;
			}

			// Make sure it doesn't end with a dead newline
			if (outStartPos < outEndPos && scratchChars[outEndPos - 1] == '\n')
			{
				outEndPos--;
			}

			string results = scratchChars.AsSpan(outStartPos, outEndPos - outStartPos).ToString();
			ArrayPool<char>.Shared.Return(scratchChars);
			return results;
		}

		/// <summary>
		/// Remove any comments marked to be ignored
		/// </summary>
		/// <param name="comments">Buffer containing comments to be processed.  Comments are removed inline</param>
		/// <param name="inLength">Length of the comments</param>
		/// <returns>New length of the comments</returns>
		private static int RemoveIgnoreComments(char[] comments, int inLength)
		{
			ReadOnlySpan<char> span = comments.AsSpan(0, inLength);
			int commentStart, commentEnd;

			// Block comments go first
			while ((commentStart = span.IndexOf("/*~", StringComparison.Ordinal)) != -1)
			{
				commentEnd = span[commentStart..].IndexOf("*/", StringComparison.Ordinal);
				if (commentEnd != -1)
				{
					commentEnd += 2;
					Array.Copy(comments, commentStart + commentEnd, comments, commentStart, span.Length - (commentStart + commentEnd));
					span = span[..(span.Length - commentEnd)];
				}
				else
				{
					// This looks like an error - an unclosed block comment.
					break;
				}
			}

			// Leftover line comments go next
			while ((commentStart = span.IndexOf("//~", StringComparison.Ordinal)) != -1)
			{
				commentEnd = span[commentStart..].IndexOf("\n", StringComparison.Ordinal);
				if (commentEnd != -1)
				{
					commentEnd++;
					Array.Copy(comments, commentStart + commentEnd, comments, commentStart, span.Length - (commentStart + commentEnd));
					span = span[..(span.Length - commentEnd)];
				}
				else
				{
					span = span[..commentStart];
					break;
				}
			}

			return span.Length;
		}

		/// <summary>
		/// Remove any block comment markers
		/// </summary>
		/// <param name="comments">Buffer containing comments to be processed.  Comments are removed inline</param>
		/// <param name="inLength">Length of the comments</param>
		/// <param name="javaDocStyle">If true, we are parsing both java and c style.  This is a strange hack for //***__ comments which end up as __</param>
		/// <returns>New length of the comments</returns>
		private static int RemoveBlockCommentMarkers(char[] comments, int inLength, bool javaDocStyle)
		{
			int outPos = 0;
			int inPos = 0;
			while (inPos < inLength)
			{
				switch (comments[inPos])
				{
					case '\r':
						inPos++;
						break;
					case '/':
						// This block of code is mimicking the old pattern of replacing "/**" with "" followed by "/*" with "".  
						// Thus "//***" -> "/*" -> ""
						if (javaDocStyle && inPos + 4 < inLength && comments[inPos + 1] == '/' && comments[inPos + 2] == '*' && comments[inPos + 3] == '*' && comments[inPos + 4] == '*')
						{
							inPos += 5;
						}
						else if (inPos + 2 < inLength && comments[inPos + 1] == '*' && comments[inPos + 2] == '*')
						{
							inPos += 3;
						}
						else if (inPos + 1 < inLength && comments[inPos + 1] == '*')
						{
							inPos += 2;
						}
						else
						{
							comments[outPos++] = comments[inPos++];
						}
						break;
					case '*':
						if (inPos + 1 < inLength && comments[inPos + 1] == '/')
						{
							inPos += 2;
						}
						else
						{
							comments[outPos++] = comments[inPos++];
						}
						break;
					default:
						comments[outPos++] = comments[inPos++];
						break;
				}
			}
			return outPos;
		}

		/// <summary>
		/// Remove any line comment markers
		/// </summary>
		/// <param name="comments">Buffer containing comments to be processed.  Comments are removed inline</param>
		/// <param name="inLength">Length of the comments</param>
		/// <returns>New length of the comments</returns>
		private static int RemoveLineCommentMarkers(char[] comments, int inLength)
		{
			ReadOnlySpan<char> span = comments.AsSpan(0, inLength);
			int outPos = 0;
			int inPos = 0;
			while (inPos < inLength)
			{
				switch (comments[inPos])
				{
					case '\r':
						inPos++;
						break;
					case '/':
						if (inPos + 1 < inLength && comments[inPos + 1] == '/')
						{
							if (inPos + 2 < inLength && comments[inPos + 2] == '/')
							{
								inPos += 3;
							}
							else
							{
								inPos += 2;
							}
						}
						else
						{
							comments[outPos++] = comments[inPos++];
						}
						break;
					case '(':
						{
							if (span[inPos..].StartsWith("(cpptext)", StringComparison.Ordinal))
							{
								inPos += 9;
							}
							else
							{
								comments[outPos++] = comments[inPos++];
							}
						}
						break;
					default:
						comments[outPos++] = comments[inPos++];
						break;
				}
			}
			return outPos;
		}
	}

	/// <summary>
	/// Token recorder
	/// </summary>
	public struct UhtTokenRecorder : IDisposable
	{
		private readonly UhtCompilerDirective _compilerDirective;
		private readonly UhtParsingScope _scope;
		private readonly UhtFunction? _function;
		private bool _flushed;

		/// <summary>
		/// Construct a new recorder
		/// </summary>
		/// <param name="scope">Scope being parsed</param>
		/// <param name="initialToken">Initial toke nto add to the recorder</param>
		public UhtTokenRecorder(UhtParsingScope scope, ref UhtToken initialToken)
		{
			_scope = scope;
			_compilerDirective = _scope.HeaderParser.GetCurrentCompositeCompilerDirective();
			_function = null;
			_flushed = false;
			if (_scope.ScopeType is UhtClass or UhtPartial)
			{
				_scope.TokenReader.EnableRecording();
				_scope.TokenReader.RecordToken(ref initialToken);
			}
		}

		/// <summary>
		/// Create a new recorder
		/// </summary>
		/// <param name="scope">Scope being parsed</param>
		/// <param name="function">Function associated with the recorder</param>
		public UhtTokenRecorder(UhtParsingScope scope, UhtFunction function)
		{
			_scope = scope;
			_compilerDirective = _scope.HeaderParser.GetCurrentCompositeCompilerDirective();
			_function = function;
			_flushed = false;
			if (_scope.ScopeType is UhtClass or UhtPartial)
			{
				_scope.TokenReader.EnableRecording();
			}
		}

		/// <summary>
		/// Stop the recording
		/// </summary>
		public void Dispose()
		{
			Stop();
		}

		/// <summary>
		/// Stop the recording
		/// </summary>
		/// <returns>True if the recorded content was added to a class or partial</returns>
		public bool Stop()
		{
			if (!_flushed)
			{
				_flushed = true;
				if (_scope.ScopeType is UhtClass classObj)
				{
					classObj.AddDeclaration(_compilerDirective, _scope.TokenReader.RecordedTokens, _function);
					_scope.TokenReader.DisableRecording();
					return true;
				}
				else if (_scope.ScopeType is UhtPartial partialObj)
				{
					partialObj.AddDeclaration(_compilerDirective, _scope.TokenReader.RecordedTokens, _function);
					_scope.TokenReader.DisableRecording();
					return true;
				}
			}
			return false;
		}
	}
}
