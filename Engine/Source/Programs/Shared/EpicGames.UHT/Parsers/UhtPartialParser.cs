// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// UPARTIAL parser object
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtPartialParser
	{
		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UPARTIALKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUPartial(topScope, token);
		}

		[UhtKeyword(Extends = UhtTableNames.Partial)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult GENERATED_BODYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtPartial partial = (UhtPartial)topScope.ScopeType;
			if (topScope.AccessSpecifier != UhtAccessSpecifier.Public)
			{
				topScope.TokenReader.LogError($"{token.Value} must be in the public scope of '{partial.SourceName}', not private or protected.");
			}

			if (partial.MacroDeclaredLineNumber != -1)
			{
				topScope.TokenReader.LogError($"Multiple {token.Value} declarations found in '{partial.SourceName}'");
			}

			partial.MacroDeclaredLineNumber = topScope.TokenReader.InputLine;

			topScope.TokenReader
				.Require('(')
				.Require(')')
				.OptionalAttributes(false);
			return UhtParseResult.Handled;
		}
		#endregion

		private static UhtParseResult ParseUPartial(UhtParsingScope parentScope, UhtToken keywordToken)
		{
			UhtPartial partial = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, keywordToken.InputLine);

			// Use Partial keyword table to allow UFUNCTIONs without class-specific specifiers
			using UhtParsingScope topScope = new(parentScope, partial, parentScope.Session.GetKeywordTable(UhtTableNames.Partial), UhtAccessSpecifier.Public);

			const string ScopeName = "partial";
			using UhtMessageContext tokenContext = new(ScopeName);

			// Parse UPARTIAL(ClassName) - the owner class name is required in parentheses
			topScope.TokenReader
				.Require('(')
				.RequireIdentifier((ref UhtToken classToken) => partial.OwnerClassName = classToken.Value.ToString())
				.Require(')');

			// No specifiers for partials (unlike USTRUCT)
			// If we want to support specifiers later, we can add them here

			// Consume the struct keyword
			topScope.TokenReader.OptionalAttributes(false);
			topScope.TokenReader.Require("struct");
			topScope.TokenReader.OptionalAttributes(true);

			// Read the partial struct name and possible API macro name
			topScope.TokenReader.TryOptionalAPIMacro(out UhtToken apiMacroToken);
			partial.SourceName = topScope.TokenReader.GetIdentifier().Value.ToString();

			topScope.AddModuleRelativePathToMetaData();

			// Strip the name (partials must start with F)
			if (partial.SourceName[0] == 'F')
			{
				partial.EngineName = partial.SourceName[1..];
			}
			else
			{
				// This will be flagged later in the validation phase
				partial.EngineName = partial.SourceName;
				topScope.TokenReader.LogError($"partial struct '{partial.SourceName}' must start with 'F' prefix");
			}

			// Check for an empty engine name
			if (partial.EngineName.Length == 0)
			{
				topScope.TokenReader.LogError($"When compiling partial definition for '{partial.SourceName}', attempting to strip prefix results in an empty name. Did you leave off a prefix?");
			}

			// Partials cannot have inheritance
			topScope.TokenReader.Optional("final");

			// Check for inheritance - partials cannot inherit
			if (topScope.TokenReader.TryOptional(':'))
			{
				topScope.TokenReader.LogError($"Partial '{partial.SourceName}' cannot have inheritance. Partials cannot inherit from other structs or partials.");
			}

			// Add the comments here for compatibility with old UHT
			topScope.TokenReader.PeekToken();
			topScope.TokenReader.CommitPendingComments();
			topScope.AddFormattedCommentsAsTooltipMetaData();

			// Add metadata to indicate this is an partial
			partial.MetaData.Add("IsPartial", "true");
			partial.MetaData.Add("OwnerClass", partial.OwnerClassName);

			UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
			partial.DefineScope |= compilerDirective.GetDefaultDefineScopes();

			partial.Outer?.AddChild(partial);

			topScope.HeaderParser.ParseStatements('{', '}', true);

			topScope.TokenReader.Require(';');

			if (partial.MacroDeclaredLineNumber == -1)
			{
				topScope.TokenReader.LogError("Expected a GENERATED_BODY() at the start of the partial");
			}

			return UhtParseResult.Handled;
		}
	}
}
