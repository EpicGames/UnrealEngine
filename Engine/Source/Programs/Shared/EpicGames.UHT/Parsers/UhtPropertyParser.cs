// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// Options that customize the parsing of properties.
	/// </summary>
	[Flags]
	public enum UhtPropertyParseOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// Don't automatically mark properties as CPF_Const
		/// </summary>
		NoAutoConst = 1 << 0,

		/// <summary>
		/// Parse for the property macros
		/// </summary>
		ParseMacros = 1 << 1,

		/// <summary>
		/// If set, then the name of the property will be parsed with the type
		/// </summary>
		FunctionNameIncluded = 1 << 2,

		/// <summary>
		/// If set, then the name of the property will be parsed with the type
		/// </summary>
		NameIncluded = 1 << 3,

		/// <summary>
		/// When parsing delegates, the name is separated by commas
		/// </summary>
		CommaSeparatedName = 1 << 4,

		/// <summary>
		/// Multiple properties can be defined separated by commas
		/// </summary>
		List = 1 << 5,

		/// <summary>
		/// Don't add a return type to the property list (return values go at the end)
		/// </summary>
		DontAddReturn = 1 << 6,

		/// <summary>
		/// If set, add the module relative path to the parameter's meta data
		/// </summary>
		AddModuleRelativePath = 1 << 7,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtParsePropertyOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtPropertyParseOptions inFlags, UhtPropertyParseOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtPropertyParseOptions inFlags, UhtPropertyParseOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtPropertyParseOptions inFlags, UhtPropertyParseOptions testFlags, UhtPropertyParseOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Macro type
	/// </summary>
	public enum UhtPropertyMacroType
	{

		/// <summary>
		/// None found
		/// </summary>
		None,

		/// <summary>
		/// Array
		/// </summary>
		LayoutArray,

		/// <summary>
		/// Editor only array
		/// </summary>
		LayoutArrayEditorOnly,

		/// <summary>
		/// Bit field
		/// </summary>
		LayoutBitfield,

		/// <summary>
		/// Editor only bit field
		/// </summary>
		LayoutBitfieldEditorOnly,

		/// <summary>
		/// Field
		/// </summary>
		LayoutField,

		/// <summary>
		/// Editor only field
		/// </summary>
		LayoutFieldEditorOnly,

		/// <summary>
		/// Field with initializer
		/// </summary>
		LayoutFieldInitialized,

		/// <summary>
		/// Verse property that isn't native
		/// </summary>
		VerseField,
	}

	/// <summary>
	/// Extensions for working with the macro type
	/// </summary>
	public static class UhtPropertyMacroTypeExtensions
	{

		/// <summary>
		/// Return true if the type is editor only
		/// </summary>
		/// <param name="propertyMacroType">Property macro type</param>
		/// <returns>True if editor only</returns>
		public static bool IsEditorOnly(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.LayoutArrayEditorOnly or
				UhtPropertyMacroType.LayoutBitfieldEditorOnly or
				UhtPropertyMacroType.LayoutFieldEditorOnly => true,
				_ => false,
			};

		/// <summary>
		/// Return true if the type is a bit field
		/// </summary>
		/// <param name="propertyMacroType">Property macro type</param>
		/// <returns>True if bit field</returns>
		public static bool IsBitfield(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.LayoutBitfield or
				UhtPropertyMacroType.LayoutBitfieldEditorOnly => true,
				_ => false,
			};

		/// <summary>
		/// Return true if the type is an array
		/// </summary>
		/// <param name="propertyMacroType">Property macro type</param>
		/// <returns>True if array</returns>
		public static bool IsArray(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.LayoutArray or
				UhtPropertyMacroType.LayoutArrayEditorOnly => true,
				_ => false,
			};

		/// <summary>
		/// Return true if the type has an initializer
		/// </summary>
		/// <param name="propertyMacroType">Layout macro type</param>
		/// <returns>True if it has an initializer</returns>
		public static bool HasInitializer(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.LayoutFieldInitialized or
				UhtPropertyMacroType.VerseField => true,
				_ => false,
			};

		/// <summary>
		/// Return true if the type is a verse field and not native
		/// </summary>
		/// <param name="propertyMacroType"></param>
		/// <returns></returns>
		public static bool IsVerseField(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.VerseField => true,
				_ => false,
			};

		/// <summary>
		/// Return the property macro name
		/// </summary>
		/// <param name="propertyMacroType">Type in question</param>
		/// <returns>Macro name</returns>
		/// <exception cref="UhtIceException">Thrown if the macro type is none or invalid</exception>
		public static StringView MacroName(this UhtPropertyMacroType propertyMacroType) =>
			propertyMacroType switch
			{
				UhtPropertyMacroType.LayoutArray => "LAYOUT_ARRAY",
				UhtPropertyMacroType.LayoutArrayEditorOnly => "LAYOUT_ARRAY_EDITORONLY",
				UhtPropertyMacroType.LayoutBitfield => "LAYOUT_BITFIELD",
				UhtPropertyMacroType.LayoutBitfieldEditorOnly => "LAYOUT_BITFIELD_EDITORONLY",
				UhtPropertyMacroType.LayoutField => "LAYOUT_FIELD",
				UhtPropertyMacroType.LayoutFieldEditorOnly => "LAYOUT_FIELD_EDITORONLY",
				UhtPropertyMacroType.LayoutFieldInitialized => "LAYOUT_FIELD_INITIALIZED",
				UhtPropertyMacroType.VerseField => "UE_VERSE_FIELD",
				_ => throw new UhtIceException("Invalid macro name for UhtPropertyMacroType"),
			};

		/// <summary>
		/// Return the macro name and value
		/// </summary>
		/// <param name="propertyMacroType">Macro name</param>
		/// <returns>Name and type</returns>
		public static KeyValuePair<StringView, UhtPropertyMacroType> MacroNameAndValue(this UhtPropertyMacroType propertyMacroType) =>
			new KeyValuePair<StringView, UhtPropertyMacroType>(propertyMacroType.MacroName(), propertyMacroType);
	}

	/// <summary>
	/// Delegate invoked to handle a parsed property 
	/// </summary>
	/// <param name="topScope">Scope being parsed</param>
	/// <param name="property">Property just parsed</param>
	/// <param name="nameToken">Name of the property</param>
	/// <param name="propertyMacroType">Property macro type</param>
	public delegate void UhtPropertyDelegate(UhtParsingScope topScope, UhtProperty property, ref UhtToken nameToken, UhtPropertyMacroType propertyMacroType);

	/// <summary>
	/// Context for property specifier parsing
	/// </summary>
	public class UhtPropertySpecifierContext : UhtSpecifierContext
	{
		/// <summary>
		/// The property settings being parsed
		/// </summary>
		public UhtPropertySettings PropertySettings { get; } = new UhtPropertySettings();

		/// <summary>
		/// If true, editor specifier seen
		/// </summary>
		public bool SeenEditSpecifier { get; set; } = false;

		/// <summary>
		/// If true, blueprint write specifier seen
		/// </summary>
		public bool SeenBlueprintWriteSpecifier { get; set; } = false;

		/// <summary>
		/// If true, blueprint readonly specifier seen
		/// </summary>
		public bool SeenBlueprintReadOnlySpecifier { get; set; } = false;

		/// <summary>
		/// If true, blueprint getter specifier seen
		/// </summary>
		public bool SeenBlueprintGetterSpecifier { get; set; } = false;
	}

	/// <summary>
	/// Helper class thread specified object cache
	/// </summary>
	public readonly struct UhtThreadBorrower<T> : IDisposable where T : new()
	{
		private static readonly ThreadLocal<List<T>> s_tls = new(() => []);
		private readonly T _instance;

		/// <summary>
		/// The borrowed instance
		/// </summary>
		public T Instance => _instance;

		/// <summary>
		/// Request a context
		/// </summary>
		public UhtThreadBorrower(bool _) // argument needed for vs2019
		{
			List<T> cache = s_tls.Value!;
			if (cache.Count == 0)
			{
				_instance = new();
			}
			else
			{
				_instance = cache[^1];
				cache.RemoveAt(cache.Count - 1);
			}
		}

		/// <summary>
		/// Return the borrowed buffer to the cache
		/// </summary>
		public void Dispose()
		{
			s_tls.Value!.Add(_instance);
		}
	}

	/// <summary>
	/// A parsed property is a property that was parsed but couldn't yet be resolved.  It retains the list of tokens needed
	/// to resolve the type of the property.  It will be replaced with the resolved property type during property resolution.
	/// </summary>
	public class UhtPreResolveProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "UHTParsedProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "invalid";

		/// <inheritdoc/>
		protected override string PGetMacroText => "invalid";

		/// <summary>
		/// Construct a new property to be resolved
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtPreResolveProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtPreResolveProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterGet(StringBuilder builder)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			throw new NotImplementedException();
		}
	}

	internal record struct UhtPropertyCategoryExtraContext(UhtPropertyCategory Category) : IUhtMessageExtraContext
	{
		#region IMessageExtraContext implementation
		/// <inheritdoc/>
		public IEnumerable<object?>? MessageExtraContext
		{
			get
			{
				Stack<object?> extraContext = new(1);
				extraContext.Push(Category.GetHintText());
				return extraContext;
			}
		}
		#endregion
	}

	/// <summary>
	/// Property parser
	/// </summary>
	public static class UhtPropertyParser
	{
		private static readonly Dictionary<StringView, UhtPropertyMacroType> s_macroTypes = new(
		[
			UhtPropertyMacroType.LayoutArray.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutArrayEditorOnly.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutBitfield.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutBitfieldEditorOnly.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutField.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutFieldEditorOnly.MacroNameAndValue(),
			UhtPropertyMacroType.LayoutFieldInitialized.MacroNameAndValue(),
			UhtPropertyMacroType.VerseField.MacroNameAndValue(),
		]);

		/// <summary>
		/// Parse the property
		/// </summary>
		/// <param name="topScope">Current parsing scope</param>
		/// <param name="disallowPropertyFlags">Flags to be disallowed</param>
		/// <param name="options">Parsing options</param>
		/// <param name="category">Property category</param>
		/// <param name="propertyDelegate">Delegate to be invoked after property has been parsed</param>
		/// <returns>The property parser</returns>
		public static void Parse(UhtParsingScope topScope, EPropertyFlags disallowPropertyFlags, UhtPropertyParseOptions options, UhtPropertyCategory category, UhtPropertyDelegate propertyDelegate)
		{
			// Initialize the property context
			using UhtThreadBorrower<UhtPropertySpecifierContext> borrower = new(true);
			UhtPropertySpecifierContext specifierContext = borrower.Instance;
			specifierContext.Type = topScope.ScopeType;
			specifierContext.TokenReader = topScope.TokenReader;
			specifierContext.AccessSpecifier = topScope.AccessSpecifier;
			specifierContext.MessageSite = topScope.TokenReader;
			specifierContext.PropertySettings.Reset(specifierContext.Type, 0, category, disallowPropertyFlags);
			specifierContext.MetaData = specifierContext.PropertySettings.MetaData;
			specifierContext.MetaNameIndex = UhtMetaData.IndexNone;
			specifierContext.SeenEditSpecifier = false;
			specifierContext.SeenBlueprintWriteSpecifier = false;
			specifierContext.SeenBlueprintReadOnlySpecifier = false;
			specifierContext.SeenBlueprintGetterSpecifier = false;

			UhtPropertyCategoryExtraContext extraContext = new(category);
			using UhtMessageContext tokenContext = new(extraContext);
			ParseInternal(topScope, options, specifierContext, propertyDelegate);
		}

		/// <summary>
		/// Resolve the given property.  This method will resolve any immediate property during the parsing phase or 
		/// resolve any previously parsed property to the final version.
		/// </summary>
		/// <param name="resolvePhase">Used to detect if the property is being parsed or resolved</param>
		/// <param name="propertySettings">The property settings.</param>
		/// <returns></returns>
		private static UhtProperty? ResolveProperty(UhtPropertyResolvePhase resolvePhase, UhtPropertySettings propertySettings)
		{
			Debug.Assert(propertySettings.TypeTokens.AllTokens.Length > 0);
			using UhtTokenReplayReaderBorrower borrowedReader = new(propertySettings.Outer, propertySettings.Outer.HeaderFile.Data.Memory, propertySettings.TypeTokens.AllTokens, UhtTokenType.EndOfType);
			UhtPropertyResolveArgs args = new(resolvePhase, propertySettings, borrowedReader.Reader);
			return args.ResolveProperty();
		}

		private static readonly ThreadLocal<UhtPropertySettings> s_tlsPropertySettings = new(() => { return new UhtPropertySettings(); });

		/// <summary>
		/// Given a type with children, resolve any children that couldn't be resolved during the parsing phase.
		/// </summary>
		/// <param name="type">The type with children</param>
		/// <param name="options">Parsing options</param>
		public static void ResolveChildren(UhtType type, UhtPropertyParseOptions options)
		{
			UhtPropertyOptions propertyOptions = UhtPropertyOptions.None;
			if (options.HasAnyFlags(UhtPropertyParseOptions.NoAutoConst))
			{
				propertyOptions |= UhtPropertyOptions.NoAutoConst;
			}
			bool inSymbolTable = type.EngineType.AddChildrenToSymbolTable();

			UhtPropertySettings? propertySettings = s_tlsPropertySettings.Value;
			if (propertySettings == null)
			{
				throw new UhtIceException("Unable to acquire threaded property settings");
			}

			for (int index = 0; index < type.Children.Count; ++index)
			{
				if (type.Children[index] is UhtPreResolveProperty property)
				{
					propertySettings.Reset(property, propertyOptions);
					UhtProperty? resolved = ResolveProperty(UhtPropertyResolvePhase.Resolving, propertySettings);
					if (resolved != null)
					{
						if (inSymbolTable && resolved != property)
						{
							type.Session.ReplaceTypeInSymbolTable(property, resolved);
						}
						type.Children[index] = resolved;
					}
				}
			}
		}

		/// <summary>
		/// Resolve the given property and finish parsing any postfix settings
		/// </summary>
		/// <param name="args">Arguments for the resolution process</param>
		/// <param name="propertyType">Property type used to resolve the property</param>
		/// <returns>Created property or null if it could not be resolved</returns>
		/// <exception cref="UhtTokenException"></exception>
		public static UhtProperty? ResolvePropertyType(UhtPropertyResolveArgs args, UhtPropertyType propertyType)
		{
			UhtProperty? outProperty = propertyType.Delegate(args);
			if (outProperty == null)
			{
				return null;
			}

			// If this is a simple type, skip the type
			if (propertyType.Options.HasAnyFlags(UhtPropertyTypeOptions.Simple))
			{
				if (!args.SkipExpectedType())
				{
					return null;
				}
			}

			// Handle any trailing const
			if (outProperty.PropertyCategory == UhtPropertyCategory.Member)
			{
				//@TODO: UCREMOVAL: 'const' member variables that will get written post-construction by default properties
				UhtClass? outerClass = outProperty.Outer as UhtClass;
				if (outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.Const))
				{
					// Eat a 'not quite truthful' const after the type; auto-generated for member variables of const classes.
					if (args.TokenReader.TryOptional("const"))
					{
						outProperty.MetaData.Add(UhtNames.NativeConst, "");
					}
				}
			}
			else
			{
				if (args.TokenReader.TryOptional("const"))
				{
					outProperty.MetaData.Add(UhtNames.NativeConst, "");
					outProperty.PropertyFlags |= EPropertyFlags.ConstParm;
				}
			}

			// Check for unexpected '*'
			if (args.TokenReader.TryOptional('*'))
			{
				args.TokenReader.LogError($"Inappropriate '*' on variable of type '{outProperty.GetUserFacingDecl()}', cannot have an exposed pointer to this type.");
			}

			// Arrays are passed by reference but are only implicitly so; setting it explicitly could cause a problem with replicated functions
			if (args.TokenReader.TryOptional('&'))
			{
				switch (outProperty.PropertyCategory)
				{
					case UhtPropertyCategory.RegularParameter:
					case UhtPropertyCategory.Return:
						outProperty.PropertyFlags |= EPropertyFlags.OutParm;

						//@TODO: UCREMOVAL: How to determine if we have a ref param?
						if (outProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
						{
							outProperty.PropertyFlags |= EPropertyFlags.ReferenceParm;
						}
						break;

					case UhtPropertyCategory.ReplicatedParameter:
						outProperty.PropertyFlags |= EPropertyFlags.ReferenceParm;
						break;

					default:
						break;
				}

				if (outProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
				{
					outProperty.RefQualifier = UhtPropertyRefQualifier.ConstRef;
				}
				else
				{
					outProperty.RefQualifier = UhtPropertyRefQualifier.NonConstRef;
				}
			}

			if (!args.IsTemplate)
			{
				if (!args.TokenReader.IsEOF)
				{
					throw new UhtTokenException(args.TokenReader, args.TokenReader.PeekToken(), "end of type declaration");
				}
			}
			else
			{
				ref UhtToken token = ref args.TokenReader.PeekToken();
				if (!token.IsSymbol(',') && !token.IsSymbol('>'))
				{
					throw new UhtTokenException(args.TokenReader, args.TokenReader.PeekToken(), "end of type declaration");
				}
			}
			return outProperty;
		}

		private static void ParseInternal(UhtParsingScope topScope, UhtPropertyParseOptions options, UhtPropertySpecifierContext specifierContext, UhtPropertyDelegate propertyDelegate)
		{
			UhtPropertySettings propertySettings = specifierContext.PropertySettings;
			IUhtTokenReader tokenReader = topScope.TokenReader;

			propertySettings.LineNumber = tokenReader.InputLine;

			if (propertySettings.PropertyCategory == UhtPropertyCategory.Member)
			{
				UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
				if (compilerDirective.HasAnyFlags(UhtCompilerDirective.WithEditorOnlyData))
				{
					propertySettings.PropertyFlags |= EPropertyFlags.EditorOnly;
					propertySettings.DefineScope |= UhtDefineScope.EditorOnlyData;
				}
				else if (compilerDirective.HasAnyFlags(UhtCompilerDirective.WithEditor))
				{
					// Checking for this error is a bit tricky given legacy code.  
					// 1) If already wrapped in WITH_EDITORONLY_DATA (see above), then we ignore the error via the else 
					// 2) Ignore any module that is an editor module
					UHTManifest.Module module = topScope.Module.Module;
					bool isEditorModule =
						module.ModuleType == UHTModuleType.EngineEditor ||
						module.ModuleType == UHTModuleType.GameEditor ||
						module.ModuleType == UHTModuleType.EngineUncooked ||
						module.ModuleType == UHTModuleType.GameUncooked;
					if (!isEditorModule)
					{
						tokenReader.LogError("UProperties should not be wrapped by WITH_EDITOR, use WITH_EDITORONLY_DATA instead.");
					}
				}
				propertySettings.DefineScope |= (compilerDirective & ~(UhtCompilerDirective.WithEditorOnlyData | UhtCompilerDirective.WithEditor)).GetDefaultDefineScopes();
			}

			// Parse type information including UPARAM that might appear in template arguments
			PreParseType(specifierContext, false);

			// Swallow inline keywords
			if (propertySettings.PropertyCategory == UhtPropertyCategory.Return)
			{
				tokenReader
					.Optional("inline")
					.Optional("FORCENOINLINE")
					.OptionalStartsWith("FORCEINLINE");
			}

			// Handle property macros
			bool hasWrapperBrackets = false;
			UhtPropertyMacroType propertyMacroType = UhtPropertyMacroType.None;
			if (options.HasAnyFlags(UhtPropertyParseOptions.ParseMacros))
			{
				ref UhtToken propertyToken = ref tokenReader.PeekToken();
				if (propertyToken.IsIdentifier())
				{
					if (s_macroTypes.TryGetValue(propertyToken.Value, out propertyMacroType))
					{
						tokenReader.ConsumeToken();
						tokenReader.Require('(');
						hasWrapperBrackets = tokenReader.TryOptional('(');
						if (propertyMacroType.IsEditorOnly())
						{
							propertySettings.PropertyFlags |= EPropertyFlags.EditorOnly;
							propertySettings.DefineScope |= UhtDefineScope.EditorOnlyData;
						}
					}
				}

				// This exists as a compatibility "shim" with UHT4/5.0.  If the fetched token wasn't an identifier,
				// it wasn't returned to the tokenizer.  So, just consume the token here.  In theory, this should be
				// removed once we have a good deprecated system.
				//@TODO - deprecate
				else // if (propertyToken.IsSymbol(';'))
				{
					tokenReader.ConsumeToken();
				}
			}

			//@TODO: Should flag as settable from a const context, but this is at least good enough to allow use for C++ land
			tokenReader.Optional("mutable");

			// Gather the type tokens and possibly the property name.
			UhtTypeTokens typeTokens = UhtTypeTokens.Gather(tokenReader);

			// Verify we at least have one type
			if (typeTokens.AllTokens.Length < 1)
			{
				throw new UhtException(tokenReader, $"{propertySettings.PropertyCategory.GetHintText()}: Missing variable type or name");
			}

			// Consume the wrapper brackets.  This is just an extra set
			if (hasWrapperBrackets)
			{
				tokenReader.Require(')');
			}

			// Check for any disallowed flags
			if (propertySettings.PropertyFlags.HasAnyFlags(propertySettings.DisallowPropertyFlags))
			{
				EPropertyFlags extraFlags = propertySettings.PropertyFlags & propertySettings.DisallowPropertyFlags;
				tokenReader.LogError($"Specified type modifiers not allowed here '{String.Join(" | ", extraFlags.ToStringList())}'");
			}

			if (options.HasAnyFlags(UhtPropertyParseOptions.AddModuleRelativePath))
			{
				UhtParsingScope.AddModuleRelativePathToMetaData(propertySettings.MetaData, topScope.HeaderFile);
			}

			// Fetch the name of the property, bit field and array size
			if (propertyMacroType != UhtPropertyMacroType.None)
			{
				tokenReader.Require(',');
				UhtToken nameToken = tokenReader.GetIdentifier();
				if (propertyMacroType.IsArray())
				{
					tokenReader.Require(',');
					RequireArray(tokenReader, propertySettings, ref nameToken, ')');
					tokenReader.Require(')');
				}
				else if (propertyMacroType.IsBitfield())
				{
					tokenReader.Require(',');
					RequireBitfield(tokenReader, propertySettings, ref nameToken);
					tokenReader.Require(')');
				}
				else if (propertyMacroType.HasInitializer())
				{
					tokenReader.Require(',');
					tokenReader.SkipBrackets('(', ')', 1); // consumes ending ) too
				}
				else
				{
					tokenReader.Require(')');
				}

				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);
			}
			else if (options.HasAnyFlags(UhtPropertyParseOptions.List))
			{
				UhtToken nameToken = typeTokens.ExtractTrailingIdentifier(tokenReader, propertySettings);
				CheckForOptionalParts(tokenReader, propertySettings, ref nameToken);

				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);

				// Check for unsupported comma delimited properties
				if (tokenReader.TryOptional(','))
				{
					throw new UhtException(tokenReader, $"Comma delimited properties are not supported");
				}
			}
			else if (options.HasAnyFlags(UhtPropertyParseOptions.CommaSeparatedName))
			{
				tokenReader.Require(',');
				UhtToken nameToken = tokenReader.GetIdentifier();
				CheckForOptionalParts(tokenReader, propertySettings, ref nameToken);
				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);
			}
			else if (options.HasAnyFlags(UhtPropertyParseOptions.FunctionNameIncluded))
			{
				UhtToken nameToken = typeTokens.AllTokens.Span[^1];
				nameToken.Value = new StringView("Function");
				if (CheckForOptionalParts(tokenReader, propertySettings, ref nameToken))
				{
					nameToken = tokenReader.GetIdentifier("function name");
				}
				else
				{
					nameToken = typeTokens.ExtractTrailingIdentifier(tokenReader, propertySettings);
				}
				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);
			}
			else if (options.HasAnyFlags(UhtPropertyParseOptions.NameIncluded))
			{
				UhtToken nameToken = typeTokens.ExtractTrailingIdentifier(tokenReader, propertySettings);
				CheckForOptionalParts(tokenReader, propertySettings, ref nameToken);
				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);
			}
			else
			{
				UhtToken nameToken = new();
				CheckForOptionalParts(tokenReader, propertySettings, ref nameToken);
				Finalize(topScope, options, specifierContext, ref nameToken, typeTokens, propertyMacroType, propertyDelegate);
			}
		}

		/// <summary>
		/// Parse the type elements excluding the type itself.
		/// </summary>
		/// <param name="specifierContext">Context of what is being parsed</param>
		/// <param name="isTemplateArgument">If true, this is part of a template argument</param>
		/// <exception cref="UhtIceException"></exception>
		/// <exception cref="UhtException"></exception>
		public static void PreParseType(UhtPropertySpecifierContext specifierContext, bool isTemplateArgument)
		{
			UhtPropertySettings propertySettings = specifierContext.PropertySettings;
			IUhtTokenReader tokenReader = specifierContext.TokenReader;
			UhtSession session = specifierContext.Type.Session;

			// We parse specifiers when:
			//
			// 1. This is the start of a member property (but not a template)
			// 2. The UPARAM identifier is found
			bool isMember = propertySettings.PropertyCategory == UhtPropertyCategory.Member;
			bool parseSpecifiers = (isMember && !isTemplateArgument) || tokenReader.TryOptional("UPARAM");

			UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, "Variable",
				isMember ? session.GetSpecifierTable(UhtTableNames.PropertyMember) : session.GetSpecifierTable(UhtTableNames.PropertyArgument));
			if (parseSpecifiers)
			{
				specifiers.ParseSpecifiers();
			}

			if (propertySettings.PropertyCategory != UhtPropertyCategory.Member && !isTemplateArgument)
			{
				// const before the variable type support (only for params)
				if (tokenReader.TryOptional("const"))
				{
					propertySettings.PropertyFlags |= EPropertyFlags.ConstParm;
					propertySettings.MetaData.Add(UhtNames.NativeConst, "");
				}
			}

			// Process the specifiers
			if (parseSpecifiers)
			{
				specifiers.ParseDeferred();
			}

			// If we saw a BlueprintGetter but did not see BlueprintSetter or 
			// or BlueprintReadWrite then treat as BlueprintReadOnly
			if (specifierContext.SeenBlueprintGetterSpecifier && !specifierContext.SeenBlueprintWriteSpecifier)
			{
				propertySettings.PropertyFlags |= EPropertyFlags.BlueprintReadOnly;
			}

			if (propertySettings.MetaData.ContainsKey(UhtNames.ExposeOnSpawn))
			{
				propertySettings.PropertyFlags |= EPropertyFlags.ExposeOnSpawn;
			}

			if (!isTemplateArgument)
			{
				UhtAccessSpecifier accessSpecifier = specifierContext.AccessSpecifier;
				if (accessSpecifier == UhtAccessSpecifier.Public || propertySettings.PropertyCategory != UhtPropertyCategory.Member)
				{
					propertySettings.PropertyFlags &= ~EPropertyFlags.Protected;
					propertySettings.PropertyExportFlags |= UhtPropertyExportFlags.Public;
					propertySettings.PropertyExportFlags &= ~(UhtPropertyExportFlags.Private | UhtPropertyExportFlags.Protected);

					propertySettings.PropertyFlags &= ~EPropertyFlags.NativeAccessSpecifiers;
					propertySettings.PropertyFlags |= EPropertyFlags.NativeAccessSpecifierPublic;
				}
				else if (accessSpecifier == UhtAccessSpecifier.Protected)
				{
					propertySettings.PropertyFlags |= EPropertyFlags.Protected;
					propertySettings.PropertyExportFlags |= UhtPropertyExportFlags.Protected;
					propertySettings.PropertyExportFlags &= ~(UhtPropertyExportFlags.Public | UhtPropertyExportFlags.Private);

					propertySettings.PropertyFlags &= ~EPropertyFlags.NativeAccessSpecifiers;
					propertySettings.PropertyFlags |= EPropertyFlags.NativeAccessSpecifierProtected;
				}
				else if (accessSpecifier == UhtAccessSpecifier.Private)
				{
					propertySettings.PropertyFlags &= ~EPropertyFlags.Protected;
					propertySettings.PropertyExportFlags |= UhtPropertyExportFlags.Private;
					propertySettings.PropertyExportFlags &= ~(UhtPropertyExportFlags.Public | UhtPropertyExportFlags.Protected);

					propertySettings.PropertyFlags &= ~EPropertyFlags.NativeAccessSpecifiers;
					propertySettings.PropertyFlags |= EPropertyFlags.NativeAccessSpecifierPrivate;
				}
				else
				{
					throw new UhtIceException("Unknown access level");
				}
			}
		}

		/// <summary>
		/// Finish creating the property
		/// </summary>
		/// <param name="topScope">Top most scope being parsed</param>
		/// <param name="options">Parsing options</param>
		/// <param name="specifierContext">Context of the property being parsed</param>
		/// <param name="nameToken">The name of the property</param>
		/// <param name="typeTokens">Series of tokens that represent the type</param>
		/// <param name="propertyMacroType">Optional property macro type being parsed</param>
		/// <param name="propertyDelegate">Delegate to invoke when processing has been completed</param>
		/// <returns>The newly created property.  During the parsing phase, this will often be a temporary property if the type references engine types.</returns>
		private static UhtProperty Finalize(UhtParsingScope topScope, UhtPropertyParseOptions options, UhtPropertySpecifierContext specifierContext, ref UhtToken nameToken,
			UhtTypeTokens typeTokens, UhtPropertyMacroType propertyMacroType, UhtPropertyDelegate propertyDelegate)
		{
			UhtPropertySettings propertySettings = specifierContext.PropertySettings;
			propertySettings.TypeTokens = new(typeTokens, 0);
			IUhtTokenReader tokenReader = specifierContext.TokenReader;

			propertySettings.SourceName = propertySettings.PropertyCategory == UhtPropertyCategory.Return ? "ReturnValue" : nameToken.Value.ToString();

			// Try to resolve the property using any immediate mode property types
			UhtProperty newProperty = ResolveProperty(UhtPropertyResolvePhase.Parsing, propertySettings) ?? new UhtPreResolveProperty(propertySettings);

			// Force the category in non-engine projects
			if (newProperty.PropertyCategory == UhtPropertyCategory.Member)
			{
				if (!newProperty.Module.IsPartOfEngine &&
					newProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.Edit | EPropertyFlags.BlueprintVisible) &&
					!newProperty.MetaData.ContainsKey(UhtNames.Category))
				{
					newProperty.MetaData.Add(UhtNames.Category, newProperty.Outer!.EngineName);
				}
			}

			// Check to see if the variable is deprecated, and if so set the flag
			{
				int deprecatedIndex = newProperty.SourceName.IndexOf("_DEPRECATED", StringComparison.Ordinal);
				int nativizedPropertyPostfixIndex = newProperty.SourceName.IndexOf("__pf", StringComparison.Ordinal); //@TODO: check OverrideNativeName in Meta Data, to be sure it's not a random occurrence of the "__pf" string.
				bool ignoreDeprecatedWord = (nativizedPropertyPostfixIndex != -1) && (nativizedPropertyPostfixIndex > deprecatedIndex);
				if ((deprecatedIndex != -1) && !ignoreDeprecatedWord)
				{
					if (deprecatedIndex != newProperty.SourceName.Length - 11)
					{
						tokenReader.LogError("Deprecated variables must end with _DEPRECATED");
					}

					// We allow deprecated properties in blueprints that have getters and setters assigned as they may be part of a backwards compatibility path
					bool blueprintVisible = newProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible);
					bool warnOnGetter = blueprintVisible && !newProperty.MetaData.ContainsKey(UhtNames.BlueprintGetter);
					bool warnOnSetter = blueprintVisible && !newProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintReadOnly) && !newProperty.MetaData.ContainsKey(UhtNames.BlueprintSetter);

					if (warnOnGetter)
					{
						tokenReader.LogWarning($"{newProperty.PropertyCategory.GetHintText()}: Deprecated property '{newProperty.SourceName}' should not be marked as blueprint visible without having a BlueprintGetter");
					}

					if (warnOnSetter)
					{
						tokenReader.LogWarning($"{newProperty.PropertyCategory.GetHintText()}: Deprecated property '{newProperty.SourceName}' should not be marked as blueprint writable without having a BlueprintSetter");
					}

					// Warn if a deprecated property is visible
					if (newProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.Edit | EPropertyFlags.EditConst) || // Property is marked as editable
						(!blueprintVisible && newProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintReadOnly) &&
						!newProperty.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.ImpliedBlueprintPure))) // Is BPRO, but not via Implied Flags and not caught by Getter/Setter path above
					{
						tokenReader.LogWarning($"{newProperty.PropertyCategory.GetHintText()}: Deprecated property '{newProperty.SourceName}' should not be marked as visible or editable");
					}

					newProperty.PropertyFlags |= EPropertyFlags.Deprecated;
					newProperty.EngineName = newProperty.SourceName[..deprecatedIndex];
				}
			}

			// Try gathering metadata for member fields
			if (newProperty.PropertyCategory == UhtPropertyCategory.Member)
			{
				UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, newProperty.SourceName,
					specifierContext.Type.Session.GetSpecifierTable(UhtTableNames.PropertyMember));
				specifiers.ParseFieldMetaData();
				tokenReader.SkipWhitespaceAndComments(); //TODO - old UHT compatibility.  Commented out initializers can cause comment/tooltip to be used as meta data.
				tokenReader.CommitPendingComments(); //TODO - old UHT compatibility.  Commented out initializers can cause comment/tooltip to be used as meta data.
				topScope.AddFormattedCommentsAsTooltipMetaData(newProperty);
			}

			// Adjust property settings for property macros.
			if (propertyMacroType.IsVerseField())
			{
				newProperty.PropertyExportFlags |= UhtPropertyExportFlags.VerseField;
			}

			// Adjust the name for verse
			(bool wasMangled, string result) = newProperty.GetMangledEngineName();
			if (wasMangled)
			{
				if (!newProperty.MetaData.ContainsKey(UhtNames.DisplayName))
				{
					newProperty.MetaData.Add(UhtNames.DisplayName, newProperty.StrippedEngineName.ToString());
				}
				newProperty.EngineName = result;
			}

			propertyDelegate(topScope, newProperty, ref nameToken, propertyMacroType);

			// Void properties don't get added when they are the return value
			if (newProperty.PropertyCategory != UhtPropertyCategory.Return || !options.HasAnyFlags(UhtPropertyParseOptions.DontAddReturn))
			{
				topScope.ScopeType.AddChild(newProperty);
			}
			return newProperty;
		}

		private static bool CheckForOptionalParts(IUhtTokenReader tokenReader, UhtPropertySettings propertySettings, ref UhtToken nameToken)
		{
			bool gotOptionalParts = false;

			if (tokenReader.TryOptional('['))
			{
				RequireArray(tokenReader, propertySettings, ref nameToken, ']');
				tokenReader.Require(']');
				gotOptionalParts = true;
			}

			if (tokenReader.TryOptional(':'))
			{
				RequireBitfield(tokenReader, propertySettings, ref nameToken);
				gotOptionalParts = true;
			}
			return gotOptionalParts;
		}

		private static void RequireBitfield(IUhtTokenReader tokenReader, UhtPropertySettings propertySettings, ref UhtToken nameToken)
		{
			if (!tokenReader.TryOptionalConstInt(out int bitfieldSize) || bitfieldSize != 1)
			{
				throw new UhtException(tokenReader, $"Bad or missing bit field size for '{nameToken.Value}', must be 1.");
			}
			propertySettings.IsBitfield = true;
		}

		private static void RequireArray(IUhtTokenReader tokenReader, UhtPropertySettings propertySettings, ref UhtToken nameToken, char terminator)
		{
			// Ignore how the actual array dimensions are actually defined - we'll calculate those with the compiler anyway.
			propertySettings.ArrayDimensions = tokenReader.GetRawString(terminator, UhtRawStringOptions.DontConsumeTerminator).ToString();
			if (propertySettings.ArrayDimensions.Length == 0)
			{
				throw new UhtException(tokenReader, $"{propertySettings.PropertyCategory.GetHintText()} {nameToken.Value}: Missing array dimensions or terminating '{terminator}'");
			}
		}
	}

	[UnrealHeaderTool]
	static class PropertyKeywords
	{
		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Class)]
		[UhtKeyword(Extends = UhtTableNames.ScriptStruct)]
		[UhtKeyword(Extends = UhtTableNames.VModule)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult UPROPERTYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtPropertyParseOptions options = UhtPropertyParseOptions.ParseMacros | UhtPropertyParseOptions.List | UhtPropertyParseOptions.AddModuleRelativePath;
			UhtPropertyParser.Parse(topScope, EPropertyFlags.ParmFlags, options, UhtPropertyCategory.Member, s_propertyDelegate);
			topScope.TokenReader.Require(';');

			// C++ UHT TODO - Skip any extra ';'.  This can be removed if we remove UhtHeaderfileParser.ParserStatement generating errors
			// when extra ';' are found.  Oddly, UPROPERTY specifically skips extra ';'
			while (true)
			{
				UhtToken nextToken = topScope.TokenReader.PeekToken();
				if (!nextToken.IsSymbol(';'))
				{
					break;
				}
				topScope.TokenReader.ConsumeToken();
			}
			return UhtParseResult.Handled;
		}
		#endregion

		private static readonly UhtPropertyDelegate s_propertyDelegate = PropertyParsed;

		private static void PropertyParsed(UhtParsingScope topScope, UhtProperty property, ref UhtToken nameToken, UhtPropertyMacroType propertyMacroType)
		{
			IUhtTokenReader tokenReader = topScope.TokenReader;

			// Skip any initialization
			if (tokenReader.TryOptional('='))
			{
				tokenReader.SkipUntil(';');
			}
			else if (tokenReader.TryOptional('{'))
			{
				tokenReader.SkipBrackets('{', '}', 1);
			}
		}
	}

	[UnrealHeaderTool]
	static class UhtVerseTypeParser
	{
		[UhtPropertyType(Keyword = "TVal", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? TValType(UhtPropertyResolveArgs args)
		{
			return args.ParseWrappedType(EPropertyFlags.None, UhtPropertyExportFlags.TVal);
		}

		[UhtPropertyType(Keyword = "verse::TPtr", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseTPtrType(UhtPropertyResolveArgs args)
		{
			return args.ParseWrappedType(EPropertyFlags.None, UhtPropertyExportFlags.TPtr);
		}

		[UhtPropertyType(Keyword = "TVerseTask", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseTaskType(UhtPropertyResolveArgs args)
		{
			return args.ParseWrappedType(EPropertyFlags.None, UhtPropertyExportFlags.TVerseTask);
		}

		[UhtPropertyType(Keyword = "TVerseCall", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseTaskCall(UhtPropertyResolveArgs args)
		{
			return args.ParseWrappedType(EPropertyFlags.None, UhtPropertyExportFlags.TVerseCall);
		}
	}

	[UnrealHeaderTool]
	static class UhtDefaultPropertyParser
	{
		[UhtPropertyType(Options = UhtPropertyTypeOptions.Default)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? DefaultProperty(UhtPropertyResolveArgs args)
		{
			UhtProperty? property;
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;
			int typeStartPos = tokenReader.PeekToken().InputStartPos;

			bool gotConst = tokenReader.TryOptional("const");

			UhtFindOptions findOptions = UhtFindOptions.DelegateFunction | UhtFindOptions.Enum | UhtFindOptions.Class | UhtFindOptions.ScriptStruct;
			if (tokenReader.TryOptional("enum"))
			{
				if (propertySettings.PropertyCategory == UhtPropertyCategory.Member)
				{
					tokenReader.LogError($"Cannot declare enum at variable declaration");
				}
				findOptions = UhtFindOptions.Enum;
			}
			else if (tokenReader.TryOptional("class"))
			{
				findOptions = UhtFindOptions.Class;
			}
			else if (tokenReader.TryOptional("struct"))
			{
				findOptions = UhtFindOptions.ScriptStruct;
			}

			UhtTokenList identifiers = tokenReader.GetCppIdentifier();
			UhtType? type = propertySettings.Outer.FindType(UhtFindOptions.SourceName | findOptions, identifiers, tokenReader);
			if (type == null)
			{
				return null;
			}
			UhtTokenListCache.Return(identifiers);

			if (type is UhtEnum enumObj)
			{
				if (propertySettings.PropertyCategory == UhtPropertyCategory.Member)
				{
					if (enumObj.CppForm != UhtEnumCppForm.EnumClass)
					{
						tokenReader.LogError("You cannot use the raw enum name as a type for member variables, instead use TEnumAsByte or a C++11 enum class with an explicit underlying type.");
					}
				}

				property = new UhtEnumProperty(propertySettings, enumObj);
			}
			else if (type is UhtScriptStruct scriptStruct)
			{
				property = new UhtStructProperty(propertySettings, scriptStruct);
			}
			else if (type is UhtFunction function)
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.MulticastDelegate))
				{
					property = new UhtDelegateProperty(propertySettings, function);
				}
				else if (function.FunctionType == UhtFunctionType.SparseDelegate)
				{
					property = new UhtMulticastSparseDelegateProperty(propertySettings, function);
				}
				else
				{
					property = new UhtMulticastInlineDelegateProperty(propertySettings, function);
				}
			}
			else if (type is UhtClass classObj)
			{
				if (gotConst)
				{
					propertySettings.MetaData.Add(UhtNames.NativeConst, "");
					gotConst = false;
				}

				// Const after variable type but before pointer symbol
				if (tokenReader.TryOptional("const"))
				{
					propertySettings.MetaData.Add(UhtNames.NativeConst, "");
				}

				// If we didn't parse a TNonNullPtr, then we need to parse a pointer 
				if (!propertySettings.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TNonNullPtr))
				{
					tokenReader.Require('*');
				}

				// Optionally emit messages about native pointer members and swallow trailing 'const' after pointer properties
				// NOTE: At this time we aren't checking TNonNullPtr declarations since the engine support isn't that good.
				// First off, TOptional<TNonNullPtr<TObjectPtr<>>> is 16 bytes.  Second, the Init method when wrapped in a 
				// TVal seems to prevent the Init method from working.
				if (!propertySettings.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TNonNullPtr))
				{
					args.ConditionalLogPointerUsage(args.Config.NativePointerMemberBehavior, "Native pointer", typeStartPos, "TObjectPtr");
				}

				if (propertySettings.PropertyCategory == UhtPropertyCategory.Member)
				{
					tokenReader.TryOptional("const");
				}

				propertySettings.PointerType = UhtPointerType.Native;

				if (classObj.ClassFlags.HasAnyFlags(EClassFlags.Interface))
				{
					property = new UhtInterfaceProperty(propertySettings, classObj);
				}
				else if (classObj.IsChildOf(args.Session.UClass))
				{
					property = new UhtClassProperty(propertySettings, UhtObjectCppForm.NativeClass, classObj);
				}
				else
				{
					property = new UhtObjectProperty(propertySettings, UhtObjectCppForm.NativeObject, classObj);
				}
			}
			else
			{
				throw new UhtIceException("Unexpected type found");
			}

			if (gotConst)
			{
				if (propertySettings.PropertyCategory == UhtPropertyCategory.Member)
				{
					tokenReader.LogError("Const properties are not supported.");
				}
				else
				{
					tokenReader.LogError($"Inappropriate keyword 'const' on variable of type {type.SourceName}");
				}
			}

			return property;
		}
	}
}
