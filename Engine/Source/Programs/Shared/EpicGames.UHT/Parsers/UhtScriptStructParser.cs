// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// USTRUCT parser object
	/// </summary>
	[UnrealHeaderTool]
	public static class UhtScriptStructParser
	{

		#region Keywords
		[UhtKeyword(Extends = UhtTableNames.Global)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult USTRUCTKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			return ParseUScriptStruct(topScope, token);
		}

		[UhtKeyword(Extends = UhtTableNames.ScriptStruct)]
		[UhtKeyword(Extends = UhtTableNames.ScriptStruct, Keyword = "GENERATED_BODY")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult GENERATED_USTRUCT_BODYKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)topScope.ScopeType;

			if (topScope.AccessSpecifier != UhtAccessSpecifier.Public)
			{
				topScope.TokenReader.LogError($"{token.Value} must be in the public scope of '{scriptStruct.SourceName}', not private or protected.");
			}

			if (scriptStruct.MacroDeclaredLineNumber != -1)
			{
				topScope.TokenReader.LogError($"Multiple {token.Value} declarations found in '{scriptStruct.SourceName}'");
			}

			scriptStruct.MacroDeclaredLineNumber = topScope.TokenReader.InputLine;

			UhtParserHelpers.ParseCompileVersionDeclaration(topScope.TokenReader, topScope.Session.Config!, scriptStruct);

			topScope.TokenReader.Optional(';');
			return UhtParseResult.Handled;
		}

		[UhtKeyword(Extends = UhtTableNames.ScriptStruct)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtParseResult RIGVM_METHODKeyword(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token)
		{
			ParseRigVM(topScope);
			return UhtParseResult.Handled;
		}
		#endregion

		private static UhtParseResult ParseUScriptStruct(UhtParsingScope parentScope, UhtToken keywordToken)
		{
			UhtScriptStruct scriptStruct = new(parentScope.HeaderFile, parentScope.HeaderParser.GetNamespace(), parentScope.ScopeType, keywordToken.InputLine);
			{
				using UhtParsingScope topScope = new(parentScope, scriptStruct, parentScope.Session.GetKeywordTable(UhtTableNames.ScriptStruct), UhtAccessSpecifier.Public);

				{
					const string ScopeName = "struct";
					using UhtMessageContext tokenContext = new(ScopeName);

					// Parse the specifiers
					UhtSpecifierContext specifierContext = new(topScope, topScope.TokenReader, scriptStruct.MetaData);
					UhtSpecifierParser specifiers = UhtSpecifierParser.GetThreadInstance(specifierContext, ScopeName, parentScope.Session.GetSpecifierTable(UhtTableNames.ScriptStruct));
					specifiers.ParseSpecifiers();

					// Consume the struct specifier
					topScope.TokenReader.OptionalAttributes(false);
					topScope.TokenReader.Require("struct");
					topScope.TokenReader.OptionalAttributes(true);

					// Read the struct name and possible API macro name
					topScope.TokenReader.TryOptionalAPIMacro(out UhtToken apiMacroToken);
					scriptStruct.SourceName = topScope.TokenReader.GetIdentifier().Value.ToString();

					topScope.AddModuleRelativePathToMetaData();

					// Strip the name
					if (scriptStruct.SourceName[0] == 'T' || scriptStruct.SourceName[0] == 'F')
					{
						scriptStruct.EngineName = scriptStruct.SourceName[1..];
					}
					else
					{
						// This will be flagged later in the validation phase
						scriptStruct.EngineName = scriptStruct.SourceName;
					}

					// Check for an empty engine name
					if (scriptStruct.EngineName.Length == 0)
					{
						topScope.TokenReader.LogError($"When compiling struct definition for '{scriptStruct.SourceName}', attempting to strip prefix results in an empty name. Did you leave off a prefix?");
					}

					// Skip optional final keyword
					topScope.TokenReader.Optional("final");

					// Parse the inheritance
					UhtParserHelpers.ParseInheritance(parentScope.HeaderParser, topScope.Session.Config!, out UhtInheritance inheritance);
					scriptStruct.SuperIdentifier = inheritance.SuperIdentifier;
					scriptStruct.BaseIdentifiers = inheritance.BaseIdentifiers;
					if (inheritance.VerseInterfacesLine != 0)
					{
						topScope.TokenReader.LogError("VINTERFACES() isn't allowed on USTRUCT definitions");
					}

					// Add the comments here for compatibility with old UHT
					//COMPATIBILITY-TODO - Move this back to where the AddModuleRelativePathToMetaData is called.
					topScope.TokenReader.PeekToken();
					topScope.TokenReader.CommitPendingComments();
					topScope.AddFormattedCommentsAsTooltipMetaData();

					// Initialize the structure flags
					scriptStruct.ScriptStructFlags |= EStructFlags.Native;

					// Record that this struct is RequiredAPI if the CORE_API style macro was present
					if (apiMacroToken)
					{
						scriptStruct.ScriptStructFlags |= EStructFlags.RequiredAPI;
					}

					// Process the deferred specifiers
					specifiers.ParseDeferred();

					if (scriptStruct.IsVerseField)
					{
						scriptStruct.MetaData.Add(UhtNames.IsBlueprintBase, "false");
						scriptStruct.MetaData.Add(UhtNames.DisplayName, scriptStruct.VerseName!);
					}

					UhtCompilerDirective compilerDirective = topScope.HeaderParser.GetCurrentCompositeCompilerDirective();
					scriptStruct.DefineScope |= compilerDirective.GetDefaultDefineScopes();

					scriptStruct.Outer?.AddChild(scriptStruct);

					topScope.HeaderParser.ParseStatements('{', '}', true);

					topScope.TokenReader.Require(';');

					if (scriptStruct.MacroDeclaredLineNumber == -1 && scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.Native))
					{
						topScope.TokenReader.LogError("Expected a GENERATED_BODY() at the start of the struct");
					}
				}
				return UhtParseResult.Handled;
			}
		}

		private static void ParseRigVM(UhtParsingScope topScope)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)topScope.ScopeType;

			{
				using UhtMessageContext tokenContext = new("RIGVM_METHOD");

				// Create the RigVM information if it doesn't already exist
				UhtRigVMStructInfo? structInfo = scriptStruct.RigVMStructInfo;
				if (structInfo == null)
				{
					structInfo = new();
					structInfo.Name = scriptStruct.EngineName;
					structInfo.IsPure = scriptStruct.MetaData.ContainsKey("Pure");
					scriptStruct.RigVMStructInfo = structInfo;
				}

				// Create a new method information and add it
				UhtRigVMMethodInfo methodInfo = new();

				topScope.TokenReader
					.Require('(');
				if (topScope.TokenReader.PeekToken().IsValue("meta"))
				{
					topScope.TokenReader
						.RequireIdentifier("meta")
						.Require('=')
						.RequireList('(', ')', ',', false, tokens =>
						{
							foreach (UhtToken token in tokens)
							{
								if (token.IsIdentifier("Predicate"))
								{
									methodInfo.IsPredicate = true;
								}
							}
						})
						.Require(')');
				}
				else
				{
					topScope.TokenReader.Require(')');
				}

				if (methodInfo.IsPredicate)
				{
					topScope.TokenReader.RequireIdentifier("static");
				}

				// NOTE: The argument list reader doesn't handle templates with multiple arguments (i.e. the ',' issue)
				topScope.TokenReader
					.OptionalEndsWith("_API", (api) => { methodInfo.DllStorage = api + " "; })
					.Optional("virtual")
					.RequireIdentifier((ref UhtToken identifier) => methodInfo.ReturnType = identifier.Value.ToString())
					.RequireIdentifier((ref UhtToken identifier) => methodInfo.Name = identifier.Value.ToString());

				bool isGetUpgradeInfo = false;
				bool isGetNextAggregateName = false;
				if (methodInfo.ReturnType == "FRigVMStructUpgradeInfo" && methodInfo.Name == "GetUpgradeInfo")
				{
					structInfo.HasGetUpgradeInfoMethod = true;
					isGetUpgradeInfo = true;
				}
				if (methodInfo.Name == "GetNextAggregateName")
				{
					structInfo.HasGetNextAggregateNameMethod = true;
					isGetNextAggregateName = true;
				}

				topScope.TokenReader
					.RequireList('(', ')', ',', false, tokens =>
					{
						if (!isGetUpgradeInfo && !isGetNextAggregateName)
						{
							StringViewBuilder builder = new();
							UhtToken lastToken = new();
							foreach (UhtToken token in tokens)
							{
								if (token.IsSymbol('='))
								{
									break;
								}
								if (lastToken)
								{
									if (builder.Length != 0)
									{
										builder.Append(' ');
									}
									builder.Append(lastToken.Value);
								}
								lastToken = token;
							}
							methodInfo.Parameters.Add(new UhtRigVMParameter(lastToken.Value.ToString(), builder.ToString()));
						}
					})
					.ConsumeUntil(';');

				if (!isGetUpgradeInfo && !isGetNextAggregateName)
				{
					if (methodInfo.Parameters.Count > 0 && !methodInfo.IsPredicate)
					{
						topScope.TokenReader.LogError($"RIGVM_METHOD {scriptStruct.SourceName}::{methodInfo.Name} has {methodInfo.Parameters.Count} parameters. Since 5.2 parameters are no longer allowed for RIGVM_METHOD functions.");
						methodInfo.Parameters.Clear();
					}
					structInfo.Methods.Add(methodInfo);
				}
			}
		}
	}
}
