// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Class responsible for parsing specifiers and the field meta data.  To reduce allocations, one specifier parser is shared between all objects in
	/// a given header file.  This makes the Action pattern being used a bit more obtuse, but it does help performance by reducing the allocations fairly
	/// significantly.
	/// </summary>
	public class UhtSpecifierParser : IUhtMessageExtraContext
	{
		struct DeferredSpecifier
		{
			public UhtSpecifier _specifier;
			public object? _value;
		}

		/// <summary>
		/// For a given header file, we share a common specifier parser to reduce the number of allocations.
		/// Before the parser can be reused, the ParseDeferred method must be called to dispatch that list.
		/// </summary>
		private static readonly ThreadLocal<UhtSpecifierParser?> s_tls = new(() => null);

		private static readonly List<KeyValuePair<StringView, StringView>> s_emptyKVPValues = [];

		private UhtSpecifierContext _specifierContext;
		private IUhtTokenReader _tokenReader;
		private UhtSpecifierTable _table;
		private StringView _context;
		private StringView _currentSpecifier = new();
		private List<DeferredSpecifier> _deferredSpecifiers = [];
		private bool _isParsingFieldMetaData = false;
		private List<KeyValuePair<StringView, StringView>>? _currentKVPValues = null;
		private List<StringView>? _currentStringValues = null;
		private int _umetaElementsParsed = 0;

		private readonly Action _parseAction;
		private readonly Action _parseFieldMetaDataAction;
		private readonly Action _parseKVPValueAction;
		private readonly Action _parseStringViewListAction;

		/// <summary>
		/// Get the cached specifier parser
		/// </summary>
		/// <param name="specifierContext">Specifier context</param>
		/// <param name="context">User facing context</param>
		/// <param name="table">Specifier table</param>
		/// <returns>Specifier parser</returns>
		public static UhtSpecifierParser GetThreadInstance(UhtSpecifierContext specifierContext, StringView context, UhtSpecifierTable table)
		{
			if (s_tls.Value == null)
			{
				s_tls.Value = new UhtSpecifierParser(specifierContext, context, table);
			}
			else
			{
				s_tls.Value.Reset(specifierContext, context, table);
			}
			return s_tls.Value;
		}

		/// <summary>
		/// Construct a new specifier parser
		/// </summary>
		/// <param name="specifierContext">Specifier context</param>
		/// <param name="context">User facing context added to messages</param>
		/// <param name="table">Specifier table</param>
		public UhtSpecifierParser(UhtSpecifierContext specifierContext, StringView context, UhtSpecifierTable table)
		{
			_specifierContext = specifierContext;
			_tokenReader = specifierContext.TokenReader;
			_context = context;
			_table = table;

			_parseAction = ParseInternal;
			_parseFieldMetaDataAction = ParseFieldMetaDataInternal;
			_parseKVPValueAction = ParseKVPValueInternal;
			_parseStringViewListAction = ParseStringViewListInternal;
		}

		/// <summary>
		/// Reset an existing parser to parse a new specifier block
		/// </summary>
		/// <param name="specifierContext">Specifier context</param>
		/// <param name="context">User facing context added to messages</param>
		/// <param name="table">Specifier table</param>
		public void Reset(UhtSpecifierContext specifierContext, StringView context, UhtSpecifierTable table)
		{
			_specifierContext = specifierContext;
			_tokenReader = specifierContext.TokenReader;
			_context = context;
			_table = table;
			_deferredSpecifiers.Clear();
		}

		/// <summary>
		/// Perform the specifier parsing
		/// </summary>
		/// <returns>The parser</returns>
		public UhtSpecifierParser ParseSpecifiers()
		{
			_isParsingFieldMetaData = false;
			_specifierContext.MetaData.LineNumber = _tokenReader.InputLine;

			using UhtMessageContext tokenContext = new(this);
			_tokenReader.RequireList('(', ')', ',', false, _parseAction);
			return this;
		}

		/// <summary>
		/// Parse field meta data
		/// </summary>
		/// <returns>Specifier parser</returns>
		public UhtSpecifierParser ParseFieldMetaData()
		{
			_tokenReader = _specifierContext.TokenReader;
			_isParsingFieldMetaData = true;

			using UhtMessageContext tokenContext = new(this);
			if (_tokenReader.TryOptional("UMETA"))
			{
				_umetaElementsParsed = 0;
				_tokenReader.RequireList('(', ')', ',', false, _parseFieldMetaDataAction);
				if (_umetaElementsParsed == 0)
				{
					_tokenReader.LogError($"No metadata specified while parsing {UhtMessage.FormatContext(this)}");
				}
			}
			return this;
		}

		/// <summary>
		/// Parse any deferred specifiers
		/// </summary>
		public void ParseDeferred()
		{
			foreach (DeferredSpecifier deferred in _deferredSpecifiers)
			{
				Dispatch(deferred._specifier, deferred._value);
			}
			_deferredSpecifiers.Clear();
		}

		#region IMessageExtraContext implementation

		/// <inheritdoc/>
		public IEnumerable<object?>? MessageExtraContext
		{
			get
			{
				Stack<object?> extraContext = new(1);
				string what = _isParsingFieldMetaData ? "metadata" : "specifiers";
				if (_context.Span.Length > 0)
				{
					extraContext.Push($"{_context} {what}");
				}
				else
				{
					extraContext.Push(what);
				}
				return extraContext;
			}
		}
		#endregion

		private void ParseInternal()
		{
			UhtToken identifier = _tokenReader.GetIdentifier();

			_currentSpecifier = identifier.Value;
			if (_table.TryGetValue(_currentSpecifier, out UhtSpecifier? specifier))
			{
				if (TryParseValue(specifier.ValueType, out object? value))
				{
					if (specifier.When == UhtSpecifierWhen.Deferred)
					{
						_deferredSpecifiers ??= [];
						_deferredSpecifiers.Add(new DeferredSpecifier { _specifier = specifier, _value = value });
					}
					else
					{
						Dispatch(specifier, value);
					}
				}
			}
			else
			{
				_tokenReader.LogError($"Unknown specifier '{_currentSpecifier}' found while parsing {UhtMessage.FormatContext(this)}");
			}
		}

		private void ParseFieldMetaDataInternal()
		{
			if (!_tokenReader.TryOptionalIdentifier(out UhtToken key))
			{
				throw new UhtException(_tokenReader, $"UMETA expects a key and optional value", this);
			}

			StringViewBuilder builder = new();
			if (_tokenReader.TryOptional('='))
			{
				if (!ReadValue(_tokenReader, builder, true))
				{
					throw new UhtException(_tokenReader, $"UMETA key '{key.Value}' expects a value", this);
				}
			}

			++_umetaElementsParsed;
			_specifierContext.MetaData.CheckedAdd(key.Value.ToString(), _specifierContext.MetaNameIndex, builder.ToString());
		}

		private void ParseKVPValueInternal()
		{
			_currentKVPValues ??= [];
			_currentKVPValues.Add(ReadKVP());
		}

		private void ParseStringViewListInternal()
		{
			_currentStringValues ??= [];
			_currentStringValues.Add(ReadValue());
		}

		private void Dispatch(UhtSpecifier specifier, object? value)
		{
			UhtSpecifierDispatchResults results = specifier.Dispatch(_specifierContext, value);
			if (results == UhtSpecifierDispatchResults.Unknown)
			{
				_tokenReader.LogError($"Unknown specifier '{specifier.Name}' found while parsing {UhtMessage.FormatContext(this)}");
			}
		}

		private bool TryParseValue(UhtSpecifierValueType valueType, out object? value)
		{
			value = null;

#pragma warning disable CS0618 // Type or member is obsolete
			switch (valueType)
			{
				case UhtSpecifierValueType.NotSet:
					throw new UhtIceException("NotSet is an invalid value for value types");

				case UhtSpecifierValueType.None:
					if (_tokenReader.TryOptional('='))
					{
						ReadValue(); // consume the value;
						_tokenReader.LogError($"The specifier '{_currentSpecifier}' found a value when none was expected", this);
						return false;
					}
					return true;

				case UhtSpecifierValueType.String:
					if (!_tokenReader.TryOptional('='))
					{
						_tokenReader.LogError($"The specifier '{_currentSpecifier}' expects a value", this);
						return false;
					}
					value = ReadValue();
					return true;

				case UhtSpecifierValueType.OptionalString:
					{
						List<StringView>? stringList = ReadValueList();
						if (stringList != null && stringList.Count > 0)
						{
							value = stringList[0];
						}
						return true;
					}

				case UhtSpecifierValueType.SingleString:
					{
						List<StringView>? stringList = ReadValueList();
						if (stringList == null || stringList.Count != 1)
						{
							_tokenReader.LogError($"The specifier '{_currentSpecifier}' expects a single value", this);
							return false;
						}
						value = stringList[0];
						return true;
					}

				case UhtSpecifierValueType.StringList:
					value = ReadValueList();
					return true;

#pragma warning disable CS0618 // Type or member is obsolete
				case UhtSpecifierValueType.Legacy:
#pragma warning restore CS0618 // Type or member is obsolete
					value = ReadValueList();
					return true;

				case UhtSpecifierValueType.NonEmptyStringList:
					{
						List<StringView>? stringList = ReadValueList();
						if (stringList == null || stringList.Count == 0)
						{
							_tokenReader.LogError($"The specifier '{_currentSpecifier}' expects at list one value", this);
							return false;
						}
						value = stringList;
						return true;
					}

				case UhtSpecifierValueType.KeyValuePairList:
					{
						_currentKVPValues = null;
						_tokenReader
							.Require('=')
							.RequireList('(', ')', ',', false, _parseKVPValueAction);
						List<KeyValuePair<StringView, StringView>> kvps = _currentKVPValues ?? s_emptyKVPValues;
						_currentKVPValues = null;
						value = kvps;
						return true;
					}

				case UhtSpecifierValueType.OptionalEqualsKeyValuePairList:
					{
						_currentKVPValues = null;
						// This parser isn't as strict as the other parsers...
						if (_tokenReader.TryOptional('='))
						{
							if (!_tokenReader.TryOptionalList('(', ')', ',', false, _parseKVPValueAction))
							{
								_parseKVPValueAction();
							}
						}
						else
						{
							_tokenReader.TryOptionalList('(', ')', ',', false, _parseKVPValueAction);
						}
						List<KeyValuePair<StringView, StringView>> kvps = _currentKVPValues ?? s_emptyKVPValues;
						_currentKVPValues = null;
						value = kvps;
						return true;
					}

				default:
					throw new UhtIceException("Unknown value type");
			}
#pragma warning restore CS0618 // Type or member is obsolete
		}

		private KeyValuePair<StringView, StringView> ReadKVP()
		{
			if (!_tokenReader.TryOptionalIdentifier(out UhtToken key))
			{
				throw new UhtException(_tokenReader, $"The specifier '{_currentSpecifier}' expects a key and optional value", this);
			}

			StringView value = "";
			if (_tokenReader.TryOptional('='))
			{
				value = ReadValue();
			}
			return new KeyValuePair<StringView, StringView>(key.Value, value);
		}

		private List<StringView>? ReadValueList()
		{
			_currentStringValues = null;

			// This parser isn't as strict as the other parsers...
			if (_tokenReader.TryOptional('='))
			{
				if (!_tokenReader.TryOptionalList('(', ')', ',', false, _parseStringViewListAction))
				{
					_parseStringViewListAction();
				}
			}
			else
			{
				_tokenReader.TryOptionalList('(', ')', ',', false, _parseStringViewListAction);
			}
			List<StringView>? stringValues = _currentStringValues;
			_currentStringValues = null;
			return stringValues;
		}

		private StringView ReadValue()
		{
			StringViewBuilder builder = new();
			if (!ReadValue(_tokenReader, builder, false))
			{
				throw new UhtException(_tokenReader, $"The specifier '{_currentSpecifier}' expects a value", this);
			}
			return builder.ToStringView();
		}

		/// <summary>
		/// Parse the sequence of meta data
		/// </summary>
		/// <param name="tokenReader">Input token reader</param>
		/// <param name="builder">Output string builder</param>
		/// <param name="respectQuotes">If true, do not convert \" to " in string constants.  This is required for UMETA data</param>
		/// <returns>True if data was read</returns>
		private static bool ReadValue(IUhtTokenReader tokenReader, StringViewBuilder builder, bool respectQuotes)
		{
			UhtToken token = tokenReader.GetToken();
			switch (token.TokenType)
			{
				case UhtTokenType.EndOfFile:
				case UhtTokenType.EndOfDefault:
				case UhtTokenType.EndOfType:
				case UhtTokenType.EndOfDeclaration:
					return false;

				case UhtTokenType.Identifier:
					// We handle true/false differently for compatibility with old UHT
					if (token.IsValue("true", true))
					{
						builder.Append("TRUE");
					}
					else if (token.IsValue("false", true))
					{
						builder.Append("FALSE");
					}
					else
					{
						builder.Append(token.Value);
					}
					if (tokenReader.TryOptional('='))
					{
						builder.Append('=');
						if (!ReadValue(tokenReader, builder, respectQuotes))
						{
							return false;
						}
					}
					break;

				case UhtTokenType.Symbol:
					builder.Append(token.Value);
					if (tokenReader.TryOptional('='))
					{
						builder.Append('=');
						if (!ReadValue(tokenReader, builder, respectQuotes))
						{
							return false;
						}
					}
					break;

				case UhtTokenType.FloatConst:
					// TODO: C++ UHT compat 
					if (token.Value.Span.Contains('e') || token.Value.Span.Contains('E'))
					{
						tokenReader.LogError($"Metadata and specifier values can not contain exponential notion at this time");
					}
					builder.Append(token.GetConstantValue(respectQuotes));
					break;

				default:
					builder.Append(token.GetConstantValue(respectQuotes));
					break;
			}

			return true;
		}
	}
}
