// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// Collection of UCLASS specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	public static class UhtClassSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void NoExportSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ClassExportFlags |= UhtClassExportFlags.NoExport;
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void IntrinsicSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Intrinsic);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void ComponentWrapperClassSpecifier(UhtSpecifierContext specifierContext)
		{
			specifierContext.MetaData.Add(UhtNames.IgnoreCategoryKeywordsInSubclasses, true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.SingleString)]
		private static void WithinSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ClassWithinIdentifier = value.ToString();
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void EditInlineNewSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.EditInlineNew);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void NotEditInlineNewSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.RemoveClassFlags(EClassFlags.EditInlineNew);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void PlaceableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.RemoveClassFlags(EClassFlags.NotPlaceable);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void NotPlaceableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.NotPlaceable);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void DefaultToInstancedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.DefaultToInstanced);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void HideDropdownSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.HideDropDown);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void HiddenSpecifier(UhtSpecifierContext specifierContext)
		{
			// Prevents class from appearing in the editor class browser and edit inline menus.
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Hidden);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void DependsOnSpecifier(UhtSpecifierContext specifierContext)
		{
			specifierContext.MessageSite.LogError("The dependsOn specifier is deprecated. Please use #include \"ClassHeaderFilename.h\" instead.");
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void MinimalAPISpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.MinimalAPI);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void ConstSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Const);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void PartialSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Partial);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void PerObjectConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.PerObjectConfig);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void PerPlatformConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.PerPlatformConfig);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void ConfigDoNotCheckDefaultsSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.ConfigDoNotCheckDefaults);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void AbstractSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Abstract);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void DeprecatedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Deprecated | EClassFlags.NotPlaceable);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void TransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Transient);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void NonTransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.RemoveClassFlags(EClassFlags.Transient);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void OptionalSpecifier(UhtSpecifierContext specifierContext)
		{
			// Optional class
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Optional);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void CustomConstructorSpecifier(UhtSpecifierContext specifierContext)
		{
			// we will not export a constructor for this class, assuming it is in the CPP block
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ClassExportFlags |= UhtClassExportFlags.HasCustomConstructor;
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.SingleString)]
		private static void ConfigSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.Config = value.ToString();
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void DefaultConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			// Save object config only to Default INIs, never to local INIs.
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.DefaultConfig);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void GlobalUserConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			// Save object config only to global user overrides, never to local INIs
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.GlobalUserConfig);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void ProjectUserConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			// Save object config only to project user overrides, never to INIs that are checked in
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.ProjectUserConfig);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.SingleString)]
		private static void EditorConfigSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			specifierContext.MetaData.Add(UhtNames.EditorConfig, value.ToString());
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void ShowCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ShowCategories.AddUniqueRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void HideCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.HideCategories.AddUniqueRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void ShowFunctionsSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ShowFunctions.AddUniqueRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void HideFunctionsSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.HideFunctions.AddUniqueRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.SingleString)]
		private static void SparseClassDataTypesSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.SparseClassDataTypes.AddUnique(value.ToString());
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void ClassGroupSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			foreach (StringView element in value)
			{
				classObj.ClassGroupNames.Add(element.ToString());
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void AutoExpandCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AutoExpandCategories.AddUniqueRange(value);
			classObj.AutoCollapseCategories.RemoveSwapRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void AutoCollapseCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AutoCollapseCategories.AddUniqueRange(value);
			classObj.AutoExpandCategories.RemoveSwapRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void PrioritizeCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.PrioritizeCategories.AddUniqueRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.NonEmptyStringList)]
		private static void DontAutoCollapseCategoriesSpecifier(UhtSpecifierContext specifierContext, List<StringView> value)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AutoCollapseCategories.RemoveSwapRange(value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void CollapseCategoriesSpecifier(UhtSpecifierContext specifierContext)
		{
			// Class' properties should not be shown categorized in the editor.
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.CollapseCategories);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void DontCollapseCategoriesSpecifier(UhtSpecifierContext specifierContext)
		{
			// Class' properties should be shown categorized in the editor.
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.RemoveClassFlags(EClassFlags.CollapseCategories);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void AdvancedClassDisplaySpecifier(UhtSpecifierContext specifierContext)
		{
			// By default the class properties are shown in advanced sections in UI
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.MetaData.Add(UhtNames.AdvancedClassDisplay, "true");
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void ConversionRootSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.MetaData.Add(UhtNames.IsConversionRoot, "true");
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void NeedsDeferredDependencyLoadingSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.NeedsDeferredDependencyLoading);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void MatchedSerializersSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'MatchedSerializers' class specifier is only valid in the NoExportTypes.h file");
			}
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.MatchedSerializers);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void InterfaceSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'Interface' class specifier is only valid in the NoExportTypes.h file");
			}
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.AddClassFlags(EClassFlags.Interface);
		}

		[UhtSpecifier(Extends = UhtTableNames.Class, ValueType = UhtSpecifierValueType.None)]
		private static void CustomFieldNotifySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.ClassExportFlags |= UhtClassExportFlags.HasCustomFieldNotify;
		}
	}
}
