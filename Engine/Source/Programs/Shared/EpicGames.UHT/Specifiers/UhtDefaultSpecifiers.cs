// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Collection of default specifiers that apply to everything
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
	public static class UhtDefaultSpecifiers
	{
		private static void SetMetaData(UhtSpecifierContext specifierContext, StringView key, StringView value)
		{
			if (key.Length == 0)
			{
				specifierContext.MessageSite.LogError("Invalid metadata, name can not be blank");
				return;
			}

			// Trim the leading and ending whitespace
			ReadOnlySpan<char> span = value.Span;
			int start = 0;
			int end = value.Length;
			for (; start < end; ++start)
			{
				if (!UhtFCString.IsWhitespace(span[start]))
				{
					break;
				}
			}
			for (; start < end; --end)
			{
				if (!UhtFCString.IsWhitespace(span[end - 1]))
				{
					break;
				}
			}

			// Trim any quotes 
			//COMPATIBILITY-TODO - This doesn't handle strings that end in an escaped ".  This is a bug in old UHT.
			// Only remove quotes if we have quotes at both ends and the last isn't escaped.
			if (start < end && span[start] == '"')
			{
				++start;
			}
			if (start < end && span[end - 1] == '"')
			{
				--end;
			}

			// Get the trimmed string
			value = new StringView(value, start, end - start);

			specifierContext.MetaData.CheckedAdd(key.ToString(), specifierContext.MetaNameIndex, value.ToString());
		}

		private static void SetMetaData(UhtSpecifierContext specifierContext, StringView key, bool value)
		{
			SetMetaData(specifierContext, key, value ? "true" : "false");
		}

		#region Specifiers
		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.KeyValuePairList, When = UhtSpecifierWhen.Immediate)]
		private static void MetaSpecifier(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> value)
		{
			foreach (KeyValuePair<StringView, StringView> kvp in (List<KeyValuePair<StringView, StringView>>)value)
			{
				SetMetaData(specifierContext, kvp.Key, kvp.Value);
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.String, When = UhtSpecifierWhen.Immediate)]
		private static void DisplayNameSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			SetMetaData(specifierContext, "DisplayName", value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.String, When = UhtSpecifierWhen.Immediate)]
		private static void FriendlyNameSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			SetMetaData(specifierContext, "FriendlyName", value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void BlueprintInternalUseOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "BlueprintInternalUseOnly", true);
			SetMetaData(specifierContext, "BlueprintType", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void BlueprintInternalUseOnlyHierarchicalSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "BlueprintInternalUseOnlyHierarchical", true);
			SetMetaData(specifierContext, "BlueprintInternalUseOnly", true);
			SetMetaData(specifierContext, "BlueprintType", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void BlueprintTypeSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "BlueprintType", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void NotBlueprintTypeSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "NotBlueprintType", true);
			specifierContext.MetaData.Remove("BlueprintType", specifierContext.MetaNameIndex);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void BlueprintableSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, UhtNames.IsBlueprintBase, true);
			SetMetaData(specifierContext, "BlueprintType", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void CallInEditorSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "CallInEditor", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void NotBlueprintableSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, UhtNames.IsBlueprintBase, false);
			specifierContext.MetaData.Remove("BlueprintType", specifierContext.MetaNameIndex);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.String, When = UhtSpecifierWhen.Immediate)]
		private static void CategorySpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			SetMetaData(specifierContext, "Category", value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void ExperimentalSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "DevelopmentStatus", "Experimental");
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void EarlyAccessPreviewSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "DevelopmentStatus", "EarlyAccess");
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.String, When = UhtSpecifierWhen.Immediate)]
		private static void DocumentationPolicySpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			SetMetaData(specifierContext, UhtNames.DocumentationPolicy, value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.String, When = UhtSpecifierWhen.Immediate)]
		private static void SparseClassDataTypeSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			SetMetaData(specifierContext, "SparseClassDataType", value);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void StateStreamStaticStateSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "StateStreamStaticState", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void StateStreamDynamicStateSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "StateStreamDynamicState", true);
		}

		[UhtSpecifier(Extends = UhtTableNames.Default, ValueType = UhtSpecifierValueType.None, When = UhtSpecifierWhen.Immediate)]
		private static void StateStreamHandleSpecifier(UhtSpecifierContext specifierContext)
		{
			SetMetaData(specifierContext, "StateStreamHandle", true);
		}
		#endregion

		#region Validators
		[UhtSpecifierValidator(Name = "UIMin", Extends = UhtTableNames.Default)]
		[UhtSpecifierValidator(Name = "UIMax", Extends = UhtTableNames.Default)]
		[UhtSpecifierValidator(Name = "ClampMin", Extends = UhtTableNames.Default)]
		[UhtSpecifierValidator(Name = "ClampMax", Extends = UhtTableNames.Default)]
		private static void ValidateNumeric(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			if (!UhtFCString.IsNumeric(value.Span))
			{
				type.LogError($"Metadata value for '{key}' is non-numeric : '{value}'");
			}
		}

		[UhtSpecifierValidator(Extends = UhtTableNames.Default)]
		private static void DevelopmentStatusSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			string[] allowedValues = ["EarlyAccess", "Experimental"];
			foreach (string allowedValue in allowedValues)
			{
				if (value.Equals(allowedValue, StringComparison.OrdinalIgnoreCase))
				{
					return;
				}
			}
			type.LogError($"'{key.Name}' metadata was '{value}' but it must be {UhtUtilities.MergeTypeNames(allowedValues, "or", false)}");
		}

		[UhtSpecifierValidator(Extends = UhtTableNames.Default)]
		private static void DocumentationPolicySpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			const string StrictValue = "Strict";
			const string NoneValue = "None";
			if (!value.Span.Equals(StrictValue, StringComparison.OrdinalIgnoreCase) &&
				!value.Span.Equals(NoneValue, StringComparison.OrdinalIgnoreCase))
			{
				type.LogError(metaData.LineNumber, $"'{key}' metadata was '{value}' but it must be '{StrictValue}' or '{NoneValue}'");
			}
		}

		[UhtSpecifierValidator(Extends = UhtTableNames.Default)]
		private static void UnitsSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			// Check for numeric property
			if (type is UhtProperty)
			{
				if (type is not UhtNumericProperty && type is not UhtStructProperty)
				{
					type.LogError("'Units' meta data can only be applied to numeric and struct properties");
				}
			}

			if (!type.Session.Config!.IsValidUnits(value))
			{
				type.LogError($"Unrecognized units '{value}' specified for '{type.FullName}'");
			}
		}

		/// <summary>
		/// Validator for metadata on various types which must be a long path name to another type/function/etc.
		/// </summary>
		[UhtSpecifierValidator(Name = "MustImplement", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "ObjectMustImplement", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "AllowedClasses", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "DisallowedClasses", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "MetaClass", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "MetaStruct", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "BaseClass", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "BaseStruct", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "BitmaskEnum", Extends = UhtTableNames.PropertyMember)]
		[UhtSpecifierValidator(Name = "HasNativeBreak", Extends = UhtTableNames.ScriptStruct)]
		[UhtSpecifierValidator(Name = "HasNativeMake", Extends = UhtTableNames.ScriptStruct)]
		[UhtSpecifierValidator(Name = "ActorClassToPoll", Extends = UhtTableNames.Function)]
		[UhtSpecifierValidator(Name = "ComponentClass", Extends = UhtTableNames.Function)]
		private static void LongPathNameMetaDataSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			foreach (string element in value.ToString().Split(',', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries))
			{
				if (!UhtUtilities.IsValidLongPathName(element))
				{
					UhtType? found = type.FindType(UhtFindOptions.TypesMask | UhtFindOptions.EngineName | UhtFindOptions.NoSelf, element);
					string message = found is not null
						? $"MetaData {key} on {type.EngineName} has value '{element}' which is not a valid long path name. Did you mean '{found.PathName}' ?"
						: $"MetaData {key} on {type.EngineName} has value '{element}' which is not a valid long path name. Replace it with a full path name specifying the package containing the type.";
					type.LogError(message);
				}
			}
		}
		#endregion
	}
}
