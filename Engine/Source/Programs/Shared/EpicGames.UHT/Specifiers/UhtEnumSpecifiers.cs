// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Collection of UENUM specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	public static class UhtEnumSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.Enum, ValueType = UhtSpecifierValueType.None)]
		private static void FlagsSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtEnum enumObj = (UhtEnum)specifierContext.Type;
			enumObj.EnumFlags |= EEnumFlags.Flags;
		}

		[UhtSpecifier(Extends = UhtTableNames.Enum, ValueType = UhtSpecifierValueType.None)]
		private static void MinimalAPISpecifier(UhtSpecifierContext specifierContext)
		{
			UhtEnum enumObj = (UhtEnum)specifierContext.Type;
			enumObj.EnumExportFlags |= UhtEnumExportFlags.MinimalAPI;
		}
	}
}
