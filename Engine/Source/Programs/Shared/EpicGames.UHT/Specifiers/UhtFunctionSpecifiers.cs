// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// Collection of UFUNCTION/UDELEGATE specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
	public static class UhtFunctionSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintAuthorityOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.BlueprintAuthorityOnly;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintCallableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.BlueprintCallable;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintCosmeticSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.BlueprintCosmetic;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintGetterSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
			{
				specifierContext.MessageSite.LogError("Function cannot be a blueprint event and a blueprint getter.");
			}

			function.FunctionExportFlags |= UhtFunctionExportFlags.SawPropertyAccessor;
			function.FunctionFlags |= EFunctionFlags.BlueprintCallable;
			function.FunctionFlags |= EFunctionFlags.BlueprintPure;
			function.MetaData.Add(UhtNames.BlueprintGetter, "");
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintImplementableEventSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent functions cannot be replicated!");
			}
			else if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent) && function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
			{
				// already a BlueprintNativeEvent
				specifierContext.MessageSite.LogError("A function cannot be both BlueprintNativeEvent and BlueprintImplementableEvent!");
			}
			else if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.SawPropertyAccessor))
			{
				specifierContext.MessageSite.LogError("A function cannot be both BlueprintImplementableEvent and a Blueprint Property accessor!");
			}
			else if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Private))
			{
				specifierContext.MessageSite.LogError("A Private function cannot be a BlueprintImplementableEvent!");
			}

			function.FunctionFlags |= EFunctionFlags.Event;
			function.FunctionFlags |= EFunctionFlags.BlueprintEvent;
			function.FunctionFlags &= ~EFunctionFlags.Native;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintNativeEventSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				specifierContext.MessageSite.LogError("BlueprintNativeEvent functions cannot be replicated!");
			}
			else if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent) && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
			{
				// already a BlueprintImplementableEvent
				specifierContext.MessageSite.LogError("A function cannot be both BlueprintNativeEvent and BlueprintImplementableEvent!");
			}
			else if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.SawPropertyAccessor))
			{
				specifierContext.MessageSite.LogError("A function cannot be both BlueprintNativeEvent and a Blueprint Property accessor!");
			}
			else if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Private))
			{
				specifierContext.MessageSite.LogError("A Private function cannot be a BlueprintNativeEvent!");
			}

			function.FunctionFlags |= EFunctionFlags.Event;
			function.FunctionFlags |= EFunctionFlags.BlueprintEvent;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void BlueprintPureSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			// This function can be called, and is also pure.
			function.FunctionFlags |= EFunctionFlags.BlueprintCallable;

			if (value == null || UhtFCString.ToBool((StringView)value))
			{
				function.FunctionFlags |= EFunctionFlags.BlueprintPure;
			}
			else
			{
				function.FunctionExportFlags |= UhtFunctionExportFlags.ForceBlueprintImpure;
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintSetterSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
			{
				specifierContext.MessageSite.LogError("Function cannot be a blueprint event and a blueprint setter.");
			}

			function.FunctionExportFlags |= UhtFunctionExportFlags.SawPropertyAccessor;
			function.FunctionFlags |= EFunctionFlags.BlueprintCallable;
			function.MetaData.Add(UhtNames.BlueprintSetter, "");
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void ClientSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as Client or Server");
			}

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Exec))
			{
				specifierContext.MessageSite.LogError("Exec functions cannot be replicated!");
			}

			function.FunctionFlags |= EFunctionFlags.Net;
			function.FunctionFlags |= EFunctionFlags.NetClient;

			if (value != null)
			{
				function.CppImplName = ((StringView)value).ToString();
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void CustomThunkSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionExportFlags |= UhtFunctionExportFlags.CustomThunk;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void ExecSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.Exec;
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				specifierContext.MessageSite.LogError("Exec functions cannot be replicated!");
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void NetMulticastSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as Multicast");
			}

			function.FunctionFlags |= EFunctionFlags.Net;
			function.FunctionFlags |= EFunctionFlags.NetMulticast;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void ReliableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.NetReliable;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void SealedEventSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionExportFlags |= UhtFunctionExportFlags.SealedEvent;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void ServerSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as Client or Server");
			}

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Exec))
			{
				specifierContext.MessageSite.LogError("Exec functions cannot be replicated!");
			}

			function.FunctionFlags |= EFunctionFlags.Net;
			function.FunctionFlags |= EFunctionFlags.NetServer;

			if (value != null)
			{
				function.CppImplName = ((StringView)value).ToString();
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void RemoteSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as Client, Server, or Remote");
			}

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Exec))
			{
				specifierContext.MessageSite.LogError("Exec functions cannot be replicated!");
			}

			function.FunctionFlags |= EFunctionFlags.Net;

			if (value != null)
			{
				function.CppImplName = ((StringView)value).ToString();
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalEqualsKeyValuePairList)]
		private static void ServiceRequestSpecifier(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as a ServiceRequest");
			}

			function.FunctionFlags |= EFunctionFlags.Net;
			function.FunctionFlags |= EFunctionFlags.NetReliable;
			function.FunctionFlags |= EFunctionFlags.NetRequest;
			function.FunctionExportFlags |= UhtFunctionExportFlags.CustomThunk;

			ParseNetServiceIdentifiers(specifierContext, value);

			if (String.IsNullOrEmpty(function.EndpointName))
			{
				specifierContext.MessageSite.LogError("ServiceRequest needs to specify an endpoint name");
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalEqualsKeyValuePairList)]
		private static void ServiceResponseSpecifier(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				specifierContext.MessageSite.LogError("BlueprintImplementableEvent or BlueprintNativeEvent functions cannot be declared as a ServiceResponse");
			}

			function.FunctionFlags |= EFunctionFlags.Net;
			function.FunctionFlags |= EFunctionFlags.NetReliable;
			function.FunctionFlags |= EFunctionFlags.NetResponse;
			//Function.FunctionExportFlags |= EFunctionExportFlags.CustomThunk;

			ParseNetServiceIdentifiers(specifierContext, value);

			if (String.IsNullOrEmpty(function.EndpointName))
			{
				specifierContext.MessageSite.LogError("ServiceResponse needs to specify an endpoint name");
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void UnreliableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionExportFlags |= UhtFunctionExportFlags.Unreliable;
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.StringList)]
		private static void WithValidationSpecifier(UhtSpecifierContext specifierContext, List<StringView>? value)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionFlags |= EFunctionFlags.NetValidate;

			if (value != null && value.Count > 0)
			{
				function.CppValidationImplName = value[0].ToString();
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.None)]
		private static void FieldNotifySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtFunction function = (UhtFunction)specifierContext.Type;
			function.FunctionExportFlags |= UhtFunctionExportFlags.FieldNotify;
		}

		[UhtSpecifierValidator(Extends = UhtTableNames.Function)]
		private static void BlueprintProtectedSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			UhtFunction function = (UhtFunction)type;
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Static))
			{
				// Given the owning class, locate the class that the owning derives from up to the point of UObject
				if (function.Outer is UhtClass outerClass)
				{
					UhtClass? classPriorToUObject = outerClass;
					for (; classPriorToUObject != null; classPriorToUObject = classPriorToUObject.SuperClass)
					{
						// If our super is UObject, then stop
						if (classPriorToUObject.Super == type.Session.UObject)
						{
							break;
						}
					}

					if (classPriorToUObject != null && classPriorToUObject.SourceName == "UBlueprintFunctionLibrary")
					{
						type.LogError(metaData.LineNumber, $"'{key.Name}' doesn't make sense on static method '{type.SourceName}' in a blueprint function library");
					}
				}
			}
		}

		[UhtSpecifierValidator(Extends = UhtTableNames.Function)]
		private static void CommutativeAssociativeBinaryOperatorSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			UhtFunction function = (UhtFunction)type;
			UhtProperty? returnProperty = function.ReturnProperty;
			ReadOnlyMemory<UhtType> parameterProperties = function.ParameterProperties;

			if (returnProperty == null || parameterProperties.Length != 2 || !((UhtProperty)parameterProperties.Span[0]).IsSameType((UhtProperty)parameterProperties.Span[1]))
			{
				function.LogError("Commutative associative binary operators must have exactly 2 parameters of the same type and a return value.");
			}
		}

		[UhtSpecifierValidator(Name = "ExpandBoolAsExecs", Extends = UhtTableNames.Function)]
		[UhtSpecifierValidator(Name = "ExpandEnumAsExecs", Extends = UhtTableNames.Function)]
		private static void ExpandsSpecifierValidator(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value)
		{
			// multiple entry parsing in the same format as eg SetParam.
			UhtType? firstInput = null;
			foreach (string rawGroup in value.ToString().Split(','))
			{
				foreach (string entry in rawGroup.Split('|'))
				{
					string trimmed = entry.Trim();
					if (String.IsNullOrEmpty(trimmed))
					{
						continue;
					}

					UhtType? foundField = type.FindType(UhtFindOptions.SourceName | UhtFindOptions.SelfOnly | UhtFindOptions.Property, trimmed, type);
					if (foundField != null)
					{
						UhtProperty property = (UhtProperty)foundField;
						if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm) &&
							(!property.PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm) ||
							 property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReferenceParm)))
						{
							if (firstInput == null)
							{
								firstInput = foundField;
							}
							else
							{
								type.LogError($"Function already specified an ExpandEnumAsExec input '{firstInput.SourceName}', but '{trimmed}' is also an input parameter. Only one is permitted.");
							}
						}
					}
				}
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.Function, ValueType = UhtSpecifierValueType.OptionalEqualsKeyValuePairList)]
		private static void VerseSpecifier(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> values)
		{
			UhtFunction functionObj = (UhtFunction)specifierContext.Type;

			// Extract all the elements of the meta data
			foreach (KeyValuePair<StringView, StringView> kvp in values)
			{
				ReadOnlySpan<char> key = kvp.Key.Span;
				if (key.Equals("native_callable", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseNativeCallable;
				}
				else if (key.Equals("verse_callable", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseCallable;
				}
				else if (key.Equals("decides", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseDecides;
				}
				else if (key.Equals("no_rollback", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseNoRollback;
				}
				else if (key.Equals("rtfm_always_open", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseRTFMAlwaysOpen;
				}
				else if (key.Equals("extension_method", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.FunctionExportFlags |= UhtFunctionExportFlags.VerseExtensionMethod;
				}
				else if (key.Equals("name", StringComparison.OrdinalIgnoreCase))
				{
					functionObj.VerseName = kvp.Value.ToString();
				}
				else
				{
					functionObj.LogError($"Verse specifier option '{key}' is unknown");
				}
			}

			if (String.IsNullOrEmpty(functionObj.VerseName))
			{
				functionObj.VerseName = functionObj.SourceName;
			}
		}

		private static void ParseNetServiceIdentifiers(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> identifiers)
		{
			IUhtTokenReader tokenReader = specifierContext.TokenReader;
			UhtFunction function = (UhtFunction)specifierContext.Type;

			foreach (KeyValuePair<StringView, StringView> kvp in identifiers)
			{
				if (kvp.Value.Span.Length > 0)
				{
					if (kvp.Key.Span.StartsWith("Id", StringComparison.OrdinalIgnoreCase))
					{
						if (!Int32.TryParse(kvp.Value.Span, out int id) || id <= 0 || id > UInt16.MaxValue)
						{
							tokenReader.LogError($"Invalid network identifier {kvp.Key} for function");
						}
						else
						{
							function.RPCId = (ushort)id;
						}
					}
					else if (kvp.Key.Span.StartsWith("ResponseId", StringComparison.OrdinalIgnoreCase) || kvp.Key.Span.StartsWith("Priority", StringComparison.OrdinalIgnoreCase))
					{
						if (!Int32.TryParse(kvp.Value.Span, out int id) || id <= 0 || id > UInt16.MaxValue)
						{
							tokenReader.LogError($"Invalid network identifier {kvp.Key} for function");
						}
						else
						{
							function.RPCResponseId = (ushort)id;
						}
					}
					else
					{
						// No error message???
					}
				}

				// Assume it's an endpoint name
				else
				{
					if (!String.IsNullOrEmpty(function.EndpointName))
					{
						tokenReader.LogError($"Function should not specify multiple endpoints - '{kvp.Key}' found but already using '{function.EndpointName}'");
					}
					else
					{
						function.EndpointName = kvp.Key.ToString();
					}
				}
			}
		}
	}
}
