// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	sealed class UhtInterfaceClassSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.Interface, ValueType = UhtSpecifierValueType.None)]
		private static void DependsOnSpecifier(UhtSpecifierContext specifierContext)
		{
			throw new UhtException(specifierContext.MessageSite, $"The dependsOn specifier is deprecated. Please use #include \"ClassHeaderFilename.h\" instead.");
		}

		[UhtSpecifier(Extends = UhtTableNames.Interface, ValueType = UhtSpecifierValueType.None)]
		private static void MinimalAPISpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass clasObj = (UhtClass)specifierContext.Type;
			clasObj.ClassFlags |= EClassFlags.MinimalAPI;
		}

		[UhtSpecifier(Extends = UhtTableNames.Interface, ValueType = UhtSpecifierValueType.None)]
		private static void ConversionRootSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtClass classObj = (UhtClass)specifierContext.Type;
			classObj.MetaData.Add(UhtNames.IsConversionRoot, "true");
		}
	}
}
