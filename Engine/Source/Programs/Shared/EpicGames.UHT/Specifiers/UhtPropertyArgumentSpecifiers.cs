// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Collection of property argument specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	public static class UhtPropertyArgumentSpecifiers
	{
		#region Argument Property Specifiers
		[UhtSpecifier(Extends = UhtTableNames.PropertyArgument, ValueType = UhtSpecifierValueType.None)]
		private static void ConstSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ConstParm;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyArgument, ValueType = UhtSpecifierValueType.None)]
		private static void RefSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.OutParm | EPropertyFlags.ReferenceParm;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyArgument, ValueType = UhtSpecifierValueType.None)]
		private static void NotReplicatedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.PropertyCategory == UhtPropertyCategory.ReplicatedParameter)
			{
				context.PropertySettings.PropertyCategory = UhtPropertyCategory.RegularParameter;
				context.PropertySettings.PropertyFlags |= EPropertyFlags.RepSkip;
			}
			else
			{
				context.MessageSite.LogError("Only parameters in service request functions can be marked NotReplicated");
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyArgument, ValueType = UhtSpecifierValueType.None)]
		private static void RequiredSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.RequiredParm;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyArgument, ValueType = UhtSpecifierValueType.OptionalEqualsKeyValuePairList)]
		private static void VerseSpecifier(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> values)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;

			// Extract all the elements of the meta data
			foreach (KeyValuePair<StringView, StringView> kvp in values)
			{
				ReadOnlySpan<char> key = kvp.Key.Span;
				if (key.Equals("name", StringComparison.OrdinalIgnoreCase))
				{
					context.PropertySettings.VerseName = kvp.Value.ToString();
				}
				else if (key.Equals("named", StringComparison.OrdinalIgnoreCase))
				{
					context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.VerseNamed;
				}
				else if (key.Equals("hasdefault", StringComparison.OrdinalIgnoreCase))
				{
					context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.VerseDefaultValue;
				}
			}
		}

		#endregion
	}
}
