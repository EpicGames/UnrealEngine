// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{

	/// <summary>
	/// Collection of property member specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	public static class UhtPropertyMemberSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void EditAnywhereSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (EditAnywhere), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void EditInstanceOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (EditInstanceOnly), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.DisableEditOnTemplate;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void EditDefaultsOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (EditDefaultsOnly), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.DisableEditOnInstance;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void VisibleAnywhereSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (VisibleAnywhere), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.EditConst;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void VisibleInstanceOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (VisibleInstanceOnly), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.EditConst | EPropertyFlags.DisableEditOnTemplate;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void VisibleDefaultsOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenEditSpecifier)
			{
				context.MessageSite.LogError("Found more than one edit/visibility specifier (VisibleDefaultsOnly), only one is allowed");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.EditConst | EPropertyFlags.DisableEditOnInstance;
			context.SeenEditSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintReadWriteSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenBlueprintReadOnlySpecifier)
			{
				context.MessageSite.LogError("Cannot specify a property as being both BlueprintReadOnly and BlueprintReadWrite.");
			}

			bool allowPrivateAccess = context.MetaData.TryGetValue(UhtNames.AllowPrivateAccess, out string? privateAccessMD) && !privateAccessMD.Equals("false", StringComparison.OrdinalIgnoreCase);
			if (specifierContext.AccessSpecifier == UhtAccessSpecifier.Private && !allowPrivateAccess)
			{
				context.MessageSite.LogError("BlueprintReadWrite should not be used on private members");
			}

			if (context.PropertySettings.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly) && context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Blueprint exposed struct members cannot be editor only");
			}

			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintVisible;
			context.SeenBlueprintWriteSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintReadOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenBlueprintWriteSpecifier)
			{
				context.MessageSite.LogError("Cannot specify both BlueprintReadOnly and BlueprintReadWrite or BlueprintSetter.");
			}

			bool allowPrivateAccess = context.MetaData.TryGetValue(UhtNames.AllowPrivateAccess, out string? privateAccessMD) && !privateAccessMD.Equals("false", StringComparison.OrdinalIgnoreCase);
			if (specifierContext.AccessSpecifier == UhtAccessSpecifier.Private && !allowPrivateAccess)
			{
				context.MessageSite.LogError("BlueprintReadOnly should not be used on private members");
			}

			if (context.PropertySettings.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly) && context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Blueprint exposed struct members cannot be editor only");
			}

			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintVisible | EPropertyFlags.BlueprintReadOnly;
			context.SeenBlueprintReadOnlySpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.SingleString)]
		private static void BlueprintSetterSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.SeenBlueprintReadOnlySpecifier)
			{
				context.MessageSite.LogError("Cannot specify a property as being both BlueprintReadOnly and having a BlueprintSetter.");
			}

			if (context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Cannot specify BlueprintSetter for a struct member.");
			}

			context.MetaData.Add(UhtNames.BlueprintSetter, value.ToString());

			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintVisible;
			context.SeenBlueprintWriteSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.SingleString)]
		private static void BlueprintGetterSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Cannot specify BlueprintGetter for a struct member.");
			}

			context.MetaData.Add(UhtNames.BlueprintGetter, value.ToString());

			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintVisible;
			context.SeenBlueprintGetterSpecifier = true;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Config;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void GlobalConfigSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.GlobalConfig | EPropertyFlags.Config;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void LocalizedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.MessageSite.LogError("The Localized specifier is deprecated");
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void TransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Transient;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void DuplicateTransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.DuplicateTransient;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void TextExportTransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.TextExportTransient;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void NonPIETransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.MessageSite.LogWarning("NonPIETransient is deprecated - NonPIEDuplicateTransient should be used instead");
			context.PropertySettings.PropertyFlags |= EPropertyFlags.NonPIEDuplicateTransient;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void NonPIEDuplicateTransientSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.NonPIEDuplicateTransient;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ExportSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ExportObject;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void EditInlineSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.MessageSite.LogError("EditInline is deprecated. Remove it, or use Instanced instead.");
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void NoClearSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.NoClear;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void EditFixedSizeSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.EditFixedSize;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ReplicatedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Struct members cannot be replicated");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Net;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.SingleString)]
		private static void ReplicatedUsingSpecifier(UhtSpecifierContext specifierContext, StringView value)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is UhtScriptStruct)
			{
				context.MessageSite.LogError("Struct members cannot be replicated");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Net;

			if (value.Span.Length == 0)
			{
				context.MessageSite.LogError("Must specify a valid function name for replication notifications");
			}
			context.PropertySettings.RepNotifyName = value.ToString();
			context.PropertySettings.PropertyFlags |= EPropertyFlags.RepNotify;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void NotReplicatedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is not UhtScriptStruct)
			{
				context.MessageSite.LogError("Only Struct members can be marked NotReplicated");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.RepSkip;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void RepRetrySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.MessageSite.LogError("'RepRetry' is deprecated.");
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void InterpSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.BlueprintVisible | EPropertyFlags.Interp;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void NonTransactionalSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.NonTransactional;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void InstancedSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.PersistentInstance | EPropertyFlags.ExportObject | EPropertyFlags.InstancedReference;
			context.MetaData.Add(UhtNames.EditInline, true);
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintAssignableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintAssignable;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintCallableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintCallable;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void BlueprintAuthorityOnlySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.BlueprintAuthorityOnly;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void AssetRegistrySearchableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.AssetRegistrySearchable;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void SimpleDisplaySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.SimpleDisplay;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void AdvancedDisplaySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.AdvancedDisplay;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void SaveGameSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.SaveGame;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void SkipSerializationSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.SkipSerialization;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ExperimentalOverridableLogicSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ExperimentalOverridableLogic;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ExperimentalAlwaysOverridenSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ExperimentalAlwaysOverriden;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ExperimentalNeverOverridenSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ExperimentalNeverOverriden;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ExperimentalExternalObjectsSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if ((context.PropertySettings.PropertyFlags & EPropertyFlags.ExperimentalOverridableLogic) == 0)
			{
				context.MessageSite.LogError("ExperimentalExternalObjects can only be used with ExperimentalOverridableLogic");
			}
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ExperimentalExternalObjects;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void ForcePostConstructLinkSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyFlags |= EPropertyFlags.ForcePostConstructLink;
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void GetterSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is not UhtClass)
			{
				context.MessageSite.LogError("Only class members can have Setters");
			}
			context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.GetterSpecified;
			if (value != null)
			{
				StringView temp = (StringView)value;
				if (temp.Length > 0)
				{
					if (temp.Equals("None", StringComparison.OrdinalIgnoreCase))
					{
						context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.GetterSpecifiedNone;
					}
					else if (temp.Equals("Auto", StringComparison.OrdinalIgnoreCase))
					{
						context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.GetterSpecifiedAuto;
					}
					else
					{
						context.PropertySettings.Getter = value.ToString();
					}
				}
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.OptionalString)]
		private static void SetterSpecifier(UhtSpecifierContext specifierContext, StringView? value)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			if (context.PropertySettings.Outer is not UhtClass)
			{
				context.MessageSite.LogError("Only class members can have Getters");
			}
			context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.SetterSpecified;
			if (value != null)
			{
				StringView temp = (StringView)value;
				if (temp.Length > 0)
				{
					if (temp.Equals("None", StringComparison.OrdinalIgnoreCase))
					{
						context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.SetterSpecifiedNone;
					}
					else if (temp.Equals("Auto", StringComparison.OrdinalIgnoreCase))
					{
						context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.SetterSpecifiedAuto;
					}
					else
					{
						context.PropertySettings.Setter = value.ToString();
					}
				}
			}
		}

		[UhtSpecifier(Extends = UhtTableNames.PropertyMember, ValueType = UhtSpecifierValueType.None)]
		private static void FieldNotifySpecifier(UhtSpecifierContext specifierContext)
		{
			UhtPropertySpecifierContext context = (UhtPropertySpecifierContext)specifierContext;
			context.PropertySettings.PropertyExportFlags |= UhtPropertyExportFlags.FieldNotify;
		}
	}
}
