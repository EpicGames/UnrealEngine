// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Parsers
{
	/// <summary>
	/// Collection of USCRIPT specifiers
	/// </summary>
	[UnrealHeaderTool]
	[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
	public static class UhtScriptStructSpecifiers
	{
		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void NoExportSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructFlags |= EStructFlags.NoExport;
			scriptStruct.ScriptStructFlags &= ~EStructFlags.Native;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void AtomicSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructFlags |= EStructFlags.Atomic;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static void ImmutableSpecifier(UhtSpecifierContext specifierContext)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructFlags |= EStructFlags.Atomic | EStructFlags.Immutable;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void HasDefaultsSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'HasDefaults' struct specifier is only valid in the NoExportTypes.h file");
			}
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructExportFlags |= UhtScriptStructExportFlags.HasDefaults;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void HasNoOpConstructorSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'HasNoOpConstructor' struct specifier is only valid in the NoExportTypes.h file");
			}
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructExportFlags |= UhtScriptStructExportFlags.HasNoOpConstructor;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void IsAlwaysAccessibleSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'IsAlwaysAccessible' struct specifier is only valid in the NoExportTypes.h file");
			}
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructExportFlags |= UhtScriptStructExportFlags.IsAlwaysAccessible;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void IsCoreTypeSpecifier(UhtSpecifierContext specifierContext)
		{
			if (!specifierContext.Type.HeaderFile.IsNoExportTypes)
			{
				specifierContext.MessageSite.LogError("The 'IsCoreType' struct specifier is only valid in the NoExportTypes.h file");
			}
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructExportFlags |= UhtScriptStructExportFlags.IsCoreType;
		}

		[UhtSpecifier(Extends = UhtTableNames.ScriptStruct, ValueType = UhtSpecifierValueType.None)]
		private static void MinimalAPISpecifier(UhtSpecifierContext specifierContext)
		{
			UhtScriptStruct scriptStruct = (UhtScriptStruct)specifierContext.Type;
			scriptStruct.ScriptStructExportFlags |= UhtScriptStructExportFlags.MinimalAPI;
		}
	}
}
