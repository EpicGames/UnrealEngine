// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Text;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Tables
{
	/// <summary>
	/// Location where code will be injected
	/// </summary>
	public enum UhtCodeGeneratorInjectionLocation
	{
		/// <summary>
		/// Injection location will be in the generated.h after built-in code generation for the respective type
		/// </summary>
		Header,
		/// <summary>
		/// Injection location within the macros for the type.
		/// </summary>
		GeneratedMacro
	}

	/// <summary>
	/// Method attribute which defines a code generation injection behaviour.
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = false)]
	public sealed class UhtCodeGeneratorInjectorAttribute : Attribute
	{
		/// <summary>
		/// The UhtType that the injector will be applied to.  
		/// </summary>
		public required Type UhtType { get; set; }
		/// <summary>
		/// Specifies the injection location for the injector to run in.
		/// </summary>
		public required UhtCodeGeneratorInjectionLocation Location { get; set; }
	}

	internal record struct UhtCodeGeneratorInjectorDesire(Type UhtType, UhtCodeGeneratorInjectionLocation Location);

	/// <summary>
	/// Entrypoint for code generation injector
	/// </summary>
	internal delegate void UhtCodeGeneratorInjectorDelegate(StringBuilder builder, UhtType type, int leadingTabs, string eolSequence);

	/// <summary>
	/// Holds a set of injectors discovered via reflection
	/// </summary>
	public sealed class UhtCodeGeneratorInjectorTable
	{
		private readonly Dictionary<UhtCodeGeneratorInjectorDesire, List<UhtCodeGeneratorInjectorDelegate>?> _injectorsTable = [];
		private readonly List<Type> _supportedLeafTypes;

		/// <summary>
		/// Constructs UhtCodeGeneratorInjectorTable
		/// </summary>
		public UhtCodeGeneratorInjectorTable()
		{
			_supportedLeafTypes =
			[
				typeof(UhtClass),
				typeof(UhtScriptStruct),
			];
		}

		internal void Inject(StringBuilder builder, UhtType type, UhtCodeGeneratorInjectionLocation location)
		{
			string eolSequence;
			int leadingTabs;
			switch (location)
			{
				case UhtCodeGeneratorInjectionLocation.Header:
					eolSequence = "\r\n";
					leadingTabs = 0;
					break;
				case UhtCodeGeneratorInjectionLocation.GeneratedMacro:
					eolSequence = " \\\r\n";
					leadingTabs = 1;
					break;
				default:
					throw new ArgumentOutOfRangeException(nameof(location), location, "Unhandled injection location");
			}

			UhtCodeGeneratorInjectorDesire desire = new(type.GetType(), location);
			if (_injectorsTable.TryGetValue(desire, out List<UhtCodeGeneratorInjectorDelegate>? injectors))
			{
				foreach (UhtCodeGeneratorInjectorDelegate injector in injectors!)
				{
					injector.Invoke(builder, type, leadingTabs, eolSequence);
				}
			}
		}

		internal void OnUhtCodeGeneratorInjectorAttribute(MethodInfo method, UhtCodeGeneratorInjectorAttribute attribute)
		{
			// Walk back from each support leaf types up to the declared type
			// If our leaf type IS of the declared type, then add a delegate for the handler of the leaf type
			// This allows for users to write injectors that pertain to multiple leaf types if required (ie. all UhtNumericProperty)
			foreach (Type supportedLeafType in _supportedLeafTypes)
			{
				if (attribute.UhtType.IsAssignableFrom(supportedLeafType))
				{
					UhtCodeGeneratorInjectorDesire desire = new(supportedLeafType, attribute.Location);
					if (!_injectorsTable.TryGetValue(desire, out List<UhtCodeGeneratorInjectorDelegate>? injectors))
					{
						injectors = [];
						_injectorsTable.Add(desire, injectors);
					}
					injectors!.Add((UhtCodeGeneratorInjectorDelegate)Delegate.CreateDelegate(typeof(UhtCodeGeneratorInjectorDelegate), method));
				}
			}
		}

		internal void Sort()
		{
			// Sort the delegates to ensure determinism across multiple runs
			foreach (List<UhtCodeGeneratorInjectorDelegate>? injectors in _injectorsTable.Values)
			{
				if (injectors == null)
				{
					continue;
				}
				injectors.Sort((lhs, rhs) =>
				{
					MethodInfo methodLhs = lhs.Method;
					MethodInfo methodRhs = rhs.Method;

					// Sort by module
					int moduleComparison = String.Compare(methodLhs.Module.Name, methodRhs.Module.Name, StringComparison.Ordinal);
					if (moduleComparison != 0)
					{
						return moduleComparison;
					}

					// Sort by declaring class
					int declaredClassComparison = String.Compare(methodLhs.DeclaringType!.Name, methodRhs.DeclaringType!.Name, StringComparison.Ordinal);
					if (declaredClassComparison != 0)
					{
						return declaredClassComparison;
					}

					// Sort by method name
					return String.Compare(methodLhs.Name, methodRhs.Name, StringComparison.Ordinal);
				});
			}
		}
	}
}
