// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// This attribute is placed on classes that represent Unreal Engine classes.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public sealed class UhtEngineClassAttribute : Attribute
	{

		/// <summary>
		/// The name of the engine class excluding any prefix
		/// </summary>
		public string? Name { get; set; } = null;

		/// <summary>
		/// If true, this class is a property
		/// </summary>
		public bool IsProperty { get; set; } = false;
	}

	/// <summary>
	/// Represents an engine class in the engine class table
	/// </summary>
	public struct UhtEngineClass
	{
		/// <summary>
		/// The name of the engine class excluding any prefix
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// If true, this class is a property
		/// </summary>
		public bool IsProperty { get; set; }
	}

	/// <summary>
	/// Table of all known engine class names.
	/// </summary>
	public class UhtEngineClassTable
	{
		/// <summary>
		/// Internal mapping from engine class name to information
		/// </summary>
		private readonly Dictionary<StringView, UhtEngineClass> _engineClasses = [];

		/// <summary>
		/// Test to see if the given class name is a property
		/// </summary>
		/// <param name="name">Name of the class without the prefix</param>
		/// <returns>True if the class name is a property.  False if the class name isn't a property or isn't an engine class.</returns>
		public bool IsValidPropertyTypeName(StringView name)
		{
			if (_engineClasses.TryGetValue(name, out UhtEngineClass engineClass))
			{
				return engineClass.IsProperty;
			}
			return false;
		}

		/// <summary>
		/// Add an entry to the table
		/// </summary>
		/// <param name="engineClassAttribute">The located attribute</param>
		public void OnEngineClassAttribute(UhtEngineClassAttribute engineClassAttribute)
		{
			if (String.IsNullOrEmpty(engineClassAttribute.Name))
			{
				throw new UhtIceException("EngineClassNames must have a name specified");
			}
			_engineClasses.Add(engineClassAttribute.Name, new UhtEngineClass { Name = engineClassAttribute.Name, IsProperty = engineClassAttribute.IsProperty });
		}
	}
}
