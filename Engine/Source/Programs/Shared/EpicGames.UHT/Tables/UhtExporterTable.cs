// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Delegate to invoke to run exporter
	/// </summary>
	/// <param name="factory">Factory used to generate export tasks and outputs</param>
	public delegate void UhtExporterDelegate(IUhtExportFactory factory);

	/// <summary>
	/// Export options
	/// </summary>
	[Flags]
	public enum UhtExporterOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// The exporter should be run by default
		/// </summary>
		Default = 1 << 0,

		/// <summary>
		/// DEPRECATED. This does not work
		/// </summary>
		[Obsolete("This is not supported anymore. Contact epic if you were using this and really needs it", error: true)]
		CompileOutput = 1 << 1,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtExporterOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtExporterOptions inFlags, UhtExporterOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtExporterOptions inFlags, UhtExporterOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtExporterOptions inFlags, UhtExporterOptions testFlags, UhtExporterOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Defines an exporter
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtExporterAttribute : Attribute
	{

		/// <summary>
		/// Name of the exporter
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Description of the export.  Used to display help
		/// </summary>
		public string Description { get; set; } = String.Empty;

		/// <summary>
		/// Exporters in plugins need to specify a module name
		/// </summary>
		public string ModuleName { get; set; } = String.Empty;

		/// <summary>
		/// Exporter options
		/// </summary>
		public UhtExporterOptions Options { get; set; } = UhtExporterOptions.None;

		/// <summary>
		/// Collection of filters used to delete old cpp files
		/// </summary>
		public string[]? CppFilters { get; set; }

		/// <summary>
		/// Collection of filters used to delete old h files
		/// </summary>
		public string[]? HeaderFilters { get; set; }

		/// <summary>
		/// Collection of filters for other file types
		/// </summary>
		public string[]? OtherFilters { get; set; }
	}

	/// <summary>
	/// Defines an exporter in the table
	/// </summary>
	public readonly struct UhtExporter
	{

		/// <summary>
		/// Name of the exporter
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Description of the export.  Used to display help
		/// </summary>
		public string Description { get; }

		/// <summary>
		/// Exporters in plugins need to specify a module name
		/// </summary>
		public string ModuleName { get; }

		/// <summary>
		/// Exporter options
		/// </summary>
		public UhtExporterOptions Options { get; }

		/// <summary>
		/// Delegate to invoke to start export
		/// </summary>
		public UhtExporterDelegate Delegate { get; }

		/// <summary>
		/// Collection of filters used to delete old cpp files
		/// </summary>
		public IReadOnlyList<string> CppFilters { get; }

		/// <summary>
		/// Collection of filters used to delete old h files
		/// </summary>
		public IReadOnlyList<string> HeaderFilters { get; }

		/// <summary>
		/// Collection of filters for other file types
		/// </summary>
		public IReadOnlyList<string> OtherFilters { get; }

		/// <summary>
		/// Construct an exporter table instance
		/// </summary>
		/// <param name="attribute">Source attribute</param>
		/// <param name="exporterDelegate">Delegate to invoke</param>
		public UhtExporter(UhtExporterAttribute attribute, UhtExporterDelegate exporterDelegate)
		{
			Name = attribute.Name;
			Description = attribute.Description;
			ModuleName = attribute.ModuleName;
			Options = attribute.Options;
			Delegate = exporterDelegate;
			CppFilters = attribute.CppFilters != null ? [.. attribute.CppFilters] : new List<string>();
			HeaderFilters = attribute.HeaderFilters != null ? [.. attribute.HeaderFilters] : new List<string>();
			OtherFilters = attribute.OtherFilters != null ? [.. attribute.OtherFilters] : new List<string>();
		}
	}

	/// <summary>
	/// Exporter table
	/// </summary>
	public class UhtExporterTable : IEnumerable<UhtExporter>
	{

		private readonly Dictionary<string, UhtExporter> _exporterValues = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Return the exporter associated with the given name
		/// </summary>
		/// <param name="name"></param>
		/// <param name="value">Exporter associated with the name</param>
		/// <returns></returns>
		public bool TryGet(string name, out UhtExporter value)
		{
			return _exporterValues.TryGetValue(name, out value);
		}

		/// <summary>
		/// Handle an exporter attribute
		/// </summary>
		/// <param name="type">Containing type</param>
		/// <param name="methodInfo">Method info</param>
		/// <param name="exporterAttribute">Defining attribute</param>
		/// <exception cref="UhtIceException">Thrown if the attribute doesn't properly define an exporter.</exception>
		public void OnExporterAttribute(Type type, MethodInfo methodInfo, UhtExporterAttribute exporterAttribute)
		{
			if (String.IsNullOrEmpty(exporterAttribute.Name))
			{
				throw new UhtIceException("An exporter must have a name");
			}

			if (Assembly.GetExecutingAssembly() != type.Assembly)
			{
				if (String.IsNullOrEmpty(exporterAttribute.ModuleName))
				{
					throw new UhtIceException("An exporter in a UBT plugin must specify a ModuleName");
				}
			}

			UhtExporter exporterValue = new(exporterAttribute, (UhtExporterDelegate)Delegate.CreateDelegate(typeof(UhtExporterDelegate), methodInfo));
			_exporterValues.Add(exporterAttribute.Name, exporterValue);
		}

		/// <summary>
		/// Return an enumerator for all the defined exporters
		/// </summary>
		/// <returns>Enumerator</returns>
		public IEnumerator<UhtExporter> GetEnumerator()
		{
			foreach (KeyValuePair<string, UhtExporter> kvp in _exporterValues)
			{
				yield return kvp.Value;
			}
		}

		/// <summary>
		/// Return an enumerator for all the defined exporters
		/// </summary>
		/// <returns>Enumerator</returns>
		IEnumerator IEnumerable.GetEnumerator()
		{
			foreach (KeyValuePair<string, UhtExporter> kvp in _exporterValues)
			{
				yield return kvp.Value;
			}
		}
	}
}
