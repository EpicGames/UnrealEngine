// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Invoke the given method when the keyword is parsed.
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtKeywordAttribute : Attribute
	{

		/// <summary>
		/// Keyword table/scope being extended
		/// </summary>
		public string? Extends { get; set; }

		/// <summary>
		/// Name of the keyword
		/// </summary>
		public string? Keyword { get; set; } = null;

		/// <summary>
		/// Text to be displayed to the user when referencing this keyword
		/// </summary>
		public string? AllowText { get; set; } = null;

		/// <summary>
		/// If true, this applies to all scopes
		/// </summary>
		public bool AllScopes { get; set; } = false;

		/// <summary>
		/// If true, do not include in usage errors
		/// </summary>
		public bool DisableUsageError { get; set; } = false;

		/// <summary>
		/// List of the allowed compiler directives. 
		/// </summary>
		public UhtCompilerDirective AllowedCompilerDirectives { get; set; } = UhtCompilerDirective.DefaultAllowedCheck;
	}

	/// <summary>
	/// Invoked as a last chance processor for a keyword
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtKeywordCatchAllAttribute : Attribute
	{

		/// <summary>
		/// Table/scope to be extended
		/// </summary>
		public string? Extends { get; set; }
	}

	/// <summary>
	/// Delegate to notify a keyword was parsed
	/// </summary>
	/// <param name="topScope">Current scope being parsed</param>
	/// <param name="actionScope">The scope who's table was matched</param>
	/// <param name="token">Matching token</param>
	/// <returns>Results of the parsing</returns>
	public delegate UhtParseResult UhtKeywordDelegate(UhtParsingScope topScope, UhtParsingScope actionScope, ref UhtToken token);

	/// <summary>
	/// Delegate to invoke as a last chance processor for a keyword
	/// </summary>
	/// <param name="topScope">Current scope being parsed</param>
	/// <param name="token">Matching token</param>
	/// <returns>Results of the parsing</returns>
	public delegate UhtParseResult UhtKeywordCatchAllDelegate(UhtParsingScope topScope, ref UhtToken token);

	/// <summary>
	/// Defines a keyword
	/// </summary>
	public readonly struct UhtKeyword
	{

		/// <summary>
		/// Name of the keyword
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Delegate to invoke
		/// </summary>
		public UhtKeywordDelegate Delegate { get; }

		/// <summary>
		/// Text to be displayed to the user when referencing this keyword
		/// </summary>
		public string? AllowText { get; }

		/// <summary>
		/// If true, this applies to all scopes
		/// </summary>
		public bool AllScopes { get; }

		/// <summary>
		/// If true, do not include in usage errors
		/// </summary>
		public bool DisableUsageError { get; }

		/// <summary>
		/// List of the allowed compiler directives. 
		/// </summary>
		public UhtCompilerDirective AllowedCompilerDirectives { get; }

		/// <summary>
		/// Construct a new keyword
		/// </summary>
		/// <param name="name">Name of the keyword</param>
		/// <param name="keywordDelegate">Delegate to invoke</param>
		/// <param name="attribute">Defining attribute</param>
		public UhtKeyword(string name, UhtKeywordDelegate keywordDelegate, UhtKeywordAttribute? attribute)
		{
			Name = name;
			Delegate = keywordDelegate;
			if (attribute != null)
			{
				AllowText = attribute.AllowText;
				AllScopes = attribute.AllScopes;
				DisableUsageError = attribute.DisableUsageError;
				AllowedCompilerDirectives = attribute.AllowedCompilerDirectives;
			}
			else
			{
				AllowText = null;
				AllScopes = false;
				DisableUsageError = false;
				AllowedCompilerDirectives = UhtCompilerDirective.DefaultAllowedCheck;
			}
		}
	}

	/// <summary>
	/// Keyword table for a specific scope
	/// </summary>
	public class UhtKeywordTable : UhtLookupTable<UhtKeyword>
	{

		/// <summary>
		/// List of catch-alls associated with this table
		/// </summary>
		public List<UhtKeywordCatchAllDelegate> CatchAlls { get; } = [];

		/// <summary>
		/// Construct a new keyword table
		/// </summary>
		public UhtKeywordTable() : base(StringViewComparer.Ordinal)
		{
		}

		/// <summary>
		/// Add the given value to the lookup table.  It will throw an exception if it is a duplicate.
		/// </summary>
		/// <param name="value">Value to be added</param>
		public UhtKeywordTable Add(UhtKeyword value)
		{
			Add(value.Name, value);
			return this;
		}

		/// <summary>
		/// Add the given catch-all to the table.
		/// </summary>
		/// <param name="catchAll">The catch-all to be added</param>
		public UhtKeywordTable AddCatchAll(UhtKeywordCatchAllDelegate catchAll)
		{
			CatchAlls.Add(catchAll);
			return this;
		}

		/// <summary>
		/// Merge the given keyword table.  Duplicates in the BaseTypeTable will be ignored.
		/// </summary>
		/// <param name="baseTable">Base table being merged</param>
		public override void Merge(UhtLookupTableBase baseTable)
		{
			base.Merge(baseTable);
			CatchAlls.AddRange(((UhtKeywordTable)baseTable).CatchAlls);
		}
	}

	/// <summary>
	/// Table of all keyword tables
	/// </summary>
	public class UhtKeywordTables : UhtLookupTables<UhtKeywordTable>
	{

		/// <summary>
		/// Construct the keyword tables
		/// </summary>
		public UhtKeywordTables() : base("keywords")
		{
		}

		/// <summary>
		/// Handle a keyword attribute
		/// </summary>
		/// <param name="type">Containing type</param>
		/// <param name="methodInfo">Method information</param>
		/// <param name="keywordCatchAllAttribute">Defining attribute</param>
		/// <exception cref="UhtIceException">Thrown if the attribute isn't well defined</exception>
		public void OnKeywordCatchAllAttribute(Type type, MethodInfo methodInfo, UhtKeywordCatchAllAttribute keywordCatchAllAttribute)
		{
			if (String.IsNullOrEmpty(keywordCatchAllAttribute.Extends))
			{
				throw new UhtIceException($"The 'KeywordCatchAlll' attribute on the {type.Name}.{methodInfo.Name} method doesn't have a table specified.");
			}

			UhtKeywordTable table = Get(keywordCatchAllAttribute.Extends);
			table.AddCatchAll((UhtKeywordCatchAllDelegate)Delegate.CreateDelegate(typeof(UhtKeywordCatchAllDelegate), methodInfo));
		}

		/// <summary>
		/// Handle a keyword attribute
		/// </summary>
		/// <param name="type">Containing type</param>
		/// <param name="methodInfo">Method information</param>
		/// <param name="keywordAttribute">Defining attribute</param>
		/// <exception cref="UhtIceException">Thrown if the attribute isn't well defined</exception>
		public void OnKeywordAttribute(Type type, MethodInfo methodInfo, UhtKeywordAttribute keywordAttribute)
		{
			string name = UhtLookupTableBase.GetSuffixedName(type, methodInfo, keywordAttribute.Keyword, "Keyword");

			if (String.IsNullOrEmpty(keywordAttribute.Extends))
			{
				throw new UhtIceException($"The 'Keyword' attribute on the {type.Name}.{methodInfo.Name} method doesn't have a table specified.");
			}

			UhtKeywordTable table = Get(keywordAttribute.Extends);
			table.Add(new UhtKeyword(name, (UhtKeywordDelegate)Delegate.CreateDelegate(typeof(UhtKeywordDelegate), methodInfo), keywordAttribute));
		}

		/// <summary>
		/// Log an unhandled error
		/// </summary>
		/// <param name="messageSite">Destination message site</param>
		/// <param name="token">Keyword</param>
		public void LogUnhandledError(IUhtMessageSite messageSite, UhtToken token)
		{
			List<string>? tables = null;
			foreach (KeyValuePair<string, UhtKeywordTable> kvp in Tables)
			{
				UhtKeywordTable keywordTable = kvp.Value;
				if (keywordTable.Internal)
				{
					continue;
				}
				if (keywordTable.TryGetValue(token.Value, out UhtKeyword info))
				{
					if (info.DisableUsageError)
					{
						// Do not log anything for this keyword in this table
					}
					else
					{
						tables ??= [];
						if (info.AllowText != null)
						{
							tables.Add($"{keywordTable.UserName} {info.AllowText}");
						}
						else
						{
							tables.Add(keywordTable.UserName);
						}
					}
				}
			}

			if (tables != null)
			{
				string text = UhtUtilities.MergeTypeNames(tables, "and");
				messageSite.LogError(token.InputLine, $"Invalid use of keyword '{token.Value}'.  It may only appear in {text} scopes");
			}
		}
	}
}
