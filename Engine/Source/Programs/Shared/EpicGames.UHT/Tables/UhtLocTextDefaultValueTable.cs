// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Delegate to invoke to sanitize a loctext default value
	/// </summary>
	/// <param name="property">Property in question</param>
	/// <param name="defaultValueReader">The default value</param>
	/// <param name="macroToken">Token for the loctext type being parsed</param>
	/// <param name="innerDefaultValue">Output sanitized value.</param>
	/// <returns>True if sanitized, false if not.</returns>
	public delegate bool UhtLocTextDefaultValueDelegate(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue);

	/// <summary>
	/// Attribute defining the loctext sanitizer
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtLocTextDefaultValueAttribute : Attribute
	{

		/// <summary>
		/// Name of the sanitizer (i.e. LOCTEXT, NSLOCTEXT, ...)
		/// </summary>
		public string? Name { get; set; }
	}

	/// <summary>
	/// Loctext sanitizer
	/// </summary>
	public struct UhtLocTextDefaultValue
	{
		/// <summary>
		/// Delegate to invoke
		/// </summary>
		public UhtLocTextDefaultValueDelegate Delegate { get; set; }
	}

	/// <summary>
	/// Table of loctext sanitizers
	/// </summary>
	public class UhtLocTextDefaultValueTable
	{

		private readonly Dictionary<StringView, UhtLocTextDefaultValue> _locTextDefaultValues = [];

		/// <summary>
		/// Return the loc text default value associated with the given name
		/// </summary>
		/// <param name="name"></param>
		/// <param name="locTextDefaultValue">Loc text default value handler</param>
		/// <returns></returns>
		public bool TryGet(StringView name, out UhtLocTextDefaultValue locTextDefaultValue)
		{
			return _locTextDefaultValues.TryGetValue(name, out locTextDefaultValue);
		}

		/// <summary>
		/// Handle a loctext default value attribute
		/// </summary>
		/// <param name="methodInfo">Method info</param>
		/// <param name="locTextDefaultValueAttribute">Defining attribute</param>
		/// <exception cref="UhtIceException">Thrown if the attribute isn't properly defined</exception>
		public void OnLocTextDefaultValueAttribute(MethodInfo methodInfo, UhtLocTextDefaultValueAttribute locTextDefaultValueAttribute)
		{
			if (String.IsNullOrEmpty(locTextDefaultValueAttribute.Name))
			{
				throw new UhtIceException("A loc text default value attribute must have a name");
			}

			UhtLocTextDefaultValue locTextDefaultValue = new()
			{
				Delegate = (UhtLocTextDefaultValueDelegate)Delegate.CreateDelegate(typeof(UhtLocTextDefaultValueDelegate), methodInfo)
			};

			_locTextDefaultValues.Add(new StringView(locTextDefaultValueAttribute.Name), locTextDefaultValue);
		}
	}
}
