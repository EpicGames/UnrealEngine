// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Property type options
	/// </summary>
	[Flags]
	public enum UhtPropertyTypeOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// Simple property type with just the property type.  (i.e. "int32 MyValue")
		/// Simple types are not required to parse the supplied token list.
		/// </summary>
		Simple = 1 << 0,

		/// <summary>
		/// Use case insensitive string compares
		/// </summary>
		CaseInsensitive = 1 << 1,

		/// <summary>
		/// This property type is to be invoked when there are no keyword matches found
		/// </summary>
		Default = 1 << 2,

		/// <summary>
		/// This property type doesn't reference any engine types an can be resolved immediately
		/// </summary>
		Immediate = 1 << 3,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtPropertyTypeOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtPropertyTypeOptions inFlags, UhtPropertyTypeOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtPropertyTypeOptions inFlags, UhtPropertyTypeOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtPropertyTypeOptions inFlags, UhtPropertyTypeOptions testFlags, UhtPropertyTypeOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Delegate invoked to resolve a tokenized type into a UHTProperty type
	/// </summary>
	/// <param name="args">Collection of arguments for property resolution</param>
	/// <returns>Created property or null if the resolution must be delayed until after symbol table has been constructed</returns>
	public delegate UhtProperty? UhtResolvePropertyDelegate(UhtPropertyResolveArgs args);

	/// <summary>
	/// Property type attribute
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtPropertyTypeAttribute : Attribute
	{

		/// <summary>
		/// The expected keyword.  Must be set unless this is the default processor
		/// </summary>
		public string? Keyword { get; set; } = null;

		/// <summary>
		/// Options
		/// </summary>
		public UhtPropertyTypeOptions Options { get; set; } = UhtPropertyTypeOptions.None;
	}

	/// <summary>
	/// Represents a property type as specified by the PropertyTypeAttribute
	/// </summary>
	public struct UhtPropertyType
	{

		/// <summary>
		/// Delegate to invoke
		/// </summary>
		public UhtResolvePropertyDelegate Delegate { get; set; }

		/// <summary>
		/// Options
		/// </summary>
		public UhtPropertyTypeOptions Options { get; set; }
	}

	/// <summary>
	/// Property type table
	/// </summary>
	public class UhtPropertyTypeTable
	{
		private readonly Dictionary<StringView, UhtPropertyType> _caseSensitive = [];
		private readonly Dictionary<StringView, UhtPropertyType> _caseInsensitive = new(StringViewComparer.OrdinalIgnoreCase);
		private UhtPropertyType? _default = null;

		/// <summary>
		/// Return the default processor
		/// </summary>
		public UhtPropertyType Default
		{
			get
			{
				if (_default == null)
				{
					throw new UhtIceException("No property type has been marked as default");
				}
				return (UhtPropertyType)_default;
			}
		}

		/// <summary>
		/// Return the property type associated with the given name
		/// </summary>
		/// <param name="name"></param>
		/// <param name="propertyType">Property type if matched</param>
		/// <returns></returns>
		public bool TryGet(StringView name, out UhtPropertyType propertyType)
		{
			return
				_caseSensitive.TryGetValue(name, out propertyType) ||
				_caseInsensitive.TryGetValue(name, out propertyType);
		}

		/// <summary>
		/// Handle a property type attribute
		/// </summary>
		/// <param name="methodInfo">Method info</param>
		/// <param name="propertyTypeAttribute">Attribute</param>
		/// <exception cref="UhtIceException">Thrown if the property type isn't properly defined.</exception>
		public void OnPropertyTypeAttribute(MethodInfo methodInfo, UhtPropertyTypeAttribute propertyTypeAttribute)
		{
			if (String.IsNullOrEmpty(propertyTypeAttribute.Keyword) && !propertyTypeAttribute.Options.HasAnyFlags(UhtPropertyTypeOptions.Default))
			{
				throw new UhtIceException("A property type must have a keyword or be marked as default");
			}

			UhtPropertyType propertyType = new()
			{
				Delegate = (UhtResolvePropertyDelegate)Delegate.CreateDelegate(typeof(UhtResolvePropertyDelegate), methodInfo),
				Options = propertyTypeAttribute.Options,
			};

			if (propertyTypeAttribute.Options.HasAnyFlags(UhtPropertyTypeOptions.Default))
			{
				if (_default != null)
				{
					throw new UhtIceException("Only one property type dispatcher can be marked as default");
				}
				_default = propertyType;
			}
			else if (!String.IsNullOrEmpty(propertyTypeAttribute.Keyword))
			{
				if (propertyTypeAttribute.Options.HasAnyFlags(UhtPropertyTypeOptions.CaseInsensitive))
				{
					_caseInsensitive.Add(propertyTypeAttribute.Keyword, propertyType);
				}
				else
				{
					_caseSensitive.Add(propertyTypeAttribute.Keyword, propertyType);
				}
			}
		}
	}
}
