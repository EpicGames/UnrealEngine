// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Parsers; // It would be nice if we didn't need this here
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Defines the different types specifiers relating to their allowed values
	/// </summary>
	public enum UhtSpecifierValueType
	{
		/// <summary>
		/// Internal value used to detect if the attribute has a valid value
		/// </summary>
		NotSet,

		/// <summary>
		/// No values of any type are allowed
		/// </summary>
		None,

		/// <summary>
		/// A string value but can not be in the form of a list (i.e. =(bob))
		/// </summary>
		String,

		/// <summary>
		/// An optional string value but can not be in the form of a list
		/// </summary>
		OptionalString,

		/// <summary>
		/// A string value or a single element string list
		/// </summary>
		SingleString,

		/// <summary>
		/// A list of values in key=value pairs
		/// </summary>
		KeyValuePairList,

		/// <summary>
		/// A list of values in key=value pairs but the equals is optional
		/// </summary>
		OptionalEqualsKeyValuePairList,

		/// <summary>
		/// A list of values.
		/// </summary>
		StringList,

		/// <summary>
		/// A list of values and must contain at least one entry
		/// </summary>
		NonEmptyStringList,

		/// <summary>
		/// Accepts a string list but the value is ignored by the specifier and is automatically deferred.  This is for legacy UHT support.
		/// </summary>
		[Obsolete("Legacy specifier mode is obsolete, please use None")]
		Legacy,
	}

	/// <summary>
	/// Results from dispatching a specifier
	/// </summary>
	public enum UhtSpecifierDispatchResults
	{

		/// <summary>
		/// Specifier was known and parsed
		/// </summary>
		Known,

		/// <summary>
		/// Specified was unknown
		/// </summary>
		Unknown,
	}

	/// <summary>
	/// The specifier context provides the default and simplest information about the specifiers being processed
	/// </summary>
	public class UhtSpecifierContext
	{
		private UhtType? _type = null;
		private IUhtTokenReader? _tokenReader = null;
		private IUhtMessageSite? _messageSite = null;
		private UhtMetaData? _metaData = null;

		/// <summary>
		/// Get the type containing the specifiers.  For properties, this is the outer object and
		/// not the property itself.
		/// </summary>
		public UhtType Type { get => _type!; set => _type = value; }

		/// <summary>
		/// Return the currently active token reader
		/// </summary>
		public IUhtTokenReader TokenReader { get => _tokenReader!; set => _tokenReader = value; }

		/// <summary>
		/// Current access specifier
		/// </summary>
		public UhtAccessSpecifier AccessSpecifier { get; set; } = UhtAccessSpecifier.None;

		/// <summary>
		/// Message site for messages
		/// </summary>
		public IUhtMessageSite MessageSite { get => _messageSite!; set => _messageSite = value; }

		/// <summary>
		/// Meta data currently being parsed.
		/// </summary>
		public UhtMetaData MetaData { get => _metaData!; set => _metaData = value; }

		/// <summary>
		/// Make data key index utilized by enumeration values
		/// </summary>
		public int MetaNameIndex { get; set; } = UhtMetaData.IndexNone;

		/// <summary>
		/// Construct a new specifier context
		/// </summary>
		/// <param name="scope"></param>
		/// <param name="messageSite"></param>
		/// <param name="metaData"></param>
		/// <param name="metaNameIndex"></param>
		public UhtSpecifierContext(UhtParsingScope scope, IUhtMessageSite messageSite, UhtMetaData metaData, int metaNameIndex = UhtMetaData.IndexNone)
		{
			Type = scope.ScopeType;
			TokenReader = scope.TokenReader;
			AccessSpecifier = scope.AccessSpecifier;
			MessageSite = messageSite;
			MetaData = metaData;
			MetaNameIndex = metaNameIndex;
		}

		/// <summary>
		/// Construct an empty context.  Scope, MessageSite, and MetaData must be set at a later point
		/// </summary>
		public UhtSpecifierContext()
		{
		}
	}

	/// <summary>
	/// Specifiers are either processed immediately when the declaration is parse or deferred until later in the parsing of the object
	/// </summary>
	public enum UhtSpecifierWhen
	{
		/// <summary>
		/// Specifier is parsed when the meta data section is parsed.
		/// </summary>
		Immediate,

		/// <summary>
		/// Specifier is executed after more of the object is parsed (but usually before members are parsed)
		/// </summary>
		Deferred,
	}

	/// <summary>
	/// The specifier table contains an instance of UhtSpecifier which is used to dispatch the parsing of
	/// a specifier to the implementation
	/// </summary>
	public abstract class UhtSpecifier
	{

		/// <summary>
		/// Name of the specifier
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Expected value type
		/// </summary>
		public UhtSpecifierValueType ValueType { get; set; }

		/// <summary>
		/// When is the specifier executed
		/// </summary>
		public UhtSpecifierWhen When { get; set; } = UhtSpecifierWhen.Deferred;

		/// <summary>
		/// Dispatch an instance of the specifier
		/// </summary>
		/// <param name="specifierContext">Current context</param>
		/// <param name="value">Specifier value</param>
		/// <returns>Results of the dispatch</returns>
		public abstract UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value);
	}

	/// <summary>
	/// Delegate for a specifier with no value
	/// </summary>
	/// <param name="specifierContext"></param>
	public delegate void UhtSpecifierNoneDelegate(UhtSpecifierContext specifierContext);

	/// <summary>
	/// Specifier with no value
	/// </summary>
	public class UhtSpecifierNone : UhtSpecifier
	{
		private readonly UhtSpecifierNoneDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierNone(string name, UhtSpecifierWhen when, UhtSpecifierNoneDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.None;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			_delegate(specifierContext);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with a string value
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierStringDelegate(UhtSpecifierContext specifierContext, StringView value);

	/// <summary>
	/// Specifier with a string value
	/// </summary>
	public class UhtSpecifierString : UhtSpecifier
	{
		private readonly UhtSpecifierStringDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierString(string name, UhtSpecifierWhen when, UhtSpecifierStringDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.String;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			if (value == null)
			{
				throw new UhtIceException("Required value is null");
			}
			_delegate(specifierContext, (StringView)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with an optional string value
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierOptionalStringDelegate(UhtSpecifierContext specifierContext, StringView? value);

	/// <summary>
	/// Specifier with an optional string value
	/// </summary>
	public class UhtSpecifierOptionalString : UhtSpecifier
	{
		private readonly UhtSpecifierOptionalStringDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierOptionalString(string name, UhtSpecifierWhen when, UhtSpecifierOptionalStringDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.OptionalString;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			_delegate(specifierContext, (StringView?)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with a string value
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierSingleStringDelegate(UhtSpecifierContext specifierContext, StringView value);

	/// <summary>
	/// Specifier with a string value
	/// </summary>
	public class UhtSpecifierSingleString : UhtSpecifier
	{
		private readonly UhtSpecifierSingleStringDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierSingleString(string name, UhtSpecifierWhen when, UhtSpecifierSingleStringDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.SingleString;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			if (value == null)
			{
				throw new UhtIceException("Required value is null");
			}
			_delegate(specifierContext, (StringView)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with list of string keys and values 
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierKeyValuePairListDelegate(UhtSpecifierContext specifierContext, List<KeyValuePair<StringView, StringView>> value);

	/// <summary>
	/// Specifier with list of string keys and values 
	/// </summary>
	public class UhtSpecifierKeyValuePairList : UhtSpecifier
	{
		private readonly UhtSpecifierKeyValuePairListDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="equalsOptional">If true this has an optional KVP list</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierKeyValuePairList(string name, UhtSpecifierWhen when, bool equalsOptional, UhtSpecifierKeyValuePairListDelegate specifierDelegate)
		{
			Name = name;
			ValueType = equalsOptional ? UhtSpecifierValueType.OptionalEqualsKeyValuePairList : UhtSpecifierValueType.KeyValuePairList;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			if (value == null)
			{
				throw new UhtIceException("Required value is null");
			}
			_delegate(specifierContext, (List<KeyValuePair<StringView, StringView>>)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with no value
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	public delegate void UhtSpecifierLegacyDelegate(UhtSpecifierContext specifierContext);

	/// <summary>
	/// Specifier delegate for legacy UHT specifiers with no value.  Will generate a information/deprecation message
	/// is a value is supplied
	/// </summary>
	public class UhtSpecifierLegacy : UhtSpecifier
	{
		private readonly UhtSpecifierLegacyDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierLegacy(string name, UhtSpecifierLegacyDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.StringList;
			When = UhtSpecifierWhen.Deferred;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			if (value != null)
			{
				specifierContext.TokenReader.LogDeprecation($"Specifier '{Name}' has a value which is unused, future versions of UnrealHeaderTool will flag this as an error.");
			}
			_delegate(specifierContext);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with an optional string list
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierStringListDelegate(UhtSpecifierContext specifierContext, List<StringView>? value);

	/// <summary>
	/// Specifier with an optional string list
	/// </summary>
	public class UhtSpecifierStringList : UhtSpecifier
	{
		private readonly UhtSpecifierStringListDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierStringList(string name, UhtSpecifierWhen when, UhtSpecifierStringListDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.StringList;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			_delegate(specifierContext, (List<StringView>?)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Specifier delegate with a list of string views
	/// </summary>
	/// <param name="specifierContext">Specifier context</param>
	/// <param name="value">Specifier value</param>
	public delegate void UhtSpecifierNonEmptyStringListDelegate(UhtSpecifierContext specifierContext, List<StringView> value);

	/// <summary>
	/// Specifier with a list of string views
	/// </summary>
	public class UhtSpecifierNonEmptyStringList : UhtSpecifier
	{
		private readonly UhtSpecifierNonEmptyStringListDelegate _delegate;

		/// <summary>
		/// Construct the specifier
		/// </summary>
		/// <param name="name">Name of the specifier</param>
		/// <param name="when">When the specifier is executed</param>
		/// <param name="specifierDelegate">Delegate to invoke</param>
		public UhtSpecifierNonEmptyStringList(string name, UhtSpecifierWhen when, UhtSpecifierNonEmptyStringListDelegate specifierDelegate)
		{
			Name = name;
			ValueType = UhtSpecifierValueType.NonEmptyStringList;
			When = when;
			_delegate = specifierDelegate;
		}

		/// <inheritdoc/>
		public override UhtSpecifierDispatchResults Dispatch(UhtSpecifierContext specifierContext, object? value)
		{
			if (value == null)
			{
				throw new UhtIceException("Required value is null");
			}
			_delegate(specifierContext, (List<StringView>)value);
			return UhtSpecifierDispatchResults.Known;
		}
	}

	/// <summary>
	/// Defines a specifier method
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtSpecifierAttribute : Attribute
	{
		/// <summary>
		/// Name of the specifier.   If not supplied, the method name must end in "Specifier" and the name will be the method name with "Specifier" stripped.
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Name of the table/scope this specifier applies
		/// </summary>
		public string? Extends { get; set; }

		/// <summary>
		/// Value type of the specifier
		/// </summary>
		public UhtSpecifierValueType ValueType { get; set; } = UhtSpecifierValueType.NotSet;

		/// <summary>
		/// When the specifier is dispatched
		/// </summary>
		public UhtSpecifierWhen When { get; set; } = UhtSpecifierWhen.Deferred;
	}

	/// <summary>
	/// Collection of specifiers for a given scope
	/// </summary>
	public class UhtSpecifierTable : UhtLookupTable<UhtSpecifier>
	{

		/// <summary>
		/// Construct a new specifier table
		/// </summary>
		public UhtSpecifierTable() : base(StringViewComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Add the given value to the lookup table.  It will throw an exception if it is a duplicate.
		/// </summary>
		/// <param name="specifier">The specifier to add</param>
		public UhtSpecifierTable Add(UhtSpecifier specifier)
		{
			Add(specifier.Name, specifier);
			return this;
		}
	}

	/// <summary>
	/// Collection of all specifier tables
	/// </summary>
	public class UhtSpecifierTables : UhtLookupTables<UhtSpecifierTable>
	{

		/// <summary>
		/// Construct the specifier table
		/// </summary>
		public UhtSpecifierTables() : base("specifiers")
		{
		}

		/// <summary>
		/// Invoke for a method that has the specifier attribute
		/// </summary>
		/// <param name="type">Type containing the method</param>
		/// <param name="methodInfo">Method info</param>
		/// <param name="specifierAttribute">Specified attributes</param>
		/// <exception cref="UhtIceException">Throw if the attribute isn't properly defined.</exception>
		public void OnSpecifierAttribute(Type type, MethodInfo methodInfo, UhtSpecifierAttribute specifierAttribute)
		{
			string name = UhtLookupTableBase.GetSuffixedName(type, methodInfo, specifierAttribute.Name, "Specifier");

			if (String.IsNullOrEmpty(specifierAttribute.Extends))
			{
				throw new UhtIceException($"The 'Specifier' attribute on the {type.Name}.{methodInfo.Name} method doesn't have a table specified.");
			}
			else
			{
			}

			if (specifierAttribute.ValueType == UhtSpecifierValueType.NotSet)
			{
				throw new UhtIceException($"The 'Specifier' attribute on the {type.Name}.{methodInfo.Name} method doesn't have a value type specified.");
			}

			UhtSpecifierTable table = Get(specifierAttribute.Extends);
			switch (specifierAttribute.ValueType)
			{
				case UhtSpecifierValueType.None:
					table.Add(new UhtSpecifierNone(name, specifierAttribute.When, (UhtSpecifierNoneDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierNoneDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.String:
					table.Add(new UhtSpecifierString(name, specifierAttribute.When, (UhtSpecifierStringDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierStringDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.OptionalString:
					table.Add(new UhtSpecifierOptionalString(name, specifierAttribute.When, (UhtSpecifierOptionalStringDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierOptionalStringDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.SingleString:
					table.Add(new UhtSpecifierSingleString(name, specifierAttribute.When, (UhtSpecifierSingleStringDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierSingleStringDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.KeyValuePairList:
					table.Add(new UhtSpecifierKeyValuePairList(name, specifierAttribute.When, false, (UhtSpecifierKeyValuePairListDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierKeyValuePairListDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.OptionalEqualsKeyValuePairList:
					table.Add(new UhtSpecifierKeyValuePairList(name, specifierAttribute.When, true, (UhtSpecifierKeyValuePairListDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierKeyValuePairListDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.StringList:
					table.Add(new UhtSpecifierStringList(name, specifierAttribute.When, (UhtSpecifierStringListDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierStringListDelegate), methodInfo)));
					break;
				case UhtSpecifierValueType.NonEmptyStringList:
					table.Add(new UhtSpecifierNonEmptyStringList(name, specifierAttribute.When, (UhtSpecifierNonEmptyStringListDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierNonEmptyStringListDelegate), methodInfo)));
					break;
#pragma warning disable CS0618 // Type or member is obsolete
				case UhtSpecifierValueType.Legacy:
#pragma warning restore CS0618 // Type or member is obsolete
					table.Add(new UhtSpecifierLegacy(name, (UhtSpecifierLegacyDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierLegacyDelegate), methodInfo)));
					break;
			}
		}
	}
}
