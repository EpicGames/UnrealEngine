// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Delegate used to validate a specifier
	/// </summary>
	/// <param name="type">Containing type</param>
	/// <param name="metaData">Containing meta data</param>
	/// <param name="key">Key of the meta data entry</param>
	/// <param name="value">Value of the meta data entry</param>
	public delegate void UhtSpecifierValidatorDelegate(UhtType type, UhtMetaData metaData, UhtMetaDataKey key, StringView value);

	/// <summary>
	/// Defines a specifier validated created from the attribute
	/// </summary>
	public class UhtSpecifierValidator
	{

		/// <summary>
		/// Name of the validator
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Delegate for the validator
		/// </summary>
		public UhtSpecifierValidatorDelegate Delegate { get; set; }

		/// <summary>
		/// Construct a new instance
		/// </summary>
		/// <param name="name">Name of the validator</param>
		/// <param name="specifierValidatorDelegate">Delegate of the validator</param>
		public UhtSpecifierValidator(string name, UhtSpecifierValidatorDelegate specifierValidatorDelegate)
		{
			Name = name;
			Delegate = specifierValidatorDelegate;
		}
	}

	/// <summary>
	/// Attribute used to create a specifier validator
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtSpecifierValidatorAttribute : Attribute
	{

		/// <summary>
		/// Name of the validator. If not supplied &quot;SpecifierValidator&quot; will be removed from the end of the method name
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Name of the table/scope for the validator
		/// </summary>
		public string? Extends { get; set; }
	}

	/// <summary>
	/// A table for validators for a given scope
	/// </summary>
	public class UhtSpecifierValidatorTable : UhtLookupTable<UhtSpecifierValidator>
	{

		/// <summary>
		/// Construct a new specifier table
		/// </summary>
		public UhtSpecifierValidatorTable() : base(StringViewComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Add the given value to the lookup table.  It will throw an exception if it is a duplicate.
		/// </summary>
		/// <param name="specifier">Validator to add</param>
		public UhtSpecifierValidatorTable Add(UhtSpecifierValidator specifier)
		{
			Add(specifier.Name, specifier);
			return this;
		}
	}

	/// <summary>
	/// Collection of specifier validators
	/// </summary>
	public class UhtSpecifierValidatorTables : UhtLookupTables<UhtSpecifierValidatorTable>
	{

		/// <summary>
		/// Construct the validator tables
		/// </summary>
		public UhtSpecifierValidatorTables() : base("specifier validators")
		{
		}

		/// <summary>
		/// Handle the attribute appearing on a method
		/// </summary>
		/// <param name="type">Type containing the method</param>
		/// <param name="methodInfo">The method</param>
		/// <param name="specifierValidatorAttribute">Attribute</param>
		/// <exception cref="UhtIceException">Thrown if the validator isn't properly defined</exception>
		public void OnSpecifierValidatorAttribute(Type type, MethodInfo methodInfo, UhtSpecifierValidatorAttribute specifierValidatorAttribute)
		{
			string name = UhtLookupTableBase.GetSuffixedName(type, methodInfo, specifierValidatorAttribute.Name, "SpecifierValidator");

			if (String.IsNullOrEmpty(specifierValidatorAttribute.Extends))
			{
				throw new UhtIceException($"The 'SpecifierValidator' attribute on the {type.Name}.{methodInfo.Name} method doesn't have a table specified.");
			}
			UhtSpecifierValidatorTable table = Get(specifierValidatorAttribute.Extends);
			table.Add(new UhtSpecifierValidator(name, (UhtSpecifierValidatorDelegate)Delegate.CreateDelegate(typeof(UhtSpecifierValidatorDelegate), methodInfo)));
		}
	}
}
