// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tables
{

	/// <summary>
	/// Delegate for invoking structure default value sanitizer
	/// </summary>
	/// <param name="property"></param>
	/// <param name="defaultValueReader"></param>
	/// <param name="innerDefaultValue"></param>
	/// <returns></returns>
	public delegate bool UhtStructDefaultValueDelegate(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue);

	/// <summary>
	/// Options for structure default value sanitizer
	/// </summary>
	[Flags]
	public enum UhtStructDefaultValueOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None = 0,

		/// <summary>
		/// This method is to be invoked when there are no keyword matches found
		/// </summary>
		Default = 1 << 2,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtStructDefaultValueOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtStructDefaultValueOptions inFlags, UhtStructDefaultValueOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtStructDefaultValueOptions inFlags, UhtStructDefaultValueOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtStructDefaultValueOptions inFlags, UhtStructDefaultValueOptions testFlags, UhtStructDefaultValueOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Structure default value sanitizer attribute
	/// </summary>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = true)]
	public sealed class UhtStructDefaultValueAttribute : Attribute
	{

		/// <summary>
		/// Name of the structure.  Not required for default processor.  
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Options
		/// </summary>
		public UhtStructDefaultValueOptions Options { get; set; } = UhtStructDefaultValueOptions.None;
	}

	/// <summary>
	/// Structure default value sanitizer
	/// </summary>
	public struct UhtStructDefaultValue
	{

		/// <summary>
		/// The delegate to invoke
		/// </summary>
		public UhtStructDefaultValueDelegate Delegate { get; set; }
	}

	/// <summary>
	/// Table of all structure default value specifiers
	/// </summary>
	public class UhtStructDefaultValueTable
	{
		private readonly Dictionary<StringView, UhtStructDefaultValue> _structDefaultValues = [];
		private UhtStructDefaultValue? _default = null;

		/// <summary>
		/// Fetch the default sanitizer
		/// </summary>
		public UhtStructDefaultValue Default
		{
			get
			{
				if (_default == null)
				{
					throw new UhtIceException("No struct default value has been marked as default");
				}
				return (UhtStructDefaultValue)_default;
			}
		}

		/// <summary>
		/// Return the structure default value associated with the given name
		/// </summary>
		/// <param name="name"></param>
		/// <param name="structDefaultValue">Structure default value handler</param>
		/// <returns></returns>
		public bool TryGet(StringView name, out UhtStructDefaultValue structDefaultValue)
		{
			return _structDefaultValues.TryGetValue(name, out structDefaultValue);
		}

		/// <summary>
		/// Handle a structure default value sanitizer attribute
		/// </summary>
		/// <param name="methodInfo">Method information</param>
		/// <param name="structDefaultValueAttribute">Found attribute</param>
		/// <exception cref="UhtIceException">Thrown if the attribute isn't property defined</exception>
		public void OnStructDefaultValueAttribute(MethodInfo methodInfo, UhtStructDefaultValueAttribute structDefaultValueAttribute)
		{
			if (String.IsNullOrEmpty(structDefaultValueAttribute.Name) && !structDefaultValueAttribute.Options.HasAnyFlags(UhtStructDefaultValueOptions.Default))
			{
				throw new UhtIceException("A struct default value attribute must have a name or be marked as default");
			}

			UhtStructDefaultValue structDefaultValue = new()
			{
				Delegate = (UhtStructDefaultValueDelegate)Delegate.CreateDelegate(typeof(UhtStructDefaultValueDelegate), methodInfo)
			};

			if (structDefaultValueAttribute.Options.HasAnyFlags(UhtStructDefaultValueOptions.Default))
			{
				if (_default != null)
				{
					throw new UhtIceException("Only one struct default value attribute dispatcher can be marked as default");
				}
				_default = structDefaultValue;
			}
			else if (!String.IsNullOrEmpty(structDefaultValueAttribute.Name))
			{
				_structDefaultValues.Add(new StringView(structDefaultValueAttribute.Name), structDefaultValue);
			}
		}
	}
}
