// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Type of the token
	/// </summary>
	public enum UhtTokenType
	{

		/// <summary>
		/// End of file token.
		/// </summary>
		EndOfFile,

		/// <summary>
		/// End of default value. 
		/// </summary>
		EndOfDefault,

		/// <summary>
		/// End of type
		/// </summary>
		EndOfType,

		/// <summary>
		/// End of declaration
		/// </summary>
		EndOfDeclaration,

		/// <summary>
		/// Line of text (when calling GetLine only)
		/// </summary>
		Line,

		/// <summary>
		/// Alphanumeric identifier.
		/// </summary>
		Identifier,

		/// <summary>
		/// Symbol.
		/// </summary>
		Symbol,

		/// <summary>
		/// Floating point constant
		/// </summary>
		FloatConst,

		/// <summary>
		/// Decimal Integer constant
		/// </summary>
		DecimalConst,

		/// <summary>
		/// Hex integer constant
		/// </summary>
		HexConst,

		/// <summary>
		/// Single character constant
		/// </summary>
		CharConst,

		/// <summary>
		/// String constant
		/// </summary>
		StringConst,
	}

	/// <summary>
	/// Series of extension methods for the token type
	/// </summary>
	public static class UhtTokenTypeExtensions
	{

		/// <summary>
		/// Return true if the token type is an end type
		/// </summary>
		/// <param name="tokenType">Token type in question</param>
		/// <returns>True if the token type is an end type</returns>
		public static bool IsEndType(this UhtTokenType tokenType)
		{
			switch (tokenType)
			{
				case UhtTokenType.EndOfFile:
				case UhtTokenType.EndOfDefault:
				case UhtTokenType.EndOfType:
				case UhtTokenType.EndOfDeclaration:
					return true;
				default:
					return false;
			}
		}
	}

	/// <summary>
	/// Token declaration
	/// </summary>
	public struct UhtToken
	{

		/// <summary>
		/// Names/Identifiers can not be longer that the following
		/// </summary>
		public const int MaxNameLength = 1024;

		/// <summary>
		/// Strings can not be longer than the following.
		/// </summary>
		public const int MaxStringLength = 1024;

		/// <summary>
		/// Type of the token
		/// </summary>
		public UhtTokenType TokenType { get; set; }

		/// <summary>
		/// Position to restore the reader
		/// </summary>
		public int UngetPos { get; set; }

		/// <summary>
		/// Line to restore the reader
		/// </summary>
		public int UngetLine { get; set; }

		/// <summary>
		/// Starting position of the token value
		/// </summary>
		public int InputStartPos { get; set; }

		/// <summary>
		/// End position of the token value
		/// </summary>
		public readonly int InputEndPos => InputStartPos + Value.Span.Length;

		/// <summary>
		/// Line containing the token
		/// </summary>
		public int InputLine { get; set; }

		/// <summary>
		/// Token value
		/// </summary>
		public StringView Value { get; set; }

		/// <summary>
		/// If true, this token is part of the header input stream
		/// </summary>
		public bool IsInHeader { get; set; }

		/// <summary>
		/// Construct a new token
		/// </summary>
		/// <param name="tokenType">Type of the token</param>
		public UhtToken(UhtTokenType tokenType)
		{
			TokenType = tokenType;
			UngetPos = 0;
			UngetLine = 0;
			InputStartPos = 0;
			InputLine = 0;
			Value = new StringView();
			IsInHeader = false;
		}

		/// <summary>
		/// Construct a new token
		/// </summary>
		/// <param name="tokenType">Type of token</param>
		/// <param name="ungetPos">Unget position</param>
		/// <param name="ungetLine">Unget line</param>
		/// <param name="inputStartPos">Start position of value</param>
		/// <param name="inputLine">Line of value</param>
		/// <param name="value">Token value</param>
		/// <param name="isInHeader">If true, the token is from a header</param>
		public UhtToken(UhtTokenType tokenType, int ungetPos, int ungetLine, int inputStartPos, int inputLine, StringView value, bool isInHeader = true)
		{
			TokenType = tokenType;
			UngetPos = ungetPos;
			UngetLine = ungetLine;
			InputStartPos = inputStartPos;
			InputLine = inputLine;
			Value = value;
			IsInHeader = isInHeader;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtToken(UhtInputCacheReader reader)
		{
			TokenType = (UhtTokenType)reader.ReadInt32();
			UngetPos = 0;
			UngetLine = 0;
			InputStartPos = reader.ReadInt32();
			InputLine = reader.ReadInt32();
			IsInHeader = reader.ReadBoolean();
			if (IsInHeader)
			{
				int length = reader.ReadInt32();
				Value = new(reader.Header.Data.Memory.Slice(InputStartPos, length));
			}
			else
			{
				Value = reader.ReadString();
			}
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public readonly void Write(UhtInputCacheWriter writer)
		{
			writer.WriteInt32((int)TokenType);
			writer.WriteInt32((int)InputStartPos);
			writer.WriteInt32((int)InputLine);
			writer.WriteBoolean(IsInHeader);
			if (IsInHeader)
			{
				writer.WriteInt32((int)Value.Length);
			}
			else
			{
				writer.WriteString(Value.Span);
			}
		}

		/// <summary>
		/// True if the token isn't an end token
		/// </summary>
		/// <param name="token">Token in question</param>
		public static implicit operator bool(UhtToken token)
		{
			return !token.IsEndType();
		}

		/// <summary>
		/// Return true if the token is an end token
		/// </summary>
		/// <returns>True if the token is an end token</returns>
		public readonly bool IsEndType()
		{
			return TokenType.IsEndType();
		}

		/// <summary>
		/// Test to see if the value matches the given character
		/// </summary>
		/// <param name="value">Value to test</param>
		/// <returns>True if the token value matches the given value</returns>
		public readonly bool IsValue(char value)
		{
			return Value.Span.Length == 1 && Value.Span[0] == value;
		}

		/// <summary>
		/// Test to see if the value matches the given string
		/// </summary>
		/// <param name="value">Value to test</param>
		/// <param name="ignoreCase">If true, ignore case</param>
		/// <returns>True if the value matches</returns>
		public readonly bool IsValue(string value, bool ignoreCase = false)
		{
			return Value.Span.Equals(value, ignoreCase ? StringComparison.OrdinalIgnoreCase : StringComparison.Ordinal);
		}

		/// <summary>
		/// Test to see if the value matches the given string
		/// </summary>
		/// <param name="value">Value to test</param>
		/// <param name="ignoreCase">If true, ignore case</param>
		/// <returns>True if the value matches</returns>
		public readonly bool IsValue(StringView value, bool ignoreCase = false)
		{
			return Value.Span.Equals(value.Span, ignoreCase ? StringComparison.OrdinalIgnoreCase : StringComparison.Ordinal);
		}

		/// <summary>
		/// Test to see if the value starts with the given string
		/// </summary>
		/// <param name="value">Value to test</param>
		/// <param name="ignoreCase">If true, ignore case</param>
		/// <returns>True is the value starts with the given string</returns>
		public readonly bool ValueStartsWith(string value, bool ignoreCase = false)
		{
			return Value.Span.StartsWith(value, ignoreCase ? StringComparison.OrdinalIgnoreCase : StringComparison.Ordinal);
		}

		/// <summary>
		/// Return true if the token is an identifier
		/// </summary>
		/// <returns>True if the token is an identifier</returns>
		public readonly bool IsIdentifier()
		{
			return TokenType == UhtTokenType.Identifier;
		}

		/// <summary>
		/// Return true if the identifier matches
		/// </summary>
		/// <param name="identifier">Identifier to test</param>
		/// <param name="ignoreCase">If true, ignore case</param>
		/// <returns>True if the identifier matches</returns>
		public readonly bool IsIdentifier(string identifier, bool ignoreCase = false)
		{
			return IsIdentifier() && IsValue(identifier, ignoreCase);
		}

		/// <summary>
		/// Return true if the identifier matches
		/// </summary>
		/// <param name="identifier">Identifier to test</param>
		/// <param name="ignoreCase">If true, ignore case</param>
		/// <returns>True if the identifier matches</returns>
		public readonly bool IsIdentifier(StringView identifier, bool ignoreCase = false)
		{
			return IsIdentifier() && IsValue(identifier, ignoreCase);
		}

		/// <summary>
		/// Return true if the token is a symbol
		/// </summary>
		/// <returns>True if the token is a symbol</returns>
		public readonly bool IsSymbol()
		{
			return TokenType == UhtTokenType.Symbol;
		}

		/// <summary>
		/// Return true if the symbol matches
		/// </summary>
		/// <param name="symbol">Symbol to test</param>
		/// <returns>True if the symbol matches</returns>
		public readonly bool IsSymbol(char symbol)
		{
			return IsSymbol() && IsValue(symbol);
		}

		/// <summary>
		/// Return true if the symbol matches
		/// </summary>
		/// <param name="symbol">Symbol to test</param>
		/// <returns>True if the symbol matches</returns>
		public readonly bool IsSymbol(string symbol)
		{
			return IsSymbol() && IsValue(symbol);
		}

		/// <summary>
		/// Return true if the symbol matches
		/// </summary>
		/// <param name="symbol">Symbol to test</param>
		/// <returns>True if the symbol matches</returns>
		public readonly bool IsSymbol(StringView symbol)
		{
			return IsSymbol() && IsValue(symbol);
		}

		/// <summary>
		/// Return true if the token is a constant integer
		/// </summary>
		/// <returns>True if constant integer</returns>
		public readonly bool IsConstInt()
		{
			return TokenType == UhtTokenType.DecimalConst || TokenType == UhtTokenType.HexConst;
		}

		/// <summary>
		/// Return true if the token is a constant floag
		/// </summary>
		/// <returns>True if constant float</returns>
		public readonly bool IsConstFloat()
		{
			return TokenType == UhtTokenType.FloatConst;
		}

		/// <summary>
		/// Get the integer value of the token.  Only supported for decimal, hexadecimal, and floating point values
		/// </summary>
		/// <param name="value">Resulting value</param>
		/// <returns>True if the value was set</returns>
		public readonly bool GetConstInt(out int value)
		{
			switch (TokenType)
			{
				case UhtTokenType.DecimalConst:
					value = (int)GetDecimalValue();
					return true;
				case UhtTokenType.HexConst:
					value = (int)GetHexValue();
					return true;
				case UhtTokenType.FloatConst:
					{
						float floatValue = GetFloatValue();
						value = (int)floatValue;
						return floatValue == value;
					}
				default:
					value = 0;
					return false;
			}
		}

		/// <summary>
		/// Get the integer value of the token.  Only supported for decimal, hexadecimal, and floating point values
		/// </summary>
		/// <param name="value">Resulting value</param>
		/// <returns>True if the value was set</returns>
		public readonly bool GetConstLong(out long value)
		{
			switch (TokenType)
			{
				case UhtTokenType.DecimalConst:
					value = GetDecimalValue();
					return true;
				case UhtTokenType.HexConst:
					value = GetHexValue();
					return true;
				case UhtTokenType.FloatConst:
					{
						float floatValue = GetFloatValue();
						value = (long)floatValue;
						return floatValue == value;
					}
				default:
					value = 0;
					return false;
			}
		}

		/// <summary>
		/// Get the float value of the token.  Only supported for decimal, hexadecimal, and floating point values
		/// </summary>
		/// <param name="value">Resulting value</param>
		/// <returns>True if the value was set</returns>
		public readonly bool GetConstFloat(out float value)
		{
			switch (TokenType)
			{
				case UhtTokenType.DecimalConst:
					value = (float)GetDecimalValue();
					return true;
				case UhtTokenType.HexConst:
					value = (float)GetHexValue();
					return true;
				case UhtTokenType.FloatConst:
					value = GetFloatValue();
					return true;
				default:
					value = 0;
					return false;
			}
		}

		/// <summary>
		/// Get the double value of the token.  Only supported for decimal, hexadecimal, and floating point values
		/// </summary>
		/// <param name="value">Resulting value</param>
		/// <returns>True if the value was set</returns>
		public readonly bool GetConstDouble(out double value)
		{
			switch (TokenType)
			{
				case UhtTokenType.DecimalConst:
					value = (double)GetDecimalValue();
					return true;
				case UhtTokenType.HexConst:
					value = (double)GetHexValue();
					return true;
				case UhtTokenType.FloatConst:
					value = GetDoubleValue();
					return true;
				default:
					value = 0;
					return false;
			}
		}

		/// <summary>
		/// Return true if the token is a constant string (or a char constant)
		/// </summary>
		/// <returns>True if the token is a string or character constant</returns>
		public readonly bool IsConstString()
		{
			return TokenType == UhtTokenType.StringConst || TokenType == UhtTokenType.CharConst;
		}

		// Return the token value for string constants

		/// <summary>
		/// Return an un-escaped string.  The surrounding quotes will be removed and escaped characters will be converted to the actual values.
		/// </summary>
		/// <param name="messageSite"></param>
		/// <returns>Resulting string</returns>
		/// <exception cref="UhtException">Thrown if the token type is not a string or character constant</exception>
		public readonly StringView GetUnescapedString(IUhtMessageSite messageSite)
		{
			switch (TokenType)
			{
				case UhtTokenType.StringConst:
					ReadOnlySpan<char> span = Value.Span[1..^1];
					int index = span.IndexOf('\\');
					if (index == -1)
					{
						return new StringView(Value, 1, span.Length);
					}
					else
					{
						StringBuilder builder = new();
						while (index >= 0)
						{
							builder.Append(span[..index]);
							if (span[index + 1] == 'n')
							{
								builder.Append('\n');
							}
							span = span[(index + 1)..];
							index = span.IndexOf('\\');
						}
						builder.Append(span);
						return new StringView(builder.ToString());
					}

				case UhtTokenType.CharConst:
					if (Value.Span[1] == '\\')
					{
						switch (Value.Span[2])
						{
							case 't':
								return new StringView("\t");
							case 'n':
								return new StringView("\n");
							case 'r':
								return new StringView("\r");
							default:
								return new StringView(Value, 2, 1);
						}
					}
					else
					{
						return new StringView(Value, 1, 1);
					}
			}

			throw new UhtException(messageSite, InputLine, "Call to GetUnescapedString on token that isn't a string or char constant");
		}

		/// <summary>
		/// Return a string representation of the token value.  This will convert numeric values and format them.
		/// </summary>
		/// <param name="respectQuotes">If true, embedded quotes will be respected</param>
		/// <returns>Resulting string</returns>
		public readonly StringView GetConstantValue(bool respectQuotes = false)
		{
			switch (TokenType)
			{
				case UhtTokenType.DecimalConst:
					return new StringView(GetDecimalValue().ToString(NumberFormatInfo.InvariantInfo));
				case UhtTokenType.HexConst:
					return new StringView(GetHexValue().ToString(NumberFormatInfo.InvariantInfo));
				case UhtTokenType.FloatConst:
					return new StringView(GetFloatValue().ToString("F6", NumberFormatInfo.InvariantInfo));
				case UhtTokenType.CharConst:
				case UhtTokenType.StringConst:
					return GetTokenString(respectQuotes);
				default:
					return "NotConstant";
			}
		}

		/// <summary>
		/// Return an un-escaped string.  The surrounding quotes will be removed and escaped characters will be converted to the actual values.
		/// </summary>
		/// <param name="respectQuotes">If true, respect embedded quotes</param>
		/// <returns>Resulting string</returns>
		public readonly StringView GetTokenString(bool respectQuotes = false)
		{
			StringViewBuilder builder = new();
			switch (TokenType)
			{
				case UhtTokenType.StringConst:
					StringView subValue = new(Value, 1, Value.Span.Length - 2);
					while (subValue.Span.Length > 0)
					{
						int slashIndex = subValue.Span.IndexOf('\\');
						if (slashIndex == -1)
						{
							builder.Append(subValue);
							break;
						}
						if (slashIndex > 0)
						{
							builder.Append(new StringView(subValue, 0, slashIndex));
						}
						if (slashIndex + 1 == subValue.Span.Length)
						{
							break;
						}

						if (slashIndex + 1 < subValue.Span.Length)
						{
							char c = subValue.Span[slashIndex + 1];
							if (c == 'n')
							{
								c = '\n';
							}
							else if (respectQuotes && c == '"')
							{
								builder.Append('\\');
							}
							builder.Append(c);
							subValue = new StringView(subValue, slashIndex + 2);
						}
					}
					break;

				case UhtTokenType.CharConst:
					char charConst = Value.Span[1];
					if (charConst == '\\')
					{
						charConst = Value.Span[2];
						switch (charConst)
						{
							case 't':
								charConst = '\t';
								break;
							case 'n':
								charConst = '\n';
								break;
							case 'r':
								charConst = '\r';
								break;
						}
					}
					builder.Append(charConst);
					break;

				default:
					throw new UhtIceException("Call to GetTokenString on a token that isn't a string or char constant");
			}
			return builder.ToStringView();
		}

		/// <summary>
		/// Join the given tokens into a string
		/// </summary>
		/// <param name="tokens">Tokens to join</param>
		/// <returns>Joined strings</returns>
		public static string Join(IEnumerable<UhtToken> tokens)
		{
			StringBuilder builder = new();
			foreach (UhtToken token in tokens)
			{
				builder.Append(token.Value.ToString());
			}
			return builder.ToString();
		}

		/// <summary>
		/// Join the given tokens into a string
		/// </summary>
		/// <param name="separator">Separator between tokens</param>
		/// <param name="tokens">Tokens to join</param>
		/// <returns>Joined strings</returns>
		public static string Join(char separator, IEnumerable<UhtToken> tokens)
		{
			StringBuilder builder = new();
			bool includeSeparator = false;
			foreach (UhtToken token in tokens)
			{
				if (!includeSeparator)
				{
					includeSeparator = true;
				}
				else
				{
					builder.Append(separator);
				}
				builder.Append(token.Value.ToString());
			}
			return builder.ToString();
		}

		/// <summary>
		/// Join the given tokens into a string
		/// </summary>
		/// <param name="separator">Separator between tokens</param>
		/// <param name="tokens">Tokens to join</param>
		/// <returns>Joined strings</returns>
		public static string Join(string separator, IEnumerable<UhtToken> tokens)
		{
			StringBuilder builder = new();
			bool includeSeparator = false;
			foreach (UhtToken token in tokens)
			{
				if (!includeSeparator)
				{
					includeSeparator = true;
				}
				else
				{
					builder.Append(separator);
				}
				builder.Append(token.Value.ToString());
			}
			return builder.ToString();
		}

		/// <summary>
		/// Convert the token to a string.  This will be the value.
		/// </summary>
		/// <returns>Value of the token</returns>
		public override readonly string ToString()
		{
			if (IsEndType())
			{
				return "<none>";
			}
			else
			{
				return Value.Span.ToString();
			}
		}

		private const NumberStyles DefaultNumberStyles = NumberStyles.AllowLeadingWhite | NumberStyles.AllowTrailingWhite |
			NumberStyles.AllowLeadingSign | NumberStyles.AllowDecimalPoint | NumberStyles.AllowThousands | NumberStyles.AllowExponent;

		private readonly float GetFloatValue()
		{
			if (Value.Span.Length > 0)
			{
				if (UhtFCString.IsFloatMarker(Value.Span[^1]))
				{
					return Single.Parse(Value.Span[0..^1], DefaultNumberStyles, CultureInfo.InvariantCulture);
				}
			}
			return Single.Parse(Value.Span, DefaultNumberStyles, CultureInfo.InvariantCulture);
		}

		private readonly double GetDoubleValue()
		{
			if (Value.Span.Length > 0)
			{
				if (UhtFCString.IsFloatMarker(Value.Span[^1]))
				{
					return Double.Parse(Value.Span[0..^1], DefaultNumberStyles, CultureInfo.InvariantCulture);
				}
			}
			return Double.Parse(Value.Span, DefaultNumberStyles, CultureInfo.InvariantCulture);
		}

		readonly long GetDecimalValue()
		{
			ReadOnlySpan<char> span = Value.Span;
			bool isUnsigned = false;
			while (span.Length > 0)
			{
				char c = span[^1];
				if (UhtFCString.IsLongMarker(c))
				{
					span = span[0..^1];
				}
				else if (UhtFCString.IsUnsignedMarker(c))
				{
					isUnsigned = true;
					span = span[0..^1];
				}
				else
				{
					break;
				}
			}
			return isUnsigned ? (long)Convert.ToUInt64(span.ToString(), 10) : Convert.ToInt64(span.ToString(), 10);
		}

		readonly long GetHexValue()
		{
			return Convert.ToInt64(Value.ToString(), 16);
		}
	}
}
