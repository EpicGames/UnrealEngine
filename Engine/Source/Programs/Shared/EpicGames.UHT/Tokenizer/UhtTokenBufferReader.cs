// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Token reader for source buffers
	/// </summary>
	public sealed class UhtTokenBufferReader : IUhtTokenReader, IUhtMessageLineNumber
	{
		private static readonly StringView[] s_emptyComments = [];

		private readonly IUhtMessageSite _messageSite;
		private List<StringView>? _comments = null;
		private readonly List<UhtToken> _recordedTokens = [];
		private IUhtTokenPreprocessor? _tokenPreprocessor = null;
		private UhtToken _currentToken = new(); // PeekToken must have been invoked first
		private readonly ReadOnlyMemory<char> _data;
		private int _prevPos = 0;
		private int _prevLine = 1;
		private bool _hasToken = false;
		private int _preCurrentTokenInputPos = 0;
		private int _preCurrentTokenInputLine = 1;
		private int _inputPos = 0;
		private int _inputLine = 1;
		private int _commentsDisableCount = 0;
		private int _committedComments = 0;
		private List<StringView>? _savedComments = null;
		private int _savedInputPos = 0;
		private int _savedInputLine = 0;
		private bool _hasSavedState = false;
		private bool _recordTokens = false;
		private int _preprocessorPendingCommentsCount = 0;

		/// <summary>
		/// Construct a new token reader
		/// </summary>
		/// <param name="messageSite">Message site for messages</param>
		/// <param name="input">Input source</param>
		public UhtTokenBufferReader(IUhtMessageSite messageSite, ReadOnlyMemory<char> input)
		{
			_messageSite = messageSite;
			_data = input;
		}

		#region IUHTMessageSite Implementation
		IUhtMessageSession IUhtMessageSite.MessageSession => _messageSite.MessageSession;
		IUhtMessageSource? IUhtMessageSite.MessageSource => _messageSite.MessageSource;
		IUhtMessageLineNumber? IUhtMessageSite.MessageLineNumber => this;
		#endregion

		#region IUHTMessageLinenumber Implementation
		int IUhtMessageLineNumber.MessageLineNumber => InputLine;
		#endregion

		#region ITokenReader Implementation
		/// <inheritdoc/>
		public bool IsEOF
		{
			get
			{
				if (_hasToken)
				{
					return _currentToken.TokenType.IsEndType();
				}
				else
				{
					return InputPos == _data.Length;
				}
			}
		}

		/// <inheritdoc/>
		public int InputPos => _hasToken ? _preCurrentTokenInputPos : _inputPos;

		/// <inheritdoc/>
		public int InputLine
		{
			get => _hasToken ? _preCurrentTokenInputLine : _inputLine;
			set
			{
				ClearToken();
				_inputLine = value;
			}
		}

		/// <inheritdoc/>
		public ReadOnlySpan<StringView> Comments
		{
			get
			{
				if (_comments != null && _committedComments != 0)
				{
					return new ReadOnlySpan<StringView>([.. _comments], 0, _committedComments);
				}
				else
				{
					return new ReadOnlySpan<StringView>(s_emptyComments);
				}
			}
		}

		/// <inheritdoc/>
		public IUhtTokenPreprocessor? TokenPreprocessor { get => _tokenPreprocessor; set => _tokenPreprocessor = value; }

		/// <inheritdoc/>
		public ref UhtToken PeekToken()
		{
			if (!_hasToken)
			{
				_currentToken = GetTokenInternal(true);
				_hasToken = true;
			}
			return ref _currentToken;
		}

		/// <inheritdoc/>
		public void SkipWhitespaceAndComments()
		{
			if (!_hasToken)
			{
				bool gotInlineComment = false;
				SkipWhitespaceAndCommentsInternal(ref gotInlineComment, true);
			}
		}

		/// <inheritdoc/>
		public void ConsumeToken()
		{
			if (_recordTokens && !_currentToken.IsEndType())
			{
				_recordedTokens.Add(_currentToken);
			}
			_hasToken = false;

			// When comments are disabled, we are still collecting comments, but aren't committing them
			if (_commentsDisableCount == 0)
			{
				if (_comments != null)
				{
					_committedComments = _comments.Count;
				}
			}
			else
			{
				ClearPendingComments();
			}
		}

		/// <inheritdoc/>
		public UhtToken GetToken()
		{
			UhtToken token = PeekToken();
			ConsumeToken();
			return token;
		}

		/// <inheritdoc/>
		public int GetReplayTokenIndex()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public StringView GetRawString(char terminator, UhtRawStringOptions options)
		{
			ReadOnlySpan<char> span = _data.Span;

			ClearToken();
			SkipWhitespaceAndComments();

			int startPos = InputPos;
			bool inQuotes = false;
			while (true)
			{
				char c = InternalGetChar(span);

				// Check for end of file
				if (c == 0)
				{
					break;
				}

				// Check for end of line
				if (c == '\r' || c == '\n')
				{
					--_inputPos;
					break;
				}

				// Check for terminator as long as we aren't in quotes
				if (c == terminator && !inQuotes)
				{
					if (options.HasAnyFlags(UhtRawStringOptions.DontConsumeTerminator))
					{
						--_inputPos;
					}
					break;
				}

				// Check for comment
				if (!inQuotes && c == '/')
				{
					char p = InternalPeekChar(span);
					if (p == '*' || p == '/')
					{
						--_inputPos;
						break;
					}
				}

				// Check for quotes
				if (c == '"' && options.HasAnyFlags(UhtRawStringOptions.RespectQuotes))
				{
					inQuotes = !inQuotes;
				}
			}

			// If EOF, then error
			if (inQuotes)
			{
				throw new UhtException(this, "Unterminated quoted string");
			}

			// Remove trailing whitespace
			int endPos = InputPos;
			for (; endPos > startPos; --endPos)
			{
				char c = span[endPos - 1];
				if (c != ' ' && c != '\t')
				{
					break;
				}
			}

			// Check for too long
			if (endPos - startPos >= UhtToken.MaxStringLength)
			{
				throw new UhtException(this, $"String exceeds maximum of {UhtToken.MaxStringLength} characters");
			}
			return new StringView(_data, startPos, endPos - startPos);
		}

		/// <inheritdoc/>
		public UhtToken GetLine()
		{
			ReadOnlySpan<char> span = _data.Span;

			ClearToken();
			_prevPos = _inputPos;
			_prevLine = _inputLine;

			if (_prevPos == span.Length)
			{
				return new UhtToken(UhtTokenType.EndOfFile, _prevPos, _prevLine, _prevPos, _prevLine, new StringView(_data.Slice(_prevPos, 0)));
			}

			int lastPos = _inputPos;
			while (true)
			{
				char c = InternalGetChar(span);
				if (c == 0)
				{
					break;
				}
				else if (c == '\r')
				{
				}
				else if (c == '\n')
				{
					++_inputLine;
					break;
				}
				else
				{
					lastPos = _inputPos;
				}
			}

			return new UhtToken(UhtTokenType.Line, _prevPos, _prevLine, _prevPos, _prevLine, new StringView(_data[_prevPos..lastPos]));
		}

		/// <inheritdoc/>
		public StringView GetStringView(int startPos, int count)
		{
			return new StringView(_data, startPos, count);
		}

		/// <inheritdoc/>
		public void ClearComments()
		{
			if (_comments != null)
			{

				// Clearing comments does not remove any uncommitted comments
				_comments.RemoveRange(0, _committedComments);
				_committedComments = 0;
			}
		}

		/// <inheritdoc/>
		public void DisableComments()
		{
			++_commentsDisableCount;
		}

		/// <inheritdoc/>
		public void EnableComments()
		{
			--_commentsDisableCount;
		}

		/// <inheritdoc/>
		public void CommitPendingComments()
		{
			if (_comments != null)
			{
				_committedComments = _comments.Count;
			}
		}

		/// <inheritdoc/>
		public bool IsFirstTokenInLine(ref UhtToken token)
		{
			return IsFirstTokenInLine(_data.Span, token.InputStartPos);
		}

		/// <inheritdoc/>
		public void SaveState()
		{
			if (_hasSavedState)
			{
				throw new UhtIceException("Can not save more than one state");
			}
			_hasSavedState = true;
			_savedInputLine = InputLine;
			_savedInputPos = InputPos;
			if (_comments != null)
			{
				_savedComments ??= [];
				_savedComments.Clear();
				_savedComments.AddRange(_comments);
			}
			_tokenPreprocessor?.SaveState();
		}

		/// <inheritdoc/>
		public void RestoreState()
		{
			if (!_hasSavedState)
			{
				throw new UhtIceException("Can not restore state when none have been saved");
			}
			_hasSavedState = false;
			ClearToken();
			_inputPos = _savedInputPos;
			_inputLine = _savedInputLine;
			if (_savedComments != null && _comments != null)
			{
				_comments.Clear();
				_comments.AddRange(_savedComments);
			}
			_tokenPreprocessor?.RestoreState();
		}

		/// <inheritdoc/>
		public void AbandonState()
		{
			if (!_hasSavedState)
			{
				throw new UhtIceException("Can not abandon state when none have been saved");
			}
			_hasSavedState = false;
			ClearToken();
		}

		/// <inheritdoc/>
		public void EnableRecording()
		{
			if (_recordTokens)
			{
				throw new UhtIceException("Can not nest token recording");
			}
			_recordTokens = true;
		}

		/// <inheritdoc/>
		public void RecordToken(ref UhtToken token)
		{
			if (!_recordTokens)
			{
				throw new UhtIceException("Attempt to disable token recording when it isn't enabled");
			}
			_recordedTokens.Add(token);
		}

		/// <inheritdoc/>
		public void DisableRecording()
		{
			if (!_recordTokens)
			{
				throw new UhtIceException("Attempt to disable token recording when it isn't enabled");
			}
			_recordedTokens.Clear();
			_recordTokens = false;
		}

		/// <inheritdoc/>
		public List<UhtToken> RecordedTokens
		{
			get
			{
				if (!_recordTokens)
				{
					throw new UhtIceException("Attempt to get recorded tokens when it isn't enabled");
				}
				return _recordedTokens;
			}
		}
		#endregion

		#region Internals
		private void ClearAllComments()
		{
			if (_comments != null)
			{
				_comments.Clear();
				_committedComments = 0;
			}
		}

		private void ClearPendingComments()
		{
			if (_comments != null)
			{
				int startingComment = _committedComments + _preprocessorPendingCommentsCount;
				_comments.RemoveRange(startingComment, _comments.Count - startingComment);
			}
		}

		private static bool IsFirstTokenInLine(ReadOnlySpan<char> span, int startPos)
		{
			for (int pos = startPos; --pos > 0;)
			{
				switch (span[pos])
				{
					case '\r':
					case '\n':
						return true;
					case ' ':
					case '\t':
						break;
					default:
						return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Get the next token
		/// </summary>
		/// <returns>Return the next token from the stream.  If the end of stream is reached, a token type of None will be returned.</returns>
		private UhtToken GetTokenInternal(bool enablePreprocessor)
		{
			ReadOnlySpan<char> span = _data.Span;
			bool gotInlineComment = false;
Restart:
			_preCurrentTokenInputLine = _inputLine;
			_preCurrentTokenInputPos = _inputPos;
			SkipWhitespaceAndCommentsInternal(ref gotInlineComment, enablePreprocessor);
			_prevPos = _inputPos;
			_prevLine = _inputLine;

			UhtTokenType tokenType = UhtTokenType.EndOfFile;
			int startPos = _inputPos;
			int startLine = _inputLine;

			char c = InternalGetChar(span);
			if (c == 0)
			{
			}

			else if (UhtFCString.IsAlpha(c) || c == '_')
			{

				for (; _inputPos < span.Length; ++_inputPos)
				{
					c = span[_inputPos];
					if (!(UhtFCString.IsAlpha(c) || UhtFCString.IsDigit(c) || c == '_'))
					{
						break;
					}
				}

				if (_inputPos - startPos >= UhtToken.MaxNameLength)
				{
					throw new UhtException(this, $"Identifier length exceeds maximum of {UhtToken.MaxNameLength}");
				}

				tokenType = UhtTokenType.Identifier;
			}

			// Check for any numerics 
			else if (IsNumeric(span, c))
			{
				// Integer or floating point constant.

				bool isFloat = c == '.';
				bool isHex = false;

				// Ignore the starting sign for this part of the parsing
				if (UhtFCString.IsSign(c))
				{
					c = InternalGetChar(span); // We know this won't fail since the code above checked the Peek
				}

				// Check for a hex valid
				if (c == '0')
				{
					isHex = UhtFCString.IsHexMarker(InternalPeekChar(span));
					if (isHex)
					{
						InternalGetChar(span);
					}
				}

				// If we have a hex constant
				if (isHex)
				{
					for (; _inputPos < span.Length && UhtFCString.IsHexDigit(span[_inputPos]); ++_inputPos)
					{
					}
				}

				// We have decimal/octal value or possibly a floating point value
				else
				{

					// Skip all digits
					for (; _inputPos < span.Length && UhtFCString.IsDigit(span[_inputPos]); ++_inputPos)
					{
					}

					// If we have a '.'
					if (_inputPos < span.Length && span[_inputPos] == '.')
					{
						isFloat = true;
						++_inputPos;

						// Skip all digits
						for (; _inputPos < span.Length && UhtFCString.IsDigit(span[_inputPos]); ++_inputPos)
						{
						}
					}

					// If we have a 'e'
					if (_inputPos < span.Length && UhtFCString.IsExponentMarker(span[_inputPos]))
					{
						isFloat = true;
						++_inputPos;

						// Skip any signs
						if (_inputPos < span.Length && UhtFCString.IsSign(span[_inputPos]))
						{
							++_inputPos;
						}

						// Skip all digits
						for (; _inputPos < span.Length && UhtFCString.IsDigit(span[_inputPos]); ++_inputPos)
						{
						}
					}

					// If we have a 'f'
					if (_inputPos < span.Length && UhtFCString.IsFloatMarker(span[_inputPos]))
					{
						isFloat = true;
						++_inputPos;
					}

					// Check for u/l markers
					while (_inputPos < span.Length &&
						(UhtFCString.IsUnsignedMarker(span[_inputPos]) || UhtFCString.IsLongMarker(span[_inputPos])))
					{
						++_inputPos;
					}
				}

				if (_inputPos - startPos >= UhtToken.MaxNameLength)
				{
					throw new UhtException(this, $"Number length exceeds maximum of {UhtToken.MaxNameLength}");
				}

				tokenType = isFloat ? UhtTokenType.FloatConst : (isHex ? UhtTokenType.HexConst : UhtTokenType.DecimalConst);
			}

			// Escaped character constant
			else if (c == '\'')
			{

				// We try to skip the character constant value. But if it is backslash, we have to skip another character
				if (InternalGetChar(span) == '\\')
				{
					InternalGetChar(span);
				}

				char nextChar = InternalGetChar(span);
				if (nextChar != '\'')
				{
					throw new UhtException(this, "Unterminated character constant");
				}

				tokenType = UhtTokenType.CharConst;
			}

			// String constant
			else if (c == '"')
			{
				for (; ; )
				{
					char nextChar = InternalGetChar(span);
					if (nextChar == '\r' || nextChar == '\n' || nextChar == 0)
					{
						throw new UhtException(this, "Unterminated character constant");
					}

					else if (nextChar == '\\')
					{
						nextChar = InternalGetChar(span);
						if (nextChar == '\r' || nextChar == '\n' || nextChar == 0)
						{
							throw new UhtException(this, "Unterminated character constant");
						}
					}

					else if (nextChar == '"')
					{
						break;
					}
				}

				if (_inputPos - startPos >= UhtToken.MaxStringLength)
				{
					throw new UhtException(this, $"String constant exceeds maximum of {UhtToken.MaxStringLength} characters");
				}

				tokenType = UhtTokenType.StringConst;
			}

			// Assume everything else is a symbol.
			// Don't handle >> or >>>
			else
			{
				{
					if (_inputPos < span.Length)
					{
						char d = span[_inputPos];
						if ((c == '<' && d == '<')
							|| (c == '!' && d == '=')
							|| (c == '<' && d == '=')
							|| (c == '>' && d == '=')
							|| (c == '+' && d == '+')
							|| (c == '-' && d == '-')
							|| (c == '+' && d == '=')
							|| (c == '-' && d == '=')
							|| (c == '*' && d == '=')
							|| (c == '/' && d == '=')
							|| (c == '&' && d == '&')
							|| (c == '|' && d == '|')
							|| (c == '^' && d == '^')
							|| (c == '=' && d == '=')
							|| (c == '*' && d == '*')
							|| (c == '~' && d == '=')
							|| (c == ':' && d == ':')
							|| (c == '[' && d == '['))
						{
							++_inputPos;
						}
					}

					// Comment processing while processing the preprocessor statements is complicated.  When a preprocessor statement
					// parsers, the tokenizer is doing some form of a PeekToken.  And comments read ahead of the '#' will still be 
					// pending.
					//
					// 1) If the block is being skipped, we want to preserve any comments ahead of the token and eliminate all other comments found in the #if block
					// 2) If the block isn't being skipped or in the case of things such as #pragma or #include, they were considered statements in the UHT
					//	  and would result in the comments being dropped.  In the new UHT, we just eliminate all pending tokens.
					if (c == '#' && enablePreprocessor && _tokenPreprocessor != null && IsFirstTokenInLine(span, startPos))
					{
						_preprocessorPendingCommentsCount = _comments != null ? _comments.Count - _committedComments : 0;
						UhtToken temp = new(tokenType, startPos, startLine, startPos, _inputLine, new StringView(_data[startPos.._inputPos]));
						bool include = _tokenPreprocessor.ParsePreprocessorDirective(ref temp, true, out bool clearComments, out bool illegalContentsCheck);
						if (!include)
						{
							++_commentsDisableCount;
							int checkStateMachine = 0;
							while (true)
							{
								if (IsEOF)
								{
									break;
								}
								UhtToken localToken = GetTokenInternal(false);
								ReadOnlySpan<char> localValueSpan = localToken.Value.Span;
								if (localToken.IsSymbol('#') && localValueSpan.Length == 1 && localValueSpan[0] == '#' && IsFirstTokenInLine(span, localToken.InputStartPos))
								{
									if (_tokenPreprocessor.ParsePreprocessorDirective(ref temp, false, out bool _, out bool scratchIllegalContentsCheck))
									{
										break;
									}
									illegalContentsCheck = scratchIllegalContentsCheck;
								}
								else if (illegalContentsCheck)
								{
									switch (checkStateMachine)
									{
										case 0:
ResetStateMachineCheck:
											if (localValueSpan.CompareTo("UPROPERTY", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("UCLASS", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("USTRUCT", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("UENUM", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("UINTERFACE", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("UDELEGATE", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("UFUNCTION", StringComparison.Ordinal) == 0)
											{
												this.LogError($"'{localValueSpan}' must not be inside preprocessor blocks, except for WITH_EDITORONLY_DATA");
											}
											else if (localValueSpan.CompareTo("void", StringComparison.Ordinal) == 0)
											{
												checkStateMachine = 1;
											}
											break;

										case 1:
											if (localValueSpan.CompareTo("Serialize", StringComparison.Ordinal) == 0)
											{
												checkStateMachine = 2;
											}
											else
											{
												checkStateMachine = 0;
												goto ResetStateMachineCheck;
											}
											break;

										case 2:
											if (localValueSpan.CompareTo("(", StringComparison.Ordinal) == 0)
											{
												checkStateMachine = 3;
											}
											else
											{
												checkStateMachine = 0;
												goto ResetStateMachineCheck;
											}
											break;

										case 3:
											if (localValueSpan.CompareTo("FArchive", StringComparison.Ordinal) == 0 ||
												localValueSpan.CompareTo("FStructuredArchiveRecord", StringComparison.Ordinal) == 0)
											{
												this.LogError($"Engine serialization functions must not be inside preprocessor blocks, except for WITH_EDITORONLY_DATA");
												checkStateMachine = 0;
											}
											else if (localValueSpan.CompareTo("FStructuredArchive", StringComparison.Ordinal) == 0)
											{
												checkStateMachine = 4;
											}
											else
											{
												checkStateMachine = 0;
												goto ResetStateMachineCheck;
											}
											break;

										case 4:
											if (localValueSpan.CompareTo("::", StringComparison.Ordinal) == 0)
											{
												checkStateMachine = 5;
											}
											else
											{
												checkStateMachine = 0;
												goto ResetStateMachineCheck;
											}
											break;

										case 5:
											if (localValueSpan.CompareTo("FRecord", StringComparison.Ordinal) == 0)
											{
												this.LogError($"Engine serialization functions must not be inside preprocessor blocks, except for WITH_EDITORONLY_DATA");
												checkStateMachine = 0;
											}
											else
											{
												checkStateMachine = 0;
												goto ResetStateMachineCheck;
											}
											break;
									}
								}
							}
							--_commentsDisableCount;

							// Clear any extra pending comments at this time
							ClearPendingComments();
						}
						_preprocessorPendingCommentsCount = 0;

						// Depending on the type of directive and/or the #if expression, we clear comments.
						if (clearComments)
						{
							ClearPendingComments();
							gotInlineComment = false;
						}
						goto Restart;
					}
				}
				tokenType = UhtTokenType.Symbol;
			}

			return new UhtToken(tokenType, startPos, startLine, startPos, _inputLine, new StringView(_data[startPos.._inputPos]));
		}

		/// <summary>
		/// Skip all leading whitespace and collect any comments
		/// </summary>
		private void SkipWhitespaceAndCommentsInternal(ref bool persistGotInlineComment, bool enablePreprocessor)
		{
			ReadOnlySpan<char> span = _data.Span;

			bool gotNewlineBetweenComments = false;
			bool gotInlineComment = persistGotInlineComment;
			for (; ; )
			{
				uint c = InternalGetChar(span);
				if (c == 0)
				{
					break;
				}
				else if (c == '\n')
				{
					gotNewlineBetweenComments |= gotInlineComment;
					++_inputLine;
				}
				else if (c == '\r' || c == '\t' || c == ' ')
				{
				}
				else if (c == '/')
				{
					uint nextChar = InternalPeekChar(span);
					if (nextChar == '*')
					{
						if (enablePreprocessor)
						{
							ClearAllComments();
						}
						int commentStart = _inputPos - 1;
						++_inputPos;
						for (; ; )
						{
							char commentChar = InternalGetChar(span);
							if (commentChar == 0)
							{
								if (enablePreprocessor)
								{
									ClearAllComments();
								}
								throw new UhtException(this, "End of header encountered inside comment");
							}
							else if (commentChar == '\n')
							{
								++_inputLine;
							}
							else if (commentChar == '*' && InternalPeekChar(span) == '/')
							{
								++_inputPos;
								break;
							}
						}
						if (enablePreprocessor)
						{
							AddComment(new StringView(_data[commentStart.._inputPos]));
						}
					}
					else if (nextChar == '/')
					{
						if (gotNewlineBetweenComments)
						{
							gotNewlineBetweenComments = false;
							if (enablePreprocessor)
							{
								ClearAllComments();
							}
						}
						gotInlineComment = true;
						int commentStart = _inputPos - 1;
						++_inputPos;

						// Scan to the end of the line
						for (; ; )
						{
							char commentChar = InternalGetChar(span);
							if (commentChar == 0)
							{
								//--Pos;
								break;
							}
							if (commentChar == '\r')
							{
							}
							else if (commentChar == '\n')
							{
								++_inputLine;
								break;
							}
						}
						if (enablePreprocessor)
						{
							AddComment(new StringView(_data[commentStart.._inputPos]));
						}
					}
					else
					{
						--_inputPos;
						break;
					}
				}
				else
				{
					--_inputPos;
					break;
				}
			}
			persistGotInlineComment = gotInlineComment;
			return;
		}

		private bool IsNumeric(ReadOnlySpan<char> span, char c)
		{
			// Check for [0..9]
			if (UhtFCString.IsDigit(c))
			{
				return true;
			}

			// Check for [+-]...
			if (UhtFCString.IsSign(c))
			{
				if (_inputPos == span.Length)
				{
					return false;
				}

				// Check for [+-][0..9]...
				if (UhtFCString.IsDigit(span[_inputPos]))
				{
					return true;
				}

				// Check for [+-][.][0..9]...
				if (span[_inputPos] != '.')
				{
					return false;
				}

				if (_inputPos + 1 == span.Length)
				{
					return false;
				}

				if (UhtFCString.IsDigit(span[_inputPos + 1]))
				{
					return true;
				}
				return false;
			}

			if (c == '.')
			{
				if (_inputPos == span.Length)
				{
					return false;
				}

				// Check for [.][0..9]...
				if (UhtFCString.IsDigit(span[_inputPos]))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Fetch the next character in the input stream or zero if we have reached the end.
		/// The current offset in the buffer is not advanced.  The method does not support UTF-8
		/// </summary>
		/// <param name="span">The span containing the data</param>
		/// <returns>Next character in the stream or zero</returns>
		private char InternalPeekChar(ReadOnlySpan<char> span)
		{
			return _inputPos < span.Length ? span[_inputPos] : '\0';
		}

		/// <summary>
		/// Fetch the next character in the input stream or zero if we have reached the end.
		/// The current offset in the buffer is advanced.  The method does not support UTF-8.
		/// </summary>
		/// <param name="span">The span containing the data</param>
		/// <returns>Next character in the stream or zero</returns>
		private char InternalGetChar(ReadOnlySpan<char> span)
		{
			return _inputPos < span.Length ? span[_inputPos++] : '\0';
		}

		/// <summary>
		/// If we have a current token, then reset the pending comments and input position back to before the token.
		/// </summary>
		private void ClearToken()
		{
			if (_hasToken)
			{
				_hasToken = false;
				if (_comments != null && _comments.Count > _committedComments)
				{
					_comments.RemoveRange(_committedComments, _comments.Count - _committedComments);
				}
				_inputPos = _currentToken.UngetPos;
				_inputLine = _currentToken.UngetLine;
			}
		}

		private void AddComment(StringView comment)
		{
			_comments ??= new List<StringView>(4);
			_comments.Add(comment);
		}
		#endregion
	}
}
