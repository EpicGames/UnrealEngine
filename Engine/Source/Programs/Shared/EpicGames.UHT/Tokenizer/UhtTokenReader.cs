// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Options for GetRawString method
	/// </summary>
	[Flags]
	public enum UhtRawStringOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None,

		/// <summary>
		/// Don't consider the terminator while in a quoted string
		/// </summary>
		RespectQuotes = 1 << 0,

		/// <summary>
		/// Don't consume the terminator.  It will be parsed later.
		/// </summary>
		DontConsumeTerminator = 1 << 1,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtRawStringOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtRawStringOptions inFlags, UhtRawStringOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtRawStringOptions inFlags, UhtRawStringOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtRawStringOptions inFlags, UhtRawStringOptions testFlags, UhtRawStringOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Interface invoked when the parser reaches a '#' preprocessor block
	/// </summary>
	public interface IUhtTokenPreprocessor
	{

		/// <summary>
		/// Parse a preprocessor directive
		/// </summary>
		/// <param name="token">Token starting the directive.  Will be only the '#'</param>
		/// <param name="isBeingIncluded">If true, the directive the source is being included.  Otherwise it is being skipped as part of an #if block</param>
		/// <param name="clearComments">If true, comments should be cleared</param>
		/// <param name="illegalContentsCheck">If true, excluded contents should be checked for unparsed UE macros (i.e. UCLASS) </param>
		/// <returns>True if the source should continue to be included</returns>
		public bool ParsePreprocessorDirective(ref UhtToken token, bool isBeingIncluded, out bool clearComments, out bool illegalContentsCheck);

		/// <summary>
		/// Save the current preprocessor state
		/// </summary>
		public void SaveState();

		/// <summary>
		/// Restore the current preprocessor state
		/// </summary>
		public void RestoreState();
	}

	/// <summary>
	/// Common token reader interfaces for all token reader.  When creating extension methods, use the interface.
	/// </summary>
	public interface IUhtTokenReader : IUhtMessageSite
	{
		/// <summary>
		/// True if the reader is at the end of the stream
		/// </summary>
		public bool IsEOF { get; }

		/// <summary>
		/// Current input position in the stream by characters
		/// </summary>
		public int InputPos { get; }

		/// <summary>
		/// Current input line in the stream
		/// </summary>
		public int InputLine { get; set; }

		/// <summary>
		/// Preprocessor attached to the token reader
		/// </summary>
		public IUhtTokenPreprocessor? TokenPreprocessor { get; set; }

		/// <summary>
		/// If the reader doesn't have a current token, then read the next token and return a reference to it.
		/// Otherwise return a reference to the current token.
		/// </summary>
		/// <returns>The current token.  Will be invalidated by other calls to ITokenReader</returns>
		public ref UhtToken PeekToken();

		/// <summary>
		/// Mark the current token as being consumed.  Any call to PeekToken or GetToken will read another token.
		/// </summary>
		public void ConsumeToken();

		/// <summary>
		/// Get the next token in the data.  If there is a current token, then that token is returned and marked as consumed.
		/// </summary>
		/// <returns></returns>
		public UhtToken GetToken();

		/// <summary>
		/// Method only implemented by the replay reader that returns the current token index
		/// </summary>
		/// <returns>Current token index</returns>
		public int GetReplayTokenIndex();

		/// <summary>
		/// Tests to see if the given token is the first token of a line
		/// </summary>
		/// <param name="token">The token to test</param>
		/// <returns>True if the token is the first token on the line</returns>
		public bool IsFirstTokenInLine(ref UhtToken token);

		/// <summary>
		/// Skip any whitespace and comments at the current buffer position
		/// </summary>
		public void SkipWhitespaceAndComments();

		/// <summary>
		/// Read the entire next line in the buffer
		/// </summary>
		/// <returns></returns>
		public UhtToken GetLine();

		/// <summary>
		/// Get a view of the buffer being read
		/// </summary>
		/// <param name="startPos">Starting character offset in the buffer.</param>
		/// <param name="count">Length of the span</param>
		/// <returns>The string view into the buffer</returns>
		public StringView GetStringView(int startPos, int count);

		/// <summary>
		/// Return a string terminated by the given character.
		/// </summary>
		/// <param name="terminator">The character to stop at.</param>
		/// <param name="options">Options</param>
		/// <returns>The parsed string</returns>
		public StringView GetRawString(char terminator, UhtRawStringOptions options);

		/// <summary>
		/// The current collection of parsed comments.  This does not include any comments parsed as part of a 
		/// call to PeekToken unless ConsumeToken has been invoked after a call to PeekToken.
		/// </summary>
		public ReadOnlySpan<StringView> Comments { get; }

		/// <summary>
		/// Clear the current collection of comments.  Any comments parsed by PeekToken prior to calling ConsomeToken will
		/// not be cleared.
		/// </summary>
		public void ClearComments();

		/// <summary>
		/// Disable the processing of comments.  This is often done when skipping a bulk of the buffer.
		/// </summary>
		/// <returns></returns>
		public void DisableComments();

		/// <summary>
		/// Enable comment collection.
		/// </summary>
		public void EnableComments();

		/// <summary>
		/// If there are any pending comments (due to a PeekToken), commit then so they will be return as current comments
		/// </summary>
		//COMPATIBILITY-TODO - Remove once the struct adding of tooltips is fixed
		public void CommitPendingComments();

		/// <summary>
		/// Save the current parsing state.  There is a limited number of states that can be saved.
		/// Invoke either RestoreState or AbandonState after calling SaveState.
		/// </summary>
		public void SaveState();

		/// <summary>
		/// Restore a previously saved state.
		/// </summary>
		public void RestoreState();

		/// <summary>
		/// Abandon a previously saved state
		/// </summary>
		public void AbandonState();

		/// <summary>
		/// Enable the recording of tokens
		/// </summary>
		public void EnableRecording();

		/// <summary>
		/// Disable the recording of tokens.  Any currently recorded tokens will be removed
		/// </summary>
		public void DisableRecording();

		/// <summary>
		/// Record the given token to the list of recorded tokens
		/// </summary>
		/// <param name="token">Token to record</param>
		public void RecordToken(ref UhtToken token);

		/// <summary>
		/// Get the current collection of recorded tokens
		/// </summary>
		public List<UhtToken> RecordedTokens { get; }
	}

	/// <summary>
	/// Represents a list of tokens. Follow the Next chain for each element in the list.
	/// </summary>
	public class UhtTokenList
	{

		/// <summary>
		/// The token
		/// </summary>
		public UhtToken Token { get; set; }

		/// <summary>
		/// The next token in the list
		/// </summary>
		public UhtTokenList? Next { get; set; }

		/// <summary>
		/// Return the last token list entry
		/// </summary>
		public UhtTokenList Last
		{
			get
			{
				UhtTokenList last = this;
				while (last.Next != null)
				{
					last = last.Next;
				}
				return last;
			}
		}

		/// <summary>
		/// Join the tokens in the list
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="separator">Separator between the tokens</param>
		/// <returns></returns>
		public void Join(StringBuilder builder, string separator = "")
		{
			builder.Append(Token.Value.ToString());
			UhtTokenList list = this;
			while (list.Next != null)
			{
				list = list.Next;
				builder.Append(separator);
				builder.Append(list.Token.Value.ToString());
			}
		}

		/// <summary>
		/// Join the tokens in the list
		/// </summary>
		/// <param name="separator">Separator between the tokens</param>
		/// <returns></returns>
		public string Join(string separator = "")
		{
			if (Next == null)
			{
				return Token.Value.ToString();
			}
			StringBuilder builder = new();
			Join(builder, separator);
			return builder.ToString();
		}

		/// <summary>
		/// Remap the last identifier
		/// </summary>
		/// <param name="config">Configuration containing remapping information</param>
		public void RedirectTypeIdentifier(IUhtConfig config)
		{
			UhtTokenList last = Last;
			UhtToken scratch = last.Token;
			config.RedirectTypeIdentifier(ref scratch);
			last.Token = scratch;
		}

		/// <summary>
		/// Return the token list as an array
		/// </summary>
		/// <returns></returns>
		public UhtToken[] ToArray()
		{
			int count = 1;
			for (UhtTokenList temp = this; temp.Next != null; temp = temp.Next)
			{
				++count;
			}
			UhtToken[] outTokens = new UhtToken[count];
			outTokens[0] = Token;
			count = 1;
			for (UhtTokenList temp = this; temp.Next != null; temp = temp.Next)
			{
				outTokens[count] = temp.Next.Token;
				++count;
			}
			return outTokens;
		}
	}

	/// <summary>
	/// Token list cache.  Token lists must be returned to the cache.
	/// </summary>
	public static class UhtTokenListCache
	{
		private static readonly ThreadLocal<UhtTokenList?> s_tls = new(() => null);

		/// <summary>
		/// Borrow a token list
		/// </summary>
		/// <param name="token">Starting token</param>
		/// <returns>Token list</returns>
		public static UhtTokenList Borrow(UhtToken token)
		{
			UhtTokenList? identifier = s_tls.Value;
			if (identifier != null)
			{
				s_tls.Value = identifier.Next;
			}
			else
			{
				identifier = new UhtTokenList();
			}
			identifier.Token = token;
			identifier.Next = null;
			return identifier;
		}

		/// <summary>
		/// Return a token list to the cache
		/// </summary>
		/// <param name="identifier"></param>
		public static void Return(UhtTokenList identifier)
		{
			UhtTokenList? tail = s_tls.Value;
			if (tail != null)
			{
				tail.Next = identifier;
			}

			for (; identifier.Next != null; identifier = identifier.Next)
			{
			}

			s_tls.Value = identifier;
		}
	}

	/// <summary>
	/// Delegate for when a token is parsed
	/// </summary>
	/// <param name="token">The token in question</param>
	public delegate void UhtTokenDelegate(ref UhtToken token);

	/// <summary>
	/// Delegate for when a token is parsed in an until block
	/// </summary>
	/// <param name="token">The token in question</param>
	/// <returns>True if parsing should continue</returns>
	public delegate bool UhtTokensUntilDelegate(ref UhtToken token);

	/// <summary>
	/// Delegate for an enumeration of tokens
	/// </summary>
	/// <param name="tokens">Parsed tokens</param>
	public delegate void UhtTokensDelegate(IEnumerable<UhtToken> tokens);

	/// <summary>
	/// Delegate for cached token list
	/// </summary>
	/// <param name="tokenList">Token list that can be cached</param>
	public delegate void UhtTokenListDelegate(UhtTokenList tokenList);

	/// <summary>
	/// Delegate for a constant float
	/// </summary>
	/// <param name="value">Value in question</param>
	public delegate void UhtTokenConstFloatDelegate(float value);

	/// <summary>
	/// Delegate for a constant double
	/// </summary>
	/// <param name="value">Value in question</param>
	public delegate void UhtTokenConstDoubleDelegate(double value);

	/// <summary>
	/// Helper struct to disable comment parsing.  Should be used in a using block
	/// </summary>
	public readonly struct UhtTokenDisableComments : IDisposable
	{
		private readonly IUhtTokenReader _tokenReader;

		/// <summary>
		/// Construct instance
		/// </summary>
		/// <param name="tokenReader">Token reader to disable</param>
		public UhtTokenDisableComments(IUhtTokenReader tokenReader)
		{
			_tokenReader = tokenReader;
			_tokenReader.DisableComments();
		}

		/// <summary>
		/// Enable comments
		/// </summary>
		public void Dispose()
		{
			_tokenReader.EnableComments();
		}
	}

	/// <summary>
	/// Helper struct to save token reader state
	/// </summary>
	public struct UhtTokenSaveState : IDisposable
	{
		private readonly IUhtTokenReader _tokenReader;
		private bool _handled;

		/// <summary>
		/// Construct instance
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		public UhtTokenSaveState(IUhtTokenReader tokenReader)
		{
			_tokenReader = tokenReader;
			_handled = false;
			_tokenReader.SaveState();
		}

		/// <summary>
		/// Restore the token reader state
		/// </summary>
		public void Dispose()
		{
			if (!_handled)
			{
				RestoreState();
			}
		}

		/// <summary>
		/// Restore the token reader state
		/// </summary>
		/// <exception cref="UhtIceException">Thrown if state has already been restored or aborted</exception>
		public void RestoreState()
		{
			if (_handled)
			{
				throw new UhtIceException("Can not call RestoreState/AbandonState more than once");
			}
			_tokenReader.RestoreState();
			_handled = true;
		}

		/// <summary>
		/// Abandon the saved state
		/// </summary>
		/// <exception cref="UhtIceException">Thrown if state has already been restored or aborted</exception>
		public void AbandonState()
		{
			if (_handled)
			{
				throw new UhtIceException("Can not call RestoreState/AbandonState more than once");
			}
			_tokenReader.AbandonState();
			_handled = true;
		}
	}
}
