// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Collection of token reader extensions for float values
	/// </summary>
	public static class UhtTokenReaderFloatExtensions
	{
		/// <summary>
		/// Get the next token as a float.  If the next token is not a float, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="value">The float value of the token</param>
		/// <returns>True if the next token was an float, false if not.</returns>
		public static bool TryOptionalConstFloat(this IUhtTokenReader tokenReader, out float value)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsConstFloat() && token.GetConstFloat(out value)) // NOTE: This is restricted to only float values
			{
				tokenReader.ConsumeToken();
				return true;
			}
			value = 0;
			return false;
		}

		/// <summary>
		/// Get the next token as a float.  If the next token is not a float, no token is consumed.
		/// </summary>
		/// <param name="tokenReader"></param>
		/// <returns>The token reader</returns>
		public static IUhtTokenReader OptionalConstFloat(this IUhtTokenReader tokenReader)
		{
			tokenReader.TryOptionalConstFloat(out float _);
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as a float.  If the next token is not a float, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="floatDelegate">Delegate to invoke with the float value</param>
		/// <returns>The token reader</returns>
		public static IUhtTokenReader OptionalConstFloat(this IUhtTokenReader tokenReader, UhtTokenConstFloatDelegate floatDelegate)
		{
			if (tokenReader.TryOptionalConstFloat(out float value))
			{
				floatDelegate(value);
			}
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as a float.  If the next token is not a float, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>True if the next token was an float, false if not.</returns>
		public static IUhtTokenReader RequireConstFloat(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstFloat(out float _))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant float", exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as a float.  If the next token is not a float, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The floating point value of the token</returns>
		public static float GetConstFloat(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstFloat(out float value))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant float", exceptionContext);
			}
			return value;
		}

		/// <summary>
		/// Get the next float.  It also handles [+/-] token followed by an float.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="value">The float value of the token</param>
		/// <returns>True if the next token was an float, false if not.</returns>
		public static bool TryOptionalLeadingSignConstFloat(this IUhtTokenReader tokenReader, out float value)
		{
			float localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return (token.IsConstInt() || token.IsConstFloat()) && token.GetConstFloat(out localValue);
			});
			value = localValue;
			return results;
		}

		/// <summary>
		/// Get the next float.  It also handles [+/-] token followed by an float.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="value">The float value of the token</param>
		/// <returns>True if the next token was an float, false if not.</returns>
		public static bool TryOptionalConstFloatExpression(this IUhtTokenReader tokenReader, out float value)
		{
			float localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return (token.IsConstInt() || token.IsConstFloat()) && token.GetConstFloat(out localValue);
			});
			value = localValue;
			return results;
		}

		/// <summary>
		/// Get the next float.  It also handles [+/-] token followed by an float.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <returns>The double value</returns>
		public static float GetConstFloatExpression(this IUhtTokenReader tokenReader)
		{
			if (!tokenReader.TryOptionalConstFloatExpression(out float localValue))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant float", null);
			}
			return localValue;
		}

		/// <summary>
		/// Get the next double.  It also handles [+/-] token followed by an double.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="value">The double value of the token</param>
		/// <returns>True if the next token was an double, false if not.</returns>
		public static bool TryOptionalConstDoubleExpression(this IUhtTokenReader tokenReader, out double value)
		{
			double localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return (token.IsConstInt() || token.IsConstFloat()) && token.GetConstDouble(out localValue);
			});
			value = localValue;
			return results;
		}

		/// <summary>
		/// Get the next double.  It also handles [+/-] token followed by an double.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="doubleDelegate">Delegate to invoke if the double is parsed</param>
		/// <returns>The supplied token reader</returns>
		public static IUhtTokenReader RequireConstDoubleExpression(this IUhtTokenReader tokenReader, UhtTokenConstDoubleDelegate doubleDelegate)
		{
			if (!tokenReader.TryOptionalConstDoubleExpression(out double localValue))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant double", null);
			}
			doubleDelegate(localValue);
			return tokenReader;
		}

		/// <summary>
		/// Get the next double.  It also handles [+/-] token followed by an double.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <returns>The double value</returns>
		public static double GetConstDoubleExpression(this IUhtTokenReader tokenReader)
		{
			if (!tokenReader.TryOptionalConstDoubleExpression(out double localValue))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant double", null);
			}
			return localValue;
		}
	}
}
