// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Collection of token reader extensions to help with flow control
	/// </summary>
	public static class UhtTokenReaderFlowControlExtensions
	{

		/// <summary>
		/// Parse an optional list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="action">Action to be invoked for each list element.</param>
		/// <returns>True if a list was read</returns>
		public static bool TryOptionalList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, Action action)
		{
			if (tokenReader.TryOptional(initiator))
			{
				tokenReader.RequireList(terminator, separator, allowTrailingSeparator, action);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Parse a required list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="action">Action to be invoked for each list element.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, Action action)
		{
			return tokenReader.RequireList(initiator, terminator, separator, allowTrailingSeparator, null, action);
		}

		/// <summary>
		/// Parse a required list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="exceptionContext">Extra context for error messages</param>
		/// <param name="action">Action to be invoked for each list element.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, object? exceptionContext, Action action)
		{
			tokenReader.Require(initiator);
			return tokenReader.RequireList(terminator, separator, allowTrailingSeparator, exceptionContext, action);
		}

		/// <summary>
		/// Parse a required list.  Initiating token must have already been parsed
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="action">Action to be invoked for each list element.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char terminator, char separator, bool allowTrailingSeparator, Action action)
		{
			return tokenReader.RequireList(terminator, separator, allowTrailingSeparator, null, action);
		}

		/// <summary>
		/// Parse a required list.  Initiating token must have already been parsed
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="exceptionContext">Extra context for error messages</param>
		/// <param name="action">Action to be invoked for each list element.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char terminator, char separator, bool allowTrailingSeparator, object? exceptionContext, Action action)
		{
			// Check for an empty list
			if (tokenReader.TryOptional(terminator))
			{
				return tokenReader;
			}

			// Process the body
			while (true)
			{

				// Read the element via the lambda
				action();

				// Make sure we haven't reached the EOF
				if (tokenReader.IsEOF)
				{
					throw new UhtTokenException(tokenReader, tokenReader.GetToken(), $"'{separator}' or '{terminator}'", exceptionContext);
				}

				// If we have a separator, then it might be a trailing separator 
				if (tokenReader.TryOptional(separator))
				{
					if (allowTrailingSeparator && tokenReader.TryOptional(terminator))
					{
						return tokenReader;
					}
					continue;
				}

				// Otherwise, we must have an terminator
				if (!tokenReader.TryOptional(terminator))
				{
					throw new UhtTokenException(tokenReader, tokenReader.GetToken(), $"'{separator}' or '{terminator}'", exceptionContext);
				}
				return tokenReader;
			}
		}

		/// <summary>
		/// Parse an optional list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="tokensDelegate">Delegate to invoke with the parsed token list</param>
		/// <returns>True if a list was read</returns>
		public static bool TryOptionalList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, UhtTokensDelegate tokensDelegate)
		{
			if (tokenReader.TryOptional(initiator))
			{
				tokenReader.RequireList(terminator, separator, allowTrailingSeparator, tokensDelegate);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Parse a required list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="tokensDelegate">Delegate to invoke with the parsed token list</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, UhtTokensDelegate tokensDelegate)
		{
			return tokenReader.RequireList(initiator, terminator, separator, allowTrailingSeparator, null, tokensDelegate);
		}

		/// <summary>
		/// Parse a required list
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating symbol</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="exceptionContext">Extra context for error messages</param>
		/// <param name="tokensDelegate">Delegate to invoke with the parsed token list</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char initiator, char terminator, char separator, bool allowTrailingSeparator, object? exceptionContext, UhtTokensDelegate tokensDelegate)
		{
			tokenReader.Require(initiator);
			return tokenReader.RequireList(terminator, separator, allowTrailingSeparator, exceptionContext, tokensDelegate);
		}

		/// <summary>
		/// Parse a required list.  Initiating token must have already been parsed
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="tokensDelegate">Delegate to invoke with the parsed token list</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char terminator, char separator, bool allowTrailingSeparator, UhtTokensDelegate tokensDelegate)
		{
			return tokenReader.RequireList(terminator, separator, allowTrailingSeparator, null, tokensDelegate);
		}

		/// <summary>
		/// Parse a required list.  Initiating token must have already been parsed
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <param name="separator">Separator symbol</param>
		/// <param name="allowTrailingSeparator">If true, allow trailing separators</param>
		/// <param name="exceptionContext">Extra context for error messages</param>
		/// <param name="tokensDelegate">Delegate to invoke with the parsed token list</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char terminator, char separator, bool allowTrailingSeparator, object? exceptionContext, UhtTokensDelegate tokensDelegate)
		{
			// Check for an empty list
			if (tokenReader.TryOptional(terminator))
			{
				return tokenReader;
			}

			// Process the body
			while (true)
			{
				List<UhtToken> tokens = [];

				// Read the tokens until we hit the end
				while (true)
				{
					// Make sure we haven't reached the EOF
					if (tokenReader.IsEOF)
					{
						throw new UhtTokenException(tokenReader, tokenReader.GetToken(), $"'{separator}' or '{terminator}'", exceptionContext);
					}

					// If we have a separator, then it might be a trailing separator 
					if (tokenReader.TryOptional(separator))
					{
						tokensDelegate(tokens);
						if (tokenReader.TryOptional(terminator))
						{
							if (allowTrailingSeparator)
							{
								return tokenReader;
							}
							throw new UhtException(tokenReader, $"A separator '{separator}' followed immediately by the terminator '{terminator}' is invalid");
						}
						break;
					}

					// If this is the terminator, then we are done
					if (tokenReader.TryOptional(terminator))
					{
						tokensDelegate(tokens);
						return tokenReader;
					}

					tokens.Add(tokenReader.GetToken());
				}
			}
		}

		/// <summary>
		/// Consume a block of tokens bounded by the two given symbols.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">The next token must be the given symbol.</param>
		/// <param name="terminator">The tokens are read until the given symbol is found.  The terminating symbol will be consumed.</param>
		/// <param name="exceptionContext">Extra context for any error messages</param>
		/// <returns>The input token reader</returns>
		public static IUhtTokenReader RequireList(this IUhtTokenReader tokenReader, char initiator, char terminator, object? exceptionContext = null)
		{
			tokenReader.Require(initiator);

			// Process the body
			while (true)
			{

				// Make sure we haven't reached the EOF
				if (tokenReader.IsEOF)
				{
					throw new UhtTokenException(tokenReader, tokenReader.GetToken(), $"'{terminator}'", exceptionContext);
				}

				// Look for the terminator
				if (tokenReader.TryOptional(terminator))
				{
					break;
				}

				// Consume the current token
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Invoke action while the next token is the given string
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match to continue invoking Action</param>
		/// <param name="action">Action to invoke if and only if the prior text was parsed.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader While(this IUhtTokenReader tokenReader, string text, Action action)
		{
			while (tokenReader.TryOptional(text))
			{
				action();
			}
			return tokenReader;
		}

		/// <summary>
		/// Invoke action while the next token is the given string
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match to continue invoking Action</param>
		/// <param name="action">Action to invoke if and only if the prior text was parsed.</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader While(this IUhtTokenReader tokenReader, char text, Action action)
		{
			while (tokenReader.TryOptional(text))
			{
				action();
			}
			return tokenReader;
		}

		/// <summary>
		/// Consume tokens until one of the strings are found.  Terminating token will not be consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminators">Strings that will terminate processing.</param>
		/// <returns>Number of tokens consumed</returns>
		public static int ConsumeUntil(this IUhtTokenReader tokenReader, string[] terminators)
		{
			int consumedTokens = 0;
			while (!tokenReader.IsEOF)
			{
				ref UhtToken token = ref tokenReader.PeekToken();
				foreach (string candidate in terminators)
				{
					if ((token.IsIdentifier() || token.IsSymbol()) && token.IsValue(candidate))
					{
						return consumedTokens;
					}
				}
				tokenReader.ConsumeToken();
				++consumedTokens;
			}
			return consumedTokens;
		}

		/// <summary>
		/// Consume until the given terminator is found.  Terminating token will be consumed
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminating symbol</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader ConsumeUntil(this IUhtTokenReader tokenReader, char terminator)
		{
			while (!tokenReader.IsEOF)
			{
				if (tokenReader.TryOptional(terminator))
				{
					break;
				}
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Throw an exception if the given symbol is found in the token stream
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="candidate">Symbol or identifier to match</param>
		/// <param name="message">Message thrown if found</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtException"></exception>
		public static IUhtTokenReader ExceptionIf(this IUhtTokenReader tokenReader, string candidate, string message)
		{
			UhtToken token = tokenReader.PeekToken();
			if ((token.IsIdentifier() || token.IsSymbol()) && token.IsValue(candidate))
			{
				throw new UhtException(tokenReader, message);
			}
			return tokenReader;
		}
	}
}
