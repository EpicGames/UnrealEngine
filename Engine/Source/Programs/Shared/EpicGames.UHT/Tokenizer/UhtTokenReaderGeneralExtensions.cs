// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Collection of general token reader extensions
	/// </summary>
	public static class UhtTokenReaderGeneralExtensions
	{

		/// <summary>
		/// Try to parse the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>True if the text matched</returns>
		public static bool TryOptional(this IUhtTokenReader tokenReader, string text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
				return true;
			}
			return false;
		}

		/// <summary>
		/// Try to parse the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>True if the text matched</returns>
		public static bool TryOptional(this IUhtTokenReader tokenReader, StringView text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
				return true;
			}
			return false;
		}

		/// <summary>
		/// Try to parse the given text.  However, the matching token will not be consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>True if the text matched</returns>
		public static bool TryPeekOptional(this IUhtTokenReader tokenReader, string text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			return token.IsIdentifier(text) || token.IsSymbol(text);
		}

		/// <summary>
		/// Try to parse the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>True if the text matched</returns>
		public static bool TryOptional(this IUhtTokenReader tokenReader, char text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
				return true;
			}
			return false;
		}

		/// <summary>
		/// Try to parse the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <param name="outToken">Open that was matched</param>
		/// <returns>True if the text matched</returns>
		public static bool TryOptional(this IUhtTokenReader tokenReader, char text, out UhtToken outToken)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				outToken = token;
				tokenReader.ConsumeToken();
				return true;
			}
			outToken = new UhtToken();
			return false;
		}

		/// <summary>
		/// Try to parse the given text.  However, the matching token will not be consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>True if the text matched</returns>
		public static bool TryPeekOptional(this IUhtTokenReader tokenReader, char text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			return token.IsSymbol(text);
		}

		/// <summary>
		/// Parse optional text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader Optional(this IUhtTokenReader tokenReader, string text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <param name="action">Action to invoke if the text was found</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader Optional(this IUhtTokenReader tokenReader, string text, Action action)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
				action();
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader Optional(this IUhtTokenReader tokenReader, char text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <param name="action">Action to invoke if the text was found</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader Optional(this IUhtTokenReader tokenReader, char text, Action action)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
				action();
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional token that starts with the given text 
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalStartsWith(this IUhtTokenReader tokenReader, string text)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier() && token.ValueStartsWith(text))
			{
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional token that starts with the given text 
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <param name="tokenDelegate">Delegate to invoke on a match</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalStartsWith(this IUhtTokenReader tokenReader, string text, UhtTokenDelegate tokenDelegate)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier() && token.ValueStartsWith(text))
			{
				UhtToken currentToken = token;
				tokenReader.ConsumeToken();
				tokenDelegate(ref currentToken);
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse optional text that ends with with the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Text to match</param>
		/// <param name="action">Action to invoke if the text was found</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalEndsWith(this IUhtTokenReader tokenReader, string text, Action<string> action)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier() && token.Value.Span.EndsWith(text, StringComparison.Ordinal))
			{
				tokenReader.ConsumeToken();
				action(token.Value.Span.ToString());
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse attributes and optionally alignment specifier 
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="enableAlignAs">If true, also parse alignment specifiers</param>
		/// <param name="attributeAction">If specified, action to be invoked for every attribute found</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalAttributes(this IUhtTokenReader tokenReader, bool enableAlignAs, Action<string>? attributeAction = null)
		{
			for (; ; )
			{
				UhtToken token = tokenReader.PeekToken();
				if (token.IsIdentifier("DEPRECATED") || token.IsIdentifier("UE_DEPRECATED") || token.IsIdentifier("UE_DEPRECATED_FORGAME") || token.IsIdentifier("UE_DEPRECATED_FORENGINE"))
				{
					tokenReader.ConsumeToken();
					tokenReader.Require('(', "deprecation macro");
					token = tokenReader.GetToken();
					if (!token.IsConstFloat() && !token.IsIdentifier("all", ignoreCase: true))
					{
						throw new UhtTokenException(tokenReader, token, "'all' or constant float version in deprecation macro");
					}
					tokenReader.Require(',', "deprecation macro")
						.RequireConstString("message in deprecation macro")
						.Require(')', "deprecation macro");
					attributeAction?.Invoke("deprecated");
				}
				else if (token.IsIdentifier("UE_EXPERIMENTAL"))
				{
					tokenReader.ConsumeToken();
					tokenReader
						.Require('(', "experimental macro")
						.RequireConstFloat()
						.Require(',', "experimental macro")
						.RequireConstString("message in experimental macro")
						.Require(')', "experimental macro");
					attributeAction?.Invoke("experimental");
				}
				else if (token.IsIdentifier("UE_INTERNAL"))
				{
					tokenReader.ConsumeToken();
				}
				else if (token.IsIdentifier("UE_NODISCARD_CTOR"))
				{
					tokenReader.ConsumeToken();
					attributeAction?.Invoke("nodiscard");
				}
				else if (token.IsIdentifier("UE_NO_UNIQUE_ADDRESS"))
				{
					tokenReader.ConsumeToken();
					attributeAction?.Invoke("no_unique_address");
				}
				else if (token.IsIdentifier("AUTORTFM_DISABLE") || token.IsIdentifier("AUTORTFM_ENABLE") || token.IsIdentifier("AUTORTFM_INFER") || token.IsIdentifier("AUTORTFM_OPEN") || token.IsIdentifier("AUTORTFM_OPEN_NO_VALIDATION"))
				{
					tokenReader.ConsumeToken();
				}
				else if (token.IsSymbol("[["))
				{
					tokenReader.ConsumeToken();

					StringBuilder identifierBuilder = StringBuilderCache.Small.Borrow();
					try
					{
						if (tokenReader.TryOptional("using"))
						{
							tokenReader
								.RequireCppIdentifier(UhtCppIdentifierOptions.None, x => x.Join(identifierBuilder, "::"))
								.Require(':');
						}
						if (identifierBuilder.Length > 0)
						{
							identifierBuilder.Append("::");
						}
						int identifierBuilderLength = identifierBuilder.Length;

						// Attributes do allow patterns such as [[,]].
						for (; ; )
						{
							token = tokenReader.GetToken();
							if (token.IsIdentifier())
							{
								tokenReader.RequireCppIdentifier(ref token, UhtCppIdentifierOptions.None, x => x.Join(identifierBuilder, "::"));

								if (tokenReader.TryOptional('('))
								{
									int nestingCount = 1;
									while (nestingCount > 0)
									{
										if (tokenReader.IsEOF)
										{
											throw new UhtTokenException(tokenReader, tokenReader.GetToken(), ")");
										}
										else if (tokenReader.TryOptional(')'))
										{
											nestingCount--;
										}
										else if (tokenReader.TryOptional('('))
										{
											nestingCount++;
										}
										tokenReader.ConsumeToken();
									}
								}

								attributeAction?.Invoke(identifierBuilder.ToString());
								identifierBuilder.Length = identifierBuilderLength;
							}
							else if (token.IsSymbol(','))
							{
							}
							else if (token.IsSymbol(']'))
							{
								// This isn't technically correct since it could be "] ]"
								tokenReader.Require(']');
								break;
							}
							else
							{
								throw new UhtTokenException(tokenReader, token, "]], ',', or identifier");
							}
						}
					}
					finally
					{
						StringBuilderCache.Small.Return(identifierBuilder);
					}
				}
				else if (enableAlignAs && token.IsIdentifier("alignas"))
				{
					tokenReader.ConsumeToken();
					tokenReader
						.Require('(', "alignment specifier")
						.RequireConstInt("alignment specifier")
						.Require(')', "alignment specifier");
				}
				else
				{
					break;
				}
			}
			return tokenReader;
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, string text, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
			}
			else
			{
				throw new UhtTokenException(tokenReader, token, text, exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="tokenDelegate">Delegate to invoke on a match</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, string text, UhtTokenDelegate tokenDelegate)
		{
			return tokenReader.Require(text, null, tokenDelegate);
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <param name="tokenDelegate">Delegate to invoke on a match</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, string text, object? exceptionContext, UhtTokenDelegate tokenDelegate)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier(text) || token.IsSymbol(text))
			{
				UhtToken currentToken = token;
				tokenReader.ConsumeToken();
				tokenDelegate(ref currentToken);
			}
			else
			{
				throw new UhtTokenException(tokenReader, token, text, exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, char text, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				tokenReader.ConsumeToken();
			}
			else
			{
				throw new UhtTokenException(tokenReader, token, text, exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="tokenDelegate">Delegate to invoke on a match</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, char text, UhtTokenDelegate tokenDelegate)
		{
			return tokenReader.Require(text, null, tokenDelegate);
		}

		/// <summary>
		/// Require the given text
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="text">Required text</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <param name="tokenDelegate">Delegate to invoke on a match</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if text is not found</exception>
		public static IUhtTokenReader Require(this IUhtTokenReader tokenReader, char text, object? exceptionContext, UhtTokenDelegate tokenDelegate)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsSymbol(text))
			{
				UhtToken currentToken = token;
				tokenReader.ConsumeToken();
				tokenDelegate(ref currentToken);
			}
			else
			{
				throw new UhtTokenException(tokenReader, token, text, exceptionContext);
			}
			return tokenReader;
		}
	}
}
