// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Options when parsing identifier
	/// </summary>
	[Flags]
	public enum UhtCppIdentifierOptions
	{

		/// <summary>
		/// No options
		/// </summary>
		None,

		/// <summary>
		/// Include template arguments when parsing identifier
		/// </summary>
		AllowTemplates = 1 << 0,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtGetCppIdentifierOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtCppIdentifierOptions inFlags, UhtCppIdentifierOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtCppIdentifierOptions inFlags, UhtCppIdentifierOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtCppIdentifierOptions inFlags, UhtCppIdentifierOptions testFlags, UhtCppIdentifierOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Collection of token reader extensions for working with identifiers
	/// </summary>
	public static class UhtTokenReaderIdentifierExtensions
	{

		/// <summary>
		/// Get the next token and verify that it is an identifier
		/// </summary>
		/// <returns>True if it is an identifier, false if not.</returns>
		public static bool TryOptionalIdentifier(this IUhtTokenReader tokenReader)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				tokenReader.ConsumeToken();
				return true;
			}
			return false;
		}

		/// <summary>
		/// Get the next token and verify that it is an identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="identifier">The fetched value of the identifier</param>
		/// <returns>True if it is an identifier, false if not.</returns>
		public static bool TryOptionalIdentifier(this IUhtTokenReader tokenReader, out UhtToken identifier)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				identifier = token;
				tokenReader.ConsumeToken();
				return true;
			}
			identifier = new UhtToken();
			return false;
		}

		/// <summary>
		/// Parse an optional identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="tokenDelegate">Invoked of an identifier is parsed</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalIdentifier(this IUhtTokenReader tokenReader, UhtTokenDelegate tokenDelegate)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				UhtToken tokenCopy = token;
				tokenReader.ConsumeToken();
				tokenDelegate(ref tokenCopy);
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse an optional namespace
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="namespaceText">The text of the namespace</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader OptionalNamespace(this IUhtTokenReader tokenReader, string namespaceText)
		{
			if (tokenReader.TryOptional(namespaceText))
			{
				tokenReader.Require("::");
			}
			return tokenReader;
		}

		/// <summary>
		/// Parse a required identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if an identifier isn't found</exception>
		public static IUhtTokenReader RequireIdentifier(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				tokenReader.ConsumeToken();
				return tokenReader;
			}
			throw new UhtTokenException(tokenReader, token, "an identifier", exceptionContext);
		}

		/// <summary>
		/// Parse a required identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="tokenDelegate">Invoked if an identifier is parsed</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireIdentifier(this IUhtTokenReader tokenReader, UhtTokenDelegate tokenDelegate)
		{
			return tokenReader.RequireIdentifier(null, tokenDelegate);
		}

		/// <summary>
		/// Parse a required identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <param name="tokenDelegate">Invoked if an identifier is parsed</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Thrown if an identifier isn't found</exception>
		public static IUhtTokenReader RequireIdentifier(this IUhtTokenReader tokenReader, object? exceptionContext, UhtTokenDelegate tokenDelegate)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				UhtToken currentToken = token;
				tokenReader.ConsumeToken();
				tokenDelegate(ref currentToken);
				return tokenReader;
			}
			throw new UhtTokenException(tokenReader, token, "an identifier", exceptionContext);
		}

		/// <summary>
		/// Get a required identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">Extra exception context</param>
		/// <returns>Identifier token</returns>
		/// <exception cref="UhtTokenException">Thrown if an identifier isn't found</exception>
		public static UhtToken GetIdentifier(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier())
			{
				UhtToken currentToken = token;
				tokenReader.ConsumeToken();
				return currentToken;
			}
			throw new UhtTokenException(tokenReader, token, exceptionContext ?? "an identifier");
		}

		/// <summary>
		/// Parse a required cpp identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="options">Parsing options</param>
		/// <param name="tokenListDelegate">Invoked when identifier is parsed</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireCppIdentifier(this IUhtTokenReader tokenReader, UhtCppIdentifierOptions options, UhtTokenListDelegate tokenListDelegate)
		{
			UhtTokenList tokenList = tokenReader.GetCppIdentifier(options);
			tokenListDelegate(tokenList);
			UhtTokenListCache.Return(tokenList);
			return tokenReader;
		}

		/// <summary>
		/// Parse a required cpp identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initialIdentifier">Initial token of the identifier</param>
		/// <param name="options">Parsing options</param>
		/// <param name="tokenListDelegate">Invoked when identifier is parsed</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader RequireCppIdentifier(this IUhtTokenReader tokenReader, ref UhtToken initialIdentifier, UhtCppIdentifierOptions options, UhtTokenListDelegate tokenListDelegate)
		{
			UhtTokenList tokenList = tokenReader.GetCppIdentifier(ref initialIdentifier, options);
			tokenListDelegate(tokenList);
			UhtTokenListCache.Return(tokenList);
			return tokenReader;
		}

		/// <summary>
		/// Get a required cpp identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="options">Parsing options</param>
		/// <returns>Token list</returns>
		public static UhtTokenList GetCppIdentifier(this IUhtTokenReader tokenReader, UhtCppIdentifierOptions options = UhtCppIdentifierOptions.None)
		{
			tokenReader.Optional("::");
			UhtToken token = tokenReader.GetIdentifier();
			return tokenReader.GetCppIdentifier(ref token, options);
		}

		/// <summary>
		/// Get a required cpp identifier
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initialIdentifier">Initial token of the identifier</param>
		/// <param name="options">Parsing options</param>
		/// <returns>Token list</returns>
		public static UhtTokenList GetCppIdentifier(this IUhtTokenReader tokenReader, ref UhtToken initialIdentifier, UhtCppIdentifierOptions options = UhtCppIdentifierOptions.None)
		{
			UhtTokenList listHead = UhtTokenListCache.Borrow(initialIdentifier);
			UhtTokenList listTail = listHead;

			if (options.HasAnyFlags(UhtCppIdentifierOptions.AllowTemplates))
			{
				while (true)
				{
					if (tokenReader.TryPeekOptional('<'))
					{
						listTail.Next = UhtTokenListCache.Borrow(tokenReader.GetToken());
						listTail = listTail.Next;
						int nestedScopes = 1;
						while (nestedScopes > 0)
						{
							UhtToken token = tokenReader.GetToken();
							if (token.TokenType.IsEndType())
							{
								throw new UhtTokenException(tokenReader, token, new List<string> { "<", ">" }, "template");
							}
							listTail.Next = UhtTokenListCache.Borrow(token);
							listTail = listTail.Next;
							if (token.IsSymbol('<'))
							{
								++nestedScopes;
							}
							else if (token.IsSymbol('>'))
							{
								--nestedScopes;
							}
						}
					}

					if (!tokenReader.TryPeekOptional("::"))
					{
						break;
					}
					listTail.Next = UhtTokenListCache.Borrow(tokenReader.GetToken());
					listTail = listTail.Next;
					listTail.Next = UhtTokenListCache.Borrow(tokenReader.GetIdentifier());
					listTail = listTail.Next;
				}
			}
			else if (tokenReader.PeekToken().IsSymbol("::"))
			{
				tokenReader.While("::", () =>
				{
					listTail.Next = UhtTokenListCache.Borrow(tokenReader.GetIdentifier());
					listTail = listTail.Next;
				});
			}
			return listHead;
		}
	}
}
