// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Notification of signed token being parsed
	/// </summary>
	/// <param name="token">Token in question</param>
	/// <returns>True if the token value is acceptable</returns>
	public delegate bool UhtParseMergedSignToken(ref UhtToken token);

	/// <summary>
	/// Collection of helper methods to parse integers
	/// </summary>
	public static class UhtTokenReaderIntegerExtensions
	{
		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="value">The integer value of the token</param>
		/// <returns>True if the next token was an integer, false if not.</returns>
		public static bool TryOptionalConstInt(this IUhtTokenReader tokenReader, out int value)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.GetConstInt(out value))
			{
				tokenReader.ConsumeToken();
				return true;
			}
			value = 0;
			return false;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <returns>The value of the constant</returns>
		public static IUhtTokenReader OptionalConstInt(this IUhtTokenReader tokenReader)
		{
			tokenReader.TryOptionalConstInt(out int _);
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the constant</returns>
		public static IUhtTokenReader RequireConstInt(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstInt(out int _))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant integer", exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the constant</returns>
		public static int GetConstInt(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstInt(out int value))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant integer", exceptionContext);
			}
			return value;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="value">The integer value of the token</param>
		/// <returns>True if the next token was an integer, false if not.</returns>
		public static bool TryOptionalConstLong(this IUhtTokenReader tokenReader, out long value)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.GetConstLong(out value))
			{
				tokenReader.ConsumeToken();
				return true;
			}
			value = 0;
			return false;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <returns>The value of the constant</returns>
		public static IUhtTokenReader OptionalConstLong(this IUhtTokenReader tokenReader)
		{
			ref UhtToken token = ref tokenReader.PeekToken();

			if (token.GetConstLong(out long _))
			{
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the constant</returns>
		public static IUhtTokenReader RequireConstLong(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstLong(out long _))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant long integer", exceptionContext);
			}
			return tokenReader;
		}

		/// <summary>
		/// Get the next token as an integer.  If the next token is not an integer, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the constant</returns>
		public static long GetConstLong(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			if (!tokenReader.TryOptionalConstLong(out long value))
			{
				throw new UhtTokenException(tokenReader, tokenReader.PeekToken(), "constant long integer", exceptionContext);
			}
			return value;
		}

		/// <summary>
		/// Helper method to combine any leading sign with the next numeric token
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="tokenDelegate">Delegate to invoke with the merged value</param>
		/// <returns>True if the next token was an parsed, false if not.</returns>
		public static bool TryOptionalLeadingSignConstNumeric(this IUhtTokenReader tokenReader, UhtParseMergedSignToken tokenDelegate)
		{
			using UhtTokenSaveState savedState = new(tokenReader);
			// Check for a leading sign token
			char sign = ' ';
			UhtToken token = tokenReader.PeekToken();
			if (token.IsSymbol() && token.Value.Length == 1 && UhtFCString.IsSign(token.Value.Span[0]))
			{
				sign = token.Value.Span[0];
				tokenReader.ConsumeToken();
				token = tokenReader.PeekToken();
				if (UhtFCString.IsSign(token.Value.Span[0]))
				{
					return false;
				}
				token.Value = new StringView($"{sign}{token.Value}");
			}
			if (tokenDelegate(ref token))
			{
				tokenReader.ConsumeToken();
				savedState.AbandonState();
				return true;
			}
			return false;
		}

		/// <summary>
		/// Get the next integer.  It also handled [+/-] token followed by an integer.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="value">The integer value of the token</param>
		/// <returns>True if the next token was an integer, false if not.</returns>
		public static bool TryOptionalConstIntExpression(this IUhtTokenReader tokenReader, out int value)
		{
			int localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return token.IsConstInt() && token.GetConstInt(out localValue);
			});
			value = localValue;
			return results;
		}

		/// <summary>
		/// Get the next integer.  It also handled [+/-] token followed by an integer.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <returns>The integer value</returns>
		public static int GetConstIntExpression(this IUhtTokenReader tokenReader)
		{
			int localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return token.IsConstInt() && token.GetConstInt(out localValue);
			});
			return localValue;
		}

		/// <summary>
		/// Get the next integer.  It also handled [+/-] token followed by an integer.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <param name="value">The integer value of the token</param>
		/// <returns>True if the next token was an integer, false if not.</returns>
		public static bool TryOptionalConstLongExpression(this IUhtTokenReader tokenReader, out long value)
		{
			long localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return token.IsConstInt() && token.GetConstLong(out localValue);
			});
			value = localValue;
			return results;
		}

		/// <summary>
		/// Get the next long.  It also handled [+/-] token followed by an long.
		/// </summary>
		/// <param name="tokenReader">Source tokens</param>
		/// <returns>The long value</returns>
		public static int GetConstLongExpression(this IUhtTokenReader tokenReader)
		{
			int localValue = 0;
			bool results = tokenReader.TryOptionalLeadingSignConstNumeric((ref UhtToken token) =>
			{
				return token.IsConstInt() && token.GetConstInt(out localValue);
			});
			return localValue;
		}
	}
}
