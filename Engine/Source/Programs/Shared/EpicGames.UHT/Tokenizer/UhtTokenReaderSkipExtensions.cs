// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{
	/// <summary>
	/// Series of extensions to token reading that are far too specialized to be included in the reader.
	/// </summary>
	public static class UhtTokenReaderSkipExtensions
	{

		/// <summary>
		/// Skip a token regardless of the type.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader SkipOne(this IUhtTokenReader tokenReader)
		{
			tokenReader.PeekToken();
			tokenReader.ConsumeToken();
			return tokenReader;
		}

		/// <summary>
		/// Skip any block of tokens wrapped by the given token symbols
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating token (i.e. &quot;(&quot;)</param>
		/// <param name="terminator">Terminating token (i.e. &quot;)&quot;)</param>
		/// <param name="initialNesting">If true, start with an initial nesting count of one (assume we already parsed an initiator)</param>
		/// <param name="exceptionContext">Extra context for any errors</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Throw if end of file is reached</exception>
		public static IUhtTokenReader SkipBrackets(this IUhtTokenReader tokenReader, char initiator, char terminator, int initialNesting, object? exceptionContext = null)
		{
			int nesting = initialNesting;
			if (nesting == 0)
			{
				tokenReader.Require(initiator, exceptionContext);
				++nesting;
			}

			do
			{
				UhtToken skipToken = tokenReader.GetToken();
				if (skipToken.TokenType.IsEndType())
				{
					throw new UhtTokenException(tokenReader, skipToken, terminator, exceptionContext);
				}
				else if (skipToken.IsSymbol(initiator))
				{
					++nesting;
				}
				else if (skipToken.IsSymbol(terminator))
				{
					--nesting;
				}
			} while (nesting != 0);
			return tokenReader;
		}

		/// <summary>
		/// Skip any block of tokens wrapped by the given token symbols
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="initiator">Initiating token (i.e. &quot;(&quot;)</param>
		/// <param name="terminator">Terminating token (i.e. &quot;)&quot;)</param>
		/// <param name="initialNesting">If true, start with an initial nesting count of one (assume we already parsed an initiator)</param>
		/// <param name="callback">Invoked for each matching token</param>
		/// <param name="exceptionContext">Extra context for any errors</param>
		/// <returns>Token reader</returns>
		/// <exception cref="UhtTokenException">Throw if end of file is reached</exception>
		public static IUhtTokenReader SkipBrackets(this IUhtTokenReader tokenReader, char initiator, char terminator, int initialNesting, Action<UhtToken> callback, object? exceptionContext = null)
		{
			int nesting = initialNesting;
			if (nesting == 0)
			{
				ref UhtToken token = ref tokenReader.PeekToken();
				if (token.IsSymbol(initiator))
				{
					callback(token);
					tokenReader.ConsumeToken();
				}
				else
				{
					throw new UhtTokenException(tokenReader, token, initiator, exceptionContext);
				}
				++nesting;
			}

			do
			{
				UhtToken skipToken = tokenReader.GetToken();
				if (skipToken.TokenType.IsEndType())
				{
					throw new UhtTokenException(tokenReader, skipToken, terminator, exceptionContext);
				}
				else if (skipToken.IsSymbol(initiator))
				{
					++nesting;
				}
				else if (skipToken.IsSymbol(terminator))
				{
					--nesting;
				}
				callback(skipToken);
			} while (nesting != 0);
			return tokenReader;
		}

		/// <summary>
		/// Skip tokens until the given terminator is found.  The terminator will not be consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="terminator">Terminator to skip until</param>
		/// <param name="exceptionContext">Extra context for any exceptions</param>
		/// <returns>Token reader</returns>
		public static IUhtTokenReader SkipUntil(this IUhtTokenReader tokenReader, char terminator, object? exceptionContext = null)
		{
			while (true)
			{
				ref UhtToken skipToken = ref tokenReader.PeekToken();
				if (skipToken.TokenType.IsEndType())
				{
					throw new UhtTokenException(tokenReader, skipToken, terminator, exceptionContext);
				}
				else if (skipToken.IsSymbol(terminator))
				{
					break;
				}
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}
	}
}
