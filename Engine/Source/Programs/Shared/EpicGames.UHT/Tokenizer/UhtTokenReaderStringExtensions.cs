// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Collection of token reader exceptions for handling strings
	/// </summary>
	public static class UhtTokenReaderStringExtensions
	{
		/// <summary>
		/// Get the next token as a string.  If the next token is not a string, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="value">The string value of the token</param>
		/// <returns>True if the next token was an string, false if not.</returns>
		public static bool TryOptionalConstString(this IUhtTokenReader tokenReader, out StringView value)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsConstString())
			{
				value = token.GetTokenString();
				tokenReader.ConsumeToken();
				return true;
			}
			value = "";
			return false;
		}

		/// <summary>
		/// Get the next token as a string.  If the next token is not a string, no token is consumed.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <returns>True if the next token was an string, false if not.</returns>
		public static IUhtTokenReader OptionalConstString(this IUhtTokenReader tokenReader)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsConstString())
			{
				tokenReader.ConsumeToken();
			}
			return tokenReader;
		}

		/// <summary>
		/// Verify that the next token is a string.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>True if the next token was a string, false if not.</returns>
		public static IUhtTokenReader RequireConstString(this IUhtTokenReader tokenReader, object? exceptionContext)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsConstString())
			{
				tokenReader.ConsumeToken();
				return tokenReader;
			}
			throw new UhtTokenException(tokenReader, token, "constant string", exceptionContext);
		}

		/// <summary>
		/// Get the next token as a string.  If the next token is not a string, an exception is thrown
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the string.</returns>
		public static StringView GetConstString(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsConstString())
			{
				StringView output = token.GetTokenString();
				tokenReader.ConsumeToken();
				return output;
			}
			throw new UhtTokenException(tokenReader, token, "constant string", exceptionContext);
		}

		/// <summary>
		/// Get the next token as a quoted string.  If the next token is not a string, an exception is thrown.
		/// Character constants are not considered strings by this routine.
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the string.</returns>
		public static StringView GetConstQuotedString(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.TokenType == UhtTokenType.StringConst)
			{
				StringView output = token.Value;
				tokenReader.ConsumeToken();
				return output;
			}
			throw new UhtTokenException(tokenReader, token, "constant quoted string", exceptionContext);
		}

		/// <summary>
		/// Get a const string that can optionally be wrapped with a TEXT() macro
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the string</returns>
		public static StringView GetWrappedConstString(this IUhtTokenReader tokenReader, object? exceptionContext = null)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier("TEXT"))
			{
				tokenReader.ConsumeToken();
				tokenReader.Require('(');
				StringView output = tokenReader.GetConstString(exceptionContext);
				tokenReader.Require(')');
				return output;
			}
			else
			{
				return tokenReader.GetConstString(exceptionContext);
			}
		}

		/// <summary>
		/// Get a typed const string that can optionally be wrapped with a TEXT() macro
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="typeName">Expected type name</param>
		/// <param name="exceptionContext">If not null, an exception will be thrown with the given text as part of the message.</param>
		/// <returns>The value of the string</returns>
		public static StringView GetTypedConstString(this IUhtTokenReader tokenReader, string typeName, object? exceptionContext = null)
		{
			StringView output = new();
			if (tokenReader.TryOptional(typeName))
			{
				tokenReader.Require('(');
				if (!tokenReader.PeekToken().IsSymbol(')'))
				{
					output = tokenReader.GetWrappedConstString(exceptionContext);
				}
				tokenReader.Require(')');
			}
			else
			{
				output = tokenReader.GetWrappedConstString(exceptionContext);
			}
			return output;
		}
	}
}
