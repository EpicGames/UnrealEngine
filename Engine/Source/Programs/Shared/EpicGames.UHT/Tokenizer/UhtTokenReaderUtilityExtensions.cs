// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{
	/// <summary>
	/// Collection of assorted utility token reader extensions
	/// </summary>
	public static class UhtTokenReaderUtilityExtensions
	{
		private static readonly HashSet<StringView> s_skipDeclarationWarningStrings =
		[
			"GENERATED_BODY",
			"GENERATED_IINTERFACE_BODY",
			"GENERATED_UCLASS_BODY",
			"GENERATED_UINTERFACE_BODY",
			"GENERATED_USTRUCT_BODY",
		];

		private static readonly HashSet<StringView> s_skipTypeWarningStrings =
		[
			"RIGVM_METHOD",
			"UCLASS",
			"UDELEGATE",
			"UENUM",
			"UFUNCTION",
			"UINTERFACE",
			"UMETA",
			"UPROPERTY",
			"USTRUCT",
		];

		/// <summary>
		/// Try to parse an optional _API macro
		/// </summary>
		/// <param name="tokenReader">Token reader</param>
		/// <param name="apiMacroToken">_API macro parsed</param>
		/// <returns>True if an _API macro was parsed</returns>
		public static bool TryOptionalAPIMacro(this IUhtTokenReader tokenReader, out UhtToken apiMacroToken)
		{
			ref UhtToken token = ref tokenReader.PeekToken();
			if (token.IsIdentifier() && token.Value.Span.EndsWith("_API", StringComparison.Ordinal))
			{
				apiMacroToken = token;
				tokenReader.ConsumeToken();
				return true;
			}
			apiMacroToken = new UhtToken();
			return false;
		}

		/// <summary>
		/// Check to see if we have reached an identifier that is a UE macro
		/// </summary>
		/// <param name="messageSite">Used to generate error message</param>
		/// <param name="token">Token being tested</param>
		/// <param name="generateError">If true, generate an error.</param>
		/// <returns>True if a UE macro was found</returns>
		private static bool CheckForUEMacro(IUhtMessageSite messageSite, ref UhtToken token, bool generateError)
		{
			ReadOnlySpan<char> span = token.Value.Span;
			if (span[0] == 'G' || span[0] == 'R' || span[0] == 'U')
			{
				if (s_skipDeclarationWarningStrings.Contains(token.Value))
				{
					if (generateError)
					{
						messageSite.LogWarning($"The identifier \'{token.Value}\' was detected in a block being skipped. Was this intentional?");
					}
					return true;
				}
				if (s_skipTypeWarningStrings.Contains(token.Value))
				{
					if (generateError)
					{
						messageSite.LogDeprecation($"The identifier \'{token.Value}\' was detected in a block being skipped and is now being flagged. This will be a warning in a future version of Unreal.");
					}
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Given a declaration/statement that starts with the given token, skip the declaration in the header.
		/// </summary>
		/// <param name="topScope">Current scope being processed</param>
		/// <param name="token">Token that started the process</param>
		/// <returns>true if there could be more header to process, false if the end was reached.</returns>
		public static void SkipDeclaration(this UhtParsingScope topScope, UhtToken token)
		{
			IUhtTokenReader tokenReader = topScope.TokenReader;

			// Store the current value of PrevComment so it can be restored after we parsed everything.
			using UhtTokenDisableComments disableComments = new(tokenReader);

			// Processing must be adjusted if we think we are processing a class or struct.
			// Macros are tricky because they usually don't end in a ";".  So we have to terminate 
			// when we find the ')'.
			// Type definitions might have a trailing variable declaration.
			bool mightBeMacro = false;
			bool mightBeClassOrStruct = false;
			if (token.IsIdentifier())
			{
				mightBeMacro = true;
				if (token.IsValue("class") || token.IsValue("struct"))
				{
					mightBeClassOrStruct = true;
				}
			}

			bool peekedToken = false;
			while (!token.TokenType.IsEndType())
			{
				if (peekedToken)
				{

					// For the moment, warn on anything skipped.  
					CheckForUEMacro(tokenReader, ref token, true);
					//if (CheckForUEMacro(tokenReader, ref token, false))
					//{
					//	break;
					//}

					// Check for poor termination of a macro
					if (token.IsSymbol('}'))
					{
						break;
					}
					tokenReader.ConsumeToken();
				}

				if (token.IsIdentifier())
				{
					if (token.IsValue("PURE_VIRTUAL"))
					{
						tokenReader.SkipBrackets('(', ')', 0, (x) => { CheckForUEMacro(tokenReader, ref x, true); });
						break;
					}
					else if (mightBeMacro && (!IsProbablyAMacro(token.Value) || token.IsValue("DECLARE_FUNCTION")))
					{
						mightBeMacro = false;
					}
				}
				else if (token.IsSymbol())
				{
					if (token.IsValue(';'))
					{
						break;
					}
					else if (token.IsValue('('))
					{
						mightBeClassOrStruct = false; // some type of function declaration
						tokenReader.SkipBrackets('(', ')', 1, (x) => { CheckForUEMacro(tokenReader, ref x, true); });
						// Could be a class declaration in all capitals, and not a macro
						if (mightBeMacro && !tokenReader.TryPeekOptional('{'))
						{
							break;
						}
					}
					else if (token.IsValue('{'))
					{
						tokenReader.SkipBrackets('{', '}', 1, (x) => { CheckForUEMacro(tokenReader, ref x, true); });
						if (mightBeClassOrStruct)
						{
							if (tokenReader.TryOptionalIdentifier())
							{
								tokenReader.Require(';');
							}
						}
						break;
					}
					mightBeMacro = false;
				}
				else
				{
					mightBeMacro = false;
				}
				peekedToken = true;
				token = tokenReader.PeekToken();
			}

			// C++ allows any number of ';' after member declaration/definition.
			while (tokenReader.TryOptional(';'))
			{
			}
		}

		private static bool IsProbablyAMacro(StringView identifier)
		{
			ReadOnlySpan<char> span = identifier.Span;
			if (span.Length == 0)
			{
				return false;
			}

			// Macros must start with a capitalized alphanumeric character or underscore
			char firstChar = span[0];
			if (firstChar != '_' && (firstChar < 'A' || firstChar > 'Z'))
			{
				return false;
			}

			// Test for known delegate and event macros.
			if (span.StartsWith("DECLARE_MULTICAST_DELEGATE", StringComparison.Ordinal) ||
				span.StartsWith("DECLARE_DELEGATE", StringComparison.Ordinal) ||
				span.StartsWith("DECLARE_EVENT", StringComparison.Ordinal))
			{
				return true;
			}

			// Failing that, we'll guess about it being a macro based on it being a fully-capitalized identifier.
			foreach (char ch in span[1..])
			{
				if (ch != '_' && (ch < 'A' || ch > 'Z') && (ch < '0' || ch > '9'))
				{
					return false;
				}
			}

			return true;
		}
	}
}
