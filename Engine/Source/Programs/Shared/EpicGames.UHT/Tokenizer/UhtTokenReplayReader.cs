// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Tokenizer
{

	/// <summary>
	/// Token reader to replay previously recorded token stream
	/// </summary>
	public sealed class UhtTokenReplayReader : IUhtTokenReader, IUhtMessageLineNumber
	{
		const int MaxSavedStates = 2;
		private struct SavedState
		{
			public int TokenIndex { get; set; }
			public bool HasToken { get; set; }
		}

		private IUhtMessageSite _messageSite;
		private ReadOnlyMemory<UhtToken> _tokens;
		private ReadOnlyMemory<char> _data;
		private int _currentTokenIndex = -1;
		private bool _hasToken = false;
		private UhtToken _currentToken = new();
		private readonly SavedState[] _savedStates = new SavedState[MaxSavedStates];
		private int _savedStateCount = 0;
		private UhtTokenType _endTokenType = UhtTokenType.EndOfFile;

		/// <summary>
		/// Construct new token reader
		/// </summary>
		/// <param name="messageSite">Message site for generating errors</param>
		/// <param name="data">Complete data for the token (i.e. original source)</param>
		/// <param name="tokens">Tokens to replay</param>
		/// <param name="endTokenType">Token type to return when end of tokens reached</param>
		public UhtTokenReplayReader(IUhtMessageSite messageSite, ReadOnlyMemory<char> data, ReadOnlyMemory<UhtToken> tokens, UhtTokenType endTokenType)
		{
			_messageSite = messageSite;
			_tokens = tokens;
			_data = data;
			_endTokenType = endTokenType;
			_currentToken = new UhtToken(endTokenType);
		}

		/// <summary>
		/// Construct token replay reader intended for caching.  Use Reset method to prepare it for use
		/// </summary>
		public UhtTokenReplayReader()
		{
			_messageSite = new UhtEmptyMessageSite();
			_tokens = Array.Empty<UhtToken>().AsMemory();
			_data = Array.Empty<char>().AsMemory();
		}

		/// <summary>
		/// Reset a cached replay reader for replaying a new stream of tokens
		/// </summary>
		/// <param name="messageSite">Message site for generating errors</param>
		/// <param name="data">Complete data for the token (i.e. original source)</param>
		/// <param name="tokens">Tokens to replay</param>
		/// <param name="endTokenType">Token type to return when end of tokens reached</param>
		/// <returns>The replay reader</returns>
		public UhtTokenReplayReader Reset(IUhtMessageSite messageSite, ReadOnlyMemory<char> data, ReadOnlyMemory<UhtToken> tokens, UhtTokenType endTokenType)
		{
			_messageSite = messageSite;
			_tokens = tokens;
			_data = data;
			_currentTokenIndex = -1;
			_hasToken = false;
			_currentToken = new UhtToken(endTokenType);
			_savedStateCount = 0;
			_endTokenType = endTokenType;
			return this;
		}

		#region IUHTMessageSite Implementation
		IUhtMessageSession IUhtMessageSite.MessageSession => _messageSite.MessageSession;
		IUhtMessageSource? IUhtMessageSite.MessageSource => _messageSite.MessageSource;
		IUhtMessageLineNumber? IUhtMessageSite.MessageLineNumber => this;
		#endregion

		#region ITokenReader Implementation
		/// <inheritdoc/>
		public bool IsEOF
		{
			get
			{
				if (!_hasToken)
				{
					GetTokenInternal();
				}
				return _currentTokenIndex == _tokens.Span.Length;
			}
		}

		/// <inheritdoc/>
		public int InputPos
		{
			get
			{
				if (!_hasToken)
				{
					GetTokenInternal();
				}
				return _currentToken.InputStartPos;
			}
		}

		/// <inheritdoc/>
		public int InputLine
		{
			get
			{
				if (!_hasToken)
				{
					GetTokenInternal();
				}
				return _currentToken.InputLine;
			}
			set => throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public IUhtTokenPreprocessor? TokenPreprocessor { get => throw new NotImplementedException(); set => throw new NotImplementedException(); }

		/// <inheritdoc/>
		public int LookAheadEnableCount { get => throw new NotImplementedException(); set => throw new NotImplementedException(); }

		/// <inheritdoc/>
		public ReadOnlySpan<StringView> Comments => throw new NotImplementedException();

		/// <inheritdoc/>
		public void ClearComments()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void ConsumeToken()
		{
			_hasToken = false;
		}

		/// <inheritdoc/>
		public void DisableComments()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void EnableComments()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void CommitPendingComments()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public UhtToken GetLine()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public StringView GetRawString(char terminator, UhtRawStringOptions options)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public StringView GetStringView(int startPos, int count)
		{
			return new StringView(_data, startPos, count);
		}

		/// <inheritdoc/>
		public UhtToken GetToken()
		{
			UhtToken token = PeekToken();
			ConsumeToken();
			return token;
		}

		/// <inheritdoc/>
		public int GetReplayTokenIndex()
		{
			return _hasToken ? _currentTokenIndex : _currentTokenIndex + 1;
		}

		/// <inheritdoc/>
		public bool IsFirstTokenInLine(ref UhtToken token)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public ref UhtToken PeekToken()
		{
			if (!_hasToken)
			{
				GetTokenInternal();
			}
			return ref _currentToken;
		}

		/// <inheritdoc/>
		public void SkipWhitespaceAndComments()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void SaveState()
		{
			if (_savedStateCount == MaxSavedStates)
			{
				throw new UhtIceException("Token reader saved states full");
			}
			_savedStates[_savedStateCount] = new SavedState { TokenIndex = _currentTokenIndex, HasToken = _hasToken };
			++_savedStateCount;
		}

		/// <inheritdoc/>
		public void RestoreState()
		{
			if (_savedStateCount == 0)
			{
				throw new UhtIceException("Attempt to restore a state when none have been saved");
			}

			--_savedStateCount;
			_currentTokenIndex = _savedStates[_savedStateCount].TokenIndex;
			_hasToken = _savedStates[_savedStateCount].HasToken;
			if (_hasToken)
			{
				_currentToken = _tokens.Span[_currentTokenIndex];
			}
		}

		/// <inheritdoc/>
		public void AbandonState()
		{
			if (_savedStateCount == 0)
			{
				throw new UhtIceException("Attempt to abandon a state when none have been saved");
			}

			--_savedStateCount;
		}

		/// <inheritdoc/>
		public void EnableRecording()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void DisableRecording()
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public void RecordToken(ref UhtToken token)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public List<UhtToken> RecordedTokens => throw new NotImplementedException();
		#endregion

		#region IUHTMessageLineNumber implementation
		int IUhtMessageLineNumber.MessageLineNumber
		{
			get
			{
				if (_tokens.Length == 0)
				{
					return -1;
				}
				if (_currentTokenIndex < 0)
				{
					return _tokens.Span[0].InputLine;
				}
				if (_currentTokenIndex < _tokens.Length)
				{
					return _tokens.Span[_currentTokenIndex].InputLine;
				}
				return _tokens.Span[_tokens.Length - 1].InputLine;
			}
		}
		#endregion

		private ref UhtToken GetTokenInternal()
		{
			if (_currentTokenIndex < _tokens.Length)
			{
				++_currentTokenIndex;
			}
			if (_currentTokenIndex < _tokens.Length)
			{
				_currentToken = _tokens.Span[_currentTokenIndex];
			}
			else if (_tokens.Length == 0)
			{
				_currentToken = new UhtToken();
			}
			else
			{
				UhtToken lastToken = _tokens.Span[_tokens.Length - 1];
				int endPos = lastToken.InputEndPos;
				_currentToken = new UhtToken(_endTokenType, endPos, lastToken.InputLine, endPos, lastToken.InputLine, new StringView(_data[endPos..endPos]));
			}
			_hasToken = true;
			return ref _currentToken;
		}
	}

	/// <summary>
	/// Borrow a thread specific token replay reader
	/// </summary>
	public readonly struct UhtTokenReplayReaderBorrower : IDisposable
	{
		private static readonly ThreadLocal<Stack<UhtTokenReplayReader>> s_tlsStack = new(() => new Stack<UhtTokenReplayReader>());

		private readonly UhtTokenReplayReader _reader;

		/// <summary>
		/// The borrowed reader
		/// </summary>
		public readonly IUhtTokenReader Reader => _reader;

		/// <summary>
		/// Borrow a thread specific replay reader
		/// </summary>
		/// <param name="messageSite">Location for messages</param>
		/// <param name="data">Source being parsed</param>
		/// <param name="tokens">Tokens to be replayed</param>
		/// <param name="endTokenType">Token type when end of tokens reached</param>
		/// <exception cref="UhtIceException"></exception>
		public UhtTokenReplayReaderBorrower(IUhtMessageSite messageSite, ReadOnlyMemory<char> data, ReadOnlyMemory<UhtToken> tokens, UhtTokenType endTokenType)
		{
			Stack<UhtTokenReplayReader> stack = s_tlsStack.Value!;
			_reader = stack.Count > 0 ? _reader = stack.Pop() : new();
			_reader.Reset(messageSite, data, tokens, endTokenType);
		}

		/// <inheritdoc/>
		public readonly void Dispose()
		{
			s_tlsStack.Value!.Push(_reader);
		}
	}
}
