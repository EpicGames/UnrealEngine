// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FAnsiStrProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "AnsiStrProperty", IsProperty = true)]
	public class UhtAnsiStrProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "AnsiStrProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FAnsiString";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.EngineClass;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FAnsiStrPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::AnsiStr";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtAnsiStrProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			// Other caps not supported until engine support catches up
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef; // | UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
															  //UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtAnsiStrProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("TEXT(\"\")");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			StringView value = defaultValueReader.GetTypedConstString("FAnsiString");
			innerDefaultValue.Append(value);
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtAnsiStrProperty;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest))
				{
					if (RefQualifier != UhtPropertyRefQualifier.ConstRef && !IsStaticArray)
					{
						this.LogError("Replicated FAnsiString parameters must be passed by const reference");
					}
				}
			}
		}

		[UhtPropertyType(Keyword = "FAnsiString")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtAnsiStrProperty? AnsiStrProperty(UhtPropertyResolveArgs args)
		{
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			if (!args.SkipExpectedType())
			{
				return null;
			}
			UhtAnsiStrProperty property = new(propertySettings);
			if (property.PropertyCategory != UhtPropertyCategory.Member)
			{
				if (tokenReader.TryOptional('&'))
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
					{
						property.PropertyFlags &= ~EPropertyFlags.ConstParm;

						// We record here that we encountered a const reference, because we need to remove that information from flags for code generation purposes.
						property.RefQualifier = UhtPropertyRefQualifier.ConstRef;
					}
					else
					{
						property.PropertyFlags |= EPropertyFlags.OutParm;

						// And we record here that we encountered a non-const reference here too.
						property.RefQualifier = UhtPropertyRefQualifier.NonConstRef;
					}
				}
			}
			return property;
		}
	}
}
