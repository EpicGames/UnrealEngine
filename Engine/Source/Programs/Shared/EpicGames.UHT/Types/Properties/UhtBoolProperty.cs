// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Type of boolean
	/// </summary>
	public enum UhtBoolType
	{

		/// <summary>
		/// Native bool
		/// </summary>
		Native,

		/// <summary>
		/// Used for all bitmask uint booleans
		/// </summary>
		UInt8,

		/// <summary>
		/// Currently unused
		/// </summary>
		UInt16,

		/// <summary>
		/// Currently unused
		/// </summary>
		UInt32,

		/// <summary>
		/// Currently unused
		/// </summary>
		UInt64,
	}

	/// <summary>
	/// Represents the FBoolProperty engine type
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "BoolProperty", IsProperty = true)]
	public class UhtBoolProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "BoolProperty";

		/// <inheritdoc/>
		protected override string CppTypeText
		{
			get
			{
				switch (BoolType)
				{
					case UhtBoolType.Native:
						return "bool";
					case UhtBoolType.UInt8:
						return "uint8";
					case UhtBoolType.UInt16:
						return "uint16";
					case UhtBoolType.UInt32:
						return "uint32";
					case UhtBoolType.UInt64:
						return "uint64";
					default:
						throw new UhtIceException("Unexpected boolean type");
				}
			}
		}

		/// <inheritdoc/>
		protected override string PGetMacroText
		{
			get
			{
				switch (BoolType)
				{
					case UhtBoolType.Native:
						return "UBOOL";
					case UhtBoolType.UInt8:
						return "UBOOL8";
					case UhtBoolType.UInt16:
						return "UBOOL16";
					case UhtBoolType.UInt32:
						return "UBOOL32";
					case UhtBoolType.UInt64:
						return "UBOOL64";
					default:
						throw new UhtIceException("Unexpected boolean type");
				}
			}
		}

		/// <summary>
		/// If true, the boolean is a native bool and not a UBOOL
		/// </summary>
		protected bool IsNativeBool => BoolType == UhtBoolType.Native;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FBoolPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => IsNativeBool ?
			"UECodeGen_Private::EPropertyGenFlags::Bool | UECodeGen_Private::EPropertyGenFlags::NativeBool" :
			"UECodeGen_Private::EPropertyGenFlags::Bool";

		/// <inheritdoc/>
		protected override bool CodeGenParamsAppendOffset => false;

		/// <summary>
		/// Type of the boolean
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtBoolType BoolType { get; }

		/// <summary>
		/// Return the engine name without and 'b' prefixes
		/// </summary>
		[JsonIgnore]
		public override string StrippedEngineName
		{
			get
			{
				string result = base.StrippedEngineName;
				if (result.StartsWith('b'))
				{
					result = result[1..];
				}
				return result;
			}
		}

		/// <summary>
		/// Construct a new boolean property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="boolType">Type of the boolean</param>
		public UhtBoolProperty(UhtPropertySettings propertySettings, UhtBoolType boolType) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.RequiresNullConstructorArg | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM | UhtPropertyCaps.SupportsVerse;
			if (boolType == UhtBoolType.Native || boolType == UhtBoolType.UInt8)
			{
				PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn;
			}
			BoolType = boolType;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtBoolProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			BoolType = (UhtBoolType)reader.ReadUInt8();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteUInt8((byte)BoolType);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument = false)
		{
			switch (textType)
			{
				case UhtPropertyTextType.Generic:
				case UhtPropertyTextType.Construction:
				case UhtPropertyTextType.FunctionThunkParameterArrayType:
				case UhtPropertyTextType.FunctionThunkRetVal:
				case UhtPropertyTextType.RigVMType:
				case UhtPropertyTextType.ExportMember:
				case UhtPropertyTextType.GenericFunctionArgOrRetVal:
					builder.Append(CppTypeText);
					break;

				default:
				case UhtPropertyTextType.Sparse:
				case UhtPropertyTextType.SparseShort:
				case UhtPropertyTextType.GenericFunctionArgOrRetValImpl:
				case UhtPropertyTextType.ClassFunctionArgOrRetVal:
				case UhtPropertyTextType.EventFunctionArgOrRetVal:
				case UhtPropertyTextType.InterfaceFunctionArgOrRetVal:
				case UhtPropertyTextType.EventParameterMember:
				case UhtPropertyTextType.EventParameterFunctionMember:
				case UhtPropertyTextType.GetterSetterArg:
					builder.Append("bool");
					break;

				case UhtPropertyTextType.VerseMangledType:
					builder.Append("logic");
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendCommonDecl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return AppendSetBit(builder, context, identifier, tabs);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			builder.Append("sizeof(").Append(CppTypeText).Append("), ");

			if (Outer == context.OuterStruct)
			{
				builder.Append($"sizeof({context.OuterIdentifier}), &{identifier.MakeStatics()}_SetBit, ");
			}
			else
			{
				builder.Append("0, nullptr, ");
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefImpl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, Action<StringBuilder>? outerFunc, string? offset, int tabs)
		{
			if (!String.IsNullOrEmpty(offset) && IsBitfield)
			{
				throw new ArgumentException("Cannot create bool property initialization for a bitfield at a specified offset");
			}
			return base.AppendConstInitDefImpl(builder, context, identifier, outerFunc, IsBitfield ? "0" : offset, tabs);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			builder.Append("(int32)sizeof(").Append(CppTypeText).Append("), ");
			builder.Append(IsNativeBool ? "true, " : "false, ");
			if (Outer == context.OuterStruct)
			{
				builder.Append($"&{identifier.MakeStatics()}_SetBit, ");
			}
			else
			{
				builder.Append("nullptr, ");
			}
			return builder;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFullDecl(StringBuilder builder, UhtPropertyTextType textType, bool skipParameterName = false)
		{
			AppendText(builder, textType);

			//@todo we currently can't have out bools.. so this isn't really necessary, but eventually out bools may be supported, so leave here for now
			if (textType.IsParameter() && PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
			{
				builder.Append('&');
			}

			builder.Append(' ');

			if (!skipParameterName)
			{
				builder.Append(SourceName);
			}

			if (ArrayDimensions != null)
			{
				builder.Append('[').Append(ArrayDimensions).Append(']');
			}
			else if (textType == UhtPropertyTextType.ExportMember && !IsNativeBool)
			{
				builder.Append(":1");
			}
			return builder;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("false");
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendTypedVerseArgumentUnmarshal(StringBuilder builder)
		{
			return builder.Append($"V_MARSHAL_PARAM_BOOL({SourceName})");
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			UhtToken identifier = defaultValueReader.GetIdentifier();
			if (identifier.IsValue("true") || identifier.IsValue("false"))
			{
				innerDefaultValue.Append(identifier.Value.ToString());
				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			// We don't test BoolType.
			return other is UhtBoolProperty;
		}

		private StringBuilder AppendSetBit(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			if (Outer == context.OuterStruct)
			{
				builder.AppendTabs(tabs).Append($"static void {identifier}_SetBit(void* Obj)\r\n");
				builder.AppendTabs(tabs).Append("{\r\n");
				builder.AppendTabs(tabs + 1).Append($"(({context.OuterIdentifier}*)Obj)->{SourceName} = 1;\r\n");
				builder.AppendTabs(tabs).Append("}\r\n");
			}
			return builder;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "bool", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtBoolProperty? BoolProperty(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.IsBitfield)
			{
				args.TokenReader.LogError("bool bitfields are not supported.");
				return null;
			}
			return new UhtBoolProperty(args.PropertySettings, UhtBoolType.Native);
		}
		#endregion
	}
}
