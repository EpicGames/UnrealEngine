// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static UhtExpressionFactory;

	/// <summary>
	/// The form that the byte was defined
	/// </summary>
	public enum UhtByteCppForm
	{

		/// <summary>
		/// uint8
		/// </summary>
		UnsignedInt8,

		/// <summary>
		/// UTF8CHAR
		/// </summary>
		Utf8Char,
	}

	/// <summary>
	/// FByteProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "ByteProperty", IsProperty = true)]
	public class UhtByteProperty : UhtNumericProperty
	{
		/// <summary>
		/// Referenced enumeration (TEnumAsByte)
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtEnum>))]
		public UhtEnum? Enum { get; set; }

		/// <inheritdoc/>
		public override string EngineClassName => "ByteProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => CppForm == UhtByteCppForm.Utf8Char ? "UTF8CHAR" : "uint8";

		/// <inheritdoc/>
		protected override string PGetMacroText
		{
			get
			{
				if (Enum != null && Enum.CppForm == UhtEnumCppForm.EnumClass)
				{
					return "ENUM";
				}
				else if (CppForm == UhtByteCppForm.Utf8Char)
				{
					return "UTF8CHAR";
				}
				else
				{
					return "PROPERTY";
				}
			}
		}

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => Enum == null || Enum.CppForm != UhtEnumCppForm.EnumClass ? UhtPGetArgumentType.EngineClass : UhtPGetArgumentType.TypeText;

		/// <summary>
		/// Form in which the uint8 was specified
		/// </summary>
		public UhtByteCppForm CppForm { get; init; } = UhtByteCppForm.UnsignedInt8;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FBytePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Byte";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">For the byte appears in code</param>
		/// <param name="enumObj">Optional referenced enum</param>
		public UhtByteProperty(UhtPropertySettings propertySettings, UhtByteCppForm cppForm = UhtByteCppForm.UnsignedInt8, UhtEnum? enumObj = null) : base(propertySettings)
		{
			CppForm = cppForm;
			Enum = enumObj;
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
			if (cppForm == UhtByteCppForm.Utf8Char)
			{
				PropertyCaps |= UhtPropertyCaps.SupportsVerse;
			}
			if (Enum != null)
			{
				PropertyCaps |= UhtPropertyCaps.IsRigVMEnum;
				PropertyCaps |= UhtPropertyCaps.IsRigVMEnumAsByte;
			}
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtByteProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			Enum = reader.ReadType() as UhtEnum;
			CppForm = (UhtByteCppForm)reader.ReadUInt8();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(Enum);
			writer.WriteUInt8((byte)CppForm);
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			if (Enum != null)
			{
				yield return Enum;
			}
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(Enum, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			if (Enum != null)
			{
				return UhtEnumProperty.AppendEnumText(builder, this, Enum, textType, isTemplateArgument);
			}
			else
			{
				switch (textType)
				{
					case UhtPropertyTextType.VerseMangledType:
						if (CppForm == UhtByteCppForm.Utf8Char)
						{
							return builder.Append("char");
						}
						else
						{
							return base.AppendText(builder, textType, isTemplateArgument);
						}

					default:
						return base.AppendText(builder, textType, isTemplateArgument);
				}
			}
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Enum, UhtSingletonType.Registered, true);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Enum, true)}, ");
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			builder.AppendObjectHash(startingLength, this, context, Enum);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterArg(StringBuilder builder)
		{
			if (Enum != null)
			{
				return UhtEnumProperty.AppendEnumFunctionThunkParameterArg(builder, this, Enum);
			}
			else
			{
				return base.AppendFunctionThunkParameterArg(builder);
			}
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			if (Enum != null)
			{
				return UhtEnumProperty.SanitizeEnumDefaultValue(this, Enum, defaultValueReader, innerDefaultValue);
			}

			int value = defaultValueReader.GetConstIntExpression();
			innerDefaultValue.Append(value);
			return value >= Byte.MinValue && value <= Byte.MaxValue;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtByteProperty otherByte)
			{
				return Enum == otherByte.Enum;
			}
			else if (other is UhtEnumProperty otherEnum)
			{
				return Enum == otherEnum.Enum;
			}
			return false;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.BlueprintCallable))
			{
				if (Enum != null && Enum.CppForm == UhtEnumCppForm.EnumClass && Enum.UnderlyingType != UhtEnumUnderlyingType.Uint8)
				{
					this.LogError("Invalid enum param for Blueprints - currently only uint8 supported");
				}
			}
		}

		#region Keyword
		[UhtPropertyType(Keyword = "uint8", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? UInt8Property(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.IsBitfield)
			{
				return new UhtBoolProperty(args.PropertySettings, UhtBoolType.UInt8);
			}
			else
			{
				return new UhtByteProperty(args.PropertySettings);
			}
		}

		[UhtPropertyType(Keyword = "verse::char8", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseChar8Property(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.IsBitfield)
			{
				return new UhtBoolProperty(args.PropertySettings, UhtBoolType.UInt8);
			}
			else
			{
				UhtByteProperty property = new(args.PropertySettings, UhtByteCppForm.Utf8Char);
				return property;
			}
		}
		#endregion
	}
}
