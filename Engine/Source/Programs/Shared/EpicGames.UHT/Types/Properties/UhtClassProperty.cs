// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FClassProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "ClassProperty", IsProperty = true)]
	public class UhtClassProperty : UhtObjectProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "ClassProperty";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Source code form of the property</param>
		/// <param name="referencedClass">Referenced class</param>
		/// <param name="extraFlags">Extra flags to apply to the property.</param>
		public UhtClassProperty(UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass, EPropertyFlags extraFlags = EPropertyFlags.None)
			: this(new UhtNoValidateConstruct { }, propertySettings, cppForm, referencedClass, extraFlags)
		{
			if (!cppForm.IsValidForClassProperty())
			{
				throw new UhtIceException($"Improper UhtObjectCppForm.{cppForm} for an UhtClassProperty");
			}
		}

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="_">Avoid validation</param>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Source code form of the property</param>
		/// <param name="referencedClass">Referenced class</param>
		/// <param name="extraFlags">Extra flags to apply to the property.</param>
		protected UhtClassProperty(UhtNoValidateConstruct _, UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass, EPropertyFlags extraFlags)
			: base(_, propertySettings, cppForm, referencedClass, extraFlags)
		{
			PropertyCaps |= UhtPropertyCaps.CanHaveConfig;
			PropertyCaps &= ~(UhtPropertyCaps.CanBeInstanced);
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtClassProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			if (addForwardDeclarations && MetaClass != null)
			{
				collector.AddForwardDeclaration(MetaClass);
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.Generic:
				case UhtPropertyTextType.Sparse:
				case UhtPropertyTextType.SparseShort:
				case UhtPropertyTextType.GenericFunctionArgOrRetVal:
				case UhtPropertyTextType.GenericFunctionArgOrRetValImpl:
				case UhtPropertyTextType.ClassFunctionArgOrRetVal:
				case UhtPropertyTextType.EventFunctionArgOrRetVal:
				case UhtPropertyTextType.InterfaceFunctionArgOrRetVal:
				case UhtPropertyTextType.ExportMember:
				case UhtPropertyTextType.Construction:
				case UhtPropertyTextType.RigVMType:
				case UhtPropertyTextType.GetterSetterArg:
					AppendTemplateType(builder);
					break;

				case UhtPropertyTextType.EventParameterMember:
				case UhtPropertyTextType.EventParameterFunctionMember:
					if (CppForm != UhtObjectCppForm.NativeClass)
					{
						AppendTemplateType(builder);
					}
					else
					{
						builder.Append("UClass*");
					}
					break;

				case UhtPropertyTextType.FunctionThunkRetVal:
					if (PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
					{
						builder.Append("const ");
					}
					if (CppForm != UhtObjectCppForm.NativeClass)
					{
						AppendTemplateType(builder);
					}
					else
					{
						builder.Append("UClass*");
					}
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					if ((PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm) && CppForm != UhtObjectCppForm.NativeClass) || isTemplateArgument)
					{
						AppendTemplateType(builder);
					}
					else
					{
						builder.Append(Class.Namespace.FullSourceName).Append(Class.SourceName);
					}
					break;

				case UhtPropertyTextType.FunctionThunkParameterArrayType:
					AppendTemplateType(builder);
					break;

				case UhtPropertyTextType.VerseMangledType:
					AppendVerseMangledType(builder, ReferencedClass);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			AppendParamsDefRef(builder, context, Class, Exporters.CodeGen.UhtSingletonType.Unregistered);
			AppendParamsDefRef(builder, context, MetaClass, Exporters.CodeGen.UhtSingletonType.Unregistered);
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Class, true)}, {ConstInitSingletonRef(context, MetaClass, true)}, ");
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);

			// UFunctions with a smart pointer as input parameter wont compile anyway, because of missing P_GET_... macro.
			// UFunctions with a smart pointer as return type will crash when called via blueprint, because they are not supported in VM.
			if (!options.HasAnyFlags(UhtValidationOptions.IsKey) && PropertyCategory != UhtPropertyCategory.Member && CppForm == UhtObjectCppForm.TObjectPtrClass)
			{
				outerStruct.LogError("UFunctions cannot take a TObjectPtr as a function parameter or return value.");
			}
		}
	}
}
