// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Common base class for containers with a value
	/// </summary>
	public abstract class UhtContainerBaseProperty : UhtProperty
	{

		/// <inheritdoc/>
		public UhtProperty ValueProperty { get; set; }

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="value">Value property</param>
		protected UhtContainerBaseProperty(UhtPropertySettings propertySettings, UhtProperty value) : base(propertySettings)
		{
			ValueProperty = value;
			PropertyCaps = (PropertyCaps & ~(UhtPropertyCaps.CanBeInstanced | UhtPropertyCaps.CanHaveConfig)) |
				(ValueProperty.PropertyCaps & (UhtPropertyCaps.CanBeInstanced | UhtPropertyCaps.CanHaveConfig));
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtContainerBaseProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ValueProperty = (reader.ReadType() as UhtProperty)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(ValueProperty);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			ValueProperty.CollectReferencesInternal(collector, addForwardDeclarations, true);
		}

		/// <inheritdoc/>
		protected override bool NeedsGCBarrierWhenPassedToFunctionImpl(UhtFunction function)
		{
			return ValueProperty.NeedsGCBarrierWhenPassedToFunction(function);
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);
			ValueProperty.Validate(outerStruct, outermostProperty, options | UhtValidationOptions.IsValue);
		}

		/// <summary>
		/// Propagate flags and meta data to/from child properties
		/// </summary>
		/// <param name="container">Container property</param>
		/// <param name="metaData">Meta data</param>
		/// <param name="inner">Inner property</param>
		protected static void PropagateFlagsFromInnerAndHandlePersistentInstanceMetadata(UhtProperty container, UhtMetaData? metaData, UhtProperty inner)
		{
			// Copy some of the property flags to the container property.
			if (inner.PropertyFlags.HasAnyFlags(EPropertyFlags.ContainsInstancedReference | EPropertyFlags.InstancedReference))
			{
				container.PropertyFlags |= EPropertyFlags.ContainsInstancedReference;
				container.PropertyFlags &= ~(EPropertyFlags.InstancedReference | EPropertyFlags.PersistentInstance); //this was propagated to the inner

				if (metaData != null && inner.PropertyFlags.HasAnyFlags(EPropertyFlags.PersistentInstance))
				{
					inner.MetaData.Add(metaData);
				}
			}
		}

		/// <summary>
		/// Resolve the child and return any new flags
		/// </summary>
		/// <param name="child">Child to resolve</param>
		/// <param name="phase">Resolve phase</param>
		/// <returns>And new flags</returns>
		protected static EPropertyFlags ResolveAndReturnNewFlags(UhtProperty child, UhtResolvePhase phase)
		{
			EPropertyFlags oldFlags = child.PropertyFlags;
			child.Resolve(phase);
			EPropertyFlags newFlags = child.PropertyFlags;
			return newFlags & ~oldFlags;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return ValueProperty.ScanForInstancedReferenced(deepScan);
		}

		///<inheritdoc/>
		public override bool IsAllowedInOptionalClass([NotNullWhen(false)] out string? propPath)
		{
			return ValueProperty.IsAllowedInOptionalClass(out propPath);
		}
	}
}
