// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FDelegatePropertyDelegateProperty
	/// </summary>
	[UhtEngineClass(Name = "DelegateProperty", IsProperty = true)]
	public class UhtDelegateProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "DelegateProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FScriptDelegate";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.EngineClass;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FDelegatePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Delegate";

		/// <summary>
		/// Referenced function
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtFunction>))]
		public UhtFunction Function { get; set; }

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="function">Referenced function</param>
		public UhtDelegateProperty(UhtPropertySettings propertySettings, UhtFunction function) : base(propertySettings)
		{
			Function = function;
			HeaderFile.AddReferencedHeader(function);
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtDelegateProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			Function = (reader.ReadType() as UhtFunction)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(Function);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					PropertyFlags |= EPropertyFlags.InstancedReference & ~DisallowPropertyFlags;
					break;
			}
			return results;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return true;
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(Function, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.ExportMember:
				case UhtPropertyTextType.RigVMType:
				case UhtPropertyTextType.EventParameterFunctionMember:
					builder.Append(CppTypeText);
					break;

				default:
					builder.Append(Function.SourceName);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Function, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Function, true)}, ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterArg(StringBuilder builder)
		{
			return builder.Append(Function.SourceName).Append('(').AppendFunctionThunkParameterName(this).Append(')');
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			builder.AppendObjectHash(startingLength, this, context, Function);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.AppendPropertyText(this, UhtPropertyTextType.Construction).Append("()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtDelegateProperty otherObject)
			{
				return Function == otherObject.Function;
			}
			return false;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest))
				{
					if (!PropertyFlags.HasAnyFlags(EPropertyFlags.RepSkip))
					{
						this.LogError("Service request functions cannot contain delegate parameters, unless marked NotReplicated");
					}
				}
				else
				{
					this.LogError("Replicated functions cannot contain delegate parameters (this would be insecure)");
				}
			}
		}
	}
}
