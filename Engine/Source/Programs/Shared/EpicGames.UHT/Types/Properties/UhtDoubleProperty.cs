// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FDoubleProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "DoubleProperty", IsProperty = true)]
	public class UhtDoubleProperty : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "DoubleProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "double";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FDoublePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Double";

		/// <summary>
		/// Create new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtDoubleProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM | UhtPropertyCaps.SupportsVerse;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtDoubleProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument = false)
		{
			switch (textType)
			{
				case UhtPropertyTextType.VerseMangledType:
					builder.Append("float");
					return builder;

				default:
					return base.AppendText(builder, textType, isTemplateArgument);
			}
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, "{0:F6}", defaultValueReader.GetConstFloatExpression());
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtDoubleProperty;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "double", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtDoubleProperty? DoubleProperty(UhtPropertyResolveArgs args)
		{
			return new UhtDoubleProperty(args.PropertySettings);
		}
		#endregion
	}
}
