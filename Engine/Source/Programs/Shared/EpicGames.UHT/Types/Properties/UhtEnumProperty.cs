// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FEnumProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "EnumProperty", IsProperty = true)]
	public class UhtEnumProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => Enum.CppForm != UhtEnumCppForm.EnumClass ? "ByteProperty" : "EnumProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "invalid";

		/// <inheritdoc/>
		protected override string PGetMacroText => Enum.CppForm != UhtEnumCppForm.EnumClass ? "PROPERTY" : "ENUM";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => Enum.CppForm != UhtEnumCppForm.EnumClass ? UhtPGetArgumentType.EngineClass : UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => Enum.CppForm != UhtEnumCppForm.EnumClass ? "FBytePropertyParams" : "FEnumPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => Enum.CppForm != UhtEnumCppForm.EnumClass ? "UECodeGen_Private::EPropertyGenFlags::Byte" : "UECodeGen_Private::EPropertyGenFlags::Enum";

		/// <summary>
		/// Referenced enum
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtEnum>))]
		public UhtEnum Enum { get; set; }

		/// <summary>
		/// Underlying property set when the enum has an underlying integer type
		/// </summary>
		[JsonIgnore]
		public UhtNumericProperty? UnderlyingProperty { get; set; }

		/// <summary>
		/// Underlying type which defaults to Int32 if the referenced enum doesn't have an underlying type
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtEnumUnderlyingType UnderlyingType => Enum.UnderlyingType != UhtEnumUnderlyingType.Unspecified ? Enum.UnderlyingType : UhtEnumUnderlyingType.Int32;

		/// <summary>
		/// Construct property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="enumObj">Referenced enum</param>
		public UhtEnumProperty(UhtPropertySettings propertySettings, UhtEnum enumObj) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.RequiresNullConstructorArg | UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM | UhtPropertyCaps.IsRigVMEnum | UhtPropertyCaps.SupportsVerse;
			Enum = enumObj;
			HeaderFile.AddReferencedHeader(enumObj);
			if (Enum.CppForm == UhtEnumCppForm.EnumClass)
			{
				UnderlyingProperty = CreateUnderlyingProperty();
			}
			else
			{
				UnderlyingProperty = null;
			}
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtEnumProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			Enum = (reader.ReadType() as UhtEnum)!;
			UnderlyingProperty = reader.ReadType() as UhtNumericProperty;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(Enum);
			writer.WriteType(UnderlyingProperty);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(Enum, UhtSingletonType.Registered);
			if (addForwardDeclarations && Enum.CppForm == UhtEnumCppForm.EnumClass)
			{
				collector.AddForwardDeclaration(Enum);
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			yield return Enum;
		}

		/// <summary>
		/// Append enum text
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="enumObj">Referenced enum</param>
		/// <param name="textType">Type of text to append</param>
		/// <param name="isTemplateArgument">If true, this property is a template argument</param>
		/// <returns></returns>
		public static StringBuilder AppendEnumText(StringBuilder builder, UhtProperty property, UhtEnum enumObj, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.GenericFunctionArgOrRetVal:
				case UhtPropertyTextType.GenericFunctionArgOrRetValImpl:
				case UhtPropertyTextType.ClassFunctionArgOrRetVal:
				case UhtPropertyTextType.InterfaceFunctionArgOrRetVal:
				case UhtPropertyTextType.EventFunctionArgOrRetVal:
					if (enumObj.CppForm == UhtEnumCppForm.EnumClass ||
						(!isTemplateArgument && (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm) ||
						!property.PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))))
					{
						builder.Append(enumObj.FullyQualifiedCppType);
					}
					else
					{
						builder.Append("TEnumAsByte<").Append(enumObj.FullyQualifiedCppType).Append('>');
					}
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.Append(enumObj.FullyQualifiedSourceName);
					break;

				case UhtPropertyTextType.RigVMType:
					if (enumObj.CppForm == UhtEnumCppForm.EnumClass || !isTemplateArgument)
					{
						builder.Append(enumObj.FullyQualifiedCppType);
					}
					else
					{
						builder.Append("TEnumAsByte<").Append(enumObj.FullyQualifiedCppType).Append('>');
					}
					break;

				case UhtPropertyTextType.GetterSetterArg:
					builder.Append(enumObj.Namespace.FullSourceName).Append(enumObj.CppType);
					break;

				case UhtPropertyTextType.VerseMangledType:
					builder.AppendVerseScopeAndName(enumObj, UhtVerseNameMode.Default);
					break;

				default:
					if (enumObj.CppForm == UhtEnumCppForm.EnumClass)
					{
						builder.Append(enumObj.FullyQualifiedCppType);
					}
					else
					{
						builder.Append("TEnumAsByte<").Append(enumObj.FullyQualifiedCppType).Append('>');
					}
					break;
			}
			return builder;
		}

		/// <summary>
		/// Append the text for a function thunk call argument
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="enumObj">Referenced enum</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendEnumFunctionThunkParameterArg(StringBuilder builder, UhtProperty property, UhtEnum enumObj)
		{
			if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
			{
				builder.Append(enumObj.FullyQualifiedCppType).Append('(').AppendFunctionThunkParameterName(property).Append(')');
			}
			else if (enumObj.CppForm == UhtEnumCppForm.EnumClass)
			{
				builder.Append($"({enumObj.FullyQualifiedCppType}&)(").AppendFunctionThunkParameterName(property).Append(')');
			}
			else
			{
				builder.Append($"(TEnumAsByte<{enumObj.FullyQualifiedCppType}>&)(").AppendFunctionThunkParameterName(property).Append(')');
			}
			return builder;
		}

		/// <summary>
		/// Sanitize the default value for an enumeration
		/// </summary>
		/// <param name="property">Property in question</param>
		/// <param name="enumObj">Referenced enumeration</param>
		/// <param name="defaultValueReader">Default value</param>
		/// <param name="innerDefaultValue">Destination builder</param>
		/// <returns>True if the default value was parsed.</returns>
		public static bool SanitizeEnumDefaultValue(UhtProperty property, UhtEnum enumObj, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			UhtTokenList cppIdentifier = defaultValueReader.GetCppIdentifier(UhtCppIdentifierOptions.None);
			UhtTokenList startingPoint = cppIdentifier.Next ?? cppIdentifier;
			startingPoint.Join(innerDefaultValue, "::");
			UhtTokenListCache.Return(cppIdentifier);

			int entryIndex = enumObj.GetIndexByName(innerDefaultValue.ToString());
			if (entryIndex == -1)
			{
				return false;
			}
			if (enumObj.MetaData.ContainsKey(UhtNames.Hidden, entryIndex))
			{
				property.LogError($"Hidden enum entries cannot be used as default values: '{property.SourceName}' '{innerDefaultValue}'");
			}
			return true;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			return AppendEnumText(builder, this, Enum, textType, isTemplateArgument);
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtChildProperty> EnumerateChildProperties(UhtCppIdentifier identifier, UhtChildPropertyOrder order)
		{
			if (Enum.CppForm == UhtEnumCppForm.EnumClass && UnderlyingProperty != null)
			{
				yield return new(UnderlyingProperty, identifier.AppendSuffix("_Underlying"));
			}
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Enum, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			base.AppendConstInitDefExtra(builder, context, identifier);
			if (Enum.CppForm == UhtEnumCppForm.EnumClass)
			{
				if (UnderlyingProperty == null)
				{
					builder.Append("0, ");
				}
				else
				{
					builder.Append("(int32)sizeof(");
					UnderlyingProperty.AppendText(builder, UhtPropertyTextType.Generic).Append("), ");
				}
			}
			builder.Append($"{ConstInitSingletonRef(context, Enum, true)}, ");
			return builder;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterArg(StringBuilder builder)
		{
			return AppendEnumFunctionThunkParameterArg(builder, this, Enum);
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			builder.AppendObjectHash(startingLength, this, context, Enum);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			if (Enum.CppForm != UhtEnumCppForm.EnumClass)
			{
				builder.Append('0');
			}
			else
			{
				builder.Append('(').Append(Enum.CppType).Append(")0");
			}
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return SanitizeEnumDefaultValue(this, Enum, defaultValueReader, innerDefaultValue);
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtEnumProperty otherEnum)
			{
				return Enum == otherEnum.Enum;
			}
			else if (other is UhtByteProperty otherByte)
			{
				return Enum == otherByte.Enum;
			}
			return false;
		}

		private UhtNumericProperty CreateUnderlyingProperty()
		{
			UhtPropertySettings propertySettings = new();
			propertySettings.Reset(this, 0, PropertyCategory, 0);
			propertySettings.SourceName = "UnderlyingType";
			switch (UnderlyingType)
			{
				case UhtEnumUnderlyingType.Int8:
					return new UhtInt8Property(propertySettings);
				case UhtEnumUnderlyingType.Int16:
					return new UhtInt16Property(propertySettings);
				case UhtEnumUnderlyingType.Int32:
					return new UhtIntProperty(propertySettings);
				case UhtEnumUnderlyingType.Int64:
					return new UhtInt64Property(propertySettings);
				case UhtEnumUnderlyingType.Uint8:
					return new UhtByteProperty(propertySettings);
				case UhtEnumUnderlyingType.Uint16:
					return new UhtUInt16Property(propertySettings);
				case UhtEnumUnderlyingType.Uint32:
					return new UhtUInt32Property(propertySettings);
				case UhtEnumUnderlyingType.Uint64:
					return new UhtUInt64Property(propertySettings);
				default:
					throw new UhtIceException($"Unexpected underlying enum type: '{UnderlyingType}'");
			}
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TEnumAsByte")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtByteProperty? EnumProperty(UhtPropertyResolveArgs args)
		{
			UhtType outer = args.PropertySettings.Outer;
			UhtEnum? enumObj = null;
			using UhtMessageContext tokenContext = new("TEnumAsByte");
			args.TokenReader
				.Require("TEnumAsByte")
				.Require('<')
				.Optional("enum")
				.RequireCppIdentifier(UhtCppIdentifierOptions.None, tokenList =>
				{
					enumObj = outer.FindType(UhtFindOptions.Enum | UhtFindOptions.SourceName, tokenList, args.TokenReader) as UhtEnum;
				})
				.Require('>');
			return enumObj != null ? new UhtByteProperty(args.PropertySettings, UhtByteCppForm.UnsignedInt8, enumObj) : null;
		}
		#endregion
	}
}
