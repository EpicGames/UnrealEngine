// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FFieldPathProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "FieldPathProperty", IsProperty = true)]
	public class UhtFieldPathProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "FieldPathProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TFieldPath";

		/// <inheritdoc/>
		protected override string PGetMacroText => "TFIELDPATH";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FFieldPathPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::FieldPath";

		/// <summary>
		/// Class name without the prefix
		/// </summary>
		public string FieldClassName { get; set; }

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="fieldClassName">Field class name</param>
		public UhtFieldPathProperty(UhtPropertySettings propertySettings, string fieldClassName) : base(propertySettings)
		{
			FieldClassName = fieldClassName;
			PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtFieldPathProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			FieldClassName = reader.ReadString();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteString(FieldClassName);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			if (addForwardDeclarations)
			{
				collector.AddForwardDeclaration(null, $"class F{FieldClassName};");
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.SparseShort:
					builder.Append("TFieldPath");
					break;

				default:
					builder.Append("TFieldPath<F").Append(FieldClassName).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append("&F").Append(FieldClassName).Append("::StaticClass, ");
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"UE_IF(IS_MONOLITHIC, &F{FieldClassName}::ConstInitClass, UTF8TEXT(").AppendUTF8LiteralString(FieldClassName).Append(")), ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("nullptr");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtFieldPathProperty otherFieldPath)
			{
				return FieldClassName == otherFieldPath.FieldClassName;
			}
			return false;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TFieldPath", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtFieldPathProperty? FieldPathProperty(UhtPropertyResolveArgs args)
		{
			using UhtMessageContext tokenContext = new("TFieldPath");
			UhtToken identifier = new();
			args.TokenReader
				.Require("TFieldPath")
				.Require('<')
				.RequireIdentifier((ref UhtToken token) => identifier = token)
				.Require('>');

			StringView fieldClassName = new(identifier.Value, 1);
			if (!args.Session.IsValidPropertyTypeName(fieldClassName))
			{
				throw new UhtException($"Undefined property type: {identifier.Value}");
			}
			return new UhtFieldPathProperty(args.PropertySettings, fieldClassName.ToString());
		}
		#endregion
	}
}
