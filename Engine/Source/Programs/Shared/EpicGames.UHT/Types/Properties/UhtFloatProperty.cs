// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FFloatProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "FloatProperty", IsProperty = true)]
	public class UhtFloatProperty : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "FloatProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "float";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FFloatPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Float";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtFloatProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtFloatProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, "{0:F6}", defaultValueReader.GetConstFloatExpression());
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtFloatProperty;
		}

		#region Keywords
		[UhtPropertyType(Keyword = "float", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtFloatProperty? FloatProperty(UhtPropertyResolveArgs args)
		{
			return new UhtFloatProperty(args.PropertySettings);
		}
		#endregion
	}
}
