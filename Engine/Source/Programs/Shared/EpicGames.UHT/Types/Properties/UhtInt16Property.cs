// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FInt16Property
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "Int16Property", IsProperty = true)]
	public class UhtInt16Property : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "Int16Property";

		/// <inheritdoc/>
		protected override string CppTypeText => "int16";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FInt16PropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Int16";

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtInt16Property(UhtPropertySettings propertySettings) : base(propertySettings)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtInt16Property(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtInt16Property;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "int16", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtInt16Property? Int16Property(UhtPropertyResolveArgs args)
		{
			return new UhtInt16Property(args.PropertySettings);
		}
		#endregion
	}
}
