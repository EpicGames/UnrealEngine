// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FInt64Property
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "Int64Property", IsProperty = true)]
	public class UhtInt64Property : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "Int64Property";

		/// <inheritdoc/>
		protected override string CppTypeText => "int64";

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtInt64Property(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsVerse;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtInt64Property(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument = false)
		{
			switch (textType)
			{
				case UhtPropertyTextType.VerseMangledType:
					builder.Append("int");
					return builder;

				default:
					return base.AppendText(builder, textType, isTemplateArgument);
			}
		}

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FInt64PropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Int64";

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.Append(defaultValueReader.GetConstLongExpression());
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtInt64Property;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "int64", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtInt64Property? Int64Property(UhtPropertyResolveArgs args)
		{
			return new UhtInt64Property(args.PropertySettings);
		}
		#endregion
	}
}
