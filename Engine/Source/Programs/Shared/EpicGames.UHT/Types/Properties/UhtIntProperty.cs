// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FIntProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "IntProperty", IsProperty = true)]
	public class UhtIntProperty : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "IntProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "int32";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FIntPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Int";

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtIntProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtIntProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.Append(defaultValueReader.GetConstIntExpression());
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtIntProperty;
		}

		#region Keywords
		[UhtPropertyType(Keyword = "int32", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtIntProperty? Int32Property(UhtPropertyResolveArgs args)
		{
			return new UhtIntProperty(args.PropertySettings);
		}

		[UhtPropertyType(Keyword = "int", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[UhtPropertyType(Keyword = "signed", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[UhtPropertyType(Keyword = "signed int", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtIntProperty? IntProperty(UhtPropertyResolveArgs args)
		{
			return new UhtIntProperty(args.PropertySettings);
		}
		#endregion
	}
}
