// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FInterfaceProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "InterfaceProperty", IsProperty = true)]
	public class UhtInterfaceProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "InterfaceProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TScriptInterface";

		/// <inheritdoc/>
		protected override string PGetMacroText => "TINTERFACE";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FInterfacePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Interface";

		/// <summary>
		/// Referenced interface class
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass InterfaceClass { get; set; }

		/// <summary>
		/// Create a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="interfaceClass">Referenced interface</param>
		public UhtInterfaceProperty(UhtPropertySettings propertySettings, UhtClass interfaceClass) : base(propertySettings)
		{
			InterfaceClass = interfaceClass;
			PropertyFlags |= EPropertyFlags.UObjectWrapper;
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.PassCppArgsByRef;
			PropertyCaps &= ~(UhtPropertyCaps.CanHaveConfig | UhtPropertyCaps.CanBeContainerKey);
			if (Session.Config!.AreRigVMUInterfacePropertiesEnabled)
			{
				PropertyCaps |= UhtPropertyCaps.SupportsRigVM;
			}
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtInterfaceProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			InterfaceClass = (reader.ReadType() as UhtClass)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(InterfaceClass);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					if (InterfaceClass.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced))
					{
						PropertyFlags |= (EPropertyFlags.InstancedReference | EPropertyFlags.ExportObject) & ~DisallowPropertyFlags;
					}
					break;
			}
			return results;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return !DisallowPropertyFlags.HasAnyFlags(EPropertyFlags.InstancedReference) && InterfaceClass.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(InterfaceClass.AlternateObject ?? InterfaceClass, UhtSingletonType.Unregistered);
			if (addForwardDeclarations)
			{
				UhtClass? exportClass = InterfaceClass;
				while (exportClass != null && !exportClass.ClassFlags.HasAnyFlags(EClassFlags.Native))
				{
					exportClass = exportClass.SuperClass;
				}
				if (exportClass != null)
				{
					collector.AddForwardDeclaration(exportClass);
				}
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			yield return InterfaceClass;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.SparseShort:
					builder.Append("TScriptInterface");
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.Append(InterfaceClass.SourceName);
					break;

				default:
					builder.Append("TScriptInterface<").Append(InterfaceClass.SourceName).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			// FScriptInterface<USomeInterface> is valid so in that case we need to pass in the interface class and not the alternate object (which in the end is the same object)
			return AppendParamsDefRef(builder, context, InterfaceClass.AlternateObject ?? InterfaceClass, UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, InterfaceClass.AlternateObject ?? InterfaceClass, true)}, ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NULL");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);
			if (PointerType == UhtPointerType.Native)
			{
				this.LogError($"Property and function argument pointers cannot be interfaces - did you mean TScriptInterface<{InterfaceClass.SourceName}>?");
			}
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtInterfaceProperty otherObject)
			{
				return InterfaceClass == otherObject.InterfaceClass;
			}
			return false;
		}

		/// <inheritdoc/>
		public override bool MustBeConstArgument([NotNullWhen(true)] out UhtType? errorType)
		{
			errorType = InterfaceClass;
			return InterfaceClass.ClassFlags.HasAnyFlags(EClassFlags.Const);
		}

		#region Keywords
		[UhtPropertyType(Keyword = "FScriptInterface", Options = UhtPropertyTypeOptions.Simple)] // This can't be immediate due to the reference to UInterface
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtInterfaceProperty? FScriptInterfaceProperty(UhtPropertyResolveArgs args)
		{
			return new UhtInterfaceProperty(args.PropertySettings, args.Session.IInterface);
		}

		[UhtPropertyType(Keyword = "TScriptInterface")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? TScriptInterfaceProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? propertyClass = args.ParseTemplateObject(UhtTemplateObjectMode.PreserveNativeInterface);
			if (propertyClass == null)
			{
				return null;
			}

			if (propertyClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
			{
				return new UhtInterfaceProperty(args.PropertySettings, propertyClass);
			}
			return new UhtObjectProperty(args.PropertySettings, UhtObjectCppForm.NativeObject, propertyClass, EPropertyFlags.UObjectWrapper);
		}
		#endregion
	}
}
