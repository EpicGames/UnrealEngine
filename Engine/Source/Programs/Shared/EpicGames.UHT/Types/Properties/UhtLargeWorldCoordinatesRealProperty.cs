// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FLargeWorldCoordinatesRealProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "LargeWorldCoordinatesRealProperty", IsProperty = true)]
	public class UhtLargeWorldCoordinatesRealProperty : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "LargeWorldCoordinatesRealProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "double";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FLargeWorldCoordinatesRealPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::LargeWorldCoordinatesReal";

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtLargeWorldCoordinatesRealProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtLargeWorldCoordinatesRealProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, "{0:F6}", defaultValueReader.GetConstFloatExpression());
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtLargeWorldCoordinatesRealProperty;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "FLargeWorldCoordinatesReal", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtLargeWorldCoordinatesRealProperty? LargeWorldCoordinatesRealProperty(UhtPropertyResolveArgs args)
		{
			if (!args.PropertySettings.Outer.HeaderFile.IsNoExportTypes)
			{
				args.TokenReader.LogError("FLargeWorldCoordinatesReal is intended for LWC support only and should not be used outside of NoExportTypes.h");
			}
			return new UhtLargeWorldCoordinatesRealProperty(args.PropertySettings);
		}
		#endregion
	}
}
