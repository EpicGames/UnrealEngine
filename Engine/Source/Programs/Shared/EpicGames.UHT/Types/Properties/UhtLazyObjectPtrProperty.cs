// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FLazyObjectProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "LazyObjectProperty", IsProperty = true)]
	public class UhtLazyObjectPtrProperty : UhtObjectPropertyBase
	{
		/// <inheritdoc/>
		public override string EngineClassName => "LazyObjectProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TLazyObjectPtr";

		/// <inheritdoc/>
		protected override string PGetMacroText => "LAZYOBJECT";

		/// <inheritdoc/>
		protected override bool PGetPassAsNoPtr => true;

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FLazyObjectPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::LazyObject";

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="referencedClass">Referenced class</param>
		public UhtLazyObjectPtrProperty(UhtPropertySettings propertySettings, UhtClass referencedClass)
			: base(new UhtNoValidateConstruct(), propertySettings, UhtObjectCppForm.NativeObject, referencedClass)
		{
			PropertyFlags |= EPropertyFlags.UObjectWrapper;
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.RequiresNullConstructorArg;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtLazyObjectPtrProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				default:
					builder.Append("TLazyObjectPtr<").Append(Class.SourceName).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Class, Exporters.CodeGen.UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Class, true)}, ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NULL");
			return builder;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtLazyObjectPtrProperty otherObject)
			{
				return Class == otherObject.Class && MetaClass == otherObject.MetaClass;
			}
			return false;
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);

			// UFunctions with a smart pointer as input parameter wont compile anyway, because of missing P_GET_... macro.
			// UFunctions with a smart pointer as return type will crash when called via blueprint, because they are not supported in VM.
			if (PropertyCategory != UhtPropertyCategory.Member)
			{
				outerStruct.LogError("UFunctions cannot take a lazy pointer as a parameter.");
			}
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TLazyObjectPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtLazyObjectPtrProperty? LazyObjectPtrProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? propertyClass = args.ParseTemplateObject(UhtTemplateObjectMode.Normal);
			if (propertyClass == null)
			{
				return null;
			}

			if (propertyClass.IsChildOf(propertyClass.Session.UClass))
			{
				args.TokenReader.LogError("Class variables cannot be lazy, they are always strong.");
			}

			return new UhtLazyObjectPtrProperty(args.PropertySettings, propertyClass);
		}
		#endregion
	}
}
