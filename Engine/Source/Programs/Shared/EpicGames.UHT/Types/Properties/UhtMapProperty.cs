// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FMapProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "MapProperty", IsProperty = true)]
	public class UhtMapProperty : UhtContainerBaseProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "MapProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TMap";

		/// <inheritdoc/>
		protected override string PGetMacroText => "TMAP";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FMapPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Map";

		/// <summary>
		/// Key property
		/// </summary>
		public UhtProperty KeyProperty { get; set; }

		/// <summary>
		/// Construct new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="key">Key property</param>
		/// <param name="value">Value property</param>
		public UhtMapProperty(UhtPropertySettings propertySettings, UhtProperty key, UhtProperty value) : base(propertySettings, value)
		{
			KeyProperty = key;

			// If the creation of the value property set more flags, then copy those flags to ourselves
			PropertyFlags |= ValueProperty.PropertyFlags & (EPropertyFlags.UObjectWrapper | EPropertyFlags.TObjectPtr);

			// Make sure the 'UObjectWrapper' flag is maintained so that both 'TMap<TSubclassOf<...>, ...>' and 'TMap<UClass*, TSubclassOf<...>>' works correctly
			KeyProperty.PropertyFlags = (ValueProperty.PropertyFlags & ~EPropertyFlags.UObjectWrapper) | (KeyProperty.PropertyFlags & EPropertyFlags.UObjectWrapper);
			KeyProperty.DisallowPropertyFlags = ~(EPropertyFlags.ContainsInstancedReference | EPropertyFlags.InstancedReference | EPropertyFlags.UObjectWrapper);

			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.SupportsVerse;
			PropertyCaps &= ~(UhtPropertyCaps.CanBeContainerValue | UhtPropertyCaps.CanBeContainerKey);
			UpdateCaps();

			PropertyFlags = ValueProperty.PropertyFlags;
			ValueProperty.SourceName = SourceName;
			ValueProperty.EngineName = EngineName;
			ValueProperty.PropertyFlags &= EPropertyFlags.PropagateKeepInInner | EPropertyFlags.PropagateToMapValue;
			ValueProperty.Outer = this;
			ValueProperty.MetaData.Clear();
			KeyProperty.SourceName = $"{SourceName}_Key";
			KeyProperty.EngineName = $"{EngineName}_Key";
			KeyProperty.PropertyFlags &= EPropertyFlags.PropagateKeepInInner | EPropertyFlags.PropagateToMapKey;
			KeyProperty.Outer = this;
			KeyProperty.MetaData.Clear();

			// With old UHT, Deprecated was applied after property create.
			// With the new, it is applied prior to creation.  Deprecated in old 
			// was not on the key.
			KeyProperty.PropertyFlags &= ~EPropertyFlags.Deprecated;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtMapProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			KeyProperty = (reader.ReadType() as UhtProperty)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(KeyProperty);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					KeyProperty.Resolve(phase);
					KeyProperty.MetaData.Clear();

					EPropertyFlags newFlags = ResolveAndReturnNewFlags(ValueProperty, phase);
					PropertyFlags |= newFlags;
					KeyProperty.PropertyFlags |= newFlags;
					MetaData.Add(ValueProperty.MetaData);
					ValueProperty.MetaData.Clear();
					ValueProperty.PropertyFlags = (ValueProperty.PropertyFlags & EPropertyFlags.PropagateKeepInInner) | (PropertyFlags & EPropertyFlags.PropagateToMapValue);

					UpdateCaps();
					PropagateFlagsFromInnerAndHandlePersistentInstanceMetadata(this, null, KeyProperty);
					PropagateFlagsFromInnerAndHandlePersistentInstanceMetadata(this, MetaData, ValueProperty);
					break;
			}
			return results;
		}

		private void UpdateCaps()
		{
			PropertyCaps &= ~(UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.CanExposeOnSpawn);
			PropertyCaps |= ValueProperty.PropertyCaps & UhtPropertyCaps.CanExposeOnSpawn;
			if (KeyProperty.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsParameterSupportedByBlueprint) && ValueProperty.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsParameterSupportedByBlueprint))
			{
				PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			}
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			KeyProperty.CollectReferencesInternal(collector, addForwardDeclarations, true);
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			foreach (UhtType type in ValueProperty.EnumerateReferencedTypes())
			{
				yield return type;
			}
			foreach (UhtType type in KeyProperty.EnumerateReferencedTypes())
			{
				yield return type;
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.SparseShort:
					builder.Append("TMap");
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.AppendFunctionThunkParameterArrayType(KeyProperty, true).Append(',').AppendFunctionThunkParameterArrayType(ValueProperty, true);
					break;

				case UhtPropertyTextType.VerseMangledType:
					builder.Append('[').AppendPropertyText(KeyProperty, textType, true).Append(']').AppendPropertyText(ValueProperty, textType, true);
					break;

				default:
					builder.Append("TMap<").AppendPropertyText(KeyProperty, textType, true).Append(',').AppendPropertyText(ValueProperty, textType, true).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtChildProperty> EnumerateChildProperties(UhtCppIdentifier identifier, UhtChildPropertyOrder order)
		{
			// Order is VERY important.  Do not change.
			if (order == UhtChildPropertyOrder.Forward)
			{
				yield return new(KeyProperty, identifier.AppendSuffix("_Key_KeyProp"), "0");
				yield return new(ValueProperty, identifier.AppendSuffix("_ValueProp"), "1");
			}
			else
			{
				yield return new(ValueProperty, identifier.AppendSuffix("_ValueProp"), "1");
				yield return new(KeyProperty, identifier.AppendSuffix("_Key_KeyProp"), "0");
			}
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append(Allocator == UhtPropertyAllocator.MemoryImage ? "EMapPropertyFlags::UsesMemoryImageAllocator" : "EMapPropertyFlags::None").Append(", ");
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			base.AppendConstInitDefExtra(builder, context, identifier);
			builder.Append(Allocator == UhtPropertyAllocator.MemoryImage ? "EMapPropertyFlags::UsesMemoryImageAllocator" : "EMapPropertyFlags::None").Append(", ");
			builder.Append("(int32)sizeof(").AppendPropertyText(KeyProperty, UhtPropertyTextType.Generic).Append("), ");
			builder.Append("(int16)alignof(").AppendPropertyText(KeyProperty, UhtPropertyTextType.Generic).Append("), ");
			builder.Append("(int32)sizeof(").AppendPropertyText(ValueProperty, UhtPropertyTextType.Generic).Append("), ");
			builder.Append("(int16)alignof(").AppendPropertyText(ValueProperty, UhtPropertyTextType.Generic).Append("), ");
			return builder;
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			KeyProperty.AppendObjectHashes(builder, startingLength, context);
			ValueProperty.AppendObjectHashes(builder, startingLength, context);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.AppendPropertyText(this, UhtPropertyTextType.Construction).Append("()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtMapProperty otherMap)
			{
				return ValueProperty.IsSameType(otherMap.ValueProperty) &&
					KeyProperty.IsSameType(otherMap.KeyProperty);
			}
			return false;
		}

		/// <inheritdoc/>
		public override void ValidateDeprecated()
		{
			base.ValidateDeprecated();
			KeyProperty.ValidateDeprecated();
		}

		/// <inheritdoc/>
		protected override bool NeedsGCBarrierWhenPassedToFunctionImpl(UhtFunction function)
		{
			return KeyProperty.NeedsGCBarrierWhenPassedToFunction(function) || ValueProperty.NeedsGCBarrierWhenPassedToFunction(function);
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);
			KeyProperty.Validate(outerStruct, outermostProperty, options | UhtValidationOptions.IsKey);
		}

		///<inheritdoc/>
		public override bool ValidateStructPropertyOkForNet(UhtProperty referencingProperty)
		{
			if (!PropertyFlags.HasAnyFlags(EPropertyFlags.RepSkip))
			{
				referencingProperty.LogError($"Maps are not supported for Replication or RPCs.  Map '{SourceName}' in '{Outer?.SourceName}'.  Origin '{referencingProperty.SourceName}'");
				return false;
			}
			return true;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest | EFunctionFlags.NetResponse))
				{
					this.LogError("Maps are not supported in an RPC.");
				}
			}
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TMap")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static UhtMapProperty? MapProperty(UhtPropertyResolveArgs args)
		{
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			using UhtMessageContext tokenContext = new("TMap");
			if (!args.SkipExpectedType())
			{
				return null;
			}
			tokenReader.Require('<');

			// Parse the key type
			UhtProperty? key = args.ParseTemplateParam("Key");
			if (key == null)
			{
				return null;
			}

			if (!key.PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeContainerKey))
			{
				tokenReader.LogError($"The type \'{key.GetUserFacingDecl()}\' can not be used as a key in a TMap");
			}

			if (propertySettings.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
			{
				tokenReader.LogError("Replicated maps are not supported.");
			}

			tokenReader.Require(',');

			// Parse the value type
			UhtProperty? value = args.ParseTemplateParam("Value");
			if (value == null)
			{
				return null;
			}

			if (!value.PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeContainerValue))
			{
				tokenReader.LogError($"The type \'{value.GetUserFacingDecl()}\' can not be used as a value in a TMap");
			}

			if (tokenReader.TryOptional(','))
			{
				UhtToken allocatorToken = tokenReader.GetIdentifier();
				if (allocatorToken.IsIdentifier("FMemoryImageSetAllocator"))
				{
					propertySettings.Allocator = UhtPropertyAllocator.MemoryImage;
				}
				else
				{
					tokenReader.LogError($"Found '{allocatorToken.Value}' - explicit allocators are not supported in TMap properties.");
				}
			}
			tokenReader.Require('>');

			//@TODO: Prevent sparse delegate types from being used in a container

			return new UhtMapProperty(propertySettings, key, value);
		}
		#endregion
	}
}
