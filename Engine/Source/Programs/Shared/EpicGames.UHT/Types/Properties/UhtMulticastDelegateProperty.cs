// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FMulticastDelegateProperty
	/// </summary>
	[UhtEngineClass(Name = "MulticastDelegateProperty", IsProperty = true)]
	public abstract class UhtMulticastDelegateProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "MulticastDelegateProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FMulticastScriptDelegate";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <summary>
		/// Referenced function
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtFunction>))]
		public UhtFunction Function { get; set; }

		/// <summary>
		/// Construct property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="function">Referenced function</param>
		protected UhtMulticastDelegateProperty(UhtPropertySettings propertySettings, UhtFunction function) : base(propertySettings)
		{
			Function = function;
			HeaderFile.AddReferencedHeader(function);
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.CanBeBlueprintAssignable | UhtPropertyCaps.CanBeBlueprintCallable |
				UhtPropertyCaps.CanBeBlueprintAuthorityOnly | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			PropertyCaps &= ~(UhtPropertyCaps.IsParameterSupportedByBlueprint);
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtMulticastDelegateProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			Function = (reader.ReadType() as UhtFunction)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(Function);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					PropertyFlags |= EPropertyFlags.InstancedReference & ~DisallowPropertyFlags;
					break;
			}
			return results;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return true;
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(Function, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.EventParameterFunctionMember:
				case UhtPropertyTextType.ExportMember:
				case UhtPropertyTextType.RigVMType:
				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.Append(CppTypeText);
					break;

				default:
					builder.Append(Function.SourceName);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			builder.AppendObjectHash(startingLength, this, context, Function);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterArg(StringBuilder builder)
		{
			return builder.Append(Function.SourceName).Append('(').AppendFunctionThunkParameterName(this).Append(')');
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.AppendPropertyText(this, UhtPropertyTextType.Construction).Append("()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtMulticastDelegateProperty otherObject)
			{
				return Function == otherObject.Function;
			}
			return false;
		}
	}
}
