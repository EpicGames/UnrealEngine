// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FMulticastSparseDelegateProperty
	/// </summary>
	[UhtEngineClass(Name = "MulticastSparseDelegateProperty", IsProperty = true)]
	public class UhtMulticastSparseDelegateProperty : UhtMulticastDelegateProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "MulticastSparseDelegateProperty";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FMulticastDelegatePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::SparseMulticastDelegate";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="function">Referenced function</param>
		public UhtMulticastSparseDelegateProperty(UhtPropertySettings propertySettings, UhtFunction function) : base(propertySettings, function)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtMulticastSparseDelegateProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.Append("FMulticastSparseDelegateProperty");
					break;

				default:
					base.AppendText(builder, textType, isTemplateArgument);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Function, Exporters.CodeGen.UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Function, true)}, ");
		}
	}
}
