// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FNameProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "NameProperty", IsProperty = true)]
	public class UhtNameProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "NameProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FName";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.EngineClass;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FNamePropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Name";


		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtNameProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtNameProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NAME_None");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			if (defaultValueReader.TryOptional("NAME_None"))
			{
				innerDefaultValue.Append("None");
			}
			else
			{
				StringView value = defaultValueReader.GetTypedConstString("FName");
				innerDefaultValue.Append(value);
			}
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtNameProperty;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "FName", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtNameProperty? NameProperty(UhtPropertyResolveArgs args)
		{
			return new UhtNameProperty(args.PropertySettings);
		}
		#endregion
	}
}
