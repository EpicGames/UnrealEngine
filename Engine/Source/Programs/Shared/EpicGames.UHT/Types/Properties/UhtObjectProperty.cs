// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FObjectProperty
	/// </summary>
	[UhtEngineClass(Name = "ObjectProperty", IsProperty = true)]
	public class UhtObjectProperty : UhtObjectPropertyBase
	{
		/// <inheritdoc/>
		public override string EngineClassName => "ObjectProperty";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Source code form of the property</param>
		/// <param name="referencedClass">Referenced class</param>
		/// <param name="extraFlags">Extra flags to add to the property</param>
		public UhtObjectProperty(UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass, EPropertyFlags extraFlags = EPropertyFlags.None)
			: this(new UhtNoValidateConstruct { }, propertySettings, cppForm, referencedClass, extraFlags)
		{
			if (!cppForm.IsValidForObjectProperty())
			{
				throw new UhtIceException($"Improper UhtObjectCppForm.{cppForm} for an UhtObjectProperty");
			}
		}

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="_">Avoid validation</param>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Source code form of the property</param>
		/// <param name="referencedClass">Referenced class</param>
		/// <param name="extraFlags">Extra flags to add to the property</param>
		protected UhtObjectProperty(UhtNoValidateConstruct _, UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass, EPropertyFlags extraFlags)
			: base(_, propertySettings, cppForm, referencedClass, extraFlags)
		{
			PropertyCaps |= UhtPropertyCaps.RequiresNullConstructorArg | UhtPropertyCaps.CanBeInstanced | UhtPropertyCaps.CanExposeOnSpawn |
				UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtObjectProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					if (Class.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced))
					{
						PropertyFlags |= EPropertyFlags.InstancedReference;
						MetaData.Add(UhtNames.EditInline, true);
					}
					break;
			}
			return results;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return Class.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.GetterSetterArg:
					if (isTemplateArgument)
					{
						AppendTemplateType(builder);
					}
					else
					{
						builder.Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('*');
					}
					break;

				case UhtPropertyTextType.FunctionThunkRetVal:
					if (isTemplateArgument)
					{
						AppendTemplateType(builder);
					}
					else
					{
						if (PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
						{
							builder.Append("const ");
						}
						AppendTemplateType(builder);
					}
					break;

				case UhtPropertyTextType.FunctionThunkParameterArrayType:
					AppendTemplateType(builder);
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					if (isTemplateArgument)
					{
						AppendTemplateType(builder);
					}
					else if (CppForm == UhtObjectCppForm.TInterfaceInstance)
					{
						builder.Append(Class.Namespace.FullSourceName).AppendClassSourceNameOrInterfaceProxyName(ReferencedClass);
					}
					else
					{
						builder.Append(Class.Namespace.FullSourceName).Append(Class.SourceName);
					}
					break;

				case UhtPropertyTextType.VerseMangledType:
					AppendVerseMangledType(builder, ReferencedClass);
					break;

				default:
					AppendTemplateType(builder);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Class, Exporters.CodeGen.UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Class, true)}, ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NULL");
			return builder;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtObjectProperty otherObject)
			{
				return CppForm.GetSameTypeCppForm() == otherObject.CppForm.GetSameTypeCppForm() && Class == otherObject.Class && MetaClass == otherObject.MetaClass;
			}
			return false;
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);

			// UFunctions with a smart pointer as input parameter wont compile anyway, because of missing P_GET_... macro.
			// UFunctions with a smart pointer as return type will crash when called via blueprint, because they are not supported in VM.
			if (PropertyCategory != UhtPropertyCategory.Member && (CppForm == UhtObjectCppForm.TObjectPtrObject || CppForm == UhtObjectCppForm.TObjectPtrClass))
			{
				// At this point, allow this to appear in TMap keys in the UPlayerMappableInputConfig class
				if (!options.HasAnyFlags(UhtValidationOptions.IsKey) ||
					!outerStruct.SourceName.Equals("GetMappingContexts", StringComparison.Ordinal) ||
					outerStruct.Outer == null ||
					!outerStruct.Outer.SourceName.Equals("UPlayerMappableInputConfig", StringComparison.Ordinal))
				{
					outerStruct.LogError("UFunctions cannot take a TObjectPtr as a function parameter or return value.");
				}
			}
		}

		/// <inheritdoc/>
		protected override void ValidateMember(UhtStruct structObj, UhtValidationOptions options)
		{
			base.ValidateMember(structObj, options);
			if (Class.NativeInterface != null && (CppForm == UhtObjectCppForm.TObjectPtrObject || CppForm == UhtObjectCppForm.TObjectPtrClass))
			{
				this.LogError($"UPROPERTY pointers cannot be interfaces - did you mean TScriptInterface<{Class.Namespace.FullSourceName}{Class.SourceName}>?");
			}
		}
	}
}
