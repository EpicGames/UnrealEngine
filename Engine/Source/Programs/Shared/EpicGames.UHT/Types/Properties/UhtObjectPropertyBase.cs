// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	#region CppForm
	/// <summary>
	/// How the object property appeared in the code
	/// </summary>
	public enum UhtObjectCppForm
	{

		/// <summary>
		/// Existing native object raw pointer
		/// </summary>
		NativeObject,

		/// <summary>
		/// Existing native class raw pointer
		/// </summary>
		NativeClass,

		/// <summary>
		/// TObjectPtr for an object
		/// </summary>
		TObjectPtrObject,

		/// <summary>
		/// TObjectPtr for a class
		/// </summary>
		TObjectPtrClass,

		/// <summary>
		/// TSoftObjectPtr
		/// </summary>
		TSoftObjectPtr,

		/// <summary>
		/// TSoftClassPtr
		/// </summary>
		TSoftClassPtr,

		/// <summary>
		/// TSubclassOf
		/// </summary>
		TSubclassOf,

		/// <summary>
		/// verse::type
		/// </summary>
		VerseType,

		/// <summary>
		/// verse::subtype
		/// </summary>
		VerseSubtype,

		/// <summary>
		/// verse::castable_type
		/// </summary>
		VerseCastableType,

		/// <summary>
		/// verse:castable_subtype
		/// </summary>
		VerseCastableSubtype,

		/// <summary>
		/// verse::concrete_type
		/// </summary>
		VerseConcreteType,

		/// <summary>
		/// verse:concrete_subtype
		/// </summary>
		VerseConcreteSubtype,

		/// <summary>
		/// verse::castable_concrete_type
		/// </summary>
		VerseCastableConcreteType,

		/// <summary>
		/// verse:castable_concrete_subtype
		/// </summary>
		VerseCastableConcreteSubtype,

		/// <summary>
		/// TInterfaceInstance
		/// </summary>
		TInterfaceInstance,
	}

	/// <summary>
	/// Collection of extension methods for UhtObjectCppForm
	/// </summary>
	public static class UhtObjectCppFormExtensions
	{

		/// <summary>
		/// Check to see if the given CPP form is valid for a object property
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>Is valid if true</returns>
		/// <exception cref="NotImplementedException"></exception>
		public static bool IsValidForObjectProperty(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TInterfaceInstance:
					return true;

				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TSubclassOf:
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return false;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Check to see if the given CPP form is valid for a class property
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>Is valid if true</returns>
		/// <exception cref="NotImplementedException"></exception>
		public static bool IsValidForClassProperty(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TSubclassOf:
					return true;

				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TInterfaceInstance:
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return false;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Check to see if the given CPP form is valid for a verse class property
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>Is valid if true</returns>
		/// <exception cref="NotImplementedException"></exception>
		public static bool IsValidForVerseClassProperty(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return true;

				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TSubclassOf:
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TInterfaceInstance:
					return false;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Return a CppForm used when doing IsSameType checks
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>CppForm to be used in same type checks</returns>
		public static UhtObjectCppForm GetSameTypeCppForm(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
					return UhtObjectCppForm.NativeObject;

				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TInterfaceInstance:
				case UhtObjectCppForm.TSubclassOf:
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return cppForm;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Returns true if this CppForm needs a verse where clause
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>True if the verse name needs a where clause</returns>
		public static bool NeedsWhereClause(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TInterfaceInstance:
				case UhtObjectCppForm.TSubclassOf:
					return false;

				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return true;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Return the params code gen name
		/// </summary>
		/// <param name="cppForm"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public static string GetCodeGenParamsStruct(this UhtObjectCppForm cppForm)
		{
			return cppForm switch
			{
				UhtObjectCppForm.NativeClass or
				UhtObjectCppForm.TObjectPtrClass or
				UhtObjectCppForm.TSubclassOf
					=> "FClassPropertyParams",

				UhtObjectCppForm.VerseType or
				UhtObjectCppForm.VerseSubtype or
				UhtObjectCppForm.VerseCastableType or
				UhtObjectCppForm.VerseCastableSubtype or
				UhtObjectCppForm.VerseConcreteType or
				UhtObjectCppForm.VerseConcreteSubtype or
				UhtObjectCppForm.VerseCastableConcreteType or
				UhtObjectCppForm.VerseCastableConcreteSubtype
					=> "FVerseClassPropertyParams",

				UhtObjectCppForm.NativeObject or
				UhtObjectCppForm.TObjectPtrObject or
				UhtObjectCppForm.TInterfaceInstance
					=> "FObjectPropertyParams",

				UhtObjectCppForm.TSoftObjectPtr
					=> "FSoftObjectPropertyParams",

				UhtObjectCppForm.TSoftClassPtr
					=> "FSoftClassPropertyParams",

				_
					=> throw new NotImplementedException(),
			};
		}

		/// <summary>
		/// Return the params code gen flags
		/// </summary>
		/// <param name="cppForm"></param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public static string GetCodeGenParamsFlags(this UhtObjectCppForm cppForm)
		{
			return cppForm switch
			{
				UhtObjectCppForm.NativeClass or
				UhtObjectCppForm.TSubclassOf
					=> "UECodeGen_Private::EPropertyGenFlags::Class",

				UhtObjectCppForm.TObjectPtrClass
					=> "UECodeGen_Private::EPropertyGenFlags::Class | UECodeGen_Private::EPropertyGenFlags::ObjectPtr",

				UhtObjectCppForm.VerseType or
				UhtObjectCppForm.VerseSubtype or
				UhtObjectCppForm.VerseCastableType or
				UhtObjectCppForm.VerseCastableSubtype or
				UhtObjectCppForm.VerseConcreteType or
				UhtObjectCppForm.VerseConcreteSubtype or
				UhtObjectCppForm.VerseCastableConcreteType or
				UhtObjectCppForm.VerseCastableConcreteSubtype
					=> "UECodeGen_Private::EPropertyGenFlags::VerseType",

				UhtObjectCppForm.NativeObject or
				UhtObjectCppForm.TInterfaceInstance
					=> "UECodeGen_Private::EPropertyGenFlags::Object",

				UhtObjectCppForm.TObjectPtrObject
					=> "UECodeGen_Private::EPropertyGenFlags::Object | UECodeGen_Private::EPropertyGenFlags::ObjectPtr",

				UhtObjectCppForm.TSoftObjectPtr
					=> "UECodeGen_Private::EPropertyGenFlags::SoftObject",

				UhtObjectCppForm.TSoftClassPtr
					=> "UECodeGen_Private::EPropertyGenFlags::SoftClass",

				_
					=> throw new NotImplementedException(),
			};
		}

		/// <summary>
		/// Return the P_GET macro text based on the cpp form
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns>P_GET macro text</returns>
		/// <exception cref="NotImplementedException"></exception>
		public static string GetPGetMacroText(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
					return "OBJECT";

				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
					return "OBJECTPTR";

				case UhtObjectCppForm.TSoftObjectPtr:
					return "SOFTOBJECT";

				case UhtObjectCppForm.TSoftClassPtr:
					return "SOFTCLASS";

				case UhtObjectCppForm.TSubclassOf:
					return "OBJECT";

				case UhtObjectCppForm.VerseType:
					return "VERSETYPE";

				case UhtObjectCppForm.VerseSubtype:
					return "VERSESUBTYPE";

				case UhtObjectCppForm.VerseCastableType:
					return "VERSECASTABLETYPE";

				case UhtObjectCppForm.VerseCastableSubtype:
					return "VERSECASTABLESUBTYPE";

				case UhtObjectCppForm.VerseConcreteType:
					return "VERSECONCRETETYPE";

				case UhtObjectCppForm.VerseConcreteSubtype:
					return "VERSECONCRETESUBTYPE";

				case UhtObjectCppForm.VerseCastableConcreteType:
					return "VERSECASTABLECONCRETETYPE";

				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return "VERSECASTABLECONCRETESUBTYPE";

				case UhtObjectCppForm.TInterfaceInstance:
					return "INTERFACEINSTANCE";

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Returns the PGET argument type for the given CppForm
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public static UhtPGetArgumentType GetPGetArgumentType(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteType:
					return UhtPGetArgumentType.None;

				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TSubclassOf:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
				case UhtObjectCppForm.TInterfaceInstance:
					return UhtPGetArgumentType.TypeText;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Return true if this form needs the NO_PTR option
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		public static bool GetPGetPassAsNoPtr(this UhtObjectCppForm cppForm)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TInterfaceInstance:
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					return false;

				case UhtObjectCppForm.TSubclassOf:
					return true;

				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Based on the cpp form, update the property settings
		/// </summary>
		/// <param name="cppForm">The form in question</param>
		/// <param name="property">Property to be updated</param>
		public static (UhtClass, UhtClass?) InitializeProperty(this UhtObjectCppForm cppForm, UhtObjectPropertyBase property)
		{
			switch (cppForm)
			{
				case UhtObjectCppForm.NativeObject:
					return (property.ReferencedClass, null);

				case UhtObjectCppForm.NativeClass:
					return (property.ReferencedClass, property.Session.UObject);

				case UhtObjectCppForm.TObjectPtrObject:
					property.PropertyFlags |= EPropertyFlags.TObjectPtr | EPropertyFlags.UObjectWrapper;
					property.PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef;
					return (property.ReferencedClass, null);

				case UhtObjectCppForm.TObjectPtrClass:
					property.PropertyFlags |= EPropertyFlags.TObjectPtr | EPropertyFlags.UObjectWrapper;
					property.PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef;
					return (property.ReferencedClass, property.Session.UObject);

				case UhtObjectCppForm.TSoftObjectPtr:
					property.PropertyFlags |= EPropertyFlags.UObjectWrapper;
					return (property.ReferencedClass, null);

				case UhtObjectCppForm.TSoftClassPtr:
					property.PropertyFlags |= EPropertyFlags.UObjectWrapper;
					return (property.Session.UClass, property.ReferencedClass);

				case UhtObjectCppForm.TInterfaceInstance:
					property.PropertyCaps |= UhtPropertyCaps.SupportsVerse;
					property.MetaData.Add("ObjectMustImplement", property.ReferencedClass.PathName);
					return (property.Session.UObject, null);

				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					property.PropertyCaps |= UhtPropertyCaps.SupportsVerse;
					property.MetaData.Add("AllowAbstract", "True");
					if (property.ReferencedClass.ClassType == UhtClassType.Interface)
					{
						property.MetaData.Add("MustImplement", property.ReferencedClass.PathName);
						return (property.Session.UClass, property.Session.UObject);
					}
					return (property.Session.UClass, property.ReferencedClass);

				case UhtObjectCppForm.TSubclassOf:
					property.PropertyFlags |= EPropertyFlags.UObjectWrapper;
					return (property.Session.UClass, property.ReferencedClass);

				default:
					throw new NotImplementedException();
			}
		}
	}
	#endregion

	/// <summary>
	/// Helper struct used to create a unique argument list that performs no type specific validation
	/// </summary>
	public record struct UhtNoValidateConstruct() { }

	/// <summary>
	/// FObjectPropertyBase
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "ObjectPropertyBase", IsProperty = true)]
	public abstract class UhtObjectPropertyBase : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "ObjectPropertyBase";

		/// <inheritdoc/>
		protected override string CppTypeText => "Object";

		/// <inheritdoc/>
		protected override string PGetMacroText => CppForm.GetPGetMacroText();

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => CppForm.GetPGetArgumentType();

		/// <inheritdoc/>
		protected override bool PGetPassAsNoPtr => CppForm.GetPGetPassAsNoPtr();

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => CppForm.GetCodeGenParamsStruct();

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => CppForm.GetCodeGenParamsFlags();


		/// <summary>
		/// Originally referenced class when property is created.
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass ReferencedClass { get; set; }

		/// <summary>
		/// Referenced UCLASS that might later be changed depending on the type (i.e. Verse's TInterfaceInstance)
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass Class { get; set; }

		/// <summary>
		/// Referenced UCLASS for class properties
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass? MetaClass { get; set; }

		/// <summary>
		/// The form that the object property appeared in the code
		/// </summary>
		public UhtObjectCppForm CppForm { get; init; }

		/// <summary>
		/// Construct a property
		/// </summary>
		/// <param name="_">Avoid validation</param>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Source code form of the property</param>
		/// <param name="referencedClass">Referenced UCLASS</param>
		/// <param name="extraFlags">Extra flags to apply to the property.</param>
		protected UhtObjectPropertyBase(UhtNoValidateConstruct _, UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass, EPropertyFlags extraFlags = EPropertyFlags.None) : base(propertySettings)
		{
			CppForm = cppForm;
			ReferencedClass = referencedClass;
			PropertyFlags |= extraFlags;

			// Based on the form, perform more initialization
			(Class, MetaClass) = CppForm.InitializeProperty(this);

			// This applies to EVERYTHING including raw pointer
			// Imply const if it's a parameter that is a pointer to a const class
			// NOTE: We shouldn't be automatically adding const param because in some cases with functions and blueprint native event, the 
			// generated code won't match.  For now, just disabled the auto add in that case and check for the error in the validation code.
			// Otherwise, the user is not warned and they will get compile errors.
			if (propertySettings.PropertyCategory != UhtPropertyCategory.Member && Class.ClassFlags.HasAnyFlags(EClassFlags.Const) && !propertySettings.Options.HasAnyFlags(UhtPropertyOptions.NoAutoConst))
			{
				PropertyFlags |= EPropertyFlags.ConstParm;
			}

			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TNonNullPtr))
			{
				PropertyCaps |= UhtPropertyCaps.SupportsVerse;
			}

			PropertyCaps &= ~(UhtPropertyCaps.CanHaveConfig);
			if (Session.Config!.AreRigVMUObjectPropertiesEnabled)
			{
				PropertyCaps |= UhtPropertyCaps.SupportsRigVM;
			}
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtObjectPropertyBase(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ReferencedClass = (reader.ReadType() as UhtClass)!;
			Class = (reader.ReadType() as UhtClass)!;
			MetaClass = reader.ReadType() as UhtClass;
			CppForm = (UhtObjectCppForm)reader.ReadUInt8();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(ReferencedClass);
			writer.WriteType(Class);
			writer.WriteType(MetaClass);
			writer.WriteUInt8((byte)CppForm);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					if (Class.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced))
					{
						PropertyFlags |= (EPropertyFlags.InstancedReference | EPropertyFlags.ExportObject) & ~DisallowPropertyFlags;
					}
					break;
			}
			return results;
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return !DisallowPropertyFlags.HasAnyFlags(EPropertyFlags.InstancedReference) && Class.HierarchyHasAnyClassFlags(EClassFlags.DefaultToInstanced);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(Class, UhtSingletonType.Unregistered);
			collector.AddObjectReference(MetaClass, UhtSingletonType.Unregistered);
			if (addForwardDeclarations)
			{
				collector.AddForwardDeclaration(Class);
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			yield return Class;
			if (MetaClass != null)
			{
				yield return MetaClass;
			}
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			if (defaultValueReader.TryOptional("NULL") ||
				defaultValueReader.TryOptional("nullptr") ||
				(defaultValueReader.TryOptionalConstInt(out int value) && value == 0))
			{
				innerDefaultValue.Append("None");
				return true;
			}
			if (defaultValueReader.TryOptional("{"))
			{
				defaultValueReader.Require("}");
				innerDefaultValue.Append("None");
				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override void ValidateDeprecated()
		{
			ValidateDeprecatedClass(Class);
			ValidateDeprecatedClass(MetaClass);
		}

		/// <inheritdoc/>
		public override bool MustBeConstArgument([NotNullWhen(true)] out UhtType? errorType)
		{
			errorType = Class;
			return Class.ClassFlags.HasAnyFlags(EClassFlags.Const);
		}

		/// <inheritdoc/>
		protected override bool NeedsGCBarrierWhenPassedToFunctionImpl(UhtFunction function)
		{
			Type type = GetType();
			return type == typeof(UhtObjectProperty)
				|| (type == typeof(UhtClassProperty) && CppForm != UhtObjectCppForm.TSubclassOf);
		}

		/// <summary>
		/// Append the mangled name for all object types
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="type">Underlying type</param>
		/// <returns>Builder</returns>
		protected StringBuilder AppendVerseMangledType(StringBuilder builder, UhtClass type)
		{
			switch (CppForm)
			{
				case UhtObjectCppForm.VerseType:
				case UhtObjectCppForm.VerseSubtype:
				case UhtObjectCppForm.VerseCastableType:
				case UhtObjectCppForm.VerseCastableSubtype:
				case UhtObjectCppForm.VerseConcreteType:
				case UhtObjectCppForm.VerseConcreteSubtype:
				case UhtObjectCppForm.VerseCastableConcreteType:
				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					builder.Append(SourceName);
					break;

				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
				case UhtObjectCppForm.TSoftObjectPtr:
				case UhtObjectCppForm.TSoftClassPtr:
				case UhtObjectCppForm.TInterfaceInstance:
				case UhtObjectCppForm.TSubclassOf:
					builder.AppendVerseScopeAndName(type, UhtVerseNameMode.Default);
					break;

				default:
					throw new NotImplementedException();
			}
			return builder;
		}

		/// <summary>
		/// Append the template declaration for the given type
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <returns></returns>
		/// <exception cref="NotImplementedException"></exception>
		protected StringBuilder AppendTemplateType(StringBuilder builder)
		{
			bool isNonNullPtr = PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TNonNullPtr);
			bool isIsWrappedType = isNonNullPtr;
			if (isNonNullPtr)
			{
				builder.Append("TNonNullPtr<");
			}
			switch (CppForm)
			{
				case UhtObjectCppForm.NativeObject:
				case UhtObjectCppForm.NativeClass:
					builder.Append(Class.Namespace.FullSourceName).Append(Class.SourceName);
					if (!isIsWrappedType)
					{
						builder.Append('*');
					}
					break;

				case UhtObjectCppForm.TObjectPtrObject:
				case UhtObjectCppForm.TObjectPtrClass:
					builder.Append("TObjectPtr<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.TSoftObjectPtr:
					builder.Append("TSoftObjectPtr<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.TSoftClassPtr:
					builder.Append("TSoftObjectPtr<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.TSubclassOf:
					builder.Append("TSubclassOf<");
					if (MetaClass != null)
					{
						builder.Append(MetaClass.Namespace.FullSourceName).Append(MetaClass.SourceName);
					}
					else
					{
						builder.Append("UClass");
					}
					builder.Append('>');
					break;

				case UhtObjectCppForm.VerseType:
					builder.Append("verse::type");
					break;

				case UhtObjectCppForm.VerseSubtype:
					builder.Append("verse::subtype<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.VerseCastableType:
					builder.Append("verse::castable_type");
					break;

				case UhtObjectCppForm.VerseCastableSubtype:
					builder.Append("verse::castable_subtype<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.VerseConcreteType:
					builder.Append("verse::concrete_type");
					break;

				case UhtObjectCppForm.VerseConcreteSubtype:
					builder.Append("verse::concrete_subtype<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.VerseCastableConcreteType:
					builder.Append("verse::castable_concrete_type");
					break;

				case UhtObjectCppForm.VerseCastableConcreteSubtype:
					builder.Append("verse::castable_concrete_subtype<").Append(Class.Namespace.FullSourceName).Append(Class.SourceName).Append('>');
					break;

				case UhtObjectCppForm.TInterfaceInstance:
					builder.Append("TInterfaceInstance<").Append(ReferencedClass.Namespace.FullSourceName).AppendClassSourceNameOrInterfaceProxyName(ReferencedClass).Append('>');
					break;

				default:
					throw new NotImplementedException();
			}
			if (isNonNullPtr)
			{
				builder.Append('>');
			}
			return builder;
		}

		#region Keywords
		[UhtPropertyType(Keyword = "TSubclassOf")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtClassProperty? SubclassOfProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? metaClass = args.ParseTemplateClass();
			if (metaClass == null)
			{
				return null;
			}

			// With TSubclassOf, MetaClass is used as a class limiter.  
			return new UhtClassProperty(args.PropertySettings, UhtObjectCppForm.TSubclassOf, metaClass);
		}

		[UhtPropertyType(Keyword = "TNonNullPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? TNonNullPtrType(UhtPropertyResolveArgs args)
		{
			return args.ParseWrappedType(EPropertyFlags.NonNullable, UhtPropertyExportFlags.TNonNullPtr);
		}

		[UhtPropertyType(Keyword = "TObjectPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? ObjectPtrProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.Normal, (referencedClass, typeStartPos) =>
			{
				args.ConditionalLogPointerUsage(args.Config.ObjectPtrMemberBehavior, "ObjectPtr", typeStartPos, null);
				if (referencedClass.IsChildOf(args.Session.UClass))
				{
					// UObject specifies that there is no limiter
					return new UhtClassProperty(args.PropertySettings, UhtObjectCppForm.TObjectPtrClass, referencedClass);
				}
				else
				{
					return new UhtObjectProperty(args.PropertySettings, UhtObjectCppForm.TObjectPtrObject, referencedClass);
				}
			});
		}

		[UhtPropertyType(Keyword = "verse::type", Options = UhtPropertyTypeOptions.Simple)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtClassProperty? VerseTypeProperty(UhtPropertyResolveArgs args)
		{
			return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseType, args.Session.UObject);
		}

		[UhtPropertyType(Keyword = "verse::subtype")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseSubTypeProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.Normal, (referencedClass, typeStartPos) =>
			{
				return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseSubtype, referencedClass);
			});
		}

		[UhtPropertyType(Keyword = "verse::castable_type", Options = UhtPropertyTypeOptions.Simple)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtClassProperty? VerseCastableTypeProperty(UhtPropertyResolveArgs args)
		{
			return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseCastableType, args.Session.UObject);
		}

		[UhtPropertyType(Keyword = "verse::castable_subtype")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseCastableSubtypeProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.Normal, (referencedClass, typeStartPos) =>
			{
				return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseCastableSubtype, referencedClass);
			});
		}

		[UhtPropertyType(Keyword = "verse::concrete_type", Options = UhtPropertyTypeOptions.Simple)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtClassProperty? VerseConcreteTypeProperty(UhtPropertyResolveArgs args)
		{
			return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseConcreteType, args.Session.UObject);
		}

		[UhtPropertyType(Keyword = "verse::concrete_subtype")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseConcreteSubtypeProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.Normal, (referencedClass, typeStartPos) =>
			{
				return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseConcreteSubtype, referencedClass);
			});
		}

		[UhtPropertyType(Keyword = "verse::castable_concrete_type", Options = UhtPropertyTypeOptions.Simple)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtClassProperty? VerseCastableConcreteTypeProperty(UhtPropertyResolveArgs args)
		{
			return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseCastableConcreteType, args.Session.UObject);
		}

		[UhtPropertyType(Keyword = "verse::castable_concrete_subtype")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? VerseCastableConcreteSubtypeProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.Normal, (referencedClass, typeStartPos) =>
			{
				return new UhtVerseClassProperty(args.PropertySettings, UhtObjectCppForm.VerseCastableConcreteSubtype, referencedClass);
			});
		}

		[UhtPropertyType(Keyword = "TInterfaceInstance")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? TInterfaceInstanceProperty(UhtPropertyResolveArgs args)
		{
			return args.ParseTemplateObjectProperty(UhtTemplateObjectMode.NativeInterfaceProxy, (referencedClass, typeStartPos) =>
			{
				return new UhtObjectProperty(args.PropertySettings, UhtObjectCppForm.TInterfaceInstance, referencedClass);
			});
		}
		#endregion
	}
}
