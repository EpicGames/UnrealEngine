// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents the FOptionalProperty engine type
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "OptionalProperty", IsProperty = true)]
	public class UhtOptionalProperty : UhtContainerBaseProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "OptionalProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TOptional";

		/// <inheritdoc/>
		protected override string PGetMacroText => "TOPTIONAL";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FGenericPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Optional";

		/// <summary>
		/// Construct a new Optional property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="value">Inner property value</param>
		public UhtOptionalProperty(UhtPropertySettings propertySettings, UhtProperty value) : base(propertySettings, value)
		{
			// If the creation of the value property set more flags, then copy those flags to ourselves
			PropertyFlags |= ValueProperty.PropertyFlags & (EPropertyFlags.UObjectWrapper | EPropertyFlags.TObjectPtr);

			if (ValueProperty.MetaData.ContainsKey(UhtNames.NativeConst))
			{
				MetaData.Add(UhtNames.NativeConstTemplateArg, "");
				ValueProperty.MetaData.Remove(UhtNames.NativeConst);
			}

			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.SupportsVerse;
			PropertyCaps &= ~(UhtPropertyCaps.CanBeContainerKey);
			UpdateCaps();

			ValueProperty.SourceName = SourceName;
			ValueProperty.EngineName = EngineName;
			ValueProperty.PropertyFlags = (ValueProperty.PropertyFlags & EPropertyFlags.PropagateKeepInInner) | (PropertyFlags & EPropertyFlags.PropagateToOptionalInner);
			ValueProperty.Outer = this;
			ValueProperty.MetaData.Clear();
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtOptionalProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					PropertyFlags |= ResolveAndReturnNewFlags(ValueProperty, phase);
					MetaData.Add(ValueProperty.MetaData);
					ValueProperty.PropertyFlags = (ValueProperty.PropertyFlags & EPropertyFlags.PropagateKeepInInner) | (PropertyFlags & EPropertyFlags.PropagateToOptionalInner);
					ValueProperty.MetaData.Clear();
					PropagateFlagsFromInnerAndHandlePersistentInstanceMetadata(this, MetaData, ValueProperty);
					UpdateCaps();
					break;
			}
			return results;
		}

		private void UpdateCaps()
		{
			PropertyCaps &= ~(UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.CanExposeOnSpawn);
			PropertyCaps |= ValueProperty.PropertyCaps & UhtPropertyCaps.CanExposeOnSpawn;
			if (ValueProperty.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsParameterSupportedByBlueprint))
			{
				PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			foreach (UhtType type in ValueProperty.EnumerateReferencedTypes())
			{
				yield return type;
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.SparseShort:
					builder.Append("TOptional");
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.AppendFunctionThunkParameterArrayType(ValueProperty, true);
					break;

				case UhtPropertyTextType.VerseMangledType:
					builder.Append('?');
					if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.VerseNamed))
					{
						builder.Append($"{VerseName}:");
					}
					builder.AppendPropertyVerseMangledType(ValueProperty);
					break;

				default:
					builder.Append("TOptional<").AppendPropertyText(ValueProperty, textType, true).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtChildProperty> EnumerateChildProperties(UhtCppIdentifier identifier, UhtChildPropertyOrder order)
		{
			yield return new(ValueProperty, identifier.AppendSuffix("_Inner"), "0");
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			ValueProperty.AppendObjectHashes(builder, startingLength, context);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.AppendPropertyText(this, UhtPropertyTextType.Construction).Append("()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest))
				{
					if (RefQualifier != UhtPropertyRefQualifier.ConstRef && !IsStaticArray)
					{
						this.LogError("Replicated TOptional parameters must be passed by const reference");
					}
				}
			}
		}

		/// <inheritdoc/>
		public override void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			base.Validate(outerStruct, outermostProperty, options);

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
			{
				this.LogError("Replicated Optionals with MemoryImageAllocators are not yet supported");
			}

			if (ValueProperty is UhtStructProperty structProperty)
			{
				if (structProperty.ScriptStruct == outerStruct)
				{
					this.LogError($"'Struct' recursion via optionals is unsupported for properties.");
				}
			}
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtOptionalProperty otherOptional)
			{
				return ValueProperty.IsSameType(otherOptional.ValueProperty);
			}
			return false;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TOptional")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static UhtOptionalProperty? OptionalProperty(UhtPropertyResolveArgs args)
		{
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			using UhtMessageContext tokenContext = new("TOptional");
			if (!args.SkipExpectedType())
			{
				return null;
			}
			tokenReader.Require('<');

			// Parse the value type
			UhtProperty? value = args.ParseTemplateParam(propertySettings.SourceName);
			if (value == null)
			{
				return null;
			}

			if (!value.PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeOptionalValue))
			{
				tokenReader.LogError($"The type \'{value.GetUserFacingDecl()}\' can not be used as a value in a TOptional");
			}

			tokenReader.Require('>');

			//@TODO: Prevent sparse delegate types from being used in a container

			return new UhtOptionalProperty(propertySettings, value);
		}
		#endregion
	}
}
