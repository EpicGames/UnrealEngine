// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FSetProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "SetProperty", IsProperty = true)]
	public class UhtSetProperty : UhtContainerBaseProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "SetProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "TSet";

		/// <inheritdoc/>
		protected override string PGetMacroText => "TSET";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FSetPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Set";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="value">Property key</param>
		public UhtSetProperty(UhtPropertySettings propertySettings, UhtProperty value) : base(propertySettings, value)
		{
			// If the creation of the value property set more flags, then copy those flags to ourselves
			PropertyFlags |= ValueProperty.PropertyFlags & (EPropertyFlags.UObjectWrapper | EPropertyFlags.TObjectPtr);

			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef;
			PropertyCaps &= ~(UhtPropertyCaps.CanBeContainerValue | UhtPropertyCaps.CanBeContainerKey);
			UpdateCaps();

			ValueProperty.SourceName = SourceName;
			ValueProperty.EngineName = EngineName;
			ValueProperty.PropertyFlags = (ValueProperty.PropertyFlags & EPropertyFlags.PropagateKeepInInner) | (PropertyFlags & EPropertyFlags.PropagateToSetElement);
			ValueProperty.Outer = this;
			ValueProperty.MetaData.Clear();
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtSetProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					PropertyFlags |= ResolveAndReturnNewFlags(ValueProperty, phase);
					MetaData.Add(ValueProperty.MetaData);
					ValueProperty.MetaData.Clear();
					ValueProperty.PropertyFlags = (ValueProperty.PropertyFlags & EPropertyFlags.PropagateKeepInInner) | (PropertyFlags & EPropertyFlags.PropagateToSetElement);
					UpdateCaps();
					PropagateFlagsFromInnerAndHandlePersistentInstanceMetadata(this, MetaData, ValueProperty);
					break;
			}
			return results;
		}

		private void UpdateCaps()
		{
			PropertyCaps &= ~(UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.CanExposeOnSpawn);
			PropertyCaps |= ValueProperty.PropertyCaps & UhtPropertyCaps.CanExposeOnSpawn;
			if (ValueProperty.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsParameterSupportedByBlueprint))
			{
				PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			foreach (UhtType type in ValueProperty.EnumerateReferencedTypes())
			{
				yield return type;
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.SparseShort:
					builder.Append("TSet");
					break;

				case UhtPropertyTextType.FunctionThunkParameterArgType:
					builder.AppendFunctionThunkParameterArrayType(ValueProperty, true);
					break;

				default:
					builder.Append("TSet<").AppendPropertyText(ValueProperty, textType, true).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtChildProperty> EnumerateChildProperties(UhtCppIdentifier identifier, UhtChildPropertyOrder order)
		{
			yield return new(ValueProperty, identifier.AppendSuffix("_ElementProp"));
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendCommonDef(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			if (ValueProperty is UhtStructProperty structProperty)
			{
				builder
					.AppendTabs(tabs)
					.Append("static_assert(TModels_V<CGetTypeHashable, ")
					.Append(structProperty.ScriptStruct.FullyQualifiedSourceName)
					.Append(">, \"The structure '")
					.Append(structProperty.ScriptStruct.FullyQualifiedSourceName)
					.Append("' is used in a TSet but does not have a GetValueTypeHash defined\");\r\n");
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			base.AppendConstInitDefExtra(builder, context, identifier);
			builder.Append("(int32)sizeof(").AppendPropertyText(ValueProperty, UhtPropertyTextType.Generic).Append("), ");
			builder.Append("(int32)alignof(").AppendPropertyText(ValueProperty, UhtPropertyTextType.Generic).Append("), ");
			return builder;
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			ValueProperty.AppendObjectHashes(builder, startingLength, context);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.AppendPropertyText(this, UhtPropertyTextType.Construction).Append("()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtSetProperty otherSet)
			{
				return ValueProperty.IsSameType(otherSet.ValueProperty);
			}
			return false;
		}

		///<inheritdoc/>
		public override bool ValidateStructPropertyOkForNet(UhtProperty referencingProperty)
		{
			if (!PropertyFlags.HasAnyFlags(EPropertyFlags.RepSkip))
			{
				referencingProperty.LogError($"Sets are not supported for Replication or RPCs.  Set '{SourceName}' in '{Outer?.SourceName}'.  Origin '{referencingProperty.SourceName}'");
				return false;
			}
			return true;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest | EFunctionFlags.NetResponse))
				{
					this.LogError("Sets are not supported in an RPC.");
				}
			}
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TSet")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static UhtSetProperty? SetProperty(UhtPropertyResolveArgs args)
		{
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			using UhtMessageContext tokenContext = new("TSet");
			if (!args.SkipExpectedType())
			{
				return null;
			}
			tokenReader.Require('<');

			// Parse the value type
			UhtProperty? value = args.ParseTemplateParam("Value");
			if (value == null)
			{
				return null;
			}

			if (!value.PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeContainerKey))
			{
				tokenReader.LogError($"The type \'{value.GetUserFacingDecl()}\' can not be used as a key in a TSet");
			}

			if (propertySettings.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
			{
				tokenReader.LogError("Replicated sets are not supported.");
			}

			if (tokenReader.TryOptional(','))
			{
				// If we found a comma, read the next thing, assume it's a keyfuncs, and report that
				UhtToken keyFuncToken = tokenReader.GetIdentifier();
				throw new UhtException(tokenReader, $"Found '{keyFuncToken.Value}' - explicit KeyFuncs are not supported in TSet properties.");
			}

			tokenReader.Require('>');

			//@TODO: Prevent sparse delegate types from being used in a container

			return new UhtSetProperty(propertySettings, value);
		}
		#endregion
	}
}
