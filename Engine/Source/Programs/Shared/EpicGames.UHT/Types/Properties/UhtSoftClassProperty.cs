// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FSoftClassProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "SoftClassProperty", IsProperty = true)]
	public class UhtSoftClassProperty : UhtSoftObjectProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "SoftClassProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "SoftClassPtr";

		/// <summary>
		/// Construct a new class property
		/// </summary>
		/// <param name="propertySettings">Property setting</param>
		/// <param name="referencedClass">Referenced class</param>
		public UhtSoftClassProperty(UhtPropertySettings propertySettings, UhtClass referencedClass)
			: base(propertySettings, UhtObjectCppForm.TSoftClassPtr, referencedClass)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtSoftClassProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(MetaClass, UhtSingletonType.Unregistered);
			if (addForwardDeclarations && MetaClass != null)
			{
				collector.AddForwardDeclaration(MetaClass);
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				default:
					builder.Append("TSoftClassPtr<").Append(MetaClass?.SourceName).Append("> ");
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, MetaClass, UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Session.UClass, true)}, {ConstInitSingletonRef(context, MetaClass, true)}, ");
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TSoftClassPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtSoftClassProperty? SoftClassPtrProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? metaClass = args.ParseTemplateClass();
			if (metaClass == null)
			{
				return null;
			}

			// With TSubclassOf, MetaClass is used as a class limiter.  
			return new UhtSoftClassProperty(args.PropertySettings, metaClass);
		}
		#endregion
	}
}
