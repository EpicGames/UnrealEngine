// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FSoftObjectProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "SoftObjectProperty", IsProperty = true)]
	public class UhtSoftObjectProperty : UhtObjectPropertyBase
	{
		/// <inheritdoc/>
		public override string EngineClassName => "SoftObjectProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "SoftObjectPtr";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="referencedClass">UCLASS being referenced</param>
		public UhtSoftObjectProperty(UhtPropertySettings propertySettings, UhtClass referencedClass)
			: this(propertySettings, UhtObjectCppForm.TSoftObjectPtr, referencedClass)
		{
		}

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="cppForm">Form of the object property</param>
		/// <param name="referencedClass">UCLASS being referenced</param>
		protected UhtSoftObjectProperty(UhtPropertySettings propertySettings, UhtObjectCppForm cppForm, UhtClass referencedClass)
			: base(new UhtNoValidateConstruct { }, propertySettings, cppForm, referencedClass)
		{
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.RequiresNullConstructorArg | UhtPropertyCaps.CanExposeOnSpawn |
				UhtPropertyCaps.CanHaveConfig | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtSoftObjectProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NULL");
			return builder;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				default:
					builder.Append("TSoftObjectPtr<").Append(Class.SourceName).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Class, Exporters.CodeGen.UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Class, true)}, ");
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtSoftObjectProperty otherObject)
			{
				return Class == otherObject.Class && MetaClass == otherObject.MetaClass;
			}
			return false;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TSoftObjectPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtSoftObjectProperty? SoftObjectPtrProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? propertyClass = args.ParseTemplateObject(UhtTemplateObjectMode.Normal);
			if (propertyClass == null)
			{
				return null;
			}

			if (propertyClass.IsChildOf(propertyClass.Session.UClass))
			{
				args.TokenReader.LogError("Class variables cannot be stored in TSoftObjectPtr, use TSoftClassPtr instead.");
			}

			return new UhtSoftObjectProperty(args.PropertySettings, propertyClass);
		}
		#endregion
	}
}
