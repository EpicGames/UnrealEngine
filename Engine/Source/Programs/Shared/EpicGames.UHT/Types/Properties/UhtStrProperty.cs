// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FStrProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "StrProperty", IsProperty = true)]
	public class UhtStrProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "StrProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FString";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.EngineClass;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FStrPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Str";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtStrProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint | UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtStrProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("TEXT(\"\")");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			StringView value = defaultValueReader.GetTypedConstString("FString");
			innerDefaultValue.Append(value);
			return true;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtStrProperty;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest))
				{
					if (RefQualifier != UhtPropertyRefQualifier.ConstRef && !IsStaticArray)
					{
						this.LogError("Replicated FString parameters must be passed by const reference");
					}
				}
			}
		}

		[UhtPropertyType(Keyword = "FString")]
		[UhtPropertyType(Keyword = "FMemoryImageString", Options = UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtStrProperty? StrProperty(UhtPropertyResolveArgs args)
		{
			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			if (!args.SkipExpectedType())
			{
				return null;
			}
			UhtStrProperty property = new(propertySettings);
			if (property.PropertyCategory != UhtPropertyCategory.Member)
			{
				if (tokenReader.TryOptional('&'))
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
					{
						// 'const FString& Foo' came from 'FString' in .uc, no flags
						property.PropertyFlags &= ~EPropertyFlags.ConstParm;

						// We record here that we encountered a const reference, because we need to remove that information from flags for code generation purposes.
						property.RefQualifier = UhtPropertyRefQualifier.ConstRef;
					}
					else
					{
						// 'FString& Foo' came from 'out FString' in .uc
						property.PropertyFlags |= EPropertyFlags.OutParm;

						// And we record here that we encountered a non-const reference here too.
						property.RefQualifier = UhtPropertyRefQualifier.NonConstRef;
					}
				}
			}
			return property;
		}
	}
}
