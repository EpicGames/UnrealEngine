// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// FStructProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "StructProperty", IsProperty = true)]
	public class UhtStructProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "StructProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "invalid";

		/// <inheritdoc/>
		protected override string PGetMacroText => "STRUCT";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FStructPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Struct";

		/// <summary>
		/// USTRUCT referenced by the property
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtScriptStruct>))]
		public UhtScriptStruct ScriptStruct { get; set; }

		/// <summary>
		/// Construct property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="scriptStruct">USTRUCT being referenced</param>
		public UhtStructProperty(UhtPropertySettings propertySettings, UhtScriptStruct scriptStruct) : base(propertySettings)
		{
			ScriptStruct = scriptStruct;
			HeaderFile.AddReferencedHeader(scriptStruct);
			PropertyCaps |= UhtPropertyCaps.SupportsRigVM | UhtPropertyCaps.SupportsVerse;

			UpdateCaps();
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtStructProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ScriptStruct = (reader.ReadType() as UhtScriptStruct)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteType(ScriptStruct);
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);
			switch (phase)
			{
				case UhtResolvePhase.Final:
					ScriptStruct.Resolve(phase);
					if (ScanForInstancedReferenced(true))
					{
						PropertyFlags |= EPropertyFlags.ContainsInstancedReference;
					}
					UpdateCaps();
					break;
			}
			return results;
		}

		private void UpdateCaps()
		{
			if (ScriptStruct.HasNoOpConstructor)
			{
				PropertyCaps |= UhtPropertyCaps.RequiresNullConstructorArg;
			}

			// There is a good chance Blueprint type was set during property resolve phase.  Check this flag again.
			const UhtPropertyCaps BlueprintCaps = UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			if (!PropertyCaps.HasExactFlags(BlueprintCaps, BlueprintCaps))
			{
				if (ScriptStruct.MetaData.GetBoolean(UhtNames.BlueprintType))
				{
					PropertyCaps |= BlueprintCaps;
				}
				else if (ScriptStruct.MetaData.GetBooleanHierarchical(UhtNames.BlueprintType))
				{
					PropertyCaps |= UhtPropertyCaps.IsParameterSupportedByBlueprint | UhtPropertyCaps.IsMemberSupportedByBlueprint;
				}
			}
		}

		/// <inheritdoc/>
		public override bool ScanForInstancedReferenced(bool deepScan)
		{
			return ScriptStruct.ScanForInstancedReferenced(deepScan);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			collector.AddObjectReference(ScriptStruct, UhtSingletonType.Registered);
			if (addForwardDeclarations && !ScriptStruct.IsCoreType)
			{
				collector.AddForwardDeclaration(ScriptStruct);
			}
		}

		/// <inheritdoc/>
		public override IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			yield return ScriptStruct;
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.VerseMangledType:
					builder.AppendVerseScopeAndName(ScriptStruct, UhtVerseNameMode.Default);
					break;

				default:
					builder.Append(ScriptStruct.FullyQualifiedSourceName);
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, ScriptStruct, UhtSingletonType.Registered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			builder.Append($"(int32)Align(sizeof({ScriptStruct.Namespace.FullSourceName}{ScriptStruct.SourceName}), alignof({ScriptStruct.Namespace.FullSourceName}{ScriptStruct.SourceName})), ");
			builder.Append($"{ConstInitSingletonRef(context, ScriptStruct, true)}, ");
			return builder;
		}

		/// <inheritdoc/>
		public override void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
			builder.AppendObjectHash(startingLength, this, context, ScriptStruct);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			bool hasNoOpConstructor = ScriptStruct.HasNoOpConstructor;
			if (isInitializer && hasNoOpConstructor)
			{
				builder.Append("ForceInit");
			}
			else
			{
				builder.AppendPropertyText(this, UhtPropertyTextType.Construction);
				if (hasNoOpConstructor)
				{
					builder.Append("(ForceInit)");
				}
				else
				{
					builder.Append("()");
				}
			}
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			if (!Session.TryGetStructDefaultValue(ScriptStruct.SourceName, out UhtStructDefaultValue structDefaultValue))
			{
				structDefaultValue = Session.DefaultStructDefaultValue;
			}
			return structDefaultValue.Delegate(this, defaultValueReader, innerDefaultValue);
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtStructProperty otherObject)
			{
				return ScriptStruct == otherObject.ScriptStruct;
			}
			return false;
		}

		/// <inheritdoc/>
		protected override void ValidateFunctionArgument(UhtFunction function, UhtValidationOptions options)
		{
			base.ValidateFunctionArgument(function, options);

			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest | EFunctionFlags.NetResponse))
				{
					Session.ValidateScriptStructOkForNet(this, ScriptStruct);
				}
			}
		}

		/// <inheritdoc/>
		protected override void ValidateMember(UhtStruct structObj, UhtValidationOptions options)
		{
			base.ValidateMember(structObj, options);
			if (PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
			{
				Session.ValidateScriptStructOkForNet(this, ScriptStruct);
			}
		}

		///<inheritdoc/>
		public override bool ValidateStructPropertyOkForNet(UhtProperty referencingProperty)
		{
			return referencingProperty.Session.ValidateScriptStructOkForNet(referencingProperty, ScriptStruct);
		}

		///<inheritdoc/>
		public override bool IsAllowedInOptionalClass([NotNullWhen(false)] out string? propPath)
		{
			foreach (UhtType child in ScriptStruct.Children)
			{
				if (child is UhtProperty property)
				{
					// if EditorOnly, and not allowed, don't bother going into the children, but if we are allowed,
					// we still need to go in because a child property may be disallowed, and when saving out, we
					// don't propagate the ALlowedInOptional flag down to children during serialization
					bool bPropertyHasAllowed = property.MetaData.ContainsKey("AllowedInOptional") || property.Deprecated;
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly) && !bPropertyHasAllowed)
					{
						propPath = SourceName + " / " + property.SourceName;
						return false;
					}
					if (!property.IsAllowedInOptionalClass(out string? innerPropPath))
					{
						propPath = SourceName + " / " + innerPropPath;
						return false;
					}
				}
			}
			propPath = null;
			return true;
		}

		/// <summary>
		/// Perform default, default value parsing
		/// </summary>
		/// <param name="defaultValueReader">Default value reader</param>
		/// <param name="innerDefaultValue">Sanitized default value</param>
		/// <returns></returns>
		public virtual bool DefaultDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			defaultValueReader
				.Require(ScriptStruct.SourceName);
			defaultValueReader
				.Require('(')
				.Require(')');
			innerDefaultValue.Append("()");
			return true;
		}

		#region Structure default value sanitizers
		[UhtStructDefaultValue(Name = "FVector")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool VectorStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			const string Format = "{0:F6},{1:F6},{2:F6}";

			defaultValueReader.Require("FVector");
			if (defaultValueReader.TryOptional("::"))
			{
				switch (defaultValueReader.GetIdentifier().Value.ToString())
				{
					case "ZeroVector": return true;
					case "UpVector": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 0, 1); return true;
					case "ForwardVector": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 1, 0, 0); return true;
					case "RightVector": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 1, 0); return true;
					default: return false;
				}
			}
			else
			{
				defaultValueReader.Require("(");
				if (defaultValueReader.TryOptional("ForceInit"))
				{
				}
				else
				{
					double x, y, z;
					x = y = z = defaultValueReader.GetConstDoubleExpression();
					if (defaultValueReader.TryOptional(','))
					{
						y = defaultValueReader.GetConstDoubleExpression();
						defaultValueReader.Require(',');
						z = defaultValueReader.GetConstDoubleExpression();
					}
					innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, x, y, z);
				}
				defaultValueReader.Require(")");
				return true;
			}
		}

		[UhtStructDefaultValue(Name = "FRotator")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool RotatorStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require("FRotator");
			if (defaultValueReader.TryOptional("::"))
			{
				switch (defaultValueReader.GetIdentifier().Value.ToString())
				{
					case "ZeroRotator": return true;
					default: return false;
				}
			}
			else
			{
				defaultValueReader.Require("(");
				if (defaultValueReader.TryOptional("ForceInit"))
				{
				}
				else
				{
					double x = defaultValueReader.GetConstDoubleExpression();
					defaultValueReader.Require(',');
					double y = defaultValueReader.GetConstDoubleExpression();
					defaultValueReader.Require(',');
					double z = defaultValueReader.GetConstDoubleExpression();
					innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, "{0:F6},{1:F6},{2:F6}", x, y, z);
				}
				defaultValueReader.Require(")");
				return true;
			}
		}

		[UhtStructDefaultValue(Name = "FVector2D")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool Vector2DStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			const string Format = "(X={0:F3},Y={1:F3})";

			defaultValueReader.Require("FVector2D");
			if (defaultValueReader.TryOptional("::"))
			{
				switch (defaultValueReader.GetIdentifier().Value.ToString())
				{
					case "ZeroVector": return true;
					case "UnitVector": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 1.0, 1.0); return true;
					default: return false;
				}
			}
			else
			{
				defaultValueReader.Require("(");
				if (defaultValueReader.TryOptional("ForceInit"))
				{
				}
				else
				{
					double x = defaultValueReader.GetConstDoubleExpression();
					defaultValueReader.Require(',');
					double y = defaultValueReader.GetConstDoubleExpression();
					innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, x, y);
				}
				defaultValueReader.Require(")");
				return true;
			}
		}

		[UhtStructDefaultValue(Name = "FLinearColor")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LinearColorStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			const string Format = "(R={0:F6},G={1:F6},B={2:F6},A={3:F6})";

			defaultValueReader.Require("FLinearColor");
			if (defaultValueReader.TryOptional("::"))
			{
				switch (defaultValueReader.GetIdentifier().Value.ToString())
				{
					case "White": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 1.0, 1.0, 1.0, 1.0); return true;
					case "Gray": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0.5, 0.5, 0.5, 1.0); return true;
					case "Black": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0.0, 0.0, 0.0, 1.0); return true;
					case "Transparent": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0.0, 0.0, 0.0, 0.0); return true;
					case "Red": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 1.0, 0.0, 0.0, 1.0); return true;
					case "Green": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0.0, 1.0, 0.0, 1.0); return true;
					case "Blue": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0.0, 0.0, 1.0, 1.0); return true;
					case "Yellow": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 1.0, 1.0, 0.0, 1.0); return true;
					default: return false;
				}
			}
			else
			{
				defaultValueReader.Require("(");
				if (defaultValueReader.TryOptional("ForceInit"))
				{
				}
				else
				{
					double r = defaultValueReader.GetConstDoubleExpression();
					defaultValueReader.Require(',');
					double g = defaultValueReader.GetConstDoubleExpression();
					defaultValueReader.Require(',');
					double b = defaultValueReader.GetConstDoubleExpression();
					double a = 1.0;
					if (defaultValueReader.TryOptional(','))
					{
						a = defaultValueReader.GetConstDoubleExpression();
					}
					innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, r, g, b, a);
				}
				defaultValueReader.Require(")");
				return true;
			}
		}

		[UhtStructDefaultValue(Name = "FColor")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool ColorStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			const string Format = "(R={0},G={1},B={2},A={3})";

			defaultValueReader.Require("FColor");
			if (defaultValueReader.TryOptional("::"))
			{
				switch (defaultValueReader.GetIdentifier().Value.ToString())
				{
					case "White": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 255, 255, 255, 255); return true;
					case "Black": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 0, 0, 255); return true;
					case "Red": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 255, 0, 0, 255); return true;
					case "Green": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 255, 0, 255); return true;
					case "Blue": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 0, 255, 255); return true;
					case "Yellow": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 255, 255, 0, 255); return true;
					case "Cyan": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 0, 255, 255, 255); return true;
					case "Magenta": innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, 255, 0, 255, 255); return true;
					default: return false;
				}
			}
			else
			{
				defaultValueReader.Require("(");
				if (defaultValueReader.TryOptional("ForceInit"))
				{
				}
				else
				{
					int r = defaultValueReader.GetConstIntExpression();
					defaultValueReader.Require(',');
					int g = defaultValueReader.GetConstIntExpression();
					defaultValueReader.Require(',');
					int b = defaultValueReader.GetConstIntExpression();
					int a = 255;
					if (defaultValueReader.TryOptional(','))
					{
						a = defaultValueReader.GetConstIntExpression();
					}
					innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, Format, r, g, b, a);
				}
				defaultValueReader.Require(")");
				return true;
			}
		}

		[UhtStructDefaultValue(Options = UhtStructDefaultValueOptions.Default)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static bool DefaultStructDefaultValue(UhtStructProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return property.DefaultDefaultValue(defaultValueReader, innerDefaultValue);
		}
		#endregion
	}

	/// <summary>
	/// FStructProperty
	/// </summary>
	[UnrealHeaderTool]
	public class UhtTemplateStructProperty : UhtStructProperty
	{
		/// <summary>
		/// When using the template wrapper pattern that provides a template wrapper to an existing structure that
		/// can reference types, this is the name of the template.  For example, FInstancedStruct has TInstancedStruct
		/// as a wrapper template.
		/// </summary>
		public string TemplateWrapperName { get; init; }

		/// <summary>
		/// The structure being wrapped by the template wrapper
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtScriptStruct>))]
		public UhtScriptStruct TemplateArgumentStruct { get; init; }

		/// <summary>
		/// Construct property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="scriptStruct">USTRUCT being referenced</param>
		/// <param name="templateWrapperName">The name of wrapping template type</param>
		/// <param name="templateArgumentStruct">The path name of the type being managed by the template</param>
		public UhtTemplateStructProperty(UhtPropertySettings propertySettings, UhtScriptStruct scriptStruct, string templateWrapperName, UhtScriptStruct templateArgumentStruct) : base(propertySettings, scriptStruct)
		{
			TemplateWrapperName = templateWrapperName;
			TemplateArgumentStruct = templateArgumentStruct;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtTemplateStructProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			TemplateWrapperName = reader.ReadString();
			TemplateArgumentStruct = (reader.ReadType() as UhtScriptStruct)!;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteString(TemplateWrapperName);
			writer.WriteType(TemplateArgumentStruct);
		}

		/// <inheritdoc/>
		public override void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
			base.CollectReferencesInternal(collector, addForwardDeclarations, isTemplateProperty);
			if (addForwardDeclarations && !TemplateArgumentStruct.IsCoreType)
			{
				if (RootProperty.Outer is UhtField)
				{
					collector.AddForwardDeclaration(TemplateArgumentStruct);
				}
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				case UhtPropertyTextType.GetterSetterArg:
					builder.Append(TemplateWrapperName).Append('<').Append(TemplateArgumentStruct.SourceName).Append('>');
					break;
				default:
					builder.Append(TemplateWrapperName).Append('<').Append(TemplateArgumentStruct.FullyQualifiedSourceName).Append('>');
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		public override bool DefaultDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			defaultValueReader
				.Require(TemplateWrapperName)
				.Require('<')
				.Optional("struct")
				.Require(TemplateArgumentStruct.SourceName)
				.Require('>');
			defaultValueReader
				.Require('(')
				.Require(')');
			innerDefaultValue.Append("()");
			return true;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "TInstancedStruct")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtTemplateStructProperty? InstancedStructProperty(UhtPropertyResolveArgs args)
		{
			UhtScriptStruct? baseScriptStruct = args.ParseTemplateScriptStruct();
			if (baseScriptStruct == null || baseScriptStruct.Session.FInstancedStruct == null)
			{
				return null;
			}

			UhtPropertySettings rootSettings = args.PropertySettings.RootSettings;
			if (rootSettings.MetaData.ContainsKey("BaseStruct"))
			{
				args.TokenReader.LogError("BaseStruct metadata is implicitly set from the TInstancedStruct template argument and should not be explicitly specified.");
				return null;
			}

			// With TInstancedStruct, BaseStruct is used as a type limiter.
			rootSettings.MetaData.Add("BaseStruct", baseScriptStruct.PathName);
			return new UhtTemplateStructProperty(args.PropertySettings, baseScriptStruct.Session.FInstancedStruct, "TInstancedStruct", baseScriptStruct);
		}

		[UhtPropertyType(Keyword = "TSubScriptStructOf")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtTemplateStructProperty? SubScriptStructOfProperty(UhtPropertyResolveArgs args)
		{
			UhtScriptStruct? baseScriptStruct = args.ParseTemplateScriptStruct();
			if (baseScriptStruct == null || baseScriptStruct.Session.FSubScriptStructOf == null)
			{
				return null;
			}

			UhtPropertySettings rootSettings = args.PropertySettings.RootSettings;
			if (rootSettings.MetaData.ContainsKey("MetaStruct"))
			{
				args.TokenReader.LogError("MetaStruct metadata is implicitly set from the TSubScriptStructOf template argument and should not be explicitly specified.");
				return null;
			}
			if (rootSettings.MetaData.ContainsKey("BaseStruct"))
			{
				args.TokenReader.LogError("BaseStruct metadata is implicitly set from the TSubScriptStructOf template argument and should not be explicitly specified.");
				return null;
			}

			// With TSubScriptStructOf, MetaStruct or BaseStruct is used as a type limiter.
			rootSettings.MetaData.Add("MetaStruct", baseScriptStruct.PathName);
			return new UhtTemplateStructProperty(args.PropertySettings, baseScriptStruct.Session.FSubScriptStructOf, "TSubScriptStructOf", baseScriptStruct);
		}

		[UhtPropertyType(Keyword = "TStateTreePropertyRef")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtStructProperty? StateTreePropertyRefProperty(UhtPropertyResolveArgs args)
		{
			const string RefTypeName = "RefType";
			const string IsRefToArrayName = "IsRefToArray";

			UhtPropertySettings propertySettings = args.PropertySettings;
			IUhtTokenReader tokenReader = args.TokenReader;

			if (args.Session.FStateTreePropertyRef == null)
			{
				return null;
			}

			UhtPropertySettings rootSettings = propertySettings.RootSettings;
			if (rootSettings.MetaData.ContainsKey(RefTypeName))
			{
				tokenReader.LogError("{0} metadata is implicitly set from the TStateTreePropertyRef template argument and should not be explicitly specified.", RefTypeName);
				return null;
			}

			if (rootSettings.MetaData.ContainsKey(IsRefToArrayName))
			{
				tokenReader.LogError("{0} metadata is implicitly set from the TStateTreePropertyRef template argument and should not be explicitly specified.", IsRefToArrayName);
				return null;
			}

			if (!args.SkipExpectedType())
			{
				return null;
			}

			tokenReader.Require('<');
			//@todo support TObjectptr, TOptional and other container.
			bool isRefToArray = tokenReader.TryOptional("TArray");
			bool isByteEnum = tokenReader.TryOptional("TEnumAsByte");

			if (isRefToArray || isByteEnum)
			{
				tokenReader.Require('<');
			}

			UhtTokenList? identifier = null;
			tokenReader
				.Optional("struct")
				.Optional("class")
				.RequireCppIdentifier(UhtCppIdentifierOptions.None, token => { identifier = token; });
			if (identifier == null)
			{
				throw new UhtIceException("Expected an identifier list");
			}
			identifier.RedirectTypeIdentifier(args.Config);

			UhtStructProperty instancedStructProperty = new(propertySettings, args.Session.FStateTreePropertyRef!);

			// TStateTreePropertyRef supports UStructs, UClasses, enums and primitive types.
			UhtType? foundType = propertySettings.Outer.FindType(UhtFindOptions.SourceName | UhtFindOptions.TypesMask, identifier);
			if (foundType is UhtStruct foundStruct)
			{
				if (foundStruct.IsChildOf(args.Session.UObject))
				{
					tokenReader.Require('*');
				}

				// It's a UStruct or UClass
				rootSettings.MetaData.Add(RefTypeName, foundStruct.PathName);
			}
			else if (foundType is UhtEnum foundEnum)
			{
				// It's an enum
				rootSettings.MetaData.Add(RefTypeName, foundEnum.PathName);
			}
			else
			{
				// It's a primitive or unknown type.
				rootSettings.MetaData.Add(RefTypeName, identifier.Join("::"));
			}

			if (isRefToArray || isByteEnum)
			{
				tokenReader.Require(">");
			}
			if (isRefToArray)
			{
				rootSettings.MetaData.Add(IsRefToArrayName, true);
			}

			tokenReader.Require(">");

			return instancedStructProperty;
		}
		#endregion
	}
}
