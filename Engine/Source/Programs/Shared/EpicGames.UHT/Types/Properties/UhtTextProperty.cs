// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// FTextProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "TextProperty", IsProperty = true)]
	public class UhtTextProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "TextProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "FText";

		/// <inheritdoc/>
		protected override string PGetMacroText => "PROPERTY";

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.EngineClass;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FTextPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::Text";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtTextProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.CanExposeOnSpawn | UhtPropertyCaps.IsParameterSupportedByBlueprint |
				UhtPropertyCaps.IsMemberSupportedByBlueprint;
			PropertyCaps &= ~(UhtPropertyCaps.CanBeContainerKey);
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtTextProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("FText::GetEmpty()");
			return builder;
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			if (defaultValueReader.TryOptional("FText"))
			{
				if (defaultValueReader.TryOptional('('))
				{
					defaultValueReader.Require(')');
					return true;
				}
				else if (defaultValueReader.TryOptional("::"))
				{
					// Handle legacy cases of FText::FromString being used as default values
					// These should be replaced with INVTEXT as FText::FromString can produce inconsistent keys
					if (defaultValueReader.TryOptional("FromString"))
					{
						defaultValueReader.Require('(');
						StringView value = defaultValueReader.GetWrappedConstString();
						defaultValueReader.Require(')');
						innerDefaultValue.Append('\"').Append(value).Append('\"');
						this.LogWarning("FText::FromString should be replaced with INVTEXT for default parameter values");
					}
					else
					{
						defaultValueReader.Require("GetEmpty").Require('(').Require(')');
					}
					return true;
				}
				else
				{
					return false;
				}
			}

			UhtToken token = defaultValueReader.GetToken();

			if (token.IsIdentifier("LOCTEXT"))
			{
				//ETSTODO - Add a test case for this
				this.LogError($"LOCTEXT default parameter values are not supported; use NSLOCTEXT instead: {SourceName}");
				return false;
			}
			else if (token.IsIdentifier("NOTELOCTEXT"))
			{
				//ETSTODO - Add a test case for this
				this.LogError($"NOTELOCTEXT default parameter values are not supported; use NSNOTELOCTEXT instead: {SourceName}");
				return false;
			}
			else
			{
				return SanitizeDefaultValue(this, defaultValueReader, ref token, innerDefaultValue, false);
			}
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtTextProperty;
		}

		#region Parsing keywords and default parsers		
		[UhtPropertyType(Keyword = "FText", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtTextProperty? TextProperty(UhtPropertyResolveArgs args)
		{
			return new UhtTextProperty(args.PropertySettings);
		}

		[UhtPropertyType(Keyword = "Text", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate | UhtPropertyTypeOptions.CaseInsensitive)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? MissingPrefixTextProperty(UhtPropertyResolveArgs args)
		{
			throw new UhtException(args.TokenReader, "'Text' is missing a prefix, expecting 'FText'");
		}

		[UhtLocTextDefaultValue(Name = "INVTEXT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool InvTextDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require('(');
			StringView value = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(')');
			innerDefaultValue.Append("INVTEXT(").Append(value).Append(')');
			return true;
		}

		[UhtLocTextDefaultValue(Name = "LOCTEXT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocTextDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			property.LogError($"LOCTEXT default parameter values are not supported; use NSLOCTEXT instead: {property.SourceName}");
			return false;
		}

		[UhtLocTextDefaultValue(Name = "NOTELOCTEXT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool NoteLocTextDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			property.LogError($"NOTELOCTEXT default parameter values are not supported; use NSNOTELOCTEXT instead: {property.SourceName}");
			return false;
		}

		[UhtLocTextDefaultValue(Name = "NSLOCTEXT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool NsLocTextDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require('(');
			StringView namespaceString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView keyString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView sourceString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(')');

			// Strip out the package name
			ReadOnlySpan<char> strippedNS = namespaceString.Span;
			if (strippedNS.Length > 1)
			{
				strippedNS = strippedNS[1..^1].Trim();
				if (strippedNS.Length > 0)
				{
					if (strippedNS[^1] == ']')
					{
						int index = strippedNS.LastIndexOf('[');
						if (index != -1)
						{
							strippedNS = strippedNS[..index].TrimEnd();
						}
					}
				}
			}

			innerDefaultValue.Append("NSLOCTEXT(\"").Append(strippedNS).Append("\", ").Append(keyString).Append(", ").Append(sourceString).Append(')');
			return true;
		}

		[UhtLocTextDefaultValue(Name = "NSNOTELOCTEXT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool NsNoteLocTextDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require('(');
			StringView namespaceString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView keyString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView sourceString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView devNotesString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(')');

			// Strip out the package name
			ReadOnlySpan<char> strippedNS = namespaceString.Span;
			if (strippedNS.Length > 1)
			{
				strippedNS = strippedNS[1..^1].Trim();
				if (strippedNS.Length > 0)
				{
					if (strippedNS[^1] == ']')
					{
						int index = strippedNS.LastIndexOf('[');
						if (index != -1)
						{
							strippedNS = strippedNS[..index].TrimEnd();
						}
					}
				}
			}

			innerDefaultValue.Append("NSNOTELOCTEXT(\"").Append(strippedNS).Append("\", ").Append(keyString).Append(", ").Append(sourceString).Append(", ").Append(devNotesString).Append(')');
			return true;
		}

		[UhtLocTextDefaultValue(Name = "LOCTABLE")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocTableDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require('(');
			StringView namespaceString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView keyString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(')');

			//ETSTODO - Validate the namespace string?
			innerDefaultValue.Append("LOCTABLE(").Append(namespaceString).Append(", ").Append(keyString).Append(')');
			return true;
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_FORMAT_NAMED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenFormatNamedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.Append("LOCGEN_FORMAT_NAMED(");
			defaultValueReader.Require('(');
			if (!SanitizeDefaultValue(property, defaultValueReader, innerDefaultValue, false))
			{
				return false;
			}

			// RequireList assumes that we have already parsed the ','.  So if it is there, consume it.
			defaultValueReader.TryOptional(',');

			bool success = true;
			defaultValueReader.RequireList(')', ',', false, () =>
			{
				StringView value = defaultValueReader.GetWrappedConstString();
				defaultValueReader.Require(',');
				innerDefaultValue.Append(", \"").Append(value).Append("\", ");
				if (!SanitizeDefaultValue(property, defaultValueReader, innerDefaultValue, true))
				{
					success = false;
				}
			});
			innerDefaultValue.Append(')');
			return success;
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_FORMAT_ORDERED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenFormatOrderedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.Append("LOCGEN_FORMAT_ORDERED(");
			defaultValueReader.Require('(');
			if (!SanitizeDefaultValue(property, defaultValueReader, innerDefaultValue, false))
			{
				return false;
			}

			// RequireList assumes that we have already parsed the ','.  So if it is there, consume it.
			defaultValueReader.TryOptional(',');

			bool success = true;
			defaultValueReader.RequireList(')', ',', false, () =>
			{
				innerDefaultValue.Append(", ");
				if (!SanitizeDefaultValue(property, defaultValueReader, innerDefaultValue, true))
				{
					success = false;
				}
			});
			innerDefaultValue.Append(')');
			return success;
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_NUMBER")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenNumberDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.None);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_NUMBER_GROUPED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenNumberGroupedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Grouped);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_NUMBER_UNGROUPED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenNumberUngroupedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Ungrouped);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_NUMBER_CUSTOM")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenNumberCustomDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Custom);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_PERCENT")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenPercentDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.None);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_PERCENT_GROUPED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenPercentGroupedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Grouped);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_PERCENT_UNGROUPED")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenPercentUngroupedDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Ungrouped);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_PERCENT_CUSTOM")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenPercentCustomDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenNumberOrPercentDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, NumberStyle.Custom);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_CURRENCY")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenCurrencyDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require('(');

			UhtToken token = defaultValueReader.GetToken();
			if (!token.GetConstDouble(out double baseValue))
			{
				return false;
			}

			defaultValueReader.Require(',');
			StringView currencyCode = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(',');
			StringView cultureNameString = defaultValueReader.GetConstQuotedString();
			defaultValueReader.Require(')');

			// With UHT, we end up outputting just the integer part of the specified value.  It is using default locale information in UHT
			innerDefaultValue.Append("LOCGEN_CURRENCY(").Append((int)baseValue).Append(", ").Append(currencyCode).Append(", ").Append(cultureNameString).Append(')');
			return true;
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATE_LOCAL")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateLocalDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, false, false, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATE_UTC")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateUtcDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, false, true, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_TIME_LOCAL")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenTimeLocalDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, false, true, false, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_TIME_UTC")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenTimeUtcDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, false, true, true, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATETIME_LOCAL")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateTimeLocalDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, true, false, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATETIME_UTC")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateTimeUtcDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, true, true, false);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATETIME_CUSTOM_LOCAL")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateTimeCustomLocalDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, true, false, true);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_DATETIME_CUSTOM_UTC")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static bool LocGenDateTimeCustomUtcDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenDateTimeDefaultValue(defaultValueReader, ref macroToken, innerDefaultValue, true, true, true, true);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_TOUPPER")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static bool LocGenToUpperDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenTransformDefaultValue(property, defaultValueReader, ref macroToken, innerDefaultValue);
		}

		[UhtLocTextDefaultValue(Name = "LOCGEN_TOLOWER")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		private static bool LocGenToLowerDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			return LocGenTransformDefaultValue(property, defaultValueReader, ref macroToken, innerDefaultValue);
		}
		#endregion

		#region Default value helper methods
		private static bool SanitizeDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue, bool allowNumerics)
		{
			UhtToken token = defaultValueReader.GetToken();
			return SanitizeDefaultValue(property, defaultValueReader, ref token, innerDefaultValue, allowNumerics);
		}

		private static bool SanitizeDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken token, StringBuilder innerDefaultValue, bool allowNumerics)
		{
			switch (token.TokenType)
			{
				case UhtTokenType.Identifier:
					if (property.Session.TryGetLocTextDefaultValue(token.Value, out UhtLocTextDefaultValue locTextDefaultValue))
					{
						if (locTextDefaultValue.Delegate(property, defaultValueReader, ref token, innerDefaultValue))
						{
							return true;
						}
					}
					return false;

				case UhtTokenType.StringConst:
					innerDefaultValue.Append(token.Value);
					return true;

				case UhtTokenType.DecimalConst:
					if (allowNumerics)
					{
						FormatDecimal(ref token, innerDefaultValue);
						return true;
					}
					return false;

				case UhtTokenType.FloatConst:
					if (allowNumerics)
					{
						FormatFloat(ref token, innerDefaultValue);
						return true;
					}
					return false;

				default:
					return false;
			}
		}

		private static void FormatDecimal(ref UhtToken token, StringBuilder innerDefaultValue)
		{
			ReadOnlySpan<char> span = token.Value.Span;
			int suffixStart = span.Length;
			bool isUnsigned = false;
			while (suffixStart > 0)
			{
				char c = span[suffixStart - 1];
				if (UhtFCString.IsUnsignedMarker(c))
				{
					isUnsigned = true;
					suffixStart--;
				}
				else if (UhtFCString.IsLongMarker(c))
				{
					suffixStart--;
				}
				else
				{
					break;
				}
			}
			if (token.GetConstLong(out long value))
			{
				if (isUnsigned)
				{
					innerDefaultValue.Append((ulong)value).Append('u');
				}
				else
				{
					innerDefaultValue.Append(value);
				}
			}
		}

		private static void FormatFloat(ref UhtToken token, StringBuilder innerDefaultValue)
		{
			char c = token.Value.Span[^1];
			bool isFloat = UhtFCString.IsFloatMarker(c);
			if (token.GetConstDouble(out double value))
			{
				innerDefaultValue.AppendFormat(CultureInfo.InvariantCulture, "{0:F6}", value);
				if (isFloat)
				{
					innerDefaultValue.Append(c);
				}
			}
		}

		enum NumberStyle
		{
			None,
			Grouped,
			Ungrouped,
			Custom,
		}

		private static bool LocGenNumberOrPercentDefaultValue(IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue, NumberStyle numberStyle)
		{
			innerDefaultValue.Append(macroToken.Value.ToString());
			defaultValueReader.Require('(');
			innerDefaultValue.Append('(');

			UhtToken token = defaultValueReader.GetToken();
			switch (token.TokenType)
			{
				case UhtTokenType.DecimalConst:
					FormatDecimal(ref token, innerDefaultValue);
					break;

				case UhtTokenType.FloatConst:
					FormatFloat(ref token, innerDefaultValue);
					break;

				default:
					return false;
			}
			defaultValueReader.Require(',');
			innerDefaultValue.Append(", ");

			if (numberStyle == NumberStyle.Custom)
			{
				while (true)
				{
					UhtToken customToken = defaultValueReader.GetToken();
					if (!customToken.IsIdentifier())
					{
						return false;
					}
					innerDefaultValue.Append(customToken.Value);

					defaultValueReader.Require('(');
					innerDefaultValue.Append('(');

					switch (customToken.Value.ToString())
					{
						case "SetAlwaysSign":
						case "SetUseGrouping":
							{
								UhtToken booleanToken = defaultValueReader.GetToken();
								if (booleanToken.IsIdentifier("true"))
								{
									innerDefaultValue.Append("true");
								}
								else if (booleanToken.IsIdentifier("false"))
								{
									innerDefaultValue.Append("false");
								}
								else
								{
									return false;
								}
							}
							break;

						case "SetRoundingMode":
							{
								defaultValueReader.Require("ERoundingMode");
								defaultValueReader.Require("::");
								StringView identifier = defaultValueReader.GetIdentifier().Value;
								innerDefaultValue.Append("ERoundingMode::").Append(identifier);
							}
							break;

						case "SetMinimumIntegralDigits":
						case "SetMaximumIntegralDigits":
						case "SetMinimumFractionalDigits":
						case "MaximumFractionalDigits":
							{
								UhtToken numericToken = defaultValueReader.GetToken();
								if (numericToken.GetConstInt(out int value))
								{
									innerDefaultValue.Append(value);
								}
								else
								{
									return false;
								}
							}
							break;

						default:
							return false;
					}

					defaultValueReader.Require(')');
					innerDefaultValue.Append(')');

					if (!defaultValueReader.TryOptional('.'))
					{
						break;
					}
					innerDefaultValue.Append('.');
				}

				defaultValueReader.Require(',');
				innerDefaultValue.Append(", ");
			}

			StringView _ = defaultValueReader.GetConstQuotedString();
			innerDefaultValue.Append("\"\""); // UHT doesn't write out the culture

			defaultValueReader.Require(')');
			innerDefaultValue.Append(')');
			return true;
		}

		private static void FormatDateTimeStyle(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			defaultValueReader.Require(',');
			defaultValueReader.Require("EDateTimeStyle");
			defaultValueReader.Require("::");
			StringView identifier = defaultValueReader.GetIdentifier().Value;
			innerDefaultValue.Append(", EDateTimeStyle::").Append(identifier);
		}

		private static bool LocGenDateTimeDefaultValue(IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue, bool isDate, bool isTime, bool isUtc, bool isCustom)
		{
			defaultValueReader.Require('(');
			innerDefaultValue.Append(macroToken.Value).Append('(');

			long unixTimestampValue;
			UhtToken timestampToken = defaultValueReader.GetToken();
			switch (timestampToken.TokenType)
			{
				case UhtTokenType.FloatConst:
					if (timestampToken.GetConstDouble(out double value))
					{
						unixTimestampValue = (long)value;
						break;
					}
					else
					{
						return false;
					}

				case UhtTokenType.DecimalConst:
					if (!timestampToken.GetConstLong(out unixTimestampValue))
					{
						return false;
					}
					break;

				default:
					return false;

			}
			innerDefaultValue.Append(unixTimestampValue);

			if (isCustom)
			{
				defaultValueReader.Require(',');
				StringView customPattern = defaultValueReader.GetConstQuotedString();
				innerDefaultValue.Append(", ").Append(customPattern);
			}
			else
			{
				if (isDate)
				{
					FormatDateTimeStyle(defaultValueReader, innerDefaultValue);
				}
				if (isTime)
				{
					FormatDateTimeStyle(defaultValueReader, innerDefaultValue);
				}
			}

			if (isUtc)
			{
				defaultValueReader.Require(',');
				StringView timeZone = defaultValueReader.GetConstQuotedString();
				innerDefaultValue.Append(", ").Append(timeZone);
			}

			defaultValueReader.Require(',');
			StringView _ = defaultValueReader.GetConstQuotedString();
			innerDefaultValue.Append(", ").Append("\"\""); // We don't write out the culture

			defaultValueReader.Require(')');
			innerDefaultValue.Append(')');
			return true;
		}

		private static bool LocGenTransformDefaultValue(UhtTextProperty property, IUhtTokenReader defaultValueReader, ref UhtToken macroToken, StringBuilder innerDefaultValue)
		{
			innerDefaultValue.Append(macroToken.Value).Append('(');
			defaultValueReader.Require('(');
			if (!SanitizeDefaultValue(property, defaultValueReader, innerDefaultValue, false))
			{
				return false;
			}
			defaultValueReader.Require(')');
			innerDefaultValue.Append(')');
			return true;
		}

		#endregion
	}
}
