// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FUInt16Property
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "UInt16Property", IsProperty = true)]
	public class UhtUInt16Property : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "UInt16Property";

		/// <inheritdoc/>
		protected override string CppTypeText => "uint16";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FUInt16PropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::UInt16";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtUInt16Property(UhtPropertySettings propertySettings) : base(propertySettings)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtUInt16Property(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtUInt16Property;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "uint16", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? UInt16Property(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.IsBitfield)
			{
				return new UhtBoolProperty(args.PropertySettings, UhtBoolType.UInt8);
			}
			else
			{
				return new UhtUInt16Property(args.PropertySettings);
			}
		}
		#endregion
	}
}
