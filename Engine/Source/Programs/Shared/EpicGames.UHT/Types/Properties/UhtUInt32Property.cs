// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// FUInt32Property
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "UInt32Property", IsProperty = true)]
	public class UhtUInt32Property : UhtNumericProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "UInt32Property";

		/// <inheritdoc/>
		protected override string CppTypeText => "uint32";

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FUInt32PropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::UInt32";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtUInt32Property(UhtPropertySettings propertySettings) : base(propertySettings)
		{
			PropertyCaps |= UhtPropertyCaps.SupportsRigVM;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtUInt32Property(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			return false;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtUInt32Property;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "uint32", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? UInt32Property(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.IsBitfield)
			{
				return new UhtBoolProperty(args.PropertySettings, UhtBoolType.UInt8);
			}
			else
			{
				return new UhtUInt32Property(args.PropertySettings);
			}
		}

		[UhtPropertyType(Keyword = "unsigned", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[UhtPropertyType(Keyword = "unsigned int", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtUInt32Property? UnsignedProperty(UhtPropertyResolveArgs args)
		{
			return new UhtUInt32Property(args.PropertySettings);
		}
		#endregion
	}
}
