// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Internal only void property for void return types
	/// </summary>
	[UnrealHeaderTool]
	public class UhtVoidProperty : UhtProperty
	{
		/// <inheritdoc/>
		public override string EngineClassName => "UHTVoidProperty";

		/// <inheritdoc/>
		protected override string CppTypeText => "void";

		/// <inheritdoc/>
		protected override string PGetMacroText => "invalid";

		/// <summary>
		/// Construct a new void property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		public UhtVoidProperty(UhtPropertySettings propertySettings) : base(propertySettings)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtVoidProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendFunctionThunkParameterGet(StringBuilder builder)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			return other is UhtVoidProperty;
		}

		#region Keyword
		[UhtPropertyType(Keyword = "void", Options = UhtPropertyTypeOptions.Simple | UhtPropertyTypeOptions.Immediate)]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtVoidProperty? VoidProperty(UhtPropertyResolveArgs args)
		{
			if (args.PropertySettings.PropertyCategory != UhtPropertyCategory.Return && !args.PropertySettings.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseCall))
			{
				args.TokenReader.LogError("void type is only valid as a return type or TVerseCall<void>");
				return null;
			}
			return new UhtVoidProperty(args.PropertySettings);
		}
		#endregion
	}
}
