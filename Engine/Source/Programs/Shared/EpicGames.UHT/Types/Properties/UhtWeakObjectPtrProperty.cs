// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	using static EpicGames.UHT.Exporters.CodeGen.UhtExpressionFactory;

	/// <summary>
	/// Represents a FWeakObjectProperty
	/// </summary>
	[UnrealHeaderTool]
	[UhtEngineClass(Name = "WeakObjectProperty", IsProperty = true)]
	public class UhtWeakObjectPtrProperty : UhtObjectPropertyBase
	{
		/// <inheritdoc/>
		public override string EngineClassName => "WeakObjectProperty";

		/// <inheritdoc/>
		protected override string PGetMacroText => PropertyFlags.HasAnyFlags(EPropertyFlags.AutoWeak) ? "AUTOWEAKOBJECT" : "WEAKOBJECT";

		/// <inheritdoc/>
		protected override bool PGetPassAsNoPtr => true;

		/// <inheritdoc/>
		protected override UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.TypeText;

		/// <inheritdoc/>
		protected override string CodeGenParamsStruct => "FWeakObjectPropertyParams";

		/// <inheritdoc/>
		protected override string CodeGenParamsFlags => "UECodeGen_Private::EPropertyGenFlags::WeakObject";

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings</param>
		/// <param name="propertyClass">Class being referenced</param>
		/// <param name="extraFlags">Extra property flags to add to the definition</param>
		public UhtWeakObjectPtrProperty(UhtPropertySettings propertySettings, UhtClass propertyClass, EPropertyFlags extraFlags = EPropertyFlags.None)
			: base(new UhtNoValidateConstruct { }, propertySettings, UhtObjectCppForm.NativeObject, propertyClass)
		{
			PropertyFlags |= EPropertyFlags.UObjectWrapper | extraFlags;
			PropertyCaps |= UhtPropertyCaps.PassCppArgsByRef | UhtPropertyCaps.RequiresNullConstructorArg | UhtPropertyCaps.IsMemberSupportedByBlueprint;
			PropertyCaps &= ~(UhtPropertyCaps.IsParameterSupportedByBlueprint);
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtWeakObjectPtrProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}

		/// <inheritdoc/>
		public override StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument)
		{
			switch (textType)
			{
				default:
					if (PropertyFlags.HasAnyFlags(EPropertyFlags.AutoWeak))
					{
						builder.Append("TAutoWeakObjectPtr<").Append(Class.SourceName).Append('>');
					}
					else
					{
						builder.Append("TWeakObjectPtr<").Append(Class.SourceName).Append('>');
					}
					break;
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return AppendParamsDefRef(builder, context, Class, Exporters.CodeGen.UhtSingletonType.Unregistered);
		}

		/// <inheritdoc/>
		protected override StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder.Append($"{ConstInitSingletonRef(context, Class, true)}, ");
		}

		/// <inheritdoc/>
		public override StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer)
		{
			builder.Append("NULL");
			return builder;
		}

		/// <inheritdoc/>
		public override bool IsSameType(UhtProperty other)
		{
			if (other is UhtWeakObjectPtrProperty otherObject)
			{
				return Class == otherObject.Class && MetaClass == otherObject.MetaClass;
			}
			return false;
		}

		#region Keywords
		private static UhtProperty CreateWeakProperty(UhtPropertySettings propertySettings, IUhtTokenReader tokenReader, UhtClass classObj, EPropertyFlags extraFlags = EPropertyFlags.None)
		{
			if (classObj.IsChildOf(classObj.Session.UClass))
			{
				tokenReader.LogError("Class variables cannot be weak, they are always strong.");
			}

			if (propertySettings.DisallowPropertyFlags.HasAnyFlags(EPropertyFlags.AutoWeak))
			{
				return new UhtObjectProperty(propertySettings, UhtObjectCppForm.NativeObject, classObj, extraFlags | EPropertyFlags.UObjectWrapper);
			}
			return new UhtWeakObjectPtrProperty(propertySettings, classObj, extraFlags);
		}

		[UhtPropertyType(Keyword = "TWeakObjectPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? WeakObjectProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? propertyClass = args.ParseTemplateObject(UhtTemplateObjectMode.Normal);
			if (propertyClass == null)
			{
				return null;
			}

			return CreateWeakProperty(args.PropertySettings, args.TokenReader, propertyClass);
		}

		[UhtPropertyType(Keyword = "TAutoWeakObjectPtr")]
		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Attribute accessed method")]
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Attribute accessed method")]
		private static UhtProperty? AutoWeakObjectProperty(UhtPropertyResolveArgs args)
		{
			UhtClass? propertyClass = args.ParseTemplateObject(UhtTemplateObjectMode.Normal);
			if (propertyClass == null)
			{
				return null;
			}

			return CreateWeakProperty(args.PropertySettings, args.TokenReader, propertyClass, EPropertyFlags.AutoWeak);
		}
		#endregion
	}
}
