// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Series of flags not part of the engine's class flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtClassExportFlags : int
	{

		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// If set, the class itself has replicated properties.
		/// </summary>
		SelfHasReplicatedProperties = 1 << 0,

		/// <summary>
		/// If set, some super class has replicated properties.
		/// </summary>
		SuperHasReplicatedProperties = 1 << 1,

		/// <summary>
		/// If set, either the class itself or a super class has replicated properties
		/// </summary>
		HasReplciatedProperties = SelfHasReplicatedProperties | SuperHasReplicatedProperties,

		/// <summary>
		/// Custom constructor specifier present
		/// </summary>
		HasCustomConstructor = 1 << 2,

		/// <summary>
		/// A default constructor was found in the class
		/// </summary>
		HasDefaultConstructor = 1 << 3,

		/// <summary>
		/// An object initializer constructor was found in the class
		/// </summary>
		HasObjectInitializerConstructor = 1 << 4,

		/// <summary>
		/// A custom vtable helper constructor was found
		/// </summary>
		HasCustomVTableHelperConstructor = 1 << 5,

		/// <summary>
		/// A constructor was found
		/// </summary>
		HasConstructor = 1 << 6,

		/// <summary>
		/// GetLifetimeReplicatedProps was found in the class
		/// </summary>
		HasGetLifetimeReplicatedProps = 1 << 7,

		/// <summary>
		/// Class should not be exported
		/// </summary>
		NoExport = 1 << 8,

		/// <summary>
		/// Class has a custom field notify
		/// </summary>
		HasCustomFieldNotify = 1 << 9,

		/// <summary>
		/// Class has a field notify
		/// </summary>
		HasFieldNotify = 1 << 10,

		/// <summary>
		/// Class has destructor
		/// </summary>
		HasDestructor = 1 << 11,

		/// <summary>
		/// The GENERATED_UCLASS_BODY, GENERATED_UINTERFACE_BODY, and GENERATED_IINTERFACE_BODY macros use the 
		/// legacy generated body.  If this flag is set, generate the legacy instead of the GENERATED_BODY macros.
		/// </summary>
		UsesGeneratedBodyLegacy = 1 << 12,

		/// <summary>
		/// Class is marked as final
		/// </summary>
		Final = 1 << 13,

		/// <summary>
		/// Class is a verse root module.
		/// </summary>
		VerseIsRootModule = 1 << 14,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtClassExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtClassExportFlags inFlags, UhtClassExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtClassExportFlags inFlags, UhtClassExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtClassExportFlags inFlags, UhtClassExportFlags testFlags, UhtClassExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Type of the class
	/// </summary>
	public enum UhtClassType
	{
		/// <summary>
		/// Class is a UCLASS
		/// </summary>
		Class,

		/// <summary>
		/// Class is a UINTERFACE
		/// </summary>
		Interface,

		/// <summary>
		/// Class is the native interface for a UINTERFACE
		/// </summary>
		NativeInterface,

		/// <summary>
		/// Class is an implementation detail for a Verse module
		/// </summary>
		VModule,
	}

	/// <summary>
	/// Type of archive serializer found
	/// </summary>
	[Flags]
	public enum UhtSerializerArchiveType
	{

		/// <summary>
		/// No serializer found
		/// </summary>
		None = 0,

		/// <summary>
		/// Archive serializer found
		/// </summary>
		Archive = 1 << 0,

		/// <summary>
		/// Structured archive serializer found
		/// </summary>
		StructuredArchiveRecord = 1 << 1,

		/// <summary>
		/// Mask of all serializer types
		/// </summary>
		All = Archive | StructuredArchiveRecord,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtSerializerArchiveTypeExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtSerializerArchiveType inFlags, UhtSerializerArchiveType testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtSerializerArchiveType inFlags, UhtSerializerArchiveType testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtSerializerArchiveType inFlags, UhtSerializerArchiveType testFlags, UhtSerializerArchiveType matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// A skipped declaration
	/// </summary>
	public struct UhtDeclaration
	{

		/// <summary>
		/// Compiler directives when declaration was parsed
		/// </summary>
		public UhtCompilerDirective CompilerDirectives { get; set; }

		/// <summary>
		/// Collection of tokens parsed in the declaration
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1051:Do not declare visible instance fields", Justification = "<Pending>")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public UhtToken[] Tokens;

		/// <summary>
		/// If this declaration is part of a UFUNCTION, this will be set
		/// </summary>
		public UhtFunction? Function { get; set; }

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtDeclaration(UhtInputCacheReader reader)
		{
			CompilerDirectives = (UhtCompilerDirective)reader.ReadUInt64();
			Tokens = reader.ReadTokenArray()!;
			Function = reader.ReadType() as UhtFunction;
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public readonly void Write(UhtInputCacheWriter writer)
		{
			writer.WriteUInt64((ulong)CompilerDirectives);
			writer.WriteVariableLengthTokenArray(Tokens);
			writer.WriteType(Function);
		}
	}

	/// <summary>
	/// Represents a declaration found by name
	/// </summary>
	public struct UhtFoundDeclaration
	{

		/// <summary>
		/// Compiler directives when declaration was parsed
		/// </summary>
		public UhtCompilerDirective CompilerDirectives { get; set; }

		/// <summary>
		/// Collection of tokens parsed in the declaration
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1051:Do not declare visible instance fields", Justification = "<Pending>")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public UhtToken[] Tokens;

		/// <summary>
		/// Token index for the matching name
		/// </summary>
		public int NameTokenIndex { get; set; }

		/// <summary>
		/// True if "virtual" was found prior to the matching name
		/// </summary>
		public bool IsVirtual { get; set; }
	}

	/// <summary>
	/// Instance of a UCLASS, UINTERFACE, or a native interface
	/// </summary>
	[UhtEngineClass(Name = "Class")]
	public class UhtClass : UhtStruct
	{
		private List<UhtDeclaration>? _declarations = null;

		/// <summary>
		/// Configuration section
		/// </summary>
		public string Config { get; set; } = String.Empty;

		/// <summary>
		/// If needed, the #if block define for the serializer
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtDefineScope SerializerDefineScope { get; set; } = UhtDefineScope.None;

		/// <summary>
		/// The class within
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass ClassWithin { get; set; }

		/// <summary>
		/// Class flags used to populate UMyClass::StaticClassFlags. 
		/// Note that these may be different to those in the instance field UClass::ClassFlags which
		/// does not respect removing class flags in a derived class.
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EClassFlags ClassFlags { get; set; } = EClassFlags.None;

		/// <summary>
		/// Class cast flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EClassCastFlags ClassCastFlags { get; set; } = EClassCastFlags.None;

		/// <summary>
		/// Export flags not present in the engine
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtClassExportFlags ClassExportFlags { get; set; } = UhtClassExportFlags.None;

		/// <summary>
		/// Type of the class
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtClassType ClassType { get; set; } = UhtClassType.Class;

		/// <summary>
		/// Type of archivers present
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtSerializerArchiveType SerializerArchiveType { get; set; } = UhtSerializerArchiveType.None;

		/// <summary>
		/// Line number of the prolog
		/// </summary>
		public int PrologLineNumber { get; set; } = -1;

		/// <summary>
		/// Collection of functions and other declarations found in the class
		/// </summary>
		[JsonIgnore]
		public IList<UhtDeclaration>? Declarations => _declarations;

		/// <summary>
		/// If this, this class is a UINTERFACE and NativeInterface is the associated native interface
		/// </summary>
		[JsonIgnore]
		public UhtClass? NativeInterface { get; set; } = null;

		/// <summary>
		/// Line number o the generated body statement
		/// </summary>
		public int GeneratedBodyLineNumber { get; set; } = -1;

		/// <summary>
		/// Access of the generated body
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtAccessSpecifier GeneratedBodyAccessSpecifier { get; set; } = UhtAccessSpecifier.None;

		/// <summary>
		/// True if GENERATED_BODY was used.  If false, GENERATED_UCLASS_BODY was used.
		/// </summary>
		public bool HasGeneratedBody { get; set; } = false;

		#region Class specifier parse time values (not interfaces)
		/// <summary>
		/// Engine class flags removed
		/// </summary>
		public EClassFlags RemovedClassFlags { get; set; } = EClassFlags.None;

		/// <summary>
		/// Class within identifier
		/// </summary>
		[JsonIgnore]
		public string ClassWithinIdentifier { get; set; } = String.Empty;

		/// <summary>
		/// Collection of show categories
		/// </summary>
		[JsonIgnore]
		public List<string> ShowCategories { get; } = [];

		/// <summary>
		/// Collection of hide categories
		/// </summary>
		[JsonIgnore]
		public List<string> HideCategories { get; } = [];

		/// <summary>
		/// Collection of auto expand categories
		/// </summary>
		[JsonIgnore]
		public List<string> AutoExpandCategories { get; } = [];

		/// <summary>
		/// Collection of auto collapse categories
		/// </summary>
		[JsonIgnore]
		public List<string> AutoCollapseCategories { get; } = [];

		/// <summary>
		/// Collection of prioritize categories
		/// </summary>
		[JsonIgnore]
		public List<string> PrioritizeCategories { get; } = [];

		/// <summary>
		/// Collection of show functions
		/// </summary>
		[JsonIgnore]
		public List<string> ShowFunctions { get; } = [];

		/// <summary>
		/// Collection of hide functions
		/// </summary>
		[JsonIgnore]
		public List<string> HideFunctions { get; } = [];

		/// <summary>
		/// Sparse class data types
		/// </summary>
		[JsonIgnore]
		public List<string> SparseClassDataTypes { get; } = [];

		/// <summary>
		/// Class group names
		/// </summary>
		[JsonIgnore]
		public List<string> ClassGroupNames { get; } = [];

		/// <summary>
		/// After functions are fully resolved, points to the first non-delegate function in this class
		/// </summary>
		public UhtDefineScopeLink<UhtFunction>? FirstFunction { get; set; }

		/// <summary>
		/// Add the given class flags
		/// </summary>
		/// <param name="flags">Flags to add</param>
		public void AddClassFlags(EClassFlags flags)
		{
			ClassFlags |= flags;
			RemovedClassFlags &= ~flags;
		}

		/// <summary>
		/// Remove the given class flags
		/// </summary>
		/// <param name="flags">Flags to remove</param>
		public void RemoveClassFlags(EClassFlags flags)
		{
			RemovedClassFlags |= flags;
			ClassFlags &= ~flags;
		}
		#endregion

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType
		{
			get
			{
				switch (ClassType)
				{
					case UhtClassType.Class:
					case UhtClassType.VModule:
						return UhtEngineType.Class;
					case UhtClassType.Interface:
						return UhtEngineType.Interface;
					case UhtClassType.NativeInterface:
						return UhtEngineType.NativeInterface;
					default:
						throw new UhtIceException($"UhtEngineType: Unknown class type {ClassType}");
				}
			}
		}

		/// <inheritdoc/>
		public override string EngineClassName => IsVerseField ? "VerseClass" : "Class";

		/// <inheritdoc/>
		public override string EngineLinkClassName => "Class";

		/// <inheritdoc/>
		public override UhtClass? EngineClass => IsVerseField ? Session.UVerseClass : Session.UClass;

		private string? _nativeFunctionCallName;

		/// <summary>
		/// The name used to scope native function calls into this type e.g. UClassName or IInterfaceName
		/// </summary>
		public string NativeFunctionCallName => _nativeFunctionCallName ??= ClassType switch
		{
			UhtClassType.Class or UhtClassType.NativeInterface or UhtClassType.VModule => SourceName,
			UhtClassType.Interface => $"I{SourceName[1..]}",
			var unknown => throw new UhtIceException($"NativeFunctionCallName: Unknown class type {unknown}"),
		};

		///<inheritdoc/>
		[JsonIgnore]
		public override bool Deprecated => ClassFlags.HasAnyFlags(EClassFlags.Deprecated);

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable
		{
			get
			{
				switch (ClassType)
				{
					case UhtClassType.Class:
						return Session.GetSpecifierValidatorTable(UhtTableNames.Class);
					case UhtClassType.Interface:
						return Session.GetSpecifierValidatorTable(UhtTableNames.Interface);
					case UhtClassType.NativeInterface:
						return Session.GetSpecifierValidatorTable(UhtTableNames.NativeInterface);
					case UhtClassType.VModule:
						return Session.GetSpecifierValidatorTable(UhtTableNames.VModule);
					default:
						throw new UhtIceException($"SpecifierValidatorTable: Unknown class type {ClassType}");
				}
			}
		}

		/// <summary>
		/// The super class
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass? SuperClass => (UhtClass?)Super;

		/// <summary>
		/// True if this class inherits from AActor
		/// </summary>
		[JsonIgnore]
		public bool IsActorClass => IsChildOf(Session.AActor);

		///<inheritdoc/>
		[JsonIgnore]
		public override string EngineNamePrefix
		{
			get
			{
				if (IsVerseField)
				{
					return "";
				}
				switch (ClassType)
				{
					case UhtClassType.Class:
						if (IsActorClass)
						{
							return ClassFlags.HasAnyFlags(EClassFlags.Deprecated) ? "ADEPRECATED_" : "A";
						}
						else
						{
							return ClassFlags.HasAnyFlags(EClassFlags.Deprecated) ? "UDEPRECATED_" : "U";
						}

					case UhtClassType.Interface:
						return "U";

					case UhtClassType.NativeInterface:
						return "I";

					case UhtClassType.VModule:
						return "";

					default:
						throw new NotImplementedException();
				}
			}
		}

		/// <summary>
		/// Line number containing the VINTERFACES macro
		/// </summary>
		public int VerseInterfacesLine { get; set; } = 0;

		/// <summary>
		/// Verse interfaces directly specified on the interface definition
		/// </summary>
		[JsonConverter(typeof(UhtTypeReadOnlyListSourceNameJsonConverter<UhtClass>))]
		public IReadOnlyList<UhtClass> VerseInterfaces => _verseInterfaces ?? s_emptyVerseInterfaces;

		/// <summary>
		/// Flattened collection of all verse interfaces. 
		/// </summary>
		[JsonConverter(typeof(UhtTypeReadOnlyListSourceNameJsonConverter<UhtClass>))]
		public IReadOnlyList<UhtClass> FlattenedVerseInterfaces => _flattenedVerseInterfaces ?? s_emptyVerseInterfaces;

		/// <summary>
		/// List of all verse interfaces implemented by this interface
		/// </summary>
		[JsonIgnore]
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "<Pending>")]
		public List<UhtToken[]>? VerseInterfaceIdentifiers { get; set; } = null;

		private static readonly List<UhtClass> s_emptyVerseInterfaces = [];
		private List<UhtClass>? _verseInterfaces = null;
		private List<UhtClass>? _flattenedVerseInterfaces = null;

		/// <summary>
		/// Construct a new instance of the class
		/// </summary>
		/// <param name="headerFile">Header being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">The outer type</param>
		/// <param name="lineNumber">Line number where class begins</param>
		public UhtClass(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
			ClassWithin = this;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtClass(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ClassWithin = this;
			_declarations = reader.ReadOptionalList<UhtDeclaration>((reader) => new(reader));
			Config = reader.ReadString();
			SerializerDefineScope = (UhtDefineScope)reader.ReadUInt64();
			ClassFlags = (EClassFlags)reader.ReadUInt64();
			ClassCastFlags = (EClassCastFlags)reader.ReadUInt64();
			ClassExportFlags = (UhtClassExportFlags)reader.ReadUInt64();
			ClassType = (UhtClassType)reader.ReadUInt8();
			SerializerArchiveType = (UhtSerializerArchiveType)reader.ReadUInt8();
			PrologLineNumber = reader.ReadInt32();
			GeneratedBodyLineNumber = reader.ReadInt32();
			GeneratedBodyAccessSpecifier = (UhtAccessSpecifier)reader.ReadUInt8();
			HasGeneratedBody = reader.ReadBoolean();
			RemovedClassFlags = (EClassFlags)reader.ReadUInt64();
			ClassWithinIdentifier = reader.ReadString();
			ShowCategories = reader.ReadList((reader) => reader.ReadString());
			HideCategories = reader.ReadList((reader) => reader.ReadString());
			AutoExpandCategories = reader.ReadList((reader) => reader.ReadString());
			AutoCollapseCategories = reader.ReadList((reader) => reader.ReadString());
			PrioritizeCategories = reader.ReadList((reader) => reader.ReadString());
			ShowFunctions = reader.ReadList((reader) => reader.ReadString());
			HideFunctions = reader.ReadList((reader) => reader.ReadString());
			SparseClassDataTypes = reader.ReadList((reader) => reader.ReadString());
			ClassGroupNames = reader.ReadList((reader) => reader.ReadString());
			VerseInterfacesLine = reader.ReadInt32();
			VerseInterfaceIdentifiers = reader.ReadOptionalList((reader) => reader.ReadTokenArray());
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteOptionalVariableLengthArray(_declarations, (writer, x) => { x.Write(writer); });
			writer.WriteString(Config);
			writer.WriteUInt64((ulong)SerializerDefineScope);
			writer.WriteUInt64((ulong)ClassFlags);
			writer.WriteUInt64((ulong)ClassCastFlags);
			writer.WriteUInt64((ulong)ClassExportFlags);
			writer.WriteUInt8((byte)ClassType);
			writer.WriteUInt8((byte)SerializerArchiveType);
			writer.WriteInt32(PrologLineNumber);
			writer.WriteInt32(GeneratedBodyLineNumber);
			writer.WriteUInt8((byte)GeneratedBodyAccessSpecifier);
			writer.WriteBoolean(HasGeneratedBody);
			writer.WriteUInt64((ulong)RemovedClassFlags);
			writer.WriteString(ClassWithinIdentifier);
			writer.WriteVariableLengthArray(ShowCategories, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(HideCategories, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(AutoExpandCategories, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(AutoCollapseCategories, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(PrioritizeCategories, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(ShowFunctions, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(HideFunctions, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(SparseClassDataTypes, (writer, x) => writer.WriteString(x));
			writer.WriteVariableLengthArray(ClassGroupNames, (writer, x) => writer.WriteString(x));
			writer.WriteInt32(VerseInterfacesLine);
			writer.WriteOptionalVariableLengthArray(VerseInterfaceIdentifiers, (writer, x) => writer.WriteVariableLengthTokenArray(x.AsSpan()));
		}

		/// <summary>
		/// Add the given list of tokens as a possible declaration
		/// </summary>
		/// <param name="compilerDirectives">Currently active compiler directives</param>
		/// <param name="tokens">List of declaration tokens</param>
		/// <param name="function">If parsed as part of a UFUNCTION, this will reference it</param>
		public void AddDeclaration(UhtCompilerDirective compilerDirectives, List<UhtToken> tokens, UhtFunction? function)
		{
			_declarations ??= [];
			_declarations.Add(new UhtDeclaration { CompilerDirectives = compilerDirectives, Tokens = [.. tokens], Function = function });
		}

		/// <summary>
		/// Search the declarations for a possible declaration match with the given name
		/// </summary>
		/// <param name="name">Name to be located</param>
		/// <param name="foundDeclaration">Information about the matched declaration</param>
		/// <returns>True if a declaration was found with the name.</returns>
		public bool TryGetDeclaration(string name, out UhtFoundDeclaration foundDeclaration)
		{
			if (name.Length > 0)
			{
				if (Declarations != null)
				{
					foreach (UhtDeclaration declaration in Declarations)
					{
						bool isVirtual = false;
						for (int index = 0; index < declaration.Tokens.Length; ++index)
						{
							if (declaration.Tokens[index].IsIdentifier("virtual"))
							{
								isVirtual = true;
							}
							else if (declaration.Tokens[index].IsIdentifier(name))
							{
								foundDeclaration = new UhtFoundDeclaration
								{
									CompilerDirectives = declaration.CompilerDirectives,
									Tokens = declaration.Tokens,
									NameTokenIndex = index,
									IsVirtual = isVirtual,
								};
								return true;
							}
						}
					}
				}
			}
			if (NativeInterface != null)
			{
				return NativeInterface.TryGetDeclaration(name, out foundDeclaration);
			}
			foundDeclaration = new UhtFoundDeclaration();
			return false;
		}

		/// <summary>
		/// Checks to see if the class or any super class has the given flags.
		/// </summary>
		/// <param name="flagsToCheck"></param>
		/// <returns>True if the flags are found</returns>
		public bool HierarchyHasAnyClassFlags(EClassFlags flagsToCheck)
		{
			for (UhtClass? current = this; current != null; current = current.SuperClass)
			{
				if (current.ClassFlags.HasAnyFlags(flagsToCheck))
				{
					return true;
				}
			}
			return false;
		}

		#region Resolution support
		struct GetterSetterToResolve
		{
			public UhtProperty Property { get; set; }
			public bool Setter { get; set; }
			public bool Found { get; set; }
		}

		/// <inheritdoc/>
		public override void BindSuperAndBases()
		{
			if (this != Session.UObject) // Don't try to find UObjectBaseUtility
			{
				BindSuper(SuperIdentifier, UhtFindOptions.Class);
			}
			BindBases(BaseIdentifiers, UhtFindOptions.Class);
			BindVerseInterfaces();
			base.BindSuperAndBases();
		}

		/// <inheritdoc/>
		protected override void ResolveSuper(UhtResolvePhase resolvePhase)
		{
			base.ResolveSuper(resolvePhase);

			if (_verseInterfaces != null)
			{
				foreach (UhtClass verseInterface in _verseInterfaces)
				{
					verseInterface.ResolveSuper(resolvePhase);
				}
			}

			// Make sure the class within is resolved.  We have to make sure we don't try to resolve ourselves.
			if (ClassWithin != null && ClassWithin != this)
			{
				ClassWithin.Resolve(resolvePhase);
			}

			switch (resolvePhase)
			{
				case UhtResolvePhase.Bases:
					UhtClass? superClass = SuperClass;

					// Force the MatchedSerializers on for anything being exported
					if (!ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
					{
						ClassFlags |= EClassFlags.MatchedSerializers;
					}

					// Roll up the verse interfaces
					if (_verseInterfaces != null)
					{
						_flattenedVerseInterfaces = [];
						_flattenedVerseInterfaces.AddRange(_verseInterfaces);
						int stopIndex = _flattenedVerseInterfaces.Count;
						for (int index = 0; index < stopIndex; index++)
						{
							UhtClass verseInterface = _flattenedVerseInterfaces[index];
							if (verseInterface._flattenedVerseInterfaces != null)
							{
								foreach (UhtClass subInterface in verseInterface._flattenedVerseInterfaces)
								{
									if (!_flattenedVerseInterfaces.Contains(subInterface))
									{
										_flattenedVerseInterfaces.Add(subInterface);
									}
								}
							}
						}
					}

					switch (ClassType)
					{
						case UhtClassType.Class:
							{
								// Validate some verse settings
								if (!IsVerseField && VerseInterfacesLine != 0)
								{
									this.LogError($"VINTERFACES can only appear of classes marked as Verse classes");
								}

								// Merge the super class flags
								if (superClass != null)
								{
									ClassFlags |= superClass.ClassFlags & EClassFlags.ScriptInherit & ~RemovedClassFlags;
								}

								foreach (UhtStruct baseStruct in Bases)
								{
									if (baseStruct is UhtClass baseClass)
									{
										if (!baseClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
										{
											this.LogError($"Class '{baseClass.SourceName}' is not an interface; Can only inherit from non-UObjects or UInterface derived interfaces");
										}
										if (baseClass.IsVerseField)
										{
											if (IsVerseField)
											{
												this.LogError($"Interface '{baseClass.SourceName}' must be listed in a VINTERFACES() macro");
											}
											else
											{
												this.LogError($"Non-Verse classes can not inherit the Verse interface '{baseClass.SourceName}");
											}
										}
										ClassFlags |= baseClass.ClassFlags & EClassFlags.ScriptInherit & ~RemovedClassFlags;
									}
								}

								ValidateVerseInterfaces();

								// These following collections only inherit from he parent if they are empty in this class
								if (SparseClassDataTypes.Count == 0)
								{
									AppendStringListMetaData(SuperClass, UhtNames.SparseClassDataTypes, SparseClassDataTypes);
								}
								if (PrioritizeCategories.Count == 0)
								{
									AppendStringListMetaData(SuperClass, UhtNames.PrioritizeCategories, PrioritizeCategories);
								}

								// Merge with categories inherited from the parent.
								MergeCategories();

								SetAndValidateWithinClass(resolvePhase);
								SetAndValidateConfigName();

								// Copy all of the lists back to the meta data
								MetaData.AddIfNotEmpty(UhtNames.ClassGroupNames, ClassGroupNames);
								MetaData.AddIfNotEmpty(UhtNames.AutoCollapseCategories, AutoCollapseCategories);
								MetaData.AddIfNotEmpty(UhtNames.AutoExpandCategories, AutoExpandCategories);
								MetaData.AddIfNotEmpty(UhtNames.PrioritizeCategories, PrioritizeCategories);
								MetaData.AddIfNotEmpty(UhtNames.HideCategories, HideCategories);
								MetaData.AddIfNotEmpty(UhtNames.ShowCategories, ShowCategories);
								MetaData.AddIfNotEmpty(UhtNames.SparseClassDataTypes, SparseClassDataTypes);
								MetaData.AddIfNotEmpty(UhtNames.HideFunctions, HideFunctions);

								MetaData.Add(UhtNames.IncludePath, HeaderFile.IncludeFilePath);
							}
							break;

						case UhtClassType.Interface:
							{
								if (IsVerseField)
								{
									ValidateVerseInterfaces();
								}

								if (superClass != null)
								{
									ClassFlags |= superClass.ClassFlags & EClassFlags.ScriptInherit;
									if (!superClass.ClassFlags.HasAnyFlags(EClassFlags.Native))
									{
										throw new UhtException(this, $"Native classes cannot extend non-native classes");
									}
									ClassWithin = superClass.ClassWithin;
								}
								else
								{
									ClassWithin = Session.UObject;
								}
							}
							break;

						case UhtClassType.NativeInterface:
							if (AlternateObject is UhtClass interfaceObj && interfaceObj.IsVerseField)
							{
								if (superClass != Session.IInterface)
								{
									this.LogError($"Verse native interface must derived from IInterface");
								}
							}
							break;

						case UhtClassType.VModule:
							ClassWithin = superClass != null ? superClass.ClassWithin : Session.UObject;

							foreach (UhtType child in Children)
							{
								if (child is UhtFunction function)
								{
									if (!function.FunctionFlags.HasAnyFlags(EFunctionFlags.Static) ||
										!function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CppStatic))
									{
										this.LogError($"Verse modules class's '{SourceName}' function '{function.SourceName}' must be declared as static");
									}
								}
							}
							break;

						default:
							throw new UhtIceException("Unexpected class type");
					}
					break;
			}
		}

		private void ValidateVerseInterfaces()
		{
			foreach (UhtClass verseInterface in VerseInterfaces)
			{
				if (verseInterface.ClassType != UhtClassType.NativeInterface)
				{
					this.LogError($"Class '{verseInterface.SourceName}' is not a native interface");
				}
				if (!verseInterface.IsVerseField)
				{
					this.LogError($"Class '{verseInterface.SourceName}' is not a Verse interface");
				}
			}
		}

		/// <summary>
		/// Return the native interface name from an interface
		/// </summary>
		/// <returns>Native interface name</returns>
		/// <exception cref="UhtIceException"></exception>
		public string GetNativeInterfaceName()
		{
			if (ClassType != UhtClassType.Interface)
			{
				throw new UhtIceException("Can only get the native interface name from interfaces");
			}
			return "I" + SourceName[1..];
		}

		/// <summary>
		/// Return the interface name from a native interface
		/// </summary>
		/// <returns>Interface name</returns>
		/// <exception cref="UhtIceException"></exception>
		public string GetInterfaceName()
		{
			if (ClassType != UhtClassType.NativeInterface)
			{
				throw new UhtIceException("Can only get the interface name from native interfaces");
			}
			return "U" + SourceName[1..];
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool result = base.ResolveSelf(phase);

			switch (phase)
			{
				case UhtResolvePhase.InvalidCheck:
					switch (ClassType)
					{
						case UhtClassType.Class:
							break;

						case UhtClassType.Interface:
							{
								string nativeInterfaceName = GetNativeInterfaceName();
								UhtType? nativeInterface = Session.FindType(null, UhtFindOptions.SourceName | UhtFindOptions.Class, nativeInterfaceName);
								if (nativeInterface == null)
								{
									this.LogError($"UInterface '{SourceName}' parsed without a corresponding '{nativeInterfaceName}'");
								}
								else
								{
									// Move native interface children into the interface
									MoveChildren(nativeInterface);
								}
							}
							break;

						case UhtClassType.NativeInterface:
							{
								string interfaceName = GetInterfaceName();
								UhtClass? interfaceObj = (UhtClass?)Session.FindType(null, UhtFindOptions.SourceName | UhtFindOptions.Class, interfaceName);
								// Only consider actual UINTERFACE classes, not regular UObject classes that happen to have a matching name
								if (interfaceObj?.ClassType != UhtClassType.Interface)
								{
									if (HasGeneratedBody || Children.Count != 0)
									{
										this.LogError($"Native interface '{SourceName}' parsed without a corresponding '{interfaceName}'");
									}
									else
									{
										Session.HideTypeInSymbolTable(this);
										result = false;
									}
								}
								else
								{
									AlternateObject = interfaceObj;
									if (interfaceObj.IsVerseField)
									{
										VerseName = SourceName;
									}
									interfaceObj.NativeInterface = this;
									//COMPATIBILITY-TODO - Use the native interface access specifier
									interfaceObj.GeneratedBodyAccessSpecifier = GeneratedBodyAccessSpecifier;

									if (GeneratedBodyLineNumber == -1)
									{
										this.LogError("Expected a GENERATED_BODY() at the start of the native interface");
									}
								}
							}
							break;

						case UhtClassType.VModule:
							break;

						default:
							throw new UhtIceException("Unexpected class type");
					}
					break;

				case UhtResolvePhase.Bases:

					// Check to see if this class matches an entry in the ClassCastFlags.  If it does, that
					// becomes our ClassCastFlags.  If not, then we inherit the flags from the super.
					// Also for legacy UObject-derived property types, try replacing the U with F. e.g. UByteProperty -> FByteProperty
					if (Enum.TryParse<EClassCastFlags>(SourceName, false, out EClassCastFlags found) || Enum.TryParse($"F{SourceName[1..]}", false, out found))
					{
						if (found != EClassCastFlags.None)
						{
							ClassCastFlags = found;
						}
					}
					if (ClassCastFlags == EClassCastFlags.None && SuperClass != null)
					{
						ClassCastFlags = SuperClass.ClassCastFlags;
					}

					// force members to be 'blueprint read only' if in a const class
					// This must be done after the Const flag has been propagated from the parent
					// and before properties have been finalized.
					if (ClassType == UhtClassType.Class && ClassFlags.HasAnyFlags(EClassFlags.Const))
					{
						foreach (UhtType child in Children)
						{
							if (child is UhtProperty property)
							{
								property.PropertyFlags |= EPropertyFlags.BlueprintReadOnly;
								property.PropertyExportFlags |= UhtPropertyExportFlags.ImpliedBlueprintPure;
							}
						}
					}
					break;

				case UhtResolvePhase.Properties:
					UhtPropertyParser.ResolveChildren(this, UhtPropertyParseOptions.AddModuleRelativePath);
					break;

				case UhtResolvePhase.Final:
					Dictionary<string, List<GetterSetterToResolve>>? gsToResolve = null;
					foreach (UhtType child in Children)
					{
						if (child is UhtProperty property)
						{
							if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.FieldNotify))
							{
								ClassExportFlags |= UhtClassExportFlags.HasFieldNotify;
							}

							if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.Net))
							{
								ClassExportFlags |= UhtClassExportFlags.SelfHasReplicatedProperties;
							}

							if (!property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecifiedNone))
							{
								if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecified))
								{
									// if it's Auto, it will be generated so no need to look for a user implementation
									if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterSpecifiedAuto))
									{
										property.PropertyExportFlags |= UhtPropertyExportFlags.GetterFound;
										property.Getter = "Get" + property.SourceName;
									}
									else
									{
										gsToResolve = AddGetterSetter(gsToResolve, property.Getter ?? $"Get{property.SourceName}", property, false);
									}
								}
							}

							if (!property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterSpecifiedNone))
							{
								if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterSpecified))
								{
									// if it's Auto, it will be generated so no need to look for a user implementation
									if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterSpecifiedAuto))
									{
										property.PropertyExportFlags |= UhtPropertyExportFlags.SetterFound;
										property.Setter = "Set" + property.SourceName;
									}
									else
									{
										gsToResolve = AddGetterSetter(gsToResolve, property.Setter ?? $"Set{property.SourceName}", property, true);
									}
								}
							}
						}
						else if (child is UhtFunction function)
						{
							if (function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.FieldNotify))
							{
								ClassExportFlags |= UhtClassExportFlags.HasFieldNotify;
							}
						}
					}

					if (SuperClass != null)
					{
						if (SuperClass.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasReplciatedProperties))
						{
							ClassExportFlags |= UhtClassExportFlags.SuperHasReplicatedProperties;
						}
					}

					if (!ClassFlags.HasAnyFlags(EClassFlags.Interface))
					{
						if (ClassExportFlags.HasAnyFlags(UhtClassExportFlags.SelfHasReplicatedProperties))
						{
							if (TryGetDeclaration("GetLifetimeReplicatedProps", out UhtFoundDeclaration _))
							{
								ClassExportFlags |= UhtClassExportFlags.HasGetLifetimeReplicatedProps;
							}
						}
					}

					// If we have any getters/setters to resolve 
					if (gsToResolve != null)
					{
						if (Declarations != null)
						{
							foreach (UhtDeclaration declaration in Declarations)
							{

								// Locate the index of the name
								int nameIndex = 0;
								for (; nameIndex < declaration.Tokens.Length - 1; ++nameIndex)
								{
									if (declaration.Tokens[nameIndex + 1].IsValue('('))
									{
										break;
									}
								}

								string name = declaration.Tokens[nameIndex].Value.ToString();
								if (gsToResolve.TryGetValue(name, out List<GetterSetterToResolve>? gsList))
								{
									for (int index = 0; index < gsList.Count; ++index)
									{
										GetterSetterToResolve gs = gsList[index];
										if (!gs.Found)
										{
											if (TryMatchGetterSetter(gs.Property, gs.Setter, declaration, nameIndex))
											{
												gs.Found = true;
												gsList[index] = gs;
												gs.Property.PropertyExportFlags |= gs.Setter ? UhtPropertyExportFlags.SetterFound : UhtPropertyExportFlags.GetterFound;
												if (gs.Setter)
												{
													gs.Property.Setter = name;
												}
												else
												{
													gs.Property.Getter = name;
												}
											}
										}
									}
								}
							}
						}
					}
					break;
			}

			return result;
		}

		/// <inheritdoc/>
		protected override void ResolveChildren(UhtResolvePhase phase)
		{
			base.ResolveChildren(phase);

			switch (phase)
			{
				case UhtResolvePhase.Final:
					LinkFunctions();
					if (ScanForInstancedReferenced(false))
					{
						ClassFlags |= EClassFlags.HasInstancedReference;
					}
					break;
			}
		}

		/// <summary>
		/// Create a linked list of functions across preprocessor definition changes
		/// </summary>
		protected void LinkFunctions()
		{
			List<UhtFunction> functions = [];
			foreach (UhtType type in Children)
			{
				// Delegates are not linked into class's field list
				if (type is UhtFunction function && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
				{
					functions.Add(function);
				}
			}
			functions.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.EngineName, y.EngineName));

			using UhtDefineScopeListBuilder<UhtFunction> functionList = new(DefineScope);
			foreach (UhtFunction function in functions)
			{
				if (function.NextFunction is not null)
				{
					throw new Exception("Unexpectedly initialized next link on property");
				}
				function.NextFunction = functionList.Add(function);
			}
			FirstFunction = functionList.Head;
		}

		/// <inheritdoc/>
		protected override bool ScanForInstancedReferencedInternal(bool deepScan)
		{
			if (ClassFlags.HasAnyFlags(EClassFlags.HasInstancedReference))
			{
				return true;
			}

			if (SuperClass != null && SuperClass.ScanForInstancedReferenced(deepScan))
			{
				return true;
			}

			return base.ScanForInstancedReferencedInternal(deepScan);
		}

		private void MergeCategories()
		{
			MergeShowCategories();

			// Merge ShowFunctions and HideFunctions
			AppendStringListMetaData(SuperClass, UhtNames.HideFunctions, HideFunctions);
			foreach (string value in ShowFunctions)
			{
				HideFunctions.RemoveSwap(value);
			}
			ShowFunctions.Clear();

			// Merge AutoExpandCategories and AutoCollapseCategories (we still want to keep AutoExpandCategories though!)
			List<string> parentAutoExpandCategories = GetStringListMetaData(SuperClass, UhtNames.AutoExpandCategories);
			List<string> parentAutoCollapseCategories = GetStringListMetaData(SuperClass, UhtNames.AutoCollapseCategories);

			foreach (string value in AutoExpandCategories)
			{
				AutoCollapseCategories.RemoveSwap(value);
				parentAutoCollapseCategories.RemoveSwap(value);
			}

			// Do the same as above but the other way around
			foreach (string value in AutoCollapseCategories)
			{
				AutoExpandCategories.RemoveSwap(value);
				parentAutoExpandCategories.RemoveSwap(value);
			}

			// Once AutoExpandCategories and AutoCollapseCategories for THIS class have been parsed, add the parent inherited categories
			AutoCollapseCategories.AddRange(parentAutoCollapseCategories);
			AutoExpandCategories.AddRange(parentAutoExpandCategories);
		}

		private void MergeShowCategories()
		{

			// Add the super class hide categories and prime the output show categories with the parent
			List<string> outShowCategories = GetStringListMetaData(SuperClass, UhtNames.ShowCategories);
			AppendStringListMetaData(SuperClass, UhtNames.HideCategories, HideCategories);

			// If this class has new show categories, then merge them
			if (ShowCategories.Count != 0)
			{
				StringBuilder subCategoryPath = new();
				foreach (string value in ShowCategories)
				{

					// if we didn't find this specific category path in the HideCategories metadata
					if (!HideCategories.RemoveSwap(value))
					{
						string[] subCategories = value.ToString().Split('|', StringSplitOptions.RemoveEmptyEntries);

						subCategoryPath.Clear();
						// look to see if any of the parent paths are excluded in the HideCategories list
						for (int categoryPathIndex = 0; categoryPathIndex < subCategories.Length - 1; ++categoryPathIndex)
						{
							subCategoryPath.Append(subCategories[categoryPathIndex]);
							// if we're hiding a parent category, then we need to flag this sub category for show
							if (HideCategories.Contains(subCategoryPath.ToString()))
							{
								outShowCategories.AddUnique(value);
								break;
							}
							subCategoryPath.Append('|');
						}
					}
				}
			}

			// Replace the show categories
			ShowCategories.Clear();
			ShowCategories.AddRange(outShowCategories);
		}

		private void SetAndValidateWithinClass(UhtResolvePhase resolvePhase)
		{
			// The class within must be a child of any super class within
			UhtClass expectedClassWithin = SuperClass != null ? SuperClass.ClassWithin : Session.UObject;

			// Process all of the class specifiers
			if (!String.IsNullOrEmpty(ClassWithinIdentifier))
			{
				UhtClass? specifiedClassWithin = (UhtClass?)Session.FindType(null, UhtFindOptions.EngineName | UhtFindOptions.Class, ClassWithinIdentifier);
				if (specifiedClassWithin == null)
				{
					this.LogError($"Within class '{ClassWithinIdentifier}' not found");
				}
				else
				{

					// Make sure the class within has been resolved to this phase.  We don't need to worry about the super since we know 
					// that it has already been resolved.
					if (specifiedClassWithin != this)
					{
						specifiedClassWithin.Resolve(resolvePhase);
					}

					if (specifiedClassWithin.IsChildOf(Session.UInterface))
					{
						this.LogError("Classes cannot be 'within' interfaces");
					}
					else if (!specifiedClassWithin.IsChildOf(expectedClassWithin))
					{
						this.LogError($"Cannot override within class with '{specifiedClassWithin.SourceName}' since it isn't a child of parent class's expected within '{expectedClassWithin.SourceName}'");
					}
					else
					{
						ClassWithin = specifiedClassWithin;
					}
				}
			}

			// If we don't have a class within set, then just use the expected within
			else
			{
				ClassWithin = expectedClassWithin;
			}
		}

		private void SetAndValidateConfigName()
		{
			// Since this flag is computed in this method, we have to re-propagate the flag from the super
			// just in case they were defined in this source file.
			if (SuperClass != null)
			{
				ClassFlags |= SuperClass.ClassFlags & EClassFlags.Config;
			}

			// Set the class config flag if any properties have config
			if (!ClassFlags.HasAnyFlags(EClassFlags.Config))
			{
				foreach (UhtProperty property in Properties)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.Config))
					{
						ClassFlags |= EClassFlags.Config;
						break;
					}
				}
			}

			if (Config.Length > 0)
			{
				// if the user specified "inherit", we're just going to use the parent class's config filename
				// this is not actually necessary but it can be useful for explicitly communicating config-ness
				if (Config.Equals("inherit", StringComparison.OrdinalIgnoreCase))
				{
					if (SuperClass == null)
					{
						this.LogError($"Cannot inherit config filename for class '{SourceName}' when it has no super class");
					}
					else if (SuperClass.Config.Length == 0)
					{
						this.LogError($"Cannot inherit config filename for class '{SourceName}' when parent class '{SuperClass.SourceName}' has no config filename");
					}
					else
					{
						Config = SuperClass.Config;
					}
				}
			}
			else if (ClassFlags.HasAnyFlags(EClassFlags.Config) && SuperClass != null)
			{
				Config = SuperClass.Config;
			}

			if (ClassFlags.HasAnyFlags(EClassFlags.Config) && Config.Length == 0)
			{
				this.LogError("Classes with config / globalconfig member variables need to specify config file.");
				Config = "Engine";
			}
		}

		private static List<string> GetStringListMetaData(UhtType? type, string key)
		{
			List<string> outStrings = [];
			AppendStringListMetaData(type, key, outStrings);
			return outStrings;
		}

		private static void AppendStringListMetaData(UhtType? type, string key, List<string> stringList)
		{
			if (type != null)
			{
				string[]? values = type.MetaData.GetStringArray(key);
				if (values != null)
				{
					foreach (string value in values)
					{
						//COMPATIBILITY-TODO - TEST - This preserves duplicates that are in old UHT
						//StringList.AddUnique(Value);
						stringList.Add(value);
					}
				}
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendForwardDeclaration(StringBuilder builder)
		{
			return builder.Append($"class {SourceName};");
		}

		/// <summary>
		/// Given a token stream, verify that it matches the expected signature for a getter/setter
		/// </summary>
		/// <param name="property">Property requesting the setter/getter</param>
		/// <param name="setter">If true, a setter is expected</param>
		/// <param name="declaration">The declaration being tested</param>
		/// <param name="nameIndex">The index of the token with the expected getter/setter name</param>
		/// <returns>True if the declaration matches</returns>
		private static bool TryMatchGetterSetter(UhtProperty property, bool setter, UhtDeclaration declaration, int nameIndex)
		{
			UhtToken[] tokens = declaration.Tokens;
			int tokenIndex = 0;
			int tokenCount = tokens.Length;

			// Skip all of the API and virtual keywords
			for (; tokenIndex < tokenCount && (tokens[tokenIndex].IsValue("virtual") || tokens[tokenIndex].Value.Span.EndsWith("_API", StringComparison.Ordinal)); ++tokenIndex)
			{
				// Nothing to do in the body
			}

			int typeIndex;
			int typeCount;

			// Verify the format of the function declaration and extract the type
			if (setter)
			{

				// The return type must be 'void'
				if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue("void"))
				{
					return false;
				}
				tokenIndex++;

				// Followed by the name
				if (tokenIndex == tokenCount || tokenIndex != nameIndex)
				{
					return false;
				}
				tokenIndex++;

				// Followed by the '('
				if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue('('))
				{
					return false;
				}
				tokenIndex++;

				// Skip any UPARAM only in actual functions
				if (declaration.Function != null && tokenIndex < tokenCount && tokens[tokenIndex].IsIdentifier("UPARAM"))
				{
					tokenIndex++;
					if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue('('))
					{
						return false;
					}
					tokenIndex++;
					int paramCount = 1;
					while (tokenIndex < tokenCount && paramCount > 0)
					{
						if (tokens[tokenIndex].IsValue(')'))
						{
							paramCount--;
						}
						else if (tokens[tokenIndex].IsValue('('))
						{
							paramCount++;
						}
						tokenIndex++;
					}
					if (paramCount > 0)
					{
						return false;
					}
				}

				// Find the ')'
				typeIndex = tokenIndex;
				while (true)
				{
					if (tokenIndex == tokenCount)
					{
						return false;
					}
					if (tokens[tokenIndex].IsValue(')'))
					{
						typeCount = tokenIndex - typeIndex;
						break;
					}
					tokenIndex++;
				}
			}
			else
			{
				// Get the type range
				typeIndex = tokenIndex;
				typeCount = nameIndex - typeIndex;
				tokenIndex = nameIndex + 1;

				// Followed by the '('
				if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue('('))
				{
					return false;
				}
				tokenIndex++;

				// Followed by the ')'
				if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue(')'))
				{
					return false;
				}
				tokenIndex++;

				// Followed by the "const"
				if (tokenIndex == tokenCount || !tokens[tokenIndex].IsValue("const"))
				{
					return false;
				}
#pragma warning disable IDE0059 // Unnecessary assignment of a value
				tokenIndex++;
#pragma warning restore IDE0059 // Unnecessary assignment of a value
			}

			// Verify the type
			{
				using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
				StringBuilder builder = borrower.StringBuilder;
				builder.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg);
				if (property.IsStaticArray)
				{
					if (declaration.Function == null)
					{
						builder.Append('*');
					}
					else
					{
						builder.Append('[').Append(property.ArrayDimensions).Append(']');
					}
				}
				string expectedType = builder.ToString();
				int typeEnd = typeIndex + typeCount;

				// We will do a somewhat brute force string compare to avoid constructing an expected
				// string or parsing the required string
				ReadOnlySpan<char> expectedSpan = expectedType.AsSpan().Trim();

				// skip const. We allow const in parameter and return values even if the property is not const
				if (typeIndex < typeEnd && tokens[typeIndex].IsIdentifier("const"))
				{
					typeIndex++;
				}

				// Loop until we are done
				while (typeIndex < typeEnd && expectedSpan.Length > 0)
				{
					UhtToken token = tokens[typeIndex];
					ReadOnlySpan<char> tokenSpan = token.Value.Span;

					// Make sure we match the expected type
					if (tokenSpan.Length > expectedSpan.Length ||
						!expectedSpan.StartsWith(tokenSpan, StringComparison.Ordinal))
					{
						return false;
					}

					// Extract the remaining part of the string we expect to match
					expectedSpan = expectedSpan[tokenSpan.Length..].TrimStart();
					typeIndex++;
				}

				// Consume any '&' for reference support
				if (typeIndex < typeEnd && tokens[typeIndex].IsSymbol('&'))
				{
					typeIndex++;
				}

				// For setters, there can be an identifier
				if (setter && typeIndex < typeEnd && tokens[typeIndex].IsIdentifier())
				{
					typeIndex++;
				}

				// We must be at the end for a match
				if (typeIndex != typeEnd)
				{
					return false;
				}
			}

			// Validate the #if blocks.  This should really be done during validation
			string funcType = setter ? "setter" : "getter";
			UhtPropertyExportFlags testFlag = setter ? UhtPropertyExportFlags.SetterSpecified : UhtPropertyExportFlags.GetterSpecified;
			if (declaration.CompilerDirectives.HasAnyFlags(UhtCompilerDirective.WithEditor))
			{
				if (property.PropertyExportFlags.HasAnyFlags(testFlag))
				{
					property.LogError(tokens[0].InputLine, $"Property {property.SourceName} {funcType} function {tokens[nameIndex].Value} "
						+ "cannot be declared within WITH_EDITOR block. Use WITH_EDITORONLY_DATA instead.");
				}
				return false;
			}
			if (declaration.CompilerDirectives.HasAnyFlags(UhtCompilerDirective.WithEditorOnlyData) && !property.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly))
			{
				if (property.PropertyExportFlags.HasAnyFlags(testFlag))
				{
					property.LogError(tokens[0].InputLine, $"Property {property.SourceName} is not editor-only but its {funcType} function '{tokens[nameIndex].Value}' is");
				}
				return false;
			}
			if (declaration.Function != null)
			{
				if (!declaration.Function.FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
				{
					property.LogError(tokens[0].InputLine, $"Property {property.SourceName} {funcType} function '{tokens[nameIndex].Value}' has to be native");
				}
				if (declaration.Function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net | EFunctionFlags.Event))
				{
					property.LogError(tokens[0].InputLine, $"Property {property.SourceName} {funcType} function '{tokens[nameIndex].Value}' cannot be a net or an event");
				}
			}
			return true;
		}

		/// <summary>
		/// Add a requested getter/setter function name
		/// </summary>
		/// <param name="gsToResolve">Dictionary containing the lookup by function name.  Will be created if null.</param>
		/// <param name="name">Name of the getter/setter</param>
		/// <param name="property">Property requesting the getter/setter</param>
		/// <param name="setter">True if this is a setter</param>
		/// <returns>Resulting dictionary</returns>
		private static Dictionary<string, List<GetterSetterToResolve>> AddGetterSetter(Dictionary<string, List<GetterSetterToResolve>>? gsToResolve, string name, UhtProperty property, bool setter)
		{
			gsToResolve ??= [];
			if (!gsToResolve.TryGetValue(name, out List<GetterSetterToResolve>? gsList))
			{
				gsList = [];
				gsToResolve.Add(name, gsList);
			}
			gsList.Add(new GetterSetterToResolve { Property = property, Setter = setter });
			return gsToResolve;
		}
		#endregion

		#region Validation support
		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);

			// Classes must start with a valid prefix
			if (!IsVerseField)
			{
				string expectedClassName = EngineNamePrefix + EngineName;
				if (expectedClassName != SourceName)
				{
					this.LogError($"Class '{SourceName}' has an invalid Unreal prefix, expecting '{expectedClassName}'");
				}
			}

			// If we have a super class
			if (SuperClass != null)
			{
				if (!ClassFlags.HasAnyFlags(EClassFlags.NotPlaceable) && SuperClass.ClassFlags.HasAnyFlags(EClassFlags.NotPlaceable))
				{
					this.LogError("The 'placeable' specifier cannot override a 'nonplaceable' base class. Classes are assumed to be placeable by default. "
						+ "Consider whether using the 'abstract' specifier on the base class would work.");
				}

				// Native interfaces don't require a super class, but UHT technically allows it.
				if (ClassType == UhtClassType.Interface || ClassType == UhtClassType.NativeInterface)
				{
					if (IsVerseField)
					{
						if (ClassType == UhtClassType.Interface && SuperClass != Session.UInterface)
						{
							this.LogError($"Verse Interface class '{SourceName}' must inherit from UInterface'");
						}
						else if (ClassType == UhtClassType.NativeInterface && SuperClass != Session.IInterface)
						{
							this.LogError($"Verse Native Interface class '{SourceName}' must inherit from IInterface'");
						}
					}
					else if (this != Session.UInterface && !SuperClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
					{
						this.LogError($"Interface class '{SourceName}' cannot inherit from non-interface class '{SuperClass.SourceName}'");
					}
				}
			}

			// If we don't have a super class
			else
			{
				if (ClassType == UhtClassType.Interface)
				{
					this.LogError($"Interface '{SourceName}' must derive from an interface or UInterface");
				}
				else if (ClassType == UhtClassType.Class && this != Session.UObject)
				{
					this.LogError($"UClass '{SourceName}' should derive from UObject or another UClass");
				}
			}

			if (ClassFlags.HasAnyFlags(EClassFlags.NeedsDeferredDependencyLoading) && !IsChildOf(Session.UClass))
			{
				// CLASS_NeedsDeferredDependencyLoading can only be set on classes derived from UClass
				this.LogError($"'NeedsDeferredDependencyLoading' is set on '{SourceName}' but the flag can only be used with classes derived from UClass.");
			}

			if (ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasFieldNotify))
			{
				if (Session.INotifyFieldValueChanged == null)
				{
					this.LogError($"UClass '{SourceName}' has FieldNotify elements but the interface 'INotifyFieldValueChanged' is not defined.");
				}
				else if (!ImplementsInterface(Session.INotifyFieldValueChanged))
				{
					this.LogError($"UClass '{SourceName}' need to implement the interface INotifyFieldValueChanged' to support FieldNotify.");
				}
			}

			ValidateProperties();

			// Perform any validation specific to the class type
			switch (ClassType)
			{
				case UhtClassType.Class:
					{
						if (ClassFlags.HasAnyFlags(EClassFlags.EditInlineNew))
						{
							// don't allow actor classes to be declared EditInlineNew
							if (IsChildOf(Session.AActor))
							{
								this.LogError("Invalid class attribute: Creating actor instances via the property window is not allowed");
							}
						}

						// Make sure both RequiredAPI and MinimalAPI aren't specified
						if (ClassFlags.HasAllFlags(EClassFlags.MinimalAPI | EClassFlags.RequiredAPI))
						{
							this.LogError("MinimalAPI cannot be specified when the class is fully exported using a MODULENAME_API macro");
						}

						// Make sure that all interface functions are implemented property
						// Iterate over all the bases looking for any interfaces
						foreach (UhtStruct baseStruct in Bases)
						{

							// Skip children that aren't interfaces or if a common ancestor
							UhtClass? baseClass = baseStruct as UhtClass;
							if (baseClass == null || baseClass.ClassType == UhtClassType.Class || IsChildOf(baseClass))
							{
								continue;
							}

							// Get the actual interface
							UhtClass interfaceClass = baseClass.AlternateObject != null ? (UhtClass)baseClass.AlternateObject : baseClass;

							// Loop through the function to make sure they are implemented
							foreach (UhtType baseChild in interfaceClass.Children)
							{
								UhtFunction? baseFunction = baseChild as UhtFunction;
								if (baseFunction == null)
								{
									continue;
								}

								// Delegate signature functions are simple stubs and aren't required to be implemented (they are not callable)
								bool implemented = baseFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate);

								// Try to find an existing function
								foreach (UhtType child in Children)
								{
									UhtFunction? function = child as UhtFunction;
									if (function == null || function.SourceName != baseFunction.SourceName)
									{
										continue;
									}

									if (baseFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Event) && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
									{
										function.LogError($"Implementation of function '{function.SourceName}' must be declared as 'event' to match declaration in interface '{baseClass.SourceName}'");
									}

									if (baseFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate) && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
									{
										function.LogError($"Implementation of function '{function.SourceName}' must be declared as 'delegate' to match declaration in interface '{baseClass.SourceName}'");
									}

									implemented = true;

									if (baseFunction.Children.Count != function.Children.Count)
									{
										function.LogError($"Implementation of function '{function.SourceName}' conflicts with interface '{baseClass.SourceName}' - different number of parameters ({function.Children.Count}/{baseFunction.Children.Count})");
										continue;
									}

									for (int index = 0; index < function.Children.Count; ++index)
									{
										UhtProperty baseProperty = (UhtProperty)baseFunction.Children[index];
										UhtProperty property = (UhtProperty)function.Children[index];
										if (!baseProperty.MatchesType(property))
										{
											if (baseProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm))
											{
												function.LogError($"Implementation of function '{function.SourceName}' conflicts only by return type with interface '{baseClass.SourceName}'");
											}
											else
											{
												function.LogError($"Implementation of function '{function.SourceName}' conflicts type with interface '{baseClass.SourceName}' - parameter {index} '{property.SourceName}'");
											}
										}
									}
								}

								// Verify that if this has blueprint-callable functions that are not implementable events, we've implemented them as a UFunction in the target class
								if (!implemented
									&& baseFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable)
									&& !baseFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent)
									&& !interfaceClass.CannotImplementInBlueprints())
								{
									this.LogError($"Missing UFunction implementation of function '{baseFunction.SourceName}' from interface '{baseClass.SourceName}'.  This function needs a UFUNCTION() declaration.");
								}
							}
						}
					}

					if (!ClassFlags.HasAnyFlags(EClassFlags.Interface))
					{
						if (ClassExportFlags.HasAnyFlags(UhtClassExportFlags.SelfHasReplicatedProperties) &&
							!ClassExportFlags.HasAnyFlags(UhtClassExportFlags.HasGetLifetimeReplicatedProps) &&
							GeneratedCodeVersion != EGeneratedCodeVersion.V1)
						{
							this.LogError($"Class {SourceName} has Net flagged properties and should declare member function: void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override");
						}
					}
					break;

				case UhtClassType.Interface:
					{

						// Verify that our supers match
						if (NativeInterface != null && SuperClass != null)
						{
							if (SuperClass == Session.UInterface)
							{
								if (NativeInterface.SuperClass != null && NativeInterface.SuperClass != Session.IInterface)
								{
									this.LogError($"Interface {SourceName} derives from {SuperClass.SourceName}, therefore {NativeInterface.SourceName} can not derive from an another native interface");
								}
							}
							else if (SuperClass.NativeInterface != null)
							{
								if (SuperClass.NativeInterface != NativeInterface.SuperClass)
								{
									this.LogError($"Interface {SourceName} derives from {SuperClass.SourceName}, therefore {NativeInterface.SourceName} must derive from {SuperClass.NativeInterface.SourceName}");
								}
							}
						}

						// Interface with blueprint data should declare explicitly Blueprintable or NotBlueprintable to be clear
						// In the backward compatible case where they declare neither, both of these bools are false
						bool canImplementInBlueprints = CanImplementInBlueprints();
						bool cannotImplementInBlueprints = CannotImplementInBlueprints(canImplementInBlueprints);
						foreach (UhtType child in Children)
						{
							if (child is UhtFunction function)
							{
								// Verify interfaces with respect to their blueprint accessible functions
								if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent) && !function.MetaData.GetBoolean(UhtNames.BlueprintInternalUseOnly))
								{
									// Ensure that blueprint events are only allowed in implementable interfaces. Internal only functions allowed
									if (cannotImplementInBlueprints)
									{
										function.LogError("Interfaces that are not implementable in blueprints cannot have Blueprint Event members.");
									}
									if (!canImplementInBlueprints)
									{
										// We do not currently warn about this case as there are a large number of existing interfaces that do not specify
										// Function.LogWarning(TEXT("Interfaces with Blueprint Events should declare Blueprintable on the interface."));
									}
								}

								if (function.FunctionFlags.HasExactFlags(EFunctionFlags.BlueprintCallable | EFunctionFlags.BlueprintEvent, EFunctionFlags.BlueprintCallable))
								{
									// Ensure that if this interface contains blueprint callable functions that are not blueprint defined, that it must be implemented natively
									if (canImplementInBlueprints)
									{
										function.LogError("Blueprint implementable interfaces cannot contain BlueprintCallable functions that are not "
											+ "BlueprintImplementableEvents. Add NotBlueprintable to the interface if you wish to keep this function.");
									}
									if (!cannotImplementInBlueprints)
									{
										// Lowered this case to a warning instead of error, they will not show up as blueprintable unless they also have events
										function.LogWarning("Interfaces with BlueprintCallable functions but no events should explicitly declare NotBlueprintable on the interface.");
									}
								}
							}
						}
					}
					break;

				case UhtClassType.NativeInterface:
					break;
			}

			return options |= UhtValidationOptions.Shadowing | UhtValidationOptions.Deprecated;
		}

		static void ValidateBlueprintPopertyGetter(UhtProperty property, string functionName, UhtFunction? targetFunction)
		{
			if (targetFunction == null)
			{
				property.LogError($"Blueprint Property getter function '{functionName}' not found");
				return;
			}

			if (targetFunction.ParameterProperties.Length != 0)
			{
				targetFunction.LogError($"Blueprint Property getter function '{targetFunction.SourceName}' must not have parameters.");
			}

			UhtProperty? returnProperty = targetFunction.ReturnProperty;
			if (returnProperty == null || !property.IsSameType(returnProperty))
			{
				targetFunction.LogError($"Blueprint Property getter function '{targetFunction.SourceName}' must have return value of type '{property.GetUserFacingDecl()}'.");
			}

			if (targetFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
			{
				targetFunction.LogError($"Blueprint Property getter function '{targetFunction.SourceName}' cannot be a blueprint event.");
			}

			if (!targetFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure))
			{
				targetFunction.LogError($"Blueprint Property getter function '{targetFunction.SourceName}' must be pure.");
			}
		}

		static void ValidateBlueprintPopertySetter(UhtProperty property, string functionName, UhtFunction? targetFunction)
		{
			if (targetFunction == null)
			{
				property.LogError($"Blueprint Property setter function '{functionName}' not found");
				return;
			}

			if (targetFunction.ReturnProperty != null)
			{
				targetFunction.LogError($"Blueprint Property setter function '{targetFunction.SourceName}' must not have a return value.");
			}

			if (targetFunction.ParameterProperties.Length != 1 || !property.IsSameType((UhtProperty)targetFunction.ParameterProperties.Span[0]))
			{
				targetFunction.LogError($"Blueprint Property setter function '{targetFunction.SourceName}' must have exactly one parameter of type '{property.GetUserFacingDecl()}'.");
			}

			if (targetFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
			{
				targetFunction.LogError($"Blueprint Property setter function '{targetFunction.SourceName}' cannot be a blueprint event.");
			}

			if (!targetFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable))
			{
				targetFunction.LogError($"Blueprint Property setter function '{targetFunction.SourceName}' must be a blueprint callable.");
			}

			if (targetFunction.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure))
			{
				targetFunction.LogError($"Blueprint Property setter function '{targetFunction.SourceName}' must not be pure.");
			}
		}

		static void ValidateRepNotifyCallback(UhtProperty property, string functionName, UhtFunction? targetFunction)
		{
			if (targetFunction == null)
			{
				property.LogError($"Replication notification function '{functionName}' not found");
				return;
			}

			if (targetFunction.ReturnProperty != null)
			{
				targetFunction.LogError($"Replication notification function '{targetFunction.SourceName}' must not have a return value.");
			}

			bool isArrayProperty = property.IsStaticArray || property is UhtArrayProperty;
			int maxParms = isArrayProperty ? 2 : 1;
			ReadOnlyMemory<UhtType> parameters = targetFunction.ParameterProperties;

			if (parameters.Length > maxParms)
			{
				targetFunction.LogError($"Replication notification function '{targetFunction.SourceName}' has too many parameters.");
			}

			if (parameters.Length >= 1)
			{
				UhtProperty parm = (UhtProperty)parameters.Span[0];
				// First parameter is always the old value:
				if (!property.IsSameType(parm))
				{
					targetFunction.LogError($"Replication notification function '{targetFunction.SourceName}' first (optional) parameter must be of type '{property.GetUserFacingDecl()}'.");
				}
			}

			if (parameters.Length >= 2)
			{
				UhtProperty parm = (UhtProperty)parameters.Span[1];
				// First parameter is always the old value:
				bool isValid = false;
				if (parm is UhtArrayProperty arrayProperty)
				{
					isValid = arrayProperty.ValueProperty is UhtByteProperty && parm.PropertyFlags.HasAllFlags(EPropertyFlags.ConstParm | EPropertyFlags.ReferenceParm);
				}
				if (!isValid)
				{
					targetFunction.LogError($"Replication notification function '{targetFunction.SourceName}' second (optional) parameter must be of type 'const TArray<uint8>&'.");
				}
			}
		}

		void ValidateProperties()
		{
			foreach (UhtType child in Children)
			{
				if (child is UhtProperty property)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.RepNotify))
					{
						string name = property.RepNotifyName ?? String.Empty;
						ValidateRepNotifyCallback(property, name, (UhtFunction?)FindType(UhtFindOptions.EngineName | UhtFindOptions.Function, name.ToString()));
					}

					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible))
					{
						if (property.MetaData.TryGetValue(UhtNames.BlueprintGetter, out string? getter) && getter.Length > 0)
						{
							ValidateBlueprintPopertyGetter(property, getter, (UhtFunction?)FindType(UhtFindOptions.EngineName | UhtFindOptions.Function, getter));
						}

						if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintReadOnly))
						{
							if (property.MetaData.TryGetValue(UhtNames.BlueprintSetter, out string? setter) && setter.Length > 0)
							{
								ValidateBlueprintPopertySetter(property, setter, (UhtFunction?)FindType(UhtFindOptions.EngineName | UhtFindOptions.Function, setter));
							}
						}

						if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintAssignable | EPropertyFlags.BlueprintCallable) && property is not UhtMulticastDelegateProperty)
						{
							if (property.IsStaticArray)
							{
								property.LogError($"Static array cannot be exposed to blueprint Class: {SourceName} Property: {property.SourceName}");
							}

							if (!property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsMemberSupportedByBlueprint))
							{
								property.LogError($"Type '{property.GetUserFacingDecl()}' is not supported by blueprint. Class: {SourceName} Property: {property.SourceName}");
							}
						}
					}

					// Validate if we are using editor only data in a class or struct definition
					if (ClassFlags.HasAnyFlags(EClassFlags.Optional))
					{
						if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly))
						{
							property.LogError("Cannot specify an editor only property inside an optional class (adding \"AllowedInOptional\" metadata will not remove this error, as this is a design issue).");
						}
						else if (!property.IsAllowedInOptionalClass(out string? propPath))
						{
							property.LogError($"Do not specify struct property ({propPath}) containing editor only properties inside an optional class. If it's needed, add \"AllowedInOptional\" metadata to the property.");
						}
					}

					// Check for getter/setter
					if (property.PropertyExportFlags.HasExactFlags(UhtPropertyExportFlags.GetterSpecified | UhtPropertyExportFlags.GetterSpecifiedNone | UhtPropertyExportFlags.GetterFound | UhtPropertyExportFlags.GetterSpecifiedAuto, UhtPropertyExportFlags.GetterSpecified))
					{
						using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
						StringBuilder builder = borrower.StringBuilder;
						string expectedName = property.Getter ?? $"Get{property.SourceName}";
						builder.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg);
						if (property.IsStaticArray)
						{
							builder.Append($"*/[{property.ArrayDimensions}]");
						}
						string expectedType = builder.ToString();
						property.LogError($"Property '{property.SourceName}' expected a getter function '[const] {expectedType} [&] {expectedName}() const', but it was not found");
					}
					if (property.PropertyExportFlags.HasExactFlags(UhtPropertyExportFlags.SetterSpecified | UhtPropertyExportFlags.SetterSpecifiedNone | UhtPropertyExportFlags.SetterFound | UhtPropertyExportFlags.SetterSpecifiedAuto, UhtPropertyExportFlags.SetterSpecified))
					{
						using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
						StringBuilder builder = borrower.StringBuilder;
						string expectedName = property.Setter ?? $"Set{property.SourceName}";
						builder.AppendPropertyText(property, UhtPropertyTextType.GetterSetterArg);
						if (property.IsStaticArray)
						{
							builder.Append($"*/[{property.ArrayDimensions}]");
						}
						string expectedType = builder.ToString();
						property.LogError($"Property '{property.SourceName}' expected a setter function 'void {expectedName}([const] {expectedType} [&] InArg)', but it was not found");
					}
				}
			}
		}

		/// <summary>
		/// Interface with blueprint data should declare explicitly Blueprintable or NotBlueprintable to be clear
		/// In the backward compatible case where they declare neither, both of these bools are false
		/// </summary>
		/// <returns>True if the interface is marked as Blueprintable</returns>
		private bool CanImplementInBlueprints()
		{
			return MetaData.GetBoolean(UhtNames.IsBlueprintBase);
		}

		/// <summary>
		/// Interface with blueprint data should declare explicitly Blueprintable or NotBlueprintable to be clear
		/// In the backward compatible case where they declare neither, both of these bools are false
		/// </summary>
		/// <returns>True if the interface is marked as NotBlueprintable</returns>
		private bool CannotImplementInBlueprints()
		{
			return CannotImplementInBlueprints(CanImplementInBlueprints());
		}

		/// <summary>
		/// Interface with blueprint data should declare explicitly Blueprintable or NotBlueprintable to be clear
		/// In the backward compatible case where they declare neither, both of these bools are false
		/// </summary>
		/// <param name="canImplementInBlueprints">If true, class has already been marked as Blueprintable</param>
		/// <returns>True if the interface is marked as NotBlueprintable</returns>
		private bool CannotImplementInBlueprints(bool canImplementInBlueprints)
		{
			return (!canImplementInBlueprints && MetaData.ContainsKey(UhtNames.IsBlueprintBase))
				|| MetaData.ContainsKey(UhtNames.CannotImplementInterfaceInBlueprint);
		}
		#endregion

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{

			// Ignore any intrinsics
			if (ClassFlags.HasAnyFlags(EClassFlags.Intrinsic))
			{
				return;
			}

			// In the original UHT, we don't export IInterface (it wasn't even in the header file)
			if (HeaderFile.IsNoExportTypes && SourceName == "IInterface")
			{
				return;
			}

			base.CollectReferences(collector);

			// Add myself as declarations and cross module references
			collector.AddDeclaration(this, UhtSingletonType.Unregistered);
			collector.AddDeclaration(this, UhtSingletonType.Registered);
			collector.AddObjectReference(this, UhtSingletonType.Unregistered);
			collector.AddObjectReference(this, UhtSingletonType.Registered);

			// Add the super class
			if (SuperClass != null)
			{
				collector.AddDeclaration(SuperClass, UhtSingletonType.Registered);
				collector.AddObjectReference(SuperClass, UhtSingletonType.Registered);
			}

			// Add any other base classes
			foreach (UhtStruct baseObj in Bases)
			{
				if (baseObj is UhtClass baseClass)
				{
					collector.AddObjectReference(baseClass, UhtSingletonType.Unregistered);
				}
			}

			// Add any verse interfaces
			foreach (UhtStruct interfaceObj in FlattenedVerseInterfaces)
			{
				collector.AddObjectReference(interfaceObj, UhtSingletonType.Unregistered);
			}

			// Add the package
			collector.AddDeclaration(Package, UhtSingletonType.Registered);
			collector.AddObjectReference(Package, UhtSingletonType.Registered);

			// Pointers that are needed for constinit only
			collector.AddObjectReference(Session.UClass, UhtSingletonType.ConstInit);
			collector.AddObjectReference(ClassWithin, UhtSingletonType.ConstInit);
			// Add all parents for constinit initializion of bases array 
			for (UhtClass? super = SuperClass; super is not null; super = super.SuperClass)
			{
				collector.AddObjectReference(super, UhtSingletonType.ConstInit);
			}

			// Collect children references
			foreach (UhtType child in Children)
			{
				child.CollectReferences(collector);
			}

			// Done at the end so any contained delegate functions are added first
			// We also add interfaces in the export type order where the native interface
			// is located.  But we add the interface...
			switch (ClassType)
			{
				case UhtClassType.NativeInterface:
					if (AlternateObject != null && AlternateObject is UhtField field)
					{
						collector.AddExportType(field);
					}
					break;
				case UhtClassType.Interface:
					break;
				case UhtClassType.Class:
				case UhtClassType.VModule:
					collector.AddExportType(this);
					break;
			}
		}

		private bool ImplementsInterface(UhtClass interfaceObj)
		{
			for (UhtClass? superClass = this; superClass != null; superClass = superClass.SuperClass)
			{
				foreach (UhtStruct structObj in superClass.Bases)
				{
					if (structObj.IsChildOf(interfaceObj))
					{
						return true;
					}
				}
			}
			return false;
		}

		private void BindVerseInterfaces()
		{
			if (VerseInterfaceIdentifiers != null)
			{
				foreach (UhtToken[] identifier in VerseInterfaceIdentifiers)
				{
					// We really only case about interfaces, but we can also handle structs
					UhtClass? verseInterface = (UhtClass?)FindType(UhtFindOptions.Class | UhtFindOptions.SourceName | UhtFindOptions.NoSelf, identifier);
					if (verseInterface != null)
					{
						_verseInterfaces ??= [];
						_verseInterfaces.Add(verseInterface);
					}
					else
					{
						string fullIdentifier = String.Join("::", identifier.Select(x => x.Value));
						this.LogError($"Verse interface {fullIdentifier} not found");
					}
				}
			}
		}
	}
}
