// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// How the enumeration was declared
	/// Must match UEnum::ECppForm
	/// </summary>
	public enum UhtEnumCppForm
	{
		/// <summary>
		/// enum Name {...}
		/// </summary>
		Regular,

		/// <summary>
		/// namespace Name { enum Type { ... } }
		/// </summary>
		Namespaced,

		/// <summary>
		/// enum class Name {...}
		/// </summary>
		EnumClass
	}

	/// <summary>
	/// Underlying type of the enumeration
	/// Match with UEnum::EUnderlyingType
	/// </summary>
	public enum UhtEnumUnderlyingType
	{

		/// <summary>
		/// Not specified
		/// </summary>
		Unspecified,

		/// <summary>
		/// Uint8
		/// </summary>
		Uint8,

		/// <summary>
		/// Uint16
		/// </summary>
		Uint16,

		/// <summary>
		/// Uint32
		/// </summary>
		Uint32,

		/// <summary>
		/// Uint64
		/// </summary>
		Uint64,

		/// <summary>
		/// Int8
		/// </summary>
		Int8,

		/// <summary>
		/// Int16
		/// </summary>
		Int16,

		/// <summary>
		/// Int32
		/// </summary>
		Int32,

		/// <summary>
		/// Int64
		/// </summary>
		Int64,

		/// <summary>
		/// Int
		/// </summary>
		Int,
	}

	/// <summary>
	/// Represents an enumeration value
	/// </summary>
	public struct UhtEnumValue
	{
		/// <summary>
		/// Name of the enumeration value
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Value of the enumeration or -1 if not parsed.
		/// </summary>
		public long Value { get; set; }

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtEnumValue(UhtInputCacheReader reader)
		{
			Name = reader.ReadString();
			Value = reader.ReadInt64();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteString(Name);
			writer.WriteInt64(Value);
		}
	}

	/// <summary>
	/// Series of flags not part of the engine's enum flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtEnumExportFlags : int
	{

		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// Indicates that the StaticEnum function should be exported to other modules
		/// </summary>
		MinimalAPI = 1 << 0,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtEnumExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtEnumExportFlags inFlags, UhtEnumExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtEnumExportFlags inFlags, UhtEnumExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtEnumExportFlags inFlags, UhtEnumExportFlags testFlags, UhtEnumExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Represents a UEnum
	/// </summary>
	[UhtEngineClass(Name = "Enum")]
	public class UhtEnum : UhtField, IUhtMetaDataKeyConversion
	{
		/// <summary>
		/// Engine enumeration flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EEnumFlags EnumFlags { get; set; } = EEnumFlags.None;

		/// <summary>
		/// UHT only script struct flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtEnumExportFlags EnumExportFlags { get; set; } = UhtEnumExportFlags.None;

		/// <summary>
		/// C++ form of the enumeration
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtEnumCppForm CppForm { get; set; } = UhtEnumCppForm.Regular;

		/// <summary>
		/// Underlying integer enumeration type
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtEnumUnderlyingType UnderlyingType { get; set; } = UhtEnumUnderlyingType.Uint8;

		/// <summary>
		/// Full enumeration type.  For namespace enumerations, this includes the namespace name and the enum type name
		/// </summary>
		public string CppType { get; set; } = String.Empty;

		/// <summary>
		/// Fully qualified cpp type name. Include enclosing namespace as well as CppType (which may include enum's own namespace) 
		/// </summary>
		public string FullyQualifiedCppType => _fullyQualifiedCppType ??= $"{Namespace.FullSourceName}{CppType}";
		private string? _fullyQualifiedCppType = null;

		/// <summary>
		/// Collection of enumeration values
		/// </summary>
		public List<UhtEnumValue> EnumValues { get; }

		/// <summary>
		/// If header didn't declare a max value, this is a generated name to be appended to the list of names in reflection data.
		/// This is a full name as returned by GetFullEnumName i.e. it may be namespaced.
		/// </summary>
		public string? GeneratedMaxName { get; private set; }

		/// <summary>
		/// If header didn't declare a max value, and UHT was able to parse all values, this the value for GeneratedMaxName
		/// </summary>
		public long? GeneratedMaxValue { get; private set; }

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType => UhtEngineType.Enum;

		/// <inheritdoc/>
		public override string EngineClassName => IsVerseField ? "VerseEnum" : "Enum";

		/// <inheritdoc/>
		public override string EngineLinkClassName => "Enum";

		/// <inheritdoc/>
		public override UhtClass EngineClass => IsVerseField ? Session.UVerseEnum : Session.UEnum;

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable => Session.GetSpecifierValidatorTable(UhtTableNames.Enum);

		/// <summary>
		/// Construct a new enumeration
		/// </summary>
		/// <param name="headerFile">Header being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">Outer type</param>
		/// <param name="lineNumber">Line number of declaration</param>
		public UhtEnum(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
			MetaData.KeyConversion = this;
			EnumValues = [];
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtEnum(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			MetaData.KeyConversion = this;
			EnumFlags = (EEnumFlags)reader.ReadUInt64();
			EnumExportFlags = (UhtEnumExportFlags)reader.ReadUInt64();
			CppForm = (UhtEnumCppForm)reader.ReadUInt8();
			UnderlyingType = (UhtEnumUnderlyingType)reader.ReadUInt8();
			CppType = reader.ReadString()!;
			EnumValues = reader.ReadList((reader) => new UhtEnumValue(reader));
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteUInt64((ulong)EnumFlags);
			writer.WriteUInt64((ulong)EnumExportFlags);
			writer.WriteUInt8((byte)CppForm);
			writer.WriteUInt8((byte)UnderlyingType);
			writer.WriteString(CppType);
			writer.WriteVariableLengthArray(EnumValues, (writer, x) => x.Write(writer));
		}

		/// <inheritdoc/>
		public override StringBuilder AppendForwardDeclaration(StringBuilder builder)
		{
			switch (CppForm)
			{
				case UhtEnumCppForm.EnumClass:
					{
						UhtEnumUnderlyingType underlyingType = UnderlyingType != UhtEnumUnderlyingType.Unspecified ? UnderlyingType : UhtEnumUnderlyingType.Int32;
#pragma warning disable CA1308 // Normalize strings to uppercase
						builder.Append($"enum class {SourceName} : {underlyingType.ToString().ToLowerInvariant()};");
#pragma warning restore CA1308 // Normalize strings to uppercase
					}
					break;
					// no support for other types ATM
			}
			return builder;
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			if (phase == UhtResolvePhase.Bases && (Session.TargetSettings?.GenerateEnumMaxValues ?? false))
			{
				// TODO: Some engine types have authored max values that don't match the established pattern. 
				// We should add metadata to explicitly tag the max value, or module settings to relax the pattern.
				string prefix = GenerateEnumPrefix();
				string maxName = CleanEnumValueName(prefix + "_MAX");
				bool hasMax = GetIndexByName(GetFullEnumName("MAX")) != -1
					|| GetIndexByName(maxName) != -1;
				if (!hasMax)
				{
					GeneratedMaxName = maxName;
					bool hasUnparsedValue = false;
					long maxValue = 0;
					if (EnumFlags.HasAllFlags(EEnumFlags.Flags))
					{
						foreach (UhtEnumValue enumValue in EnumValues)
						{
							hasUnparsedValue = hasUnparsedValue || enumValue.Value == -1;
							if (System.Numerics.BitOperations.IsPow2(enumValue.Value))
							{
								maxValue |= enumValue.Value;
							}
						}
					}
					else
					{
						maxValue = EnumValues.Count == 0 ? 0 : EnumValues[0].Value;
						hasUnparsedValue = maxValue == -1;
						for (int i = 1; i < EnumValues.Count; ++i)
						{
							maxValue = Math.Max(maxValue, EnumValues[i].Value);
							hasUnparsedValue = hasUnparsedValue || EnumValues[i].Value == -1;
						}
					}
					if (!hasUnparsedValue)
					{
						GeneratedMaxValue = maxValue + 1;
					}
				}
			}
			return base.ResolveSelf(phase);
		}

		/// <summary>
		/// Test to see if the value is a known enum value
		/// </summary>
		/// <param name="value">Value in question</param>
		/// <returns>True if the value is known</returns>
		public bool IsValidEnumValue(long value)
		{
			foreach (UhtEnumValue enumValue in EnumValues)
			{
				if (enumValue.Value == value)
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Return the index of the given enumeration value name
		/// </summary>
		/// <param name="name">Value name in question</param>
		/// <returns>Index of the value or -1 if not found.</returns>
		public int GetIndexByName(string name)
		{
			name = CleanEnumValueName(name);
			for (int index = 0; index < EnumValues.Count; ++index)
			{
				if (EnumValues[index].Name == name)
				{
					return index;
				}
			}
			return -1;
		}

		/// <summary>
		/// Converts meta data name and index to a full meta data key name
		/// </summary>
		/// <param name="name">Meta data key name</param>
		/// <param name="nameIndex">Meta data key index</param>
		/// <returns>Meta data name with the enum value name</returns>
		public string GetMetaDataKey(string name, int nameIndex)
		{
			string enumName = EnumValues[nameIndex].Name;
			if (CppForm != UhtEnumCppForm.Regular)
			{
				int scopeIndex = enumName.IndexOf("::", StringComparison.Ordinal);
				if (scopeIndex >= 0)
				{
					enumName = enumName[(scopeIndex + 2)..];
				}
			}
			return $"{enumName}.{name}";
		}

		/// <summary>
		/// Given an enumeration value name, return the full enumeration name
		/// </summary>
		/// <param name="shortEnumName">Enum value name</param>
		/// <returns>If required, enum type name combined with value name.  Otherwise just the value name.</returns>
		/// <exception cref="UhtIceException">Unexpected enum form</exception>
		public string GetFullEnumName(string shortEnumName)
		{
			switch (CppForm)
			{
				case UhtEnumCppForm.Namespaced:
				case UhtEnumCppForm.EnumClass:
					return $"{SourceName}::{shortEnumName}";

				case UhtEnumCppForm.Regular:
					return shortEnumName;

				default:
					throw new UhtIceException("Unexpected UhtEnumCppForm value");
			}
		}

		/// <summary>
		/// Return the enum name without any namespacing
		/// </summary>
		/// <param name="longEnumName"></param>
		/// <returns></returns>
		public string GetShortEnumName(string longEnumName)
		{
			switch (CppForm)
			{
				case UhtEnumCppForm.Namespaced:
				case UhtEnumCppForm.EnumClass:
					int lastColons = longEnumName.LastIndexOf("::", StringComparison.Ordinal);
					return lastColons == -1 ? longEnumName : longEnumName[(lastColons + 2)..];

				case UhtEnumCppForm.Regular:
					return longEnumName;

				default:
					throw new UhtIceException("Unexpected UhtEnumCppForm value");
			}
		}

		/// <summary>
		/// Return a prefix common to all members of this enumeration, or the enum name, without a trailing underscore.
		/// This should return the same strings as UEnum::GenerateEnumPrefix.
		/// Note that UEnum::GenerateEnumPrefix generates some surprising results, such as giving a namespaced enum such as 
		/// ESomeEnum_WithUnderScore whose short names share no prefix the prefix ESomeEnum_, giving the generated max name
		/// of ESomeEnum_WithUnderScore::ESomeEnum_MAX
		/// </summary>
		/// <returns>Depending on the type of enum and where underscores appear, this may return a short or long enum name</returns>
		private string GenerateEnumPrefix()
		{
			if (EnumValues.Count == 0)
			{
				return SourceName;
			}
			string prefix = EnumValues[0].Name;

			// For each item in the enumeration, trim the prefix as much as necessary to keep it a prefix.
			// This ensures that once all items have been processed, a common prefix will have been constructed.
			// This will be the longest common prefix since as little as possible is trimmed at each step.
			for (int nameIdx = 1; nameIdx < EnumValues.Count; nameIdx++)
			{
				if (!prefix.Contains('_', StringComparison.InvariantCulture))
				{
					// See below - if we can't generate a prefix containing an underscore we'll discard it at the end anyway
					return SourceName;
				}
				string enumItemName = EnumValues[nameIdx].Name;

				// Find the length of the longest common prefix of prefix and enumItemName.
				int prefixIdx = 0;
				while (prefixIdx < prefix.Length && prefixIdx < enumItemName.Length && prefix[prefixIdx] == enumItemName[prefixIdx])
				{
					prefixIdx++;
				}

				// Trim the prefix to the length of the common prefix.
				prefix = prefix.Substring(0, prefixIdx);
			}

			// Find the index of the rightmost underscore in the prefix.
			int underscoreIdx = prefix.LastIndexOf('_');

			// If an underscore was found, trim the prefix so only the part before the rightmost underscore is included.
			if (underscoreIdx > 0)
			{
				prefix = prefix[..underscoreIdx];
			}
			else
			{
				// no underscores in the common prefix - this probably indicates that the names
				// for this enum are not using Epic's notation, so just empty the prefix so that
				// the max item will use the full name of the enum
				prefix = "";
			}

			// If no common prefix was found, or if the enum does not contain any entries,
			// use the name of the enumeration instead.
			if (prefix.Length == 0)
			{
				prefix = SourceName;
			}
			return prefix;
		}

		/// <summary>
		/// Add a new enum value.
		/// </summary>
		/// <param name="shortEnumName">Name of the enum value.</param>
		/// <param name="value">Enumeration value or -1 if the value can't be determined.</param>
		/// <returns></returns>
		public int AddEnumValue(string shortEnumName, long value)
		{
			int enumIndex = EnumValues.Count;
			EnumValues.Add(new UhtEnumValue { Name = GetFullEnumName(shortEnumName), Value = value });
			return enumIndex;
		}

		/// <summary>
		/// Reconstruct the full enum name.  Any existing enumeration name will be stripped and replaced 
		/// with this enumeration name.
		/// </summary>
		/// <param name="name">Name to reconstruct.</param>
		/// <returns>Reconstructed enum name</returns>
		private string CleanEnumValueName(string name)
		{
			int lastColons = name.LastIndexOf("::", StringComparison.Ordinal);
			return lastColons == -1 ? GetFullEnumName(name) : GetFullEnumName(name[(lastColons + 2)..]);
		}

		#region Validation support
		///<inheritdoc/>
		protected override void ValidateDocumentationPolicy(UhtDocumentationPolicy policy)
		{
			if (policy.ClassOrStructCommentRequired)
			{
				if (!MetaData.ContainsKey(UhtNames.ToolTip))
				{
					this.LogError(MetaData.LineNumber, $"Enum '{SourceName}' does not provide a tooltip / comment (DocumentationPolicy)");
				}
			}

			Dictionary<string, string> toolTips = new(StringComparer.OrdinalIgnoreCase);
			for (int enumIndex = 0; enumIndex < EnumValues.Count; ++enumIndex)
			{
				if (!MetaData.TryGetValue(UhtNames.Name, enumIndex, out string? entryName))
				{
					continue;
				}

				if (!MetaData.TryGetValue(UhtNames.ToolTip, enumIndex, out string? toolTip))
				{
					this.LogError(MetaData.LineNumber, $"Enum entry '{SourceName}::{entryName}' does not provide a tooltip / comment (DocumentationPolicy)");
					continue;
				}

				if (toolTips.TryGetValue(toolTip, out string? dupName))
				{
					this.LogError(MetaData.LineNumber, $"Enum entries '{SourceName}::{entryName}' and '{SourceName}::{dupName}' have identical tooltips / comments (DocumentationPolicy)");
				}
				else
				{
					toolTips.Add(toolTip, entryName);
				}
			}
		}
		#endregion

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			collector.AddExportType(this);
			collector.AddDeclaration(this, UhtSingletonType.Registered);
			collector.AddObjectReference(this, UhtSingletonType.Registered);
			collector.AddObjectReference(Package, UhtSingletonType.Registered);

			collector.AddObjectReference(Session.UEnum, UhtSingletonType.ConstInit);
		}
	}
}
