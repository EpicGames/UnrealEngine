// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Series of flags not part of the engine's field flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtFieldExportFlags : int
	{

		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// Do not generate an alias in the verse namespace
		/// </summary>
		NoVerseAlias = 1 << 0,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtFieldExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtFieldExportFlags inFlags, UhtFieldExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtFieldExportFlags inFlags, UhtFieldExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtFieldExportFlags inFlags, UhtFieldExportFlags testFlags, UhtFieldExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Controls how the verse name is appended to the builder
	/// </summary>
	public enum UhtVerseNameMode
	{
		/// <summary>
		/// Default mode
		/// </summary>
		Default,

		/// <summary>
		/// Fully qualified mode
		/// </summary>
		Qualified,

		/// <summary>
		/// Used to generate the name but not including the package.  Unlike the default. '/' is used instead of '_' 
		/// </summary>
		PackageRelative,
	}

	/// <summary>
	/// Represents a UField
	/// </summary>
	public abstract class UhtField : UhtObject
	{
		/// <inheritdoc/>
		public override string EngineClassName => "Field";

		/// <summary>
		/// UHT only field flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtFieldExportFlags FieldExportFlags { get; set; } = UhtFieldExportFlags.None;

		/// <summary>
		/// Name of the module containing the type
		/// </summary>
		public string? VerseModule { get; set; } = null;

		/// <summary>
		/// Cased name of the verse field
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1721:Property names should not match get methods", Justification = "<Pending>")]
		public string? VerseName { get; set; } = null;

		/// <summary>
		/// Returns true if field is a verse element
		/// </summary>
		public bool IsVerseField => VerseName != null;

		/// <summary>
		/// Namespace where the field was defined
		/// </summary>
		public UhtNamespace Namespace { get; set; }

		/// <summary>
		/// Fully qualified name of the field in source, i.e. Namespace and SourceName concatenated.
		/// </summary>
		public string FullyQualifiedSourceName => _fullyQualifiedSourceName ??= $"{Namespace.FullSourceName}{SourceName}";
		private string? _fullyQualifiedSourceName = null;

		/// <summary>
		/// Construct a new field
		/// </summary>
		/// <param name="headerFile">Header file being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">Outer object</param>
		/// <param name="lineNumber">Line number of declaration</param>
		protected UhtField(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, outer, lineNumber)
		{
			Namespace = namespaceObj;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtField(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			Namespace = reader.ReadNamespace();
			FieldExportFlags = (UhtFieldExportFlags)reader.ReadUInt64();
			VerseModule = reader.ReadOptionalString();
			VerseName = reader.ReadOptionalString();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteNamespace(Namespace);
			writer.WriteUInt64((ulong)FieldExportFlags);
			writer.WriteOptionalString(VerseModule);
			writer.WriteOptionalString(VerseName);
		}

		/// <summary>
		/// Return the scope and name of the verse type (scope:)name
		/// </summary>
		/// <param name="mode">Controls how the name is generated</param>
		/// <returns></returns>
		public string GetVerseScopeAndName(UhtVerseNameMode mode)
		{
			StringBuilder builder = new();
			AppendVerseScopeAndName(builder, mode);
			return builder.ToString();
		}

		/// <summary>
		/// Return the encoded verse scope and name of the field
		/// </summary>
		/// <param name="mode">Controls how the name is generated</param>
		/// <returns></returns>
		public string GetEncodedVerseScopeAndName(UhtVerseNameMode mode)
		{
			StringBuilder builder = new();
			AppendVerseScopeAndName(builder, mode);
			string name = builder.ToString();
			builder.Clear().AppendEncodedVerseName(name);
			return builder.ToString();
		}

		/// <summary>
		/// Return the verse scope of the field
		/// </summary>
		/// <param name="mode">Controls how the name is generated</param>
		/// <returns></returns>
		public string GetVerseScope(UhtVerseNameMode mode)
		{
			StringBuilder builder = new();
			AppendVerseScope(builder, mode);
			return builder.ToString();
		}

		/// <summary>
		/// Return the verse name of the field
		/// </summary>
		/// <param name="mode">Controls how the name is generated</param>
		/// <returns></returns>
		public string GetVerseName(UhtVerseNameMode mode)
		{
			StringBuilder builder = new();
			AppendVerseName(builder, mode);
			return builder.ToString();
		}

		/// <summary>
		/// Append the scope and name of the verse type (scope:)name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="mode">Controls how the name is generated</param>
		public virtual void AppendVerseScopeAndName(StringBuilder builder, UhtVerseNameMode mode)
		{
			switch (mode)
			{
				case UhtVerseNameMode.Default:
				case UhtVerseNameMode.Qualified:
					builder.Append('(');
					AppendVerseScope(builder, mode);
					builder.Append(":)");
					break;

				case UhtVerseNameMode.PackageRelative:
					{
						int length = builder.Length;
						AppendVerseScope(builder, mode);
						if (builder.Length != length)
						{
							builder.Append('/');
						}
					}
					break;
			}
			AppendVerseName(builder, mode);
		}

		/// <summary>
		/// Append the verse scope part to the given builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="mode">Controls how the name is generated</param>
		public void AppendVerseScope(StringBuilder builder, UhtVerseNameMode mode)
		{
			AppendVerseScopeInternal(builder, mode, true);
		}

		/// <summary>
		/// Append the verse scope to the given builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="mode">Controls how the name is generated</param>
		/// <param name="isTopLevel">If true, this is the type that initiated the verse scope</param>
		/// <exception cref="UhtException"></exception>
		private void AppendVerseScopeInternal(StringBuilder builder, UhtVerseNameMode mode, bool isTopLevel)
		{
			if (!IsVerseField)
			{
				throw new UhtException(this, "Attempt to write the Verse name on a field that isn't part of Verse");
			}
			if (Outer is UhtField outerField)
			{
				outerField.AppendVerseScopeInternal(builder, mode, false);
			}
			else
			{
				if (mode != UhtVerseNameMode.PackageRelative)
				{
					builder.Append(Module.Module.VersePath);
				}
				if (!String.IsNullOrEmpty(VerseModule))
				{
					if (mode != UhtVerseNameMode.PackageRelative)
					{
						builder.Append('/');
					}
					builder.Append(VerseModule);
				}
			}
			AppendVerseMyScope(builder, mode, isTopLevel);
		}

		/// <summary>
		/// Helper method that appends only this path contribution to the verse path.
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="mode">Controls how the name is generated</param>
		/// <param name="isTopLevel">If true, this is the type that initiated the verse path</param>
		protected virtual void AppendVerseMyScope(StringBuilder builder, UhtVerseNameMode mode, bool isTopLevel)
		{
			if (!isTopLevel)
			{
				builder.Append('/').Append(VerseName);
			}
		}

		/// <summary>
		/// Helper method that appends only this instance contribution to the verse path.
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="mode">Controls how the name is generated</param>
		protected virtual void AppendVerseName(StringBuilder builder, UhtVerseNameMode mode)
		{
			builder.Append(VerseName);
		}

		/// <summary>
		/// Append the field's forward declaration
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <returns>Builder</returns>
		public virtual StringBuilder AppendForwardDeclaration(StringBuilder builder)
		{
			throw new NotImplementedException();
		}
	}

	/// <summary>
	/// Helper extension methods for fields
	/// </summary>
	public static class UhtFieldStringBuilderExtensions
	{

		/// <summary>
		/// Append the Verse UE VNI package name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="fieldObj">Field to serialize</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendVerseUEVNIPackageName(this StringBuilder builder, UhtField fieldObj)
		{
			if (!fieldObj.IsVerseField)
			{
				throw new UhtException(fieldObj, "Attempt to write the Verse VNI package name on a field that isn't part of Verse");
			}
			UhtModule module = fieldObj.Module;
			return builder.Append('/').Append(module.Module.VerseMountPoint).Append("/_Verse/VNI/").Append(module.Module.Name);
		}

		/// <summary>
		/// Append the Verse UE package name (without a leading forward slash)
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="fieldObj">Field to serialize</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendVerseUEPackageName(this StringBuilder builder, UhtField fieldObj)
		{
			if (!fieldObj.IsVerseField)
			{
				throw new UhtException(fieldObj, "Attempt to write the Verse package name on a field that isn't part of Verse");
			}
			UhtModule module = fieldObj.Module;
			return builder.Append(module.Module.VerseMountPoint).Append('/').Append(module.Module.Name);
		}

		/// <summary>
		/// Append the scope and name of the verse type (scope:)name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="fieldObj">Field to serialize</param>
		/// <param name="mode">Controls the type of string generated</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendVerseScopeAndName(this StringBuilder builder, UhtField fieldObj, UhtVerseNameMode mode)
		{
			fieldObj.AppendVerseScopeAndName(builder, mode);
			return builder;
		}

		/// <summary>
		/// Append the fields forward declaration
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="fieldObj">Field in question</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendForwardDeclaration(this StringBuilder builder, UhtField fieldObj)
		{
			return fieldObj.AppendForwardDeclaration(builder);
		}
	}
}
