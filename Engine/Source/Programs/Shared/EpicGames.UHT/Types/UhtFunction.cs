// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Series of flags not part of the engine's function flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtFunctionExportFlags : int
	{

		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// Function declaration included "final" keyword.  Used to differentiate between functions that have FUNC_Final only because they're private
		/// </summary>
		Final = 1 << 0,

		/// <summary>
		/// Function should be exported as a public API function
		/// </summary>
		RequiredAPI = 1 << 1,

		/// <summary>
		/// Export as an inline static C++ function
		/// </summary>
		Inline = 1 << 2,

		/// <summary>
		/// Export as a real C++ static function, causing thunks to call via ClassName::FuncName instead of this->FuncName
		/// </summary>
		CppStatic = 1 << 3,

		/// <summary>
		/// Export no thunk function; the user will manually define a custom one
		/// </summary>
		CustomThunk = 1 << 4,

		/// <summary>
		/// Function is marked as virtual
		/// </summary>
		Virtual = 1 << 5,

		/// <summary>
		/// The unreliable specified was present
		/// </summary>
		Unreliable = 1 << 6,

		/// <summary>
		/// The function is a sealed event
		/// </summary>
		SealedEvent = 1 << 7,

		/// <summary>
		/// Blueprint pure is being forced to false
		/// </summary>
		ForceBlueprintImpure = 1 << 8,

		/// <summary>
		/// If set, the BlueprintPure was automatically set
		/// </summary>
		AutoBlueprintPure = 1 << 9,

		/// <summary>
		/// If set, a method matching the CppImplName was found.  The search is only performed if it
		/// differs from the function name.
		/// </summary>
		ImplFound = 1 << 10,

		/// <summary>
		/// If the ImplFound flag is set, then if this flag is set, the method is virtual
		/// </summary>
		ImplVirtual = 1 << 11,

		/// <summary>
		/// If set, a method matching the CppValidationImplName was found.  The search is only performed if 
		/// CppImplName differs from the function name
		/// </summary>
		ValidationImplFound = 1 << 12,

		/// <summary>
		/// If the ValidationImplFound flag is set, then if this flag is set, the method id virtual.
		/// </summary>
		ValidationImplVirtual = 1 << 13,

		/// <summary>
		/// Set true if the function itself is declared const.  The is required for the automatic setting of BlueprintPure.
		/// </summary>
		DeclaredConst = 1 << 14,

		/// <summary>
		/// Final flag was set automatically and should not be considered for validation
		/// </summary>
		AutoFinal = 1 << 15,

		/// <summary>
		/// Generate the entry for the FieldNotificationClassDescriptor
		/// </summary>
		FieldNotify = 1 << 16,

		/// <summary>
		/// True if the function specifier has a getter/setter specified
		/// </summary>
		SawPropertyAccessor = 1 << 17,

		/// <summary>
		/// True if the function is verse function callable from native code
		/// </summary>
		VerseNativeCallable = 1 << 18,

		/// <summary>
		/// True if the function is a decides function
		/// </summary>
		VerseDecides = 1 << 19,

		/// <summary>
		/// True if the return value is wrapped with TVerseTask
		/// </summary>
		VerseSuspends = 1 << 20,

		/// <summary>
		/// Function can be called from verse
		/// </summary>
		VerseCallable = 1 << 21,

		/// <summary>
		/// True if the function is a no_rollback function
		/// </summary>
		VerseNoRollback = 1 << 22,

		/// <summary>
		/// True if the function is marked as rtfm_always_open
		/// </summary>
		VerseRTFMAlwaysOpen = 1 << 23,

		/// <summary>
		/// True if the function is an extension method
		/// </summary>
		VerseExtensionMethod = 1 << 24,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtFunctionExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtFunctionExportFlags inFlags, UhtFunctionExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtFunctionExportFlags inFlags, UhtFunctionExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtFunctionExportFlags inFlags, UhtFunctionExportFlags testFlags, UhtFunctionExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Type of function
	/// </summary>
	public enum UhtFunctionType
	{

		/// <summary>
		/// UFUNCTION
		/// </summary>
		Function,

		/// <summary>
		/// UDELEGATE/DECLARE_DYNAMIC_...
		/// </summary>
		Delegate,

		/// <summary>
		/// UDELEGATE/DECLARE_DYNAMIC_...SPARSE_...
		/// </summary>
		SparseDelegate,
	}

	/// <summary>
	/// Extension 
	/// </summary>
	public static class UhtFunctionTypeExtensions
	{

		/// <summary>
		/// Test to see if the function type is a delegate type
		/// </summary>
		/// <param name="functionType">Function type being tested</param>
		/// <returns>True if the type is a delegate</returns>
		public static bool IsDelegate(this UhtFunctionType functionType)
		{
			return functionType == UhtFunctionType.Delegate || functionType == UhtFunctionType.SparseDelegate;
		}
	}

	/// <summary>
	/// Represents a UFUNCTION/delegate
	/// </summary>
	public class UhtFunction : UhtStruct
	{
		private string? _strippedFunctionName = null;

		/// <summary>
		/// Suffix added to delegate engine names 
		/// </summary>
		public static readonly string GeneratedDelegateSignatureSuffix = "__DelegateSignature";

		/// <summary>
		/// Engine function flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EFunctionFlags FunctionFlags { get; set; } = EFunctionFlags.None;

		/// <summary>
		/// UHT specific function flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtFunctionExportFlags FunctionExportFlags { get; set; } = UhtFunctionExportFlags.None;

		/// <summary>
		/// The type of function
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtFunctionType FunctionType { get; set; } = UhtFunctionType.Function;

		/// <summary>
		/// Super struct type for inheritance, i.e. functions do not inherit properties from their Super
		/// </summary>
		[JsonIgnore]
		public override UhtStruct? InheritanceSuperStruct => null;

		/// <summary>
		/// The line number for the macro.
		/// </summary>
		public int MacroLineNumber { get; set; } = 1;

		/// <summary>
		/// Owning class name for sparse functions
		/// </summary>
		public string? SparseOwningClassName { get; set; } = null;

		/// <summary>
		/// Sparse delegate name
		/// </summary>
		public string? SparseDelegateName { get; set; } = null;

		/// <summary>
		/// The super function.  Currently unused
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtFunction>))]
		public UhtFunction? SuperFunction => (UhtFunction?)Super;

		/// <summary>
		/// After functions are fully resolved, points to the next function in the owning struct in each define scope.
		/// </summary>
		public UhtDefineScopeLink<UhtFunction>? NextFunction { get; set; }

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType
		{
			get
			{
				switch (FunctionType)
				{
					case UhtFunctionType.Function:
						return UhtEngineType.Function;
					case UhtFunctionType.Delegate:
						return UhtEngineType.Delegate;
					case UhtFunctionType.SparseDelegate:
						return UhtEngineType.SparseDelegate;
					default:
						throw new UhtIceException("Invalid function type");
				}
			}
		}

		/// <inheritdoc/>
		public override string EngineClassName
		{
			get
			{
				switch (FunctionType)
				{
					case UhtFunctionType.Function:
						return IsVerseField ? "VerseFunction" : "Function";
					case UhtFunctionType.Delegate:
						return "DelegateFunction";
					case UhtFunctionType.SparseDelegate:
						return "SparseDelegateFunction";
					default:
						throw new UhtIceException("Invalid function type");
				}
			}
		}

		/// <inheritdoc/>
		public override string EngineLinkClassName => "Function";

		/// <inheritdoc/>
		public override UhtClass? EngineClass
		{
			get
			{
				switch (FunctionType)
				{
					case UhtFunctionType.Function:
						return IsVerseField ? Session.UVerseFunction : Session.UFunction;
					case UhtFunctionType.Delegate:
						return Session.UDelegateFunction;
					case UhtFunctionType.SparseDelegate:
						return Session.USparseDelegateFunction;
					default:
						throw new UhtIceException("Invalid function type");
				}
			}
		}

		/// <summary>
		/// Stripped function name without the generated delegate suffix
		/// </summary>
		[JsonIgnore]
		public string StrippedFunctionName
		{
			get => _strippedFunctionName ?? EngineName;
			set => _strippedFunctionName = value;
		}

		///<inheritdoc/>
		[JsonIgnore]
		public override bool Deprecated => MetaData.ContainsKey(UhtNames.DeprecatedFunction);

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable => Session.GetSpecifierValidatorTable(UhtTableNames.Function);

		/// <summary>
		/// Identifier for an RPC call to a platform service
		/// </summary>
		public ushort RPCId { get; set; } = 0;

		/// <summary>
		/// Identifier for an RPC call expecting a response
		/// </summary>
		public ushort RPCResponseId { get; set; } = 0;

		/// <summary>
		/// Name of the actual implementation
		/// </summary>
		public string CppImplName { get; set; } = String.Empty;

		/// <summary>
		/// Name of the actual validation implementation
		/// </summary>
		public string CppValidationImplName { get; set; } = String.Empty;

		/// <summary>
		/// Name of the wrapper function that marshals the arguments and does the indirect call
		/// </summary>
		public string MarshalAndCallName { get; set; } = String.Empty;

		/// <summary>
		/// Name for callback-style names
		/// </summary>
		public string UnMarshalAndCallName { get; set; } = String.Empty;

		/// <summary>
		/// Endpoint name
		/// </summary>
		public string EndpointName { get; set; } = String.Empty;

		/// <summary>
		/// True if the function has a return value.
		/// </summary>
		[JsonIgnore]
		public bool HasReturnProperty => Children.Count > 0 && ((UhtProperty)Children[^1]).PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm);

		/// <summary>
		/// The return value property or null if the function doesn't have a return value
		/// </summary>
		[JsonIgnore]
		public UhtProperty? ReturnProperty => HasReturnProperty ? (UhtProperty)Children[^1] : null;

		/// <summary>
		/// True if the function has parameters.
		/// </summary>
		[JsonIgnore]
		public bool HasParameters => Children.Count > 0 && !((UhtProperty)Children[0]).PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm);

		/// <summary>
		/// Return read only memory of all the function parameters
		/// </summary>
		[JsonIgnore]
		public ReadOnlyMemory<UhtType> ParameterProperties => new([.. Children], 0, HasReturnProperty ? Children.Count - 1 : Children.Count);

		/// <summary>
		/// True if the function has any outputs including a return value
		/// </summary>
		[JsonIgnore]
		public bool HasAnyOutputs
		{
			get
			{
				foreach (UhtType type in Children)
				{
					if (type is UhtProperty property)
					{
						if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm | EPropertyFlags.OutParm))
						{
							return true;
						}
					}
				}
				return false;
			}
		}

		/// <summary>
		/// Construct a new instance of a function
		/// </summary>
		/// <param name="headerFile">Header being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">The parent object</param>
		/// <param name="lineNumber">The line number where the function is defined</param>
		public UhtFunction(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtFunction(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			_strippedFunctionName = reader.ReadOptionalString();
			FunctionFlags = (EFunctionFlags)reader.ReadUInt64();
			FunctionExportFlags = (UhtFunctionExportFlags)reader.ReadUInt64();
			FunctionType = (UhtFunctionType)reader.ReadUInt64();
			MacroLineNumber = reader.ReadInt32();
			SparseOwningClassName = reader.ReadOptionalString();
			SparseDelegateName = reader.ReadOptionalString();
			RPCId = reader.ReadUInt16();
			RPCResponseId = reader.ReadUInt16();
			CppImplName = reader.ReadString();
			CppValidationImplName = reader.ReadString();
			MarshalAndCallName = reader.ReadString();
			UnMarshalAndCallName = reader.ReadString();
			EndpointName = reader.ReadString();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteOptionalString(_strippedFunctionName);
			writer.WriteUInt64((ulong)FunctionFlags);
			writer.WriteUInt64((ulong)FunctionExportFlags);
			writer.WriteUInt64((ulong)FunctionType);
			writer.WriteInt32(MacroLineNumber);
			writer.WriteOptionalString(SparseOwningClassName);
			writer.WriteOptionalString(SparseDelegateName);
			writer.WriteUInt16(RPCId);
			writer.WriteUInt16(RPCResponseId);
			writer.WriteString(CppImplName);
			writer.WriteString(CppValidationImplName);
			writer.WriteString(MarshalAndCallName);
			writer.WriteString(UnMarshalAndCallName);
			writer.WriteString(EndpointName);
		}

		/// <inheritdoc/>
		public override void AddChild(UhtType type)
		{
			if (type is UhtProperty property)
			{
				if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.Parm))
				{
					throw new UhtIceException("Only properties marked as parameters can be added to functions");
				}
				if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm) && HasReturnProperty)
				{
					throw new UhtIceException("Attempt to set multiple return properties on function");
				}
			}
			else
			{
				throw new UhtIceException("Invalid type added");
			}
			base.AddChild(type);
		}

		/// <inheritdoc/>
		public override void AppendVerseScopeAndName(StringBuilder builder, UhtVerseNameMode mode)
		{
			// suspends methods don't get the path part added to the name
			if (!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseSuspends))
			{
				base.AppendVerseScopeAndName(builder, mode);
			}
			else
			{
				AppendVerseName(builder, mode);
			}
		}

		/// <inheritdoc/>
		protected override void AppendVerseName(StringBuilder builder, UhtVerseNameMode mode)
		{
			UhtProperty? returnProperty = ReturnProperty;
			ReadOnlyMemory<UhtType> parameterProperties = ParameterProperties;
			if (Session.IsIncompleteReturn(returnProperty))
			{
				parameterProperties = parameterProperties[..^1];
			}

			builder.Append(VerseName);
			if (!parameterProperties.IsEmpty)
			{
				bool needsWhereClause = false;
				bool isFirst = true;
				builder.Append('(');
				foreach (UhtType type in parameterProperties.Span)
				{
					if (type is UhtProperty property)
					{
						if (!isFirst)
						{
							builder.Append(',');
						}
						builder.Append(':').AppendPropertyVerseMangledType(property);
						if (property.DefaultValueTokens != null || property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.VerseDefaultValue))
						{
							builder.Append(" = ...");
						}
						isFirst = false;
						if (!needsWhereClause)
						{
							needsWhereClause = type is UhtObjectPropertyBase objectProperty && objectProperty.CppForm.NeedsWhereClause();
						}
					}
				}

				if (needsWhereClause)
				{
					isFirst = true;
					builder.Append(" where");
					foreach (UhtType type in parameterProperties.Span)
					{
						if (type is UhtObjectPropertyBase objectProperty && objectProperty.CppForm.NeedsWhereClause())
						{
							if (!isFirst)
							{
								builder.Append(',');
							}
							builder.Append(' ').Append(objectProperty.SourceName);
							isFirst = false;
						}
					}
				}
				builder.Append(')');
			}
		}

		#region Resolution support
		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool results = base.ResolveSelf(phase);

			// If true, we need to echo the default values into the meta data
			bool storeCppDefaultValueInMetaData = 
				FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable | EFunctionFlags.Exec) || MetaData.ContainsKey("AICallable");

			switch (phase)
			{
				case UhtResolvePhase.Bases:
					if (FunctionType == UhtFunctionType.Function && Outer is UhtClass outerClass)
					{
						// non-static functions in a const class must be const themselves
						if (outerClass.ClassFlags.HasAnyFlags(EClassFlags.Const))
						{
							FunctionFlags |= EFunctionFlags.Const;
						}
					}
					break;

				case UhtResolvePhase.Properties:
					UhtPropertyParser.ResolveChildren(this, GetPropertyParseOptions(false));
					if (storeCppDefaultValueInMetaData)
					{
						foreach (UhtProperty property in Properties)
						{
							if (property.DefaultValueTokens != null)
							{
								string key = "CPP_Default_" + property.EngineName;
								if (!MetaData.ContainsKey(key))
								{
									bool parsed = false;
									try
									{
										// All tokens MUST be consumed from the reader
										StringBuilder builder = new();
										using UhtTokenReplayReaderBorrower borrowedReader = new(property, HeaderFile.Data.Memory, property.DefaultValueTokens.ToArray(), UhtTokenType.EndOfDefault);
										parsed = property.SanitizeDefaultValue(borrowedReader.Reader, builder) && borrowedReader.Reader.IsEOF;
										if (parsed)
										{
											MetaData.Add(key, builder.ToString());
										}
									}
									catch (Exception)
									{
										// Ignore the exception for now
									}

									if (!parsed)
									{
										StringView defaultValueText = new(HeaderFile.Data, property.DefaultValueTokens.First().InputStartPos,
											property.DefaultValueTokens.Last().InputEndPos - property.DefaultValueTokens.First().InputStartPos);
										property.LogError($"C++ Default parameter not parsed: {property.SourceName} '{defaultValueText}'");
									}
								}
							}
						}
					}
					break;

				case UhtResolvePhase.Final:
					if (Outer is UhtClass classObj)
					{
						if (FunctionFlags.HasAnyFlags(EFunctionFlags.Native) &&
							!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CustomThunk) &&
							CppImplName != SourceName)
						{
							if (classObj.TryGetDeclaration(CppImplName, out UhtFoundDeclaration declaration))
							{
								FunctionExportFlags |= UhtFunctionExportFlags.ImplFound;
								if (declaration.IsVirtual)
								{
									FunctionExportFlags |= UhtFunctionExportFlags.ImplVirtual;
								}
							}
							if (classObj.TryGetDeclaration(CppValidationImplName, out declaration))
							{
								FunctionExportFlags |= UhtFunctionExportFlags.ValidationImplFound;
								if (declaration.IsVirtual)
								{
									FunctionExportFlags |= UhtFunctionExportFlags.ValidationImplVirtual;
								}
							}
						}

						// If the function has already been marked as blueprint pure, don't bother.  This is important to being
						// able to detect interfaces where the value has been specified.
						if (FunctionType == UhtFunctionType.Function && FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.DeclaredConst) && !FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure))
						{
							// @todo: the presence of const and one or more outputs does not guarantee that there are
							// no side effects. On GCC and clang we could use __attribure__((pure)) or __attribute__((const))
							// or we could just rely on the use marking things BlueprintPure. Either way, checking the C++
							// const identifier to determine purity is not desirable. We should remove the following logic:

							// If its a const BlueprintCallable function with some sort of output and is not being marked as an BlueprintPure=false function, mark it as BlueprintPure as well
							if (HasAnyOutputs && FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable) &&
								!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ForceBlueprintImpure))
							{
								FunctionFlags |= EFunctionFlags.BlueprintPure;
								FunctionExportFlags |= UhtFunctionExportFlags.AutoBlueprintPure; // Disable error for pure being set
							}
						}
					}

					foreach (UhtType type in Children)
					{
						if (type is not UhtProperty property)
						{
							continue;
						}

						if (property.PropertyFlags.HasExactFlags(EPropertyFlags.OutParm | EPropertyFlags.ReturnParm, EPropertyFlags.OutParm))
						{
							FunctionFlags |= EFunctionFlags.HasOutParms;
						}
						// The following code is only performed on functions in a class.
						if (property is UhtStructProperty structProperty && Outer is UhtClass)
						{
							if (structProperty.ScriptStruct.HasDefaults)
							{
								FunctionFlags |= EFunctionFlags.HasDefaults;
							}
						}
					}

					break;
			}
			return results;
		}
		#endregion

		#region Validation support
		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);

			ValidateSpecifierFlags();

			if (DefineScope.HasAnyFlags(UhtDefineScope.Tests) && Outer is UhtClass && !Outer.DefineScope.HasAnyFlags(UhtDefineScope.Tests))
			{
				this.LogError($"Functions are not supported within a {UhtNames.WithTests} scope.");
			}

			// Some parameter test are optionally done
			bool checkForBlueprint = false;

			// Function type specific validation
			switch (FunctionType)
			{
				case UhtFunctionType.Function:

					// Get the outer class - either directly or from partial's Owner class
					UhtClass? outerClass = Outer as UhtClass;
					if (outerClass == null && Outer is UhtPartial partial)
					{
						outerClass = partial.OwnerClass;
					}

					if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure) && outerClass != null &&
						outerClass.ClassFlags.HasAnyFlags(EClassFlags.Interface) &&
						!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.AutoBlueprintPure))
					{
						this.LogError("BlueprintPure specifier is not allowed for interface functions");
					}

					// If this function is blueprint callable or blueprint pure, require a category 
					if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable | EFunctionFlags.BlueprintPure))
					{
						bool blueprintAccessor = MetaData.ContainsKey(UhtNames.BlueprintSetter) || MetaData.ContainsKey(UhtNames.BlueprintGetter);
						bool hasMenuCategory = MetaData.ContainsKey(UhtNames.Category);
						bool internalOnly = MetaData.GetBoolean(UhtNames.BlueprintInternalUseOnly);

						if (!hasMenuCategory && !internalOnly && !Deprecated && !blueprintAccessor)
						{
							// To allow for quick iteration, don't enforce the requirement that game functions have to be categorized
							if (Module.IsPartOfEngine)
							{
								this.LogError("An explicit Category specifier is required for Blueprint accessible functions in an Engine module.");
							}
						}
					}

					// Process the virtual-ness
					if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Virtual))
					{

						// if this is a BlueprintNativeEvent or BlueprintImplementableEvent in an interface, make sure it's not "virtual"
						if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
						{
							if (outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
							{
								this.LogError("BlueprintImplementableEvents in Interfaces must not be declared 'virtual'");
							}

							// if this is a BlueprintNativeEvent, make sure it's not "virtual"
							else if (FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
							{
								this.LogError("BlueprintNativeEvent functions must be non-virtual.");
							}

							else
							{
								this.LogWarning("BlueprintImplementableEvents should not be virtual. Use BlueprintNativeEvent instead.");
							}
						}
					}
					else
					{
						// if this is a function in an Interface, it must be marked 'virtual' unless it's an event
						if (outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.Interface) && !FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
						{
							this.LogError("Interface functions that are not BlueprintImplementableEvents must be declared 'virtual'");
						}
					}

					// This is a final (prebinding, non-overridable) function
					if (FunctionFlags.HasAnyFlags(EFunctionFlags.Final) || FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Final))
					{
						if (outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.Interface))
						{
							this.LogError("Interface functions cannot be declared 'final'");
						}
						else if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent) && !FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.AutoFinal))
						{
							this.LogError("Blueprint events cannot be declared 'final'");
						}
					}

					if (FunctionFlags.HasAnyFlags(EFunctionFlags.RequiredAPI) &&
						outerClass != null && outerClass.ClassFlags.HasAnyFlags(EClassFlags.RequiredAPI))
					{
						this.LogError($"API must not be used on methods of a class that is marked with an API itself.");
					}

					// Make sure that blueprint pure functions have some type of output
					if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure) && !HasAnyOutputs)
					{
						this.LogError("BlueprintPure specifier is not allowed for functions with no return value and no output parameters.");
					}

					if (FunctionFlags.HasExactFlags(EFunctionFlags.Net | EFunctionFlags.NetRequest | EFunctionFlags.NetResponse, EFunctionFlags.Net) && ReturnProperty != null)
					{
						this.LogError("Replicated functions can't have return values");
					}

					checkForBlueprint = FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintCallable | EFunctionFlags.BlueprintEvent);

					if (Outer is UhtStruct outerStruct)
					{
						UhtStruct? outerSuperStruct = outerStruct.SuperStruct;
						if (outerSuperStruct != null)
						{
							// We use the engine name as the find option for a caseless compare.
							UhtType? overriddenFunction = outerSuperStruct.FindType(UhtFindOptions.EngineName | UhtFindOptions.Function, SourceName);
							if (overriddenFunction != null)
							{
								// Native function overrides should be done in CPP text, not in a UFUNCTION() declaration (you can't change flags, and it'd otherwise be a burden to keep them identical)
								this.LogError(
									$"Override of UFUNCTION '{SourceName}' in parent '{overriddenFunction.Outer?.SourceName}' cannot have a UFUNCTION() declaration above it; it will use the same parameters as the original declaration.");
							}
						}
					}

					// Make sure that the replication flags set on an overridden function match the parent function
					UhtFunction? superFunction = SuperFunction;
					if (superFunction != null)
					{
						if ((FunctionFlags & EFunctionFlags.NetFuncFlags) != (superFunction.FunctionFlags & EFunctionFlags.NetFuncFlags))
						{
							this.LogError($"Overridden function '{SourceName}' cannot specify different replication flags when overriding a function.");
						}
					}

					// if this function is an RPC in state scope, verify that it is an override
					// this is required because the networking code only checks the class for RPCs when initializing network data, not any states within it
					if (FunctionFlags.HasAnyFlags(EFunctionFlags.Net) && superFunction == null && Outer is not UhtClass)
					{
						this.LogError($"Function '{SourceName}' base implementation of RPCs cannot be in a state. Add a stub outside state scope.");
					}

					// Check implemented RPC functions
					if (CppImplName != EngineName &&
						!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.CustomThunk) &&
						outerClass!.GeneratedCodeVersion > EGeneratedCodeVersion.V1)
					{
						bool isNative = FunctionFlags.HasAnyFlags(EFunctionFlags.Native);
						bool isNet = FunctionFlags.HasAnyFlags(EFunctionFlags.Net);
						bool isNetValidate = FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate);
						bool isNetResponse = FunctionFlags.HasAnyFlags(EFunctionFlags.NetResponse);
						bool isBlueprintEvent = FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent);

						bool needsImplementation = (isNet && !isNetResponse) || isBlueprintEvent || isNative;
						bool needsValidate = (isNative || isNet) && !isNetResponse && isNetValidate;

						bool hasImplementation = FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ImplFound);
						bool hasValidate = FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ValidationImplFound);
						bool hasVirtualImplementation = FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ImplVirtual);
						bool hasVirtualValidate = FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ValidationImplVirtual);

						if (needsImplementation || needsValidate)
						{
							// Check if functions are missing.
							if (needsImplementation && !hasImplementation)
							{
								LogRpcFunctionError(false, CppImplName);
							}

							if (needsValidate && !hasValidate)
							{
								LogRpcFunctionError(false, CppValidationImplName);
							}

							// If all needed functions are declared, check if they have virtual specifiers.
							if (needsImplementation && hasImplementation && !hasVirtualImplementation)
							{
								LogRpcFunctionError(true, CppImplName);
							}

							if (needsValidate && hasValidate && !hasVirtualValidate)
							{
								LogRpcFunctionError(true, CppValidationImplName);
							}
						}
					}

					// Validate field notify
					if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.FieldNotify))
					{
						if (outerClass == null)
						{
							this.LogError($"FieldNofity function '{SourceName}' are only valid as UClass member function.");
						}

						if (ParameterProperties.Length != 0)
						{
							this.LogError($"FieldNotify function '{SourceName}' must not have parameters.");
						}

						if (ReturnProperty == null)
						{
							this.LogError($"FieldNotify function '{SourceName}' must return a value.");
						}

						if (FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
						{
							this.LogError($"FieldNotify function '{SourceName}' cannot be a blueprint event.");
						}

						if (!FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure))
						{
							this.LogError($"FieldNotify function '{SourceName}' must be pure.");
						}
					}

					// Verse validations
					if (IsVerseField)
					{
						if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseDecides))
						{
							UhtProperty? returnProperty = ReturnProperty;
							if (returnProperty == null || returnProperty is not UhtOptionalProperty)
							{
								this.LogError($"Verse function '{SourceName}' marked as 'decides' must have a 'TOptional<...>' return value");
							}
						}

						if (Outer is UhtClass parent && !parent.IsVerseField)
						{
							this.LogError($"Function '{SourceName}' is marked as a Verse function but appears in a class that isn't a Verse class");
						}

						if (!FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable | UhtFunctionExportFlags.VerseCallable))
						{
							this.LogError($"Verse function '{SourceName}' must either be marked as NativeCallable or VerseCallable");
						}

						if (GetVerseScopeAndName(UhtVerseNameMode.Default) == SourceName)
						{
							this.LogError($"Verse function '{SourceName}' has the same name as the Verse generated name.  Change the source name and then specify the name in the UFUNCTION attributes");
						}
					}
					break;

				case UhtFunctionType.Delegate:
				case UhtFunctionType.SparseDelegate:

					// Check for shadowing if requested
					if (options.HasAnyFlags(UhtValidationOptions.Shadowing))
					{
						UhtType? existing = Outer!.FindType(UhtFindOptions.DelegateFunction | UhtFindOptions.EngineName | UhtFindOptions.SelfOnly, EngineName);
						if (existing != null && existing != this)
						{
							this.LogError($"Can't override delegate signature function '{SourceName}'");
						}
					}

					// Disable shadow checks for delegates
					options &= ~UhtValidationOptions.Shadowing;
					break;
			}

			// Make sure this isn't a duplicate type.  In some ways this has already been check above
			UhtType? existingType = Outer!.FindType(UhtFindOptions.Function | UhtFindOptions.EngineName | UhtFindOptions.SelfOnly, EngineName);
			if (existingType != null && existingType != this)
			{
				this.LogError($"'{EngineName}' conflicts with '{existingType.FullName}'");
			}

			// Do a more generic validation of the arguments
			foreach (UhtType type in Children)
			{
				if (type is UhtProperty property)
				{
					// This code is technically incorrect but here for compatibility reasons.  Container types (TObjectPtr, TArray, etc...) should have the 
					// const on the inside of the template arguments but it places the const on the outside which isn't correct.  This needs to be addressed
					// in any updates to the type parser.  See EGetVarTypeOptions::NoAutoConst for the parser side of the problem.
					if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm) && property.MustBeConstArgument(out UhtType? errorType))
					{
						if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm))
						{
							property.LogError($"Return value must be 'const' since '{errorType.SourceName}' is marked 'const'");
						}
						else
						{
							property.LogError($"Argument '{property.SourceName}' must be 'const' since '{errorType.SourceName}' is marked 'const'");
						}
					}

					// Due to structures that might not be fully parsed at parse time, do blueprint validation here
					if (checkForBlueprint)
					{
						if (property.IsStaticArray)
						{
							this.LogError($"Static array cannot be exposed to blueprint. Function: {SourceName} Parameter {property.SourceName}");
						}
						if (!property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsParameterSupportedByBlueprint))
						{
							this.LogError($"Type '{property.GetUserFacingDecl()}' is not supported by blueprint. Function:  {SourceName} Parameter {property.SourceName}");
						}
					}
				}
			}

			if (!Deprecated)
			{
				options |= UhtValidationOptions.Deprecated;
			}
			else
			{
				options &= ~UhtValidationOptions.Deprecated;
			}
			return options;
		}

		private void LogRpcFunctionError(bool virtualError, string functionName)
		{
			StringBuilder builder = new();

			if (virtualError)
			{
				builder.Append("Declared function \"");
			}
			else
			{
				builder.Append($"Function {SourceName} was marked as ");
				if (FunctionFlags.HasAnyFlags(EFunctionFlags.Native))
				{
					builder.Append("Native, ");
				}
				if (FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
				{
					builder.Append("Net, ");
				}
				if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
				{
					builder.Append("BlueprintEvent, ");
				}
				if (FunctionFlags.HasAnyFlags(EFunctionFlags.NetValidate))
				{
					builder.Append("NetValidate, ");
				}
				builder.Length -= 2;
				builder.Append(". Declare function \"virtual ");
			}

			UhtProperty? returnType = ReturnProperty;
			if (returnType != null)
			{
				builder.AppendPropertyText(returnType, UhtPropertyTextType.EventFunctionArgOrRetVal);
				builder.Append(' ');
			}
			else
			{
				builder.Append("void ");
			}

			builder.Append(Outer!.SourceName).Append("::").Append(functionName).Append('(');

			bool first = true;
			foreach (UhtType property in ParameterProperties.Span)
			{
				UhtProperty? arg = property as UhtProperty;
				if (arg != null)
				{
					if (!first)
					{
						builder.Append(", ");
					}
					first = false;
					builder.AppendFullDecl(arg, UhtPropertyTextType.EventFunctionArgOrRetVal, true);
				}
			}

			builder.Append(')');
			if (FunctionFlags.HasAnyFlags(EFunctionFlags.Const))
			{
				builder.Append(" const");
			}

			if (virtualError)
			{
				builder.Append("\" is not marked as virtual.");
			}
			else
			{
				builder.Append('\"');
			}
			this.LogError(builder.ToString());
		}

		private void ValidateSpecifierFlags()
		{
			if (FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.Exec))
				{
					this.LogError("Replicated functions can not be blueprint events or exec");
				}

				bool isNetService = FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest | EFunctionFlags.NetResponse);
				bool isNetReliable = FunctionFlags.HasAnyFlags(EFunctionFlags.NetReliable);

				if (FunctionFlags.HasAnyFlags(EFunctionFlags.Static))
				{
					this.LogError("Static functions can't be replicated");
				}

				if (!isNetReliable && !FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Unreliable) && !isNetService)
				{
					this.LogError("Replicated function: 'reliable' or 'unreliable' is required");
				}

				if (isNetReliable && FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Unreliable) && !isNetService)
				{
					this.LogError("'reliable' and 'unreliable' are mutually exclusive");
				}
			}
			else if (FunctionFlags.HasAnyFlags(EFunctionFlags.NetReliable))
			{
				this.LogError("'reliable' specified without 'client' or 'server'");
			}
			else if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.Unreliable))
			{
				this.LogError("'unreliable' specified without 'client' or 'server'");
			}

			if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.SealedEvent) && !FunctionFlags.HasAnyFlags(EFunctionFlags.Event))
			{
				this.LogError("SealedEvent may only be used on events");
			}

			if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.SealedEvent) && FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent))
			{
				this.LogError("SealedEvent cannot be used on Blueprint events");
			}

			if (FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.ForceBlueprintImpure) && FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintPure))
			{
				this.LogError("BlueprintPure (or BlueprintPure=true) and BlueprintPure=false should not both appear on the same function, they are mutually exclusive");
			}
		}

		/// <inheritdoc/>
		protected override void ValidateDocumentationPolicy(UhtDocumentationPolicy policy)
		{
			if (FunctionType != UhtFunctionType.Function)
			{
				return;
			}

			// Get the outer class - either directly or from partial's Owner class
			UhtClass? outerClass = Outer as UhtClass;
			if (outerClass == null && Outer is UhtPartial partial)
			{
				outerClass = partial.OwnerClass;
			}

			if (policy.FunctionToolTipsRequired)
			{
				if (!MetaData.ContainsKey(UhtNames.ToolTip))
				{
					this.LogError($"Function '{outerClass?.SourceName}::{SourceName}' does not provide a tooltip / comment (DocumentationPolicy).");
				}
			}

			if (policy.ParameterToolTipsRequired)
			{
				if (!MetaData.ContainsKey(UhtNames.Comment))
				{
					this.LogError($"Function '{outerClass?.SourceName}::{SourceName}' does not provide a comment (DocumentationPolicy).");
				}

				Dictionary<StringView, StringView> paramToolTips = GetParameterToolTipsFromFunctionComment(MetaData.GetValueOrDefault(UhtNames.Comment));
				bool hasAnyParamToolTips = paramToolTips.Count > 1 || (paramToolTips.Count == 1 && !paramToolTips.ContainsKey(UhtNames.ReturnValue));

				// only apply the validation for parameter tooltips if a function has any @param statements at all.
				if (hasAnyParamToolTips)
				{
					// ensure each parameter has a tooltip
					HashSet<StringView> existingFields = new(StringViewComparer.OrdinalIgnoreCase);
					foreach (UhtType parameter in ParameterProperties.Span)
					{
						existingFields.Add(parameter.SourceName);
						if (!paramToolTips.ContainsKey(parameter.SourceName))
						{
							this.LogError($"Function '{outerClass?.SourceName}::{SourceName}' doesn't provide a tooltip for parameter '{parameter.SourceName}' (DocumentationPolicy).");
						}
					}

					// ensure we don't have parameter tooltips for parameters that don't exist
					foreach (KeyValuePair<StringView, StringView> kvp in paramToolTips)
					{
						if (kvp.Key == UhtNames.ReturnValue)
						{
							continue;
						}

						if (!existingFields.Contains(kvp.Key))
						{
							this.LogError($"Function '{outerClass?.SourceName}::{SourceName}' provides a tooltip for an unknown parameter '{kvp.Key}'");
						}
					}

					// check for duplicate tooltips
					Dictionary<StringView, StringView> toolTipToParam = new(StringViewComparer.OrdinalIgnoreCase);
					foreach (KeyValuePair<StringView, StringView> kvp in paramToolTips)
					{
						if (kvp.Key == UhtNames.ReturnValue)
						{
							continue;
						}

						if (toolTipToParam.TryGetValue(kvp.Value, out StringView existing))
						{
							this.LogError($"Function '{outerClass?.SourceName}::{SourceName}' uses identical tooltips for parameters '{kvp.Key}' and '{existing}' (DocumentationPolicy).");
						}
						else
						{
							toolTipToParam.Add(kvp.Value, kvp.Key);
						}
					}
				}
			}
		}

		/// <summary>
		/// Add a parameter tooltip to the collection
		/// </summary>
		/// <param name="name">Name of the parameter</param>
		/// <param name="text">Text of the parameter tooltip</param>
		/// <param name="paramMap">Parameter map to add the tooltip</param>
		private static void AddParameterToolTip(StringView name, StringView text, Dictionary<StringView, StringView> paramMap)
		{
			paramMap.TryAdd(name, new StringView(UhtFCString.TabsToSpaces(text.Memory.Trim(), 4, false)));
		}

		/// <summary>
		/// Add a parameter tooltip to the collection where the name of the parameter is the first word of the line
		/// </summary>
		/// <param name="text">Text of the parameter tooltip</param>
		/// <param name="paramMap">Parameter map to add the tooltip</param>
		private static void AddParameterToolTip(StringView text, Dictionary<StringView, StringView> paramMap)
		{
			ReadOnlyMemory<char> trimmed = text.Memory.Trim();
			int nameEnd = trimmed.Span.IndexOf(' ');
			if (nameEnd >= 0)
			{
				AddParameterToolTip(new StringView(trimmed, 0, nameEnd), new StringView(trimmed, nameEnd + 1), paramMap);
			}
		}

		/// <summary>
		/// Parse the function comment looking for parameter documentation.
		/// </summary>
		/// <param name="input">The function input comment</param>
		/// <returns>Dictionary of parameter names and the documentation.  The return value will have a name of "ReturnValue"</returns>
		private static Dictionary<StringView, StringView> GetParameterToolTipsFromFunctionComment(StringView input)
		{
			const string ParamTag = "@param";
			const string ReturnTag = "@return";

			Dictionary<StringView, StringView> paramMap = new(StringViewComparer.OrdinalIgnoreCase);

			// Loop until we are out of text
			while (input.Length > 0)
			{

				// Extract the line
				StringView line = input;
				int lineEnd = input.Span.IndexOf('\n');
				if (lineEnd >= 0)
				{
					line = new StringView(input, 0, lineEnd);
					input = new StringView(input, lineEnd + 1);
				}
				else
				{
					input = new StringView();
				}

				// If this is a parameter, invoke the method to extract the name first
				int paramStart = line.Span.IndexOf(ParamTag, StringComparison.OrdinalIgnoreCase);
				if (paramStart >= 0)
				{
					AddParameterToolTip(new StringView(line, paramStart + ParamTag.Length), paramMap);
					continue;
				}

				// If this is the return value, we have a known name.  Invoke the method where we already know the name
				paramStart = line.Span.IndexOf(ReturnTag, StringComparison.OrdinalIgnoreCase);
				if (paramStart >= 0)
				{
					AddParameterToolTip(new StringView(UhtNames.ReturnValue), new StringView(line, paramStart + ReturnTag.Length), paramMap);
				}
			}

			return paramMap;
		}
		#endregion

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			if (FunctionType != UhtFunctionType.Function)
			{
				collector.AddExportType(this);
				collector.AddSingleton(this);
				collector.AddDeclaration(this, UhtSingletonType.Registered);
				collector.AddObjectReference(this, UhtSingletonType.Registered);
				collector.AddObjectReference(Package, UhtSingletonType.Registered);
			}
			foreach (UhtType child in Children)
			{
				child.CollectReferences(collector);
			}

			switch (FunctionType)
			{
				case UhtFunctionType.Function:
					collector.AddObjectReference(Session.UFunction, UhtSingletonType.ConstInit);
					collector.AddObjectReference(this, UhtSingletonType.ConstInit);
					collector.AddObjectReference(Package, UhtSingletonType.ConstInit);
					break;
				case UhtFunctionType.Delegate:
					collector.AddObjectReference(Session.UDelegateFunction, UhtSingletonType.ConstInit);
					break;
				case UhtFunctionType.SparseDelegate:
					collector.AddObjectReference(Session.USparseDelegateFunction, UhtSingletonType.ConstInit);
					break;
				default:
					break;
			}
		}

		/// <summary>
		/// Return the property parse options for the given function
		/// </summary>
		/// <param name="returnValue">If true, return property parse options for the return value</param>
		/// <returns>Property parse options</returns>
		/// <exception cref="UhtIceException"></exception>
		protected internal UhtPropertyParseOptions GetPropertyParseOptions(bool returnValue)
		{
			switch (FunctionType)
			{
				case UhtFunctionType.Delegate:
				case UhtFunctionType.SparseDelegate:
					return (returnValue ? UhtPropertyParseOptions.None : UhtPropertyParseOptions.CommaSeparatedName) | UhtPropertyParseOptions.DontAddReturn;

				case UhtFunctionType.Function:
					UhtPropertyParseOptions options = UhtPropertyParseOptions.DontAddReturn; // Fetch the function name
					options |= returnValue ? UhtPropertyParseOptions.FunctionNameIncluded : UhtPropertyParseOptions.NameIncluded;
					if (FunctionFlags.HasAllFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.Native))
					{
						options |= UhtPropertyParseOptions.NoAutoConst;
					}
					return options;

				default:
					throw new UhtIceException("Unknown enumeration value");
			}
		}
	}
}
