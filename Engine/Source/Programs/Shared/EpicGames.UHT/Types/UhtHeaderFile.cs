// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents the different directories where headers can appear
	/// </summary>
	public enum UhtHeaderFileType
	{

		/// <summary>
		/// Classes folder
		/// </summary>
		Classes,

		/// <summary>
		/// Public folder
		/// </summary>
		Public,

		/// <summary>
		/// Internal folder
		/// </summary>
		Internal,

		/// <summary>
		/// Private folder
		/// </summary>
		Private,
	}

	/// <summary>
	/// Series of flags not part of the engine's class flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtHeaderFileExportFlags : int
	{
		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// This header is being included by another header
		/// </summary>
		Referenced = 0x00000001,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtHeaderFileExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtHeaderFileExportFlags inFlags, UhtHeaderFileExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtHeaderFileExportFlags inFlags, UhtHeaderFileExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtHeaderFileExportFlags inFlags, UhtHeaderFileExportFlags testFlags, UhtHeaderFileExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Type of reference being added
	/// </summary>
	public enum UhtHeaderReferenceType
	{

		/// <summary>
		/// The header being referenced in a direct include
		/// </summary>
		Include,

		/// <summary>
		/// The header being referenced is a passive reference (i.e. NoExportTypes.h or a header referenced by a header being included)
		/// </summary>
		Passive,
	}

	/// <summary>
	/// Represents a header file.
	/// </summary>
	public class UhtHeaderFile : IUhtMessageSite, IUhtMessageLineNumber
	{
		private readonly UhtSimpleMessageSite _messageSite;
		private readonly UhtSourceFile _sourceFile;
		private readonly List<UhtHeaderFile> _referencedHeaders = [];
		private readonly List<UhtType> _children = [];

		/// <summary>
		/// Contents of the header
		/// </summary>
		[JsonIgnore]
		public StringView Data => _sourceFile.Data;

		/// <summary>
		/// Path of the header
		/// </summary>
		[JsonIgnore]
		public string FilePath => _sourceFile.FilePath;

		/// <summary>
		/// Currently running session
		/// </summary>
		[JsonIgnore]
		public UhtSession Session => Module.Session;

		/// <summary>
		/// Module associated with the header
		/// </summary>
		[JsonIgnore]
		public UhtModule Module { get; }

		/// <summary>
		/// File name without the extension
		/// </summary>
		[JsonIgnore]
		public string FileNameWithoutExtension { get; }

		/// <summary>
		/// Required name for the generated.h file name.  Used to validate parsed code
		/// </summary>
		[JsonIgnore]
		public string GeneratedHeaderFileName { get; }

		/// <summary>
		/// True if this header is NoExportTypes.h
		/// </summary>
		[JsonIgnore]
		public bool IsNoExportTypes { get; }

		/// <summary>
		/// The file path of the header relative to the module location
		/// </summary>
		public string ModuleRelativeFilePath { get; set; } = String.Empty;

		/// <summary>
		/// Include file path added as meta data to the types
		/// </summary>
		public string IncludeFilePath { get; set; } = String.Empty;

		/// <summary>
		/// Location where the header file was found
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtHeaderFileType HeaderFileType { get; set; } = UhtHeaderFileType.Private;

		/// <summary>
		/// Unique index of the header file
		/// </summary>
		[JsonIgnore]
		public int HeaderFileTypeIndex { get; }

		/// <summary>
		/// If true, this header file came from the input cache
		/// </summary>
		[JsonIgnore]
		public bool WasCached { get; set; }

		/// <summary>
		/// Contains the last write time of the source file or cached source.
		/// If the file was cached, then the data is the cached contents
		/// </summary>
		[JsonIgnore]
		public UhtInputCacheEntry InputCacheEntry { get; set; }

		/// <summary>
		/// UHT flags for the header
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtHeaderFileExportFlags HeaderFileExportFlags { get; set; } = UhtHeaderFileExportFlags.None;

		/// <summary>
		/// Children types of this type
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeListJsonConverter<UhtType>))]
		public IReadOnlyList<UhtType> Children => _children;

		/// <inheritdoc/>
		public override string ToString() { return FilePath; }

		/// <summary>
		/// If true, the header file should be exported
		/// </summary>
		[JsonIgnore]
		public bool ShouldExport => HeaderFileExportFlags.HasAnyFlags(UhtHeaderFileExportFlags.Referenced) || Children.Count > 0;

		/// <summary>
		/// Resource collector for the header file
		/// </summary>
		[JsonIgnore]
		public UhtReferenceCollector References { get; init; }

		/// <summary>
		/// Collection of headers directly included by this header
		/// </summary>
		[JsonIgnore]
		public List<UhtHeaderFile> IncludedHeaders { get; } = [];

		#region IUHTMessageSite implementation
		/// <inheritdoc/>
		[JsonIgnore]
		public IUhtMessageSession MessageSession => _messageSite.MessageSession;

		/// <inheritdoc/>
		[JsonIgnore]
		public IUhtMessageSource? MessageSource => _messageSite.MessageSource;

		/// <inheritdoc/>
		[JsonIgnore]
		public IUhtMessageLineNumber? MessageLineNumber => this;
		#endregion

		#region IUhtMessageLineNumber implementation
		[JsonIgnore]
		int IUhtMessageLineNumber.MessageLineNumber => 1;
		#endregion

		/// <summary>
		/// Construct a new header file
		/// </summary>
		/// <param name="module">Owning module</param>
		/// <param name="path">Path to the header file</param>
		public UhtHeaderFile(UhtModule module, string path)
		{
			Module = module;
			References = new UhtReferenceCollector(module);
			HeaderFileTypeIndex = Session.GetNextHeaderFileTypeIndex();
			_messageSite = new UhtSimpleMessageSite(Session);
			_sourceFile = new UhtSourceFile(Session, path);
			_messageSite.MessageSource = _sourceFile;
			FileNameWithoutExtension = Path.GetFileNameWithoutExtension(_sourceFile.FilePath);
			GeneratedHeaderFileName = FileNameWithoutExtension + ".generated.h";
			IsNoExportTypes = String.Equals(_sourceFile.FileName, "NoExportTypes", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Read the contents of the header
		/// </summary>
		public void Read()
		{
			Reset();
			if (!IsNoExportTypes && Session.NoExportTypesHeader != null)
			{
				AddReferencedHeader(Session.NoExportTypesHeader, UhtHeaderReferenceType.Passive);
			}
			_sourceFile.Read();
			Session.FilesRead.Increment();
		}

		/// <summary>
		/// Attempt to read the file from the cache
		/// </summary>
		public void ReadFromCache()
		{
			Reset();
			FileItem headerFile = FileItem.GetItemByFileReference(new FileReference(_sourceFile.FilePath));
			WasCached = Module.TryGetCachedInput(_sourceFile.FilePath, out UhtInputCacheEntry entry);
			if (WasCached)
			{
				if (headerFile.LastWriteTimeUtc > entry.LastWriteTimeUtc)
				{
					WasCached = false;
				}
				else
				{
					InputCacheEntry = entry;
				}
			}

			if (WasCached)
			{
				try
				{
					using UhtInputCacheReader reader = new(this, entry);
					reader.ReadHeader();
					Session.FilesCacheRead.Increment();
				}
				catch (Exception ex)
				{
					Session.FilesCacheReadErrors.Increment();
					Session.Logger.LogInformation(ex, "File {FileName} failed to deserialize from input cache: {Message}", _sourceFile.FilePath, ex.Message);
				}
			}
			else
			{
				InputCacheEntry = new() { LastWriteTimeUtc = headerFile.LastWriteTimeUtc };
			}
		}

		/// <summary>
		/// Add a reference to the given header
		/// </summary>
		/// <param name="id">Path of the header</param>
		/// <param name="referenceType">How is the include in question referenced</param>
		public void AddReferencedHeader(string id, UhtHeaderReferenceType referenceType)
		{
			UhtHeaderFile? headerFile = FindHeaderFile(id);
			// Check that the found header file matches the path fragment from the include
			// directive as the session currently only maps header files by their leaf file name
			// and does not mimic full C++ include paths
			if (headerFile != null && headerFile.ModuleRelativeFilePath.EndsWith(id, StringComparison.Ordinal))
			{
				AddReferencedHeader(headerFile, referenceType);
			}
		}

		/// <summary>
		/// Add a reference to the header that defines the given type
		/// </summary>
		/// <param name="type">Type in question</param>
		public void AddReferencedHeader(UhtType type)
		{
			AddReferencedHeader(type.HeaderFile, UhtHeaderReferenceType.Passive);
		}

		/// <summary>
		/// Add a reference to the given header file
		/// </summary>
		/// <param name="headerFile">Header file in question</param>
		/// <param name="referenceType">How is the include in question referenced</param>
		public void AddReferencedHeader(UhtHeaderFile headerFile, UhtHeaderReferenceType referenceType)
		{

			// Ignore direct references to myself
			if (referenceType != UhtHeaderReferenceType.Include && headerFile == this)
			{
				return;
			}

			lock (_referencedHeaders)
			{
				// Check for a duplicate
				foreach (UhtHeaderFile reference in _referencedHeaders)
				{
					if (reference == headerFile)
					{
						return;
					}
				}

				// There is questionable compatibility hack where a source file will always be exported
				// regardless of having types when it is being included by the SAME package.
				if (headerFile.Module == Module)
				{
					headerFile.HeaderFileExportFlags |= UhtHeaderFileExportFlags.Referenced;
				}
				_referencedHeaders.Add(headerFile);
				if (referenceType == UhtHeaderReferenceType.Include)
				{
					IncludedHeaders.Add(headerFile);
				}
			}
		}

		/// <summary>
		/// Return an enumerator without locking.  This method can only be utilized AFTER all header parsing is complete. 
		/// </summary>
		[JsonIgnore]
		public IEnumerable<UhtHeaderFile> ReferencedHeadersNoLock => _referencedHeaders;

		/// <summary>
		/// Return an enumerator of all current referenced headers under a lock.  This should be used during parsing.
		/// </summary>
		[JsonIgnore]
		public IEnumerable<UhtHeaderFile> ReferencedHeadersLocked
		{
			get
			{
				lock (_referencedHeaders)
				{
					foreach (UhtHeaderFile reference in _referencedHeaders)
					{
						yield return reference;
					}
				}
			}
		}

		/// <summary>
		/// Write the header to the archive writer
		/// </summary>
		/// <param name="writer">Destination cache writer</param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteString(Data.Span);
			writer.WriteUInt64((ulong)HeaderFileExportFlags);
			writer.WriteVariableLengthArray(_referencedHeaders, (writer, x) => writer.WriteString(x.FilePath));
			writer.WriteVariableLengthArray(IncludedHeaders, (writer, x) => writer.WriteString(x.FilePath));
		}

		/// <summary>
		/// Read the header from the archive
		/// </summary>
		/// <param name="reader">Source reader</param>
		public void Read(UhtInputCacheReader reader)
		{
			string source = reader.ReadString();
			_sourceFile.SetSource(source);
			HeaderFileExportFlags = (UhtHeaderFileExportFlags)reader.ReadUInt64();
			ReadHeaderFiles(reader, _referencedHeaders);
			ReadHeaderFiles(reader, IncludedHeaders);
		}

		/// <summary>
		/// Add a type as a child
		/// </summary>
		/// <param name="child">The child to be added.</param>
		public void AddChild(UhtType child)
		{
			_children.Add(child);
		}

		/// <summary>
		/// Resolve all types owned by the header
		/// </summary>
		/// <param name="resolvePhase">Phase of the resolution process</param>
		public void Resolve(UhtResolvePhase resolvePhase)
		{

			// Use the invalid check which happens prior to all other phase to propagate the UhtHeaderFileExportFlags.Referenced hack.
			// This is only needed when we are reading from the input cache.
			if (Session.EnableInputCacheRead && resolvePhase == UhtResolvePhase.InvalidCheck)
			{
				foreach (UhtHeaderFile headerFile in _referencedHeaders)
				{
					// There is questionable compatibility hack where a source file will always be exported
					// regardless of having types when it is being included by the SAME package.
					if (headerFile.Module == Module)
					{
						headerFile.HeaderFileExportFlags |= UhtHeaderFileExportFlags.Referenced;
					}
				}
			}

			UhtType.ResolveChildren(_children, resolvePhase);
		}

		/// <summary>
		/// Bind all the super structs and base classes
		/// </summary>
		public void BindSuperAndBases()
		{
			foreach (UhtType child in Children)
			{
				child.BindSuperAndBases();
			}
		}

		/// <summary>
		/// Validate the state of the header file
		/// </summary>
		/// <param name="options">Validation options</param>
		/// <returns></returns>
		public void Validate(UhtValidationOptions options)
		{
			options |= UhtValidationOptions.Shadowing;

			Dictionary<int, UhtFunction> usedRPCIds = [];
			Dictionary<int, UhtFunction> rpcsNeedingHookup = [];
			foreach (UhtType type in Children)
			{
				if (type is UhtClass classObj)
				{
					foreach (UhtType child in classObj.Children)
					{
						if (child is UhtFunction function)
						{
							if (function.FunctionType != UhtFunctionType.Function || !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
							{
								continue;
							}

							if (function.RPCId > 0)
							{
								if (usedRPCIds.TryGetValue(function.RPCId, out UhtFunction? existingFunc))
								{
									function.LogError($"Function {existingFunc.SourceName} already uses identifier {function.RPCId}");
								}
								else
								{
									usedRPCIds.Add(function.RPCId, function);
								}

								if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.NetResponse))
								{
									// Look for another function expecting this response
									rpcsNeedingHookup.Remove(function.RPCId);
								}
							}

							if (function.RPCResponseId > 0 && function.EndpointName != "JSBridge")
							{
								// Look for an existing response function, if not found then add to the list of ids awaiting hookup
								if (!usedRPCIds.ContainsKey(function.RPCResponseId))
								{
									rpcsNeedingHookup.Add(function.RPCResponseId, function);
								}
							}
						}
					}
				}
			}

			if (rpcsNeedingHookup.Count > 0)
			{
				foreach (KeyValuePair<int, UhtFunction> kvp in rpcsNeedingHookup)
				{
					kvp.Value.LogError($"Request function '{kvp.Value.SourceName}' is missing a response function with the id of '{kvp.Key}'");
				}
			}

			foreach (UhtType child in Children)
			{
				UhtType.ValidateType(child, options);
			}
		}

		/// <summary>
		/// Collect all things referenced by the given header
		/// </summary>
		public void CollectReferences()
		{
			foreach (UhtType child in Children)
			{
				child.CollectReferences(References);
			}
			foreach (UhtHeaderFile refHeaderFile in References.ReferencedHeaders)
			{
				AddReferencedHeader(refHeaderFile, UhtHeaderReferenceType.Passive);
			}
		}

		private void Reset()
		{
			HeaderFileExportFlags = UhtHeaderFileExportFlags.None;
			_referencedHeaders.Clear();
			IncludedHeaders.Clear();
		}

		private UhtHeaderFile? FindHeaderFile(string filePath)
		{
			string shortPath = Path.GetFileName(filePath);
			return Session.FindHeaderFile(shortPath);
		}

		private void ReadHeaderFiles(UhtInputCacheReader reader, List<UhtHeaderFile> output)
		{
			List<UhtHeaderFile?> list = reader.ReadList<UhtHeaderFile?>((reader) => FindHeaderFile(reader.ReadString()));
			foreach (UhtHeaderFile? headerFile in list)
			{
				if (headerFile != null)
				{
					output.Add(headerFile);
				}
			}
		}
	}
}
