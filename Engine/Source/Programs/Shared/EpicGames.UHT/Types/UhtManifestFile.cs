// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents the UHT manifest file
	/// </summary>
	public class UhtManifestFile : IUhtMessageSite
	{

		/// <summary>
		/// Loaded manifest from the json manifest file
		/// </summary>
		public UHTManifest? Manifest { get; set; } = null;

		private readonly UhtSourceFile _sourceFile;

		#region IUHTMessageSite implementation

		/// <inheritdoc/>
		public IUhtMessageSession MessageSession => _sourceFile.MessageSession;

		/// <inheritdoc/>
		public IUhtMessageSource? MessageSource => _sourceFile.MessageSource;

		/// <inheritdoc/>
		public IUhtMessageLineNumber? MessageLineNumber => null;
		#endregion

		/// <summary>
		/// Construct a new manifest file
		/// </summary>
		/// <param name="session">Current session</param>
		/// <param name="filePath">Path of the file</param>
		public UhtManifestFile(UhtSession session, string filePath)
		{
			_sourceFile = new UhtSourceFile(session, filePath);
		}

		/// <summary>
		/// Read the contents of the file
		/// </summary>
		public void Read()
		{
			_sourceFile.Read();
			Manifest = JsonSerializer.Deserialize<UHTManifest>(_sourceFile.Data.ToString());
		}
	}
}
