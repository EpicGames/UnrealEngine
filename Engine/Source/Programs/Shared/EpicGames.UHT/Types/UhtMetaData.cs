// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// UhtEnum types need to convert a meta data name index to the enum name
	/// </summary>
	public interface IUhtMetaDataKeyConversion
	{

		/// <summary>
		/// Convert a name and index into a full meta data name
		/// </summary>
		/// <param name="name">Base name of the meta data</param>
		/// <param name="nameIndex">Index of the meta data.  -1 for the root object.</param>
		/// <returns>Complete meta data key name</returns>
		string GetMetaDataKey(string name, int nameIndex);
	}

	/// <summary>
	/// Uniquely identifies a meta data element
	/// </summary>
	public struct UhtMetaDataKey
	{

		/// <summary>
		/// The name of the meta data
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// The index of the meta data name (i.e. enum value index) or -1 for the owning object meta data
		/// </summary>
		public int Index { get; set; }

		/// <summary>
		/// Construct a new meta data key
		/// </summary>
		/// <param name="name">Meta data name</param>
		/// <param name="index">Meta data index</param>
		public UhtMetaDataKey(string name, int index = UhtMetaData.IndexNone)
		{
			Name = name;
			Index = index;
		}

		/// <summary>
		/// Read from the archive reader
		/// </summary>
		/// <param name="reader">Source cache reader</param>
		public UhtMetaDataKey(UhtInputCacheReader reader)
		{
			Name = reader.ReadString();
			Index = reader.ReadInt32();
		}

		/// <summary>
		/// Write to the archive writer
		/// </summary>
		/// <param name="writer">Destination cache writer</param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteString(Name);
			writer.WriteInt32(Index);
		}

		/// <summary>
		/// Convert the key to a string
		/// </summary>
		/// <returns>String representation</returns>
		public override readonly string ToString()
		{
			if (Index == UhtMetaData.IndexNone)
			{
				return Name.ToString();
			}
			else
			{
				return $"{Name}:{Index}";
			}
		}
	}

	/// <summary>
	/// Comparer for meta data keys
	/// </summary>
	sealed class UhtMetaDataKeyComparer : IEqualityComparer<UhtMetaDataKey>, IComparer<UhtMetaDataKey>
	{
		/// <summary>
		/// Compare two keys
		/// </summary>
		/// <param name="x">First key</param>
		/// <param name="y">Second key</param>
		/// <returns>-1, 0, or 1 depending on the relationship</returns>
		public int Compare([AllowNull] UhtMetaDataKey x, [AllowNull] UhtMetaDataKey y)
		{
			if (x.Index < y.Index)
			{
				return -1;
			}
			else if (x.Index > y.Index)
			{
				return 1;
			}
			return String.Compare(x.Name, y.Name, StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Test to see if two meta data keys are equal
		/// </summary>
		/// <param name="x">First key</param>
		/// <param name="y">Second key</param>
		/// <returns>True if the keys match</returns>
		public bool Equals([AllowNull] UhtMetaDataKey x, [AllowNull] UhtMetaDataKey y)
		{
			return x.Index == y.Index && x.Name.Equals(y.Name, StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Get the hash code of the meta data key
		/// </summary>
		/// <param name="obj">Key</param>
		/// <returns>Hash code</returns>
		public int GetHashCode([DisallowNull] UhtMetaDataKey obj)
		{
			return obj.Index.GetHashCode() ^ obj.Name.GetHashCode(StringComparison.OrdinalIgnoreCase);
		}
	}

	/// <summary>
	/// Represents a collection of key/value pairs.  Each type has this collection and enumerations also
	/// have key/value pairs for each enumeration value index.
	/// </summary>
	public class UhtMetaData
	{

		/// <summary>
		/// Helper comparer for meta data keys
		/// </summary>
		private static readonly UhtMetaDataKeyComparer s_comparer = new();

		/// <summary>
		/// Empty collection of meta data
		/// </summary>
		public static readonly UhtMetaData Empty = new(null, null);

		/// <summary>
		/// The meta data of the outer object for the type that owns this meta data
		/// </summary>
		[JsonIgnore]
		public UhtMetaData? Parent { get; set; } = null;

		/// <summary>
		/// Message site associated with the meta data.  That in combination with the line number is used to generate errors
		/// </summary>
		[JsonIgnore]
		public IUhtMessageSite? MessageSite { get; set; } = null;

		/// <summary>
		/// Configuration object used to redirect meta data
		/// </summary>
		[JsonIgnore]
		public IUhtConfig? Config { get; set; } = null;

		/// <summary>
		/// Source code line number where the meta data is declared
		/// </summary>
		[JsonIgnore]
		public int LineNumber { get; set; } = 1;

		/// <summary>
		/// Contains the meta data entries.  Due to the small size of these dictionaries, a SortedList performs
		/// better than a Dictionary.
		/// </summary>
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "")]
		public SortedList<UhtMetaDataKey, string>? Dictionary { get; set; } = null;

		/// <summary>
		/// Enumerations implement this interface so that the index part of the key can be converted to a string
		/// </summary>
		internal IUhtMetaDataKeyConversion? KeyConversion { get; set; } = null;

		/// <summary>
		/// Index for a meta data key associated with the owning object
		/// </summary>
		public const int IndexNone = -1;

		/// <summary>
		/// Construct new meta data
		/// </summary>
		/// <param name="messageSite">Message site for generating errors</param>
		/// <param name="config">Configuration for redirects</param>
		public UhtMetaData(IUhtMessageSite? messageSite, IUhtConfig? config)
		{
			MessageSite = messageSite;
			Config = config;
		}

		/// <summary>
		/// Read from the archive reader
		/// </summary>
		/// <param name="reader">Source reader</param>
		public void Read(UhtInputCacheReader reader)
		{
			LineNumber = reader.ReadInt32();

			int count = reader.ReadInt32();
			for (int index = 0; index < count; index++)
			{
				UhtMetaDataKey key = new(reader);
				string value = reader.ReadString();
				GetDictionary().Add(key, value);
			}
		}

		/// <summary>
		/// Write to the archive writer
		/// </summary>
		/// <param name="writer">Destination cache writer</param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteInt32(LineNumber);
			if (Dictionary == null)
			{
				writer.WriteInt32(0);
			}
			else
			{
				writer.WriteInt32(Dictionary.Count);
				foreach (KeyValuePair<UhtMetaDataKey, string> kvp in Dictionary)
				{
					kvp.Key.Write(writer);
					writer.WriteString(kvp.Value);
				}
			}
		}

		/// <summary>
		/// Test to see if the meta data object contains no entries
		/// </summary>
		/// <returns>True if the meta data object contains no entries</returns>
		public bool IsEmpty()
		{
			return Dictionary == null || Dictionary.Count == 0;
		}

		/// <summary>
		/// Remove all meta data entries
		/// </summary>
		public void Clear()
		{
			Dictionary?.Clear();
		}

		/// <summary>
		/// Test to see if the meta data contains the given key
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="nameIndex">Enumeration value index or -1 for the type's meta data</param>
		/// <returns>True if the key is found</returns>
		public bool ContainsKey(string name, int nameIndex = IndexNone)
		{
			if (Dictionary == null)
			{
				return false;
			}
			return Dictionary.ContainsKey(new UhtMetaDataKey(name, nameIndex));
		}

		/// <summary>
		/// Test to see if the meta data or parent meta data contains the given key
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="nameIndex">Enumeration value index or -1 for the type's meta data</param>
		/// <returns>True if the key is found</returns>
		public bool ContainsKeyHierarchical(string name, int nameIndex = IndexNone)
		{
			for (UhtMetaData? current = this; current != null; current = current.Parent)
			{
				if (current.ContainsKey(name, nameIndex))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Attempt to get the value associated with the key
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="value">Found value</param>
		/// <returns>True if the key is found</returns>
		public bool TryGetValue(string name, [NotNullWhen(true)] out string? value)
		{
			return TryGetValue(name, IndexNone, out value);
		}

		/// <summary>
		/// Attempt to get the value associated with the key including parent meta data
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="value">Found value</param>
		/// <returns>True if the key is found</returns>
		public bool TryGetValueHierarchical(string name, [NotNullWhen(true)] out string? value)
		{
			return TryGetValueHierarchical(name, IndexNone, out value);
		}

		/// <summary>
		/// Attempt to get the value associated with the key
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="nameIndex">Enumeration value index or -1 for the type's meta data</param>
		/// <param name="value">Found value</param>
		/// <returns>True if the key is found</returns>
		public bool TryGetValue(string name, int nameIndex, [NotNullWhen(true)] out string? value)
		{
			if (Dictionary == null)
			{
				value = String.Empty;
				return false;
			}
			return Dictionary.TryGetValue(new UhtMetaDataKey(name, nameIndex), out value);
		}

		/// <summary>
		/// Attempt to get the value associated with the key including parent meta data
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="nameIndex">Enumeration value index or -1 for the type's meta data</param>
		/// <param name="value">Found value</param>
		/// <returns>True if the key is found</returns>
		public bool TryGetValueHierarchical(string name, int nameIndex, [NotNullWhen(true)] out string? value)
		{
			for (UhtMetaData? current = this; current != null; current = current.Parent)
			{
				if (current.TryGetValue(name, nameIndex, out value))
				{
					return true;
				}
			}
			value = String.Empty;
			return false;
		}

		/// <summary>
		/// Get the string value of the given meta data key or the default value.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Meta data value or empty string if not found.</returns>
		public string GetValueOrDefault(string name, int nameIndex = IndexNone)
		{
			if (TryGetValue(name, nameIndex, out string? output))
			{
				return output;
			}
			return String.Empty;
		}

		/// <summary>
		/// Get the string value of the given meta data searching the whole meta data chain.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Meta data value or empty string if not found.</returns>
		public string GetValueOrDefaultHierarchical(string name, int nameIndex = IndexNone)
		{
			if (TryGetValueHierarchical(name, nameIndex, out string? output))
			{
				return output;
			}
			return String.Empty;
		}

		/// <summary>
		/// Get the boolean value of the given meta data.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Boolean value or false if not found</returns>
		public bool GetBoolean(string name, int nameIndex = IndexNone)
		{
			return UhtFCString.ToBool(GetValueOrDefault(name, nameIndex));
		}

		/// <summary>
		/// Get the boolean value of the given meta data searching the whole meta data chain.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Boolean value or false if not found</returns>
		public bool GetBooleanHierarchical(string name, int nameIndex = IndexNone)
		{
			return UhtFCString.ToBool(GetValueOrDefaultHierarchical(name, nameIndex));
		}

		/// <summary>
		/// Get the double value of the given meta data.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Double value or zero if not found</returns>
		public double GetDouble(string name, int nameIndex = IndexNone)
		{
			string value = GetValueOrDefault(name, nameIndex);
			if (!Double.TryParse(value, out double result))
			{
				result = 0;
			}
			return result;
		}

		/// <summary>
		/// Get the string array value of the given meta data.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>String array or null if not found</returns>
		public string[]? GetStringArray(string name, int nameIndex = IndexNone)
		{
			if (TryGetValue(name, nameIndex, out string? temp))
			{
				return temp.ToString().Split(' ', StringSplitOptions.RemoveEmptyEntries);
			}
			return null;
		}

		/// <summary>
		/// Get the string array value of the given meta data searching the whole meta data chain.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>String array or null if not found</returns>
		public string[]? GetStringArrayHierarchical(string name, int nameIndex = IndexNone)
		{
			if (TryGetValueHierarchical(name, nameIndex, out string? temp))
			{
				return temp.ToString().Split(' ', StringSplitOptions.RemoveEmptyEntries);
			}
			return null;
		}

		/// <summary>
		/// Add new meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="value">Value of the meta data</param>
		public void Add(string name, string value)
		{
			Add(name, IndexNone, value);
		}

		/// <summary>
		/// Add new meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <param name="value">Value of the meta data</param>
		public void Add(string name, int nameIndex, string value)
		{
			AddInternalWithRedirect(name, nameIndex, value, false);
		}

		/// <summary>
		/// Add new meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <param name="value">Value of the meta data</param>
		public void CheckedAdd(string name, int nameIndex, string value)
		{
			AddInternalWithRedirect(name, nameIndex, value, true);
		}

		/// <summary>
		/// Add new meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="value">Value of the meta data</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		public void Add(string name, bool value, int nameIndex = IndexNone)
		{
			Add(name, nameIndex, value ? "true" : "false");
		}

		/// <summary>
		/// Add new meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="strings">Value of the meta data</param>
		/// <param name="separator">Separator to use to join the strings</param>
		public void Add(string name, List<string> strings, char separator = ' ')
		{
			Add(name, String.Join(separator, strings));
		}

		/// <summary>
		/// Add new meta data if there are strings in the value.
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="strings">Value of the meta data</param>
		/// <param name="separator">Separator to use to join the strings</param>
		public void AddIfNotEmpty(string name, List<string> strings, char separator = ' ')
		{
			if (strings.Count != 0)
			{
				Add(name, String.Join(separator, strings));
			}
		}

		/// <summary>
		/// Add the meta data from another meta data block
		/// </summary>
		/// <param name="metaData"></param>
		public void Add(UhtMetaData metaData)
		{
			if (metaData.Dictionary != null)
			{
				SortedList<UhtMetaDataKey, string> dictionary = GetDictionary();
				foreach (KeyValuePair<UhtMetaDataKey, string> kvp in metaData.Dictionary)
				{
					AddInternal(dictionary, kvp.Key, kvp.Value, true);
				}
			}
		}

		/// <summary>
		/// Add the given meta data but test for redirection of meta data name
		/// </summary>
		/// <param name="name">Name of the meta data</param>
		/// <param name="nameIndex">Index of the meta data</param>
		/// <param name="value">Value of the meta data</param>
		/// <param name="checkValueChange">If true, verifies that the meta data value is not changing</param>
		private void AddInternalWithRedirect(string name, int nameIndex, string value, bool checkValueChange)
		{
			if (Config != null)
			{
				if (Config.RedirectMetaDataKey(name, out string remappedName))
				{
					MessageSite?.LogWarning(LineNumber, $"Remapping old metadata key '{name}' to new key '{remappedName}', please update the declaration.");
				}
			}
			AddInternal(GetDictionary(), new UhtMetaDataKey(name, nameIndex), value, checkValueChange);
		}

		/// <summary>
		/// Add the value to the meta data
		/// </summary>
		/// <param name="dictionary">Destination dictionary</param>
		/// <param name="key">Key of the meta data</param>
		/// <param name="value">Value of the meta data</param>
		/// <param name="checkValueChange">If true, verifies that the meta data value is not changing</param>
		private void AddInternal(SortedList<UhtMetaDataKey, string> dictionary, UhtMetaDataKey key, string value, bool checkValueChange)
		{
			if (checkValueChange && dictionary.TryGetValue(key, out string? oldValue))
			{
				if (!oldValue.Equals(value, StringComparison.OrdinalIgnoreCase))
				{
					MessageSite?.LogError(LineNumber, $"Metadata key '{key.Name}' first seen with value '{oldValue}' then '{value}'");
				}
			}
			else
			{
				dictionary[key] = value;
			}
		}

		/// <summary>
		/// Remove the given meta data
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		public void Remove(string name, int nameIndex = IndexNone)
		{
			Dictionary?.Remove(new UhtMetaDataKey(name, nameIndex));
		}

		/// <summary>
		/// Given a key, return the full meta data name
		/// </summary>
		/// <param name="key">Meta data key</param>
		/// <returns>Full meta data name</returns>
		public string GetKeyString(UhtMetaDataKey key)
		{
			return GetKeyString(key.Name, key.Index);
		}

		/// <summary>
		/// Given a key, return the full meta data name
		/// </summary>
		/// <param name="name">Name of the meta data key</param>
		/// <param name="nameIndex">Index of the meta data key</param>
		/// <returns>Full meta data name</returns>
		/// <exception cref="UhtIceException">Thrown if an index is supplied (not -1) and no key conversion interface is set</exception>
		public string GetKeyString(string name, int nameIndex)
		{
			if (nameIndex == IndexNone)
			{
				return name;
			}
			else if (KeyConversion != null)
			{
				return KeyConversion.GetMetaDataKey(name, nameIndex);
			}
			else
			{
				throw new UhtIceException("Attempt to generate an indexed meta data key name but no key conversion interface was set.");
			}
		}

		/// <summary>
		/// Given a type, return an array of all the meta data formatted and sorted by name
		/// </summary>
		/// <returns>List of meta data key and value pairs</returns>
		public List<KeyValuePair<string, string>> GetSorted()
		{
			List<KeyValuePair<string, string>> output = new(Dictionary != null ? Dictionary.Count : 0);
			if (Dictionary != null && Dictionary.Count > 0)
			{
				foreach (KeyValuePair<UhtMetaDataKey, string> kvp in Dictionary)
				{
					output.Add(new KeyValuePair<string, string>(GetKeyString(kvp.Key), kvp.Value));
				}

				output.Sort((lhs, rhs) =>
				{
					return StringComparerUE.OrdinalIgnoreCase.Compare(lhs.Key, rhs.Key);
				});
			}
			return output;
		}

		private SortedList<UhtMetaDataKey, string> GetDictionary()
		{
			Dictionary ??= new SortedList<UhtMetaDataKey, string>(s_comparer);
			return Dictionary;
		}
	}
}
