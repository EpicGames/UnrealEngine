// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Utils;
using UnrealBuildBase;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Index assignments for other modules which are a dependency of this module, and objects which are linkable in this module from others.
	/// </summary>
	/// <param name="ReferencedModules">Mapping of dependency module to the index to use for encoded pointers when linking to it from this module.</param>
	/// <param name="LinkableObjects">Mapping of public linkable objects in this module to indexes that other modules can use when encoding pointers to them.</param>
	public record UhtModuleLinking(IReadOnlyDictionary<UhtModule, int> ReferencedModules, IReadOnlyDictionary<UhtObject, int> LinkableObjects);

	/// <summary>
	/// Represents a UHT module
	/// </summary>
	public class UhtModule
	{
		private readonly List<UhtHeaderFile> _headers = [];
		private readonly SortedDictionary<string, UhtPackage> _packages = [];
		private readonly HashSet<string> _ubtModuleDependencies = [];
		private readonly Lazy<UhtInputCache> _inputCache;

		/// <summary>
		/// The running session
		/// </summary>
		[JsonIgnore]
		public UhtSession Session { get; }

		/// <summary>
		/// UHT module from UBT (1 to 1 relationship)
		/// </summary>
		public UHTManifest.Module Module { get; }

		/// <summary>
		/// Primary /Script/ package
		/// </summary>
		[JsonIgnore]
		public UhtPackage ScriptPackage { get; }

		/// <summary>
		/// Name of the module based on the script package name with out the "/Script/" prefix
		/// </summary>
		public string ShortName { get; }

		/// <summary>
		/// C++ API name for the module with a trailing space
		/// </summary>
		[JsonIgnore]
		public string Api { get; }

		/// <summary>
		/// C++ API non-attributed name for the module with a trailing space
		/// </summary>
		[JsonIgnore]
		public string NonAttributedApi { get; }

		/// <summary>
		/// Enumeration of all the packages in the module.
		/// </summary>
		public IEnumerable<UhtPackage> Packages
		{
			get
			{
				yield return ScriptPackage;
				lock (_packages)
				{
					foreach (KeyValuePair<string, UhtPackage> kvp in _packages)
					{
						yield return kvp.Value;
					}
				}
			}
		}

		/// <summary>
		/// Collection of headers owned by the module
		/// </summary>
		[JsonIgnore]
		public IReadOnlyList<UhtHeaderFile> Headers => _headers;

		/// <summary>
		/// Collection of module dependencies from UBT (may also include the names of modules not present in UHT)
		/// </summary>
		public IReadOnlySet<string> UBTModuleDependencies => _ubtModuleDependencies;

		/// <summary>
		/// True if the package is part of the engine
		/// </summary>
		[JsonIgnore]
		public bool IsPartOfEngine
		{
			get
			{
				switch (Module.ModuleType)
				{
					case UHTModuleType.Program:
						return Module.BaseDirectory.Replace('\\', '/').StartsWith(Unreal.EngineDirectory.FullName.Replace('\\', '/'), StringComparison.Ordinal);
					case UHTModuleType.EngineRuntime:
					case UHTModuleType.EngineUncooked:
					case UHTModuleType.EngineDeveloper:
					case UHTModuleType.EngineEditor:
					case UHTModuleType.EngineThirdParty:
						return true;
					case UHTModuleType.GameRuntime:
					case UHTModuleType.GameUncooked:
					case UHTModuleType.GameDeveloper:
					case UHTModuleType.GameEditor:
					case UHTModuleType.GameThirdParty:
						return false;
					default:
						throw new UhtIceException("Invalid module type");
				}
			}
		}

		/// <summary>
		/// True if the package is a plugin
		/// </summary>
		[JsonIgnore]
		public bool IsPlugin => Module.BaseDirectory.Replace('\\', '/').Contains("/Plugins/", StringComparison.Ordinal);

		/// <summary>
		/// Information about linking objects between this module and others generated by UhtSession after reference collection. 
		/// </summary>
		public UhtModuleLinking? Linking { get; set; }

		/// <summary>
		/// Create a new module from the given manifest module
		/// </summary>
		/// <param name="session">Running session</param>
		/// <param name="module">Source module from UBT</param>
		public UhtModule(UhtSession session, UHTManifest.Module module)
		{
			Session = session;
			Module = module;
			foreach (string dep in module.ModuleDependencies)
			{
				_ubtModuleDependencies.Add(dep);
			}

			EPackageFlags packageFlags = EPackageFlags.ContainsScript | EPackageFlags.Compiling;

			switch (module.OverrideModuleType)
			{
				case EPackageOverrideType.None:
					switch (module.ModuleType)
					{
						case UHTModuleType.GameEditor:
						case UHTModuleType.EngineEditor:
							packageFlags |= EPackageFlags.EditorOnly;
							break;

						case UHTModuleType.GameDeveloper:
						case UHTModuleType.EngineDeveloper:
							packageFlags |= EPackageFlags.Developer;
							break;

						case UHTModuleType.GameUncooked:
						case UHTModuleType.EngineUncooked:
							packageFlags |= EPackageFlags.UncookedOnly;
							break;
					}
					break;

				case EPackageOverrideType.EditorOnly:
					packageFlags |= EPackageFlags.EditorOnly;
					break;

				case EPackageOverrideType.EngineDeveloper:
				case EPackageOverrideType.GameDeveloper:
					packageFlags |= EPackageFlags.Developer;
					break;

				case EPackageOverrideType.EngineUncookedOnly:
				case EPackageOverrideType.GameUncookedOnly:
					packageFlags |= EPackageFlags.UncookedOnly;
					break;
			}

			string packageName;
			int lastSlashIndex = Module.Name.LastIndexOf('/');
			if (lastSlashIndex == -1)
			{
				packageName = $"/Script/{Module.Name}";
				ShortName = Module.Name;
			}
			else
			{
				packageName = Module.Name;
				ShortName = packageName[(lastSlashIndex + 1)..];
			}

			Api = $"{ShortName.ToUpperInvariant()}_API ";
			NonAttributedApi = $"{ShortName.ToUpperInvariant()}_NON_ATTRIBUTED_API ";

			ScriptPackage = new(this, packageName, packageFlags);

			_inputCache = new(() => UhtInputCache.Read(this, Session.ConfigData.Span));
		}

		/// <summary>
		/// Prepare to parse all the headers in the module.  This creates all the UhtHeaderFiles and validates there
		/// are no conflicts.
		/// </summary>
		/// <param name="addHeaderFileAction">Action to invoke to register the headers with the session</param>
		public void PrepareHeaders(Action<UhtHeaderFile> addHeaderFileAction)
		{
			PrepareHeaders(Module.ClassesHeaders, UhtHeaderFileType.Classes, addHeaderFileAction);
			PrepareHeaders(Module.PublicHeaders, UhtHeaderFileType.Public, addHeaderFileAction);
			PrepareHeaders(Module.InternalHeaders, UhtHeaderFileType.Internal, addHeaderFileAction);
			PrepareHeaders(Module.PrivateHeaders, UhtHeaderFileType.Private, addHeaderFileAction);
		}

		/// <summary>
		/// Create a new package with the given name.  Will return an existing instance if it already exists
		/// </summary>
		/// <param name="packageName">Name of the package</param>
		/// <param name="packageFlags">Flags to set if the package is created</param>
		/// <returns>Package</returns>
		public UhtPackage CreatePackage(string packageName, EPackageFlags packageFlags = EPackageFlags.None)
		{
			if (packageName == ScriptPackage.SourceName)
			{
				return ScriptPackage;
			}
			lock (_packages)
			{
				if (_packages.TryGetValue(packageName, out UhtPackage? package))
				{
					return package;
				}
				package = new(this, packageName, packageFlags);
				_packages.Add(packageName, package);
				return package;
			}
		}

		/// <summary>
		/// Try to fetch a cached version of the file from the input cache
		/// </summary>
		/// <param name="fileName"></param>
		/// <param name="entry"></param>
		/// <returns></returns>
		public bool TryGetCachedInput(string fileName, out UhtInputCacheEntry entry)
		{
			if (!Session.EnableInputCacheRead)
			{
				entry = new();
				return false;
			}

			return _inputCache.Value.TryGetFile(fileName, out entry);
		}

		/// <summary>
		/// Given a collection of headers, create the UhtHeaders
		/// </summary>
		/// <param name="headerFiles">Collection of header names</param>
		/// <param name="headerFileType">Type of the headers</param>
		/// <param name="addHeaderFileAction">Action to take to notify the caller of the header being created</param>
		private void PrepareHeaders(IEnumerable<string> headerFiles, UhtHeaderFileType headerFileType, Action<UhtHeaderFile> addHeaderFileAction)
		{
			string typeDirectory = headerFileType.ToString() + '/';
			string normalizedModuleBaseFullFilePath = GetNormalizedFullFilePath(Module.BaseDirectory);
			foreach (string headerFilePath in headerFiles)
			{

				// Make sure this isn't a duplicate
				string normalizedFullFilePath = GetNormalizedFullFilePath(headerFilePath);
				string fileName = Path.GetFileName(normalizedFullFilePath);
				UhtHeaderFile? existingHeaderFile = Session.FindHeaderFile(fileName);
				if (existingHeaderFile != null)
				{
					string normalizedExistingFullFilePath = GetNormalizedFullFilePath(existingHeaderFile.FilePath);
					if (!String.Equals(normalizedFullFilePath, normalizedExistingFullFilePath, StringComparison.OrdinalIgnoreCase))
					{
						IUhtMessageSite site = (IUhtMessageSite?)Session.ManifestFile ?? Session;
						site.LogError($"Two headers with the same name is not allowed. '{headerFilePath}' conflicts with '{existingHeaderFile.FilePath}'");
						continue;
					}
				}

				// Create the header file and add to the collections
				UhtHeaderFile headerFile = new(this, headerFilePath);
				headerFile.HeaderFileType = headerFileType;
				addHeaderFileAction(headerFile);
				_headers.Add(headerFile);

				// Save metadata for the class path, both for it's include path and relative to the module base directory
				if (normalizedFullFilePath.StartsWith(normalizedModuleBaseFullFilePath, true, null))
				{
					int stripLength = normalizedModuleBaseFullFilePath.Length;
					if (stripLength < normalizedFullFilePath.Length && normalizedFullFilePath[stripLength] == '/')
					{
						++stripLength;
					}

					headerFile.ModuleRelativeFilePath = normalizedFullFilePath[stripLength..];

					if (normalizedFullFilePath[stripLength..].StartsWith(typeDirectory, true, null))
					{
						stripLength += typeDirectory.Length;
					}

					headerFile.IncludeFilePath = normalizedFullFilePath[stripLength..];
				}
			}
		}

		/// <summary>
		/// Return the normalized path converted to a full path if possible. 
		/// Code should NOT depend on a full path being returned.
		/// 
		/// In general, it is assumed that during normal UHT, all paths are already full paths.
		/// Only the test harness deals in relative paths.
		/// </summary>
		/// <param name="filePath">Path to normalize</param>
		/// <returns>Normalized path possibly converted to a full path.</returns>
		private string GetNormalizedFullFilePath(string filePath)
		{
			return Session.FileManager!.GetFullFilePath(filePath).Replace('\\', '/');
		}
	}
}
