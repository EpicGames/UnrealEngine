// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents an element that makes up a namespace.  This includes a previously looked up namespace
	/// </summary>
	/// <param name="Name">Name of the namespace</param>
	/// <param name="Namespace">If already looked up, the namespace object</param>
	public record struct UhtNamespaceElement(StringView Name, UhtNamespace? Namespace)
	{
	}

	/// <summary>
	/// Represents a C++ namespace
	/// </summary>
	public class UhtNamespace
	{

		/// <summary>
		/// Parent namespace.  If null, then this is the global namespace
		/// </summary>
		public UhtNamespace? Parent { get; init; }

		/// <summary>
		/// Name of this namespace
		/// </summary>
		public string SourceName { get; init; }

		/// <summary>
		/// The full name of the namespace
		/// </summary>
		public string FullSourceName { get; init; }

		/// <summary>
		/// True if this is the global namespace
		/// </summary>
		public bool IsGlobal { get; init; }

		/// <summary>
		/// Construct a namespace instance
		/// </summary>
		/// <param name="parent">Parent namespace</param>
		/// <param name="sourceName">Name (will be empty if global)</param>
		/// <param name="fullSourceName">Full name of the namespace</param>
		/// <param name="isGlobal">True if this is the global namespace</param>
		public UhtNamespace(UhtNamespace? parent, string sourceName, string fullSourceName, bool isGlobal)
		{
			Parent = parent;
			SourceName = sourceName;
			FullSourceName = fullSourceName;
			IsGlobal = isGlobal;
		}

		/// <summary>
		/// Append the body inside the namespace on a single line
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="body">Body of the namespace</param>
		/// <returns>Destination builder</returns>
		public StringBuilder AppendSingleLine(StringBuilder builder, Action<StringBuilder> body)
		{
			ReadOnlySpan<char> namespaceName = FullSourceName.AsSpan();
			if (namespaceName.EndsWith("::", StringComparison.Ordinal))
			{
				namespaceName = namespaceName[..^2];
			}
			if (namespaceName.Length > 0)
			{
				builder.Append($"namespace {namespaceName} {{ ");
			}
			body(builder);
			if (namespaceName.Length > 0)
			{
				builder.Append(" }");
			}
			builder.Append("\r\n");
			return builder;
		}

		/// <summary>
		/// Append the body inside the namespace on multiple lines
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="body">Body of the namespace</param>
		/// <returns>Destination builder</returns>
		public StringBuilder AppendMultipleLines(StringBuilder builder, Action<StringBuilder> body)
		{
			ReadOnlySpan<char> namespaceName = FullSourceName.AsSpan();
			if (namespaceName.EndsWith("::", StringComparison.Ordinal))
			{
				namespaceName = namespaceName[..^2];
			}
			if (namespaceName.Length > 0)
			{
				builder.Append($"namespace {namespaceName}\r\n{{\r\n");
			}
			body(builder);
			if (namespaceName.Length > 0)
			{
				builder.Append($"}} // {namespaceName}\r\n");
			}
			return builder;
		}
	}
}
