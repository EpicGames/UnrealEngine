// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Represents a UObject in the engine
	/// </summary>
	[UhtEngineClass(Name = "Object")]
	public abstract class UhtObject : UhtType
	{

		/// <summary>
		/// Internal object flags.
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EInternalObjectFlags InternalObjectFlags { get; set; } = EInternalObjectFlags.None;

		/// <summary>
		/// Unique index of the object
		/// </summary>
		[JsonIgnore]
		public int ObjectTypeIndex { get; }

		/// <summary>
		/// The alternate object is used by the interface system where the native interface will
		/// update this setting to point to the UInterface derived companion object.
		/// </summary>
		[JsonIgnore]
		public UhtObject? AlternateObject { get; set; } = null;

		/// <summary>
		/// Linkable top-level objects per packages should be in a specific order to simplify codegen and runtime initialization
		/// </summary>
		public int LinkOrder => this switch
		{
			UhtClass => 0,
			UhtScriptStruct => 1,
			UhtEnum => 2,
			_ => 3,
		};

		/// <inheritdoc/>
		public override string EngineClassName => "Object";

		/// <summary>
		/// Construct a new instance of the object
		/// </summary>
		/// <param name="module">Owning module</param>
		protected UhtObject(UhtModule module) : base(module)
		{
			ObjectTypeIndex = Session.GetNextObjectTypeIndex();
		}

		/// <summary>
		/// Construct a new instance of the object
		/// </summary>
		/// <param name="headerFile">Header file being compiled</param>
		/// <param name="outer">Outer object</param>
		/// <param name="lineNumber">Line number where object is defined</param>
		protected UhtObject(UhtHeaderFile headerFile, UhtType outer, int lineNumber) : base(headerFile, outer, lineNumber)
		{
			ObjectTypeIndex = Session.GetNextObjectTypeIndex();
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtObject(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ObjectTypeIndex = Session.GetNextObjectTypeIndex();
		}

		/// <summary>
		/// Write the type to the archive writer
		/// </summary>
		/// <param name="writer">Destination cache writer</param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
		}
	}
}
