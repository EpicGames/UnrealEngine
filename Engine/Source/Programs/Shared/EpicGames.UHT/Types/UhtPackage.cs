// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Tables;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents a UPackage in the engine
	/// </summary>
	[UhtEngineClass(Name = "Package")]
	public class UhtPackage : UhtObject
	{

		/// <summary>
		/// Unique index of the package
		/// </summary>
		[JsonIgnore]
		public int PackageTypeIndex { get; }

		/// <summary>
		/// Engine package flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EPackageFlags PackageFlags { get; set; } = EPackageFlags.None;

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType => UhtEngineType.Package;

		/// <inheritdoc/>
		public override string EngineClassName => "Package";

		/// <inheritdoc/>
		public override UhtClass? EngineClass => Session.UPackage;

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtPackage Package => this;

		/// <summary>
		/// Construct a new instance of a package
		/// </summary>
		/// <param name="module">Source module of the package</param>
		/// <param name="packageName">Name of the package</param>
		/// <param name="packageFlags">Assorted package flags</param>
		public UhtPackage(UhtModule module, string packageName, EPackageFlags packageFlags) : base(module)
		{
			PackageFlags = packageFlags;
			PackageTypeIndex = Session.GetNextPackageTypeIndex();
			SourceName = packageName;
		}

		/// <inheritdoc/>
		public override void AddChild(UhtType type)
		{

			// Package children are initially added to the header and then added to the package when we aren't going wide
			type.HeaderFile.AddChild(type);
		}
	}
}
