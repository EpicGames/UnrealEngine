// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Represents an partial - a struct that extends a UObject-derived class with additional properties.
	/// Partials are similar to USTRUCTs but their properties are merged directly into the owner class.
	/// </summary>
	public class UhtPartial : UhtStruct
	{
		private List<UhtDeclaration>? _declarations = null;

		/// <summary>
		/// Name of the class that this partial extends (e.g., "APlayerController")
		/// </summary>
		public string OwnerClassName { get; set; } = String.Empty;

		/// <summary>
		/// Reference to the partial class (resolved during resolution phase)
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtClass>))]
		public UhtClass? OwnerClass { get; set; } = null;

		/// <summary>
		/// Line number where GENERATED_BODY macro was found
		/// </summary>
		public int MacroDeclaredLineNumber { get; set; } = -1;

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType => UhtEngineType.ScriptStruct;

		/// <inheritdoc/>
		public override string EngineClassName => "Partial";

		/// <inheritdoc/>
		public override string EngineLinkClassName => "Partial";

		/// <inheritdoc/>
		public override UhtClass? EngineClass => null; // Partials don't have a UClass equivalent

		/// <inheritdoc/>
		[JsonIgnore]
		public override string EngineNamePrefix => "F";

		/// <summary>
		/// Namespace export name for partials
		/// </summary>
		public string NamespaceExportName => Namespace.FullSourceName;

		/// <summary>
		/// Collection of functions and other declarations found in the partial
		/// </summary>
		[JsonIgnore]
		public IList<UhtDeclaration>? Declarations => _declarations;

		/// <summary>
		/// C++ code has BeginDestroy declared
		/// </summary>
		public bool HasBeginDestroy { get; set; }

		/// <summary>
		/// After functions are fully resolved, points to the first non-delegate function in this class
		/// </summary>
		public UhtDefineScopeLink<UhtFunction>? FirstFunction { get; set; }

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable => Session.GetSpecifierValidatorTable(UhtTableNames.ScriptStruct);

		/// <summary>
		/// Construct a new partial
		/// </summary>
		/// <param name="headerFile">Header being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">Outer type</param>
		/// <param name="lineNumber">Line number of the definition</param>
		public UhtPartial(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtPartial(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			_declarations = reader.ReadOptionalList<UhtDeclaration>((reader) => new(reader));
			OwnerClassName = reader.ReadString();
			OwnerClass = reader.ReadType() as UhtClass;
			MacroDeclaredLineNumber = reader.ReadInt32();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteOptionalVariableLengthArray(_declarations, (writer, x) => { x.Write(writer); });
			writer.WriteString(OwnerClassName);
			writer.WriteType(OwnerClass);
			writer.WriteInt32(MacroDeclaredLineNumber);
		}

		#region Resolution support

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase resolvePhase)
		{
			bool result = base.ResolveSelf(resolvePhase);

			switch (resolvePhase)
			{
				case UhtResolvePhase.Bases:
					// Resolve the owner class reference
					if (!String.IsNullOrEmpty(OwnerClassName))
					{
						OwnerClass = Session.FindType(null, UhtFindOptions.SourceName | UhtFindOptions.Class, OwnerClassName) as UhtClass;
						if (OwnerClass == null)
						{
							this.LogError($"Partial '{SourceName}' extends unknown class '{OwnerClassName}'");
						}
						else if (!OwnerClass.ClassFlags.HasAnyFlags(EClassFlags.Native))
						{
							this.LogError($"Partial '{SourceName}' can only extend native classes ('{OwnerClassName}' is not native)");
						}
					}
					break;

				case UhtResolvePhase.Properties:
				// Resolve property types (convert UhtPreResolveProperty to actual property types)
				UhtPropertyParser.ResolveChildren(this, UhtPropertyParseOptions.AddModuleRelativePath);

					// Validate that partial properties are compatible with the Owner class
					if (OwnerClass != null)
					{
						// Partials cannot have base structs
						if (Super != null)
						{
							this.LogError($"Partial '{SourceName}' cannot have a base struct (partials cannot inherit from other partials or structs)");
						}

						// Validate that all properties are compatible
						foreach (UhtProperty property in Properties)
						{
							// Properties must have default constructors since they're constructed via placement new
							if (property.PropertyCategory == UhtPropertyCategory.RegularParameter)
							{
								this.LogError($"Partial '{SourceName}' property '{property.SourceName}' cannot be a parameter type");
							}
						}
					}
					break;
			}

			return result;
		}

		/// <inheritdoc/>
		protected override void ResolveChildren(UhtResolvePhase phase)
		{
			base.ResolveChildren(phase);

			switch (phase)
			{
				case UhtResolvePhase.Final:
					LinkFunctions();
					break;
			}
		}

		/// <summary>
		/// Create a linked list of functions across preprocessor definition changes
		/// </summary>
		protected void LinkFunctions()
		{
			List<UhtFunction> functions = [];
			foreach (UhtType type in Children)
			{
				// Delegates are not linked into class's field list
				if (type is UhtFunction function && !function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
				{
					functions.Add(function);
				}
			}
			functions.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.EngineName, y.EngineName));

			using UhtDefineScopeListBuilder<UhtFunction> functionList = new(DefineScope);
			foreach (UhtFunction function in functions)
			{
				if (function.NextFunction is not null)
				{
					throw new Exception("Unexpectedly initialized next link on property");
				}
				function.NextFunction = functionList.Add(function);
			}
			FirstFunction = functionList.Head;
		}

		#endregion

		#region Forward declaration support

		/// <inheritdoc/>
		public override StringBuilder AppendForwardDeclaration(StringBuilder builder)
		{
			return builder.Append($"struct {SourceName};");
		}

		#endregion

		#region Reference collection support

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			collector.AddExportType(this);
			// Note: Partials don't add declarations or object references for themselves
			// because they're plain structs, not UObjects with Z_Construct functions
			collector.AddForwardDeclaration(OwnerClass!);
			collector.AddObjectReference(OwnerClass, UhtSingletonType.Registered);

			foreach (UhtType child in Children)
			{
				child.CollectReferences(collector);
			}
		}

		#endregion

		#region Declaration support

		/// <summary>
		/// Add a declaration to the partial (for functions and other declarations)
		/// </summary>
		/// <param name="compilerDirectives">Currently active compiler directives</param>
		/// <param name="tokens">List of declaration tokens</param>
		/// <param name="function">If parsed as part of a UFUNCTION, this will reference it</param>
		public void AddDeclaration(UhtCompilerDirective compilerDirectives, List<UhtToken> tokens, UhtFunction? function)
		{
			_declarations ??= [];
			_declarations.Add(new UhtDeclaration { CompilerDirectives = compilerDirectives, Tokens = [.. tokens], Function = function });
		}

		#endregion

		#region Validation support

		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);

			// Validate that GENERATED_BODY was found
			if (MacroDeclaredLineNumber == -1)
			{
				this.LogError("Expected a GENERATED_BODY() at the start of the partial");
			}

			// Validate owner class is specified
			if (String.IsNullOrEmpty(OwnerClassName))
			{
				this.LogError("Partial must specify the class it extends via UPARTIAL(ClassName) macro");
			}

			return options;
		}

		#endregion
	}
}
