// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Collection of UHT only flags associated with properties
	/// </summary>
	[Flags]
	public enum UhtPropertyExportFlags
	{
		/// <summary>
		/// Property should be exported as public
		/// </summary>
		Public = 1 << 0,

		/// <summary>
		/// Property should be exported as private
		/// </summary>
		Private = 1 << 1,

		/// <summary>
		/// Property should be exported as protected
		/// </summary>
		Protected = 1 << 2,

		/// <summary>
		/// The BlueprintPure flag was set in software and should not be considered an error
		/// </summary>
		ImpliedBlueprintPure = 1 << 3,

		/// <summary>
		/// If true, the property has a getter function
		/// </summary>
		GetterSpecified = 1 << 4,

		/// <summary>
		/// If true, the property has a setter function
		/// </summary>
		SetterSpecified = 1 << 5,

		/// <summary>
		/// If true, the getter has been disabled in the specifiers
		/// </summary>
		GetterSpecifiedNone = 1 << 6,

		/// <summary>
		/// If true, the setter has been disabled in the specifiers
		/// </summary>
		SetterSpecifiedNone = 1 << 7,

		/// <summary>
		/// If true, the getter function was found
		/// </summary>
		GetterFound = 1 << 8,

		/// <summary>
		/// If true, the property has a setter function
		/// </summary>
		SetterFound = 1 << 9,

		/// <summary>
		/// Property is marked as a field notify
		/// </summary>
		FieldNotify = 1 << 10,

		/// <summary>
		/// If true, the property should have a generated getter function
		/// </summary>
		GetterSpecifiedAuto = 1 << 11,

		/// <summary>
		/// If true, the property should have a generated setter function
		/// </summary>
		SetterSpecifiedAuto = 1 << 12,

		/// <summary>
		/// If true, the type is a TVal
		/// </summary>
		TVal = 1 << 13,

		/// <summary>
		/// If true, the type is a TPtr
		/// </summary>
		TPtr = 1 << 14,

		/// <summary>
		/// If true, the type is a TVerseTask
		/// </summary>
		TVerseTask = 1 << 15,

		/// <summary>
		/// If true, the type is a TVerseCall
		/// </summary>
		TVerseCall = 1 << 16,

		/// <summary>
		/// If true, the type is a TNonNullPtr 
		/// </summary>
		TNonNullPtr = 1 << 17,

		/// <summary>
		/// If true, this function argument is named
		/// </summary>
		VerseNamed = 1 << 18,

		/// <summary>
		/// If true, this function argument has a default value
		/// </summary>
		VerseDefaultValue = 1 << 19,

		/// <summary>
		/// This property is wrapped by UE_VERSE_FIELD
		/// </summary>
		VerseField = 1 << 20,
	};

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtPropertyExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtPropertyExportFlags inFlags, UhtPropertyExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtPropertyExportFlags inFlags, UhtPropertyExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtPropertyExportFlags inFlags, UhtPropertyExportFlags testFlags, UhtPropertyExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// The context of the property.
	/// </summary>
	public enum UhtPropertyCategory
	{
		/// <summary>
		/// Function parameter for a function that isn't marked as NET
		/// </summary>
		RegularParameter,

		/// <summary>
		/// Function parameter for a function that is marked as NET
		/// </summary>
		ReplicatedParameter,

		/// <summary>
		/// Function return value
		/// </summary>
		Return,

		/// <summary>
		/// Class or a script structure member property
		/// </summary>
		Member,
	};

	/// <summary>
	/// Helper methods for the property category
	/// </summary>
	public static class UhtPropertyCategoryExtensions
	{

		/// <summary>
		/// Return the hint text for the property category
		/// </summary>
		/// <param name="propertyCategory">Property category</param>
		/// <returns>The user facing hint text</returns>
		/// <exception cref="UhtIceException">Unexpected category</exception>
		public static string GetHintText(this UhtPropertyCategory propertyCategory)
		{
			switch (propertyCategory)
			{
				case UhtPropertyCategory.ReplicatedParameter:
				case UhtPropertyCategory.RegularParameter:
					return "Function parameter";

				case UhtPropertyCategory.Return:
					return "Function return type";

				case UhtPropertyCategory.Member:
					return "Member variable declaration";

				default:
					throw new UhtIceException("Unknown variable category");
			}
		}
	}

	/// <summary>
	/// Allocator used for container
	/// </summary>
	public enum UhtPropertyAllocator
	{
		/// <summary>
		/// Default allocator
		/// </summary>
		Default,

		/// <summary>
		/// Memory image allocator
		/// </summary>
		MemoryImage
	};

	/// <summary>
	/// Type of pointer
	/// </summary>
	public enum UhtPointerType
	{

		/// <summary>
		/// No pointer specified
		/// </summary>
		None,

		/// <summary>
		/// Native pointer specified
		/// </summary>
		Native,
	}

	/// <summary>
	/// Type of reference
	/// </summary>
	public enum UhtPropertyRefQualifier
	{

		/// <summary>
		/// Property is not a reference
		/// </summary>
		None,

		/// <summary>
		/// Property is a const reference
		/// </summary>
		ConstRef,

		/// <summary>
		/// Property is a non-const reference
		/// </summary>
		NonConstRef,
	};

	/// <summary>
	/// Options that customize the properties.
	/// </summary>
	[Flags]
	public enum UhtPropertyOptions
	{

		/// <summary>
		/// No property options
		/// </summary>
		None = 0,

		/// <summary>
		/// Don't automatically mark properties as CPF_Const
		/// </summary>
		NoAutoConst = 1 << 0,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtPropertyOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtPropertyOptions inFlags, UhtPropertyOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtPropertyOptions inFlags, UhtPropertyOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtPropertyOptions inFlags, UhtPropertyOptions testFlags, UhtPropertyOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Property capabilities.  Use the caps system instead of patterns such as "Property is UhtObjectProperty"
	/// </summary>
	[Flags]
	public enum UhtPropertyCaps
	{
		/// <summary>
		/// No property caps
		/// </summary>
		None = 0,

		/// <summary>
		/// If true, the property will be passed by reference when generating the full type string
		/// </summary>
		PassCppArgsByRef = 1 << 0,

		/// <summary>
		/// If true, the an argument will need to be added to the constructor
		/// </summary>
		RequiresNullConstructorArg = 1 << 1,

		/// <summary>
		/// If true, the property type can be TArray or TMap value
		/// </summary>
		CanBeContainerValue = 1 << 2,

		/// <summary>
		/// If true, the property type can be a TSet or TMap key
		/// </summary>
		CanBeContainerKey = 1 << 3,

		/// <summary>
		/// True if the property can be instanced
		/// </summary>
		CanBeInstanced = 1 << 4,

		/// <summary>
		/// True if the property can be exposed on spawn
		/// </summary>
		CanExposeOnSpawn = 1 << 5,

		/// <summary>
		/// True if the property can have a config setting
		/// </summary>
		CanHaveConfig = 1 << 6,

		/// <summary>
		/// True if the property allows the BlueprintAssignable flag.
		/// </summary>
		CanBeBlueprintAssignable = 1 << 7,

		/// <summary>
		/// True if the property allows the BlueprintCallable flag.
		/// </summary>
		CanBeBlueprintCallable = 1 << 8,

		/// <summary>
		/// True if the property allows the BlueprintAuthorityOnly flag.
		/// </summary>
		CanBeBlueprintAuthorityOnly = 1 << 9,

		/// <summary>
		/// True to see if the function parameter property is supported by blueprint
		/// </summary>
		IsParameterSupportedByBlueprint = 1 << 10,

		/// <summary>
		/// True to see if the member property is supported by blueprint
		/// </summary>
		IsMemberSupportedByBlueprint = 1 << 11,

		/// <summary>
		/// True if the property supports RigVM
		/// </summary>
		SupportsRigVM = 1 << 12,

		/// <summary>
		/// True if the property should codegen as an enumeration
		/// </summary>
		IsRigVMEnum = 1 << 13,

		/// <summary>
		/// True if the property should codegen as an array
		/// </summary>
		IsRigVMArray = 1 << 14,

		/// <summary>
		/// True if the property should codegen as a byte enumeration
		/// </summary>
		IsRigVMEnumAsByte = 1 << 15,

		/// <summary>
		/// If true, the property type can be TOptional value
		/// </summary>
		CanBeOptionalValue = 1 << 16,

		/// <summary>
		/// If true, the property type can be used in conjunction with verse
		/// </summary>
		SupportsVerse = 1 << 17,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtPropertyCapsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtPropertyCaps inFlags, UhtPropertyCaps testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtPropertyCaps inFlags, UhtPropertyCaps testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtPropertyCaps inFlags, UhtPropertyCaps testFlags, UhtPropertyCaps matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Text can be formatted in different context with slightly different results
	/// </summary>
	public enum UhtPropertyTextType
	{
		/// <summary>
		/// Generic type
		/// </summary>
		Generic,

		/// <summary>
		/// Generic function argument or return value
		/// </summary>
		GenericFunctionArgOrRetVal,

		/// <summary>
		/// Generic function argument or return value implementation (specific to booleans and will always return "bool")
		/// </summary>
		GenericFunctionArgOrRetValImpl,

		/// <summary>
		/// Class function argument or return value
		/// </summary>
		ClassFunctionArgOrRetVal,

		/// <summary>
		/// Event function argument or return value
		/// </summary>
		EventFunctionArgOrRetVal,

		/// <summary>
		/// Interface function argument or return value
		/// </summary>
		InterfaceFunctionArgOrRetVal,

		/// <summary>
		/// Sparse property declaration
		/// </summary>
		Sparse,

		/// <summary>
		/// Sparse property short name
		/// </summary>
		SparseShort,

		/// <summary>
		/// Class or structure member
		/// </summary>
		ExportMember,

		/// <summary>
		/// Members of the event parameters structure used by code generation to invoke events
		/// </summary>
		EventParameterMember,

		/// <summary>
		/// Members of the event parameters structure used by code generation to invoke functions
		/// </summary>
		EventParameterFunctionMember,

		/// <summary>
		/// Instance of the property is being constructed
		/// </summary>
		Construction,

		/// <summary>
		/// Used to get the type argument for a function thunk.  This is used for P_GET_ARRAY_*
		/// </summary>
		FunctionThunkParameterArrayType,

		/// <summary>
		/// If the P_GET macro requires an argument, this is used to fetch that argument
		/// </summary>
		FunctionThunkParameterArgType,

		/// <summary>
		/// Used to get the return type for function thunks
		/// </summary>
		FunctionThunkRetVal,

		/// <summary>
		/// Basic RigVM type
		/// </summary>
		RigVMType,

		/// <summary>
		/// Type expected in a getter/setter argument list
		/// </summary>
		GetterSetterArg,

		/// <summary>
		/// Type expected from a getter
		/// </summary>
		GetterRetVal,

		/// <summary>
		/// Type expected as a setter argument
		/// </summary>
		SetterParameterArgType,

		/// <summary>
		/// Used when mangling a function
		/// </summary>
		VerseMangledType,
	}

	/// <summary>
	/// Extension methods for the property text type
	/// </summary>
	public static class UhtPropertyTextTypeExtensions
	{

		/// <summary>
		/// Test to see if the text type is for a function
		/// </summary>
		/// <param name="textType">Type of text</param>
		/// <returns>True if the text type is a function</returns>
		public static bool IsParameter(this UhtPropertyTextType textType)
		{
			return
				textType == UhtPropertyTextType.GenericFunctionArgOrRetVal ||
				textType == UhtPropertyTextType.GenericFunctionArgOrRetValImpl ||
				textType == UhtPropertyTextType.ClassFunctionArgOrRetVal ||
				textType == UhtPropertyTextType.EventFunctionArgOrRetVal ||
				textType == UhtPropertyTextType.InterfaceFunctionArgOrRetVal ||
				textType == UhtPropertyTextType.SetterParameterArgType;
		}

		/// <summary>
		/// Test to see if the text type is for a getter retVal or setter argType
		/// </summary>
		/// <param name="textType">Type of text</param>
		/// <returns>True if the text type is a retVal or argType</returns>
		public static bool IsGetOrSet(this UhtPropertyTextType textType)
		{
			return
				textType == UhtPropertyTextType.GetterSetterArg ||
				textType == UhtPropertyTextType.GetterRetVal ||
				textType == UhtPropertyTextType.SetterParameterArgType;
		}
	}

	//ETSTODO - This can be removed since FunctionThunkParameterArgType is only used in conjunction with UhtPGetArgumentType
	/// <summary>
	/// Specifies how the PGet argument type is to be formatted
	/// </summary>
	public enum UhtPGetArgumentType
	{
		/// <summary>
		/// </summary>
		None,

		/// <summary>
		/// </summary>
		EngineClass,

		/// <summary>
		/// </summary>
		TypeText,
	}

	/// <summary>
	/// Property member context provides extra context when formatting the member declaration and definitions.
	/// </summary>
	public interface IUhtPropertyMemberContext : IUhtObjectLinker
	{
		/// <summary>
		/// The outer/owning structure
		/// </summary>
		public UhtStruct OuterStruct { get; }

		/// <summary>
		/// The OuterStruct's identifier
		/// </summary>
		public UhtCppIdentifier OuterIdentifier { get; }

		/// <summary>
		/// If true, we are generating the legacy version of the property
		/// </summary>
		public bool IsLegacy { get; }

		/// <summary>
		/// Return the hash code for a given object
		/// </summary>
		/// <param name="obj">Object in question</param>
		/// <returns>Hash code</returns>
		public IoHash GetTypeHash(UhtObject obj);
	}

	/// <summary>
	/// Information about a child property mostly intended for code generation
	/// </summary>
	/// <param name="Property">Child property</param>
	/// <param name="Identifier">Code generation identifier</param>
	/// <param name="Offset">Offset of the element inside of the containing property</param>
	public record struct UhtChildProperty(UhtProperty Property, UhtCppIdentifier Identifier, string Offset = "0");

	/// <summary>
	/// Property settings is a transient object used during the parsing of properties
	/// </summary>
	[DebuggerDisplay("Type = {TypeTokens}")]
	public class UhtPropertySettings
	{

		/// <summary>
		/// For nested properties, this points to the parent
		/// </summary>
		public UhtPropertySettings? ParentSettings { get; set; } = null;

		/// <summary>
		/// The root settings for a nested property
		/// </summary>
		public UhtPropertySettings RootSettings
		{
			get
			{
				UhtPropertySettings settings = this;
				while (settings.ParentSettings != null)
				{
					settings = settings.ParentSettings;
				}
				return settings;
			}
		}

		/// <summary>
		/// Source name of the property
		/// </summary>
		public string SourceName { get; set; } = String.Empty;

		/// <summary>
		/// Engine name of the property
		/// </summary>
		public string EngineName { get; set; } = String.Empty;

		/// <summary>
		/// Verse name of the property 
		/// </summary>
		public string VerseName { get; set; } = String.Empty;

		/// <summary>
		/// Property's meta data
		/// </summary>
		public UhtMetaData MetaData { get; set; } = UhtMetaData.Empty;

		/// <summary>
		/// Property outer object
		/// </summary>
		public UhtType Outer { get; set; }

		/// <summary>
		/// Line number of the property declaration
		/// </summary>
		public int LineNumber { get; set; }

		/// <summary>
		/// Property category
		/// </summary>
		public UhtPropertyCategory PropertyCategory { get; set; }

		/// <summary>
		/// Engine property flags
		/// </summary>
		public EPropertyFlags PropertyFlags { get; set; }

		/// <summary>
		/// Property flags not allowed by the context of the property parsing
		/// </summary>
		public EPropertyFlags DisallowPropertyFlags { get; set; }

		/// <summary>
		/// UHT specified property flags
		/// </summary>
		public UhtPropertyExportFlags PropertyExportFlags { get; set; }

		/// <summary>
		/// #define scope of where the property exists
		/// </summary>
		public UhtDefineScope DefineScope { get; set; }

		/// <summary>
		/// Allocator used for containers
		/// </summary>
		public UhtPropertyAllocator Allocator { get; set; }

		/// <summary>
		/// Options for property parsing
		/// </summary>
		public UhtPropertyOptions Options { get; set; }

		/// <summary>
		/// Property pointer type
		/// </summary>
		public UhtPointerType PointerType { get; set; }

		/// <summary>
		/// Replication notify name
		/// </summary>
		public string? RepNotifyName { get; set; }

		/// <summary>
		/// If set, the array size of the property
		/// </summary>
		public string? ArrayDimensions { get; set; }

		/// <summary>
		/// Getter method
		/// </summary>
		public string? Setter { get; set; }

		/// <summary>
		/// Setter method
		/// </summary>
		public string? Getter { get; set; }

		/// <summary>
		/// Default value of the property
		/// </summary>
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "")]
		public List<UhtToken>? DefaultValueTokens { get; set; }

		/// <summary>
		/// Collection of the tokens used to declare the property type.  The type tokens are only valid for
		/// the outermost property.
		/// </summary>
		public UhtTypeTokensRef TypeTokens { get; set; } = new();

		/// <summary>
		/// If true, the property is a bit field
		/// </summary>
		public bool IsBitfield { get; set; }

		/// <summary>
		/// Construct a new, uninitialized version of the property settings
		/// </summary>
#pragma warning disable CS8618 // Non-nullable field must contain a non-null value when exiting constructor. Consider declaring as nullable.
		public UhtPropertySettings()
#pragma warning restore CS8618 // Non-nullable field must contain a non-null value when exiting constructor. Consider declaring as nullable.
		{
		}

		/// <summary>
		/// Construct property settings based on the property settings for a parent container
		/// </summary>
		/// <param name="parentPropertySettings">Parent container property</param>
		/// <param name="sourceName">Name of the property</param>
		/// <param name="messageSite">Message site used to construct meta data object</param>
		public void Reset(UhtPropertySettings parentPropertySettings, string sourceName, IUhtMessageSite messageSite)
		{
			ParentSettings = parentPropertySettings;
			SourceName = sourceName;
			EngineName = sourceName;
			VerseName = String.Empty;
			MetaData = new UhtMetaData(messageSite, parentPropertySettings.Outer.Session.Config);
			Outer = parentPropertySettings.Outer;
			LineNumber = parentPropertySettings.LineNumber;
			PropertyCategory = parentPropertySettings.PropertyCategory;
			PropertyFlags = parentPropertySettings.PropertyFlags;
			DisallowPropertyFlags = parentPropertySettings.DisallowPropertyFlags;
			PropertyExportFlags = UhtPropertyExportFlags.Public;
			DefineScope = parentPropertySettings.DefineScope;
			RepNotifyName = null;
			Allocator = UhtPropertyAllocator.Default;
			Options = parentPropertySettings.Options;
			PointerType = UhtPointerType.None;
			ArrayDimensions = null;
			DefaultValueTokens = null;
			Setter = null;
			Getter = null;
			IsBitfield = false;
			TypeTokens = parentPropertySettings.TypeTokens;
		}

		/// <summary>
		/// Copy property settings
		/// </summary>
		/// <param name="propertySettings">Setting to copy</param>
		public void Copy(UhtPropertySettings propertySettings)
		{
			ParentSettings = propertySettings.ParentSettings;
			SourceName = propertySettings.SourceName;
			EngineName = propertySettings.EngineName;
			VerseName = propertySettings.VerseName;
			MetaData = propertySettings.MetaData;
			Outer = propertySettings.Outer;
			LineNumber = propertySettings.LineNumber;
			PropertyCategory = propertySettings.PropertyCategory;
			PropertyFlags = propertySettings.PropertyFlags;
			DisallowPropertyFlags = propertySettings.DisallowPropertyFlags;
			PropertyExportFlags = propertySettings.PropertyExportFlags;
			DefineScope = propertySettings.DefineScope;
			RepNotifyName = propertySettings.RepNotifyName;
			Allocator = propertySettings.Allocator;
			Options = propertySettings.Options;
			PointerType = propertySettings.PointerType;
			ArrayDimensions = propertySettings.ArrayDimensions;
			DefaultValueTokens = propertySettings.DefaultValueTokens;
			Setter = propertySettings.Setter;
			Getter = propertySettings.Getter;
			IsBitfield = propertySettings.IsBitfield;
			TypeTokens = propertySettings.TypeTokens;
		}

		/// <summary>
		/// Reset the property settings.  Used on a cached property settings object
		/// </summary>
		/// <param name="outer">Outer/owning type</param>
		/// <param name="lineNumber">Line number of property</param>
		/// <param name="propertyCategory">Category of property</param>
		/// <param name="disallowPropertyFlags">Property flags that are not allowed</param>
		public void Reset(UhtType outer, int lineNumber, UhtPropertyCategory propertyCategory, EPropertyFlags disallowPropertyFlags)
		{
			ParentSettings = null;
			SourceName = String.Empty;
			EngineName = String.Empty;
			VerseName = String.Empty;
			MetaData = new UhtMetaData(outer, outer.Session.Config);
			Outer = outer;
			LineNumber = lineNumber;
			PropertyCategory = propertyCategory;
			PropertyFlags = EPropertyFlags.None;
			DisallowPropertyFlags = disallowPropertyFlags;
			DefineScope = UhtDefineScope.None;
			PropertyExportFlags = UhtPropertyExportFlags.Public;
			RepNotifyName = null;
			Allocator = UhtPropertyAllocator.Default;
			PointerType = UhtPointerType.None;
			ArrayDimensions = null;
			DefaultValueTokens = null;
			Setter = null;
			Getter = null;
			IsBitfield = false;
			TypeTokens = new();
		}

		/// <summary>
		/// Reset property settings based on the given property.  Used to prepare a cached property settings for parsing.
		/// </summary>
		/// <param name="property">Source property</param>
		/// <param name="options">Property options</param>
		/// <exception cref="UhtIceException">Thrown if the input property doesn't have an outer</exception>
		public void Reset(UhtProperty property, UhtPropertyOptions options)
		{
			if (property.Outer == null)
			{
				throw new UhtIceException("Property must have an outer specified");
			}
			ParentSettings = null;
			SourceName = property.SourceName;
			EngineName = property.EngineName;
			VerseName = property.VerseNameOverride;
			MetaData = property.MetaData;
			Outer = property.Outer;
			LineNumber = property.LineNumber;
			PropertyCategory = property.PropertyCategory;
			PropertyFlags = property.PropertyFlags;
			DisallowPropertyFlags = property.DisallowPropertyFlags;
			PropertyExportFlags = property.PropertyExportFlags;
			DefineScope = property.DefineScope;
			Allocator = property.Allocator;
			Options = options;
			PointerType = property.PointerType;
			RepNotifyName = property.RepNotifyName;
			ArrayDimensions = property.ArrayDimensions;
			DefaultValueTokens = property.DefaultValueTokens;
			Setter = property.Setter;
			Getter = property.Getter;
			IsBitfield = property.IsBitfield;
			TypeTokens = property.TypeTokens;
		}
	}

	/// <summary>
	/// Represents FProperty fields
	/// </summary>
	[UhtEngineClass(Name = "Field", IsProperty = true)] // This is here just so it is defined
	[UhtEngineClass(Name = "Property", IsProperty = true)]
	[DebuggerDisplay("Name = {SourceName}, Type = {TypeTokens}")]
	public abstract class UhtProperty : UhtType
	{
		#region Constants
		/// <summary>
		/// Collection of recognized casts when parsing array dimensions
		/// </summary>
		private static readonly string[] s_casts =
		[
			"(uint32)",
			"(int32)",
			"(uint16)",
			"(int16)",
			"(uint8)",
			"(int8)",
			"(int)",
			"(unsigned)",
			"(signed)",
			"(unsigned int)",
			"(signed int)",
			"static_cast<uint32>",
			"static_cast<int32>",
			"static_cast<uint16>",
			"static_cast<int16>",
			"static_cast<uint8>",
			"static_cast<int8>",
			"static_cast<int>",
			"static_cast<unsigned>",
			"static_cast<signed>",
			"static_cast<unsigned int>",
			"static_cast<signed int>",
		];

		/// <summary>
		/// Collection of invalid names for parameters 
		/// </summary>
		private static readonly HashSet<string> s_invalidParamNames = new(StringComparer.OrdinalIgnoreCase) { "self" };

		/// <summary>
		/// Prefix used when declaring P_GET_ parameters
		/// </summary>
		protected const string FunctionParameterThunkPrefix = "Z_Param_";
		#endregion

		#region Protected property configuration used to simplify implementation details
		/// <summary>
		/// Simple native CPP type text.  Will not include any template arguments
		/// </summary>
		protected abstract string CppTypeText { get; }

		/// <summary>
		/// P_GET_ macro name
		/// </summary>
		protected abstract string PGetMacroText { get; }

		/// <summary>
		/// If true, then references must be passed without a pointer
		/// </summary>
		protected virtual bool PGetPassAsNoPtr => false;

		/// <summary>
		/// Type of the PGet argument if one is required
		/// </summary>
		protected virtual UhtPGetArgumentType PGetTypeArgument => UhtPGetArgumentType.None;

		/// <summary>
		/// If true, then with the new VM, wrap the type in a rest value
		/// </summary>
		public virtual bool CodeGenWrapInRestValue => PropertyExportFlags.HasFlag(UhtPropertyExportFlags.VerseField);

		/// <summary>
		/// Name of the code generation structure used when doing params code gen
		/// </summary>
		protected virtual string CodeGenParamsStruct => throw new UhtIceException("Property type didn't define a params structure name");

		/// <summary>
		/// Flags to include when generating the params code gen structure
		/// </summary>
		protected virtual string CodeGenParamsFlags => throw new UhtIceException("Property type didn't define a params structure flags");

		/// <summary>
		/// If true, add the offset to the start of the params declaration
		/// </summary>
		protected virtual bool CodeGenParamsAppendOffset => true;
		#endregion

		/// <summary>
		/// If set, the name was set via a UPROPERTY or UPARAM
		/// </summary>
		public string VerseNameOverride { get; set; } = String.Empty;

		/// <summary>
		/// Verse name of the property 
		/// </summary>
		public string VerseName => String.IsNullOrEmpty(VerseNameOverride) ? SourceName : VerseNameOverride;

		/// <summary>
		/// Property category
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPropertyCategory PropertyCategory { get; set; } = UhtPropertyCategory.Member;

		/// <summary>
		/// Engine property flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EPropertyFlags PropertyFlags { get; set; }

		/// <summary>
		/// Capabilities of the property.  Use caps system instead of testing for specific property types.
		/// </summary>
		[JsonIgnore]
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPropertyCaps PropertyCaps { get; set; }

		/// <summary>
		/// Engine flags that are disallowed on this property
		/// </summary>
		[JsonIgnore]
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EPropertyFlags DisallowPropertyFlags { get; set; } = EPropertyFlags.None;

		/// <summary>
		/// UHT specified property flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPropertyExportFlags PropertyExportFlags { get; set; } = UhtPropertyExportFlags.Public;

		/// <summary>
		/// Reference type of the property
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPropertyRefQualifier RefQualifier { get; set; } = UhtPropertyRefQualifier.None;

		/// <summary>
		/// Pointer type of the property
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPointerType PointerType { get; set; } = UhtPointerType.None;

		/// <summary>
		/// Allocator to be used with containers
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtPropertyAllocator Allocator { get; set; } = UhtPropertyAllocator.Default;

		/// <summary>
		/// Collection of tokens that declare the type of the property.
		/// </summary>
		[JsonIgnore]
		public UhtTypeTokensRef TypeTokens { get; set; }

		/// <summary>
		/// Replication notify name
		/// </summary>
		public string? RepNotifyName { get; set; } = null;

		/// <summary>
		/// Fixed array size
		/// </summary>
		public string? ArrayDimensions { get; set; } = null;

		/// <summary>
		/// Property setter
		/// </summary>
		public string? Setter { get; set; } = null;

		/// <summary>
		/// Property getter
		/// </summary>
		public string? Getter { get; set; } = null;

		/// <summary>
		/// Default value of property
		/// </summary>
		[JsonIgnore]
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "<Pending>")]
		public List<UhtToken>? DefaultValueTokens { get; set; } = null;

		/// <summary>
		/// If true, this property is a bit field
		/// </summary>
		public bool IsBitfield { get; set; } = false;

		///<inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType => UhtEngineType.Property;

		///<inheritdoc/>
		[JsonIgnore]
		public override UhtClass? EngineClass => null; // Properties do not have a UObject derived class

		///<inheritdoc/>
		[JsonIgnore]
		public override bool Deprecated => PropertyFlags.HasAnyFlags(EPropertyFlags.Deprecated);

		/// <summary>
		/// Return the engine name without and 'b' prefixes
		/// </summary>
		[JsonIgnore]
		public virtual string StrippedEngineName
		{
			get
			{
				string strippedName = EngineName;
				if (strippedName.StartsWith(UhtNames.VerseNotNativePrefix, StringComparison.Ordinal))
				{
					strippedName = strippedName[(UhtNames.VerseNotNativePrefix.Length)..];
				}
				return strippedName;
			}
		}

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable
		{
			get
			{
				switch (PropertyCategory)
				{
					case UhtPropertyCategory.Member:
						return Session.GetSpecifierValidatorTable(UhtTableNames.PropertyMember);
					case UhtPropertyCategory.RegularParameter:
					case UhtPropertyCategory.ReplicatedParameter:
					case UhtPropertyCategory.Return:
						return Session.GetSpecifierValidatorTable(UhtTableNames.PropertyArgument);
					default:
						throw new UhtIceException("Unknown property category type");
				}
			}
		}

		/// <summary>
		/// If true, the property is a fixed/static array
		/// </summary>
		[JsonIgnore]
		public bool IsStaticArray => !String.IsNullOrEmpty(ArrayDimensions);

		/// <summary>
		/// If true, the property is editor only
		/// </summary>
		[JsonIgnore]
		public bool IsEditorOnlyProperty => PropertyFlags.HasAnyFlags(EPropertyFlags.EditorOnly);

		/// <summary>
		/// After properties are fully resolved, points to the next property in the owning struct in each define scope.
		/// </summary>
		public UhtDefineScopeLink<UhtProperty>? NextProperty { get; set; }

		/// <summary>
		/// Returns whether this property has a setter or getter function and therefor needs to be wrapped in a template property type
		/// </summary>
		public bool HasGetterOrSetter => PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterFound | UhtPropertyExportFlags.GetterFound);

		/// <summary>
		/// Return the root property for the given property.  If the property is outside of a container, 
		/// return the property itself.  Otherwise, return the property associated with the container.
		/// </summary>
		public UhtProperty RootProperty
		{
			get
			{
				UhtProperty root = this;
				while (true)
				{
					UhtProperty? outer = root.Outer as UhtProperty;
					if (outer == null)
					{
						return root;
					}
					root = outer;
				}
			}
		}

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="outer">Outer type of the property</param>
		/// <param name="lineNumber">Line number where property was declared</param>
		protected UhtProperty(UhtType outer, int lineNumber) : base(outer.HeaderFile, outer, lineNumber)
		{
			PropertyFlags = EPropertyFlags.None;
			PropertyCaps = UhtPropertyCaps.CanBeContainerValue | UhtPropertyCaps.CanBeContainerKey | UhtPropertyCaps.CanBeOptionalValue | UhtPropertyCaps.CanHaveConfig;
		}

		/// <summary>
		/// Construct a new property
		/// </summary>
		/// <param name="propertySettings">Property settings from parsing</param>
		protected UhtProperty(UhtPropertySettings propertySettings) : base(propertySettings.Outer.HeaderFile, propertySettings.Outer, propertySettings.LineNumber, propertySettings.MetaData)
		{
			SourceName = propertySettings.SourceName;
			// Engine name defaults to source name.  If it doesn't match what is coming in, then set it.
			if (propertySettings.EngineName.Length > 0 && propertySettings.SourceName != propertySettings.EngineName)
			{
				EngineName = propertySettings.EngineName;
			}
			VerseNameOverride = propertySettings.VerseName;
			PropertyCategory = propertySettings.PropertyCategory;
			PropertyFlags = propertySettings.PropertyFlags;
			DisallowPropertyFlags = propertySettings.DisallowPropertyFlags;
			PropertyExportFlags = propertySettings.PropertyExportFlags;
			DefineScope = propertySettings.DefineScope;
			Allocator = propertySettings.Allocator;
			PointerType = propertySettings.PointerType;
			RepNotifyName = propertySettings.RepNotifyName;
			ArrayDimensions = propertySettings.ArrayDimensions;
			DefaultValueTokens = propertySettings.DefaultValueTokens;
			Getter = propertySettings.Getter;
			Setter = propertySettings.Setter;
			IsBitfield = propertySettings.IsBitfield;
			PropertyCaps = UhtPropertyCaps.CanBeContainerValue | UhtPropertyCaps.CanBeContainerKey | UhtPropertyCaps.CanBeOptionalValue | UhtPropertyCaps.CanHaveConfig;
			TypeTokens = propertySettings.TypeTokens;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtProperty(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			VerseNameOverride = reader.ReadString();
			PropertyCategory = (UhtPropertyCategory)reader.ReadUInt64();
			PropertyFlags = (EPropertyFlags)reader.ReadUInt64();
			PropertyCaps = (UhtPropertyCaps)reader.ReadUInt64();
			DisallowPropertyFlags = (EPropertyFlags)reader.ReadUInt64();
			PropertyExportFlags = (UhtPropertyExportFlags)reader.ReadUInt64();
			Allocator = (UhtPropertyAllocator)reader.ReadUInt64();
			RefQualifier = (UhtPropertyRefQualifier)reader.ReadUInt64();
			PointerType = (UhtPointerType)reader.ReadUInt64();
			RepNotifyName = reader.ReadOptionalString();
			ArrayDimensions = reader.ReadOptionalString();
			Setter = reader.ReadOptionalString();
			Getter = reader.ReadOptionalString();
			DefaultValueTokens = reader.ReadOptionalTokenList();
			IsBitfield = reader.ReadBoolean();

			if (Outer is UhtProperty outerProperty)
			{
				TypeTokens = new(outerProperty.TypeTokens.AllTokens, outerProperty.TypeTokens.Segments, reader);
			}
			else
			{
				UhtToken[] allTokens = reader.ReadTokenArray()!;
				UhtTypeSegment[] segments = reader.ReadVariableLengthArray((reader) => new UhtTypeSegment(reader))!;
				TypeTokens = new(allTokens, segments, reader);
			}
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteString(VerseNameOverride);
			writer.WriteUInt64((ulong)PropertyCategory);
			writer.WriteUInt64((ulong)PropertyFlags);
			writer.WriteUInt64((ulong)PropertyCaps);
			writer.WriteUInt64((ulong)DisallowPropertyFlags);
			writer.WriteUInt64((ulong)PropertyExportFlags);
			writer.WriteUInt64((ulong)Allocator);
			writer.WriteUInt64((ulong)RefQualifier);
			writer.WriteUInt64((ulong)PointerType);
			writer.WriteOptionalString(RepNotifyName);
			writer.WriteOptionalString(ArrayDimensions);
			writer.WriteOptionalString(Setter);
			writer.WriteOptionalString(Getter);
			writer.WriteOptionalVariableLengthTokenArray(DefaultValueTokens);
			writer.WriteBoolean(IsBitfield);

			if (Outer is not UhtProperty)
			{
				writer.WriteVariableLengthTokenArray(TypeTokens.AllTokens.Span);
				writer.WriteVariableLengthArray(TypeTokens.Segments.Span, (writer, x) => x.Write(writer));
			}
			TypeTokens.Write(writer);
		}

		#region Text and code generation support
		/// <summary>
		/// Internal version of AppendText.  Don't append any text to the builder to get the default behavior
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="textType">Text type of where the property is being referenced</param>
		/// <param name="isTemplateArgument">If true, this property is a template arguments</param>
		/// <returns>Destination builder</returns>
		public virtual StringBuilder AppendText(StringBuilder builder, UhtPropertyTextType textType, bool isTemplateArgument = false)
		{
			// By default, we assume it will be just the simple text
			return builder.Append(CppTypeText);
		}

		/// <summary>
		/// Append the full declaration including such things as property name and const<amp/> requirements
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="textType">Text type of where the property is being referenced</param>
		/// <param name="skipParameterName">If true, do not include the property name</param>
		/// <returns>Destination builder</returns>
		public virtual StringBuilder AppendFullDecl(StringBuilder builder, UhtPropertyTextType textType, bool skipParameterName = false)
		{
			UhtPropertyCaps caps = PropertyCaps;

			bool isParameter = textType.IsParameter();
			bool isGetOrSet = textType.IsGetOrSet();
			bool isInterfaceProp = this is UhtInterfaceProperty;

			// When do we need a leading const:
			// 1) If this is a object or object ptr property and the referenced class is const
			// 2) If this is not an out parameter or reference parameter then,
			//		if this is a parameter
			//		AND - if this is a const param OR (if this is an interface property and not an out param)
			// 3) If this is a parameter without array dimensions, must be passed by reference, but not an out parameter or const param
			bool passCppArgsByRef = caps.HasAnyFlags(UhtPropertyCaps.PassCppArgsByRef);
			bool isConstParam = isParameter && (PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm) || (isInterfaceProp && !PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm)));
			bool isConstArgsByRef = isParameter && ArrayDimensions == null && passCppArgsByRef && !PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm | EPropertyFlags.OutParm);
			bool isOnConstClass = false;
			if (this is UhtObjectProperty objectProperty)
			{
				isOnConstClass = objectProperty.Class.ClassFlags.HasAnyFlags(EClassFlags.Const);
			}
			bool shouldHaveRef = PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm | EPropertyFlags.ReferenceParm);

			bool constAtTheBeginning = isGetOrSet | isOnConstClass || isConstArgsByRef || (isConstParam && !shouldHaveRef);
			if (constAtTheBeginning)
			{
				builder.Append("const ");
			}

			AppendText(builder, textType);

			bool fromConstClass = false;
			if (textType == UhtPropertyTextType.ExportMember && Outer != null)
			{
				if (Outer is UhtClass outerClass)
				{
					fromConstClass = outerClass.ClassFlags.HasAnyFlags(EClassFlags.Const);
				}
			}
			bool constAtTheEnd = fromConstClass || (isConstParam && shouldHaveRef);
			if (constAtTheEnd)
			{
				builder.Append(" const");
			}

			shouldHaveRef = textType == UhtPropertyTextType.SetterParameterArgType || (textType == UhtPropertyTextType.GetterRetVal && passCppArgsByRef);
			if (shouldHaveRef || isParameter && ArrayDimensions == null && (passCppArgsByRef || PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm | EPropertyFlags.ReferenceParm)))
			{
				builder.Append('&');
			}

			builder.Append(' ');
			if (!skipParameterName)
			{
				builder.Append(SourceName);
			}

			if (ArrayDimensions != null)
			{
				builder.Append('[').Append(ArrayDimensions).Append(']');
			}
			return builder;
		}

		/// <summary>
		/// Specifies the order that the child properties are to be returned
		/// </summary>
		public enum UhtChildPropertyOrder
		{
			/// <summary>
			/// Ordered from smallest to largest offset
			/// </summary>
			Forward,

			/// <summary>
			/// Ordered from largest to smallest offset
			/// </summary>
			Reverse,
		}

		//
		// Working with property declaration and definition support:
		//
		// UHT currently supports two forms of property definitions, params and const init.  Params is the traditional style used by UE
		// where new properties are generated from the parameter structures.  Const init is a memory savings system for targets such as 
		// clients and servers where the output it the actual property definitions and not used to create the properties.
		//
		// Property code gen tends to be VERY boiler plate.  
		//
		// Params declarations need three properties defined on the property type:
		//
		//		CodeGenParamsStruct - Name of the C++ params structure
		//		CodeGenParamsFlags - Flags setting in the params structure
		//		CodeGenParamsAppendOffset - If the offset should automatically be populated.  True for everything except boolean properties.
		//
		// Child properties can be automatically handled by implementing the EnumerateChildProperties method.
		//
		// Code generation is broken into parts.  Each of the routines in question can usually be overridden
		//
		//	AppendMetaData - Appends all the metadata definitions
		//
		//	Append[Params|ConstInit]Decl - Appends all the declarations inside of the _Statics section.
		//		AppendCommonDecl - Appends any declarations common to both params and const init.
		//		Append[Params|ConstInit]Decl - Invoked for each child property
		//		Code - Emit the declaration for the given property
		//
		//	Append[Params|ConstInit]Def - Appends all the definitions (i.e. value initialization) outside of the _Statics section.
		//		AppendCommonDef - Appends any definitions common to both params and const init.
		//		Append[Params|ConstInit]Def - Invoked for each child property
		//		Append[Params|ConstInit]DefImpl - Implementation of just the given properties definition
		// 			Append[Params|ConstInit]DefStart - Start the property definition
		//			Append[Params|ConstInit]DefExtra - Any extra value initialization in the definition.  For const init, this also contains the child property pointers
		//			Append[Params|ConstInit]DefEnd - End the property definition
		//
		//	AppendParamsPtr - Appends pointers to the property declarations.  Children are appended first.
		//		When creating the property list at runtime, the list is iterated in reverse and for each, it knows how many child properties are expected
		//		and can decrement the pointer as required.
		//

		/// <summary>
		/// Return a collection of child properties
		/// </summary>
		/// <param name="identifier">Identifier for this property</param>
		/// <param name="order">The order that the properties are to be returned</param>
		/// <returns></returns>
		public virtual IEnumerable<UhtChildProperty> EnumerateChildProperties(UhtCppIdentifier identifier, UhtChildPropertyOrder order)
		{
			return [];
		}

		#region Metadata support
		/// <summary>
		/// Append the required code to declare the properties meta data
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Current context</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendMetaData(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{
				builder.AppendMetaData(child.Property, child.Identifier, tabs);
			}
			return builder.AppendMetaData(this, identifier, tabs);
		}
		#endregion

		#region Params code gen support

		/// <summary>
		/// Append any code common to all code gen styles into the statics declaration
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Current context</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendCommonDecl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return builder;
		}

		/// <summary>
		/// Append any code common to all code gen styles outside of the statics definition
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Current context</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendCommonDef(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return builder;
		}

		/// <summary>
		/// Append the required code to declare the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Current context</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendParamsDecl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			AppendCommonDecl(builder, context, identifier, tabs);
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{
				child.Property.AppendParamsDecl(builder, context, child.Identifier, tabs);
			}
			return AppendParamsDecl(builder, identifier, tabs, CodeGenParamsStruct);
		}

		/// <summary>
		/// Append the required code to declare the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <param name="paramsStructName">Structure name</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendParamsDecl(StringBuilder builder, UhtCppIdentifier identifier, int tabs, string paramsStructName)
		{
			builder.AppendTabs(tabs).Append($"static const UECodeGen_Private::{paramsStructName} {identifier};\r\n");
			return builder;
		}

		/// <summary>
		/// Append the required code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendParamsDef(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, string? offset, int tabs)
		{
			AppendCommonDef(builder, context, identifier, tabs);
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{
				child.Property.AppendParamsDef(builder, context, child.Identifier, child.Offset, tabs);
			}
			return AppendParamsDefImpl(builder, context, identifier, offset, tabs);
		}

		/// <summary>
		/// Append the required code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendParamsDefImpl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, string? offset, int tabs)
		{
			AppendParamsDefStart(builder, this, context, identifier, offset, tabs, CodeGenParamsStruct, CodeGenParamsFlags, CodeGenParamsAppendOffset);
			AppendParamsDefExtra(builder, context, identifier);
			AppendParamsDefEnd(builder, this, context, identifier);
			return builder;
		}

		/// <summary>
		/// Append the required code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendParamsDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder;
		}

		/// <summary>
		/// Append the required start of code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property being emitted</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <param name="paramsStructName">Structure name</param>
		/// <param name="paramsGenFlags">Structure flags</param>
		/// <param name="appendOffset">If true, add the offset parameter</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendParamsDefStart(StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, string? offset, int tabs,
			string paramsStructName, string paramsGenFlags, bool appendOffset)
		{
			builder
				.AppendTabs(tabs)
				.Append($"const UECodeGen_Private::{paramsStructName} {identifier.MakeStatics()} = {{ ")
				.AppendUTF8LiteralString(property.EngineName).Append(", ")
				.AppendNotifyFunc(property).Append(", ")
				.AppendFlags(property.PropertyFlags).Append(", ")
				.Append(paramsGenFlags).Append(", ");

			if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterFound))
			{
				builder.Append('&').Append(property.Outer!.SourceName).Append("::").AppendPropertySetterWrapperName(property).Append(", ");
			}
			else
			{
				builder.Append("nullptr, ");
			}

			if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterFound))
			{
				builder.Append('&').Append(property.Outer!.SourceName).Append("::").AppendPropertyGetterWrapperName(property).Append(", ");
			}
			else
			{
				builder.Append("nullptr, ");
			}

			builder.AppendArrayDim(property, context).Append(", ");

			if (appendOffset)
			{
				if (!String.IsNullOrEmpty(offset))
				{
					builder.Append(offset).Append(", ");
				}
				else
				{
					builder.Append($"STRUCT_OFFSET({context.OuterIdentifier}, {property.SourceName}), ");
				}
			}
			return builder;
		}

		/// <summary>
		/// Append the required end of code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property to add</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendParamsDefEnd(StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			return builder
				.AppendMetaDataParams(context.IsLegacy ? property.MetaData : null, identifier)
				.Append(" };")
				.AppendObjectHashes(property, context)
				.Append("\r\n");
		}

		/// <summary>
		/// Append the a type reference to the member definition
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="obj">Referenced object</param>
		/// <param name="type">Type of declaration to get</param>
		/// <param name="appendNull">True if a "nullptr" is to be appended if the object is null</param>
		/// <returns>Output builder</returns>
		protected static StringBuilder AppendParamsDefRef(StringBuilder builder, IUhtPropertyMemberContext context, UhtObject? obj, UhtSingletonType type, bool appendNull = false)
		{
			if (obj != null)
			{
				if (type == UhtSingletonType.ConstInit)
				{
					builder.Append('&');
				}
				builder.AppendSingletonName(context, obj, type).Append(", ");
			}
			else if (appendNull)
			{
				builder.Append("nullptr, ");
			}
			return builder;
		}

		/// <summary>
		/// Append the required code to add the properties to a pointer array
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendParamsPtr(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{ 
				child.Property.AppendParamsPtr(builder, context, child.Identifier, tabs);
			}
			builder.AppendTabs(tabs).Append($"(const UECodeGen_Private::FPropertyParamsBase*)&{identifier.MakeStatics()},\r\n");
			return builder;
		}
		#endregion

		#region Const init code gen support
		/// <summary>
		/// Append a declaration of the given property as a constinit variable.
		/// For non-compound properties this is simply a declaration of a single variable of the appropriate type.
		/// For compound properties this must be overridden to declare the inner/key/value property.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Current context</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendConstInitDecl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			AppendCommonDecl(builder, context, identifier, tabs);
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{
				child.Property.AppendConstInitDecl(builder, context, child.Identifier, tabs);
			}
			AppendConstInitDecl(builder, this, identifier, tabs, EngineClassName);
			return builder;
		}

		/// <summary>
		/// Append a declaration of the given property as a constinit variable.
		/// For non-compound properties this is simply a declaration of a single variable of the appropriate type.
		/// For compound properties this must be overridden to declare the inner/key/value property.
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property being emitted</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <param name="engineClassName">Engine class name to be used</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendConstInitDecl(StringBuilder builder, UhtProperty property, UhtCppIdentifier identifier, int tabs, string engineClassName)
		{
			string? typePrefix = property.HasGetterOrSetter ? "TPropertyWithSetterAndGetter<" : null;
			string? typeSuffix = property.HasGetterOrSetter ? ">" : null;
			return builder.AppendTabs(tabs).Append($"static UE_CONSTINIT_UOBJECT_DECL TNoDestroy<{typePrefix}F{engineClassName}{typeSuffix}> {identifier};\r\n");
		}

		/// <summary>
		/// Append a full definition of constinit variable(s) for this property
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="outerFunc">Optional function to append an expression for the address of the outer property of this property</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendConstInitDef(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, Action<StringBuilder>? outerFunc, string? offset, int tabs)
		{
			AppendCommonDef(builder, context, identifier, tabs);
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Reverse))
			{
				child.Property.AppendConstInitDef(builder, context, child.Identifier, (builder) => AppendConstInitPtr(builder, identifier, tabs, ""), child.Offset, tabs);
			}

			AppendConstInitDefImpl(builder, context, identifier, outerFunc, offset, tabs);
			return builder;
		}

		/// <summary>
		/// Append a full definition of constinit variable(s) for this property
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="outerFunc">Optional function to append an expression for the address of the outer property of this property</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendConstInitDefImpl(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, Action<StringBuilder>? outerFunc, string? offset, int tabs)
		{
			AppendConstInitDefStart(builder, this, context, identifier, outerFunc, offset, tabs, EngineClassName);
			AppendConstInitDefExtra(builder, context, identifier);
			AppendConstInitDefEnd(builder, this, context);
			return builder;
		}

		/// <summary>
		/// Append the required code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <returns>Output builder</returns>
		protected virtual StringBuilder AppendConstInitDefExtra(StringBuilder builder, IUhtPropertyMemberContext context, UhtCppIdentifier identifier)
		{
			foreach (UhtChildProperty child in EnumerateChildProperties(identifier, UhtChildPropertyOrder.Forward))
			{
				child.Property.AppendConstInitPtr(builder, child.Identifier, 0, ", ");
			}
			return builder;
		}

		/// <summary>
		/// Append the required start of code to define a constinit property variable for the member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property being emitted</param>
		/// <param name="context">Context of the call</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="outerFunc">Optional function to append an expression for the outer property of this property</param>
		/// <param name="offset">Offset of the property</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <param name="engineClassName">Engine class name to be used</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendConstInitDefStart(StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, 
			Action<StringBuilder>? outerFunc, string? offset, int tabs, string engineClassName)
		{
			string? typePrefix = property.HasGetterOrSetter ? "TPropertyWithSetterAndGetter<" : null;
			string? typeSuffix = property.HasGetterOrSetter ? ">" : null;
			builder.AppendTabs(tabs)
				.Append($"UE_CONSTINIT_UOBJECT_DECL TNoDestroy<{typePrefix}F{engineClassName}{typeSuffix}> {identifier.MakeStatics()}{{NoDestroyConstEval, ");

			if (property.HasGetterOrSetter)
			{
				if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.SetterFound))
				{
					builder.Append('&').Append(property.Outer!.SourceName).Append("::").AppendPropertySetterWrapperName(property).Append(", ");
				}
				else
				{
					builder.Append("nullptr, ");
				}

				if (property.PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.GetterFound))
				{
					builder.Append('&').Append(property.Outer!.SourceName).Append("::").AppendPropertyGetterWrapperName(property).Append(", ");
				}
				else
				{
					builder.Append("nullptr, ");
				}
			}

			builder.Append($"UE::CodeGen::ConstInit::FPropertyParams{{ .Owner = ");
			if (outerFunc is not null)
			{
				outerFunc(builder);
				builder.Append(", ");
			}
			else if (property.Outer is UhtPartial partial)
			{
				builder.Append($"&{context.GetSingletonName(partial.OwnerClass, UhtSingletonType.ConstInit)}, ");
			}
			else if (property.Outer is UhtObject obj)
			{
				builder.Append($"&{context.GetSingletonName(obj, UhtSingletonType.ConstInit)}, ");
			}
			else
			{
				throw new UhtException("Property had an outer which was not a UhtObject and no outerFunc was passed");
			}
			// Only properties inside UhtStruct have a next property - properties inside other properties do not 
			if (property.NextProperty is not null)
			{
				builder.Append(".NextProperty = ");
				if (property.NextProperty.Next.Count > 1)
				{
					// Call the GetNextProperty function we defined
					builder.Append($"GetNextProperty_{property.EngineName}(), ");
				}
				else
				{
					// 0 or 1, so we should be appending null or a property defined in the same scopes as this. Otherwise the builder should have appended an explicit null
					UhtProperty? next = property.NextProperty.Next[0];
					builder.AppendConstInitPtr(next, UhtNames.GetPropertyIdentifier(next), 0, ", ");
				}
			}
			builder.Append(".NameUTF8 = UTF8TEXT(").AppendUTF8LiteralString(property.EngineName).Append("), ");
			if (property.RepNotifyName is not null)
			{
				builder.Append(".RepNotifyFuncUTF8 = UTF8TEXT(").AppendUTF8LiteralString(property.RepNotifyName).Append("), ");
			}
			builder.Append(".PropertyFlags = ").AppendFlags(property.PropertyFlags).Append(", ");

			builder.Append(".ArrayDim = ").AppendArrayDim(property, context).Append(", ");

			if (!String.IsNullOrEmpty(offset))
			{
				builder.Append($".Offset = {offset}, ");
			}
			else
			{
				builder.Append($".Offset = STRUCT_OFFSET({context.OuterIdentifier}, {property.SourceName}), ");
			}
			builder.Append(".ElementSize = 0, "); // At present this is overridden by properties with reference to cpp type info. In future this can be used to provide size for e.g. optional properties by referencing inner property type.

			if (!context.IsLegacy || property.MetaData.IsEmpty())
			{
				builder.Append("IF_WITH_METADATA(.MetaData = {},)");
			}
			else
			{
				builder.Append($"IF_WITH_METADATA(.MetaData = MakeConstArrayView({identifier.MakeStatics()}_MetaData),)");
			}
			builder.Append("}, ");
			return builder;
		}

		/// <summary>
		/// Append the required end of code to define the property as a member
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="property">Property being emitted</param>
		/// <param name="context">Context of the call</param>
		/// <returns>Output builder</returns>
		public static StringBuilder AppendConstInitDefEnd(StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context)
		{
			builder
				.Append(" };")
				.AppendObjectHashes(property, context)
				.Append("\r\n");
			return builder;
		}

		/// <summary>
		/// Append an expression to get a pointer to the property object
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="identifier">Identifier of the property.</param>
		/// <param name="tabs">Number of tabs prefix the line with</param>
		/// <param name="post">String to append afterwards</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendConstInitPtr(StringBuilder builder, UhtCppIdentifier identifier, int tabs, string? post)
		{
			builder.AppendTabs(tabs).Append($"&{identifier.MakeStatics()}{post}");
			return builder;
		}

		/// <summary>
		/// If this property needs a different 'next property' pointer under different preprocessor definitions, append an inline function that returns 
		/// the correct property in each preprocessor state.
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="tabs"></param>
		public StringBuilder AppendGetNextProperty(StringBuilder builder, int tabs)
		{
			if (NextProperty?.Next?.Count > 1)
			{
				// Note that the caller is responsible for ensuring this is scoped to when this property is defined
				builder.AppendTabs(tabs).Append("static consteval FProperty* GetNextProperty_").Append(EngineName).Append("() {\r\n");
				builder.AppendScopeLink(NextProperty, tabs + 1, "return ", ";\r\n");
				builder.AppendTabs(tabs).Append("}\r\n");
			}
			return builder;
		}
		#endregion

		/// <summary>
		/// Append a P_GET macro
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <returns>Destination builder</returns>
		public virtual StringBuilder AppendFunctionThunkParameterGet(StringBuilder builder)
		{
			builder.Append("P_GET_");
			if (ArrayDimensions != null)
			{
				builder.Append("ARRAY");
			}
			else
			{
				builder.Append(PGetMacroText);
			}
			if (PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
			{
				if (PGetPassAsNoPtr)
				{
					builder.Append("_REF_NO_PTR");
				}
				else
				{
					builder.Append("_REF");
				}
			}
			builder.Append('(');
			if (ArrayDimensions != null)
			{
				builder.AppendFunctionThunkParameterArrayType(this).Append(',');
			}
			else
			{
				switch (PGetTypeArgument)
				{
					case UhtPGetArgumentType.None:
						break;

					case UhtPGetArgumentType.EngineClass:
						builder.Append('F').Append(EngineClassName).Append(',');
						break;

					case UhtPGetArgumentType.TypeText:
						builder.AppendFunctionThunkParameterArgType(this).Append(',');
						break;
				}
			}
			builder.AppendFunctionThunkParameterName(this).Append(')');
			return builder;
		}

		/// <summary>
		/// Append the text for a function thunk call argument
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <returns>Output builder</returns>
		public virtual StringBuilder AppendFunctionThunkParameterArg(StringBuilder builder)
		{
			return builder.AppendFunctionThunkParameterName(this);
		}

		/// <summary>
		/// Append the name of a function thunk parameter
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <returns>Output builder</returns>
		public StringBuilder AppendFunctionThunkParameterName(StringBuilder builder)
		{
			builder.Append(FunctionParameterThunkPrefix);
			if (PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
			{
				builder.Append("Out_");
			}
			builder.Append(EngineName);
			return builder;
		}

		/// <summary>
		/// Append the Verse function unmarshaling for the given argument
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="isVerseVM">If true this is Verse VM and not BPVM</param>
		/// <returns></returns>
		/// <exception cref="UhtIceException"></exception>
		public StringBuilder AppendVerseArgumentUnmarshal(StringBuilder builder, bool isVerseVM)
		{
			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.VerseNamed))
			{
				if (!isVerseVM && (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.VerseDefaultValue) || DefaultValueTokens != null))
				{
					return AppendTypedVerseArgumentUnmarshal(builder);
				}
				else
				{
					return builder.Append($"V_MARSHAL_PARAM_TYPED({SourceName}, V_COMMA_SEPARATED(").AppendFunctionThunkParameterArgType(this, true).Append("))");
				}
			}
			return AppendTypedVerseArgumentUnmarshal(builder);
		}

		/// <summary>
		/// Append the Verse function marshaling for the given argument
		/// </summary>
		/// <param name="builder"></param>
		/// <returns></returns>
		/// <exception cref="UhtIceException"></exception>
		protected virtual StringBuilder AppendTypedVerseArgumentUnmarshal(StringBuilder builder)
		{
			return builder.Append($"V_MARSHAL_PARAM_TYPED({SourceName}, V_COMMA_SEPARATED(").AppendFunctionThunkParameterArgType(this, true).Append("))");
		}

		/// <summary>
		/// Append the appropriate values to initialize the property to a "NULL" value;
		/// </summary>
		/// <param name="builder"></param>
		/// <param name="isInitializer"></param>
		/// <returns></returns>
		public abstract StringBuilder AppendNullConstructorArg(StringBuilder builder, bool isInitializer);

		/// <summary>
		/// Return the basic declaration type text for user facing messages
		/// </summary>
		/// <returns></returns>
		public string GetUserFacingDecl()
		{
			StringBuilder builder = new();
			AppendText(builder, UhtPropertyTextType.Generic);
			return builder.ToString();
		}

		/// <summary>
		/// Return the RigVM type
		/// </summary>
		/// <returns></returns>
		public string GetRigVMType()
		{
			using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
			AppendText(borrower.StringBuilder, UhtPropertyTextType.RigVMType);
			return borrower.StringBuilder.ToString();
		}

		/// <summary>
		/// Appends any applicable objects and child properties
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="startingLength">Initial length of the builder prior to appending the hashes</param>
		/// <param name="context">Context used to lookup the hashes</param>
		public virtual void AppendObjectHashes(StringBuilder builder, int startingLength, IUhtPropertyMemberContext context)
		{
		}

		/// <summary>
		/// Fetch the mangled name for a property
		/// </summary>
		/// <returns>True if the name needed to be mangled, false if not.</returns>
		public (bool WasMangled, string Result) GetMangledEngineName()
		{
			if (PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm))
			{
				return (false, EngineName);
			}
			if (Outer is UhtField fieldObj && fieldObj.IsVerseField)
			{
				return VerseNameMangling.MangleCasedName(String.IsNullOrEmpty(VerseNameOverride) ? StrippedEngineName : VerseNameOverride);
			}
			return (false, EngineName);
		}
		#endregion

		#region Parsing support
		/// <summary>
		/// Parse a default value for the property and return a sanitized string representation.
		/// 
		/// All tokens in the token reader must be consumed.  Otherwise the default value will be considered to be invalid.
		/// </summary>
		/// <param name="defaultValueReader">Reader containing the default value</param>
		/// <param name="innerDefaultValue">Sanitized representation of default value</param>
		/// <returns>True if a default value was parsed.</returns>
		public abstract bool SanitizeDefaultValue(IUhtTokenReader defaultValueReader, StringBuilder innerDefaultValue);
		#endregion

		#region Resolution support
		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase phase)
		{
			bool result = base.ResolveSelf(phase);

			switch (phase)
			{
				case UhtResolvePhase.Final:
					if (ArrayDimensions != null)
					{
						ReadOnlySpan<char> dim = ArrayDimensions.AsSpan();

						bool again;
						do
						{
							again = false;

							// Remove any irrelevant whitespace
							dim = dim.Trim();

							// Remove any outer brackets
							if (dim.Length > 0 && dim[0] == '(')
							{
								for (int index = 1, depth = 1; index < dim.Length; ++index)
								{
									if (dim[index] == ')')
									{
										if (--depth == 0)
										{
											if (index == dim.Length - 1)
											{
												dim = dim[1..index];
												again = true;
											}
											break;
										}
									}
									else if (dim[index] == '(')
									{
										++depth;
									}
								}
							}

							// Remove any well known casts
							if (dim.Length > 0)
							{
								foreach (string cast in s_casts)
								{
									if (dim.StartsWith(cast, StringComparison.Ordinal))
									{
										dim = dim[cast.Length..];
										again = true;
										break;
									}
								}
							}
						} while (again && dim.Length > 0);

						//COMPATIBILITY-TODO - This method is more robust, but causes differences.  See UhtSession for future
						// plans on fix in 
						//// Now try to see if this is an enum
						//UhtEnum? Enum = null;
						//int Sep = Dim.IndexOf("::");
						//if (Sep >= 0)
						//{
						//	//COMPATIBILITY-TODO "Bob::Type::Value" did not generate a match with the old code
						//	if (Dim.LastIndexOf("::") != Sep)
						//	{
						//		break;
						//	}
						//	Dim = Dim.Slice(0, Sep);
						//}
						//else
						//{
						//	Enum = Session.FindRegularEnumValue(Dim.ToString());
						//}
						//if (Enum == null)
						//{
						//	Enum = Session.FindType(Outer, UhtFindOptions.Enum | UhtFindOptions.SourceName, Dim.ToString()) as UhtEnum;
						//}
						//if (Enum != null)
						//{
						//	MetaData.Add(UhtNames.ArraySizeEnum, Enum.GetPathName());
						//}

						if (dim.Length > 0 && !UhtFCString.IsDigit(dim[0]))
						{
							UhtEnum? enumObj = Session.FindRegularEnumValue(dim.ToString());
							enumObj ??= Session.FindType(Outer, UhtFindOptions.Enum | UhtFindOptions.SourceName, dim.ToString()) as UhtEnum;
							if (enumObj != null)
							{
								MetaData.Add(UhtNames.ArraySizeEnum, enumObj.PathName);
							}
						}
					}
					break;
			}
			return result;
		}

		/// <summary>
		/// Check properties to see if any instances are referenced.
		/// This method does NOT cache the result.
		/// </summary>
		/// <param name="deepScan">If true, the ScanForInstancedReferenced method on the properties will also be called.</param>
		/// <returns></returns>
		public virtual bool ScanForInstancedReferenced(bool deepScan)
		{
			return false;
		}
		#endregion

		#region Validation support
		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);

			// The outer should never be null...
			if (Outer == null)
			{
				return options;
			}

			// Shadowing checks are done at this level, not in the properties themselves
			if (options.HasAnyFlags(UhtValidationOptions.Shadowing) && PropertyCategory != UhtPropertyCategory.Return)
			{

				// First check for duplicate name in self and then duplicate name in parents
				UhtType? existing = Outer.FindType(UhtFindOptions.PropertyOrFunction | UhtFindOptions.EngineName | UhtFindOptions.SelfOnly, EngineName);
				if (existing == this)
				{
					existing = Outer.FindType(UhtFindOptions.PropertyOrFunction | UhtFindOptions.EngineName | UhtFindOptions.ParentsOnly | UhtFindOptions.NoGlobal | UhtFindOptions.NoIncludes, EngineName);
				}

				if (existing != null && existing != this)
				{
					if (existing is UhtProperty existingProperty)
					{
						//@TODO: This exception does not seem sound either, but there is enough existing code that it will need to be
						// fixed up first before the exception it is removed.
						bool existingPropDeprecated = existingProperty.PropertyFlags.HasAnyFlags(EPropertyFlags.Deprecated);
						bool newPropDeprecated = PropertyCategory == UhtPropertyCategory.Member && PropertyFlags.HasAnyFlags(EPropertyFlags.Deprecated);

						//@TODO: Work around to the issue that C++ UHT does not recognize shadowing errors between function parameters and properties that are 
						// in the same structure but appear later in the file
						// Note: The old UHT would check for all property type where this version only checks for member variables.
						// In the old code if you defined the property after the function with argument with the same name, UHT would
						// not issue an error.  However, if the property was defined PRIOR to the function with the matching argument name,
						// UHT would generate an error.
						bool appearsLater = existingProperty.HeaderFile == HeaderFile && existingProperty.LineNumber > LineNumber;
						if (!newPropDeprecated && !existingPropDeprecated && !appearsLater)
						{
							LogShadowingError(existingProperty);
						}
					}
					else if (existing is UhtFunction existingFunction)
					{
						if (PropertyCategory == UhtPropertyCategory.Member)
						{
							LogShadowingError(existingFunction);
						}
					}
				}
			}

			Validate((UhtStruct)Outer, this, options);
			return options;
		}

		private void LogShadowingError(UhtType shadows)
		{
			this.LogError($"{PropertyCategory.GetHintText()}: '{SourceName}' cannot be defined in '{Outer?.SourceName}' as it is already defined in scope '{shadows.Outer?.SourceName}' (shadowing is not allowed)");
		}

		/// <summary>
		/// Validate the property settings
		/// </summary>
		/// <param name="outerStruct">The outer structure for the property.  For properties inside containers, this will be the owning struct of the container</param>
		/// <param name="outermostProperty">Outer most property being validated.  For properties in containers, 
		/// this will be the container property.  For properties outside of containers or the container itself, this will be the property.</param>
		/// <param name="options"></param>
		public virtual void Validate(UhtStruct outerStruct, UhtProperty outermostProperty, UhtValidationOptions options)
		{
			// Check for deprecation
			if (options.HasAnyFlags(UhtValidationOptions.Deprecated) && !Deprecated)
			{
				ValidateDeprecated();
			}

			// Validate the types allowed with arrays
			if (ArrayDimensions != null)
			{
				switch (PropertyCategory)
				{
					case UhtPropertyCategory.Return:
						this.LogError("Arrays aren't allowed as return types");
						break;

					case UhtPropertyCategory.RegularParameter:
					case UhtPropertyCategory.ReplicatedParameter:
						this.LogError("Arrays aren't allowed as function parameters");
						break;
				}

				if (this is UhtContainerBaseProperty)
				{
					this.LogError("Static arrays of containers are not allowed");
				}

				if (this is UhtBoolProperty)
				{
					this.LogError("Bool arrays are not allowed");
				}
			}

			if (!options.HasAnyFlags(UhtValidationOptions.IsKey) && PropertyFlags.HasAnyFlags(EPropertyFlags.PersistentInstance) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeInstanced))
			{
				this.LogError("'Instanced' is only allowed on an object property, an array of objects, a set of objects, or a map with an object value type.");
			}

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.Config) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanHaveConfig))
			{
				this.LogError("Not allowed to use 'config' with object variables");
			}

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.ExposeOnSpawn))
			{
				if (PropertyFlags.HasAnyFlags(EPropertyFlags.DisableEditOnInstance))
				{
					this.LogWarning("Property cannot have both 'DisableEditOnInstance' and 'ExposeOnSpawn' flags");
				}
				if (!PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible))
				{
					this.LogWarning("Property cannot have 'ExposeOnSpawn' without 'BlueprintVisible' flag.");
				}
			}

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintAssignable) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeBlueprintAssignable))
			{
				this.LogError("'BlueprintAssignable' is only allowed on multicast delegate properties");
			}

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintCallable) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeBlueprintCallable))
			{
				this.LogError("'BlueprintCallable' is only allowed on multicast delegate properties");
			}

			if (PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintAuthorityOnly) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanBeBlueprintAuthorityOnly))
			{
				this.LogError("'BlueprintAuthorityOnly' is only allowed on multicast delegate properties");
			}

			// Check for invalid transients
			EPropertyFlags transients = PropertyFlags & (EPropertyFlags.DuplicateTransient | EPropertyFlags.TextExportTransient | EPropertyFlags.NonPIEDuplicateTransient);
			if (transients != 0 && outerStruct is not UhtClass)
			{
				this.LogError($"'{String.Join(", ", transients.ToStringList(false))}' specifier(s) are only allowed on class member variables");
			}

			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVal) && PropertyCategory != UhtPropertyCategory.Member)
			{
				this.LogError($"'TVal' is only valid on member variables");
			}

			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TPtr) && PropertyCategory != UhtPropertyCategory.Member)
			{
				this.LogError($"'TPtr' is only valid on member variables");
			}

			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.TVerseTask))
			{
				bool isValid = PropertyCategory == UhtPropertyCategory.Return && outerStruct.IsVerseField && outerStruct is UhtFunction function && function.FunctionExportFlags.HasAnyFlags(UhtFunctionExportFlags.VerseNativeCallable);
				if (!isValid)
				{
					this.LogError($"'TVerseTask' is only valid on the return value for Verse native_callable functions");
				}
			}

			if (outerStruct.IsVerseField && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.SupportsVerse))
			{
				this.LogError($"Property type '{outermostProperty.TypeTokens}' isn't support as a Verse type");
			}

			if (!options.HasAnyFlags(UhtValidationOptions.IsKeyOrValue))
			{
				switch (PropertyCategory)
				{
					case UhtPropertyCategory.Member:
						ValidateMember(outerStruct, options);
						break;

					case UhtPropertyCategory.RegularParameter:
					case UhtPropertyCategory.ReplicatedParameter:
						ValidateFunctionArgument((UhtFunction)outerStruct, options);
						break;

					case UhtPropertyCategory.Return:
						break;
				}
			}
			return;
		}

		/// <summary>
		/// Validate that we don't reference any deprecated classes
		/// </summary>
		public virtual void ValidateDeprecated()
		{
		}

		/// <summary>
		/// Verify function argument
		/// </summary>
		protected virtual void ValidateFunctionArgument(UhtFunction func, UhtValidationOptions options)
		{
			if (func.FunctionFlags.HasAnyFlags(EFunctionFlags.Net))
			{
				if (PropertyFlags.HasExactFlags(EPropertyFlags.ReferenceParm | EPropertyFlags.ConstParm, EPropertyFlags.ReferenceParm))
				{
					this.LogError($"Replicated parameters cannot be passed by non-const reference");
				}

				if (func.FunctionFlags.HasAnyFlags(EFunctionFlags.NetRequest))
				{
					if (PropertyFlags.HasExactFlags(EPropertyFlags.OutParm | EPropertyFlags.RepSkip, EPropertyFlags.OutParm))
					{
						// This is difficult to trigger since NotReplicated also sets the property category
						this.LogError("Service request functions cannot contain out parameters, unless marked NotReplicated");
					}
				}
				else
				{
					if (PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
					{
						this.LogError("Replicated functions cannot contain out parameters");
					}

					if (PropertyFlags.HasAnyFlags(EPropertyFlags.RepSkip))
					{
						// This is difficult to trigger since NotReplicated also sets the property category
						this.LogError("Only service request functions cannot contain NotReplicated parameters");
					}
				}
			}

			// The following checks are not performed on the value of a container
			if (func.FunctionFlags.HasAnyFlags(EFunctionFlags.BlueprintEvent | EFunctionFlags.BlueprintCallable))
			{
				// Check that the parameter name is valid and does not conflict with pre-defined types
				if (s_invalidParamNames.Contains(SourceName))
				{
					this.LogError($"Parameter name '{SourceName}' in function is invalid, '{SourceName}' is a reserved name.");
				}
			}
		}

		/// <summary>
		/// Validate member settings
		/// </summary>
		/// <param name="structObj">Containing struct.  This is either a UhtScriptStruct or UhtClass</param>
		/// <param name="options">Validation options</param>
		protected virtual void ValidateMember(UhtStruct structObj, UhtValidationOptions options)
		{
			if (!options.HasAnyFlags(UhtValidationOptions.IsKeyOrValue))
			{
				// First check if the category was specified at all and if the property was exposed to the editor.
				if (!MetaData.TryGetValue(UhtNames.Category, out string? category))
				{
					if (PropertyFlags.HasAnyFlags(EPropertyFlags.Edit | EPropertyFlags.BlueprintVisible))
					{
						if (Module.IsPartOfEngine)
						{
							this.LogError("An explicit Category specifier is required for any property exposed to the editor or Blueprints in an Engine module.");
						}
					}
				}

				// If the category was specified explicitly, it wins
				if (!String.IsNullOrEmpty(category) && !PropertyFlags.HasAnyFlags(EPropertyFlags.Edit | EPropertyFlags.BlueprintVisible |
					EPropertyFlags.BlueprintAssignable | EPropertyFlags.BlueprintCallable))
				{
					this.LogWarning("Property has a Category set but is not exposed to the editor or Blueprints with EditAnywhere, BlueprintReadWrite, " +
						"VisibleAnywhere, BlueprintReadOnly, BlueprintAssignable, BlueprintCallable keywords.");
				}
			}

			// Make sure that editblueprint variables are editable
			if (!PropertyFlags.HasAnyFlags(EPropertyFlags.Edit))
			{
				if (PropertyFlags.HasAnyFlags(EPropertyFlags.DisableEditOnInstance))
				{
					this.LogError("Property cannot have 'DisableEditOnInstance' without being editable");
				}

				if (PropertyFlags.HasAnyFlags(EPropertyFlags.DisableEditOnTemplate))
				{
					this.LogError("Property cannot have 'DisableEditOnTemplate' without being editable");
				}
			}

			string exposeOnSpawnValue = MetaData.GetValueOrDefault(UhtNames.ExposeOnSpawn);
			if (exposeOnSpawnValue.Equals("true", StringComparison.OrdinalIgnoreCase) && !PropertyCaps.HasAnyFlags(UhtPropertyCaps.CanExposeOnSpawn))
			{
				this.LogError("ExposeOnSpawn - Property cannot be exposed");
			}

			if (PropertyExportFlags.HasAnyFlags(UhtPropertyExportFlags.FieldNotify))
			{
				if (Outer is UhtClass classObj)
				{
					if (classObj.ClassType != UhtClassType.Class)
					{
						this.LogError("FieldNotify are not valid on UInterface.");
					}
				}
				else
				{
					this.LogError("FieldNotify property are only valid as UClass member variable.");
				}
			}
		}

		/// <summary>
		/// Generate an error if the class has been deprecated
		/// </summary>
		/// <param name="classObj">Class to check</param>
		protected void ValidateDeprecatedClass(UhtClass? classObj)
		{
			if (classObj == null)
			{
				return;
			}

			if (!classObj.Deprecated)
			{
				return;
			}

			if (PropertyCategory == UhtPropertyCategory.Member)
			{
				this.LogError($"Property is using a deprecated class: '{classObj.SourceName}'.  Property should be marked deprecated as well.");
			}
			else
			{
				this.LogError($"Function is using a deprecated class: '{classObj.SourceName}'.  Function should be marked deprecated as well.");
			}
		}

		/// <summary>
		/// Check to see if the property is valid as a member of a networked structure
		/// </summary>
		/// <param name="referencingProperty">The property referencing the structure property.  All error should be logged on the referencing property.</param>
		/// <returns>True if the property is valid, false if not.  If the property is not valid, an error should also be generated.</returns>
		public virtual bool ValidateStructPropertyOkForNet(UhtProperty referencingProperty)
		{
			return true;
		}

		/// <summary>
		/// Test to see if this property references something that requires the argument to be marked as const
		/// </summary>
		/// <param name="errorType">If const is required, returns the type that is forcing the const</param>
		/// <returns>True if the argument must be marked as const</returns>
		public virtual bool MustBeConstArgument([NotNullWhen(true)] out UhtType? errorType)
		{
			errorType = null;
			return false;
		}

		/// <summary>
		/// Test to see if this property (and it's fields for a struct) are allowed in Optional classes. Due to how
		/// we choose to save the properties when saving (while filtering editor-only), just marking a struct property
		/// as allowed in optional (with AllowedInOptional metadata) won't force all nested properties to be serialized,
		/// so the whole hierarchy of properties must be non-EditorOnly or be marked with AllowedInOptional. So, this
		/// function checks itself and all children properties, looking for any EditorOnly properties without AllowedInOptional
		/// Additionally, if any bad property is found, it will return a string describing the path to the property
		/// </summary>
		/// <param name="propPath">A displayable string to show the path to the property that is editor only</param>
		/// <returns>True if his and child properties are allowed in Optional classes</returns>
		public virtual bool IsAllowedInOptionalClass([NotNullWhen(false)] out string? propPath)
		{
			propPath = null;
			return true;
		}
		#endregion

		#region Reference support
		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			CollectReferencesInternal(collector, PropertyFlags.HasAnyFlags(EPropertyFlags.ParmFlags), false);
			if (Outer is UhtObject outerObject)
			{
				collector.AddObjectReference(outerObject, UhtSingletonType.ConstInit);
			}
		}

		/// <summary>
		/// Collect the references for the property.  This method is used by container properties to
		/// collect the contained property's references.
		/// </summary>
		/// <param name="collector">Reference collector</param>
		/// <param name="addForwardDeclarations">If true, forward declarations should be added</param>
		/// <param name="isTemplateProperty">If true, this is a property in a container</param>
		public virtual void CollectReferencesInternal(IUhtReferenceCollector collector, bool addForwardDeclarations, bool isTemplateProperty)
		{
		}

		/// <summary>
		/// Enumerate all reference types.  This method is used exclusively by FindNoExportStructsRecursive
		/// </summary>
		/// <returns>Type enumerator</returns>
		public virtual IEnumerable<UhtType> EnumerateReferencedTypes()
		{
			return [];
		}
		#endregion

		#region Incremental GC Support
		/// <summary>
		/// Determines whether or not GC barriers need to run after passing this to a function
		/// </summary>
		/// <returns>True if GC barriers need to run</returns>				
		public bool NeedsGCBarrierWhenPassedToFunction(UhtFunction function)
		{
			if (RefQualifier != UhtPropertyRefQualifier.NonConstRef)
			{
				return false;
			}
			return NeedsGCBarrierWhenPassedToFunctionImpl(function);
		}

		/// <summary>
		/// Customization point for subclasses for NeedsGCBarrierWhenPassedToFunction
		/// </summary>
		/// <returns>True if GC barriers need to run</returns>		
		protected virtual bool NeedsGCBarrierWhenPassedToFunctionImpl(UhtFunction uhtFunction)
		{
			return false;
		}
		#endregion

		#region Helper methods
		/// <summary>
		/// Test to see if the two properties are the same type
		/// </summary>
		/// <param name="other">Other property to test</param>
		/// <returns>True if the properies are the same type</returns>
		public abstract bool IsSameType(UhtProperty other);

		/// <summary>
		/// Test to see if the two properties are the same type and ConstParm/OutParm flags somewhat match
		/// </summary>
		/// <param name="other">The other property to test</param>
		/// <returns></returns>
		public bool MatchesType(UhtProperty other)
		{
			if (PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
			{
				if (!other.PropertyFlags.HasAnyFlags(EPropertyFlags.OutParm))
				{
					return false;
				}

				if (PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm) && !other.PropertyFlags.HasAnyFlags(EPropertyFlags.ConstParm))
				{
					return false;
				}
			}
			if (IsStaticArray != other.IsStaticArray)
			{
				return false;
			}
			return IsSameType(other);
		}
		#endregion
	}

	/// <summary>
	/// Assorted StringBuilder extensions for properties
	/// </summary>
	public static class UhtPropertyStringBuilderExtensions
	{

		/// <summary>
		/// Add the given property text
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="textType">Type of text to append</param>
		/// <param name="isTemplateArgument">If true, this is a template argument property</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendPropertyText(this StringBuilder builder, UhtProperty property, UhtPropertyTextType textType, bool isTemplateArgument = false)
		{
			if (textType == UhtPropertyTextType.GetterRetVal || textType == UhtPropertyTextType.SetterParameterArgType)
			{
				return builder.AppendFullDecl(property, textType, true);
			}

			return property.AppendText(builder, textType, isTemplateArgument);
		}

		/// <summary>
		/// Append the member declaration
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendParamsDecl(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return property.AppendParamsDecl(builder, context, identifier, tabs);
		}

		/// <summary>
		/// Append the member definition
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="offset">Offset of the property in the parent</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendParamsDef(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, string? offset, int tabs)
		{
			return property.AppendParamsDef(builder, context, identifier, offset, tabs);
		}

		/// <summary>
		/// Append the member pointer
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendParamsPtr(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return property.AppendParamsPtr(builder, context, identifier, tabs);
		}

		/// <summary>
		/// Append a declaration of a constinit variable representing the given property.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendConstInitDecl(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, int tabs)
		{
			return property.AppendConstInitDecl(builder, context, identifier, tabs);
		}

		/// <summary>
		/// Append a definition of the constinit variable representing the property
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="outerFunc">Optional function to add expression for outer property</param>
		/// <param name="offset">Offset of the property in the parent</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendConstInitDef(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context, UhtCppIdentifier identifier, Action<StringBuilder>? outerFunc, string? offset, int tabs)
		{
			return property.AppendConstInitDef(builder, context, identifier, outerFunc, offset, tabs);
		}

		/// <summary>
		/// Append an expression resolving to a pointer to the constinit object instance for this property
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="identifier">Property identifier</param>
		/// <param name="tabs">Number of tabs in the formatting</param>
		/// <param name="post">String to append afterwards</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendConstInitPtr(this StringBuilder builder, UhtProperty? property, UhtCppIdentifier identifier, int tabs, string? post)
		{
			if (property is null)
			{
				return builder.Append("nullptr").Append(post);
			}
			return property.AppendConstInitPtr(builder, identifier, tabs, post);
		}

		/// <summary>
		/// Append the full declaration including such things as const, *, and &amp;
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="textType">Type of text to append</param>
		/// <param name="skipParameterName">If true, don't append the parameter name</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFullDecl(this StringBuilder builder, UhtProperty property, UhtPropertyTextType textType, bool skipParameterName)
		{
			return property.AppendFullDecl(builder, textType, skipParameterName);
		}

		/// <summary>
		/// Append the function thunk parameter get
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterGet(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendFunctionThunkParameterGet(builder);
		}

		/// <summary>
		/// Append the function thunk parameter array type
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="isTemplateArgument">If true, this is a template argument instead of the top level property</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterArrayType(this StringBuilder builder, UhtProperty property, bool isTemplateArgument = false)
		{
			return property.AppendText(builder, UhtPropertyTextType.FunctionThunkParameterArrayType, isTemplateArgument);
		}

		/// <summary>
		/// Append the function thunk parameter argument type
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="isTemplateArgument">If true, this is a template argument instead of the top level property</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterArgType(this StringBuilder builder, UhtProperty property, bool isTemplateArgument = false)
		{
			return property.AppendText(builder, UhtPropertyTextType.FunctionThunkParameterArgType, isTemplateArgument);
		}

		/// <summary>
		/// Append the function thunk parameter argument
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterArg(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendFunctionThunkParameterArg(builder);
		}

		/// <summary>
		/// Append the function thunk return
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkReturn(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendText(builder, UhtPropertyTextType.FunctionThunkRetVal);
		}

		/// <summary>
		/// Append the function thunk parameter name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFunctionThunkParameterName(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendFunctionThunkParameterName(builder);
		}

		/// <summary>
		/// Append the sparse type
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendSparse(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendText(builder, UhtPropertyTextType.Sparse);
		}

		/// <summary>
		/// Append the property's null constructor arg
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="isInitializer">If true this is in an initializer context.</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendNullConstructorArg(this StringBuilder builder, UhtProperty property, bool isInitializer)
		{
			property.AppendNullConstructorArg(builder, isInitializer);
			return builder;
		}

		/// <summary>
		/// Append the replication notify function or a 'nullptr'
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendNotifyFunc(this StringBuilder builder, UhtProperty property)
		{
			if (property.RepNotifyName != null)
			{
				builder.AppendUTF8LiteralString(property.RepNotifyName);
			}
			else
			{
				builder.Append("nullptr");
			}
			return builder;
		}

		/// <summary>
		/// Append the parameter flags
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="propertyFlags">Property flags</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendFlags(this StringBuilder builder, EPropertyFlags propertyFlags)
		{
			propertyFlags &= ~EPropertyFlags.ComputedFlags;
			return builder.Append("(EPropertyFlags)0x").AppendFormat(CultureInfo.InvariantCulture, "{0:x16}", (ulong)propertyFlags);
		}

		/// <summary>
		/// Append the property array dimensions as a CPP_ARRAY_DIM macro or '1' if not a fixed array.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayDim(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context)
		{
			if (property.ArrayDimensions != null)
			{
				builder.Append($"CPP_ARRAY_DIM({property.SourceName}, {context.OuterIdentifier})");
			}
			else
			{
				builder.Append('1');
			}
			return builder;
		}

		/// <summary>
		/// Given an object, append the hash (if applicable) to the builder
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="referingType">Type asking for an object hash</param>
		/// <param name="startingLength">Initial length of the builder prior to appending the hashes</param>
		/// <param name="context">Context used to lookup the hashes</param>
		/// <param name="obj">Object being appended</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendObjectHash(this StringBuilder builder, int startingLength, UhtType referingType, IUhtPropertyMemberContext context, UhtObject? obj)
		{
			if (obj == null)
			{
				return builder;
			}
			else if (obj is UhtClass classObj)
			{
				if (classObj.ClassExportFlags.HasAnyFlags(UhtClassExportFlags.NoExport))
				{
					return builder;
				}
			}
			else if (obj is UhtScriptStruct scriptStruct)
			{
				if (scriptStruct.ScriptStructFlags.HasAnyFlags(EStructFlags.NoExport))
				{
					return builder;
				}
			}

			builder.Append(builder.Length == startingLength ? " // " : " ");
			IoHash hash = context.GetTypeHash(obj);
			if (hash == IoHash.Zero)
			{
				string type = referingType is UhtProperty ? "property" : "object";
				referingType.LogError($"The {type} \"{referingType.SourceName}\" references type \"{obj.SourceName}\" but the code generation hash is zero.  Check for circular dependencies or missing includes.");
			}
			builder.Append(context.GetTypeHash(obj));
			return builder;
		}

		/// <summary>
		/// Given an object, append the hash (if applicable) to the builder
		/// </summary>
		/// <param name="builder">Output builder</param>
		/// <param name="referingType">Type asking for an object hash</param>
		/// <param name="context">Context used to lookup the hashes</param>
		/// <param name="obj">Object being appended</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendObjectHash(this StringBuilder builder, UhtType referingType, IUhtPropertyMemberContext context, UhtObject? obj)
		{
			return builder.AppendObjectHash(builder.Length, referingType, context, obj);
		}

		/// <summary>
		/// Append the object hashes for all referenced objects
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="context">Context of the property</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendObjectHashes(this StringBuilder builder, UhtProperty property, IUhtPropertyMemberContext context)
		{
			property.AppendObjectHashes(builder, builder.Length, context);
			return builder;
		}

		/// <summary>
		/// Append the singleton name for the given type
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="provider">Object to provide the names of singletons</param>
		/// <param name="type">Type to append</param>
		/// <param name="singletonType">Type of declaration to append</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendSingletonName(this StringBuilder builder, IUhtSingletonNameProvider provider, UhtObject? type, UhtSingletonType singletonType)
		{
			return builder.Append(provider.GetSingletonName(type, singletonType));
		}

		/// <summary>
		/// Append the getter wrapper name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendPropertyGetterWrapperName(this StringBuilder builder, UhtProperty property)
		{
			return builder.Append("Get").Append(property.SourceName).Append("_WrapperImpl");
		}

		/// <summary>
		/// Append the setter wrapper name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendPropertySetterWrapperName(this StringBuilder builder, UhtProperty property)
		{
			return builder.Append("Set").Append(property.SourceName).Append("_WrapperImpl");
		}

		/// <summary>
		/// Append the mangled type for function name mangling
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendPropertyVerseMangledType(this StringBuilder builder, UhtProperty property)
		{
			return property.AppendText(builder, UhtPropertyTextType.VerseMangledType, false);
		}

		/// <summary>
		/// Append the Verse function marshaling for the given argument
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="property">Property in question</param>
		/// <param name="isVerseVM">If true, this is verse VM and not BPVM</param>
		/// <returns></returns>
		/// <exception cref="UhtIceException"></exception>
		public static StringBuilder AppendVerseArgumentUnmarshal(this StringBuilder builder, UhtProperty property, bool isVerseVM)
		{
			return property.AppendVerseArgumentUnmarshal(builder, isVerseVM);
		}
	}
}
