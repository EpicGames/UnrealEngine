// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Flags to represent information about a RigVM parameter
	/// </summary>
	[Flags]
	public enum UhtRigVMParameterFlags : int
	{
		/// <summary>
		/// No RigVM flags
		/// </summary>
		None = 0,

		/// <summary>
		/// "Constant" metadata specified
		/// </summary>
		Constant = 0x00000001,

		/// <summary>
		/// "Input" metadata specified
		/// </summary>
		Input = 0x00000002,

		/// <summary>
		/// "Output" metadata specified
		/// </summary>
		Output = 0x00000004,

		/// <summary>
		/// "Singleton" metadata specified
		/// </summary>
		Singleton = 0x00000008,

		/// <summary>
		/// Set if the property is editor only
		/// </summary>
		EditorOnly = 0x00000010,

		/// <summary>
		/// "ExternalVariable" metadata specified — marks an Input pin as a mutable input variable.
		/// The parameter is directionally an input but generates a non-const reference (T&amp; instead of const T&amp;).
		/// </summary>
		InputVariable = 0x00000020,

		/// <summary>
		/// Set if the property is an enum
		/// </summary>
		IsEnum = 0x00010000,

		/// <summary>
		/// Set if the property is an array
		/// </summary>
		IsArray = 0x00020000,

		/// <summary>
		/// Set if the property is an enum as byte
		/// </summary>
		IsEnumAsByte = 0x00040000,

		/// <summary>
		/// Computes the value lazily
		/// </summary>
		IsLazy = 0x00080000,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtRigVMParameterFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtRigVMParameterFlags inFlags, UhtRigVMParameterFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtRigVMParameterFlags inFlags, UhtRigVMParameterFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtRigVMParameterFlags inFlags, UhtRigVMParameterFlags testFlags, UhtRigVMParameterFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// The FRigVMParameter represents a single parameter of a method
	/// marked up with RIGVM_METHOD.
	/// Each parameter can be marked with Constant, Input or Output
	/// metadata - this struct simplifies access to that information.
	/// </summary>
	public class UhtRigVMParameter
	{
		/// <summary>
		/// Property associated with the RigVM parameter
		/// </summary>
		[JsonConverter(typeof(UhtTypeSourceNameJsonConverter<UhtProperty>))]
		public UhtProperty? Property { get; }

		/// <summary>
		/// Name of the property
		/// </summary>
		public string Name { get; } = String.Empty;

		/// <summary>
		/// Type of the property
		/// </summary>
		public string Type { get; } = String.Empty;

		/// <summary>
		/// Cast name
		/// </summary>
		[JsonIgnore]
		public string? CastName { get; }

		/// <summary>
		/// Cast type
		/// </summary>
		[JsonIgnore]
		public string? CastType { get; }

		/// <summary>
		/// Flags associated with the parameter
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtRigVMParameterFlags ParameterFlags { get; set; } = UhtRigVMParameterFlags.None;

		/// <summary>
		/// True if the parameter is marked as "Constant"
		/// </summary>
		[JsonIgnore]
		public bool Constant => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.Constant);

		/// <summary>
		/// True if the parameter is marked as "Input"
		/// </summary>
		[JsonIgnore]
		public bool Input => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.Input);

		/// <summary>
		/// True if the parameter is marked as "Output"
		/// </summary>
		[JsonIgnore]
		public bool Output => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.Output);

		/// <summary>
		/// True if the parameter is marked as "Singleton"
		/// </summary>
		[JsonIgnore]
		public bool Singleton => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.Singleton);

		/// <summary>
		/// True if the parameter is marked as "ExternalVariable" — a mutable input variable
		/// </summary>
		[JsonIgnore]
		public bool InputVariable => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.InputVariable);

		/// <summary>
		/// True if the parameter is editor only
		/// </summary>
		[JsonIgnore]
		public bool EditorOnly => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.EditorOnly);

		/// <summary>
		/// True if the parameter is an enum
		/// </summary>
		[JsonIgnore]
		public bool IsEnum => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.IsEnum);

		/// <summary>
		/// True if the parameter is an enum as byte
		/// </summary>
		[JsonIgnore]
		public bool IsEnumAsByte => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.IsEnumAsByte);

		/// <summary>
		/// True if the parameter should be computed lazily
		/// </summary>
		[JsonIgnore]
		public bool IsLazy => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.IsLazy);

		/// <summary>
		/// True if the parameter is an array
		/// </summary>
		[JsonIgnore]
		public bool IsArray => ParameterFlags.HasAnyFlags(UhtRigVMParameterFlags.IsArray);

		/// <summary>
		/// Create a new RigVM parameter from a property
		/// </summary>
		/// <param name="property">Source property</param>
		/// <param name="index">Parameter index.  Used to create a unique cast name.</param>
		public UhtRigVMParameter(UhtProperty property, int index)
		{
			Property = property;

			Name = property.EngineName;
			ParameterFlags |= property.MetaData.ContainsKey("Constant") ? UhtRigVMParameterFlags.Constant : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.MetaData.ContainsKey("Input") ? UhtRigVMParameterFlags.Input : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.MetaData.ContainsKey("Output") ? UhtRigVMParameterFlags.Output : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.IsEditorOnlyProperty ? UhtRigVMParameterFlags.EditorOnly : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.MetaData.ContainsKey("Singleton") ? UhtRigVMParameterFlags.Singleton : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.MetaData.ContainsKey("Lazy") ? UhtRigVMParameterFlags.IsLazy : UhtRigVMParameterFlags.None;
			ParameterFlags |= property.MetaData.ContainsKey("ExternalVariable") ? UhtRigVMParameterFlags.InputVariable : UhtRigVMParameterFlags.None;

			// Validate ExternalVariable meta combinations before flag injection from Visible/Constant
			// so that error messages reflect what the user actually wrote, not injected flags.
			// Note: the Lazy + InputVariable check also lives here for the same reason (self-contained
			// validation), even though the legacy Lazy + Output check remains in CollectRigVMMembers().
			if (InputVariable)
			{
				if (!Input)
				{
					property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' has 'ExternalVariable' metadata without 'Input'. ExternalVariable requires Input.");
				}
				if (Output)
				{
					property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' has both 'Output' and 'ExternalVariable'. ExternalVariable is not compatible with Output.");
				}
				if (Constant)
				{
					property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' has both 'Constant' and 'ExternalVariable'. ExternalVariable is not compatible with Constant.");
				}
				if (property.MetaData.ContainsKey("Visible"))
				{
					// 'Visible' means read-only display pin; ExternalVariable means mutable input variable.
					// These are semantically incompatible — check here rather than relying on Visible's
					// side-effect of injecting the Constant flag, which would produce a confusing error.
					property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' has both 'Visible' and 'ExternalVariable'. ExternalVariable is not compatible with Visible.");
				}
				// TRigVMLazyValue<const T&> wraps the value for deferred evaluation — it is incompatible
				// with the mutable T& that InputVariable requires.
				if (IsLazy)
				{
					property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' has both 'Lazy' and 'ExternalVariable'. ExternalVariable is not compatible with Lazy.");
				}
			}

			if (property.MetaData.ContainsKey("Visible"))
			{
				ParameterFlags |= UhtRigVMParameterFlags.Constant | UhtRigVMParameterFlags.Input;
				ParameterFlags &= ~UhtRigVMParameterFlags.Output;
			}

			if (EditorOnly)
			{
				property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' is editor only - WITH_EDITORONLY_DATA not allowed on structs with RIGVM_METHOD.");
			}

			if (property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.SupportsRigVM))
			{
				Type = property.GetRigVMType();
				if (property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsRigVMEnum))
				{
					ParameterFlags |= UhtRigVMParameterFlags.IsEnum;
				}
				if (property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsRigVMEnumAsByte))
				{
					ParameterFlags |= UhtRigVMParameterFlags.IsEnumAsByte;
					CastType = $"TEnumAsByte<{Type}>";
				}
				if (property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsRigVMArray))
				{
					ParameterFlags |= UhtRigVMParameterFlags.IsArray;
					if (IsConst() && !IsLazy)
					{
						string extendedType = ExtendedType(false);
						CastName = $"{Name}_{index}_Array";
						CastType = $"TArrayView<const {extendedType[1..^1]}>";
					}
				}
			}
			else
			{
				property.LogError($"RigVM Struct '{Property.Outer?.SourceName}' - Member '{Property.SourceName}' type '{Property.GetUserFacingDecl()}' not supported by RigVM.");
			}
		}

		/// <summary>
		/// Create a new parameter
		/// </summary>
		/// <param name="name">Name of the parameter</param>
		/// <param name="type">Type of the parameter</param>
		public UhtRigVMParameter(string name, string type)
		{
			Name = name;
			Type = type;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtRigVMParameter(UhtInputCacheReader reader)
		{
			Property = reader.ReadType() as UhtProperty;
			Name = reader.ReadString();
			Type = reader.ReadString();
			CastName = reader.ReadOptionalString();
			CastType = reader.ReadOptionalString();
			ParameterFlags = (UhtRigVMParameterFlags)reader.ReadUInt64();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteType(Property);
			writer.WriteString(Name);
			writer.WriteString(Type);
			writer.WriteOptionalString(CastName);
			writer.WriteOptionalString(CastType);
			writer.WriteUInt64((ulong)ParameterFlags);
		}

		/// <summary>
		/// Get the name of the parameter
		/// </summary>
		/// <param name="castName">If true, return the cast name</param>
		/// <returns>Parameter name</returns>
		public string NameOriginal(bool castName = false)
		{
			return castName && CastName != null ? CastName : Name;
		}

		/// <summary>
		/// Get the type of the parameter
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <param name="wrapLazyType">If true, return the wrapped lazy type as needed</param>
		/// <returns>Parameter type</returns>
		public string TypeOriginal(bool castType = false, bool wrapLazyType = true)
		{
			return GetLazyType(castType && CastType != null ? CastType : Type, wrapLazyType);
		}

		/// <summary>
		/// Get the full declaration (type and name)
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <param name="castName">If true, return the cast name</param>
		/// <returns>Parameter declaration</returns>
		public string Declaration(bool castType = false, bool castName = false)
		{
			return $"{TypeOriginal(castType)} {NameOriginal(castName)}";
		}

		/// <summary>
		/// Return the base type without any template arguments
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Base parameter type</returns>
		public string BaseType(bool castType = false)
		{
			string typeOriginal = TypeOriginal(castType, false);

			int lesserPos = typeOriginal.IndexOf('<', StringComparison.Ordinal);
			if (lesserPos >= 0)
			{
				return typeOriginal[..lesserPos];
			}
			else
			{
				return typeOriginal;
			}
		}

		/// <summary>
		/// Template arguments of the type or type if not a template type.
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Template arguments of the type</returns>
		public string ExtendedType(bool castType = false)
		{
			string typeOriginal = TypeOriginal(castType, false);

			int lesserPos = typeOriginal.IndexOf('<', StringComparison.Ordinal);
			if (lesserPos >= 0)
			{
				return typeOriginal[lesserPos..];
			}
			else
			{
				return typeOriginal;
			}
		}

		/// <summary>
		/// Return the type with a const reference
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Type with a const reference</returns>
		public string TypeConstRef(bool castType = false)
		{
			string typeNoRef = TypeNoRef(castType);
			int namespaceEndIndex = typeNoRef.LastIndexOf(':');
			int firstTypeCharIndex = namespaceEndIndex >= 0 ? (namespaceEndIndex + 1) : 0;

			if (typeNoRef.Length > 0 && (typeNoRef[firstTypeCharIndex] == 'T' || typeNoRef[firstTypeCharIndex] == 'F'))
			{
				return $"const {typeNoRef}&";
			}
			else
			{
				return $"const {typeNoRef}";
			}
		}

		/// <summary>
		/// Return the type with a reference
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Type with a reference</returns>
		public string TypeRef(bool castType = false)
		{
			string typeNoRef = TypeNoRef(castType);
			return $"{typeNoRef}&";
		}

		/// <summary>
		/// Return the type without reference
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Type without the reference</returns>
		public string TypeNoRef(bool castType = false)
		{
			string typeOriginal = TypeOriginal(castType);
			if (typeOriginal.EndsWith('&'))
			{
				return typeOriginal[0..^1];
			}
			else
			{
				return typeOriginal;
			}
		}

		/// <summary>
		/// Return the type as a reference
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <returns>Type as a reference</returns>
		public string TypeVariableRef(bool castType = false)
		{
			return IsConst() ? TypeConstRef(castType) : TypeRef(castType);
		}

		/// <summary>
		/// Return the type wrapped with a lazy struct as needed
		/// </summary>
		/// <param name="typeToWrap">The type to wrap with a lazy struct</param>
		/// <param name="wrapLazyType">If true the type will be wrapped as needed</param>
		/// <returns>Type wrapped as needed</returns>
		public string GetLazyType(string typeToWrap, bool wrapLazyType)
		{
			if (IsLazy && wrapLazyType)
			{
				return $"TRigVMLazyValue<{typeToWrap}>";
			}
			return typeToWrap;
		}

		/// <summary>
		/// Return a variable declaration for the parameter
		/// </summary>
		/// <param name="castType">If true, return the cast type</param>
		/// <param name="castName">If true, return the cast name</param>
		/// <returns>Parameter as a variable declaration</returns>
		public string Variable(bool castType = false, bool castName = false)
		{
			return $"{TypeVariableRef(castType)} {NameOriginal(castName)}";
		}

		/// <summary>
		/// True if the parameter is constant.
		/// InputVariable params are mutable inputs — they emit T&amp; not const T&amp;, so they are not const
		/// even though they have the Input flag set.
		/// </summary>
		/// <returns>True if the parameter is constant</returns>
		public bool IsConst()
		{
			return Constant || (Input && !Output && !InputVariable);
		}

		/// <summary>
		/// Return true if the parameter requires a cast
		/// </summary>
		/// <returns>True if the parameter requires a cast</returns>
		public bool RequiresCast()
		{
			return !IsEmptyExecuteContext() && CastType != null && CastName != null;
		}

		/// <summary>
		/// Return true if the parameter is an execute context
		/// </summary>
		/// <returns>True if the parameter is an execute context</returns>
		public bool IsExecuteContext()
		{
			if (Property is UhtStructProperty structProperty)
			{
				UhtScriptStruct? scriptStruct = structProperty.ScriptStruct;
				while (scriptStruct != null)
				{
					if (scriptStruct.SourceName == "FRigVMExecuteContext")
					{
						return true;
					}
					scriptStruct = scriptStruct.SuperScriptStruct;
				}
			}
			return false;
		}

		/// <summary>
		/// Return true if the parameter is an empty execute context
		/// </summary>
		/// <returns>True if the parameter is an empty execute context</returns>
		public bool IsEmptyExecuteContext()
		{
			if (Property is UhtStructProperty structProperty)
			{
				UhtScriptStruct? scriptStruct = structProperty.ScriptStruct;
				while (scriptStruct != null)
				{
					if (scriptStruct.SourceName == "FRigVMExecutePin")
					{
						return true;
					}
					scriptStruct = scriptStruct.SuperScriptStruct;
				}
			}
			return false;
		}
	}

	/// <summary>
	/// A single info dataset for a function marked with RIGVM_METHOD.
	/// This struct provides access to its name, the return type and all parameters.
	/// </summary>
	public class UhtRigVMMethodInfo
	{
		private static readonly string s_noPrefix = String.Empty;
		private const string ReturnPrefixInternal = "return ";

		/// <summary>
		/// Return type of the method
		/// </summary>
		public string ReturnType { get; set; } = String.Empty;

		/// <summary>
		/// Name of the method
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Method parameters
		/// </summary>
		public List<UhtRigVMParameter> Parameters { get; } = [];

		/// <summary>
		/// If the method has a return value, return "return".  Otherwise return nothing.
		/// </summary>
		/// <returns>Prefix required for the return value</returns>
		public string ReturnPrefix()
		{
			return (ReturnType.Length == 0 || ReturnType == "void") ? s_noPrefix : ReturnPrefixInternal;
		}
		/// <summary>
		/// Whether or not this method is a predicate.
		/// </summary>
		/// <returns>True if this method is a predicate</returns>
		public bool IsPredicate { get; set; } = false;

		/// <summary>
		/// Is true if method has dll storage such as UE_API
		/// </summary>
		/// <returns>True if this method is a predicate</returns>
		public string DllStorage { get; set; } = String.Empty;

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		public UhtRigVMMethodInfo()
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtRigVMMethodInfo(UhtInputCacheReader reader)
		{
			ReturnType = reader.ReadString();
			Name = reader.ReadString();
			Parameters = reader.ReadList((reader) => new UhtRigVMParameter(reader));
			IsPredicate = reader.ReadBoolean();
			DllStorage = reader.ReadString();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteString(ReturnType);
			writer.WriteString(Name);
			writer.WriteVariableLengthArray(Parameters, (writer, x) => x.Write(writer));
			writer.WriteBoolean(IsPredicate);
			writer.WriteString(DllStorage);
		}
	}

	/// <summary>
	/// An info dataset providing access to all functions marked with RIGVM_METHOD
	/// for each struct.
	/// </summary>
	public class UhtRigVMStructInfo
	{

		/// <summary>
		/// True if the GetUpgradeInfoMethod was found. 
		/// </summary>
		public bool HasGetUpgradeInfoMethod { get; set; } = false;

		/// <summary>
		/// True if the GetNextAggregateNameMethod was found. 
		/// </summary>
		public bool HasGetNextAggregateNameMethod { get; set; } = false;

		/// <summary>
		/// Engine name of the owning script struct
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The name of the execute context to use for this struct's RigVM methods
		/// </summary>
		public string ExecuteContextType { get; set; } = "FRigVMExecuteContext";

		/// <summary>
		/// The name of a member on the struct providing an execute context
		/// </summary>
		public string ExecuteContextMember { get; set; } = String.Empty;

		/// <summary>
		/// true if any of this structs members is an execute context (also an empty one)
		/// </summary>
		public bool HasAnyExecuteContextMember { get; set; } = false;

		/// <summary>
		/// true if this struct has the Pure metadata (should receive const execute context)
		/// </summary>
		public bool IsPure { get; set; } = false;

		/// <summary>
		/// List of the members
		/// </summary>
		public List<UhtRigVMParameter> Members { get; } = [];

		/// <summary>
		/// List of the methods
		/// </summary>
		public List<UhtRigVMMethodInfo> Methods { get; } = [];

		/// <summary>
		/// Construct a default instance
		/// </summary>
		public UhtRigVMStructInfo()
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtRigVMStructInfo(UhtInputCacheReader reader)
		{
			HasGetUpgradeInfoMethod = reader.ReadBoolean();
			HasGetNextAggregateNameMethod = reader.ReadBoolean();
			Name = reader.ReadString();
			ExecuteContextType = reader.ReadString();
			ExecuteContextMember = reader.ReadString();
			HasAnyExecuteContextMember = reader.ReadBoolean();
			IsPure = reader.ReadBoolean();
			Members = reader.ReadList((reader) => new UhtRigVMParameter(reader));
			Methods = reader.ReadList((reader) => new UhtRigVMMethodInfo(reader));
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteBoolean(HasGetUpgradeInfoMethod);
			writer.WriteBoolean(HasGetNextAggregateNameMethod);
			writer.WriteString(Name);
			writer.WriteString(ExecuteContextType);
			writer.WriteString(ExecuteContextMember);
			writer.WriteBoolean(HasAnyExecuteContextMember);
			writer.WriteBoolean(IsPure);
			writer.WriteVariableLengthArray(Members, (writer, x) => x.Write(writer));
			writer.WriteVariableLengthArray(Methods, (writer, x) => x.Write(writer));
		}
	};

	/// <summary>
	/// Series of flags not part of the engine's script struct flags that affect code generation or verification
	/// </summary>
	[Flags]
	public enum UhtScriptStructExportFlags : int
	{

		/// <summary>
		/// No export flags
		/// </summary>
		None = 0,

		/// <summary>
		/// "HasDefaults" specifier present
		/// </summary>
		HasDefaults = 1 << 0,

		/// <summary>
		/// "HasNoOpConstructor" specifier present
		/// </summary>
		HasNoOpConstructor = 1 << 1,

		/// <summary>
		/// "IsAlwaysAccessible" specifier present
		/// </summary>
		IsAlwaysAccessible = 1 << 2,

		/// <summary>
		/// "IsCoreType" specifier present
		/// </summary>
		IsCoreType = 1 << 3,

		/// <summary>
		/// "Parametric" specifier present
		/// </summary>
		IsVerseParametric = 1 << 4,

		/// <summary>
		/// Indicates that the autogenerated functions of this struct will be DLL exported/imported
		/// </summary>
		MinimalAPI = 1 << 5,

		/// <summary>
		/// Mark the verse structure as persona constructible.
		/// </summary>
		IsVersePersonaConstructible = 1 << 6,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtScriptStructExportFlagsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtScriptStructExportFlags inFlags, UhtScriptStructExportFlags testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtScriptStructExportFlags inFlags, UhtScriptStructExportFlags testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtScriptStructExportFlags inFlags, UhtScriptStructExportFlags testFlags, UhtScriptStructExportFlags matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Represents the USTRUCT object
	/// </summary>
	[UhtEngineClass(Name = "ScriptStruct")]
	public class UhtScriptStruct : UhtStruct
	{
		/// <summary>
		/// Script struct engine flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EStructFlags ScriptStructFlags { get; set; } = EStructFlags.NoFlags;

		/// <summary>
		/// UHT only script struct flags
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtScriptStructExportFlags ScriptStructExportFlags { get; set; } = UhtScriptStructExportFlags.None;

		/// <summary>
		/// Line number where GENERATED_BODY/GENERATED_USTRUCT_BODY macro was found
		/// </summary>
		public int MacroDeclaredLineNumber { get; set; } = -1;

		/// <summary>
		/// RigVM structure info
		/// </summary>
		public UhtRigVMStructInfo? RigVMStructInfo { get; set; } = null;

		/// <inheritdoc/>
		[JsonIgnore]
		public override UhtEngineType EngineType => UhtEngineType.ScriptStruct;

		/// <summary>
		/// True if the struct has the "HasDefaults" specifier
		/// </summary>
		[JsonIgnore]
		public bool HasDefaults => ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.HasDefaults);

		/// <summary>
		/// True if the struct has the "IsAlwaysAccessible" specifier
		/// </summary>
		[JsonIgnore]
		public bool IsAlwaysAccessible => ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.IsAlwaysAccessible);

		/// <summary>
		/// True if the struct has the "IsCoreType" specifier
		/// </summary>
		[JsonIgnore]
		public bool IsCoreType => ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.IsCoreType);

		/// <summary>
		/// True if the struct has the "HasNoOpConstructor" specifier
		/// </summary>
		[JsonIgnore]
		public bool HasNoOpConstructor => ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.HasNoOpConstructor);

		/// <inheritdoc/>
		public override string EngineClassName => IsVerseField ? "VerseStruct" : "ScriptStruct";

		/// <inheritdoc/>
		public override string EngineLinkClassName => "ScriptStruct";

		/// <inheritdoc/>
		public override UhtClass? EngineClass => IsVerseField ? Session.UVerseStruct : Session.UScriptStruct;

		/// <inheritdoc/>
		[JsonIgnore]
		public override string EngineNamePrefix => IsVerseField ? "" : (Session.Config!.IsStructWithTPrefix(EngineName) ? "T" : "F");

		/// <summary>
		/// In certain cases (usually with NoExportTypes), a locally generated version of the structures is created and we do code 
		/// generation based on it.  In such cases, we can't use the namespace.
		/// </summary>
		public string NamespaceExportName => !ScriptStructFlags.HasAnyFlags(EStructFlags.Native) && !IsAlwaysAccessible ? "" : Namespace.FullSourceName;

		///<inheritdoc/>
		[JsonIgnore]
		protected override UhtSpecifierValidatorTable? SpecifierValidatorTable => Session.GetSpecifierValidatorTable(UhtTableNames.ScriptStruct);

		/// <summary>
		/// Super struct
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtScriptStruct>))]
		public UhtScriptStruct? SuperScriptStruct => (UhtScriptStruct?)Super;

		/// <summary>
		/// Construct a new script struct
		/// </summary>
		/// <param name="headerFile">Header being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">Outer type</param>
		/// <param name="lineNumber">Line number of the definition</param>
		public UhtScriptStruct(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		public UhtScriptStruct(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			ScriptStructFlags = (EStructFlags)reader.ReadUInt64();
			ScriptStructExportFlags = (UhtScriptStructExportFlags)reader.ReadUInt64();
			MacroDeclaredLineNumber = reader.ReadInt32();
			RigVMStructInfo = reader.ReadOptionalObject<UhtRigVMStructInfo>((reader) => new UhtRigVMStructInfo(reader));
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteUInt64((ulong)ScriptStructFlags);
			writer.WriteUInt64((ulong)ScriptStructExportFlags);
			writer.WriteInt32(MacroDeclaredLineNumber);
			writer.WriteOptionalObject(RigVMStructInfo, (reader) => RigVMStructInfo!.Write(writer));
		}

		#region Resolution support

		/// <inheritdoc/>
		public override void BindSuperAndBases()
		{
			BindSuper(SuperIdentifier, UhtFindOptions.ScriptStruct);
			base.BindSuperAndBases();
		}

		/// <inheritdoc/>
		protected override void ResolveSuper(UhtResolvePhase resolvePhase)
		{
			base.ResolveSuper(resolvePhase);

			switch (resolvePhase)
			{
				case UhtResolvePhase.Bases:

					// if we have a base struct, propagate inherited struct flags now
					UhtScriptStruct? superScriptStruct = SuperScriptStruct;
					if (superScriptStruct != null)
					{
						ScriptStructFlags |= superScriptStruct.ScriptStructFlags & EStructFlags.Inherit;
					}
					break;
			}
		}

		/// <inheritdoc/>
		protected override bool ResolveSelf(UhtResolvePhase resolvePhase)
		{
			bool result = base.ResolveSelf(resolvePhase);

			switch (resolvePhase)
			{
				case UhtResolvePhase.Properties:
					UhtPropertyParser.ResolveChildren(this, UhtPropertyParseOptions.AddModuleRelativePath);
					break;
			}
			return result;
		}

		/// <inheritdoc/>
		protected override void ResolveChildren(UhtResolvePhase phase)
		{

			// Setup additional property as well as script struct def flags
			// for structs / properties being used for the RigVM.
			// The Input / Output / Constant metadata tokens can be used to mark
			// up an input / output pin of a RigVMNode. To allow authoring of those
			// nodes we'll mark up the property as accessible in Blueprint / Python
			// as well as make the struct a blueprint type.
			switch (phase)
			{
				case UhtResolvePhase.Properties:
					foreach (UhtType child in Children)
					{
						if (child is UhtProperty property)
						{
							EPropertyFlags originalFlags = property.PropertyFlags;
							if (property.MetaData.ContainsKey(UhtNames.Constant))
							{
								property.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.EditConst | EPropertyFlags.BlueprintVisible;
							}
							if (property.MetaData.ContainsKey(UhtNames.Input) || property.MetaData.ContainsKey(UhtNames.Visible))
							{
								property.PropertyFlags |= EPropertyFlags.Edit | EPropertyFlags.BlueprintVisible;
							}
							if (property.MetaData.ContainsKey(UhtNames.Output))
							{
								if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible))
								{
									property.PropertyFlags |= EPropertyFlags.BlueprintVisible | EPropertyFlags.BlueprintReadOnly;
								}
							}

							if (originalFlags != property.PropertyFlags &&
								property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible | EPropertyFlags.BlueprintReadOnly))
							{
								if (!MetaData.GetBooleanHierarchical(UhtNames.BlueprintType))
								{
									MetaData.Add(UhtNames.BlueprintType, true);
								}

								if (!property.MetaData.ContainsKey(UhtNames.Category))
								{
									property.MetaData.Add(UhtNames.Category, UhtNames.Pins);
								}
							}
						}
					}
					break;
			}

			base.ResolveChildren(phase);

			switch (phase)
			{
				case UhtResolvePhase.Final:
					if (ScanForInstancedReferenced(false))
					{
						ScriptStructFlags |= EStructFlags.HasInstancedReference;
					}
					CollectRigVMMembers();
					break;
			}
		}

		/// <inheritdoc/>
		protected override bool ScanForInstancedReferencedInternal(bool deepScan)
		{
			if (ScriptStructFlags.HasAnyFlags(EStructFlags.HasInstancedReference))
			{
				return true;
			}

			if (SuperScriptStruct != null && SuperScriptStruct.ScanForInstancedReferenced(deepScan))
			{
				return true;
			}

			return base.ScanForInstancedReferencedInternal(deepScan);
		}
		#endregion

		#region Validation support
		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);

			if (ScriptStructFlags.HasAnyFlags(EStructFlags.Immutable))
			{
				if (HeaderFile != Session.NoExportTypesHeader)
				{
					this.LogError("Immutable is being phased out in favor of SerializeNative, and is only legal on the mirror structs declared in NoExportTypes.h");
				}
			}

			// Make sure that we don't have too many base structs or classes
			if (BaseIdentifiers != null && BaseIdentifiers.Count > 0)
			{
				this.LogError("USTRUCTs can only have one USTRUCT base.  Wrap any extra bases in a '#if CPP' block.  Replication will not be supported on any extra bases.");
			}

			// Validate the engine name
			if (!IsVerseField)
			{
				string expectedName = $"{EngineNamePrefix}{EngineName}";
				if (SourceName != expectedName)
				{
					this.LogError($"Struct '{SourceName}' has an invalid Unreal prefix, expecting '{expectedName}");
				}
			}

			// Validate RigVM
			if (RigVMStructInfo != null && MetaData.ContainsKey(UhtNames.Deprecated) && !RigVMStructInfo.HasGetUpgradeInfoMethod)
			{
				this.LogError($"RigVMStruct '{SourceName}' is marked as deprecated but is missing 'GetUpgradeInfo method.");
				this.LogError("Please implement a method like below:");
				this.LogError("RIGVM_METHOD()");
				this.LogError("virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;");
			}

			ValidateProperties();

			return options |= UhtValidationOptions.Shadowing | UhtValidationOptions.Deprecated;
		}

		void ValidateProperties()
		{
			bool hasTestedStruct = false;
			foreach (UhtType child in Children)
			{
				if (child is UhtProperty property)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible))
					{
						if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintAssignable | EPropertyFlags.BlueprintCallable) && property is not UhtMulticastDelegateProperty)
						{
							if (!hasTestedStruct)
							{
								hasTestedStruct = true;
								if (!MetaData.GetBooleanHierarchical(UhtNames.BlueprintType))
								{
									property.LogError($"Cannot expose property to blueprints in a struct that is not a BlueprintType. Struct: {SourceName} Property: {property.SourceName}");
								}
							}

							if (property.IsStaticArray)
							{
								property.LogError($"Static array cannot be exposed to blueprint Class: {SourceName} Property: {property.SourceName}");
							}

							if (!property.PropertyCaps.HasAnyFlags(UhtPropertyCaps.IsMemberSupportedByBlueprint))
							{
								property.LogError($"Type '{property.GetUserFacingDecl()}' is not supported by blueprint. Struct: {SourceName} Property: {property.SourceName}");
							}
						}
					}
				}
			}
		}
		#endregion

		/// <inheritdoc/>
		protected override void AppendVerseMyScope(StringBuilder builder, UhtVerseNameMode mode, bool isTopLevel)
		{
			if (!isTopLevel || (mode == UhtVerseNameMode.Qualified && ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.IsVerseParametric)))
			{
				builder.Append('/').Append(VerseName);
			}
		}

		/// <inheritdoc/>
		protected override void AppendVerseName(StringBuilder builder, UhtVerseNameMode mode)
		{
			builder.Append(VerseName);
			if (mode == UhtVerseNameMode.Qualified && ScriptStructExportFlags.HasAnyFlags(UhtScriptStructExportFlags.IsVerseParametric))
			{
				builder.Append("(t)");
			}
		}

		/// <inheritdoc/>
		public override StringBuilder AppendForwardDeclaration(StringBuilder builder)
		{
			return builder.Append($"struct {SourceName};");
		}

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			if (ScriptStructFlags.HasAnyFlags(EStructFlags.NoExport))
			{
				collector.AddSingleton(this);
			}
			collector.AddExportType(this);
			collector.AddDeclaration(this, UhtSingletonType.Registered);
			collector.AddObjectReference(this, UhtSingletonType.Registered);
			if (SuperScriptStruct != null)
			{
				collector.AddObjectReference(SuperScriptStruct, UhtSingletonType.Registered);
			}
			collector.AddObjectReference(Package, UhtSingletonType.Registered);
			foreach (UhtType child in Children)
			{
				child.CollectReferences(collector);
			}

			collector.AddObjectReference(Session.UScriptStruct, UhtSingletonType.ConstInit);
			for (UhtStruct? superStruct = SuperScriptStruct; superStruct is not null; superStruct = superStruct.SuperStruct)
			{
				collector.AddObjectReference(superStruct, UhtSingletonType.ConstInit);
			}
		}

		private void CollectRigVMMembers()
		{
			if (RigVMStructInfo != null)
			{
				if (MetaData.TryGetValueHierarchical("ExecuteContext", out string? executeContextMetadata))
				{
					RigVMStructInfo.ExecuteContextType = executeContextMetadata;
				}

				for (UhtStruct? current = this; current != null; current = current.SuperStruct)
				{
					foreach (UhtProperty property in current.Properties)
					{
						UhtRigVMParameter parameter = new(property, RigVMStructInfo.Members.Count);
						if (parameter.IsExecuteContext())
						{
							if (String.IsNullOrEmpty(RigVMStructInfo.ExecuteContextMember))
							{
								RigVMStructInfo.ExecuteContextMember = parameter.Name;
							}
							if (RigVMStructInfo.ExecuteContextType == "FRigVMExecuteContext")
							{
								RigVMStructInfo.ExecuteContextType = parameter.Type;
							}
							else if (RigVMStructInfo.ExecuteContextType != parameter.Type)
							{
								this.LogError($"RigVM Struct {SourceName} contains properties of varying execute context type {RigVMStructInfo.ExecuteContextType} vs {parameter.Type}.");
							}
							RigVMStructInfo.HasAnyExecuteContextMember = true;
						}
						else if (parameter.IsEmptyExecuteContext())
						{
							RigVMStructInfo.HasAnyExecuteContextMember = true;
						}
						else
						{
							if (parameter.IsLazy && parameter.Output)
							{
								this.LogError($"RigVM Struct {SourceName} - Member {parameter.Name} is both an output and a lazy input.");
							}
							// Note: Lazy + InputVariable is validated in UhtRigVMParameter constructor
							// alongside all other ExternalVariable combination checks.
							RigVMStructInfo.Members.Add(parameter);
						}
					}
				}

				if (RigVMStructInfo.Members.Count > 64)
				{
					this.LogError($"RigVM Struct '{SourceName}' - has {RigVMStructInfo.Members.Count} members (64 is the limit).");
				}

				// ExternalVariable pins are mutable references — the node must have an execute
				// context pin so it can be scheduled as a mutable (non-pure) operation.
				// A struct with ExternalVariable but no execute pin would be neither meaningfully
				// pure nor mutable, which is an invalid state.
				bool bHasInputVariableMember = false;
				foreach (UhtRigVMParameter m in RigVMStructInfo.Members)
				{
					if (m.InputVariable) { bHasInputVariableMember = true; break; }
				}

				if (!RigVMStructInfo.HasAnyExecuteContextMember && bHasInputVariableMember)
				{
					this.LogError($"RigVM Struct '{SourceName}' has one or more ExternalVariable properties but no execute context pin. ExternalVariable requires a mutable node — either declare an FRigVMExecuteContext (or subclass) member with no metadata on this struct, or inherit one from a base struct.");
				}
			}
		}
	}
}