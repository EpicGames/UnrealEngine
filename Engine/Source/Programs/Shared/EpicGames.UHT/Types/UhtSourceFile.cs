// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Represents a source file in UHT
	/// </summary>
	public class UhtSourceFile : IUhtMessageSite, IUhtMessageSource
	{
		private readonly UhtSimpleMessageSite _messageSite;
		private UhtSourceFragment _sourceFragment;

		/// <summary>
		/// The session associated with the source
		/// </summary>
		public UhtSession Session { get; }

		/// <summary>
		/// The full file path
		/// </summary>
		public string FilePath { get; }

		/// <summary>
		/// The file name
		/// </summary>
		public string FileName { get; }

		/// <summary>
		/// The contents of the source file
		/// </summary>
		public StringView Data => _sourceFragment.Data;

		/// <summary>
		/// If this source is from a fragment, the original source file
		/// </summary>
		public UhtSourceFile? FragmentSourceFile => _sourceFragment.SourceFile;

		/// <summary>
		/// If this source is from a fragment, the starting line of the fragment
		/// </summary>
		public int FragmentLineNumber => _sourceFragment.LineNumber;

		/// <summary>
		/// Full file path of the source file
		/// </summary>
		public string FullFilePath => _sourceFragment.FilePath;

		#region IUHTMessageSite implementation
		/// <inheritdoc/>
		public IUhtMessageSession MessageSession => _messageSite.MessageSession;

		/// <inheritdoc/>
		public IUhtMessageSource? MessageSource => _messageSite.MessageSource;

		/// <inheritdoc/>
		public IUhtMessageLineNumber? MessageLineNumber => null;
		#endregion

		#region IUHTMessageSource implementation
		/// <inheritdoc/>
		public string MessageFilePath => FilePath;

		/// <inheritdoc/>
		public string MessageFullFilePath => FullFilePath;

		/// <inheritdoc/>
		public bool MessageIsFragment => FragmentSourceFile != null;

		/// <inheritdoc/>
		public string MessageFragmentFilePath => FragmentSourceFile != null ? FragmentSourceFile.MessageFilePath : "";

		/// <inheritdoc/>
		public string MessageFragmentFullFilePath => FragmentSourceFile != null ? FragmentSourceFile.MessageFullFilePath : "";

		/// <inheritdoc/>
		public int MessageFragmentLineNumber => FragmentLineNumber;
		#endregion

		/// <summary>
		/// Construct a new instance of a source file
		/// </summary>
		/// <param name="session">The owning session</param>
		/// <param name="filePath">The full file path</param>
		public UhtSourceFile(UhtSession session, string filePath)
		{
			Session = session;
			FilePath = filePath;
			FileName = Path.GetFileNameWithoutExtension(FilePath);
			_messageSite = new UhtSimpleMessageSite(Session, this);
		}

		/// <summary>
		/// Read the contents of the source file
		/// </summary>
		public virtual void Read()
		{
			_sourceFragment = Session.ReadSource(FilePath);
		}

		/// <summary>
		/// Sets the source fragment from the given string
		/// </summary>
		/// <param name="source"></param>
		public void SetSource(StringView source)
		{
			_sourceFragment = new() { SourceFile = this, FilePath = FilePath, LineNumber = 0, Data = source };
		}
	}
}
