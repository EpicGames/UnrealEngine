// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text.Json.Serialization;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{
	/// <summary>
	/// Base class for all types that contain properties and functions.
	/// Also support the ability to specifier super class and base classes
	/// </summary>
	[UhtEngineClass(Name = "Struct")]
	public abstract class UhtStruct : UhtField
	{

		/// <summary>
		/// Generated code version of the type.  Set via specifiers
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public EGeneratedCodeVersion GeneratedCodeVersion { get; set; } = EGeneratedCodeVersion.None;

		/// <summary>
		/// Return a collection of children that are properties
		/// 
		/// NOTE: This method allocates memory to construct the enumerator.  In code
		/// invoked a large number of times, the loop should be written directly into
		/// the code and not use this method.  Also, the Linq version performs even
		/// worse.
		/// </summary>
		[JsonIgnore]
		public IEnumerable<UhtProperty> Properties
		{
			get
			{
				foreach (UhtType type in Children)
				{
					if (type is UhtProperty property)
					{
						yield return property;
					}
				}
			}
		}

		/// <summary>
		/// Return a collection of children that are functions
		/// 
		/// NOTE: This method allocates memory to construct the enumerator.  In code
		/// invoked a large number of times, the loop should be written directly into
		/// the code and not use this method.  Also, the Linq version performs even
		/// worse.
		/// </summary>
		[JsonIgnore]
		public IEnumerable<UhtFunction> Functions
		{
			get
			{
				foreach (UhtType type in Children)
				{
					if (type is UhtFunction function)
					{
						yield return function;
					}
				}
			}
		}

		/// <inheritdoc/>
		[JsonIgnore]
		public virtual string EngineNamePrefix => "F";

		/// <inheritdoc/>
		public override string EngineClassName => IsVerseField ? "VerseStruct" : "Struct";

		/// <summary>
		/// Super type
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeSourceNameJsonConverter<UhtStruct>))]
		public UhtStruct? Super { get; set; } = null;

		/// <summary>
		/// Base types
		/// </summary>
		[JsonConverter(typeof(UhtTypeReadOnlyListSourceNameJsonConverter<UhtStruct>))]
		public IReadOnlyList<UhtStruct> Bases => _bases ?? s_emptyBases;

		/// <summary>
		/// Super identifier
		/// </summary>
		[JsonIgnore]
		public UhtToken[] SuperIdentifier { get; set; } = [];

		/// <summary>
		/// Base identifiers
		/// </summary>
		[JsonIgnore]
		[SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "<Pending>")]
		public List<UhtToken[]>? BaseIdentifiers { get; set; } = null;

		/// <summary>
		/// Super struct type
		/// </summary>
		[JsonIgnore]
		public UhtStruct? SuperStruct => Super;

		/// <summary>
		/// Super struct type for inheritance, i.e. functions do not inherit properties from their Super
		/// </summary>
		[JsonIgnore]
		public virtual UhtStruct? InheritanceSuperStruct => Super;

		/// <summary>
		/// After properties are fully resolved, points to the first property in this struct
		/// </summary>
		public UhtDefineScopeLink<UhtProperty>? FirstProperty { get; set; }

		/// <summary>
		/// For classes, struct type for sparse class data shared between all instances
		/// </summary>
		public UhtScriptStruct? SparseClassDataStruct { get; set; }

		/// <summary>
		/// Stack used to test to see if we are recursing in a ScanForInstanceReferenced call
		/// </summary>
		private static readonly ThreadLocal<List<UhtType>> s_scanForInstanceReferencedStack = new(() =>
		{
			return [];
		});

		/// <summary>
		/// Construct a new instance
		/// </summary>
		/// <param name="headerFile">Header file being parsed</param>
		/// <param name="namespaceObj">Namespace where the field was defined</param>
		/// <param name="outer">Outer type</param>
		/// <param name="lineNumber">Line number where definition begins</param>
		protected UhtStruct(UhtHeaderFile headerFile, UhtNamespace namespaceObj, UhtType outer, int lineNumber) : base(headerFile, namespaceObj, outer, lineNumber)
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtStruct(UhtInputCacheReader reader, UhtType outer) : base(reader, outer)
		{
			GeneratedCodeVersion = (EGeneratedCodeVersion)reader.ReadUInt8();
			SuperIdentifier = reader.ReadTokenArray();
			BaseIdentifiers = reader.ReadOptionalList<UhtToken[]>((reader) => reader.ReadTokenArray());
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public override void Write(UhtInputCacheWriter writer)
		{
			base.Write(writer);
			writer.WriteUInt8((byte)GeneratedCodeVersion);
			writer.WriteVariableLengthTokenArray(SuperIdentifier);
			writer.WriteOptionalVariableLengthArray(BaseIdentifiers, (writer, tokens) => writer.WriteVariableLengthTokenArray(tokens));
		}

		/// <summary>
		/// Test to see if the given struct is derived from the base structure
		/// </summary>
		/// <param name="baseStruct">Base structure.</param>
		/// <returns>True if the given structure is the specified base or derives from the base.  If the base is null, the false is returned.</returns>
		public bool IsChildOf(UhtStruct? baseStruct)
		{
			if (baseStruct == null)
			{
				return false;
			}
			for (UhtStruct? current = this; current != null; current = current.Super)
			{
				if (current == baseStruct)
				{
					return true;
				}
			}
			return false;
		}

		private static readonly List<UhtStruct> s_emptyBases = [];
		private List<UhtStruct>? _bases = null;

		#region Resolution support
		/// <inheritdoc/>
		protected override void ResolveSuper(UhtResolvePhase resolvePhase)
		{
			Super?.Resolve(resolvePhase);

			foreach (UhtStruct baseStruct in Bases)
			{
				baseStruct.Resolve(resolvePhase);
			}

			base.ResolveSuper(resolvePhase);
		}

		/// <inheritdoc/>
		protected override void ResolveChildren(UhtResolvePhase phase)
		{
			base.ResolveChildren(phase);

			if (phase == UhtResolvePhase.Final)
			{
				// Build a linked list of each property of this struct/class/function
				using UhtDefineScopeListBuilder<UhtProperty> propertyList = new(DefineScope);
				foreach (UhtType type in Children)
				{
					if (type is UhtProperty property)
					{
						if (property.NextProperty is not null)
						{
							throw new Exception("Unexpectedly initialized next link on property");
						}
						property.NextProperty = propertyList.Add(property);
					}
				}
				FirstProperty = propertyList.Head;
			}
		}

		/// <summary>
		/// Check properties to see if any instances are referenced.
		/// This method does NOT cache the result.
		/// </summary>
		/// <param name="deepScan">If true, the ScanForInstancedReferenced method on the properties will also be called.</param>
		/// <returns></returns>
		public bool ScanForInstancedReferenced(bool deepScan)
		{
			List<UhtType> scanForInstanceReferencedStack = s_scanForInstanceReferencedStack.Value!;
			if (scanForInstanceReferencedStack.Contains(this))
			{
				return false;
			}

			scanForInstanceReferencedStack.Add(this);
			try
			{
				return ScanForInstancedReferencedInternal(deepScan);
			}
			finally
			{
				scanForInstanceReferencedStack.RemoveAt(scanForInstanceReferencedStack.Count - 1);
			}
		}

		/// <summary>
		/// Check properties to see if any instances are referenced.
		/// This method does NOT cache the result.
		/// </summary>
		/// <param name="deepScan">If true, the ScanForInstancedReferenced method on the properties will also be called.</param>
		/// <returns></returns>
		protected virtual bool ScanForInstancedReferencedInternal(bool deepScan)
		{
			foreach (UhtType type in Children)
			{
				if (type is UhtProperty property)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.ContainsInstancedReference | EPropertyFlags.InstancedReference))
					{
						return true;
					}
					if (deepScan && property.ScanForInstancedReferenced(deepScan))
					{
						return true;
					}
				}
			}
			return false;
		}
		#endregion

		#region Super and base binding helper methods
		/// <summary>
		/// Resolve super identifier
		/// </summary>
		/// <param name="superIdentifier">Token that represent the super</param>
		/// <param name="findOptions">Find options to restrict types</param>
		/// <exception cref="UhtException">Thrown if super can not be found</exception>
		public void BindSuper(UhtToken[] superIdentifier, UhtFindOptions findOptions)
		{
			if (superIdentifier.Length > 0)
			{
				Super = (UhtStruct?)FindType(findOptions | UhtFindOptions.SourceName | UhtFindOptions.NoSelf, superIdentifier);
				if (Super == null)
				{
					string fullIdentifier = String.Join("::", superIdentifier.Select(x => x.Value));
					throw new UhtException(this, $"Unable to find parent {EngineType.ShortLowercaseText()} type for '{SourceName}' named '{fullIdentifier}'");
				}
				HeaderFile.AddReferencedHeader(Super);
				MetaData.Parent = Super.MetaData;
			}
		}

		/// <summary>
		/// Resolve bases.  Unlike super, this routine will not generate an error if the type can not be found.
		/// Having unrecognized types is expected.
		/// </summary>
		/// <param name="baseIdentifiers">Collection of bases</param>
		/// <param name="findOptions">Options to restrict types being searched</param>
		public void BindBases(List<UhtToken[]>? baseIdentifiers, UhtFindOptions findOptions)
		{
			if (baseIdentifiers != null)
			{
				foreach (UhtToken[] baseIdentifier in baseIdentifiers)
				{
					// We really only case about interfaces, but we can also handle structs
					UhtStruct? baseStruct = (UhtStruct?)FindType(findOptions | UhtFindOptions.Class | UhtFindOptions.ScriptStruct | UhtFindOptions.SourceName | UhtFindOptions.NoSelf, baseIdentifier);
					if (baseStruct != null)
					{
						_bases ??= [];
						_bases.Add(baseStruct);
						HeaderFile.AddReferencedHeader(baseStruct);
					}
				}
			}
		}
		#endregion

		#region Validation support
		/// <inheritdoc/>
		protected override UhtValidationOptions Validate(UhtValidationOptions options)
		{
			options = base.Validate(options);
			ValidateSparseClassData();
			return options;
		}

		private static bool CheckUIMinMaxRangeFromMetaData(UhtType child)
		{
			bool inputFlag = child.MetaData.ContainsKey(UhtNames.Input);
			bool visibleFlag = child.MetaData.ContainsKey(UhtNames.Visible);
			if (!inputFlag && !visibleFlag)
			{
				return true;
			}

			string uiMin = child.MetaData.GetValueOrDefault(UhtNames.UIMin);
			string uiMax = child.MetaData.GetValueOrDefault(UhtNames.UIMax);
			if (uiMin.Length == 0 || uiMax.Length == 0)
			{
				return true;
			}

			// NOTE: Old UHT didn't handle parse errors
			if (!Double.TryParse(uiMin, out double minValue))
			{
				minValue = 0;
			}

			if (!Double.TryParse(uiMax, out double maxValue))
			{
				maxValue = 0;
			}

			// NOTE: that we actually allow UIMin == UIMax to disable the range manually.
			return minValue <= maxValue;
		}

		/// <inheritdoc/>
		protected override void ValidateDocumentationPolicy(UhtDocumentationPolicy policy)
		{
			if (policy.ClassOrStructCommentRequired)
			{
				string classTooltip = MetaData.GetValueOrDefault(UhtNames.ToolTip);
				if (classTooltip.Length == 0 || classTooltip.Equals(EngineName, StringComparison.OrdinalIgnoreCase))
				{
					this.LogError($"{EngineType.CapitalizedText()} '{SourceName}' does not provide a tooltip / comment (DocumentationPolicy).");
				}
			}

			if (policy.MemberToolTipsRequired)
			{
				Dictionary<string, UhtProperty> toolTipToType = [];
				foreach (UhtProperty property in Properties)
				{
					// we don't require tooltips / comments on hidden properties.
					// only properties exposed to the user interface require documentation.
					bool inputFlag = property.MetaData.ContainsKey(UhtNames.Input);
					bool outputFlag = property.MetaData.ContainsKey(UhtNames.Output);
					bool visibleFlag = property.MetaData.ContainsKey(UhtNames.Visible);
					if (!inputFlag && !outputFlag && !visibleFlag)
					{
						continue;
					}

					string toolTip = property.GetToolTipText();
					if (toolTip.Length == 0 || toolTip == property.GetDisplayNameText())
					{
						property.LogError($"Property '{SourceName}::{property.SourceName}' does not provide a tooltip / comment (DocumentationPolicy).");
						continue;
					}

					if (toolTipToType.TryGetValue(toolTip, out UhtProperty? existing))
					{
						property.LogError($"Property '{SourceName}::{existing.SourceName}' and '{SourceName}::{property.SourceName}' are using identical tooltips (DocumentationPolicy).");
					}
					else
					{
						toolTipToType.Add(toolTip, property);
					}
				}
			}

			if (policy.FloatRangesRequired)
			{
				foreach (UhtType child in Children)
				{
					if (child is UhtDoubleProperty || child is UhtFloatProperty)
					{
						if (!CheckUIMinMaxRangeFromMetaData(child))
						{
							child.LogError($"Property '{SourceName}::{child.SourceName}' does not provide a valid UIMin / UIMax (DocumentationPolicy).");
						}
					}
				}
			}

			// also compare all tooltips to see if they are unique
			if (policy.FunctionToolTipsRequired)
			{
				if (this is UhtClass)
				{
					Dictionary<string, UhtType> toolTipToType = [];
					foreach (UhtType child in Children)
					{
						if (child is UhtFunction function)
						{
							string toolTip = function.GetToolTipText();
							if (toolTip.Length == 0)
							{
								// NOTE: This does not fire because it doesn't check to see if it matches the display name as above.
								child.LogError($"Function '{SourceName}::{function.SourceName}' does not provide a tooltip / comment (DocumentationPolicy).");
								continue;
							}

							if (toolTipToType.TryGetValue(toolTip, out UhtType? existing))
							{
								child.LogError($"Functions '{SourceName}::{existing.SourceName}' and '{SourceName}::{function.SourceName}' are using identical tooltips / comments (DocumentationPolicy).");
							}
							else
							{
								toolTipToType.Add(toolTip, function);
							}
						}
					}
				}
			}
		}

		private void ValidateSparseClassData()
		{
			// Fetch the data types
			string[]? sparseClassDataTypes = MetaData.GetStringArray(UhtNames.SparseClassDataTypes);
			if (sparseClassDataTypes == null)
			{
				return;
			}

			// Make sure we don't try to have sparse class data inside of a struct instead of a class
			UhtClass? classObj = this as UhtClass;
			if (classObj == null)
			{
				this.LogError($"{EngineType.CapitalizedText()} '{SourceName}' contains sparse class data but is not a class.");
				return;
			}

			// for now we only support one sparse class data structure per class
			if (sparseClassDataTypes.Length > 1)
			{
				this.LogError($"Class '{SourceName}' contains multiple sparse class data types");
				return;
			}
			if (sparseClassDataTypes.Length == 0)
			{
				this.LogError($"Class '{SourceName}' has sparse class metadata but does not specify a type");
				return;
			}

			foreach (string sparseClassDataTypeName in sparseClassDataTypes)
			{
				SparseClassDataStruct = Session.FindType(null, UhtFindOptions.EngineName | UhtFindOptions.ScriptStruct, sparseClassDataTypeName) as UhtScriptStruct;

				// make sure the sparse class data struct actually exists
				if (SparseClassDataStruct == null)
				{
					this.LogError($"Unable to find sparse data type '{sparseClassDataTypeName}' for class '{SourceName}'");
					continue;
				}

				// check the data struct for invalid properties
				foreach (UhtProperty property in SparseClassDataStruct.Properties)
				{
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintAssignable))
					{
						property.LogError($"Sparse class data types can not contain blueprint assignable delegates. Type '{SparseClassDataStruct.EngineName}' Delegate '{property.SourceName}'");
					}

					// all sparse properties should have EditDefaultsOnly
					if (!property.PropertyFlags.HasFlag(EPropertyFlags.Deprecated) && !property.PropertyFlags.HasAllFlags(EPropertyFlags.Edit | EPropertyFlags.DisableEditOnInstance))
					{
						property.LogError($"Sparse class data types must be VisibleDefaultsOnly or EditDefaultsOnly. Type '{SparseClassDataStruct.EngineName}' Property '{property.SourceName}'");
					}

					// no sparse properties should have BlueprintReadWrite
					if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintVisible) && !property.PropertyFlags.HasAnyFlags(EPropertyFlags.BlueprintReadOnly))
					{
						property.LogError($"Sparse class data types must not be BlueprintReadWrite. Type '{SparseClassDataStruct.EngineName}' Property '{property.SourceName}'");
					}
				}

				// if the class's parent has a sparse class data struct then the current class must also use the same struct or one that inherits from it
				UhtClass? superClass = classObj.SuperClass;
				if (superClass != null)
				{
					string[]? superSparseClassDataTypes = superClass.MetaData.GetStringArray(UhtNames.SparseClassDataTypes);
					if (superSparseClassDataTypes != null)
					{
						foreach (string superSparseClassDataTypeName in superSparseClassDataTypes)
						{
							UhtScriptStruct? superSparseScriptStruct = Session.FindType(null, UhtFindOptions.EngineName | UhtFindOptions.ScriptStruct, superSparseClassDataTypeName) as UhtScriptStruct;
							if (superSparseScriptStruct != null)
							{
								if (!SparseClassDataStruct.IsChildOf(superSparseScriptStruct))
								{
									this.LogError(
										$"Class '{SourceName}' is a child of '{superClass.SourceName}' but its sparse class data struct " +
										$"'{SparseClassDataStruct.EngineName}', does not inherit from '{superSparseScriptStruct.EngineName}'.");
								}
							}
						}
					}
				}
			}
		}
		#endregion

		/// <inheritdoc/>
		public override void CollectReferences(IUhtReferenceCollector collector)
		{
			base.CollectReferences(collector);
			collector.AddObjectReference(SparseClassDataStruct, UhtSingletonType.ConstInit);
		}
	}
}
