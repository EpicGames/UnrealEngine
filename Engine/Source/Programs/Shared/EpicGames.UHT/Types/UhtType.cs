// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Text.Json.Serialization;
using System.Threading;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Utils;

namespace EpicGames.UHT.Types
{

	/// <summary>
	/// Enumeration for all the different engine types
	/// </summary>
	public enum UhtEngineType
	{

		/// <summary>
		/// Type is a package (UhtPackage)
		/// </summary>
		Package,

		/// <summary>
		/// Type is a class (UhtClass)
		/// </summary>
		Class,

		/// <summary>
		/// Type is an interface (UhtClass)
		/// </summary>
		Interface,

		/// <summary>
		/// Type is a native interface (UhtClass)
		/// </summary>
		NativeInterface,

		/// <summary>
		/// Type is a script struct (UhtScriptStruct)
		/// </summary>
		ScriptStruct,

		/// <summary>
		/// Type is an enumeration (UhtEnum)
		/// </summary>
		Enum,

		/// <summary>
		/// Type is a function (UhtFunction)
		/// </summary>
		Function,

		/// <summary>
		/// Type is a delegate (UhtFunction)
		/// </summary>
		Delegate,

		/// <summary>
		/// Type is a sparse delegate (UhtFunction)
		/// </summary>
		SparseDelegate,

		/// <summary>
		/// Type is a property (UhtProperty)
		/// </summary>
		Property,
	}

	/// <summary>
	/// Collection of extension methods used to query information about the type
	/// </summary>
	public static class UhtEngineTypeExtensions
	{

		/// <summary>
		/// Get the name of the type where shared distinctions aren't made between different class and function types.
		/// </summary>
		/// <param name="engineType">The engine type in question</param>
		/// <returns>Lowercase name</returns>
		/// <exception cref="UhtIceException">If the requested type isn't supported</exception>
		public static string ShortLowercaseText(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Package: return "package";
				case UhtEngineType.Class:
				case UhtEngineType.Interface:
				case UhtEngineType.NativeInterface: return "class";
				case UhtEngineType.ScriptStruct: return "struct";
				case UhtEngineType.Enum: return "enum";
				case UhtEngineType.Function:
				case UhtEngineType.SparseDelegate:
				case UhtEngineType.Delegate: return "function";
				case UhtEngineType.Property: return "property";
				default:
					throw new UhtIceException("Invalid engine extended type");
			}
		}

		/// <summary>
		/// Get the name of the type where shared distinctions aren't made between different class and function types.
		/// </summary>
		/// <param name="engineType">The engine type in question</param>
		/// <returns>Capitalized name</returns>
		/// <exception cref="UhtIceException">If the requested type isn't supported</exception>
		public static string ShortCapitalizedText(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Package: return "Package";
				case UhtEngineType.Class:
				case UhtEngineType.Interface:
				case UhtEngineType.NativeInterface: return "Class";
				case UhtEngineType.ScriptStruct: return "Struct";
				case UhtEngineType.Enum: return "Enum";
				case UhtEngineType.Function:
				case UhtEngineType.SparseDelegate:
				case UhtEngineType.Delegate: return "Function";
				case UhtEngineType.Property: return "Property";
				default:
					throw new UhtIceException("Invalid engine extended type");
			}
		}

		/// <summary>
		/// Get the name of the type
		/// </summary>
		/// <param name="engineType">The engine type in question</param>
		/// <returns>Lowercase name</returns>
		/// <exception cref="UhtIceException">If the requested type isn't supported</exception>
		public static string LowercaseText(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Package: return "package";
				case UhtEngineType.Class: return "class";
				case UhtEngineType.Interface: return "interface";
				case UhtEngineType.NativeInterface: return "native interface";
				case UhtEngineType.ScriptStruct: return "struct";
				case UhtEngineType.Enum: return "enum";
				case UhtEngineType.Function: return "function";
				case UhtEngineType.Delegate: return "delegate";
				case UhtEngineType.SparseDelegate: return "sparse delegate";
				case UhtEngineType.Property: return "property";
				default:
					throw new UhtIceException("Invalid engine extended type");
			}
		}

		/// <summary>
		/// Get the name of the type
		/// </summary>
		/// <param name="engineType">The engine type in question</param>
		/// <returns>Capitalized name</returns>
		/// <exception cref="UhtIceException">If the requested type isn't supported</exception>
		public static string CapitalizedText(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Package: return "Package";
				case UhtEngineType.Class: return "Class";
				case UhtEngineType.Interface: return "Interface";
				case UhtEngineType.NativeInterface: return "Native interface";
				case UhtEngineType.ScriptStruct: return "Struct";
				case UhtEngineType.Enum: return "Enum";
				case UhtEngineType.Function: return "Function";
				case UhtEngineType.Delegate: return "Delegate";
				case UhtEngineType.SparseDelegate: return "SparseDelegate";
				case UhtEngineType.Property: return "Property";
				default:
					throw new UhtIceException("Invalid engine extended type");
			}
		}

		/// <summary>
		/// Return true if the type must be unique in the symbol table
		/// </summary>
		/// <param name="engineType">Type in question</param>
		/// <returns>True if it must be unique</returns>
		public static bool MustBeUnique(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Class:
				case UhtEngineType.Interface:
				case UhtEngineType.NativeInterface:
				case UhtEngineType.ScriptStruct:
				case UhtEngineType.Enum:
					return true;

				default:
					return false;
			}
		}

		/// <summary>
		/// Find options to be used when attempting to find a duplicate name
		/// </summary>
		/// <param name="engineType">Type in question</param>
		/// <returns>Find options to be used for search</returns>
		public static UhtFindOptions MustBeUniqueFindOptions(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Class:
				case UhtEngineType.Interface:
				case UhtEngineType.ScriptStruct:
				case UhtEngineType.Enum:
					return UhtFindOptions.Class | UhtFindOptions.ScriptStruct | UhtFindOptions.Enum;

				case UhtEngineType.NativeInterface:
					return UhtFindOptions.Class;

				default:
					return (UhtFindOptions)0;
			}
		}

		/// <summary>
		/// Return true if the type must not be a reserved name.
		/// </summary>
		/// <param name="engineType">Type in question</param>
		/// <returns>True if the type must not have a reserved name.</returns>
		public static bool MustNotBeReserved(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Class:
				case UhtEngineType.Interface:
				case UhtEngineType.NativeInterface:
				case UhtEngineType.ScriptStruct:
				case UhtEngineType.Enum:
					return true;

				default:
					return false;
			}
		}

		/// <summary>
		/// Return true if the type should be added to the engine symbol table
		/// </summary>
		/// <param name="engineType">Type in question</param>
		/// <returns>True if the type should be added to the symbol table</returns>
		public static bool HasEngineName(this UhtEngineType engineType)
		{
			return engineType != UhtEngineType.NativeInterface;
		}

		/// <summary>
		/// Return true if the children of the type should be added to the symbol table
		/// </summary>
		/// <param name="engineType">Type in question</param>
		/// <returns>True if children should be added</returns>
		public static bool AddChildrenToSymbolTable(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Property:
					return false;

				default:
					return true;
			}
		}

		/// <summary>
		/// Convert the engine type to find options
		/// </summary>
		/// <param name="engineType">Engine type</param>
		/// <returns>Find options</returns>
		/// <exception cref="UhtIceException">Engine type is invalid</exception>
		public static UhtFindOptions FindOptions(this UhtEngineType engineType)
		{
			switch (engineType)
			{
				case UhtEngineType.Package: return 0;
				case UhtEngineType.Class: return UhtFindOptions.Class;
				case UhtEngineType.Interface: return UhtFindOptions.Class;
				case UhtEngineType.NativeInterface: return UhtFindOptions.Class;
				case UhtEngineType.ScriptStruct: return UhtFindOptions.ScriptStruct;
				case UhtEngineType.Enum: return UhtFindOptions.Enum;
				case UhtEngineType.Function: return UhtFindOptions.Function;
				case UhtEngineType.Delegate: return UhtFindOptions.DelegateFunction;
				case UhtEngineType.SparseDelegate: return UhtFindOptions.DelegateFunction;
				case UhtEngineType.Property: return UhtFindOptions.Property;
				default:
					throw new UhtIceException("Invalid engine extended type");
			}
		}
	}

	/// <summary>
	/// Options to customize how symbols are found
	/// </summary>
	[Flags]
	public enum UhtFindOptions
	{
		/// <summary>
		/// No find options
		/// </summary>
		None = 0,

		/// <summary>
		/// Search using the engine name.  This is the default if neither engine/source is specified.
		/// </summary>
		EngineName = 1 << 0,

		/// <summary>
		/// Search using the source name.  This can not be set if EngineName is also set.
		/// </summary>
		SourceName = 1 << 1,

		/// <summary>
		/// Search for UhtEnum types
		/// </summary>
		Enum = 1 << 8,

		/// <summary>
		/// Search for UhtScriptStruct types
		/// </summary>
		ScriptStruct = 1 << 9,

		/// <summary>
		/// Search for UhtClass types
		/// </summary>
		Class = 1 << 10,

		/// <summary>
		/// Search for UhtFunction types that are delegate functions
		/// </summary>
		DelegateFunction = 1 << 11,

		/// <summary>
		/// Search for UhtFunction types that are not delegate functions
		/// </summary>
		Function = 1 << 12,

		/// <summary>
		/// Search for UhtProperty types
		/// </summary>
		Property = 1 << 13,

		/// <summary>
		/// Search for any properties or functions
		/// </summary>
		PropertyOrFunction = Property | Function | DelegateFunction,

		/// <summary>
		/// Do not search the super chain
		/// </summary>
		NoParents = 1 << 16,

		/// <summary>
		/// Do not search the outer chain
		/// </summary>
		NoOuter = 1 << 17,

		/// <summary>
		/// Do not search include files
		/// </summary>
		NoIncludes = 1 << 18,

		/// <summary>
		/// Do not do a last resort global namespace search
		/// </summary>
		NoGlobal = 1 << 19,

		/// <summary>
		/// Do not search the children of the starting point
		/// </summary>
		NoSelf = 1 << 20,

		/// <summary>
		/// Only search the parent chain
		/// </summary>
		ParentsOnly = NoOuter | NoIncludes | NoGlobal | NoSelf,

		/// <summary>
		/// Search only the children of the starting type
		/// </summary>
		SelfOnly = NoParents | NoOuter | NoIncludes | NoGlobal,

		/// <summary>
		/// Case compare (usually for SourceName searches)
		/// </summary>
		CaseCompare = 1 << 21,

		/// <summary>
		/// Caseless compare (usually for EngineName searches)
		/// </summary>
		CaselessCompare = 1 << 22,

		/// <summary>
		/// Mask of name flags
		/// </summary>
		NamesMask = EngineName | SourceName,

		/// <summary>
		/// Mask of type flags
		/// </summary>
		TypesMask = Enum | ScriptStruct | Class | DelegateFunction | Function | Property,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtFindOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtFindOptions inFlags, UhtFindOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtFindOptions inFlags, UhtFindOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtFindOptions inFlags, UhtFindOptions testFlags, UhtFindOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// Options during validation checks
	/// </summary>
	[Flags]
	public enum UhtValidationOptions
	{

		/// <summary>
		/// No validation options
		/// </summary>
		None,

		/// <summary>
		/// Test to see if any referenced types have been deprecated.  This should not
		/// be set when the parent object or property has already been marked as deprecated.
		/// </summary>
		Deprecated = 1 << 0,

		/// <summary>
		/// Test to see if the name of the property conflicts with another property in the super chain
		/// </summary>
		Shadowing = 1 << 1,

		/// <summary>
		/// The property is part of a key in a TMap
		/// </summary>
		IsKey = 1 << 2,

		/// <summary>
		/// The property is part of a value in a TArray, TSet, or TMap
		/// </summary>
		IsValue = 1 << 3,

		/// <summary>
		/// The property is either a key or a value part of a TArray, TSet, or TMap
		/// </summary>
		IsKeyOrValue = IsKey | IsValue,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtValidationOptionsExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtValidationOptions inFlags, UhtValidationOptions testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtValidationOptions inFlags, UhtValidationOptions testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtValidationOptions inFlags, UhtValidationOptions testFlags, UhtValidationOptions matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}
	}

	/// <summary>
	/// #define scope where the type was defined.  This only includes macros such as WITH_EDITOR, WITH_EDITORONLY_DATA, and WITH_VERSE_VM.
	/// Not all types support the given options.
	/// </summary>
	[Flags]
	[SuppressMessage("Usage", "CA2217:Do not mark enums with FlagsAttribute")]
	public enum UhtDefineScope
	{

		/// <summary>
		/// The scope is invalid.  Used during code generation.
		/// </summary>
		Invalid = -1,

		/// <summary>
		/// Exists outside of a define scope
		/// </summary>
		None = 0,

		/// <summary>
		/// Exists inside of a WITH_EDITORONLY_DATA block
		/// </summary>
		EditorOnlyData = 1 << 0,

		/// <summary>
		/// Exists inside of a WITH_VERSE_VM block
		/// </summary>
		VerseVM = 1 << 1,

		/// <summary>
		/// Exists inside of a WITH_TESTS block
		/// </summary>
		Tests = 1 << 2,

		/// <summary>
		/// Exists inside of a WITH_VERSE_BPVM block
		/// </summary>
		VerseBPVM = 1 << 3,

		/// <summary>
		/// Exists inside of a WITH_EDITOR block
		/// </summary>
		Editor = 1 << 4,

		/// <summary>
		/// Number of unique scope combinations
		/// </summary>
		ScopeCount = 1 << 5,
	}

	/// <summary>
	/// Helper methods for testing flags.  These methods perform better than the generic HasFlag which hits
	/// the GC and stalls.
	/// </summary>
	public static class UhtDefineScopeExtensions
	{

		/// <summary>
		/// Test to see if any of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if any of the flags are set</returns>
		public static bool HasAnyFlags(this UhtDefineScope inFlags, UhtDefineScope testFlags)
		{
			return (inFlags & testFlags) != 0;
		}

		/// <summary>
		/// Test to see if all of the specified flags are set
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <returns>True if all the flags are set</returns>
		public static bool HasAllFlags(this UhtDefineScope inFlags, UhtDefineScope testFlags)
		{
			return (inFlags & testFlags) == testFlags;
		}

		/// <summary>
		/// Test to see if a specific set of flags have a specific value.
		/// </summary>
		/// <param name="inFlags">Current flags</param>
		/// <param name="testFlags">Flags to test for</param>
		/// <param name="matchFlags">Expected value of the tested flags</param>
		/// <returns>True if the given flags have a specific value.</returns>
		public static bool HasExactFlags(this UhtDefineScope inFlags, UhtDefineScope testFlags, UhtDefineScope matchFlags)
		{
			return (inFlags & testFlags) == matchFlags;
		}

		/// <summary>
		/// Return the compiler directive the matches the given scope
		/// </summary>
		/// <param name="defineScope">Scope to return compiler directive</param>
		/// <returns></returns>
		public static UhtCompilerDirective GetCompilerDirective(this UhtDefineScope defineScope)
		{
			switch (defineScope)
			{
				case UhtDefineScope.EditorOnlyData:
					return UhtCompilerDirective.WithEditorOnlyData;
				case UhtDefineScope.Editor:
					return UhtCompilerDirective.WithEditor;
				case UhtDefineScope.VerseVM:
					return UhtCompilerDirective.WithVerseVM;
				case UhtDefineScope.VerseBPVM:
					return UhtCompilerDirective.WithVerseBPVM;
				case UhtDefineScope.Tests:
					return UhtCompilerDirective.WithTests;
				default:
					throw new UhtIceException("A single valid define scope must be passed to GetCompilerDirective()");
			}
		}
	}

	/// <summary>
	/// Access specifier for type
	/// </summary>
	public enum UhtAccessSpecifier
	{

		/// <summary>
		/// No access specifier seen
		/// </summary>
		None,

		/// <summary>
		/// Public type
		/// </summary>
		Public,

		/// <summary>
		/// Private type
		/// </summary>
		Private,

		/// <summary>
		/// Protected type
		/// </summary>
		Protected,
	}

	/// <summary>
	/// Requested resolve phase
	/// </summary>
	public enum UhtResolvePhase : int
	{
		/// <summary>
		/// Initial resolve phase
		/// </summary>
		None,

		/// <summary>
		/// Check for any type that is invalid.  Invalid items will be removed from the children. 
		/// Used to detect things that looked like native interfaces but no interface was found
		/// </summary>
		InvalidCheck = 1,

		/// <summary>
		/// Resolve any super references
		/// </summary>
		Bases,

		/// <summary>
		/// Resolve property types that reference other types
		/// </summary>
		Properties,

		/// <summary>
		/// Final resolve phase for anything that needs fully resolved properties.
		/// </summary>
		Final,
	}

	/// <summary>
	/// Extension methods of UhtResolvePhase
	/// </summary>
	public static class UhtResolvePhaseExtensions
	{

		/// <summary>
		/// Returns true if the phase should execute multi-threaded.
		/// </summary>
		/// <param name="resolvePhase">Phase to be tested</param>
		/// <returns>True if the phase should execute multi-threaded</returns>
		public static bool IsMultiThreadedResolvePhase(this UhtResolvePhase resolvePhase)
		{
			switch (resolvePhase)
			{
				case UhtResolvePhase.Final:
					return false;
				default:
					return true;
			}
		}
	}

	/// <summary>
	/// Flags for required documentation
	/// </summary>
	public class UhtDocumentationPolicy
	{
		/// <summary>
		/// A tool tip must be defined
		/// </summary>
		public bool ClassOrStructCommentRequired { get; set; }

		/// <summary>
		/// Functions must have a tool tip
		/// </summary>
		public bool FunctionToolTipsRequired { get; set; }

		/// <summary>
		/// Property members must have a tool tip
		/// </summary>
		public bool MemberToolTipsRequired { get; set; }

		/// <summary>
		/// Function parameters must have a tool tip
		/// </summary>
		public bool ParameterToolTipsRequired { get; set; }

		/// <summary>
		/// Float properties must have a range specified
		/// </summary>
		public bool FloatRangesRequired { get; set; }

		/// <summary>
		/// Write the structure contents
		/// </summary>
		/// <param name="writer">Destination writer</param>
		public void Write(IMemoryWriter writer)
		{
			writer.WriteBoolean(ClassOrStructCommentRequired);
			writer.WriteBoolean(FunctionToolTipsRequired);
			writer.WriteBoolean(MemberToolTipsRequired);
			writer.WriteBoolean(ParameterToolTipsRequired);
			writer.WriteBoolean(FloatRangesRequired);
		}
	};

	/// <summary>
	/// Base type for all UHT types.
	/// These mostly map to UnrealEngine types.
	/// </summary>
	[DebuggerDisplay("SourceName = {SourceName}")]
	public abstract class UhtType : IUhtMessageSite, IUhtMessageLineNumber
	{
		/// <summary>
		/// Empty list of type used when children are requested but no children have been added.
		/// </summary>
		private static readonly List<UhtType> s_emptyTypeList = [];

		/// <summary>
		/// All UHT runs are associated with a given session.  The session holds all the global information for a run.
		/// </summary>
		[JsonIgnore]
		public UhtSession Session => Module.Session;

		/// <summary>
		/// UHT module of the package (1 to 1 relationship)
		/// </summary>
		[JsonIgnore]
		public UhtModule Module { get; }

		/// <summary>
		/// The header file containing the type
		/// </summary>
		[JsonIgnore]
		public UhtHeaderFile HeaderFile => _headerFile ?? throw new UhtIceException("Attempt to fetch the header file for a type that isn't associated with a header");

		/// <summary>
		/// Every type in a session is assigned a unique, non-zero id.
		/// </summary>
		[JsonIgnore]
		public int TypeIndex { get; }

		/// <summary>
		/// The outer object containing this type.  For example, the header file would be the outer for a class defined in the global scope.
		/// </summary>
		[JsonIgnore]
		public UhtType? Outer { get; set; } = null;

		/// <summary>
		/// The owning package of the type
		/// </summary>
		[JsonIgnore]
		public virtual UhtPackage Package
		{
			get
			{
				UhtType type = this;
				while (type.Outer != null)
				{
					type = type.Outer;
				}
				return (UhtPackage)type;
			}
		}

		/// <summary>
		/// The name of the type as it appears in the source file
		/// </summary>
		public string SourceName { get; set; }

		/// <summary>
		/// The name of the type as used by the engines type system.  If not set, will default to the source name
		/// </summary>
		public string EngineName { get => _engineName ?? SourceName; set => _engineName = value; }

		/// <summary>
		/// Simple enumeration that can be used to detect the different types of objects
		/// </summary>
		[JsonIgnore]
		public abstract UhtEngineType EngineType { get; }

		/// <summary>
		/// The name of the engine's class for this type i.e. Class, Function
		/// </summary>
		public abstract string EngineClassName { get; }

		/// <summary>
		/// The name of the engine's class for this type for linking with registration functions. 
		/// i.e. EngineClassName may be VerseStruct, but EngineLinkClassName may be ScriptStruct
		/// </summary>
		public virtual string EngineLinkClassName => EngineClassName;

		/// <summary>
		/// Return the runtime type of this object in the engine after parsing, when the Session
		/// has references to such types available.
		/// </summary>
		public abstract UhtClass? EngineClass { get; }

		/// <summary>
		/// Return true if the type has been deprecated
		/// </summary>
		[JsonIgnore]
		public virtual bool Deprecated => false;

		/// <summary>
		/// The line number of where the definition began
		/// </summary>
		public int LineNumber { get; set; }

		/// <summary>
		/// #define scope where the type was defined. None means defined regardless of preprocessor definitions.
		/// </summary>
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public UhtDefineScope DefineScope { get; set; } = UhtDefineScope.None;

		/// <summary>
		/// If true, the type has not been marked as invalid.
		/// </summary>
		[JsonIgnore]
		public bool IsValid { get; set; } = true;

		/// <summary>
		/// Return a combination of the engine type name followed by the path name of the type
		/// </summary>
		[JsonIgnore]
		public virtual string FullName
		{
			get
			{
				StringBuilder builder = new();
				AppendFullName(builder);
				return builder.ToString();
			}
		}

		/// <summary>
		/// Return the path name of the type which includes all parent outer objects excluding the header file
		/// </summary>
		[JsonIgnore]
		public string PathName
		{
			get
			{
				StringBuilder builder = new();
				AppendPathName(builder, null);
				return builder.ToString();
			}
		}

		/// <summary>
		/// Return the name of the documentation policy to be used
		/// </summary>
		[JsonIgnore]
		public string DocumentationPolicyName => MetaData.GetValueOrDefaultHierarchical(UhtNames.DocumentationPolicy);

		/// <summary>
		/// Return the specifier validation table to be used to validate the meta data on this type
		/// </summary>
		[JsonIgnore]
		protected virtual UhtSpecifierValidatorTable? SpecifierValidatorTable => null;

		/// <summary>
		/// Meta data associated with the type
		/// </summary>
		[JsonConverter(typeof(UhtMetaDataConverter))]
		public UhtMetaData MetaData { get; set; }

		/// <summary>
		/// Children types of this type
		/// </summary>
		[JsonConverter(typeof(UhtNullableTypeListJsonConverter<UhtType>))]
		//TODO - this should return a IReadOnlyList
		public List<UhtType> Children => _children ?? s_emptyTypeList;

		/// <summary>
		/// Helper to string method
		/// </summary>
		/// <returns></returns>
		public override string ToString() { return SourceName; }

		private string? _engineName = null;
		private List<UhtType>? _children = null;
		private readonly UhtHeaderFile? _headerFile = null;
		private int _resolveState = (int)UhtResolvePhase.None * 2;

		#region IUHTMessageSite implementation
		/// <inheritdoc/>
		[JsonIgnore]
		public virtual IUhtMessageSession MessageSession => HeaderFile.MessageSession;

		/// <inheritdoc/>
		[JsonIgnore]
		public virtual IUhtMessageSource? MessageSource => HeaderFile.MessageSource;

		/// <inheritdoc/>
		[JsonIgnore]
		public IUhtMessageLineNumber? MessageLineNumber => this;
		#endregion

		#region IUHTMessageListNumber implementation
		int IUhtMessageLineNumber.MessageLineNumber => LineNumber;
		#endregion

		/// <summary>
		/// Construct the base type information.  This constructor is used exclusively by UhtPackage which is at 
		/// the root of all type hierarchies.
		/// </summary>
		/// <param name="module>"></param>
		protected UhtType(UhtModule module)
		{
			Module = module;
			Outer = null;
			SourceName = String.Empty;
			LineNumber = 1;
			TypeIndex = Session.GetNextTypeIndex();
			MetaData = new UhtMetaData(this, Session.Config);
		}

		/// <summary>
		/// Construct instance of the type given a parent type.
		/// </summary>
		/// <param name="headerFile">Header file being parsed</param>
		/// <param name="outer">The outer type of the type being constructed.  For example, a class defined in a given header will have the header as the outer.</param>
		/// <param name="lineNumber">The line number in the source file where the type was defined.</param>
		/// <param name="metaData">Optional meta data to be associated with the type.</param>
		protected UhtType(UhtHeaderFile headerFile, UhtType outer, int lineNumber, UhtMetaData? metaData = null)
		{
			_headerFile = headerFile;
			Module = _headerFile.Module;
			Outer = outer;
			SourceName = String.Empty;
			LineNumber = lineNumber;
			TypeIndex = Session.GetNextTypeIndex();
			MetaData = metaData ?? new UhtMetaData(this, Session.Config);
			MetaData.MessageSite = this; // Make sure the site is correct
			MetaData.Config = Session.Config;
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		/// <param name="outer">Outer type</param>
		protected UhtType(UhtInputCacheReader reader, UhtType outer)
		{
			reader.AddType(this);
			_headerFile = reader.Header;
			Module = _headerFile.Module;
			Outer = outer;
			TypeIndex = Session.GetNextTypeIndex();
			MetaData = new UhtMetaData(this, Session.Config);
			MetaData.MessageSite = this; // Make sure the site is correct
			MetaData.Config = Session.Config;

			LineNumber = reader.ReadInt32();
			MetaData.Read(reader);
			SourceName = reader.ReadString();
			_engineName = reader.ReadOptionalString();
			DefineScope = (UhtDefineScope)reader.ReadUInt64();
			IsValid = reader.ReadBoolean();
		}

		/// <summary>
		/// Write the type to the archive writer
		/// </summary>
		/// <param name="writer">Destination cache writer</param>
		public virtual void Write(UhtInputCacheWriter writer)
		{
			writer.WriteInt32(LineNumber);
			MetaData.Write(writer);
			writer.WriteString(SourceName);
			writer.WriteOptionalString(_engineName);
			writer.WriteUInt64((ulong)DefineScope);
			writer.WriteBoolean(IsValid);
		}

		/// <summary>
		/// Add a type as a child
		/// </summary>
		/// <param name="child">The child to be added.</param>
		public virtual void AddChild(UhtType child)
		{
			AddChildDirectly(child);
		}

		/// <summary>
		/// Non-virtual method that always adds the child directly to the child list
		/// </summary>
		/// <param name="child">Child to be added</param>
		public void AddChildDirectly(UhtType child)
		{
			_children ??= [];
			_children.Add(child);
		}

		/// <summary>
		/// Move all children from the given source type to this type
		/// </summary>
		/// <param name="source">Source of the children</param>
		public void MoveChildren(UhtType source)
		{
			if (_headerFile == null || _headerFile != source._headerFile)
			{
				throw new UhtIceException("Children can not be moved to a package or between header files");
			}
			if (source._children != null)
			{
				_children ??= [];
				foreach (UhtType child in source._children)
				{
					child.Outer = this;
					_children.Add(child);
				}
				source._children = null;
			}
		}

		#region Type resolution

		/// <summary>
		/// Resolve the type.  To customize behavior, override ResolveSuper, ResolveSelf, or ResolveChildren.
		/// </summary>
		/// <param name="phase">Resolution phase</param>
		/// <returns>False if the type fails any invalid checks during InvalidPhase check.</returns>
		public bool Resolve(UhtResolvePhase phase)
		{
			int initState = (int)phase << 1;
			int currentState = Interlocked.Add(ref _resolveState, 0);

			// If we are already passed the init phase for this state, then we are good
			if (currentState > initState)
			{
				return true;
			}

			// Attempt to make myself the resolve thread for this type.  If we aren't the ones
			// to get the init state, then we have to wait until the thread that did completes
			currentState = Interlocked.CompareExchange(ref _resolveState, initState, currentState);
			if (currentState > initState)
			{
				return true;
			}
			else if (currentState == initState)
			{
				if (phase.IsMultiThreadedResolvePhase())
				{
					do
					{
						Thread.Yield();
						currentState = Interlocked.Add(ref _resolveState, 0);
					} while (currentState == initState);
				}
				return true;
			}
			else
			{
				bool result = true;
				try
				{
					ResolveSuper(phase);
					result = ResolveSelf(phase);
					ResolveChildren(phase);
				}
				finally
				{
					Interlocked.Increment(ref _resolveState);
				}
				return result;
			}
		}

		/// <summary>
		/// Resolve any super types
		/// </summary>
		/// <param name="phase">Resolution phase</param>
		protected virtual void ResolveSuper(UhtResolvePhase phase)
		{
		}

		/// <summary>
		/// Resolve self
		/// </summary>
		/// <param name="phase">Resolution phase</param>
		/// <returns>False if the type fails any invalid checks during InvalidPhase check.</returns>
		protected virtual bool ResolveSelf(UhtResolvePhase phase)
		{
			return true;
		}

		/// <summary>
		/// Resolve children
		/// </summary>
		/// <param name="phase">Resolution phase</param>
		protected virtual void ResolveChildren(UhtResolvePhase phase)
		{
			if (_children != null)
			{
				ResolveChildren(_children, phase);
			}
		}

		/// <summary>
		/// Helper method to resolve a list of types
		/// </summary>
		/// <param name="children">Children to resolve</param>
		/// <param name="phase">Phase of the resolve</param>
		public static void ResolveChildren(List<UhtType> children, UhtResolvePhase phase)
		{
			if (phase == UhtResolvePhase.InvalidCheck)
			{
				int outIndex = 0;
				for (int index = 0; index < children.Count; ++index)
				{
					UhtType child = children[index];
					if (child.Resolve(phase))
					{
						children[outIndex++] = child;
					}
					else
					{
						child.IsValid = false;
					}
				}
				if (outIndex < children.Count)
				{
					children.RemoveRange(outIndex, children.Count - outIndex);
				}
			}
			else
			{
				foreach (UhtType child in children)
				{
					child.Resolve(phase);
				}
			}
		}

		/// <summary>
		/// Remove any invalid children that were previously marked bad.  This is only
		/// executed on a UhtPackage
		/// </summary>
		public void RemoveInvalidChildren()
		{
			if (_children != null)
			{
				int outIndex = 0;
				for (int index = 0; index < _children.Count; ++index)
				{
					UhtType child = _children[index];
					if (child.IsValid)
					{
						_children[outIndex++] = child;
					}
				}
				if (outIndex < _children.Count)
				{
					_children.RemoveRange(outIndex, _children.Count - outIndex);
				}
			}
		}

		/// <summary>
		/// Resolve all the super and base structures and classes
		/// </summary>
		public virtual void BindSuperAndBases()
		{
			if (_children != null)
			{
				foreach (UhtType child in _children)
				{
					child.BindSuperAndBases();
				}
			}
		}
		#endregion

		#region FindType support
		/// <summary>
		/// Find the given type in the type hierarchy
		/// </summary>
		/// <param name="options">Options controlling what is searched</param>
		/// <param name="name">Name of the type.</param>
		/// <param name="messageSite">If supplied, then a error message will be generated if the type can not be found</param>
		/// <param name="lineNumber">Source code line number requesting the lookup.</param>
		/// <returns>The located type of null if not found</returns>
		public UhtType? FindType(UhtFindOptions options, string name, IUhtMessageSite? messageSite = null, int lineNumber = -1)
		{
			return Session.FindType(this, options, name, messageSite, lineNumber);
		}

		/// <summary>
		/// Find the given type in the type hierarchy
		/// </summary>
		/// <param name="options">Options controlling what is searched</param>
		/// <param name="name">Name of the type.</param>
		/// <param name="messageSite">If supplied, then a error message will be generated if the type can not be found</param>
		/// <returns>The located type of null if not found</returns>
		public UhtType? FindType(UhtFindOptions options, ref UhtToken name, IUhtMessageSite? messageSite = null)
		{
			return Session.FindType(this, options, ref name, messageSite);
		}

		/// <summary>
		/// Find the given type in the type hierarchy
		/// </summary>
		/// <param name="options">Options controlling what is searched</param>
		/// <param name="identifiers">Enumeration of identifiers.</param>
		/// <param name="messageSite">If supplied, then a error message will be generated if the type can not be found</param>
		/// <param name="lineNumber">Source code line number requesting the lookup.</param>
		/// <returns>The located type of null if not found</returns>
		public UhtType? FindType(UhtFindOptions options, UhtTokenList identifiers, IUhtMessageSite? messageSite = null, int lineNumber = -1)
		{
			return Session.FindType(this, options, identifiers, messageSite, lineNumber);
		}

		/// <summary>
		/// Find the given type in the type hierarchy
		/// </summary>
		/// <param name="options">Options controlling what is searched</param>
		/// <param name="identifiers">Enumeration of identifiers.</param>
		/// <param name="messageSite">If supplied, then a error message will be generated if the type can not be found</param>
		/// <param name="lineNumber">Source code line number requesting the lookup.</param>
		/// <returns>The located type of null if not found</returns>
		public UhtType? FindType(UhtFindOptions options, UhtToken[] identifiers, IUhtMessageSite? messageSite = null, int lineNumber = -1)
		{
			return Session.FindType(this, options, identifiers, messageSite, lineNumber);
		}
		#endregion

		#region Validation support
		/// <summary>
		/// Validate the type settings
		/// </summary>
		/// <param name="options">Validation options</param>
		/// <returns>Updated validation options.  This will be used to validate the children.</returns>
		protected virtual UhtValidationOptions Validate(UhtValidationOptions options)
		{
			ValidateMetaData();
			ValidateDocumentationPolicy();
			return options;
		}

		private void ValidateMetaData()
		{
			UhtSpecifierValidatorTable? table = SpecifierValidatorTable;
			if (table != null)
			{
				if (MetaData.Dictionary != null && MetaData.Dictionary.Count > 0)
				{
					foreach (KeyValuePair<UhtMetaDataKey, string> kvp in MetaData.Dictionary)
					{
						if (table.TryGetValue(kvp.Key.Name, out UhtSpecifierValidator? specifier))
						{
							specifier.Delegate(this, MetaData, kvp.Key, kvp.Value);
						}
					}
				}
			}
		}

		private void ValidateDocumentationPolicy()
		{
			if (Session.Config == null)
			{
				return;
			}

			// deprecated structs don't require the documentation policy.
			string deprecatedValue = MetaData.GetValueOrDefaultHierarchical(UhtNames.Deprecated);
			if (!String.IsNullOrEmpty(deprecatedValue))
			{
				return;
			}

			string policyName = DocumentationPolicyName;
			if (String.IsNullOrEmpty(policyName))
			{
				policyName = Session.Config.DefaultDocumentationPolicy;
			}

			if (String.IsNullOrEmpty(policyName))
			{
				return;
			}
			else if (Session.Config.DocumentationPolicies.TryGetValue(policyName, out UhtDocumentationPolicy? policy))
			{
				ValidateDocumentationPolicy(policy);
			}
			else
			{
				this.LogError(MetaData.LineNumber, $"Documentation policy '{policyName}' is not known");
			}
		}

		/// <summary>
		/// Validate the documentation policy
		/// </summary>
		/// <param name="policy">The expected documentation policy</param>
		protected virtual void ValidateDocumentationPolicy(UhtDocumentationPolicy policy)
		{
		}

		/// <summary>
		/// Validate the given type.  Validates itself and the children
		/// </summary>
		/// <param name="type">Type to validate</param>
		/// <param name="options">Validation options</param>
		public static void ValidateType(UhtType type, UhtValidationOptions options)
		{
			// Invoke the new method
			options = type.Validate(options);

			// Invoke validate on the children
			foreach (UhtType child in type.Children)
			{
				ValidateType(child, options);
			}
		}
		#endregion

		#region Name and user facing text support
		/// <summary>
		/// Return the path name of the type which includes the outers
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="stopOuter">Type to stop at when generating the path</param>
		public virtual void AppendPathName(StringBuilder builder, UhtType? stopOuter = null)
		{
			AppendPathNameInternal(builder, stopOuter);
			if (builder.Length == 0)
			{
				builder.Append("None");
			}
		}

		private void AppendPathNameInternal(StringBuilder builder, UhtType? stopOuter = null)
		{
			if (this != stopOuter)
			{
				UhtType? outer = Outer;
				if (outer != null && outer != stopOuter)
				{
					outer.AppendPathName(builder, stopOuter);

					// SubObjectDelimiter is used to indicate that this object's outer is not a UPackage
					if (outer is not UhtPackage && outer.Outer is UhtPackage)
					{
						builder.Append(UhtFCString.SubObjectDelimiter);
					}

					// SubObjectDelimiter is also used to bridge between the object:properties in the path
					else if (outer is not UhtProperty && this is UhtProperty)
					{
						builder.Append(UhtFCString.SubObjectDelimiter);
					}
					else
					{
						builder.Append('.');
					}
				}
				builder.Append(EngineName);
			}
		}

		/// <summary>
		/// Get the full type name which is the engine class name followed by the path name
		/// </summary>
		/// <param name="builder">Destination builder</param>
		public virtual void AppendFullName(StringBuilder builder)
		{
			builder.Append(EngineClassName);
			builder.Append(' ');
			AppendPathName(builder);
		}

		/// <summary>
		/// Finds the localized display name or native display name as a fallback.
		/// </summary>
		/// <returns>The display name for this object.</returns>
		public string GetDisplayNameText()
		{
			if (MetaData.TryGetValue(UhtNames.DisplayName, out string? nativeDisplayName))
			{
				return nativeDisplayName;
			}

			// In the engine, this is a call to NameToDisplayString.  UHT doesn't store these values
			// so it is just important that they be reasonably unique.  This allows us to not have to 
			// support changes to NameToDisplayString in the engine
			return SourceName;
		}

		/// <summary>
		/// Finds the localized tooltip or native tooltip as a fallback.
		/// </summary>
		/// <param name="shortToolTip">Look for a shorter version of the tooltip (falls back to the long tooltip if none was specified)</param>
		/// <returns>The tooltip for this object.</returns>
		public virtual string GetToolTipText(bool shortToolTip = false)
		{
			string nativeToolTip = String.Empty;
			if (shortToolTip)
			{
				nativeToolTip = MetaData.GetValueOrDefault(UhtNames.ShortToolTip);
			}

			if (nativeToolTip.Length == 0)
			{
				nativeToolTip = MetaData.GetValueOrDefault(UhtNames.ToolTip);
			}

			if (nativeToolTip.Length == 0)
			{
				// In the engine, this is a call to NameToDisplayString.  UHT doesn't store these values
				// so it is just important that they be reasonably unique.  This allows us to not have to 
				// support changes to NameToDisplayString in the engine
				return SourceName;
			}
			else
			{
				return nativeToolTip.TrimEnd().Replace("@see", "See:", StringComparison.Ordinal);
			}
		}
		#endregion

		/// <summary>
		/// Collect all the references for the type
		/// </summary>
		/// <param name="collector">Object collecting the references</param>
		public virtual void CollectReferences(IUhtReferenceCollector collector)
		{
		}
	}

	/// <summary>
	/// String builder extensions for UhtTypes
	/// </summary>
	public static class UhtTypeStringBuilderExtensions
	{
		/// <summary>
		/// Append all of the outer names associated with the type
		/// </summary>
		/// <param name="builder">Output string builder</param>
		/// <param name="outer">Outer to be appended</param>
		/// <returns>Output string builder</returns>
		public static StringBuilder AppendOuterNames(this StringBuilder builder, UhtType? outer)
		{
			if (outer == null)
			{
				return builder;
			}

			if (outer is UhtClass)
			{
				builder.Append('_');
				builder.Append(outer.SourceName);
			}
			else if (outer is UhtScriptStruct)
			{
				// Structs can also have UPackage outer.
				if (outer.Outer is not UhtPackage)
				{
					AppendOuterNames(builder, outer.Outer);
				}
				builder.Append('_');
				builder.Append(outer.SourceName);
			}
			else if (outer is UhtPackage package)
			{
				builder.Append('_');
				builder.Append(package.Module.ShortName);
			}
			else
			{
				AppendOuterNames(builder, outer.Outer);
				builder.Append('_');
				builder.Append(outer.EngineName);
			}

			return builder;
		}
	}
}
