// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Numerics;
using System.Text;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// Rent and return pooled buffers
	/// </summary>
	public static class UhtPoolBuffers
	{

		/// <summary>
		/// Rent a new buffer
		/// </summary>
		/// <param name="size">Minimum size of the buffer</param>
		/// <returns></returns>
		public static UhtPoolBuffer<T> Rent<T>(int size)
		{
			T[] block = ArrayPool<T>.Shared.Rent(size);
			return new UhtPoolBuffer<T>(block, size);
		}

		/// <summary>
		/// Return a buffer
		/// </summary>
		/// <param name="buffer">Buffer being returned</param>
		public static void Return<T>(UhtPoolBuffer<T> buffer)
		{
			if (buffer.IsSet)
			{
				ArrayPool<T>.Shared.Return(buffer.GetBlock());
			}
		}
	}

	/// <summary>
	/// Pooled buffers
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public struct UhtPoolBuffer<T>
	{

		/// <summary>
		/// THe backing array
		/// </summary>
		private readonly T[]? _block = null;

		/// <summary>
		/// Memory region sized to the requested size
		/// </summary>
		public Memory<T> Memory { get; set; }

		/// <summary>
		/// Return true if we have a pool block
		/// </summary>
		public readonly bool IsSet => _block != null;

		/// <summary>
		/// Create a pool buffer
		/// </summary>
		/// <param name="block">Array block</param>
		/// <param name="size">Size of the requested block</param>
		public UhtPoolBuffer(T[] block, int size)
		{
			_block = block;
			Memory = new Memory<T>(block, 0, size);
		}

		/// <summary>
		/// The backing array.  The size of the array will normally be larger than the requested size.
		/// </summary>
		public readonly T[] GetBlock()
		{
			return _block ?? [];
		}

		/// <summary>
		/// Reset the memory region to the given size
		/// </summary>
		/// <param name="size"></param>
		public void Reset(int size)
		{
			Memory = new Memory<T>(GetBlock(), 0, size);
		}
	}

	/// <summary>
	/// Helper class for using pattern to borrow and return a buffer.
	/// </summary>
	public struct UhtRentedPoolBuffer<T> : IDisposable
	{

		/// <summary>
		/// The borrowed buffer
		/// </summary>
		public UhtPoolBuffer<T> Buffer { get; set; }

		/// <summary>
		/// Borrow a buffer with the given size
		/// </summary>
		/// <param name="size">The size to borrow</param>
		public UhtRentedPoolBuffer(int size)
		{
			Buffer = UhtPoolBuffers.Rent<T>(size);
		}

		/// <summary>
		/// Return the borrowed buffer to the cache
		/// </summary>
		public readonly void Dispose()
		{
			UhtPoolBuffers.Return<T>(Buffer);
		}
	}

	/// <summary>
	/// Collection of helper methods to convert string builder to borrow buffers
	/// </summary>
	public static class UhtPoolBufferStringBuilderExtensions
	{
		/// <summary>
		/// Return a buffer initialized with the string builder.
		/// </summary>
		/// <param name="builder">Source builder content</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtRentedPoolBuffer<char> RentPoolBuffer(this StringBuilder builder)
		{
			int length = builder.Length;
			UhtRentedPoolBuffer<char> buffer = new(length);
			builder.CopyTo(0, buffer.Buffer.Memory.Span, length);
			return buffer;
		}

		/// <summary>
		/// Return a buffer initialized with the string builder sub string.
		/// </summary>
		/// <param name="builder">Source builder content</param>
		/// <param name="startIndex">Starting index in the builder</param>
		/// <param name="length">Length of the content</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtRentedPoolBuffer<char> RentPoolBuffer(this StringBuilder builder, int startIndex, int length)
		{
			UhtRentedPoolBuffer<char> buffer = new(length);
			builder.CopyTo(startIndex, buffer.Buffer.Memory.Span, length);
			return buffer;
		}
	}

	/// <summary>
	/// Cached character buffer system.
	/// 
	/// Invoke UhtBuffer.Borrow method to get a buffer of the given size.
	/// Invoke UhtBuffer.Return to return the buffer to the cache.
	/// </summary>
	[Obsolete("Use UhtPoolBuffer<T> instead of UhtBuffer")]
	public class UhtBuffer
	{
		/// <summary>
		/// Any requests of the given size or smaller will be placed in bucket zero with the given size.
		/// </summary>
		private const int MinSize = 1024 * 16;

		/// <summary>
		/// Adjustment to the bucket index to account for the minimum bucket size
		/// </summary>
		private static readonly int s_buckedAdjustment = BitOperations.Log2((uint)MinSize);

		/// <summary>
		/// Total number of supported buckets
		/// </summary>
		private static readonly int s_bucketCount = 32 - s_buckedAdjustment;

		/// <summary>
		/// Bucket lookaside list
		/// </summary>
		private static readonly UhtBuffer?[] s_lookAsideArray = new UhtBuffer?[s_bucketCount];

		/// <summary>
		/// The bucket index associated with the buffer
		/// </summary>
		private int Bucket { get; }

		/// <summary>
		/// Single list link to the next cached buffer
		/// </summary>
		private UhtBuffer? NextBuffer { get; set; } = null;

		/// <summary>
		/// The backing character block.  The size of the array will normally be larger than the 
		/// requested size.
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1051:Do not declare visible instance fields", Justification = "<Pending>")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public char[] Block;

		/// <summary>
		/// Memory region sized to the requested size
		/// </summary>
		public Memory<char> Memory { get; set; }

		/// <summary>
		/// Construct a new buffer
		/// </summary>
		/// <param name="size">The initial size of the buffer</param>
		/// <param name="bucket">The bucket associated with the buffer</param>
		/// <param name="bucketSize">The size all blocks in this bucket</param>
		private UhtBuffer(int size, int bucket, int bucketSize)
		{
			Block = new char[bucketSize];
			Bucket = bucket;
			Reset(size);
		}

		/// <summary>
		/// Reset the memory region to the given size
		/// </summary>
		/// <param name="size"></param>
		public void Reset(int size)
		{
			Memory = new Memory<char>(Block, 0, size);
		}

		/// <summary>
		/// Borrow a new buffer of the given size
		/// </summary>
		/// <param name="size">Size of the buffer</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtBuffer Borrow(int size)
		{
			if (size <= MinSize)
			{
				return BorrowInternal(size, 0, MinSize);
			}
			else
			{

				// Round up the size to the next larger power of two if it isn't a power of two
				uint usize = (uint)size;
				--usize;
				usize |= usize >> 1;
				usize |= usize >> 2;
				usize |= usize >> 4;
				usize |= usize >> 8;
				usize |= usize >> 16;
				++usize;
				int bucket = BitOperations.Log2(usize) - s_buckedAdjustment;
				return BorrowInternal(size, bucket, (int)usize);
			}
		}

		/// <summary>
		/// Return a buffer initialized with the string builder.
		/// </summary>
		/// <param name="builder">Source builder content</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtBuffer Borrow(StringBuilder builder)
		{
			int length = builder.Length;
			UhtBuffer buffer = Borrow(length);
			builder.CopyTo(0, buffer.Memory.Span, length);
			return buffer;
		}

		/// <summary>
		/// Return a buffer initialized with the string builder sub string.
		/// </summary>
		/// <param name="builder">Source builder content</param>
		/// <param name="startIndex">Starting index in the builder</param>
		/// <param name="length">Length of the content</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtBuffer Borrow(StringBuilder builder, int startIndex, int length)
		{
			UhtBuffer buffer = Borrow(length);
			builder.CopyTo(startIndex, buffer.Memory.Span, length);
			return buffer;
		}

		/// <summary>
		/// Return the buffer to the cache.  The buffer should no longer be accessed.
		/// </summary>
		/// <param name="buffer">The buffer to be returned.</param>
		public static void Return(UhtBuffer buffer)
		{
			lock (s_lookAsideArray)
			{
				buffer.NextBuffer = s_lookAsideArray[buffer.Bucket];
				s_lookAsideArray[buffer.Bucket] = buffer;
			}
		}

		/// <summary>
		/// Internal helper to allocate a buffer
		/// </summary>
		/// <param name="size">The initial size of the buffer</param>
		/// <param name="bucket">The bucket associated with the buffer</param>
		/// <param name="bucketSize">The size all blocks in this bucket</param>
		/// <returns>The allocated buffer</returns>
		private static UhtBuffer BorrowInternal(int size, int bucket, int bucketSize)
		{
			lock (s_lookAsideArray)
			{
				if (s_lookAsideArray[bucket] != null)
				{
					UhtBuffer buffer = s_lookAsideArray[bucket]!;
					s_lookAsideArray[bucket] = buffer.NextBuffer;
					buffer.Reset(size);
					return buffer;
				}
			}
			return new UhtBuffer(size, bucket, bucketSize);
		}
	}

	/// <summary>
	/// Helper class for using pattern to borrow and return a buffer.
	/// </summary>
	[Obsolete("Use UhtRentedPoolBuffer<T> instead of UhtBorrowBuffer")]
	public struct UhtBorrowBuffer : IDisposable
	{

		/// <summary>
		/// The borrowed buffer
		/// </summary>
		public UhtBuffer Buffer { get; set; }

		/// <summary>
		/// Borrow a buffer with the given size
		/// </summary>
		/// <param name="size">The size to borrow</param>
		public UhtBorrowBuffer(int size)
		{
			Buffer = UhtBuffer.Borrow(size);
		}

		/// <summary>
		/// Borrow a buffer populated with the builder contents
		/// </summary>
		/// <param name="builder">Initial contents of the buffer</param>
		public UhtBorrowBuffer(StringBuilder builder)
		{
			Buffer = UhtBuffer.Borrow(builder);
		}

		/// <summary>
		/// Borrow a buffer populated with the builder contents
		/// </summary>
		/// <param name="builder">Initial contents of the buffer</param>
		/// <param name="startIndex">Starting index into the builder</param>
		/// <param name="length">Length of the data in the builder</param>
		public UhtBorrowBuffer(StringBuilder builder, int startIndex, int length)
		{
			Buffer = UhtBuffer.Borrow(builder, startIndex, length);
		}

		/// <summary>
		/// Return the borrowed buffer to the cache
		/// </summary>
		public readonly void Dispose()
		{
			UhtBuffer.Return(Buffer);
		}
	}

	/// <summary>
	/// Cached character buffer system.
	/// 
	/// Invoke UhtBuffer.Borrow method to get a buffer of the given size.
	/// Invoke UhtBuffer.Return to return the buffer to the cache.
	/// </summary>
	[Obsolete("Use UhtPoolBuffer<T> instead of UhtByteBuffer")]
	public class UhtByteBuffer
	{
		/// <summary>
		/// Any requests of the given size or smaller will be placed in bucket zero with the given size.
		/// </summary>
		private const int MinSize = 1024 * 16;

		/// <summary>
		/// Adjustment to the bucket index to account for the minimum bucket size
		/// </summary>
		private static readonly int s_buckedAdjustment = BitOperations.Log2((uint)MinSize);

		/// <summary>
		/// Total number of supported buckets
		/// </summary>
		private static readonly int s_bucketCount = 32 - s_buckedAdjustment;

		/// <summary>
		/// Bucket lookaside list
		/// </summary>
		private static readonly UhtByteBuffer?[] s_lookAsideArray = new UhtByteBuffer?[s_bucketCount];

		/// <summary>
		/// The bucket index associated with the buffer
		/// </summary>
		private int Bucket { get; }

		/// <summary>
		/// Single list link to the next cached buffer
		/// </summary>
		private UhtByteBuffer? NextBuffer { get; set; } = null;

		/// <summary>
		/// The backing character block.  The size of the array will normally be larger than the 
		/// requested size.
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1051:Do not declare visible instance fields", Justification = "<Pending>")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public byte[] Block;

		/// <summary>
		/// Memory region sized to the requested size
		/// </summary>
		public Memory<byte> Memory { get; set; }

		/// <summary>
		/// Construct a new buffer
		/// </summary>
		/// <param name="size">The initial size of the buffer</param>
		/// <param name="bucket">The bucket associated with the buffer</param>
		/// <param name="bucketSize">The size all blocks in this bucket</param>
		private UhtByteBuffer(int size, int bucket, int bucketSize)
		{
			Block = new byte[bucketSize];
			Bucket = bucket;
			Reset(size);
		}

		/// <summary>
		/// Reset the memory region to the given size
		/// </summary>
		/// <param name="size"></param>
		public void Reset(int size)
		{
			Memory = new Memory<byte>(Block, 0, size);
		}

		/// <summary>
		/// Borrow a new buffer of the given size
		/// </summary>
		/// <param name="size">Size of the buffer</param>
		/// <returns>Buffer that should be returned with a call to Return</returns>
		public static UhtByteBuffer Borrow(int size)
		{
			if (size <= MinSize)
			{
				return BorrowInternal(size, 0, MinSize);
			}
			else
			{

				// Round up the size to the next larger power of two if it isn't a power of two
				uint usize = (uint)size;
				--usize;
				usize |= usize >> 1;
				usize |= usize >> 2;
				usize |= usize >> 4;
				usize |= usize >> 8;
				usize |= usize >> 16;
				++usize;
				int bucket = BitOperations.Log2(usize) - s_buckedAdjustment;
				return BorrowInternal(size, bucket, (int)usize);
			}
		}

		/// <summary>
		/// Return the buffer to the cache.  The buffer should no longer be accessed.
		/// </summary>
		/// <param name="buffer">The buffer to be returned.</param>
		public static void Return(UhtByteBuffer buffer)
		{
			lock (s_lookAsideArray)
			{
				buffer.NextBuffer = s_lookAsideArray[buffer.Bucket];
				s_lookAsideArray[buffer.Bucket] = buffer;
			}
		}

		/// <summary>
		/// Internal helper to allocate a buffer
		/// </summary>
		/// <param name="size">The initial size of the buffer</param>
		/// <param name="bucket">The bucket associated with the buffer</param>
		/// <param name="bucketSize">The size all blocks in this bucket</param>
		/// <returns>The allocated buffer</returns>
		private static UhtByteBuffer BorrowInternal(int size, int bucket, int bucketSize)
		{
			lock (s_lookAsideArray)
			{
				if (s_lookAsideArray[bucket] != null)
				{
					UhtByteBuffer buffer = s_lookAsideArray[bucket]!;
					s_lookAsideArray[bucket] = buffer.NextBuffer;
					buffer.Reset(size);
					return buffer;
				}
			}
			return new UhtByteBuffer(size, bucket, bucketSize);
		}
	}

	/// <summary>
	/// Helper class for using pattern to borrow and return a buffer.
	/// </summary>
	[Obsolete("Use UhtRentedPoolBuffer<T> instead of UhtBorrowByteBuffer")]
	public struct UhtBorrowByteBuffer : IDisposable
	{

		/// <summary>
		/// The borrowed buffer
		/// </summary>
		public UhtByteBuffer Buffer { get; set; }

		/// <summary>
		/// Borrow a buffer with the given size
		/// </summary>
		/// <param name="size">The size to borrow</param>
		public UhtBorrowByteBuffer(int size)
		{
			Buffer = UhtByteBuffer.Borrow(size);
		}

		/// <summary>
		/// Return the borrowed buffer to the cache
		/// </summary>
		public readonly void Dispose()
		{
			UhtByteBuffer.Return(Buffer);
		}
	}
}
