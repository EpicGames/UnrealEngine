// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// C++ name that can be formatted
	/// </summary>
	public readonly struct UhtCppIdentifier : ISpanFormattable
	{
		/// <summary>
		/// Name of the namespace containing the definition with or without trailing "::"
		/// </summary>
		public string? Namespace { get; } = null;

		/// <summary>
		/// Optional prefix for the identifier
		/// </summary>
		public string? Prefix { get; } = null;

		/// <summary>
		/// Name of the identifier
		/// </summary>
		public string Name { get; } = String.Empty;

		/// <summary>
		/// Suffix for the identifier
		/// </summary>
		public string? Suffix { get; } = null;

		/// <summary>
		/// Construct a simple identifier
		/// </summary>
		/// <param name="name"></param>
		public UhtCppIdentifier(string name)
		{
			Name = name;
		}

		/// <summary>
		/// Construct an identifier with an optional prefix and suffix
		/// </summary>
		/// <param name="prefix"></param>
		/// <param name="name"></param>
		/// <param name="suffix"></param>
		public UhtCppIdentifier(string? prefix, string name, string? suffix)
		{
			Prefix = prefix;
			Name = name;
			Suffix = suffix;
		}

		/// <summary>
		/// Construct an identifier with an optional prefix and suffix
		/// </summary>
		/// <param name="ns">Namespace with or without trailing "::"</param>
		/// <param name="prefix"></param>
		/// <param name="name"></param>
		/// <param name="suffix"></param>
		public UhtCppIdentifier(string? ns, string? prefix, string name, string? suffix)
		{
			Namespace = ns;
			Prefix = prefix;
			Name = name;
			Suffix = suffix;
		}

		/// <summary>
		/// Construct an identifier with an optional prefix and suffix
		/// </summary>
		/// <param name="ns">Namespace with or without trailing "::"</param>
		/// <param name="id"></param>
		public UhtCppIdentifier(string ns, UhtCppIdentifier id)
		{
			Namespace = ns;
			Prefix = id.Prefix;
			Name = id.Name;
			Suffix = id.Suffix;
		}

		/// <summary>
		/// Return the "statics" version of the identifier
		/// </summary>
		/// <returns></returns>
		public UhtCppIdentifier MakeStatics()
		{
			return new(UhtNames.Statics, this);
		}

		/// <summary>
		/// Append the given suffix to the existing identifier and return the resulting identifier
		/// </summary>
		/// <param name="suffix"></param>
		/// <returns></returns>
		public UhtCppIdentifier AppendSuffix(string suffix)
		{
			return new(Namespace, Prefix, Name, String.IsNullOrEmpty(Suffix) ? suffix : $"{Suffix}{suffix}");
		}

		/// <inheritdoc/>
		public string ToString(string? format, IFormatProvider? formatProvider) => $"{this}";

		/// <inheritdoc/>
		public bool TryFormat(Span<char> destination, out int charsWritten, ReadOnlySpan<char> format, IFormatProvider? provider)
		{
			if (String.IsNullOrEmpty(Namespace))
			{
				return destination.TryWrite($"{Prefix}{Name}{Suffix}", out charsWritten);
			}
			else if (Namespace.EndsWith("::", StringComparison.Ordinal))
			{
				return destination.TryWrite($"{Namespace}{Prefix}{Name}{Suffix}", out charsWritten);
			}
			else
			{
				return destination.TryWrite($"{Namespace}::{Prefix}{Name}{Suffix}", out charsWritten);
			}
		}
	}
}
