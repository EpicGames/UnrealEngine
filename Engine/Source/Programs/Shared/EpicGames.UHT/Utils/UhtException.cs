// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.UHT.Tokenizer;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// Compiler exception that immediately stops the processing of the current header file.
	/// </summary>
	public class UhtException : Exception
	{
		/// <summary>
		/// The generated message
		/// </summary>
		public UhtMessage UhtMessage { get; set; }

		/// <summary>
		/// Internal do nothing constructor
		/// </summary>
		protected UhtException()
		{
		}

		/// <summary>
		/// Exception with a simple message.  Context will be the current header file.
		/// </summary>
		/// <param name="message">Text of the error</param>
		public UhtException(string message)
		{
			UhtMessage = UhtMessage.MakeMessage(UhtMessageType.Error, null, null, 1, message);
		}

		/// <summary>
		/// Exception with a simple message.  Context from the given message site.
		/// </summary>
		/// <param name="messageSite">Site generating the exception</param>
		/// <param name="message">Text of the error</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public UhtException(IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			if (extraContext != null)
			{
				message = $"{message} while parsing {UhtMessage.FormatContext(extraContext)}";
			}
			UhtMessage = UhtMessage.MakeMessage(UhtMessageType.Error, messageSite.MessageSource, null, messageSite.GetLineNumber(), message);
		}

		/// <summary>
		/// Make an exception to be thrown
		/// </summary>
		/// <param name="messageSite">Message site to be associated with the exception</param>
		/// <param name="lineNumber">Line number of the error</param>
		/// <param name="message">Text of the error</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public UhtException(IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			if (extraContext != null)
			{
				message = $"{message} while parsing {UhtMessage.FormatContext(extraContext)}";
			}
			UhtMessage = UhtMessage.MakeMessage(UhtMessageType.Error, messageSite.MessageSource, null, messageSite.GetLineNumber(lineNumber), message);
		}

		/// <inheritdoc/>
		public override string Message => UhtMessage.Message;
	}

	/// <summary>
	/// Exception where the current token isn't what was expected
	/// </summary>
	public class UhtTokenException : UhtException
	{

		/// <summary>
		/// Make a parsing error for when there is a mismatch between the expected token and what was parsed.
		/// </summary>
		/// <param name="messageSite">Message site to be associated with the exception</param>
		/// <param name="got">The parsed token.  Support for EOF also provided.</param>
		/// <param name="expected">What was expected.</param>
		/// <param name="extraContext">Extra context to be appended to the error message</param>
		/// <returns>The exception object to throw</returns>
		public UhtTokenException(IUhtMessageSite messageSite, UhtToken got, object? expected, object? extraContext = null)
		{
			string message = expected != null
				? $"Found {UhtMessage.FormatContext(got)} when expecting {UhtMessage.FormatContext(expected)}{FormatExtraContext(extraContext)}"
				: $"Found {UhtMessage.FormatContext(got)}{FormatExtraContext(extraContext)}";
			UhtMessage = UhtMessage.MakeMessage(UhtMessageType.Error, messageSite.MessageSource, null, got.InputLine, message);
		}

		/// <summary>
		/// Format any extra context supplied by the caller or the message site
		/// </summary>
		/// <param name="extraContext">Additional caller supplied context</param>
		/// <returns></returns>
		private static string FormatExtraContext(object? extraContext = null)
		{
			StringBuilder builder = new(" while parsing ");
			int startingLength = builder.Length;
			if (extraContext != null)
			{
				builder.Append(UhtMessage.FormatContext(extraContext));
			}
			UhtMessage.Append(builder, UhtTlsMessageExtraContext.GetMessageExtraContext(), startingLength != builder.Length);

			return builder.Length != startingLength ? builder.ToString() : String.Empty;
		}
	}

	/// <summary>
	/// Internal compiler error exception
	/// </summary>
	public class UhtIceException : UhtException
	{
		/// <summary>
		/// Exception with a simple message.  Context will be the current header file.
		/// </summary>
		/// <param name="message">Text of the error</param>
		public UhtIceException(string message)
		{
			UhtMessage = UhtMessage.MakeMessage(UhtMessageType.Ice, null, null, 1, message);
		}
	}
}
