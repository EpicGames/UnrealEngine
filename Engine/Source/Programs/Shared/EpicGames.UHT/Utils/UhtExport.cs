// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Delegate invoked by a factory
	/// </summary>
	/// <param name="factory">Invoking factory</param>
	public delegate void UhtExportTaskDelegate(IUhtExportFactory factory);

	/// <summary>
	/// Factory object used to generate export tasks
	/// </summary>
	public interface IUhtExportFactory
	{

		/// <summary>
		/// Session being run
		/// </summary>
		public UhtSession Session { get; }

		/// <summary>
		/// If this exporter is from a plugin, this points to the module of the plugin
		/// </summary>
		public UHTManifest.Module? PluginModule { get; }

		/// <summary>
		/// Create a task
		/// </summary>
		/// <param name="prereqs">Tasks that must be completed prior to this task running</param>
		/// <param name="action">Action to be invoked to generate the output(s)</param>
		/// <returns>Task object or Task.CompletedTask if the task was immediately executed.</returns>
		public Task CreateTask(List<Task?>? prereqs, UhtExportTaskDelegate action);

		/// <summary>
		/// Create a task
		/// </summary>
		/// <param name="action">Action to be invoked to generate the output(s)</param>
		/// <returns>Task object or Task.CompletedTask if the task was immediately executed.</returns>
		public Task CreateTask(UhtExportTaskDelegate action);

		/// <summary>
		/// Commit the contents of the string builder as the output.
		/// If you have a string builder, use this method so that a 
		/// temporary buffer can be used.
		/// </summary>
		/// <param name="filePath">Destination file path</param>
		/// <param name="builder">Source for the content</param>
		public void CommitOutput(string filePath, StringBuilder builder);

		/// <summary>
		/// Commit the value of the string as the output
		/// </summary>
		/// <param name="filePath">Destination file path</param>
		/// <param name="output">Output to commit</param>
		public void CommitOutput(string filePath, StringView output);

		/// <summary>
		/// Make a path for an output based on the header file name.
		/// </summary>
		/// <param name="headerFile">Header file being exported.</param>
		/// <param name="suffix">Suffix to be added to the file name.</param>
		/// <returns>Output file path</returns>
		public string MakePath(UhtHeaderFile headerFile, string suffix);

		/// <summary>
		/// Make a path for an output based on the module name.
		/// </summary>
		/// <param name="moduleObj">Module being exported</param>
		/// <param name="suffix">Suffix to be added to the file name.</param>
		/// <returns>Output file path</returns>
		public string MakePath(UhtModule moduleObj, string suffix);

		/// <summary>
		/// Make a path for the given file name and extension.  This is only valid for plugins.
		/// </summary>
		/// <param name="fileName">Name of the file</param>
		/// <param name="extension">Extension to add to the file</param>
		/// <returns>Output file path</returns>
		public string MakePath(string fileName, string extension);

		/// <summary>
		/// Add an external dependency to the given file path
		/// </summary>
		/// <param name="filePath">External dependency to add</param>
		public void AddExternalDependency(string filePath);

		/// <summary>
		/// Return the shortest relative include path
		/// </summary>
		/// <param name="moduleObj">Module in question</param>
		/// <param name="filePath">File path</param>
		/// <returns>Shortest include path relative to the given file path</returns>
		public string GetModuleShortestIncludePath(UhtModule moduleObj, string filePath);

		/// <summary>
		/// Return the shortest relative include path
		/// </summary>
		/// <param name="filePath">File path</param>
		/// <returns>Shortest include path relative to the given file path</returns>
		public string GetPluginShortestIncludePath(string filePath);
	}
}
