// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using EpicGames.Core;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Collection of helper methods to emulate UHT string functions
	/// </summary>
	public static class UhtFCString
	{

		/// <summary>
		/// Delimiter between the subobjects
		/// </summary>
		public static readonly char SubObjectDelimiter = ':';

		/// <summary>
		/// Test to see if the string is a boolean
		/// </summary>
		/// <param name="value">Boolean to test</param>
		/// <returns>True if the value is true</returns>
		public static bool ToBool(StringView value)
		{
			ReadOnlySpan<char> span = value.Span;
			if (span.Length == 0)
			{
				return false;
			}
			else if (
				span.Equals("true", StringComparison.OrdinalIgnoreCase) ||
				span.Equals("yes", StringComparison.OrdinalIgnoreCase) ||
				span.Equals("on", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			else if (
				span.Equals("false", StringComparison.OrdinalIgnoreCase) ||
				span.Equals("no", StringComparison.OrdinalIgnoreCase) ||
				span.Equals("off", StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}
			else
			{
				return Int32.TryParse(span, out int intValue) && intValue != 0;
			}
		}

		/// <summary>
		/// Test to see if the string is a boolean
		/// </summary>
		/// <param name="value">Boolean to test</param>
		/// <returns>True if the value is true</returns>
		public static bool ToBool(string? value)
		{
			if (String.IsNullOrEmpty(value))
			{
				return false;
			}
			else if (String.Equals(value, "true", StringComparison.OrdinalIgnoreCase) ||
				String.Equals(value, "yes", StringComparison.OrdinalIgnoreCase) ||
				String.Equals(value, "on", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			else if (String.Equals(value, "false", StringComparison.OrdinalIgnoreCase) ||
				String.Equals(value, "no", StringComparison.OrdinalIgnoreCase) ||
				String.Equals(value, "off", StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}
			else
			{
				return Int32.TryParse(value, out int intValue) && intValue != 0;
			}
		}

		/// <summary>
		/// Test to see if the character is a digit
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsDigit(char c)
		{
			return c >= '0' && c <= '9';
		}

		/// <summary>
		/// Test to see if the character is an alphabet character
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsAlpha(char c)
		{
			return (c >= 'A' && c <= 'Z') || (c >= 'a' && c <= 'z');
		}

		/// <summary>
		/// Test to see if the character is a digit or alphabet character
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsAlnum(char c)
		{
			return IsDigit(c) || IsAlpha(c);
		}

		/// <summary>
		/// Test to see if the character is a hex digit (A-F)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsHexAlphaDigit(char c)
		{
			return (c >= 'A' && c <= 'F') || (c >= 'a' && c <= 'f');
		}

		/// <summary>
		/// Test to see if the character is a hex digit (0-9a-f)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsHexDigit(char c)
		{
			return IsDigit(c) || IsHexAlphaDigit(c);
		}

		/// <summary>
		/// Test to see if the character is whitespace
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsWhitespace(char c)
		{
			return Char.IsWhiteSpace(c);
		}

		/// <summary>
		/// Test to see if the character is a sign
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsSign(char c)
		{
			return c == '+' || c == '-';
		}

		/// <summary>
		/// Test to see if the character is a hex marker (xX)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsHexMarker(char c)
		{
			return c == 'x' || c == 'X';
		}

		/// <summary>
		/// Test to see if the character is a float marker (fF)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsFloatMarker(char c)
		{
			return c == 'f' || c == 'F';
		}

		/// <summary>
		/// Test to see if the character is an exponent marker (eE)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsExponentMarker(char c)
		{
			return c == 'e' || c == 'E';
		}

		/// <summary>
		/// Test to see if the character is an unsigned marker (uU)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsUnsignedMarker(char c)
		{
			return c == 'u' || c == 'U';
		}

		/// <summary>
		/// Test to see if the character is a long marker (lL)
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsLongMarker(char c)
		{
			return c == 'l' || c == 'L';
		}

		/// <summary>
		/// Test to see if the span is a numeric value
		/// </summary>
		/// <param name="span">Span to test</param>
		/// <returns>True if it is</returns>
		public static bool IsNumeric(ReadOnlySpan<char> span)
		{
			int index = 0;
			char c = span[index];
			if (c == '-' || c == '+')
			{
				++index;
			}

			bool hasDot = false;
			for (; index < span.Length; ++index)
			{
				c = span[index];
				if (c == '.')
				{
					if (hasDot)
					{
						return false;
					}
					hasDot = true;
				}
				else if (!IsDigit(c))
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Test to see if the character is a linebreak character
		/// </summary>
		/// <param name="c">Character to test</param>
		/// <returns>True if it is</returns>
		public static bool IsLinebreak(char c)
		{
			return (c >= 0x0a && c <= 0x0d) || c == 0x85 || c == 0x2028 || c == 0x2029;
		}

		/// <summary>
		/// Return an unescaped string
		/// </summary>
		/// <param name="text">Text to unescape</param>
		/// <returns>Resulting string</returns>
		public static string UnescapeText(string text)
		{
			StringBuilder result = new();
			for (int idx = 0; idx < text.Length; idx++)
			{
				if (text[idx] == '\\' && idx + 1 < text.Length)
				{
					switch (text[++idx])
					{
						case 't':
							result.Append('\t');
							break;
						case 'r':
							result.Append('\r');
							break;
						case 'n':
							result.Append('\n');
							break;
						case '\'':
							result.Append('\'');
							break;
						case '\"':
							result.Append('\"');
							break;
						case 'u':
							bool parsed = false;
							if (idx + 4 < text.Length)
							{
								if (Int32.TryParse(text.Substring(idx + 1, 4), System.Globalization.NumberStyles.HexNumber, null, out int value))
								{
									parsed = true;
									result.Append((char)value);
									idx += 4;
								}
							}
							if (!parsed)
							{
								result.Append('\\');
								result.Append('u');
							}
							break;
						default:
							result.Append(text[idx]);
							break;
					}
				}
				else
				{
					result.Append(text[idx]);
				}
			}
			return result.ToString();
		}

		/// <summary>
		/// Replace tabs to spaces in a string containing only a single line.
		/// </summary>
		/// <param name="input">Input string</param>
		/// <param name="tabSpacing">Number of spaces to exchange for tabs</param>
		/// <param name="emulateCrBug">Due to a bug in UE ConvertTabsToSpacesInline, any \n is considered part of the line length.</param>
		/// <returns>Resulting string or the original string if the string didn't contain any spaces.</returns>
		public static ReadOnlyMemory<char> TabsToSpaces(ReadOnlyMemory<char> input, int tabSpacing, bool emulateCrBug)
		{
			// If we have any tab characters, then we need to convert them to spaces
			int tabIndex = input.Span.IndexOf('\t');
			if (tabIndex == -1)
			{
				return input;
			}
			using BorrowStringBuilder borrower = new(StringBuilderCache.Small);
			StringBuilder builder = borrower.StringBuilder;
			TabsToSpaces(input.Span, tabSpacing, emulateCrBug, tabIndex, builder);
			return builder.ToString().AsMemory();
		}

		/// <summary>
		/// Replace tabs to spaces in a string containing zero or more lines.
		/// </summary>
		/// <param name="span">Input string to convert</param>
		/// <param name="tabSpacing">Number of spaces to exchange for tabs</param>
		/// <param name="emulateCrBug">Due to a bug in UE ConvertTabsToSpacesInline, any \n is considered part of the line length.</param>
		/// <param name="tabIndex">Initial tab index</param>
		/// <param name="builder">Destination string builder</param>
		public static void TabsToSpaces(ReadOnlySpan<char> span, int tabSpacing, bool emulateCrBug, int tabIndex, StringBuilder builder)
		{
			// Locate the last \n since all tabs have to be computed relative to 
			int crPos = span[..tabIndex].LastIndexOf('\n') + 1;

			int committed = 0;
			do
			{
				// Commit everything prior to the tab to the builder
				builder.Append(span[..tabIndex]);
				span = span[(tabIndex + 1)..];
				committed += tabIndex;

				// Add the appropriate number of spaces
				int adjustedCrPos = crPos;
				if (emulateCrBug && adjustedCrPos > 0)
				{
					--adjustedCrPos;
				}
				int spacesToInsert = tabSpacing - (committed - adjustedCrPos) % tabSpacing;
				builder.AppendSpaces(spacesToInsert);
				committed += spacesToInsert;

				// Search for the next \t or \n
				for (tabIndex = 0; tabIndex < span.Length; ++tabIndex)
				{
					if (span[tabIndex] == '\n')
					{
						crPos = committed + tabIndex + 1;
					}
					else if (span[tabIndex] == '\t')
					{
						break;
					}
				}
			} while (tabIndex < span.Length);

			// Commit the remaining data
			builder.Append(span);
		}
	}
}
