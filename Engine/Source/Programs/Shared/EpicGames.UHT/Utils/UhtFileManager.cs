// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;
using EpicGames.Core;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Interface used to read/write files
	/// </summary>
	public interface IUhtFileManager
	{
		/// <summary>
		/// Return the full file path for a partial path
		/// </summary>
		/// <param name="filePath">The partial file path</param>
		/// <returns>The full file path</returns>
		public string GetFullFilePath(string filePath);

		/// <summary>
		/// Read the given source file
		/// </summary>
		/// <param name="filePath">File path</param>
		/// <param name="fragment">Read fragment information</param>
		/// <returns>True if the file was read</returns>
		public bool ReadSource(string filePath, out UhtSourceFragment fragment);

		/// <summary>
		/// Read the given source file
		/// </summary>
		/// <param name="filePath">File path</param>
		/// <param name="output">Read output</param>
		/// <returns>True if the file was read</returns>
		public bool ReadOutput(string filePath, out UhtPoolBuffer<char> output);

		/// <summary>
		/// Write the given contents to the file
		/// </summary>
		/// <param name="filePath">Path to write to</param>
		/// <param name="contents">Contents to write</param>
		/// <returns>True if the source was written</returns>
		public bool WriteOutput(string filePath, ReadOnlySpan<char> contents);

		/// <summary>
		/// Rename the given file
		/// </summary>
		/// <param name="oldFilePath">Old file path name</param>
		/// <param name="newFilePath">New file path name</param>
		/// <returns>True if the file was renamed</returns>
		public bool RenameOutput(string oldFilePath, string newFilePath);
	}

	/// <summary>
	/// Implementation of a file manager that reads/writes from disk
	/// </summary>
	public class UhtStdFileManager : IUhtFileManager
	{
		/// <summary>
		/// Construct a new file manager
		/// </summary>
		public UhtStdFileManager()
		{
		}

		/// <inheritdoc/>
		public string GetFullFilePath(string filePath)
		{
			return filePath;
		}

		/// <inheritdoc/>
		public bool ReadSource(string filePath, out UhtSourceFragment fragment)
		{
			if (ReadFile(filePath, out StringView data))
			{
				fragment = new UhtSourceFragment { SourceFile = null, FilePath = filePath, LineNumber = 0, Data = data };
				return true;
			}
			fragment = new UhtSourceFragment { SourceFile = null, FilePath = String.Empty, LineNumber = 0, Data = new StringView() };
			return false;
		}

		/// <inheritdoc/>
		public bool ReadOutput(string filePath, out UhtPoolBuffer<char> output)
		{
			// Exceptions are very expensive.  Don't bother trying to open the file if it doesn't exist
			if (!File.Exists(filePath))
			{
				output = default;
				return false;
			}

			try
			{
				using FileStream fs = new(filePath, FileMode.Open, FileAccess.Read, FileShare.Read, 1, FileOptions.SequentialScan);
				using UhtRentedPoolBuffer<byte> byteBuffer = new((int)fs.Length);
				Span<byte> byteSpan = byteBuffer.Buffer.Memory.Span;
				int readLength = fs.Read(byteSpan);
				byteSpan = byteSpan[..readLength];

				Encoding encoding = FileUtils.GetEncoding(byteSpan, out int skipBytes);
				byteSpan = byteSpan[skipBytes..];

				int charCount = encoding.GetCharCount(byteBuffer.Buffer.Memory.Span);
				output = UhtPoolBuffers.Rent<char>(charCount);
				encoding.GetChars(byteBuffer.Buffer.Memory.Span, output.Memory.Span);
				return true;
			}
			catch (IOException)
			{
				output = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public bool WriteOutput(string filePath, ReadOnlySpan<char> contents)
		{
			try
			{
				string? fileDirectory = Path.GetDirectoryName(filePath);
				if (!String.IsNullOrEmpty(fileDirectory))
				{
					Directory.CreateDirectory(fileDirectory);
				}
				using StreamWriter writer = new(filePath, false, new UTF8Encoding(false, true), 16 * 1024);
				writer.Write(contents);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		/// <inheritdoc/>
		public bool RenameOutput(string oldFilePath, string newFilePath)
		{
			try
			{
				File.Move(oldFilePath, newFilePath, true);
				return true;
			}
			catch (Exception)
			{
				return false;
			}
		}

		/// <summary>
		/// Read the given source file
		/// </summary>
		/// <param name="filePath">Full file path</param>
		/// <param name="contents">Contents of the file</param>
		/// <returns>True if the file was read, false if not</returns>
		private static bool ReadFile(string filePath, out StringView contents)
		{
			// Exceptions are very expensive.  Don't bother trying to open the file if it doesn't exist
			if (!File.Exists(filePath))
			{
				contents = new StringView();
				return false;
			}

			try
			{
				using FileStream fs = new(filePath, FileMode.Open, FileAccess.Read, FileShare.Read, 1, FileOptions.SequentialScan);
				using UhtRentedPoolBuffer<byte> byteBuffer = new((int)fs.Length);
				Span<byte> byteSpan = byteBuffer.Buffer.Memory.Span;
				int readLength = fs.Read(byteSpan);
				byteSpan = byteSpan[..readLength];

				Encoding encoding = FileUtils.GetEncoding(byteSpan, out int skipBytes);
				byteSpan = byteSpan[skipBytes..];

				int charCount = encoding.GetCharCount(byteSpan);
				char[] initialBuffer = new char[charCount];
				encoding.GetChars(byteSpan, initialBuffer);
				contents = new StringView(new ReadOnlyMemory<char>(initialBuffer, 0, charCount));
				return true;
			}
			catch (IOException)
			{
				contents = new StringView();
				return false;
			}
		}
	}
}
