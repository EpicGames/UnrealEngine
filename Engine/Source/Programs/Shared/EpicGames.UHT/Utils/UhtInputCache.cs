// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;
using Microsoft.Extensions.Logging;

namespace EpicGames.UHT.Utils
{
	enum UhtInputCacheMarker : int
	{
		EndOfType = 0x12340000,
		EndOfTypeBase = 0x12340001,
	}

	/// <summary>
	/// Tracks an input file
	/// </summary>
	struct UhtInputFile
	{
		/// <summary>
		/// Name of the file
		/// </summary>
		public string FileName { get; set; }

		/// <summary>
		/// File write time
		/// </summary>
		public long LastWriteTimeUtc { get; set; }

		/// <summary>
		/// Size of the uncompressed data
		/// </summary>
		public int UncompressedSize { get; set; }

		/// <summary>
		/// Starting position in the cache file for this file
		/// </summary>
		public long CacheStart { get; set; }

		/// <summary>
		/// Ending position in the cache file for this file
		/// </summary>
		public long CacheEnd { get; set; }

		/// <summary>
		/// Construct an instance from a reader
		/// </summary>
		/// <param name="reader"></param>
		public UhtInputFile(IMemoryReader reader)
		{
			FileName = reader.ReadString()!;
			LastWriteTimeUtc = reader.ReadInt64();
			UncompressedSize = reader.ReadInt32();
			CacheStart = reader.ReadInt64();
			CacheEnd = reader.ReadInt64();
		}

		/// <summary>
		/// Write the input file information
		/// </summary>
		/// <param name="writer">Destination writer</param>
		public void Write(IMemoryWriter writer)
		{
			writer.WriteString(FileName);
			writer.WriteInt64(LastWriteTimeUtc);
			writer.WriteInt32(UncompressedSize);
			writer.WriteInt64(CacheStart);
			writer.WriteInt64(CacheEnd);
		}
	}

	/// <summary>
	/// Populate the types from a cache entry
	/// </summary>
	public sealed class UhtInputCacheReader : IDisposable
	{
		/// <summary>
		/// Header file being written
		/// </summary>
		public UhtHeaderFile Header { get; init; }
		private readonly MemoryReader _reader;
		private readonly List<UhtPackage> _packages = [];
		private readonly List<ConstructorInfo> _types = [];
		private readonly List<UhtNamespace> _namespaces = [];
		private readonly List<UhtType> _objects = [];
		private readonly byte[] _uncompressed;

		/// <summary>
		/// Construct a reader from a byte buffer
		/// </summary>
		/// <param name="header">Header being read</param>
		/// <param name="entry">Cache entry</param>
		public UhtInputCacheReader(UhtHeaderFile header, UhtInputCacheEntry entry)
		{
			Header = header;
			_uncompressed = ArrayPool<byte>.Shared.Rent(entry.UncompressedSize);
			Span<byte> uncompressedSpan = _uncompressed.AsSpan()[..entry.UncompressedSize];
			if (!BrotliDecoder.TryDecompress(entry.CompressedData.Span, uncompressedSpan, out int bytesWritten))
			{
				throw new UhtException($"Error uncompressing input cache buffer for file '{Header.FilePath}'");
			}
			if (bytesWritten != entry.UncompressedSize)
			{
				throw new UhtException($"Input cache buffer uncompressed size mismatch for file '{Header.FilePath}'");
			}
			_reader = new(_uncompressed.AsMemory()[..entry.UncompressedSize]);
		}

		/// <summary>
		/// Read a boolean
		/// </summary>
		/// <returns>Read value</returns>
		public bool ReadBoolean()
		{
			return _reader.ReadBoolean();
		}

		/// <summary>
		/// Reads a single byte from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public sbyte ReadInt8()
		{
			return _reader.ReadInt8();
		}

		/// <summary>
		/// Read an short
		/// </summary>
		/// <returns>Read value</returns>
		public short ReadInt16()
		{
			return _reader.ReadInt16();
		}

		/// <summary>
		/// Read an integer
		/// </summary>
		/// <returns>Read value</returns>
		public int ReadInt32()
		{
			return _reader.ReadInt32();
		}

		/// <summary>
		/// Read an long
		/// </summary>
		/// <returns>Read value</returns>
		public long ReadInt64()
		{
			return _reader.ReadInt64();
		}

		/// <summary>
		/// Reads a single byte from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public byte ReadUInt8()
		{
			return _reader.ReadUInt8();
		}

		/// <summary>
		/// Reads a single unsigned short from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public ushort ReadUInt16()
		{
			return _reader.ReadUInt16();
		}

		/// <summary>
		/// Reads a single unsigned int from the stream
		/// </summary>
		/// <returns>The value that was read</returns>
		public uint ReadUInt32()
		{
			return _reader.ReadUInt32();
		}

		/// <summary>
		/// Read an unsigned long
		/// </summary>
		/// <returns>Read value</returns>
		public ulong ReadUInt64()
		{
			return _reader.ReadUInt64();
		}

		/// <summary>
		/// Read a string
		/// </summary>
		/// <returns>Read value</returns>
		public string ReadString()
		{
			return _reader.ReadString();
		}

		/// <summary>
		/// Read a string
		/// </summary>
		/// <returns>Read value</returns>
		public string? ReadOptionalString()
		{
			return _reader.ReadBoolean() ? _reader.ReadString() : null;
		}

		/// <summary>
		/// Reads a byte array from the stream
		/// </summary>
		/// <returns>The data that was read</returns>
		public ReadOnlyMemory<byte>? ReadVariableLengthBytes()
		{
			return _reader.ReadVariableLengthBytes();
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="length">Length of the array to read</param>
		/// <param name="readItem">Delegate to read an individual item</param>
		public T[] ReadFixedLengthArray<T>(int length, Func<UhtInputCacheReader, T> readItem)
		{
			if (length == 0)
			{
				return [];
			}

			T[] array = new T[length];
			for (int idx = 0; idx < length; idx++)
			{
				array[idx] = readItem(this);
			}
			return array;
		}

		/// <summary>
		/// Reads an array of items
		/// </summary>
		/// <param name="readItem">Delegate to read an individual item</param>
		/// <returns>New array</returns>
		public T[] ReadVariableLengthArray<T>(Func<UhtInputCacheReader, T> readItem)
		{
			int length = _reader.ReadInt32();
			return ReadFixedLengthArray(length, readItem);
		}

		/// <summary>
		/// Reads an array of items
		/// </summary>
		/// <param name="readItem">Delegate to read an individual item</param>
		/// <returns>New array</returns>
		public T[]? ReadOptionalVariableLengthArray<T>(Func<UhtInputCacheReader, T> readItem)
		{
			int length = _reader.ReadInt32();
			if (length == -1)
			{
				return [];
			}
			return ReadFixedLengthArray(length, readItem);
		}

		/// <summary>
		/// Reads a list of items
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="length">Number of items in the list</param>
		/// <param name="readItem">Delegate used to read a single element</param>
		/// <returns>List of items</returns>
		public List<T> ReadList<T>(int length, Func<UhtInputCacheReader, T> readItem)
		{
			List<T> list = [];
			list.EnsureCapacity(list.Count + length);

			for (int idx = 0; idx < length; idx++)
			{
				list.Add(readItem(this));
			}
			return list;
		}

		/// <summary>
		/// Reads a list of items
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="readItem">Delegate used to read a single element</param>
		/// <returns>List of items</returns>
		public List<T> ReadList<T>(Func<UhtInputCacheReader, T> readItem)
		{
			int length = (int)_reader.ReadInt32();
			return ReadList(length, readItem);
		}

		/// <summary>
		/// Reads a list of items
		/// </summary>
		/// <typeparam name="T">The element type for the list</typeparam>
		/// <param name="readItem">Delegate used to read a single element</param>
		/// <returns>List of items</returns>
		public List<T>? ReadOptionalList<T>(Func<UhtInputCacheReader, T> readItem)
		{
			int length = (int)_reader.ReadInt32();
			if (length == -1)
			{
				return null;
			}
			return ReadList(length, readItem);
		}

		/// <summary>
		/// Reads an object, which may be null, from the archive. Does not handle de-duplicating object references. 
		/// </summary>
		/// <typeparam name="T">Type of the object to read</typeparam>
		/// <param name="read">Delegate used to read the object</param>
		/// <returns>The object instance</returns>
		public T? ReadOptionalObject<T>(Func<UhtInputCacheReader, T> read) where T : class
		{
			return _reader.ReadBoolean() ? read(this) : null;
		}

		/// <summary>
		/// Read a token array
		/// </summary>
		/// <returns>Collection of tokens</returns>
		public UhtToken[] ReadTokenArray()
		{
			return ReadVariableLengthArray((writer) => new UhtToken(writer));
		}

		/// <summary>
		/// Read a token array
		/// </summary>
		/// <returns>Collection of tokens</returns>
		public UhtToken[]? ReadOptionalTokenArray()
		{
			return ReadOptionalVariableLengthArray((writer) => new UhtToken(writer));
		}

		/// <summary>
		/// Read a token list
		/// </summary>
		/// <returns>Collection of tokens</returns>
		public List<UhtToken> ReadTokenList()
		{
			return ReadList((writer) => new UhtToken(writer));
		}

		/// <summary>
		/// Read a token list
		/// </summary>
		/// <returns>Collection of tokens</returns>
		public List<UhtToken>? ReadOptionalTokenList()
		{
			return ReadOptionalList((writer) => new UhtToken(writer));
		}

		/// <summary>
		/// Verify that the marker is set as expected
		/// </summary>
		/// <param name="marker"></param>
		private void CheckMarker(UhtInputCacheMarker marker)
		{
			UhtInputCacheMarker value = (UhtInputCacheMarker)_reader.ReadInt32();
			if (value != marker)
			{
				throw new UhtException($"Input cache definition marker missing: Expected = {(int)marker}, Got = {(int)value}");
			}
		}

		/// <summary>
		/// Read the contents of the header
		/// </summary>
		public void ReadHeader()
		{
			Header.Read(this);

			int count = _reader.ReadInt32();
			for (int index = 0; index < count; index++)
			{
				UhtType type = ReadType()!;
				Header.AddChild(type);
			}
		}

		/// <summary>
		/// Read a namespace from the stream
		/// </summary>
		public UhtNamespace ReadNamespace()
		{
			int index = _reader.ReadInt32();
			if (index == -1)
			{
				return Header.Session.GlobalNamespace;
			}
			if (index < -1 || index > _namespaces.Count)
			{
				throw new UhtException("Unexpected index into namespace array");
			}
			if (index < _namespaces.Count)
			{
				return _namespaces[index];
			}

			string fullName = _reader.ReadString();
			UhtNamespace namespaceObj = Header.Session.GetNamespace(fullName);
			_namespaces.Add(namespaceObj);
			return namespaceObj;
		}

		/// <summary>
		/// Read a package from the stream
		/// </summary>
		private UhtPackage ReadPackage()
		{
			int index = _reader.ReadInt32();
			if (index == -1)
			{
				return Header.Module.ScriptPackage;
			}
			if (index < -1 || index > _packages.Count)
			{
				throw new UhtException("Unexpected index into package array");
			}
			if (index < _packages.Count)
			{
				return _packages[index];
			}

			string packageName = _reader.ReadString();
			EPackageFlags flags = (EPackageFlags)_reader.ReadUInt64();
			UhtPackage package = Header.Module.CreatePackage(packageName, flags);
			_packages.Add(package);
			return package;
		}

		/// <summary>
		/// Read a type from the stream
		/// </summary>
		/// <returns>The read type</returns>
		public UhtType? ReadType()
		{
			int typeIndex = _reader.ReadInt32();
			if (typeIndex == -1)
			{
				return null;
			}
			if (typeIndex < -1 || typeIndex > _objects.Count)
			{
				throw new UhtException("Unexpected index into object array");
			}
			if (typeIndex < _objects.Count)
			{
				return _objects[typeIndex];
			}

			// Read the constructor 
			ConstructorInfo constructor = ReadTypeConstructor();

			// Read the outer
			UhtType? outer = _reader.ReadBoolean() ? ReadPackage() : ReadType();
			if (outer == null)
			{
				throw new UhtException("Unexpected to have read an outer type");
			}

			// Construct the type
			UhtType type = (UhtType)constructor.Invoke([this, outer]);
			if (typeIndex >= _objects.Count || _objects[typeIndex] != type)
			{
				throw new UhtException("Unexpected entry in the object array");
			}
			CheckMarker(UhtInputCacheMarker.EndOfTypeBase);

			// Read the children
			if (type is not UhtProperty)
			{
				int count = _reader.ReadInt32();
				for (int childIndex = 0; childIndex < count; childIndex++)
				{
					UhtType child = ReadType()!;
					if (child.Outer != type)
					{
						throw new UhtException("Mismatch in deserialized outer");
					}
					type.AddChild(child);
				}
			}
			CheckMarker(UhtInputCacheMarker.EndOfType);
			return type;
		}

		/// <summary>
		/// Invoked by UhtType::UhtType to add the type to the table of objects.
		/// This is required just in case a constructor needs to write other types that reference
		/// this type.
		/// </summary>
		/// <param name="type">Type being added</param>
		public void AddType(UhtType type)
		{
			_objects.Add(type);
		}

		private ConstructorInfo ReadTypeConstructor()
		{
			int index = _reader.ReadInt32();
			if (index < 0 || index > _types.Count)
			{
				throw new UhtException("Unexpected index into types array");
			}
			if (index < _types.Count)
			{
				return _types[index];
			}

			string typeName = _reader.ReadString();
			Type? type = Type.GetType(typeName);
			if (type == null)
			{
				throw new UhtException($"Unable to find type {type}");
			}

			ConstructorInfo? constructor = type.GetConstructor([typeof(UhtInputCacheReader), typeof(UhtType)]);
			if (constructor == null)
			{
				throw new UhtException($"Unable to find constructor for type {type}");
			}
			_types.Add(constructor);
			return constructor;
		}

		/// <summary>
		/// Dispose of this object
		/// </summary>
		public void Dispose()
		{
			ArrayPool<byte>.Shared.Return(_uncompressed);
		}
	}

	/// <summary>
	/// Wrapper class for writing to the input cache
	/// </summary>
	public sealed class UhtInputCacheWriter : IDisposable
	{
		/// <summary>
		/// Header file being written
		/// </summary>
		public UhtHeaderFile Header { get; init; }
		private readonly ChunkedMemoryWriter _writer;
		private readonly Dictionary<UhtPackage, int> _packages = [];
		private readonly Dictionary<Type, int> _types = [];
		private readonly Dictionary<UhtNamespace, int> _namespaces = [];
		private readonly Dictionary<UhtType, int> _objects = [];

		/// <summary>
		/// Construct a writer from a stream
		/// </summary>
		/// <param name="header">Header file being written</param>
		public UhtInputCacheWriter(UhtHeaderFile header)
		{
			Header = header;
			_writer = new ChunkedMemoryWriter(128 * 1024, 128 * 1024);
		}

		/// <summary>
		/// Writes a bool to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteBoolean(bool value)
		{
			_writer.WriteBoolean(value);
		}

		/// <summary>
		/// Writes a single signed byte to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteInt8(sbyte value)
		{
			_writer.WriteInt8(value);
		}

		/// <summary>
		/// Writes a single short to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteInt16(short value)
		{
			_writer.WriteInt16(value);
		}

		/// <summary>
		/// Writes a single int to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteInt32(int value)
		{
			_writer.WriteInt32(value);
		}

		/// <summary>
		/// Writes a single long to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteInt64(long value)
		{
			_writer.WriteInt64(value);
		}

		/// <summary>
		/// Writes a single byte to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUInt8(byte value)
		{
			_writer.WriteUInt8(value);
		}

		/// <summary>
		/// Writes a single unsigned short to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUInt16(ushort value)
		{
			_writer.WriteUInt16(value);
		}

		/// <summary>
		/// Writes a single unsigned int to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUInt32(ushort value)
		{
			_writer.WriteUInt32(value);
		}

		/// <summary>
		/// Writes a single unsigned long to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteUInt64(ulong value)
		{
			_writer.WriteUInt64(value);
		}

		/// <summary>
		/// Writes a string to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteString(string value)
		{
			_writer.WriteString(value);
		}

		/// <summary>
		/// Writes a string to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteOptionalString(string? value)
		{
			if (value == null)
			{
				_writer.WriteBoolean(false);
			}
			else
			{
				_writer.WriteBoolean(true);
				_writer.WriteString(value);
			}
		}

		/// <summary>
		/// Writes a string to the output
		/// </summary>
		/// <param name="value">Value to write</param>
		public void WriteString(ReadOnlySpan<char> value)
		{
			Encoding encoding = Encoding.UTF8;
			int stringBytes = encoding.GetByteCount(value);
			int lengthBytes = VarInt.MeasureUnsigned(stringBytes);

			Span<byte> span = _writer.GetSpan(lengthBytes + stringBytes);
			VarInt.WriteUnsigned(span, stringBytes);
			encoding.GetBytes(value, span[lengthBytes..]);

			_writer.Advance(lengthBytes + stringBytes);
		}

		/// <summary>
		/// Writes an array of bytes to the output
		/// </summary>
		/// <param name="data">Data to write. May be null.</param>
		public void WriteVariableLengthBytes(ReadOnlySpan<byte> data)
		{
			_writer.WriteVariableLengthBytes(data);
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public void WriteFixedLengthArray<T>(IReadOnlyList<T> list, Action<UhtInputCacheWriter, T> writeItem)
		{
			for (int idx = 0; idx < list.Count; idx++)
			{
				writeItem(this, list[idx]);
			}
		}

		/// <summary>
		/// Writes a fixed length array
		/// </summary>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public void WriteFixedLengthArray<T>(ReadOnlySpan<T> list, Action<UhtInputCacheWriter, T> writeItem)
		{
			for (int idx = 0; idx < list.Length; idx++)
			{
				writeItem(this, list[idx]);
			}
		}

		/// <summary>
		/// Write an array of items to the archive
		/// </summary>
		/// <typeparam name="T">Type of the element</typeparam>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public void WriteVariableLengthArray<T>(IReadOnlyList<T> list, Action<UhtInputCacheWriter, T> writeItem)
		{
			_writer.WriteInt32(list.Count);
			WriteFixedLengthArray<T>(list, writeItem);
		}

		/// <summary>
		/// Write an array of items to the archive
		/// </summary>
		/// <typeparam name="T">Type of the element</typeparam>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public void WriteVariableLengthArray<T>(ReadOnlySpan<T> list, Action<UhtInputCacheWriter, T> writeItem)
		{
			_writer.WriteInt32(list.Length);
			WriteFixedLengthArray<T>(list, writeItem);
		}

		/// <summary>
		/// Write an array of items to the archive
		/// </summary>
		/// <typeparam name="T">Type of the element</typeparam>
		/// <param name="list">The array to write</param>
		/// <param name="writeItem">Delegate to write an individual item</param>
		public void WriteOptionalVariableLengthArray<T>(IReadOnlyList<T>? list, Action<UhtInputCacheWriter, T> writeItem)
		{
			if (list == null)
			{
				_writer.WriteInt32(-1);
				return;
			}
			_writer.WriteInt32(list.Count);
			WriteFixedLengthArray<T>(list, writeItem);
		}

		/// <summary>
		/// Write a token array
		/// </summary>
		/// <param name="tokens">Collection of tokens</param>
		public void WriteVariableLengthTokenArray(ReadOnlySpan<UhtToken> tokens)
		{
			WriteVariableLengthArray(tokens, (writer, token) => token.Write(writer));
		}

		/// <summary>
		/// Write a token array
		/// </summary>
		/// <param name="tokens">Collection of tokens</param>
		public void WriteOptionalVariableLengthTokenArray(IReadOnlyList<UhtToken>? tokens)
		{
			if (tokens == null)
			{
				_writer.WriteInt32(-1);
				return;
			}
			WriteVariableLengthArray(tokens, (writer, token) => token.Write(writer));
		}

		/// <summary>
		/// Writes an object to the output, checking whether it is null or not. Does not preserve object references; each object written is duplicated.
		/// </summary>
		/// <typeparam name="T">Type of the object to serialize</typeparam>
		/// <param name="obj">Reference to check for null before serializing</param>
		/// <param name="writeObject">Delegate used to write the object</param>
		public void WriteOptionalObject<T>(T? obj, Action<UhtInputCacheWriter> writeObject) where T : class
		{
			if (obj == null)
			{
				_writer.WriteBoolean(false);
			}
			else
			{
				_writer.WriteBoolean(true);
				writeObject(this);
			}
		}

		/// <summary>
		/// Write the marker to the stream (debug only)
		/// </summary>
		/// <param name="marker"></param>
		private void WriteMarker(UhtInputCacheMarker marker)
		{
			_writer.WriteInt32((int)marker);
		}

		/// <summary>
		/// Write the contents of the header
		/// </summary>
		/// <param name="lastWriteTimeUtc">Last write time of the header file</param>
		public UhtInputCacheEntry WriteHeader(DateTime lastWriteTimeUtc)
		{
			Header.Write(this);
			WriteChildren(Header.Children);

			int length = _writer.Length;
			byte[]? writtenBytes = null;
			byte[]? compressedBytes = null;
			try
			{
				writtenBytes = ArrayPool<byte>.Shared.Rent(length);
				_writer.CopyTo(writtenBytes);
				ReadOnlySpan<byte> writtenSpan = writtenBytes.AsSpan()[..length];

				int maxSize = BrotliEncoder.GetMaxCompressedLength(length);
				compressedBytes = ArrayPool<byte>.Shared.Rent(maxSize);
				Span<byte> compressedSpan = compressedBytes.AsSpan();

				if (BrotliEncoder.TryCompress(writtenSpan, compressedSpan, out int encodedLength, 0, 24))
				{
					return new()
					{
						LastWriteTimeUtc = lastWriteTimeUtc,
						UncompressedSize = length,
						CompressedData = compressedSpan[..encodedLength].ToArray().AsMemory(),
					};
				}
				return new()
				{
					LastWriteTimeUtc = DateTime.MinValue,
					UncompressedSize = -1,
					CompressedData = new(),
				};
			}
			finally
			{
				if (writtenBytes != null)
				{
					ArrayPool<byte>.Shared.Return(writtenBytes);
				}
				if (compressedBytes != null)
				{
					ArrayPool<byte>.Shared.Return(compressedBytes);
				}
			}
		}

		/// <summary>
		/// Write the given package to the stream
		/// </summary>
		/// <param name="package"></param>
		private void WritePackage(UhtPackage package)
		{
			if (package == package.Module.ScriptPackage)
			{
				_writer.WriteInt32(-1);
				return;
			}

			if (_packages.TryGetValue(package, out int index))
			{
				_writer.WriteInt32(index);
				return;
			}

			index = _packages.Count;
			_packages.Add(package, index);
			_writer.WriteInt32(index);
			_writer.WriteString(package.SourceName);
			_writer.WriteUInt64((ulong)package.PackageFlags);
		}

		/// <summary>
		/// Write the given namespace to the stream
		/// </summary>
		/// <param name="namespaceObj"></param>
		public void WriteNamespace(UhtNamespace namespaceObj)
		{
			if (namespaceObj.IsGlobal)
			{
				_writer.WriteInt32(-1);
				return;
			}

			if (_namespaces.TryGetValue(namespaceObj, out int index))
			{
				_writer.WriteInt32(index);
				return;
			}

			index = _namespaces.Count;
			_namespaces.Add(namespaceObj, index);
			_writer.WriteInt32(index);
			_writer.WriteString(namespaceObj.FullSourceName);
		}

		/// <summary>
		/// Write the given type
		/// </summary>
		/// <param name="type"></param>
		public void WriteType(UhtType? type)
		{
			// Write the object index.  If we have already written this object, then
			// just write the index
			if (type == null)
			{
				_writer.WriteInt32(-1);
				return;
			}
			if (type.HeaderFile != Header)
			{
				throw new UhtException("Attempt to cache a type not associated with the header");
			}
			if (type.GetType() == typeof(UhtPackage))
			{
				throw new UhtException("UhtPackage types should not be part of the input cache");
			}
			if (_objects.TryGetValue(type, out int objectIndex))
			{
				_writer.WriteInt32(objectIndex);
				return;
			}
			objectIndex = _objects.Count;
			_objects.Add(type, objectIndex);
			_writer.WriteInt32(objectIndex);

			// Write the information needed to create the type
			Type metaType = type.GetType();
			if (_types.TryGetValue(metaType, out int index))
			{
				_writer.WriteInt32(index);
			}
			else
			{
				index = _types.Count;
				_types.Add(metaType, index);
				_writer.WriteInt32(index);
				_writer.WriteString(metaType.FullName!);
			}

			// If the outer is a package, then write it
			if (type.Outer == null)
			{
				throw new UhtException("Expected an outer when writing to cache");
			}
			if (type.Outer is UhtPackage package)
			{
				_writer.WriteBoolean(true);
				WritePackage(package);
			}
			else
			{
				_writer.WriteBoolean(false);
				WriteType(type.Outer);
			}

			// Allow the type to write the contents
			type.Write(this);
			WriteMarker(UhtInputCacheMarker.EndOfTypeBase);

			// Write the children
			if (type is not UhtProperty)
			{
				WriteChildren(type.Children);
			}
			WriteMarker(UhtInputCacheMarker.EndOfType);
		}

		/// <summary>
		/// Write the given list of children to the 
		/// </summary>
		/// <param name="children"></param>
		public void WriteChildren(IReadOnlyList<UhtType> children)
		{
			WriteInt32(children.Count);
			foreach (UhtType child in children)
			{
				WriteType(child);
			}
		}

		/// <summary>
		/// Dispose of the object
		/// </summary>
		public void Dispose()
		{
			_writer.Dispose();
		}
	}

	/// <summary>
	/// Information about a input cache entry
	/// </summary>
	public readonly struct UhtInputCacheEntry
	{

		/// <summary>
		/// The last write time for the header file for this entry
		/// </summary>
		public DateTime LastWriteTimeUtc { get; init; }

		/// <summary>
		/// Size of the uncompressed data
		/// </summary>
		public int UncompressedSize { get; init; }

		/// <summary>
		/// Binary archive of the cache entry
		/// </summary>
		public ReadOnlyMemory<byte> CompressedData { get; init; }
	}

	/// <summary>
	/// Helper class to read/write parsed headers
	/// </summary>
	public class UhtInputCache
	{
		private const int CacheVersion = 2;
		private readonly Dictionary<string, UhtInputFile> _files = [];
		private readonly ReadOnlyMemory<byte> _data;

		private UhtInputCache()
		{
			_data = new();
		}

		private UhtInputCache(List<UhtInputFile> cachedFiles, ReadOnlyMemory<byte> data)
		{
			foreach (UhtInputFile file in cachedFiles)
			{
				_files.Add(file.FileName, file);
			}
			_data = data;
		}

		/// <summary>
		/// Try to get a cached entry from the cache
		/// </summary>
		/// <param name="fileName">Name of the header file</param>
		/// <param name="entry">Information about the binary archive</param>
		/// <returns>True if the entry was found, false if not</returns>
		public bool TryGetFile(string fileName, out UhtInputCacheEntry entry)
		{
			if (_files.TryGetValue(fileName, out UhtInputFile value))
			{
				entry = new()
				{
					LastWriteTimeUtc = new(value.LastWriteTimeUtc),
					UncompressedSize = value.UncompressedSize,
					CompressedData = _data[(int)value.CacheStart..(int)value.CacheEnd],
				};
				return true;
			}
			entry = new() { LastWriteTimeUtc = new(), UncompressedSize = 0, CompressedData = new(), };
			return false;
		}

		/// <summary>
		/// Read the input cache file for the given module
		/// </summary>
		/// <param name="module">Module to be read</param>
		/// <param name="configData">Configuration data</param>
		/// <returns>Input cache or null if not found</returns>
		public static UhtInputCache Read(UhtModule module, ReadOnlySpan<byte> configData)
		{
			if (!module.Session.EnableInputCacheRead)
			{
				return new();
			}
			FileReference cacheFile = GetCacheFileReference(module);
			if (!File.Exists(cacheFile.FullName))
			{
				module.Session.CacheFilesSkippedFNF.Increment();
				return new();
			}

			try
			{

				// Don't use a cache created earlier than the last time UHT was compiled
				DateTime tocDateTime = File.GetLastAccessTimeUtc(cacheFile.FullName);
				if (tocDateTime < module.Session.AssemblyDateTime)
				{
					module.Session.CacheFilesSkippedAssemblyDateTime.Increment();
					return new();
				}

				byte[] headerData = File.ReadAllBytes(cacheFile.FullName);
				MemoryReader reader = new(headerData);
				int version = reader.ReadInt32();
				if (version != CacheVersion)
				{
					module.Session.CacheFilesSkippedVersion.Increment();
					return new();
				}

				ReadOnlyMemory<byte> cachedConfigData = reader.ReadVariableLengthBytes();
				if (!cachedConfigData.Span.SequenceEqual(configData))
				{
					module.Session.CacheFilesSkippedConfigChanged.Increment();
					return new();
				}

				List<UhtInputFile> cachedFiles = [];
				reader.ReadList(cachedFiles, () => new UhtInputFile(reader));

				ReadOnlyMemory<byte> data = reader.RemainingMemory;
				foreach (UhtInputFile cachedFile in cachedFiles)
				{
					if (cachedFile.CacheStart < 0 || cachedFile.CacheStart > data.Length ||
						cachedFile.CacheEnd < 0 || cachedFile.CacheEnd > data.Length ||
						cachedFile.CacheStart >= cachedFile.CacheEnd)
					{
						module.Session.CacheFilesSkippedCorrupted.Increment();
						return new();
					}
				}

				module.Session.CacheFilesRead.Increment();
				return new(cachedFiles, reader.RemainingMemory);
			}
			catch (Exception ex)
			{
				module.Session.Logger.LogTrace(ex, "Error reading input cache {Message}", ex.Message);
				DeleteCache(module);
				module.Session.CacheFilesSkippedCorrupted.Increment();
				return new();
			}
		}

		/// <summary>
		/// Write the input cache file for the given module
		/// </summary>
		/// <param name="module">Module to be written</param>
		/// <param name="configData">Configuration data</param>
		public static void Write(UhtModule module, ReadOnlySpan<byte> configData)
		{
			FileReference cacheFile = GetCacheFileReference(module);

			try
			{
				DirectoryReference.CreateDirectory(cacheFile.Directory);

				// Collect a list of all the cached files with offsets
				List<UhtInputFile> cachedFiles = [];
				long startPosition = 0;
				foreach (UhtHeaderFile header in module.Headers)
				{
					if (!header.InputCacheEntry.CompressedData.IsEmpty)
					{
						long endPosition = startPosition + header.InputCacheEntry.CompressedData.Length;
						UhtInputFile inputFile = new()
						{
							FileName = header.FilePath,
							LastWriteTimeUtc = header.InputCacheEntry.LastWriteTimeUtc.Ticks,
							UncompressedSize = header.InputCacheEntry.UncompressedSize,
							CacheStart = startPosition,
							CacheEnd = endPosition
						};
						cachedFiles.Add(inputFile);
						startPosition = endPosition;
					}
				}

				{
					using Stream cacheStream = File.Open(cacheFile.FullName, FileMode.Create, FileAccess.Write, FileShare.Read);

					using ChunkedMemoryWriter headerWriter = new();
					headerWriter.WriteInt32(CacheVersion);
					headerWriter.WriteVariableLengthBytes(configData);
					headerWriter.WriteList(cachedFiles, (writer, cachedFile) => cachedFile.Write(headerWriter));
					cacheStream.Write(headerWriter.ToByteArray());

					foreach (UhtHeaderFile header in module.Headers)
					{
						cacheStream.Write(header.InputCacheEntry.CompressedData.Span);
					}
				}

				module.Session.CacheFilesWritten.Increment();
			}
			catch (Exception ex)
			{
				module.Session.Logger.LogTrace(ex, "Error creating input cache {Message}", ex.Message);
				DeleteCache(module);
			}
		}

		private static void DeleteCache(UhtModule module)
		{
			DeleteCacheFile(GetCacheFileReference(module));
		}

		private static void DeleteCacheFile(FileReference file)
		{
			if (File.Exists(file.FullName))
			{
				try
				{
					File.Delete(file.FullName);
				}
				catch
				{
				}
			}
		}

		private static FileReference GetCacheFileReference(UhtModule module)
		{
			return FileReference.Combine(new(module.Module.OutputDirectory), $".inputcache.data");
		}
	}
}
