// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// JSON converter to output the meta data
	/// </summary>
	public class UhtMetaDataConverter : JsonConverter<UhtMetaData>
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override UhtMetaData Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="type">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, UhtMetaData type, JsonSerializerOptions options)
		{
			writer.WriteStartArray();
			foreach (KeyValuePair<string, string> kvp in type.GetSorted())
			{
				writer.WriteStartObject();
				writer.WriteString("Key", kvp.Key);
				writer.WriteString("Value", kvp.Value);
				writer.WriteEndObject();
			}
			writer.WriteEndArray();
		}
	}

	/// <summary>
	/// JSON converter to output the source name of a type
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtTypeSourceNameJsonConverter<T> : JsonConverter<T> where T : UhtType
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override T Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="type">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, T type, JsonSerializerOptions options)
		{
			writer.WriteStringValue(type.SourceName);
		}
	}

	/// <summary>
	/// Read/Write type source name for an optional type value
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtNullableTypeSourceNameJsonConverter<T> : JsonConverter<T> where T : UhtType
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override T Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="type">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, T? type, JsonSerializerOptions options)
		{
			if (type != null)
			{
				writer.WriteStringValue(type.SourceName);
			}
		}
	}

	/// <summary>
	/// Read/Write type list source name for an optional type value
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtTypeReadOnlyListSourceNameJsonConverter<T> : JsonConverter<IReadOnlyList<T>> where T : UhtType
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override IReadOnlyList<T> Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="collection">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, IReadOnlyList<T> collection, JsonSerializerOptions options)
		{
			writer.WriteStartArray();
			foreach (UhtType type in collection)
			{
				writer.WriteStringValue(type.SourceName);
			}
			writer.WriteEndArray();
		}
	}

	/// <summary>
	/// Serialize a list of types
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtTypeListJsonConverter<T> : JsonConverter<List<T>> where T : UhtType
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override List<T> Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="collection">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, List<T> collection, JsonSerializerOptions options)
		{
			writer.WriteStartArray();
			foreach (UhtType type in collection)
			{
				JsonSerializer.Serialize(writer, type, type.GetType(), options);
			}
			writer.WriteEndArray();
		}
	}

	/// <summary>
	/// Serialize a nullable list of types
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class UhtNullableTypeListJsonConverter<T> : JsonConverter<List<T>> where T : UhtType
	{
		/// <summary>
		/// Read the JSON value
		/// </summary>
		/// <param name="reader">Source reader</param>
		/// <param name="typeToConvert">Type to convert</param>
		/// <param name="options">Serialization options</param>
		/// <returns>Read value</returns>
		/// <exception cref="NotImplementedException"></exception>
		public override List<T> Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Write the JSON value
		/// </summary>
		/// <param name="writer">Destination writer</param>
		/// <param name="collection">Value being written</param>
		/// <param name="options">Serialization options</param>
		public override void Write(Utf8JsonWriter writer, List<T>? collection, JsonSerializerOptions options)
		{
			writer.WriteStartArray();
			if (collection != null)
			{
				foreach (UhtType type in collection)
				{
					JsonSerializer.Serialize(writer, type, type.GetType(), options);
				}
			}
			writer.WriteEndArray();
		}
	}
}
