// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Threading;
using EpicGames.UHT.Tokenizer;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Type of message
	/// </summary>
	public enum UhtMessageType
	{
		/// <summary>
		/// The message is an error and goes to the log and console
		/// </summary>
		Error,

		/// <summary>
		/// The message is a warning and goes to the log and console
		/// </summary>
		Warning,

		/// <summary>
		/// The message is for information only and goes to the log and console
		/// </summary>
		Info,

		/// <summary>
		/// The message is for debugging and goes to the log only
		/// </summary>
		Trace,

		/// <summary>
		/// The message is an informational message about deprecated patterns
		/// </summary>
		Deprecation,

		/// <summary>
		/// The message is an internal error and goes to the log and console
		/// </summary>
		Ice,
	}

	/// <summary>
	/// A message session is the destination object for all generated messages
	/// </summary>
	public interface IUhtMessageSession
	{

		/// <summary>
		/// Add the given message
		/// </summary>
		/// <param name="message">The message to be added</param>
		void AddMessage(UhtMessage message);
	}

	/// <summary>
	/// A message source represents the source file where the message occurred.
	/// </summary>
	public interface IUhtMessageSource
	{
		/// <summary>
		/// File path of the file being parsed
		/// </summary>
		string MessageFilePath { get; }

		/// <summary>
		/// The full file path of being parsed
		/// </summary>
		string MessageFullFilePath { get; }

		/// <summary>
		/// If true, the source is a source fragment from the testing harness
		/// </summary>
		bool MessageIsFragment { get; }

		/// <summary>
		/// If this is a fragment, this is the container file path of the fragment
		/// </summary>
		string MessageFragmentFilePath { get; }

		/// <summary>
		/// If this is a fragment, this is the container full file path of the fragment
		/// </summary>
		string MessageFragmentFullFilePath { get; }

		/// <summary>
		/// If this is a fragment, this is the line number in the container file where the fragment is defined.
		/// </summary>
		int MessageFragmentLineNumber { get; }
	}

	/// <summary>
	/// A message site can automatically provide a line number where the site was defined
	/// in the source.  If no line number is provided when the message is created or if the
	/// site doesn't support this interface, the line number of '1' will be used.
	/// </summary>
	public interface IUhtMessageLineNumber
	{

		/// <summary>
		/// Line number where the type was defined.
		/// </summary>
		int MessageLineNumber { get; }
	}

	/// <summary>
	/// This interface provides a mechanism for things to provide more context for an error
	/// </summary>
	public interface IUhtMessageExtraContext
	{
		/// <summary>
		/// Enumeration of objects to add as extra context.
		/// </summary>
		IEnumerable<object?>? MessageExtraContext { get; }
	}

	/// <summary>
	/// A message site is any object that can generate a message.  In general, all 
	/// types are also message sites. This provides a convenient method to log messages
	/// where the type was defined.
	/// </summary>
	public interface IUhtMessageSite
	{

		/// <summary>
		/// Destination message session for the messages
		/// </summary>
		public IUhtMessageSession MessageSession { get; }

		/// <summary>
		/// Source file generating messages
		/// </summary>
		public IUhtMessageSource? MessageSource { get; }

		/// <summary>
		/// Optional line number where type was defined
		/// </summary>
		public IUhtMessageLineNumber? MessageLineNumber { get; }
	}

	/// <summary>
	/// Represents a UHT message
	/// </summary>
	public struct UhtMessage
	{
		/// <summary>
		/// The type of message
		/// </summary>
		public UhtMessageType MessageType { get; set; }

		/// <summary>
		/// Optional message source for the message.  Either the MessageSource or FilePath must be set.
		/// </summary>
		public IUhtMessageSource? MessageSource { get; set; }

		/// <summary>
		/// Optional file path for the message.  Either the MessageSource or FilePath must be set.
		/// </summary>
		public string? FilePath { get; set; }

		/// <summary>
		/// Line number where error occurred.
		/// </summary>
		public int LineNumber { get; set; }

		/// <summary>
		/// Text of the message
		/// </summary>
		public string Message { get; set; }

		/// <summary>
		/// Make a new message with the given settings
		/// </summary>
		/// <param name="messageType">Type of message</param>
		/// <param name="messageSource">Source of the message</param>
		/// <param name="filePath">File path of the message</param>
		/// <param name="lineNumber">Line number where message occurred</param>
		/// <param name="message">Text of the message</param>
		/// <returns>Created message</returns>
		public static UhtMessage MakeMessage(UhtMessageType messageType, IUhtMessageSource? messageSource, string? filePath, int lineNumber, string message)
		{
			return new UhtMessage
			{
				MessageType = messageType,
				MessageSource = messageSource,
				FilePath = filePath,
				LineNumber = lineNumber,
				Message = message
			};
		}

		/// <summary>
		/// Format an object to be included in a message
		/// </summary>
		/// <param name="context">Contextual object</param>
		/// <returns>Formatted context</returns>
		public static string FormatContext(object context)
		{
			if (context is IUhtMessageExtraContext extraContextInterface)
			{
				StringBuilder builder = new();
				Append(builder, extraContextInterface, false);
				return builder.ToString();
			}
			else if (context is UhtToken token)
			{
				switch (token.TokenType)
				{
					case UhtTokenType.EndOfFile:
						return "EOF";
					case UhtTokenType.EndOfDefault:
						return "'end of default value'";
					case UhtTokenType.EndOfType:
						return "'end of type'";
					case UhtTokenType.EndOfDeclaration:
						return "'end of declaration'";
					case UhtTokenType.StringConst:
						return "string constant";
					default:
						return $"'{token.Value}'";
				}
			}
			else if (context is char c)
			{
				return $"'{c}'";
			}
			else if (context is IEnumerable<string> stringEnumerable)
			{
				return UhtUtilities.MergeTypeNames(stringEnumerable, "or", true);
			}
			else
			{
				return context.ToString() ?? String.Empty;
			}
		}

		/// <summary>
		/// Append message context
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="messageExtraContextInterface">Extra context to append</param>
		/// <param name="alwaysIncludeSeparator">If true, always include the separator</param>
		public static void Append(StringBuilder builder, IUhtMessageExtraContext? messageExtraContextInterface, bool alwaysIncludeSeparator)
		{
			if (messageExtraContextInterface == null)
			{
				return;
			}

			IEnumerable<object?>? extraContextList = messageExtraContextInterface.MessageExtraContext;
			if (extraContextList != null)
			{
				int startingLength = builder.Length;
				foreach (object? ec in extraContextList)
				{
					if (ec != null)
					{
						if (builder.Length != startingLength || alwaysIncludeSeparator)
						{
							builder.Append(" in ");
						}
						builder.Append(FormatContext(ec));
					}
				}
			}
		}
	}

	/// <summary>
	/// A placeholder message site
	/// </summary>
	public class UhtEmptyMessageSite : IUhtMessageSite
	{
		/// <inheritdoc/>
		public IUhtMessageSession MessageSession => throw new NotImplementedException();

		/// <inheritdoc/>
		public IUhtMessageSource? MessageSource => throw new NotImplementedException();

		/// <inheritdoc/>
		public IUhtMessageLineNumber? MessageLineNumber => throw new NotImplementedException();
	}

	/// <summary>
	/// Creates a message site from a message session interface and a message source interface
	/// </summary>
	public class UhtSimpleMessageSite : IUhtMessageSite
	{
		private readonly IUhtMessageSession _messageSession;
		private IUhtMessageSource? _messageSource;

		#region IUHTMessageSite implementation
		/// <inheritdoc/>
		public IUhtMessageSession MessageSession => _messageSession;
		/// <inheritdoc/>
		public IUhtMessageSource? MessageSource { get => _messageSource; set => _messageSource = value; }
		/// <inheritdoc/>
		public IUhtMessageLineNumber? MessageLineNumber => null;
		#endregion

		/// <summary>
		/// Create a simple message site for the given session and source
		/// </summary>
		/// <param name="messageSession">Associated message session</param>
		/// <param name="messageSource">Source for the messages</param>
		public UhtSimpleMessageSite(IUhtMessageSession messageSession, IUhtMessageSource? messageSource = null)
		{
			_messageSession = messageSession;
			_messageSource = messageSource;
		}
	}

	/// <summary>
	/// Simple message site for the given file.
	/// </summary>
	public class UhtSimpleFileMessageSite : UhtSimpleMessageSite, IUhtMessageSource
	{
		#region IUHTMessageSource implementation
		/// <inheritdoc/>
		public string MessageFilePath => _filePath;
		/// <inheritdoc/>
		public string MessageFullFilePath => _filePath;
		/// <inheritdoc/>
		public bool MessageIsFragment => false;
		/// <inheritdoc/>
		public string MessageFragmentFilePath => "";
		/// <inheritdoc/>
		public string MessageFragmentFullFilePath => "";
		/// <inheritdoc/>
		public int MessageFragmentLineNumber => -1;
		#endregion

		private readonly string _filePath;

		/// <summary>
		/// Create a simple file site
		/// </summary>
		/// <param name="messageSession">Associated message session</param>
		/// <param name="filePath">File associated with the site</param>
		public UhtSimpleFileMessageSite(IUhtMessageSession messageSession, string filePath) : base(messageSession, null)
		{
			_filePath = filePath;
			MessageSource = this;
		}
	}

	/// <summary>
	/// Series of extensions for message sites.
	/// </summary>
	public static class UhtMessageSiteExtensions
	{
		/// <summary>
		/// Get the line number generating the error
		/// </summary>
		/// <param name="messageSite">The message site generating the error.</param>
		/// <param name="lineNumber">An override line number</param>
		/// <returns>Either the overriding line number or the line number from the message site.</returns>
		public static int GetLineNumber(this IUhtMessageSite messageSite, int lineNumber = -1)
		{
			if (lineNumber != -1)
			{
				return lineNumber;
			}
			else if (messageSite.MessageLineNumber != null)
			{
				return messageSite.MessageLineNumber.MessageLineNumber;
			}
			else
			{
				return 1;
			}
		}

		/// <summary>
		/// Log an error
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the error</param>
		/// <param name="message">Text of the error</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogError(this IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Error, messageSite, lineNumber, message, extraContext);
		}

		/// <summary>
		/// Log an error
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="message">Text of the error</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogError(this IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Error, messageSite, -1, message, extraContext);
		}

		/// <summary>
		/// Log a warning
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the warning</param>
		/// <param name="message">Text of the warning</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogWarning(this IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Warning, messageSite, lineNumber, message, extraContext);
		}

		/// <summary>
		/// Log a warning
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="message">Text of the warning</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogWarning(this IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Warning, messageSite, -1, message, extraContext);
		}

		/// <summary>
		/// Log a message directly to the log
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the information</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogTrace(this IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Trace, messageSite, lineNumber, message, extraContext);
		}

		/// <summary>
		/// Log a message directly to the log
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogTrace(this IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Trace, messageSite, -1, message, extraContext);
		}

		/// <summary>
		/// Log information
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the information</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogInfo(this IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Info, messageSite, lineNumber, message, extraContext);
		}

		/// <summary>
		/// Log a information
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogInfo(this IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Info, messageSite, -1, message, extraContext);
		}

		/// <summary>
		/// Log deprecation
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the information</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogDeprecation(this IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Deprecation, messageSite, lineNumber, message, extraContext);
		}

		/// <summary>
		/// Log a deprecation
		/// </summary>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		public static void LogDeprecation(this IUhtMessageSite messageSite, string message, object? extraContext = null)
		{
			LogMessage(UhtMessageType.Deprecation, messageSite, -1, message, extraContext);
		}

		/// <summary>
		/// Log a message
		/// </summary>
		/// <param name="messageType">Type of the message being generated</param>
		/// <param name="messageSite">Message site associated with the message</param>
		/// <param name="lineNumber">Line number of the information</param>
		/// <param name="message">Text of the information</param>
		/// <param name="extraContext">Addition context to be appended to the error message</param>
		private static void LogMessage(UhtMessageType messageType, IUhtMessageSite messageSite, int lineNumber, string message, object? extraContext)
		{
			if (extraContext != null)
			{
				message = $"{message} in {UhtMessage.FormatContext(extraContext)}";
			}
			messageSite.MessageSession.AddMessage(UhtMessage.MakeMessage(messageType, messageSite.MessageSource, null, messageSite.GetLineNumber(lineNumber), message));
		}
	}

	/// <summary>
	/// Thread based message context.  Used to improve performance by avoiding allocations.
	/// </summary>
	public sealed class UhtTlsMessageExtraContext : IUhtMessageExtraContext
	{
		private Stack<object?>? _extraContexts;
		private static readonly ThreadLocal<UhtTlsMessageExtraContext> s_tls = new(() => new UhtTlsMessageExtraContext());

		#region IUHTMessageExtraContext implementation
		IEnumerable<object?>? IUhtMessageExtraContext.MessageExtraContext => _extraContexts;
		#endregion

		/// <summary>
		/// Add an extra context
		/// </summary>
		/// <param name="exceptionContext"></param>
		public void PushExtraContext(object? exceptionContext)
		{
			_extraContexts ??= new Stack<object?>(8);
			_extraContexts.Push(exceptionContext);
		}

		/// <summary>
		/// Pop the top most extra context
		/// </summary>
		public void PopExtraContext()
		{
			_extraContexts?.Pop();
		}

		/// <summary>
		/// Get the extra context associated with this thread
		/// </summary>
		/// <returns>Extra context</returns>
		public static UhtTlsMessageExtraContext? GetTls() { return s_tls.Value; }

		/// <summary>
		/// Get the extra context interface
		/// </summary>
		/// <returns>Extra context interface</returns>
		public static IUhtMessageExtraContext? GetMessageExtraContext() { return s_tls.Value; }
	}

	/// <summary>
	/// A "using" object to automate the push/pop of extra context to the thread's current extra context
	/// </summary>
	public readonly struct UhtMessageContext : IDisposable
	{
		private readonly UhtTlsMessageExtraContext? _stack;

		/// <summary>
		/// Construct a new entry
		/// </summary>
		/// <param name="extraContext">Extra context to be added</param>
		public UhtMessageContext(object? extraContext)
		{
			_stack = UhtTlsMessageExtraContext.GetTls();
			_stack?.PushExtraContext(extraContext);
		}

		/// <summary>
		/// Replace the extra context.  This replaces the top level context and thus 
		/// can have unexpected results if done when a more deeper context has been added
		/// </summary>
		/// <param name="extraContext">New extra context</param>
		public void Reset(object? extraContext)
		{
			if (_stack != null)
			{
				_stack.PopExtraContext();
				_stack.PushExtraContext(extraContext);
			}
		}

		/// <summary>
		/// Dispose the object and auto-remove the added context
		/// </summary>
		public void Dispose()
		{
			_stack?.PopExtraContext();
		}
	}
}
