// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable 1591 // Disable documentation warning for this file

using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// Collection of common name strings
	/// </summary>
	public static class UhtNames
	{
		// FNames
		public const string AdvancedClassDisplay = "AdvancedClassDisplay";
		public const string AdvancedDisplay = "AdvancedDisplay";
		public const string AllowPrivateAccess = "AllowPrivateAccess";
		public const string ArraySizeEnum = "ArraySizeEnum";
		public const string AutoCollapseCategories = "AutoCollapseCategories";
		public const string AutoExpandCategories = "AutoExpandCategories";
		public const string BlueprintGetter = "BlueprintGetter";
		public const string BlueprintInternalUseOnly = "BlueprintInternalUseOnly";
		public const string BlueprintSetter = "BlueprintSetter";
		public const string CallInEditor = "CallInEditor";
		public const string BlueprintType = "BlueprintType";
		public const string StateStreamStaticState = "StateStreamStaticState";
		public const string StateStreamDynamicState = "StateStreamDynamicState";
		public const string StateStreamHandle = "StateStreamHandle";
		public const string CannotImplementInterfaceInBlueprint = "CannotImplementInterfaceInBlueprint";
		public const string Category = "Category";
		public const string ClassGroupNames = "ClassGroupNames";
		public const string Comment = "Comment";
		public const string Constant = "Constant";
		public const string CppFromBpEvent = "CppFromBpEvent";
		public const string CustomThunk = "CustomThunk";
		public const string Deprecated = "Deprecated";
		public const string DeprecatedFunction = "DeprecatedFunction";
		public const string DisplayName = "DisplayName";
		public const string DocumentationPolicy = "DocumentationPolicy";
		public const string EditorConfig = "EditorConfig";
		public const string EditInline = "EditInline";
		public const string EnumDisplayNameFn = "EnumDisplayNameFn";
		public const string ExposeOnSpawn = "ExposeOnSpawn";
		public const string GetByRef = "GetByRef";
		public const string Hidden = "Hidden";
		public const string HideCategories = "HideCategories";
		public const string HideFunctions = "HideFunctions";
		public const string IgnoreCategoryKeywordsInSubclasses = "IgnoreCategoryKeywordsInSubclasses";
		public const string IncludePath = "IncludePath";
		public const string Input = "Input";
		public const string Interface = "Interface";
		public const string IsBlueprintBase = "IsBlueprintBase";
		public const string IsConversionRoot = "IsConversionRoot";
		public const string ModuleRelativePath = "ModuleRelativePath";
		public const string Name = "Name";
		public const string NativeConst = "NativeConst";
		public const string NativeConstTemplateArg = "NativeConstTemplateArg";
		public const string NoGetter = "NoGetter";
		public const string ObjectInitializerConstructorDeclared = "ObjectInitializerConstructorDeclared";
		public const string Output = "Output";
		public const string Pins = "Pins";
		public const string PrioritizeCategories = "PrioritizeCategories";
		public const string ReturnValue = "ReturnValue";
		public const string ShortToolTip = "ShortToolTip";
		public const string ShowCategories = "ShowCategories";
		public const string ShowFunctions = "ShowFunctions";
		public const string SparseClassDataTypes = "SparseClassDataTypes";
		public const string ToolTip = "ToolTip";
		public const string UIMin = "UIMin";
		public const string UIMax = "UIMax";
		public const string Visible = "Visible";

		// Verse names
		public const string VerseProxySuffix = "__Proxy";
		public const string VerseNotNativePrefix = "__VerseOnly_";

		// Compiler directives
		public const string WithHotReload = "WITH_HOT_RELOAD";
		public const string WithEditor = "WITH_EDITOR";
		public const string WithEditorOnlyData = "WITH_EDITORONLY_DATA";
		public const string WithEngine = "WITH_ENGINE";
		public const string WithCoreUObject = "WITH_COREUOBJECT";
		public const string WithVerseVM = "WITH_VERSE_VM";
		public const string WithVerseBPVM = "WITH_VERSE_BPVM";
		public const string WithTests = "WITH_TESTS";

		// Other strings
		public const string Statics = "UHT_STATICS";
		public const string PropertyCppIdentifierPrefix = "NewProp_";

		// Identifiers
		public static readonly UhtCppIdentifier TypeId = new("Type");
		public static readonly UhtCppIdentifier StaticsTypeId = TypeId.MakeStatics();
		public static readonly UhtCppIdentifier PropPointersId = new("PropPointers");
		public static readonly UhtCppIdentifier StaticsPropPointersId = PropPointersId.MakeStatics();
		public static readonly UhtCppIdentifier SingletonFuncArrayId = new("SingletonFuncArray");
		public static readonly UhtCppIdentifier StaticsSingletonFuncArrayId = SingletonFuncArrayId.MakeStatics();
		public static readonly UhtCppIdentifier FuncInfoId = new("FuncInfo");
		public static readonly UhtCppIdentifier StaticsFuncInfoId = FuncInfoId.MakeStatics();
		public static readonly UhtCppIdentifier FuncsId = new("Funcs");
		public static readonly UhtCppIdentifier StaticsFuncsId = FuncsId.MakeStatics();
		public static readonly UhtCppIdentifier ClassInfoId = new("ClassInfo");
		public static readonly UhtCppIdentifier StaticsClassInfoId = ClassInfoId.MakeStatics();
		public static readonly UhtCppIdentifier ScriptStructInfoId = new("ScriptStructInfo");
		public static readonly UhtCppIdentifier StaticsScriptStructInfoId = ScriptStructInfoId.MakeStatics();
		public static readonly UhtCppIdentifier EnumInfoId = new("EnumInfo");
		public static readonly UhtCppIdentifier StaticsEnumInfoId = EnumInfoId.MakeStatics();
		public static readonly UhtCppIdentifier RigVMStructsId = new("RigVMStructs");
		public static readonly UhtCppIdentifier StaticsRigVMStructsId = RigVMStructsId.MakeStatics();
		public static readonly UhtCppIdentifier VerseFuncsId = new("VerseFuncs");
		public static readonly UhtCppIdentifier StaticsVerseFuncsId = VerseFuncsId.MakeStatics();
		public static readonly UhtCppIdentifier PartialInfoId = new("PartialInfo");
		public static readonly UhtCppIdentifier StaticsPartialInfoId = PartialInfoId.MakeStatics();

		// Helper methods to generate cpp identifiers
		public static UhtCppIdentifier GetPropertyIdentifier(UhtProperty? property)
		{
			return property != null ? new(PropertyCppIdentifierPrefix, property.EngineName, null) : new();
		}
	}
}
