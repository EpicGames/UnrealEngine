// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using EpicGames.UHT.Parsers;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// The phase of UHT where the property is being resolved
	/// </summary>
	public enum UhtPropertyResolvePhase
	{

		/// <summary>
		/// Resolved during the source processing phase.  Immediate property types only.
		/// </summary>
		Parsing,

		/// <summary>
		/// Resolved during the resolve phase.  Non-immediate property types only.
		/// </summary>
		Resolving,
	}

	/// <summary>
	/// Controls what is returned from the parsing of a template object
	/// </summary>
	public enum UhtTemplateObjectMode
	{
		/// <summary>
		/// If the specified type.  If the type is a native interface, return the UInterface
		/// </summary>
		Normal,

		/// <summary>
		/// If a native interface is specified, return it
		/// </summary>
		PreserveNativeInterface,

		/// <summary>
		/// The specified type must be an interface proxy
		/// </summary>
		NativeInterfaceProxy,
	}

	/// <summary>
	/// Structure defining the arguments used in the delegate to resolve property types
	/// </summary>
	/// <param name="ResolvePhase">Specifies if this is being resolved during the parsing phase or the resolution phase.  Type lookups can not happen during the parsing phase</param>
	/// <param name="PropertySettings">The configuration of the property</param>
	/// <param name="TokenReader">The token reader containing the type</param>
	public record struct UhtPropertyResolveArgs(UhtPropertyResolvePhase ResolvePhase, UhtPropertySettings PropertySettings, IUhtTokenReader TokenReader)
	{

		/// <summary>
		/// Currently running UHT session
		/// </summary>
		public readonly UhtSession Session => PropertySettings.Outer.Session;

		/// <summary>
		/// UHT configuration
		/// </summary>
		public readonly IUhtConfig Config => Session.Config!;

		/// <summary>
		/// 
		/// </summary>
		public readonly UhtTypeTokensRef TypeTokens => PropertySettings.TypeTokens;

		/// <summary>
		/// If true, a template argument is being resolved
		/// </summary>
		public readonly bool IsTemplate => TypeTokens.DeclarationSegmentIndex != 0;

		/// <summary>
		/// When processing type, make sure that the next token is the expected token
		/// </summary>
		/// <returns>true if there could be more header to process, false if the end was reached.</returns>
		public readonly bool SkipExpectedType()
		{
			if (PropertySettings.PropertyCategory == UhtPropertyCategory.Member && TokenReader.TryOptional("const"))
			{
				TokenReader.LogError("Const properties are not supported.");
			}
			foreach (UhtToken token in TypeTokens.TypeTokens.Span)
			{
				if (!TokenReader.TryOptional(token.Value))
				{
					TokenReader.LogError($"Inappropriate keyword '{TokenReader.PeekToken().Value}' on variable of type '{token.Value}'");
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Resolve the given property.  This method will resolve any immediate property during the parsing phase or 
		/// resolve any previously parsed property to the final version.
		/// </summary>
		/// <returns></returns>
		public readonly UhtProperty? ResolveProperty()
		{
			if (ResolvePropertyType(out UhtPropertyType propertyType))
			{
				if (ResolvePhase == UhtPropertyResolvePhase.Resolving || propertyType.Options.HasAnyFlags(UhtPropertyTypeOptions.Immediate))
				{
					return UhtPropertyParser.ResolvePropertyType(this, propertyType);
				}
			}

			// Try the default processor.  We only use the default processor when trying to resolve something post parsing phase.
			if (ResolvePhase == UhtPropertyResolvePhase.Resolving)
			{
				return UhtPropertyParser.ResolvePropertyType(this, Session.DefaultPropertyType);
			}
			return null;
		}

		/// <summary>
		/// Parse a template parameter
		/// </summary>
		/// <param name="paramName">Name of the template parameter</param>
		/// <returns>Parsed property</returns>
		public readonly UhtProperty? ParseTemplateParam(StringView paramName)
		{
			using UhtThreadBorrower<UhtPropertySpecifierContext> borrower = new(true);
			UhtPropertySpecifierContext specifierContext = borrower.Instance;
			specifierContext.Type = PropertySettings.Outer;
			specifierContext.TokenReader = TokenReader;
			specifierContext.AccessSpecifier = UhtAccessSpecifier.None;
			specifierContext.MessageSite = TokenReader;
			specifierContext.PropertySettings.Reset(PropertySettings, paramName.ToString(), TokenReader);
			specifierContext.PropertySettings.TypeTokens = new(TypeTokens, GetTypeTokenSegment());
			specifierContext.MetaData = specifierContext.PropertySettings.MetaData;
			specifierContext.MetaNameIndex = UhtMetaData.IndexNone;
			specifierContext.SeenEditSpecifier = false;
			specifierContext.SeenBlueprintWriteSpecifier = false;
			specifierContext.SeenBlueprintReadOnlySpecifier = false;
			specifierContext.SeenBlueprintGetterSpecifier = false;

			// Pre-parse any UPARAM 
			UhtPropertyParser.PreParseType(specifierContext, true);

			// Parse the type
			UhtPropertyResolveArgs templateArgs = new(ResolvePhase, specifierContext.PropertySettings, TokenReader);
			return templateArgs.ResolveProperty();
		}

		/// <summary>
		/// For types such as TPtr or TVal, this method parses the template parameter and extracts the type
		/// </summary>
		/// <returns></returns>
		public readonly UhtProperty? ParseWrappedType()
		{
			using UhtThreadBorrower<UhtPropertySpecifierContext> borrower = new(true);
			UhtPropertySpecifierContext specifierContext = borrower.Instance;
			specifierContext.Type = PropertySettings.Outer;
			specifierContext.TokenReader = TokenReader;
			specifierContext.AccessSpecifier = UhtAccessSpecifier.None;
			specifierContext.MessageSite = TokenReader;
			specifierContext.PropertySettings.Copy(PropertySettings);
			specifierContext.PropertySettings.TypeTokens = new(TypeTokens, GetTypeTokenSegment(), TypeTokens.FullDeclarationSegmentIndex);
			specifierContext.MetaData = PropertySettings.MetaData;
			specifierContext.MetaNameIndex = UhtMetaData.IndexNone;
			specifierContext.SeenEditSpecifier = false;
			specifierContext.SeenBlueprintWriteSpecifier = false;
			specifierContext.SeenBlueprintReadOnlySpecifier = false;
			specifierContext.SeenBlueprintGetterSpecifier = false;

			// Pre-parse any UPARAM 
			UhtPropertyParser.PreParseType(specifierContext, true);

			// Parse the type
			UhtPropertyResolveArgs templateArgs = new(ResolvePhase, specifierContext.PropertySettings, TokenReader);
			return templateArgs.ResolveProperty();
		}

		/// <summary>
		/// For types such as TPtr or TVal, this method parses the template parameter and extracts the type
		/// </summary>
		/// <param name="additionalFlags">Additional flags to be applied to the property</param>
		/// <param name="additionalExportFlags">Additional export flags to be applied to the property</param>
		/// <returns></returns>
		public readonly UhtProperty? ParseWrappedType(EPropertyFlags additionalFlags, UhtPropertyExportFlags additionalExportFlags)
		{
			if (!SkipExpectedType())
			{
				return null;
			}

			PropertySettings.PropertyFlags |= additionalFlags;
			PropertySettings.PropertyExportFlags |= additionalExportFlags;
			TokenReader.Require('<');
			UhtProperty? inner = ParseWrappedType();
			if (inner == null)
			{
				return null;
			}
			TokenReader.Require('>');
			return inner;
		}

		/// <summary>
		/// Parse a template type
		/// </summary>
		/// <param name="mode">Adjust what is returned and/or what is searched</param>
		/// <returns>Referenced class</returns>
		public readonly UhtClass? ParseTemplateObject(UhtTemplateObjectMode mode)
		{
			if (TokenReader.TryOptional("const"))
			{
				PropertySettings.MetaData.Add(UhtNames.NativeConst, "");
			}

			if (!SkipExpectedType())
			{
				return null;
			}

			bool isNativeConstTemplateArg = false;
			UhtTokenList? identifier = null;
			TokenReader
				.Require('<')
				.Optional("const", () => { isNativeConstTemplateArg = true; })
				.Optional("class")
				.RequireCppIdentifier(UhtCppIdentifierOptions.None, tokenList => { identifier = tokenList; })
				.Optional("const", () => { isNativeConstTemplateArg = true; })
				.Require('>');

			if (identifier == null)
			{
				throw new UhtIceException("Expected an identifier list");
			}
			identifier.RedirectTypeIdentifier(Config);

			if (isNativeConstTemplateArg)
			{
				PropertySettings.MetaData.Add(UhtNames.NativeConstTemplateArg, "");
			}

			string scratchString;
			if (mode == UhtTemplateObjectMode.NativeInterfaceProxy)
			{
				UhtTokenList last = identifier.Last;
				UhtToken token = last.Token;
				if (!token.Value.Span.StartsWith("I", StringComparison.Ordinal) || !token.Value.Span.EndsWith(UhtNames.VerseProxySuffix, StringComparison.Ordinal))
				{
					TokenReader.LogError(token.InputLine, $"Expected a Verse interface proxy.  Must begin with 'I' and end with '{UhtNames.VerseProxySuffix}'");
					return null;
				}
				scratchString = $"U{token.Value.Span[1..(token.Value.Length - UhtNames.VerseProxySuffix.Length)]}";
				token.Value = scratchString;
				last.Token = token;
			}

			UhtClass? returnClass = PropertySettings.Outer.FindType(UhtFindOptions.SourceName | UhtFindOptions.Class, identifier, TokenReader) as UhtClass;
			if (returnClass != null && returnClass.AlternateObject != null && mode == UhtTemplateObjectMode.Normal)
			{
				returnClass = returnClass.AlternateObject as UhtClass;
			}
			return returnClass;
		}

		/// <summary>
		/// Parse the token stream for a class type and then invoke the factory to create the property
		/// </summary>
		/// <param name="mode">Adjust what is returned and/or what is searched</param>
		/// <param name="propertyFactory">Factory invoked to create the property</param>
		/// <returns>Created property or null on error</returns>
		public readonly UhtProperty? ParseTemplateObjectProperty(UhtTemplateObjectMode mode, Func<UhtClass, int, UhtProperty?> propertyFactory)
		{
			int typeStartPos = TokenReader.PeekToken().InputStartPos;
			UhtClass? referencedClass = ParseTemplateObject(mode);
			if (referencedClass == null)
			{
				return null;
			}

			return propertyFactory(referencedClass, typeStartPos);
		}

		/// <summary>
		/// Parse a template type
		/// </summary>
		/// <returns>Referenced class</returns>
		public readonly UhtClass? ParseTemplateClass()
		{
			if (TokenReader.TryOptional("const"))
			{
				PropertySettings.MetaData.Add(UhtNames.NativeConst, "");
			}

			if (!SkipExpectedType())
			{
				return null;
			}

			UhtTokenList? identifier = null;
			TokenReader
				.Require('<')
				.Optional("class")
				.RequireCppIdentifier(UhtCppIdentifierOptions.None, tokenList => { identifier = tokenList; })
				.Require('>');

			if (identifier == null)
			{
				throw new UhtIceException("Expected an identifier list");
			}
			identifier.RedirectTypeIdentifier(Config);

			UhtClass? returnClass = PropertySettings.Outer.FindType(UhtFindOptions.SourceName | UhtFindOptions.Class, identifier, TokenReader) as UhtClass;
			if (returnClass != null && returnClass.AlternateObject != null)
			{
				returnClass = returnClass.AlternateObject as UhtClass;
			}
			return returnClass;
		}

		/// <summary>
		/// Parse a template type
		/// </summary>
		/// <returns>Referenced class</returns>
		public readonly UhtScriptStruct? ParseTemplateScriptStruct()
		{
			if (TokenReader.TryOptional("const"))
			{
				PropertySettings.MetaData.Add(UhtNames.NativeConst, "");
			}

			if (!SkipExpectedType())
			{
				return null;
			}

			UhtTokenList? identifier = null;
			TokenReader
				.Require('<')
				.Optional("struct")
				.RequireCppIdentifier(UhtCppIdentifierOptions.None, tokenList => { identifier = tokenList; })
				.Require('>');

			if (identifier == null)
			{
				throw new UhtIceException("Expected an identifier list");
			}
			identifier.RedirectTypeIdentifier(Config);

			return PropertySettings.Outer.FindType(UhtFindOptions.SourceName | UhtFindOptions.ScriptStruct, identifier, TokenReader) as UhtScriptStruct;
		}

		/// <summary>
		/// Logs message for Object pointers to convert UObject* to TObjectPtr or the reverse
		/// </summary>
		/// <param name="behaviorSet">Behavior set being tested against</param>
		/// <param name="pointerTypeDesc">Description of the pointer type</param>
		/// <param name="typeStartPos">Starting character position of the type</param>
		/// <param name="alternativeTypeDesc">Suggested alternate declaration</param>
		/// <exception cref="UhtIceException">Thrown if the behavior type is unexpected</exception>
		public readonly void ConditionalLogPointerUsage(UhtIssueBehaviorSet behaviorSet, string pointerTypeDesc, int typeStartPos, string? alternativeTypeDesc)
		{
			if (PropertySettings.PropertyCategory != UhtPropertyCategory.Member)
			{
				return;
			}

			UhtModule module = PropertySettings.Outer.Module;
			UhtIssueBehavior behavior = behaviorSet.GetBehavior(module);

			// HACK for CoreUObject tests - allow raw pointers in non-intrinsic classes there.
			if (module.ShortName == "CoreUObject" && PropertySettings.Outer.HeaderFile.ModuleRelativeFilePath.StartsWith("Tests", StringComparison.InvariantCultureIgnoreCase))
			{
				behavior = UhtIssueBehavior.AllowSilently;
			}

			if (behavior == UhtIssueBehavior.AllowSilently)
			{
				return;
			}

			string type = TokenReader.GetStringView(typeStartPos, TokenReader.InputPos - typeStartPos).ToString();
			type = type.Replace("\n", "\\n", StringComparison.Ordinal);
			type = type.Replace("\r", "\\r", StringComparison.Ordinal);
			type = type.Replace("\t", "\\t", StringComparison.Ordinal);

			switch (behavior)
			{
				case UhtIssueBehavior.Disallow:
					if (!String.IsNullOrEmpty(alternativeTypeDesc))
					{
						TokenReader.LogError($"{pointerTypeDesc} usage in member declaration detected [[[{type}]]].  This is disallowed for the target/module, consider {alternativeTypeDesc} as an alternative.");
					}
					else
					{
						TokenReader.LogError($"{pointerTypeDesc} usage in member declaration detected [[[{type}]]].");
					}
					break;

				case UhtIssueBehavior.AllowAndLog:
					if (!String.IsNullOrEmpty(alternativeTypeDesc))
					{
						TokenReader.LogInfo($"{pointerTypeDesc} usage in member declaration detected [[[{type}]]].  Consider {alternativeTypeDesc} as an alternative.");
					}
					else
					{
						TokenReader.LogInfo($"{pointerTypeDesc} usage in member declaration detected [[[{type}]]].");
					}
					break;

				default:
					throw new UhtIceException("Unknown enum value");
			}
		}

		/// <summary>
		/// Using the current type in the type tokens, try to locate a property type
		/// </summary>
		/// <returns>The referenced property type or null if we don't have a parser for the type in question.</returns>
		private readonly bool ResolvePropertyType(out UhtPropertyType propertyType)
		{
			ReadOnlySpan<UhtToken> type = TypeTokens.TypeTokens.Span;

			// Remove any leading 'class', 'struct', or 'enum' keyword
			if (TypeTokens.DeclarationSegment.FindFilter != UhtFindOptions.None)
			{
				type = type[1..];
			}

			// Remove any leading '::'
			if (type.Length > 0 && type[0].IsSymbol("::"))
			{
				type = type[1..];
			}

			// If we have no types remaining, then just ignore
			if (type.Length == 0)
			{
				propertyType = new();
				return false;
			}

			// If we have only one token defining the type, there is no reason to building a lookup string.
			if (type.Length == 1)
			{
				UhtToken copy = type[0];
				Config.RedirectTypeIdentifier(ref copy);
				return Session.TryGetPropertyType(copy.Value, out propertyType);
			}

			// Otherwise we need to generate a string
			UhtToken[] copiedArray = new UhtToken[type.Length];
			for (int i = 0; i < type.Length; i++)
			{
				copiedArray[i] = type[i];
				if (copiedArray[i].IsIdentifier())
				{
					Config.RedirectTypeIdentifier(ref copiedArray[i]);
				}
			}
			string fullType = UhtTypeTokens.ToString(copiedArray);
			return Session.TryGetPropertyType(fullType, out propertyType);
		}

		/// <summary>
		/// Given the current position of the token reader, return the segment being parsed
		/// </summary>
		/// <returns>Segment index being parsed</returns>
		/// <exception cref="UhtIceException"></exception>
		private readonly int GetTypeTokenSegment()
		{
			// From the current segment index, look forward for the template segment being parsed
			int tokenIndex = TokenReader.GetReplayTokenIndex();
			int templateSegmentIndex = TypeTokens.DeclarationSegmentIndex + 1;
			foreach (UhtTypeSegment segment in TypeTokens.Segments[(TypeTokens.DeclarationSegmentIndex + 1)..].Span)
			{
				if (tokenIndex < segment.Start)
				{
					throw new UhtIceException("Error parsing template argument");
				}
				if (tokenIndex < segment.End)
				{
					break;
				}
				templateSegmentIndex++;
			}
			if (templateSegmentIndex == TypeTokens.Segments.Length)
			{
				throw new UhtIceException("Error parsing template argument");
			}
			return templateSegmentIndex;
		}
	}
}
