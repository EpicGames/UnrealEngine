// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// Interface used to collect all the objects referenced by a given type.
	/// Not all types such as UhtPackage and UhtHeaderFile support collecting
	/// references due to assorted reasons.
	/// </summary>
	public interface IUhtReferenceCollector
	{
		/// <summary>
		/// Add a reference to an object which needs to be linked. Depending on the type parameter and the configuration, 
		/// it may be linked as a direct pointer, encoded pointer, or a pointer to a factory method.
		/// Also records the module the object is in as a module-module dependency
		/// </summary>
		/// <param name="obj">Object/type being referenced</param>
		/// <param name="type">Type of reference required</param>
		void AddObjectReference(UhtObject? obj, UhtSingletonType type);

		/// <summary>
		/// Add an object declaration
		/// </summary>
		/// <param name="obj">Object in question</param>
		/// <param name="type">Type of reference required</param>
		void AddDeclaration(UhtObject obj, UhtSingletonType type);

		/// <summary>
		/// Add a field as a singleton for exporting
		/// </summary>
		/// <param name="field">Field to be added</param>
		void AddSingleton(UhtField field);

		/// <summary>
		/// Add a field as a type being exported
		/// </summary>
		/// <param name="field">Field to be added</param>
		void AddExportType(UhtField field);

		/// <summary>
		/// Add a forward declaration.  This is the preferred way of adding a forward declaration since the exporter will
		/// handle all the formatting requirements.
		/// </summary>
		/// <param name="field"></param>
		void AddForwardDeclaration(UhtField field);

		/// <summary>
		/// Add a forward declaration.  The string can contain multiple declarations but must only exist on one line.
		/// </summary>
		/// <param name="namespaceObj">The namespace to place the string</param>
		/// <param name="declaration">The declarations to add</param>
		void AddForwardDeclaration(UhtNamespace? namespaceObj, string? declaration);
	}

	/// <summary>
	/// Maintains a list of referenced object indices.
	/// </summary>
	public class UhtUniqueReferenceCollection
	{
		/// <summary>
		/// Returns whether there is anything in this collection
		/// </summary>
		public bool IsEmpty => Uniques.Count == 0;

		/// <summary>
		/// List of all unique references and the types required.
		/// </summary>
		private Dictionary<UhtObject, UhtSingletonTypeFlag> Uniques { get; } = [];

		private static readonly UhtSingletonTypeFlag[] s_typeFlags = Enum.GetValues<UhtSingletonTypeFlag>();

		private static UhtSingletonType FlagToValue(UhtSingletonTypeFlag flag) => flag switch
		{
			UhtSingletonTypeFlag.Registered => UhtSingletonType.Registered,
			UhtSingletonTypeFlag.Unregistered => UhtSingletonType.Unregistered,
			UhtSingletonTypeFlag.ConstInit => UhtSingletonType.ConstInit,
			UhtSingletonTypeFlag.Statics => UhtSingletonType.Statics,
			_ => throw new UhtIceException("flag should be single valued")
		};

		/// <summary>
		/// Add the given object to the references
		/// </summary>
		/// <param name="obj">Object to be added</param>
		/// <param name="type">Type of reference required</param>
		public void Add(UhtObject? obj, UhtSingletonType type)
		{
			if (obj != null)
			{
				UhtSingletonTypeFlag flag = (UhtSingletonTypeFlag)(1 << (int)type);
				if (!Uniques.TryAdd(obj, flag))
				{
					Uniques[obj] |= flag;
				}
			}
		}

		/// <summary>
		/// Get references of a given set of types as a sorted enumerable.
		/// </summary>
		/// <param name="typesFilter"></param>
		/// <returns></returns>
		public IEnumerable<UhtObject> SortReferences(UhtSingletonTypeFlag typesFilter)
		{
			return Uniques.Select((UhtObject Obj, UhtSingletonTypeFlag Matching) (p) => (p.Key, p.Value & typesFilter))
				.Where(p => p.Matching != UhtSingletonTypeFlag.None)
				.Select(p => p.Obj)
				.OrderBy(o => o.Module.ShortName)
				.ThenBy(o => o.EngineName);
		}

		/// <summary>
		/// Return the collection of references sorted by the API string returned by the delegate.
		/// </summary>
		/// <param name="referenceToString">Function to invoke to return the requested object API string, taking the object index and the UhtSingletonType.</param>
		/// <returns>Read only memory region of all the string.</returns>
		public IReadOnlyList<string> GetSortedReferences(Func<int, UhtSingletonType, string?> referenceToString)
		{
			IEnumerable<string> GetStrings(int objIndex, UhtSingletonTypeFlag flags)
			{
				foreach (UhtSingletonTypeFlag test in s_typeFlags)
				{
					if ((flags & test) != UhtSingletonTypeFlag.None)
					{
						if (referenceToString(objIndex, FlagToValue(test)) is string s)
						{
							yield return s;
						}
					}
				}
			}

			// Collect the unsorted array
			List<string> sorted = [.. Uniques.SelectMany(p => GetStrings(p.Key.ObjectTypeIndex, p.Value))];

			// Sort the array
			sorted.Sort(StringComparerUE.OrdinalIgnoreCase);

			// Remove duplicates.  In some instances the different keys might return the same string.
			// This removes those duplicates
			if (sorted.Count > 1)
			{
				int priorOut = 0;
				for (int index = 1; index < sorted.Count; ++index)
				{
					if (sorted[index] != sorted[priorOut])
					{
						++priorOut;
						sorted[priorOut] = sorted[index];
					}
				}

				if (priorOut < sorted.Count - 1)
				{
					sorted.RemoveRange(priorOut + 1, sorted.Count - priorOut - 1);
				}
			}
			return sorted;
		}
	}

	/// <summary>
	/// Standard implementation of the reference collector interface. Used by a single header file in
	/// a module to collect references which are needed for the generated code for that particular header.
	/// </summary>
	public class UhtReferenceCollector(UhtModule thisModule) : IUhtReferenceCollector
	{
		/// <summary>
		/// Other modules encountered during reference collection for building module-to-module links
		/// </summary>
		public HashSet<UhtModule> Modules { get; } = [];

		/// <summary>
		/// Collection of unique object references in this module
		/// </summary>
		public UhtUniqueReferenceCollection InternalObjects { get; set; } = new();

		/// <summary>
		/// Collection of unique object references in other modules
		/// </summary>
		public UhtUniqueReferenceCollection ExternalObjects { get; set; } = new();

		/// <summary>
		/// Collection of unique declarations
		/// </summary>
		public UhtUniqueReferenceCollection Declaration { get; set; } = new();

		/// <summary>
		/// Collection of singletons
		/// </summary>
		public List<UhtField> Singletons { get; } = [];

		/// <summary>
		/// Collection of types to export
		/// </summary>
		public List<UhtField> ExportTypes { get; } = [];

		/// <summary>
		/// Collection of fields needing forward declarations
		/// </summary>
		public HashSet<UhtField> ForwardDeclarations { get; } = [];

		/// <summary>
		/// Collection of forward declarations in text form
		/// </summary>
		public HashSet<string> ForwardDeclarationStrings { get; } = [];

		/// <summary>
		/// Collection of referenced headers
		/// </summary>
		public HashSet<UhtHeaderFile> ReferencedHeaders { get; } = [];

		/// <inheritdoc/>
		public void AddObjectReference(UhtObject? obj, UhtSingletonType type)
		{
			if (obj is not null)
			{
				if (obj.Module == thisModule)
				{
					InternalObjects.Add(obj, type);
				}
				else
				{
					ExternalObjects.Add(obj, type);
					Modules.Add(obj.Module);
				}
			}
			if (obj != null && obj is not UhtPackage && type == UhtSingletonType.Registered)
			{
				ReferencedHeaders.Add(obj.HeaderFile);
			}
		}

		/// <inheritdoc/>
		public void AddDeclaration(UhtObject obj, UhtSingletonType type)
		{
			Declaration.Add(obj, type);
		}

		/// <inheritdoc/>
		public void AddSingleton(UhtField field)
		{
			Singletons.Add(field);
		}

		/// <inheritdoc/>
		public void AddExportType(UhtField field)
		{
			ExportTypes.Add(field);
		}

		/// <inheritdoc/>
		public void AddForwardDeclaration(UhtField field)
		{
			ForwardDeclarations.Add(field);
		}

		/// <inheritdoc/>
		public void AddForwardDeclaration(UhtNamespace? namespaceObj, string? declaration)
		{
			if (!String.IsNullOrEmpty(declaration))
			{
				if (namespaceObj == null || namespaceObj.IsGlobal)
				{
					ForwardDeclarationStrings.Add(declaration);
				}
				else
				{
					// This is hardly used so just wrap the declaration in the namespace instead of 
					// trying to group them all together in a multiple line namespace declaration.
					StringBuilder builder = new();
					namespaceObj.AppendSingleLine(builder, (builder) => builder.Append(declaration));
				}
			}
		}
	}
}
