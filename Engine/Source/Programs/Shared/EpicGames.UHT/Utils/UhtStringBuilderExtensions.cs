// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	static class UhtStringBuilderExtensions
	{
		/// <summary>
		/// String of tabs used to generate code with proper indentation
		/// </summary>
		public static StringView TabsString = new(new string('\t', 128));

		/// <summary>
		/// String of spaces used to generate code with proper indentation
		/// </summary>
		public static StringView SpacesString = new(new string(' ', 128));

		/// <summary>
		/// Append tabs to the builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="tabs">Number of tabs to insert</param>
		/// <returns>Destination builder</returns>
		/// <exception cref="ArgumentOutOfRangeException">Thrown if the number of tabs is out of range</exception>
		public static StringBuilder AppendTabs(this StringBuilder builder, int tabs)
		{
			if (tabs < 0 || tabs > TabsString.Length)
			{
				throw new ArgumentOutOfRangeException($"Number of tabs specified must be between 0 and {TabsString.Length - 1} inclusive");
			}
			else if (tabs > 0)
			{
				builder.Append(TabsString.Span[..tabs]);
			}
			return builder;
		}

		/// <summary>
		/// Append spaces to the builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="spaces">Number of spaces to insert</param>
		/// <returns>Destination builder</returns>
		/// <exception cref="ArgumentOutOfRangeException">Thrown if the number of spaces is out of range</exception>
		public static StringBuilder AppendSpaces(this StringBuilder builder, int spaces)
		{
			if (spaces < 0 || spaces > SpacesString.Length)
			{
				throw new ArgumentOutOfRangeException($"Number of spaces specified must be between 0 and {SpacesString.Length - 1} inclusive");
			}
			else if (spaces > 0)
			{
				builder.Append(SpacesString.Span[..spaces]);
			}
			return builder;
		}

		/// <summary>
		/// Append a list of items, transforming each into a StringBuilder append operation
		/// </summary>
		/// <typeparam name="T">Type of element</typeparam>
		/// <param name="builder">StringBuilder to append to </param>
		/// <param name="items">List of items</param>
		/// <param name="transform">Action to transform each item into some kind of Append call on the stringbuilder</param>
		/// <returns></returns>
		public static StringBuilder AppendEach<T>(this StringBuilder builder, IReadOnlyCollection<T> items, Action<StringBuilder, T> transform)
		{
			foreach (T item in items)
			{
				transform(builder, item);
			}
			return builder;
		}

		/// <summary>
		/// Optionally append a list of items with a prefix and suffix. If the list is empty, the prefix and suffix are also omitted
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="builder"></param>
		/// <param name="items"></param>
		/// <param name="prefix"></param>
		/// <param name="transform"></param>
		/// <param name="suffix"></param>
		/// <returns></returns>
		public static StringBuilder AppendEach<T>(this StringBuilder builder, IReadOnlyCollection<T> items, Action<StringBuilder> prefix, Action<StringBuilder, T> transform, Action<StringBuilder> suffix)
		{
			if (items.Count == 0)
			{
				return builder;
			}
			prefix(builder);
			foreach (T item in items)
			{
				transform(builder, item);
			}
			suffix(builder);
			return builder;
		}

		/// <summary>
		/// Append an array view construction expression for the given variable
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="identifier">The name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendArrayView(this StringBuilder builder, UhtCppIdentifier identifier, int tabs, string? endl)
		{
			return builder.AppendTabs(tabs).Append($"MakeArrayView({identifier}){endl}");
		}

		/// <summary>
		/// Append the given text as a UTF8 encoded string
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="useText">If false, don't encode the text but include a nullptr</param>
		/// <param name="text">Text to include or an empty string if null.</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendUTF8LiteralString(this StringBuilder builder, bool useText, string? text)
		{
			if (!useText)
			{
				builder.Append("nullptr");
			}
			else if (text == null)
			{
				builder.Append("");
			}
			else
			{
				builder.AppendUTF8LiteralString(text);
			}
			return builder;
		}

		/// <summary>
		/// Append the given text as a UTF8 encoded string
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="text">Text to include or an empty string if null.</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendUTF8LiteralString(this StringBuilder builder, string? text)
		{
			if (text == null)
			{
				builder.Append("\"\"");
			}
			else
			{
				builder.AppendUTF8LiteralString(new StringView(text));
			}
			return builder;
		}

		/// <summary>
		/// Append the given text as a UTF8 encoded string
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="text">Text to be encoded</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendUTF8LiteralString(this StringBuilder builder, StringView text)
		{
			builder.Append('\"');

			ReadOnlySpan<char> span = text.Span;
			int length = span.Length;

			if (length > 0)
			{

				bool trailingHex = false;
				int index = 0;
				while (true)
				{
					// Scan forward looking for anything that can just be blindly copied
					int startIndex = index;
					while (index < length)
					{
						char cskip = span[index];
						if (cskip < 31 || cskip > 127 || cskip == '"' || cskip == '\\')
						{
							break;
						}
						++index;
					}

					// If we found anything
					if (startIndex < index)
					{
						// We close and open the literal here in order to ensure that successive hex characters aren't appended to the hex sequence, causing a different number
						if (trailingHex && UhtFCString.IsHexDigit(span[startIndex]))
						{
							builder.Append("\"\"");
						}
						builder.Append(span[startIndex..index]);
					}

					// We have either reached the end of the string, break
					if (index == length)
					{
						break;
					}

					// This character requires special processing
					char c = span[index++];
					switch (c)
					{
						case '\r':
							trailingHex = false;
							break;
						case '\n':
							trailingHex = false;
							builder.Append("\\n");
							break;
						case '\\':
							trailingHex = false;
							builder.Append("\\\\");
							break;
						case '\"':
							trailingHex = false;
							builder.Append("\\\"");
							break;
						default:
							if (c < 31)
							{
								trailingHex = true;
								builder.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", (uint)c);
							}
							else
							{
								trailingHex = false;
								if (Char.IsHighSurrogate(c))
								{
									if (index == length)
									{
										builder.Append('?');
										break;
									}

									char clow = span[index];
									if (Char.IsLowSurrogate(clow))
									{
										++index;
										builder.AppendEscapedUtf32((ulong)Char.ConvertToUtf32(c, clow));
										trailingHex = true;
									}
									else
									{
										builder.Append('?');
									}
								}
								else if (Char.IsLowSurrogate(c))
								{
									builder.Append('?');
								}
								else
								{
									builder.AppendEscapedUtf32(c);
									trailingHex = true;
								}
							}
							break;
					}
				}
			}

			builder.Append('\"');
			return builder;
		}

		/// <summary>
		/// Encode a single UTF32 value as UTF8 characters
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="c">Character to encode</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendEscapedUtf32(this StringBuilder builder, ulong c)
		{
			if (c < 0x80)
			{
				builder
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", c);
			}
			else if (c < 0x800)
			{
				builder
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0xC0 + (c >> 6))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + (c & 0x3f));
			}
			else if (c < 0x10000)
			{
				builder
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0xE0 + (c >> 12))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + ((c >> 6) & 0x3f))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + (c & 0x3f));
			}
			else
			{
				builder
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0xF0 + (c >> 18))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + ((c >> 12) & 0x3f))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + ((c >> 6) & 0x3f))
					.Append("\\x").AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", 0x80 + (c & 0x3f));
			}
			return builder;
		}

		/// <summary>
		/// Append the given name of the class but always encode interfaces as the native interface name (i.e. "I...")
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="classObj">Class to append name</param>
		/// <returns>Destination builder</returns>
		/// <exception cref="NotImplementedException">Class has an unexpected class type</exception>
		public static StringBuilder AppendClassSourceNameOrInterfaceName(this StringBuilder builder, UhtClass classObj)
		{
			return builder.Append(classObj.Namespace.FullSourceName).Append(classObj.NativeFunctionCallName);
		}

		/// <summary>
		/// Append the given name of the class but always encode interfaces as the native interface proxy name (i.e. "I...")
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="classObj">Class to append name</param>
		/// <returns>Destination builder</returns>
		/// <exception cref="NotImplementedException">Class has an unexpected class type</exception>
		public static StringBuilder AppendClassSourceNameOrInterfaceProxyName(this StringBuilder builder, UhtClass classObj)
		{
			switch (classObj.ClassType)
			{
				case UhtClassType.Class:
				case UhtClassType.VModule:
					return builder.Append(classObj.SourceName);
				case UhtClassType.NativeInterface:
					return builder.Append(classObj.SourceName).Append(UhtNames.VerseProxySuffix);
				case UhtClassType.Interface:
					return builder.Append('I').Append(classObj.SourceName[1..]).Append(UhtNames.VerseProxySuffix);
				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Given a verse name, encode it so it can be represented as an FName
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="name">Name to encode</param>
		/// <returns>Builder</returns>
		public static StringBuilder AppendEncodedVerseName(this StringBuilder builder, ReadOnlySpan<char> name)
		{
			bool isFirstChar = true;
			while (!name.IsEmpty)
			{
				char c = name[0];
				name = name[1..];

				if ((c >= 'a' && c <= 'z')
					|| (c >= 'A' && c <= 'Z')
					|| (c >= '0' && c <= '9' && !isFirstChar))
				{
					builder.Append(c);
				}
				else if (c == '[' && !name.IsEmpty && name[0] == ']')
				{
					name = name[1..];
					builder.Append("_K");
				}
				else if (c == '-' && !name.IsEmpty && name[0] == '>')
				{
					name = name[1..];
					builder.Append("_T");
				}
				else if (c == '_')
				{
					builder.Append("__");
				}
				else if (c == '(')
				{
					builder.Append("_L");
				}
				else if (c == ',')
				{
					builder.Append("_M");
				}
				else if (c == ':')
				{
					builder.Append("_N");
				}
				else if (c == '^')
				{
					builder.Append("_P");
				}
				else if (c == '?')
				{
					builder.Append("_Q");
				}
				else if (c == ')')
				{
					builder.Append("_R");
				}
				else if (c == '\'')
				{
					builder.Append("_U");
				}
				else
				{
					builder.Append('_').AppendFormat(CultureInfo.InvariantCulture, "{0:x2}", (uint)c);
				}

				isFirstChar = false;
			}
			return builder;
		}
	}

	/// <summary>
	/// Provides a cache of StringBuilders
	/// </summary>
	public class StringBuilderCache
	{

		/// <summary>
		/// Cache of StringBuilders with large initial buffer sizes
		/// </summary>
		public static readonly StringBuilderCache Big = new(256, 256 * 1024);

		/// <summary>
		/// Cache of StringBuilders with small initial buffer sizes
		/// </summary>
		public static readonly StringBuilderCache Small = new(256, 1 * 1024);

		/// <summary>
		/// Capacity of the cache
		/// </summary>
		private readonly int _capacity;

		/// <summary>
		/// Initial buffer size for new StringBuilders.  Resetting StringBuilders might result
		/// in the initial chunk size being smaller.
		/// </summary>
		private readonly int _initialBufferSize;

		/// <summary>
		/// Stack of cached StringBuilders
		/// </summary>
		private readonly Stack<StringBuilder> _stack;

		/// <summary>
		/// Create a new StringBuilder cache
		/// </summary>
		/// <param name="capacity">Maximum number of StringBuilders to cache</param>
		/// <param name="initialBufferSize">Initial buffer size for newly created StringBuilders</param>
		public StringBuilderCache(int capacity, int initialBufferSize)
		{
			_capacity = capacity;
			_initialBufferSize = initialBufferSize;
			_stack = new Stack<StringBuilder>(_capacity);
		}

		/// <summary>
		/// Borrow a StringBuilder from the cache.
		/// </summary>
		/// <returns></returns>
		public StringBuilder Borrow()
		{
			lock (_stack)
			{
				if (_stack.Count > 0)
				{
					return _stack.Pop();
				}
			}

			return new StringBuilder(_initialBufferSize);
		}

		/// <summary>
		/// Return a StringBuilder to the cache
		/// </summary>
		/// <param name="builder">The builder being returned</param>
		public void Return(StringBuilder builder)
		{
			// Sadly, clearing the builder (sets length to 0) will reallocate chunks.
			builder.Clear();
			lock (_stack)
			{
				if (_stack.Count < _capacity)
				{
					_stack.Push(builder);
				}
			}
		}
	}

	/// <summary>
	/// Structure to automate the borrowing and returning of a StringBuilder.
	/// Use some form of a "using" pattern.
	/// </summary>
	public readonly struct BorrowStringBuilder : IDisposable
	{

		/// <summary>
		/// Owning cache
		/// </summary>
		private StringBuilderCache Cache { get; }

		/// <summary>
		/// Borrowed string builder
		/// </summary>
		public StringBuilder StringBuilder { get; }

		/// <summary>
		/// Borrow a string builder from the given cache
		/// </summary>
		/// <param name="cache">String builder cache</param>
		public BorrowStringBuilder(StringBuilderCache cache)
		{
			Cache = cache;
			StringBuilder = Cache.Borrow();
		}

		/// <summary>
		/// Return the string builder to the cache
		/// </summary>
		public void Dispose()
		{
			Cache.Return(StringBuilder);
		}
	}
}
