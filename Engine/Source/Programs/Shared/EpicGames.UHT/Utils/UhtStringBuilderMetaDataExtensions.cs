// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using EpicGames.UHT.Exporters.CodeGen;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	static class UhtStringBuilderMetaDataExtensions
	{
		/// <summary>
		/// Names of meta data entries that will not appear in shipping builds for game code
		/// </summary>
		private static readonly HashSet<string> s_hiddenMetaDataNames = new([UhtNames.Comment, UhtNames.ToolTip], StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Append an array view construction expression for metadata if the metadata collection is not empty.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="metaData">MetaData object to check for emptiness</param>
		/// <param name="identifier">The name of the array</param>
		/// <param name="tabs">Number of tabs to start the line</param>
		/// <param name="endl">Text to end the line</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendMetaDataArrayView(this StringBuilder builder, UhtMetaData metaData, UhtCppIdentifier identifier, int tabs, string endl)
		{
			if (metaData.IsEmpty())
			{
				return builder.AppendTabs(tabs).Append("{}").Append(endl);
			}
			return builder.AppendTabs(tabs).Append($"MakeArrayView({identifier}_MetaData)").Append(endl);
		}

		/// <summary>
		/// Append the meta data parameters.  This is intended to be used as arguments to a function call.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="type">Source type containing the meta data</param>
		/// <param name="identifier">Name prefix</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendMetaDataParams(this StringBuilder builder, UhtType type, UhtCppIdentifier identifier)
		{
			return AppendMetaDataParams(builder, type.MetaData, identifier);
		}

		/// <summary>
		/// Append the meta data parameters.  This is intended to be used as arguments to a function call.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="metaData">Meta data</param>
		/// <param name="identifier">Name prefix</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendMetaDataParams(this StringBuilder builder, UhtMetaData? metaData, UhtCppIdentifier identifier)
		{
			if (metaData != null && !metaData.IsEmpty())
			{
				return builder.Append($"METADATA_PARAMS(UE_ARRAY_COUNT({identifier}_MetaData), {identifier}_MetaData)");
			}
			else
			{
				return builder.Append("METADATA_PARAMS(0, nullptr)");
			}
		}

		/// <summary>
		/// Append the metadata definition, i.e. a definition for a array of structures containing name/value string pairs.
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="type">Source type containing the meta data</param>
		/// <param name="identifier">Identifier</param>
		/// <param name="tabs">Number of tabs to indent</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendMetaData(this StringBuilder builder, UhtType type, UhtCppIdentifier identifier, int tabs)
		{
			if (!type.MetaData.IsEmpty())
			{
				UhtSession session = type.Session;
				if (session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.ConstInit))
				{
					using UhtConditionalMacroBlock block = new(builder, "UE_WITH_CONSTINIT_UOBJECT", session.IsUsingMultipleCompiledInObjectFormats);
					builder.AppendMetaData(type, identifier, "UE::CodeGen::ConstInit::FMetaData", tabs);
				}
				if (session.IsUsingCompiledInObjectFormat(UhtCompiledInObjectFormat.Params))
				{
					using UhtConditionalMacroBlock block = new(builder, "!UE_WITH_CONSTINIT_UOBJECT", session.IsUsingMultipleCompiledInObjectFormats);
					builder.AppendMetaData(type, identifier, "UECodeGen_Private::FMetaDataPairParam", tabs);
				}
			}
			return builder;
		}

		private static StringBuilder AppendMetaData(this StringBuilder builder, UhtType type, UhtCppIdentifier identifier, string declType, int tabs)
		{
			bool isPartOfEngine = type.Module.IsPartOfEngine;
			List<KeyValuePair<string, string>> sortedMetaData = type.MetaData.GetSorted();
			builder.AppendTabs(tabs).Append($"static constexpr {declType} {identifier}_MetaData[] = {{\r\n");
			foreach (KeyValuePair<string, string> kvp in sortedMetaData)
			{
				bool restricted = !isPartOfEngine && s_hiddenMetaDataNames.Contains(kvp.Key);
				if (restricted)
				{
					builder.Append("#if !UE_BUILD_SHIPPING\r\n");
				}
				// TODO: Make UTF8TEXT and change code generation to emit \u and \U instad of \x 
				builder.AppendTabs(tabs + 1).Append("{ ").AppendUTF8LiteralString(kvp.Key).Append(", ").AppendUTF8LiteralString(kvp.Value).Append(" },\r\n");
				if (restricted)
				{
					builder.Append("#endif\r\n");
				}
			}
			builder.AppendTabs(tabs).Append("};\r\n");
			return builder;
		}

		/// <summary>
		/// Append a metadata definition for a type with properties (e.g. a class or struct)
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="type">Source type containing the meta data</param>
		/// <param name="identifier">Identifier</param>
		/// <param name="propertyContext">Context for formatting properties</param>
		/// <param name="properties">Optional collection of properties to output</param>
		/// <param name="tabs">Number of tabs to indent</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendConditionalMetaData(this StringBuilder builder, UhtType type, UhtCppIdentifier identifier, IUhtPropertyMemberContext? propertyContext, UhtUsedDefineScopes<UhtProperty>? properties, int tabs)
		{
			if (!type.MetaData.IsEmpty() || (propertyContext != null && properties != null && properties.Instances.Any(x => !x.MetaData.IsEmpty())))
			{
				builder.Append("#if WITH_METADATA\r\n");
				builder.AppendMetaData(type, identifier, tabs); // Appends an entire array definition
				if (propertyContext != null && properties != null)
				{
					builder.AppendInstances(properties,
						(builder, property) =>
						{
							property.AppendMetaData(builder, propertyContext, UhtNames.GetPropertyIdentifier(property), tabs);
						});
				}
				builder.Append("#endif // WITH_METADATA\r\n");
			}
			return builder;
		}
	}
}
