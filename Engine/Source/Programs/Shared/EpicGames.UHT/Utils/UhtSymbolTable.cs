// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Symbol table
	/// </summary>
	internal sealed class UhtSymbolTable
	{

		/// <summary>
		/// Represents symbol name lookup chain start.  Symbol chains are based on 
		/// the caseless name.
		/// </summary>
		struct Lookup
		{

			/// <summary>
			/// The type index of the symbol
			/// </summary>
			public int SymbolIndex { get; set; }

			/// <summary>
			/// When searching the caseless chain, the cased index is used to match the symbol based
			/// on the case named.
			/// </summary>
			public int CasedIndex { get; set; }
		}

		/// <summary>
		/// Entry in the symbol table
		/// </summary>
		struct Symbol
		{

			/// <summary>
			/// The type associated with the symbol
			/// </summary>
			public UhtType Type { get; set; }

			/// <summary>
			/// Mask of different find options which will match this symbol
			/// </summary>
			public UhtFindOptions MatchOptions { get; set; }

			/// <summary>
			/// The case lookup index for matching by case
			/// </summary>
			public int CasedIndex { get; set; }

			/// <summary>
			/// The next index in the symbol change based on cassless lookup
			/// </summary>
			public int NextIndex { get; set; }

			/// <summary>
			/// The last index in the chain.  This index is only used when the symbol entry is also acting as the list
			/// </summary>
			public int LastIndex { get; set; }
		}

		/// <summary>
		/// Number of unique cased symbol names.  
		/// </summary>
		private int _casedCount = 0;

		/// <summary>
		/// Case name lookup table that returns the symbol index and the case index
		/// </summary>
		private readonly Dictionary<string, Lookup> _casedDictionary = new(StringComparer.Ordinal);

		/// <summary>
		/// Caseless name lookup table that returns the symbol index
		/// </summary>
		private readonly Dictionary<string, int> _caselessDictionary = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Collection of symbols in the table
		/// </summary>
		private readonly Symbol[] _symbols;

		/// <summary>
		/// Constructs a new symbol table.
		/// </summary>
		/// <param name="typeCount">Number of types in the table.</param>
		public UhtSymbolTable(int typeCount)
		{
			_symbols = new Symbol[typeCount];
		}

		/// <summary>
		/// Add a new type to the symbol table
		/// </summary>
		/// <param name="type">The type being added</param>
		/// <param name="name">The name of the type which could be the source name or the engine name</param>
		public void Add(UhtType type, string name)
		{
			if (_casedDictionary.TryGetValue(name, out Lookup existing))
			{
				AddExisting(type, existing.CasedIndex, existing.SymbolIndex);
				return;
			}

			int casedIndex = ++_casedCount;

			if (_caselessDictionary.TryGetValue(name, out int symbolIndex))
			{
				_casedDictionary.Add(name, new Lookup { SymbolIndex = symbolIndex, CasedIndex = casedIndex });
				AddExisting(type, casedIndex, symbolIndex);
				return;
			}

			symbolIndex = type.TypeIndex;
			_symbols[symbolIndex] = new Symbol { Type = type, MatchOptions = type.EngineType.FindOptions(), CasedIndex = casedIndex, NextIndex = 0, LastIndex = symbolIndex };
			_caselessDictionary.Add(name, symbolIndex);
			_casedDictionary.Add(name, new Lookup { SymbolIndex = symbolIndex, CasedIndex = casedIndex });
		}

		/// <summary>
		/// Replace an entry in the symbol table.  This is used during property resolution to replace the 
		/// parser property (which could not resolve the property prior to the symbol table being created)
		/// with the fully resolved property.
		/// </summary>
		/// <param name="oldType">The old type being replaced.</param>
		/// <param name="newType">The new type.</param>
		/// <param name="name">The name of the type.</param>
		/// <exception cref="UhtIceException">Thrown if the symbol wasn't found.</exception>
		public void Replace(UhtType oldType, UhtType newType, string name)
		{
			if (_caselessDictionary.TryGetValue(name, out int symbolIndex))
			{
				for (; symbolIndex != 0; symbolIndex = _symbols[symbolIndex].NextIndex)
				{
					if (_symbols[symbolIndex].Type == oldType)
					{
						_symbols[symbolIndex].Type = newType;
						return;
					}
				}
			}
			throw new UhtIceException("Attempt to replace a type that wasn't found");
		}

		/// <summary>
		/// Hide the given type in the symbol table
		/// </summary>
		/// <param name="typeToHide">Type to be hidden</param>
		/// <param name="name">The name of the type.</param>
		/// <exception cref="UhtIceException">Thrown if the symbol wasn't found.</exception>
		public void Hide(UhtType typeToHide, string name)
		{
			if (_caselessDictionary.TryGetValue(name, out int symbolIndex))
			{
				for (; symbolIndex != 0; symbolIndex = _symbols[symbolIndex].NextIndex)
				{
					if (_symbols[symbolIndex].Type == typeToHide)
					{
						_symbols[symbolIndex].MatchOptions = 0;
						return;
					}
				}
			}
			throw new UhtIceException("Attempt to hide a type that wasn't found");
		}

		/// <summary>
		/// Lookup the given name using cased string compare.
		/// </summary>
		/// <param name="startingType">Starting type used to limit the scope of the search.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="name">Name to locate.</param>
		/// <returns>Found type or null if not found.</returns>
		public UhtType? FindCasedType(UhtType? startingType, UhtFindOptions options, string name)
		{
			if (_casedDictionary.TryGetValue(name, out Lookup existing))
			{
				return FindType(startingType, options, existing);
			}
			return null;
		}

		/// <summary>
		/// Lookup the given name using caseless string compare.
		/// </summary>
		/// <param name="startingType">Starting type used to limit the scope of the search.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="name">Name to locate.</param>
		/// <returns>Found type or null if not found.</returns>
		public UhtType? FindCaselessType(UhtType? startingType, UhtFindOptions options, string name)
		{
			if (_caselessDictionary.TryGetValue(name, out int symbolIndex))
			{
				return FindType(startingType, options, new Lookup { SymbolIndex = symbolIndex, CasedIndex = 0 });
			}
			return null;
		}

		/// <summary>
		/// Lookup the given name.
		/// </summary>
		/// <param name="startingType">Starting type used to limit the scope of the search.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="lookup">Starting lookup location.</param>
		/// <returns>Found type or null if not found.</returns>
		private UhtType? FindType(UhtType? startingType, UhtFindOptions options, Lookup lookup)
		{
			if (startingType != null)
			{
				UhtType? found = FindTypeSuperChain(startingType, options, ref lookup);
				if (found != null)
				{
					return found;
				}

				found = FindTypeOuterChain(startingType, options, ref lookup);
				if (found != null)
				{
					return found;
				}

				if (!options.HasAnyFlags(UhtFindOptions.NoIncludes))
				{
					UhtHeaderFile headerFile = startingType.HeaderFile;
					foreach (UhtHeaderFile includedFile in headerFile.IncludedHeaders)
					{
						found = FindSymbolChain(includedFile, options, ref lookup);
						if (found != null)
						{
							break;
						}
					}
					if (found != null)
					{
						return found;
					}
				}
			}

			// Global search.  Match anything that has an owner of parent
			if (!options.HasAnyFlags(UhtFindOptions.NoGlobal))
			{
				for (int index = lookup.SymbolIndex; index != 0; index = _symbols[index].NextIndex)
				{
					if (IsMatch(options, index, lookup.CasedIndex) && _symbols[index].Type.Outer is UhtPackage)
					{
						return _symbols[index].Type;
					}
				}
			}

			// Can't find at all
			return null;
		}

		/// <summary>
		/// Lookup the given name using the super class/struct chain.
		/// </summary>
		/// <param name="startingType">Starting type used to limit the scope of the search.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="lookup">Starting lookup location.</param>
		/// <returns>Found type or null if not found.</returns>
		private UhtType? FindTypeSuperChain(UhtType startingType, UhtFindOptions options, ref Lookup lookup)
		{
			UhtType? currentType = startingType;

			// In a super chain search, we have to start at a UHTStruct that contributes to the symbol table
			for (; currentType != null; currentType = currentType.Outer)
			{
				if (currentType is UhtStruct structObj && structObj.EngineType.AddChildrenToSymbolTable())
				{
					break;
				}
			}

			// Not symbol that supports a super chain
			if (currentType == null)
			{
				return null;
			}

			// If requested, skip self
			if (options.HasAnyFlags(UhtFindOptions.NoSelf) && currentType == startingType)
			{
				if (currentType is UhtFunction)
				{
					if (currentType.Outer is UhtStruct outerStructType)
					{
						currentType = outerStructType;
					}
					else
					{
						currentType = null;
					}
				}
				else
				{
					currentType = ((UhtStruct)currentType).Super;
				}
			}

			// Search the chain
			for (; currentType != null; currentType = ((UhtStruct)currentType).Super)
			{
				UhtType? foundType = FindSymbolChain(currentType, options, ref lookup);
				if (foundType != null)
				{
					return foundType;
				}
				if (options.HasAnyFlags(UhtFindOptions.NoParents))
				{
					return null;
				}
			}
			return null;
		}

		/// <summary>
		/// Lookup the given name using the outer chain
		/// </summary>
		/// <param name="startingType">Starting type used to limit the scope of the search.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="lookup">Starting lookup location.</param>
		/// <returns>Found type or null if not found.</returns>
		private UhtType? FindTypeOuterChain(UhtType startingType, UhtFindOptions options, ref Lookup lookup)
		{
			UhtType? currentType = startingType;

			// If requested, skip self
			if (options.HasAnyFlags(UhtFindOptions.NoSelf) && currentType == startingType)
			{
				currentType = currentType.Outer;
			}

			// Search the chain
			for (; currentType != null; currentType = currentType.Outer)
			{

				// When we hit the package, we assume that we want to do a search of the whole header file.  But this
				// is only done when starting type isn't the package itself
				if (currentType is UhtPackage)
				{
					return startingType != currentType ? FindSymbolChain(startingType.HeaderFile, options, ref lookup) : null;
				}
				UhtType? foundType = FindSymbolChain(currentType, options, ref lookup);
				if (foundType != null)
				{
					return foundType;
				}
				if (options.HasAnyFlags(UhtFindOptions.NoOuter))
				{
					return null;
				}
			}
			return null;
		}

		/// <summary>
		/// Lookup the given name 
		/// </summary>
		/// <param name="owner">Matching owner.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="lookup">Starting lookup location.</param>
		/// <returns>Found type or null if not found.</returns>
		private UhtType? FindSymbolChain(UhtType owner, UhtFindOptions options, ref Lookup lookup)
		{
			for (int index = lookup.SymbolIndex; index != 0; index = _symbols[index].NextIndex)
			{
				if (IsMatch(options, index, lookup.CasedIndex) && _symbols[index].Type.Outer == owner)
				{
					return _symbols[index].Type;
				}
			}
			return null;
		}

		/// <summary>
		/// Lookup the given name 
		/// </summary>
		/// <param name="header">Header file being searched.</param>
		/// <param name="options">Options controlling what is search and what is returned.</param>
		/// <param name="lookup">Starting lookup location.</param>
		/// <returns>Found type or null if not found.</returns>
		private UhtType? FindSymbolChain(UhtHeaderFile header, UhtFindOptions options, ref Lookup lookup)
		{
			for (int index = lookup.SymbolIndex; index != 0; index = _symbols[index].NextIndex)
			{
				// Match on the same header file, but only top level types
				if (IsMatch(options, index, lookup.CasedIndex) && _symbols[index].Type.HeaderFile == header && _symbols[index].Type.Outer is UhtPackage)
				{
					return _symbols[index].Type;
				}
			}
			return null;
		}

		/// <summary>
		/// Add a new type to the given symbol chain
		/// </summary>
		/// <param name="type">Type being added</param>
		/// <param name="casedIndex">Cased index</param>
		/// <param name="symbolIndex">Symbol index</param>
		private void AddExisting(UhtType type, int casedIndex, int symbolIndex)
		{
			int typeIndex = type.TypeIndex;
			_symbols[typeIndex] = new Symbol { Type = type, MatchOptions = type.EngineType.FindOptions(), CasedIndex = casedIndex, NextIndex = 0, LastIndex = 0 };
			_symbols[_symbols[symbolIndex].LastIndex].NextIndex = typeIndex;
			_symbols[symbolIndex].LastIndex = typeIndex;
		}

		/// <summary>
		/// Test to see if the given symbol matches the options
		/// </summary>
		/// <param name="options">Options to match</param>
		/// <param name="symbolIndex">Symbol index</param>
		/// <param name="casedIndex">Case index</param>
		/// <returns>True if the symbol is a match</returns>
		private bool IsMatch(UhtFindOptions options, int symbolIndex, int casedIndex)
		{
			return (casedIndex == 0 || casedIndex == _symbols[symbolIndex].CasedIndex) &&
				(_symbols[symbolIndex].MatchOptions & options) != 0;
		}
	}
}
