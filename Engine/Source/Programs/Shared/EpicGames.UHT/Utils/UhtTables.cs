// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using EpicGames.UHT.Tables;
using EpicGames.UHT.Types;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Defines all the table names for the standard UHT types
	/// </summary>
	public static class UhtTableNames
	{
		/// <summary>
		/// The class base table is common to UCLASS, UINTERFACE and native interfaces
		/// </summary>
		public const string ClassBase = "ClassBase";

		/// <summary>
		/// Table for UCLASS
		/// </summary>
		public const string Class = "Class";

		/// <summary>
		/// Default table applied to all other tables
		/// </summary>
		public const string Default = "Default";

		/// <summary>
		/// Table for UENUM
		/// </summary>
		public const string Enum = "Enum";

		/// <summary>
		/// Table for all types considered a UField
		/// </summary>
		public const string Field = "Field";

		/// <summary>
		/// Table for all functions
		/// </summary>
		public const string Function = "Function";

		/// <summary>
		/// Table for the global/file scope
		/// </summary>
		public const string Global = "Global";

		/// <summary>
		/// Table for UINTERFACE
		/// </summary>
		public const string Interface = "Interface";

		/// <summary>
		/// Table for interfaces
		/// </summary>
		public const string NativeInterface = "NativeInterface";

		/// <summary>
		/// Table for all UObject types
		/// </summary>
		public const string Object = "Object";

		/// <summary>
		/// Table for properties that are function arguments
		/// </summary>
		public const string PropertyArgument = "PropertyArgument";

		/// <summary>
		/// Table for properties that are struct/class members
		/// </summary>
		public const string PropertyMember = "PropertyMember";

		/// <summary>
		/// Table for USTRUCT
		/// </summary>
		public const string ScriptStruct = "ScriptStruct";

		/// <summary>
		/// Table for all UStruct objects (structs, classes, and functions)
		/// </summary>
		public const string Struct = "Struct";

		/// <summary>
		/// Table for VModules
		/// </summary>
		public const string VModule = "VModule";

		/// <summary>
		/// Table for UPARTIAL
		/// </summary>
		public const string Partial = "Partial";
	}

	/// <summary>
	/// Base class for table lookup system.
	/// </summary>
	public class UhtLookupTableBase
	{
		/// <summary>
		/// This table inherits entries for the given table
		/// </summary>
		public UhtLookupTableBase? ParentTable { get; set; } = null;

		/// <summary>
		/// Name of the table
		/// </summary>
		public string TableName { get; set; } = String.Empty;

		/// <summary>
		/// User facing name of the table
		/// </summary>
		public string UserName
		{
			get => String.IsNullOrEmpty(_userName) ? TableName : _userName;
			set => _userName = value;
		}

		/// <summary>
		/// Check to see if the table is internal
		/// </summary>
		public bool Internal { get; set; } = false;

		/// <summary>
		/// If true, this table has been implemented and not just created on demand by another table
		/// </summary>
		public bool Implemented { get; set; } = false;

		/// <summary>
		/// Internal version of the user name.  If it hasn't been set, then the table name will be used
		/// </summary>
		private string _userName = String.Empty;

		/// <summary>
		/// Merge the lookup table.  Duplicates will be ignored.
		/// </summary>
		/// <param name="baseTable">Base table being merged</param>
		public virtual void Merge(UhtLookupTableBase baseTable)
		{
		}

		/// <summary>
		/// Given a method name, try to extract the entry name for a table
		/// </summary>
		/// <param name="classType">Class containing the method</param>
		/// <param name="methodInfo">Method information</param>
		/// <param name="inName">Optional name supplied by the attributes.  If specified, this name will be returned instead of extracted from the method name</param>
		/// <param name="suffix">Required suffix</param>
		/// <returns>The extracted name or the supplied name</returns>
		public static string GetSuffixedName(Type classType, MethodInfo methodInfo, string? inName, string suffix)
		{
			string name = inName ?? String.Empty;
			if (String.IsNullOrEmpty(name))
			{
				if (methodInfo.Name.EndsWith(suffix, StringComparison.Ordinal))
				{
					name = methodInfo.Name[..^suffix.Length];
				}
				else
				{
					throw new UhtIceException($"The '{suffix}' attribute on the {classType.Name}.{methodInfo.Name} method doesn't have a name specified or the method name doesn't end in '{suffix}'.");
				}
			}
			return name;
		}
	}

	/// <summary>
	/// Lookup tables provide a method of associating actions with given C++ keyword or UE specifier
	/// </summary>
	/// <typeparam name="TValue">Keyword or specifier information</typeparam>
	public class UhtLookupTable<TValue> : UhtLookupTableBase
	{

		/// <summary>
		/// Lookup dictionary for the specifiers
		/// </summary>
		private readonly Dictionary<StringView, TValue> _lookup;

		/// <summary>
		/// Construct a new table
		/// </summary>
		public UhtLookupTable(StringViewComparer comparer)
		{
			_lookup = new Dictionary<StringView, TValue>(comparer);
		}

		/// <summary>
		/// Add the given value to the lookup table.  It will throw an exception if it is a duplicate.
		/// </summary>
		/// <param name="key">Key to be added</param>
		/// <param name="value">Value to be added</param>
		public UhtLookupTable<TValue> Add(string key, TValue value)
		{
			_lookup.Add(key, value);
			return this;
		}

		/// <summary>
		/// Attempt to fetch the value associated with the key
		/// </summary>
		/// <param name="key">Lookup key</param>
		/// <param name="value">Value associated with the key</param>
		/// <returns>True if the key was found, false if not</returns>
		public bool TryGetValue(StringView key, [MaybeNullWhen(false)] out TValue value)
		{
			return _lookup.TryGetValue(key, out value);
		}

		/// <summary>
		/// Merge the lookup table.  Duplicates will be ignored.
		/// </summary>
		/// <param name="baseTable">Base table being merged</param>
		public override void Merge(UhtLookupTableBase baseTable)
		{
			foreach (KeyValuePair<StringView, TValue> kvp in ((UhtLookupTable<TValue>)baseTable)._lookup)
			{
				_lookup.TryAdd(kvp.Key, kvp.Value);
			}
		}
	}

	/// <summary>
	/// A collection of lookup tables by name.
	/// </summary>
	/// <typeparam name="TTable">Table type</typeparam>
	public class UhtLookupTables<TTable> where TTable : UhtLookupTableBase, new()
	{

		/// <summary>
		/// Collection of named tables
		/// </summary>
		public Dictionary<string, TTable> Tables { get; } = [];

		/// <summary>
		/// The name of the group of tables
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Create a new group of tables
		/// </summary>
		/// <param name="name">The name of the group</param>
		public UhtLookupTables(string name)
		{
			Name = name;
		}

		/// <summary>
		/// Given a table name, return the table.  If not found, a new one will be added with the given name.
		/// </summary>
		/// <param name="tableName">The name of the table to return</param>
		/// <returns>The table associated with the name.</returns>
		public TTable Get(string tableName)
		{
			if (!Tables.TryGetValue(tableName, out TTable? table))
			{
				table = new TTable();
				table.TableName = tableName;
				Tables.Add(tableName, table);
			}
			return table;
		}

		/// <summary>
		/// Create a table with the given information.  If the table already exists, it will be initialized with the given data.
		/// </summary>
		/// <param name="tableName">The name of the table</param>
		/// <param name="userName">The user facing name of the name</param>
		/// <param name="parentTableName">The parent table name used to merge table entries</param>
		/// <param name="tableIsInternal">If true, the table is internal and won't be visible to the user.</param>
		/// <returns>The created table</returns>
		public TTable Create(string tableName, string userName, string? parentTableName, bool tableIsInternal = false)
		{
			TTable table = Get(tableName);
			table.UserName = userName;
			table.Internal = tableIsInternal;
			table.Implemented = true;
			if (!String.IsNullOrEmpty(parentTableName))
			{
				table.ParentTable = Get(parentTableName);
			}
			return table;
		}

		/// <summary>
		/// Merge the contents of all parent tables into their children.  This is done so that the 
		/// parent chain need not be searched when looking for table entries.
		/// </summary>
		/// <exception cref="UhtIceException">Thrown if there are problems with the tables.</exception>
		public void Merge()
		{
			List<TTable> orderedList = new(Tables.Count);
			List<TTable> remainingList = new(Tables.Count);
			HashSet<UhtLookupTableBase> doneTables = [];

			// Collect all of the tables
			foreach (KeyValuePair<string, TTable> kvp in Tables)
			{
				if (!kvp.Value.Implemented)
				{
					throw new UhtIceException($"{Name} table '{kvp.Value.TableName}' has been referenced but not implemented");
				}
				remainingList.Add(kvp.Value);
			}

			// Perform a topological sort of the tables
			while (remainingList.Count != 0)
			{
				bool addedOne = false;
				for (int i = 0; i < remainingList.Count;)
				{
					TTable table = remainingList[i];
					if (table.ParentTable == null || doneTables.Contains(table.ParentTable))
					{
						orderedList.Add(table);
						doneTables.Add(table);
						remainingList[i] = remainingList[^1];
						remainingList.RemoveAt(remainingList.Count - 1);
						addedOne = true;
					}
					else
					{
						++i;
					}
				}
				if (!addedOne)
				{
					throw new UhtIceException($"Circular dependency in {GetType().Name}.{Name} tables detected");
				}
			}

			// Merge the tables
			foreach (TTable table in orderedList)
			{
				if (table.ParentTable != null)
				{
					table.Merge((TTable)table.ParentTable);
				}
			}
		}
	}

	/// <summary>
	/// Bootstraps the standard UHT tables
	/// </summary>
	public static class UhtStandardTables
	{

		/// <summary>
		/// Enumeration that specifies if the table is a specifier and/or keyword table
		/// </summary>
		[Flags]
		public enum EUhtCreateTablesFlags
		{
			/// <summary>
			/// A keyword table will be created
			/// </summary>
			Keyword = 1 << 0,

			/// <summary>
			/// A specifier table will be created
			/// </summary>
			Specifiers = 1 << 1,
		}

		/// <summary>
		/// Create all of the standard scope tables.
		/// </summary>
		/// <param name="tables">UHT tables</param>
		public static void InitStandardTables(UhtTables tables)
		{
			CreateTables(tables, UhtTableNames.Default, "Default", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, null, true);
			CreateTables(tables, UhtTableNames.Global, "Global", EUhtCreateTablesFlags.Keyword, UhtTableNames.Default, false);
			CreateTables(tables, UhtTableNames.PropertyArgument, "Argument/Return", EUhtCreateTablesFlags.Specifiers, UhtTableNames.Default, false);
			CreateTables(tables, UhtTableNames.PropertyMember, "Member", EUhtCreateTablesFlags.Specifiers, UhtTableNames.Default, false);
			CreateTables(tables, UhtTableNames.Object, "Object", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Default, true);
			CreateTables(tables, UhtTableNames.Field, "Field", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Object, true);
			CreateTables(tables, UhtTableNames.Enum, "Enum", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Field, false);
			CreateTables(tables, UhtTableNames.Struct, "Struct", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Field, true);
			CreateTables(tables, UhtTableNames.ClassBase, "ClassBase", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Struct, true);
			CreateTables(tables, UhtTableNames.Class, "Class", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.ClassBase, false);
			CreateTables(tables, UhtTableNames.Interface, "Interface", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.ClassBase, false);
			CreateTables(tables, UhtTableNames.NativeInterface, "IInterface", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.ClassBase, false);
			CreateTables(tables, UhtTableNames.Function, "Function", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Struct, false);
			CreateTables(tables, UhtTableNames.ScriptStruct, "Struct", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Struct, false);
			CreateTables(tables, UhtTableNames.VModule, "VModule", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.Field, false);
			CreateTables(tables, UhtTableNames.Partial, "Partial", EUhtCreateTablesFlags.Specifiers | EUhtCreateTablesFlags.Keyword, UhtTableNames.ScriptStruct, false);
		}

		/// <summary>
		/// Creates a series of tables given the supplied setup
		/// </summary>
		/// <param name="tables">UHT tables</param>
		/// <param name="tableName">Name of the table.</param>
		/// <param name="tableUserName">Name presented to the user via error messages.</param>
		/// <param name="createTables">Types of tables to be created.</param>
		/// <param name="parentTableName">Name of the parent table or null for none.</param>
		/// <param name="tableIsInternal">If true, this table will not be included in any error messages.</param>
		public static void CreateTables(UhtTables tables, string tableName, string tableUserName,
			EUhtCreateTablesFlags createTables, string? parentTableName, bool tableIsInternal = false)
		{
			if (createTables.HasFlag(EUhtCreateTablesFlags.Keyword))
			{
				tables.KeywordTables.Create(tableName, tableUserName, parentTableName, tableIsInternal);
			}
			if (createTables.HasFlag(EUhtCreateTablesFlags.Specifiers))
			{
				tables.SpecifierTables.Create(tableName, tableUserName, parentTableName, tableIsInternal);
				tables.SpecifierValidatorTables.Create(tableName, tableUserName, parentTableName, tableIsInternal);
			}
		}
	}

	/// <summary>
	/// This attribute must be applied to any class that contains other UHT attributes.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class UnrealHeaderToolAttribute : Attribute
	{

		/// <summary>
		/// If specified, this method will be invoked once during the scan for attributes.
		/// It can be used to perform some one time initialization.
		/// </summary>
		public string InitMethod { get; set; } = String.Empty;
	}

	/// <summary>
	/// UnrealHeaderTool avoids hard coding any table contents by using attributes to add entries to the tables.
	/// 
	/// There are two basic styles of tables in UHT.  
	/// 
	/// The first style is just a simple association of a string and an attribute.  For example, 
	/// the engine class table is just a collection of all the engine class names supported by the engine.
	/// 
	/// The second style is a table of tables.  Depending on the context (i.e. is a "class" or "function" 
	/// being parsed), attributes will "extend" a given table adding an entry to that table and every
	/// table the derives from that table.  For example, the Keywords table will add "private" to the 
	/// "ClassBase" table.  Since the "Class", "Interface", and "NativeInterface" tables derive from
	/// "ClassBase", all of those tables will contain the keyword "private".
	/// 
	/// See UhtTables.cs for a list of table names and how they relate to each other.
	/// 
	/// Tables are implemented in the following source files:
	/// 
	///		UhtEngineClassTable.cs - Collection of all the engine class names.
	///		UhtKeywordTable.cs - Collection of the C++ keywords that UHT understands.
	///		UhtLocTextDefaultValueTable.cs - Collection of loctext default value parsing
	///		UhtPropertyTypeTable.cs - Collection of the property type keywords.
	///		UhtSpecifierTable.cs - Collection of the known specifiers
	///		UhtSpecifierValidatorTable.cs - Collection of the specifier validators
	///		UhtStructDefaultValueTable.cs - Collection of structure default value parsing
	/// </summary>
	public class UhtTables
	{
		/// <summary>
		/// Collection of specifier tables
		/// </summary>
		public UhtSpecifierTables SpecifierTables { get; } = new UhtSpecifierTables();

		/// <summary>
		/// Collection of specifier validator tables
		/// </summary>
		public UhtSpecifierValidatorTables SpecifierValidatorTables { get; } = new UhtSpecifierValidatorTables();

		/// <summary>
		/// Collection of keyword tables
		/// </summary>
		public UhtKeywordTables KeywordTables { get; } = new UhtKeywordTables();

		/// <summary>
		/// Collection of property types
		/// </summary>
		public UhtPropertyTypeTable PropertyTypeTable { get; } = new UhtPropertyTypeTable();

		/// <summary>
		/// Collection of structure default values
		/// </summary>
		public UhtStructDefaultValueTable StructDefaultValueTable { get; } = new UhtStructDefaultValueTable();

		/// <summary>
		/// Collection of engine class types
		/// </summary>
		public UhtEngineClassTable EngineClassTable { get; } = new UhtEngineClassTable();

		/// <summary>
		/// Collection of exporters
		/// </summary>
		public UhtExporterTable ExporterTable { get; } = new UhtExporterTable();

		/// <summary>
		/// Collection loc text default values
		/// </summary>
		public UhtLocTextDefaultValueTable LocTextDefaultValueTable { get; } = new UhtLocTextDefaultValueTable();

		/// <summary>
		/// Collection of methods used to inject code into the code generator
		/// </summary>
		public UhtCodeGeneratorInjectorTable CodeGeneratorInjectorTable { get; } = new UhtCodeGeneratorInjectorTable();

		/// <summary>
		/// Collection of all known UhtTypes
		/// </summary>
		public IReadOnlyList<Type> KnownTypes => _knownTypes;
		private readonly List<Type> _knownTypes = [];

		/// <summary>
		/// Construct a new table collection
		/// </summary>
		public UhtTables()
		{
			UhtStandardTables.InitStandardTables(this);
			CheckForAttributes(Assembly.GetExecutingAssembly());
			PerformPostInitialization();
		}

		/// <summary>
		/// Add a collection of plugin assembly file paths
		/// </summary>
		/// <param name="pluginAssembliesFilePaths">Collection of plugins to load</param>
		public void AddPlugins(IEnumerable<string> pluginAssembliesFilePaths)
		{
			foreach (string assemblyFilePath in pluginAssembliesFilePaths)
			{
				CheckForAttributes(LoadAssembly(assemblyFilePath));
			}
			PerformPostInitialization();
		}

		/// <summary>
		/// Check to see if the assembly is a UHT plugin
		/// </summary>
		/// <param name="assemblyFilePath">Path to the assembly file</param>
		/// <returns></returns>
		public static bool IsUhtPlugin(string assemblyFilePath)
		{
			Assembly? assembly = LoadAssembly(assemblyFilePath);
			if (assembly != null)
			{
				foreach (Type type in assembly.SafeGetLoadedTypes())
				{
					if (type.IsClass)
					{
						foreach (Attribute classAttribute in type.GetCustomAttributes(false).Cast<Attribute>())
						{
							if (classAttribute is UnrealHeaderToolAttribute || classAttribute is UhtEngineClassAttribute)
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Load the given assembly from the file path
		/// </summary>
		/// <param name="assemblyFilePath">Path to the file</param>
		/// <returns>Assembly if it is already loaded or could be loaded</returns>
		public static Assembly? LoadAssembly(string assemblyFilePath)
		{
			Assembly? assembly = FindAssemblyByName(Path.GetFileNameWithoutExtension(assemblyFilePath));
			if (assembly == null)
			{
				assembly = Assembly.LoadFile(assemblyFilePath);
			}
			return assembly;
		}

		/// <summary>
		/// Locate the assembly by name
		/// </summary>
		/// <param name="name">Name of the assembly</param>
		/// <returns>Assembly or null</returns>
		private static Assembly? FindAssemblyByName(string? name)
		{
			if (name != null)
			{
				foreach (Assembly assembly in AppDomain.CurrentDomain.GetAssemblies())
				{
					AssemblyName assemblyName = assembly.GetName();
					if (assemblyName.Name != null && String.Equals(assemblyName.Name, name, StringComparison.OrdinalIgnoreCase))
					{
						return assembly;
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Check for UHT attributes on all types in the given assembly
		/// </summary>
		/// <param name="assembly">Assembly to scan</param>
		private void CheckForAttributes(Assembly? assembly)
		{
			if (assembly != null)
			{
				foreach (Type type in assembly.SafeGetLoadedTypes())
				{
					CheckForAttributes(type);
					if (type == typeof(UhtType) || type.IsSubclassOf(typeof(UhtType)))
					{
						_knownTypes.Add(type);
					}
				}
			}
		}

		/// <summary>
		/// Verify that all the known types support caching
		/// </summary>
		/// <param name="logger"></param>
		/// <exception cref="UhtException"></exception>
		public void ValidateCacheableTypes(ILogger logger)
		{
			bool hasBadType = false;
			foreach (Type type in _knownTypes)
			{
				if (type != typeof(UhtPackage) && !type.IsAbstract)
				{
					ConstructorInfo? constructor = type.GetConstructor([typeof(UhtInputCacheReader), typeof(UhtType)]);
					if (constructor == null)
					{
						logger.LogError("The UhtType '{TypeName}' must defined the constructor '{ConstructorName}(UhtInputCacheReader reader, UhtType outer)'", type.Name, type.Name);
						hasBadType = true;
					}
					MethodInfo? method = type.GetMethod("Write", [typeof(UhtInputCacheWriter)]);
					if (method == null || method.DeclaringType != type)
					{
						logger.LogError("The UhtType '{TypeName}' must defined the method 'public override void Write(UhtInputCacheWriter writer)'", type.Name);
						hasBadType = true;
					}
				}
			}
			if (hasBadType)
			{
				throw new UhtIceException("Not all types implement the expect input cache methods");
			}
		}

		/// <summary>
		/// For the given type, look for any table related attributes 
		/// </summary>
		/// <param name="type">Type in question</param>
		private void CheckForAttributes(Type type)
		{
			if (type.IsClass)
			{
				// Loop through the attributes
				foreach (Attribute classAttribute in type.GetCustomAttributes(false).Cast<Attribute>())
				{
					if (classAttribute is UnrealHeaderToolAttribute parserAttribute)
					{
						HandleUnrealHeaderToolAttribute(type, parserAttribute);
					}
					else if (classAttribute is UhtEngineClassAttribute engineClassAttribute)
					{
						EngineClassTable.OnEngineClassAttribute(engineClassAttribute);
					}
				}
			}
		}

		private void PerformPostInitialization()
		{
			KeywordTables.Merge();
			SpecifierTables.Merge();
			SpecifierValidatorTables.Merge();
			CodeGeneratorInjectorTable.Sort();

			// Invoke this to generate an exception if there is no default
			_ = PropertyTypeTable.Default;
			_ = StructDefaultValueTable.Default;
		}

		private void HandleUnrealHeaderToolAttribute(Type type, UnrealHeaderToolAttribute parserAttribute)
		{
			if (!String.IsNullOrEmpty(parserAttribute.InitMethod))
			{
				MethodInfo? initInfo = type.GetMethod(parserAttribute.InitMethod, BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic);
				if (initInfo == null)
				{
					throw new Exception($"Unable to find UnrealHeaderTool attribute InitMethod {parserAttribute.InitMethod}");
				}
				initInfo.Invoke(null, []);
			}

			// Scan the methods for things we are interested in
			foreach (MethodInfo methodInfo in type.GetMethods(BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic))
			{

				// Scan for attributes we care about
				foreach (Attribute methodAttribute in methodInfo.GetCustomAttributes())
				{
					if (methodAttribute is UhtSpecifierAttribute specifierAttribute)
					{
						SpecifierTables.OnSpecifierAttribute(type, methodInfo, specifierAttribute);
					}
					else if (methodAttribute is UhtSpecifierValidatorAttribute specifierValidatorAttribute)
					{
						SpecifierValidatorTables.OnSpecifierValidatorAttribute(type, methodInfo, specifierValidatorAttribute);
					}
					else if (methodAttribute is UhtKeywordCatchAllAttribute keywordCatchAllAttribute)
					{
						KeywordTables.OnKeywordCatchAllAttribute(type, methodInfo, keywordCatchAllAttribute);
					}
					else if (methodAttribute is UhtKeywordAttribute keywordAttribute)
					{
						KeywordTables.OnKeywordAttribute(type, methodInfo, keywordAttribute);
					}
					else if (methodAttribute is UhtPropertyTypeAttribute propertyTypeAttribute)
					{
						PropertyTypeTable.OnPropertyTypeAttribute(methodInfo, propertyTypeAttribute);
					}
					else if (methodAttribute is UhtStructDefaultValueAttribute structDefaultValueAttribute)
					{
						StructDefaultValueTable.OnStructDefaultValueAttribute(methodInfo, structDefaultValueAttribute);
					}
					else if (methodAttribute is UhtExporterAttribute exporterAttribute)
					{
						ExporterTable.OnExporterAttribute(type, methodInfo, exporterAttribute);
					}
					else if (methodAttribute is UhtLocTextDefaultValueAttribute locTextDefaultValueAttribute)
					{
						LocTextDefaultValueTable.OnLocTextDefaultValueAttribute(methodInfo, locTextDefaultValueAttribute);
					}
					else if (methodAttribute is UhtCodeGeneratorInjectorAttribute codeGeneratorInjectorAttribute)
					{
						CodeGeneratorInjectorTable.OnUhtCodeGeneratorInjectorAttribute(methodInfo, codeGeneratorInjectorAttribute);
					}
				}
			}
		}
	}
}
