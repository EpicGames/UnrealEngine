// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Text;
using EpicGames.UHT.Tokenizer;
using EpicGames.UHT.Types;

namespace EpicGames.UHT.Utils
{
	/// <summary>
	/// Represents the decoded tokens for part of a type definition
	/// </summary>
	public struct UhtTypeSegment
	{

		/// <summary>
		/// Starting token index of this type
		/// </summary>
		public int Start { get; set; } = -1;

		/// <summary>
		/// Ending token index of this type (non-inclusive)
		/// </summary>
		public int End { get; set; } = -1;

		/// <summary>
		/// Starting token index of the type
		/// </summary>
		public int TypeStart { get; set; } = -1;

		/// <summary>
		/// Ending token index of the type
		/// </summary>
		public int TypeEnd { get; set; } = -1;

		/// <summary>
		/// Set if the type beings with 'class', 'struct', or 'enum'
		/// </summary>
		public UhtFindOptions FindFilter { get; set; } = UhtFindOptions.None;

		/// <summary>
		/// Construct a new type segment
		/// </summary>
		public UhtTypeSegment()
		{
		}

		/// <summary>
		/// Construct a type from the cache
		/// </summary>
		/// <param name="reader">Reader</param>
		public UhtTypeSegment(UhtInputCacheReader reader)
		{
			Start = reader.ReadInt32();
			End = reader.ReadInt32();
			TypeStart = reader.ReadInt32();
			TypeEnd = reader.ReadInt32();
			FindFilter = (UhtFindOptions)reader.ReadUInt64();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteInt32(Start);
			writer.WriteInt32(End);
			writer.WriteInt32(TypeStart);
			writer.WriteInt32(TypeEnd);
			writer.WriteUInt64((ulong)FindFilter);
		}
	}

	/// <summary>
	/// Information about a parsed type
	/// </summary>
	public struct UhtTypeTokens
	{
		private readonly Memory<UhtTypeSegment> _segments;

		/// <summary>
		/// Collection of tokens that describe the type
		/// </summary>
		public ReadOnlyMemory<UhtToken> AllTokens { get; internal set; }

		/// <summary>
		/// Collection of type segments
		/// </summary>
		public readonly ReadOnlyMemory<UhtTypeSegment> Segments => _segments;

		private UhtTypeTokens(ReadOnlyMemory<UhtToken> typeTokens, Memory<UhtTypeSegment> segments)
		{
			AllTokens = typeTokens;
			_segments = segments;
		}

		/// <summary>
		/// Gather a type definition from the given token reader
		/// </summary>
		/// <param name="tokenReader">Token reader containing the type</param>
		/// <returns></returns>
		public static UhtTypeTokens Gather(IUhtTokenReader tokenReader)
		{
			List<UhtToken> typeTokens = [];
			List<UhtTypeSegment> segments = [];

			Gather(typeTokens, segments, tokenReader, 0);
			return new UhtTypeTokens(typeTokens.ToArray().AsMemory(), segments.ToArray().AsMemory());
		}

		/// <summary>
		/// Extract the trailing token from the type string since it is the property/function name
		/// </summary>
		/// <param name="messageSite">Message site for errors</param>
		/// <param name="propertySettings">Property setting for what is being parsed</param>
		/// <returns>Trailing identifier</returns>
		/// <exception cref="UhtException"></exception>
		public UhtToken ExtractTrailingIdentifier(IUhtMessageSite messageSite, UhtPropertySettings propertySettings)
		{
			// Extract the property name from the types
			if (AllTokens.Length < 2 || !AllTokens.Span[^1].IsIdentifier())
			{
				throw new UhtException(messageSite, $"{propertySettings.PropertyCategory.GetHintText()}: Expected name");
			}

			Debug.Assert(_segments.Span[0].End == AllTokens.Length);
			UhtToken nameToken = AllTokens.Span[^1];
			AllTokens = AllTokens[..^1];
			ref UhtTypeSegment segment = ref _segments.Span[0];
			segment.End = AllTokens.Length;
			segment.TypeEnd = Math.Min(segment.TypeEnd, segment.End);
			return nameToken;
		}

		/// <inheritdoc/>
		public override readonly string ToString()
		{
			return ToString(AllTokens.Span);
		}

		/// <summary>
		/// Convert the token span to a string
		/// </summary>
		/// <param name="tokens">Tokens to convert</param>
		/// <returns>Resulting string</returns>
		public static string ToString(ReadOnlySpan<UhtToken> tokens)
		{
			StringBuilder builder = new();
			builder.AppendTokens(tokens);
			return builder.ToString();
		}

		private static int Gather(List<UhtToken> typeTokens, List<UhtTypeSegment> segments, IUhtTokenReader tokenReader, int depth)
		{
			int argumentCount = 1;
			int segmentIndex = segments.Count;
			UhtTypeSegment segment = new();
			segment.Start = typeTokens.Count;
			segments.Add(segment);

			while (true)
			{
				if (tokenReader.IsEOF)
				{
					throw new UhtTokenException(tokenReader, tokenReader.GetToken(), null, null);
				}
				ref UhtToken token = ref tokenReader.PeekToken();

				char symbolChar = token.IsSymbol() && token.Value.Length == 1 ? token.Value.Span[0] : '\0';
				switch (symbolChar)
				{
					case '(':
					case ')':
					case ';':
					case '[':
					case ':':
					case '=':
					case '{':
					case ',':
						if (depth == 0)
						{
							// Don't consume the token
							segment.End = typeTokens.Count;
							segments[segmentIndex] = segment;
							return argumentCount;
						}
						else if (symbolChar == ',')
						{
							segment.End = typeTokens.Count;
							segments[segmentIndex] = segment;

							tokenReader.ConsumeToken();
							typeTokens.Add(token);

							segment = new();
							segment.Start = typeTokens.Count;
							segmentIndex = segments.Count;
							segments.Add(segment);
							argumentCount++;
						}
						else
						{
							tokenReader.ConsumeToken();
							typeTokens.Add(token);
						}
						break;

					case '>':
						if (depth == 0)
						{
							throw new UhtTokenException(tokenReader, token, "',' or ')'");
						}
						else
						{
							segment.End = typeTokens.Count;
							segments[segmentIndex] = segment;
							tokenReader.ConsumeToken(); // go on ahead and add the closing '>' to the token stream but not included in current segment
							typeTokens.Add(token);
							return argumentCount;
						}

					case '<':
						tokenReader.ConsumeToken();
						typeTokens.Add(token);
						Gather(typeTokens, segments, tokenReader, depth + 1);
						break;

					default:
						tokenReader.ConsumeToken();
						typeTokens.Add(token);
						if (token.IsIdentifier("UPARAM"))
						{
							tokenReader.SkipBrackets('(', ')', 0, x => typeTokens.Add(x));
						}
						else if (token.IsIdentifier("const"))
						{
						}
						else if (token.IsSymbol("::") || token.IsIdentifier())
						{
							if (segment.TypeEnd == -1)
							{
								if (token.IsIdentifier("class"))
								{
									segment.FindFilter = UhtFindOptions.Class;
								}
								else if (token.IsIdentifier("struct"))
								{
									segment.FindFilter = UhtFindOptions.ScriptStruct;
								}
								else if (token.IsIdentifier("enum"))
								{
									segment.FindFilter = UhtFindOptions.Enum;
								}
								segment.TypeStart = typeTokens.Count - 1;
								segment.TypeEnd = typeTokens.Count;
							}
							else if (segment.TypeEnd == typeTokens.Count - 1)
							{
								segment.TypeEnd = typeTokens.Count;
							}
						}
						break;
				}
			}
		}
	}

	/// <summary>
	/// Reference to a token segment
	/// </summary>
	public readonly struct UhtTypeTokensRef
	{
		/// <summary>
		/// Collection of tokens that describe the type
		/// </summary>
		public readonly ReadOnlyMemory<UhtToken> AllTokens { get; }

		/// <summary>
		/// Collection of type segments
		/// </summary>
		public readonly ReadOnlyMemory<UhtTypeSegment> Segments { get; }

		/// <summary>
		/// Index of the segment that contains only the declaration used to generate the property.
		/// This will not include such things as TPtr which themselves do not generate a property.
		/// </summary>
		public readonly int DeclarationSegmentIndex { get; }

		/// <summary>
		/// Index of the full segment that includes not only the declaration used to generate the 
		/// property, but any templates such as TPtr that wrap the property declaration.
		/// </summary>
		public readonly int FullDeclarationSegmentIndex { get; }

		/// <summary>
		/// The segment that contains only the declaration used to generate the property.
		/// This will not include such things as TPtr which themselves do not generate a property.
		/// </summary>
		public readonly UhtTypeSegment DeclarationSegment => Segments.Span[DeclarationSegmentIndex];

		/// <summary>
		/// the full segment that includes not only the declaration used to generate the 
		/// property, but any templates such as TPtr that wrap the property declaration.
		/// </summary>
		public readonly UhtTypeSegment FullDeclarationSegment => Segments.Span[FullDeclarationSegmentIndex];

		/// <summary>
		/// The tokens that represent the property's basic type.
		/// Example: "const USomeType*" has type tokens of "USomeType" 
		/// Example: "const verse::my_verse_type*" has type tokens of "verse::my_verse_type" 
		/// </summary>
		public readonly ReadOnlyMemory<UhtToken> TypeTokens
		{
			get
			{
				int typeStart = DeclarationSegment.TypeStart;
				int typeEnd = DeclarationSegment.TypeEnd;
				return typeStart >= 0 ? AllTokens[typeStart..typeEnd] : new();
			}
		}

		/// <summary>
		/// The collection of tokens that define this type
		/// </summary>
		public readonly ReadOnlyMemory<UhtToken> DeclarationTokens
		{
			get
			{
				int start = DeclarationSegment.Start;
				int end = DeclarationSegment.End;
				return AllTokens[start..end];
			}
		}

		/// <summary>
		/// The collection of tokens that define this type
		/// </summary>
		public readonly ReadOnlyMemory<UhtToken> FullDeclarationTokens
		{
			get
			{
				int start = FullDeclarationSegment.Start;
				int end = FullDeclarationSegment.End;
				return AllTokens[start..end];
			}
		}

		/// <summary>
		/// Construct a new instance of a type tokens
		/// </summary>
		/// <param name="typeTokens">Complete type tokens</param>
		/// <param name="declarationSegmentIndex">Segment index</param>
		public UhtTypeTokensRef(UhtTypeTokens typeTokens, int declarationSegmentIndex)
		{
			AllTokens = typeTokens.AllTokens;
			Segments = typeTokens.Segments;
			DeclarationSegmentIndex = declarationSegmentIndex;
			FullDeclarationSegmentIndex = declarationSegmentIndex;
		}

		/// <summary>
		/// Construct a new instance of a type tokens
		/// </summary>
		/// <param name="typeTokens">Complete type tokens</param>
		/// <param name="declarationSegmentIndex">Segment index</param>
		public UhtTypeTokensRef(UhtTypeTokensRef typeTokens, int declarationSegmentIndex)
		{
			AllTokens = typeTokens.AllTokens;
			Segments = typeTokens.Segments;
			DeclarationSegmentIndex = declarationSegmentIndex;
			FullDeclarationSegmentIndex = declarationSegmentIndex;
		}

		/// <summary>
		/// Construct a new instance of a type tokens
		/// </summary>
		/// <param name="typeTokens">Complete type tokens</param>
		/// <param name="declarationSegmentIndex">Segment index</param>
		/// <param name="fullDeclarationSegmentIndex">Segment index for wrapped type</param>
		public UhtTypeTokensRef(UhtTypeTokensRef typeTokens, int declarationSegmentIndex, int fullDeclarationSegmentIndex)
		{
			AllTokens = typeTokens.AllTokens;
			Segments = typeTokens.Segments;
			DeclarationSegmentIndex = declarationSegmentIndex;
			FullDeclarationSegmentIndex = fullDeclarationSegmentIndex;
		}

		/// <summary>
		/// Construct a new instance of a type tokens
		/// </summary>
		/// <param name="allTokens">Collection of tokens</param>
		/// <param name="segments">Collection of segments</param>
		/// <param name="reader">Binary archive</param>
		public UhtTypeTokensRef(ReadOnlyMemory<UhtToken> allTokens, ReadOnlyMemory<UhtTypeSegment> segments, UhtInputCacheReader reader)
		{
			AllTokens = allTokens;
			Segments = segments;
			DeclarationSegmentIndex = reader.ReadInt32();
			FullDeclarationSegmentIndex = reader.ReadInt32();
		}

		/// <summary>
		/// Write the output type
		/// </summary>
		/// <param name="writer"></param>
		public void Write(UhtInputCacheWriter writer)
		{
			writer.WriteInt32(DeclarationSegmentIndex);
			writer.WriteInt32(FullDeclarationSegmentIndex);
		}

		/// <inheritdoc/>
		public override readonly string ToString()
		{
			return UhtTypeTokens.ToString(DeclarationTokens.Span);
		}
	}

	/// <summary>
	/// Helper methods to append type tokens to a string builder
	/// </summary>
	public static class UhtTypeTokensStringBuilderExtensions
	{

		/// <summary>
		/// Append the type tokens to the given builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="tokens">Tokens to append</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendTokens(this StringBuilder builder, ReadOnlySpan<UhtToken> tokens)
		{
			if (tokens.IsEmpty)
			{
				builder.Append("(empty type)");
			}
			else
			{
				bool priorIsIdentifier = false;
				foreach (UhtToken token in tokens)
				{
					bool isIdentifier = token.IsIdentifier();
					if (priorIsIdentifier && token.IsIdentifier())
					{
						builder.Append(' ');
					}
					builder.Append(token.ToString());
					if (token.IsSymbol(','))
					{
						builder.Append(' ');
					}
					priorIsIdentifier = isIdentifier;
				}
			}
			return builder;
		}

		/// <summary>
		/// Append the type tokens to the given builder
		/// </summary>
		/// <param name="builder">Destination builder</param>
		/// <param name="tokens">Tokens to append</param>
		/// <returns>Destination builder</returns>
		public static StringBuilder AppendTokens(this StringBuilder builder, ReadOnlyMemory<UhtToken> tokens)
		{
			return builder.AppendTokens(tokens.Span);
		}
	}
}
