// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using EpicGames.Core;

namespace EpicGames.UHT.Utils
{

	/// <summary>
	/// Extensions to List that provides some uniqueness support to list elements 
	/// </summary>
	public static class UhtListExtensions
	{

		/// <summary>
		/// Add the given value if it isn't already contained within the list
		/// </summary>
		/// <param name="container">Destination container</param>
		/// <param name="value">Value to be added</param>
		/// <returns>True if the value was added, false if it was already present</returns>
		public static bool AddUnique(this List<string> container, string value)
		{
			if (container.Contains(value, StringComparer.OrdinalIgnoreCase))
			{
				return false;
			}
			container.Add(value);
			return true;
		}

		/// <summary>
		/// Add the given values if they aren't already contained within the list
		/// </summary>
		/// <param name="container">Destination container</param>
		/// <param name="values">Values to be added</param>
		public static void AddUniqueRange(this List<string> container, IEnumerable<StringView>? values)
		{
			if (values != null)
			{
				foreach (StringView value in values)
				{
					AddUnique(container, value.ToString());
				}
			}
		}

		/// <summary>
		/// Remove the given value but swap the last entry into the eliminated slot
		/// </summary>
		/// <param name="container">Container being modified</param>
		/// <param name="value">Value to be removed</param>
		/// <returns>True if the value was removed, false if not</returns>
		public static bool RemoveSwap(this List<string> container, string value)
		{
			int index = container.FindIndex(n => value.Equals(n, StringComparison.OrdinalIgnoreCase));
			if (index >= 0)
			{
				if (index != container.Count - 1)
				{
					container[index] = container[^1];
				}
				container.RemoveAt(container.Count - 1);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Remove a range of values from a container using swapping
		/// </summary>
		/// <param name="container">Container to be modified</param>
		/// <param name="values">List of values to be removed</param>
		public static void RemoveSwapRange(this List<string> container, IEnumerable<StringView>? values)
		{
			if (values != null)
			{
				foreach (StringView value in values)
				{
					RemoveSwap(container, value.ToString());
				}
			}
		}
	}

	/// <summary>
	/// UnrealEngine names often differ from the names in the source file.  The following 
	/// structure represents the different parts of the name
	/// </summary>
	public struct UhtEngineNameParts
	{

		/// <summary>
		/// Any prefix removed from the source name to create the engine name
		/// </summary>
		public StringView Prefix { get; set; }

		/// <summary>
		/// The name to be used by the Unreal Engine.
		/// </summary>
		public StringView EngineName { get; set; }

		/// <summary>
		/// The name contained the "DEPRECATED" text which has been removed from the engine name
		/// </summary>
		public bool IsDeprecated { get; set; }
	}

	/// <summary>
	/// Assorted utility functions
	/// </summary>
	public static class UhtUtilities
	{

		/// <summary>
		/// Return the latest of two date/times
		/// </summary>
		/// <param name="lhs">First date/time to compare</param>
		/// <param name="rhs">Second date/time to compare</param>
		/// <returns>Latest of the two dates</returns>
		private static DateTime LatestDateTime(DateTime lhs, DateTime rhs)
		{
			return lhs > rhs ? lhs : rhs;
		}

		/// <summary>
		/// Return the latest date time between UBT and UHT assembly and all plugin assemblies
		/// </summary>
		/// <param name="plugins"></param>
		/// <returns></returns>
		public static DateTime GetLatestAssemblyDateTime(IEnumerable<string>? plugins)
		{
			// Combined time of UBT and UHT
			DateTime compositeTimestamp = LatestDateTime(
				new FileInfo(Assembly.GetExecutingAssembly().Location).LastWriteTimeUtc,
				new FileInfo(typeof(UhtSession).Assembly.Location).LastWriteTimeUtc);

			if (plugins != null)
			{
				foreach (string plugin in plugins)
				{
					compositeTimestamp = LatestDateTime(compositeTimestamp, new FileInfo(plugin).LastWriteTimeUtc);
				}
			}
			return compositeTimestamp;
		}

		/// <summary>
		/// Given a collection of names, return a string containing the text of those names concatenated.
		/// </summary>
		/// <param name="typeNames">Collect of names to be merged</param>
		/// <param name="andOr">Text used to separate the names</param>
		/// <param name="quote">If true, add quotes around the names</param>
		/// <returns>Merged names</returns>
		public static string MergeTypeNames(IEnumerable<string> typeNames, string andOr, bool quote = false)
		{
			List<string> local = [.. typeNames];

			if (local.Count == 0)
			{
				return "";
			}

			local.Sort();

			StringBuilder builder = new();
			for (int index = 0; index < local.Count; ++index)
			{
				if (index != 0)
				{
					builder.Append(", ");
					if (index == local.Count - 1)
					{
						builder.Append(andOr);
						builder.Append(' ');
					}
				}
				if (quote)
				{
					builder.Append('\'');
					builder.Append(local[index]);
					builder.Append('\'');
				}
				else
				{
					builder.Append(local[index]);
				}
			}
			return builder.ToString();
		}

		/// <summary>
		/// Split the given source name into the engine name parts
		/// </summary>
		/// <param name="sourceName">Source name</param>
		/// <returns>Resulting engine name parts</returns>
		public static UhtEngineNameParts GetEngineNameParts(StringView sourceName)
		{
			if (sourceName.Span.Length == 0)
			{
				return new UhtEngineNameParts { Prefix = new StringView(String.Empty), EngineName = new StringView(String.Empty), IsDeprecated = false };
			}

			switch (sourceName.Span[0])
			{
				case 'I':
				case 'A':
				case 'U':
					// If it is a class prefix, check for deprecated class prefix also
					if (sourceName.Span.Length > 12 && sourceName.Span[1..].StartsWith("DEPRECATED_", StringComparison.Ordinal))
					{
						return new UhtEngineNameParts { Prefix = new StringView(sourceName, 0, 12), EngineName = new StringView(sourceName, 12), IsDeprecated = true };
					}
					else
					{
						return new UhtEngineNameParts { Prefix = new StringView(sourceName, 0, 1), EngineName = new StringView(sourceName, 1), IsDeprecated = false };
					}

				case 'F':
				case 'T':
					// Struct prefixes are also fine.
					return new UhtEngineNameParts { Prefix = new StringView(sourceName, 0, 1), EngineName = new StringView(sourceName, 1), IsDeprecated = false };

				default:
					// If it's not a class or struct prefix, it's invalid
					return new UhtEngineNameParts { Prefix = new StringView(String.Empty), EngineName = new StringView(sourceName), IsDeprecated = false };
			}
		}

		private static readonly SearchValues<char> s_pathNameDelimiters = SearchValues.Create(['.', ':']);

		/// <summary>
		/// Returns whether the given string is a valid long path name for an object.
		/// See FSoftObjectPath::SetPath for reference.
		/// e.g. MyClass is a short name, but /Script/MyModule.MyClass is a full path name.
		/// </summary>
		/// <param name="name"></param>
		/// <returns></returns>
		public static bool IsValidLongPathName(StringView name)
		{
			if (name.Span.Length < 1 || name.Span[0] != '/')
			{
				return false;
			}
			if (!name.Span.ContainsAny(s_pathNameDelimiters))
			{
				return false;
			}

			// Reject paths that contain two consecutive delimiters in any position 
			for (ReadOnlySpan<char> spanIt = name.Span; !spanIt.IsEmpty;)
			{
				int index = spanIt.IndexOfAny(s_pathNameDelimiters);
				if (index < 0)
				{
					break;
				}
				// Note that we don't reject a trailing delimiter, like FSoftObjectPath
				if (index + 1 < spanIt.Length && s_pathNameDelimiters.Contains(spanIt[index + 1]))
				{
					return false;
				}
				spanIt = spanIt[(index + 1)..];
			}
			// Code in FSoftObjectPath continues to parse after this point but will no longer reject any strings
			return true;
		}

		/// <summary>
		/// Returns whether the given string is a valid long path name for an object.
		/// See FSoftObjectPath::SetPath for reference.
		/// e.g. MyClass is a short name, but /Script/MyModule.MyClass is a full path name.
		/// </summary>
		/// <param name="name"></param>
		/// <returns></returns>
		public static bool IsValidLongPathName(string name)
		{
			return IsValidLongPathName(new StringView(name));
		}
	}

	/// <summary>
	/// String builder class that has support for StringView so that if a single instance of
	/// a StringView is appended, it is returned.
	/// </summary>
	public class StringViewBuilder
	{

		/// <summary>
		/// When only a string view has been appended, this references that StringView
		/// </summary>
		private StringView _stringView = new();

		/// <summary>
		/// Represents more complex data being appended
		/// </summary>
		private StringBuilder? _stringBuilder = null;

		/// <summary>
		/// Set to true when the builder has switched to a StringBuilder (NOTE: This can probably be removed)
		/// </summary>
		private bool _useStringBuilder = false;

		/// <summary>
		/// The length of the appended data
		/// </summary>
		public int Length
		{
			get
			{
				if (_useStringBuilder && _stringBuilder != null)
				{
					return _stringBuilder.Length;
				}
				else
				{
					return _stringView.Span.Length;
				}
			}
		}

		/// <summary>
		/// Return the appended data as a StringView
		/// </summary>
		/// <returns>Contents of the builder</returns>
		public StringView ToStringView()
		{
			return _useStringBuilder ? new StringView(_stringBuilder!.ToString()) : _stringView;
		}

		/// <summary>
		/// Return the appended data as a string
		/// </summary>
		/// <returns>Contents of the builder</returns>
		public override string ToString()
		{
			return _useStringBuilder ? _stringBuilder!.ToString() : _stringView.ToString();
		}

		/// <summary>
		/// Append a StringView
		/// </summary>
		/// <param name="text">Text to be appended</param>
		/// <returns>The string builder</returns>
		public StringViewBuilder Append(StringView text)
		{
			if (_useStringBuilder)
			{
				_stringBuilder!.Append(text.Span);
			}
			else if (_stringView.Span.Length > 0)
			{
				SwitchToStringBuilder();
				_stringBuilder!.Append(text.Span);
			}
			else
			{
				_stringView = text;
			}
			return this;
		}

		/// <summary>
		/// Append a character
		/// </summary>
		/// <param name="c">Character to be appended</param>
		/// <returns>The string builder</returns>
		public StringViewBuilder Append(char c)
		{
			SwitchToStringBuilder();
			_stringBuilder!.Append(c);
			return this;
		}

		/// <summary>
		/// If not already done, switch the builder to using a StringBuilder
		/// </summary>
		private void SwitchToStringBuilder()
		{
			if (!_useStringBuilder)
			{
				_stringBuilder ??= new StringBuilder();
				_useStringBuilder = true;
				_stringBuilder.Append(_stringView.Span);
			}
		}
	}

	/// <summary>
	/// Helper structure used to compute UE style CRC values
	/// </summary>
	internal readonly struct Crc
	{
		private static readonly uint[] s_table = [
			0x00000000, 0x77073096, 0xee0e612c, 0x990951ba, 0x076dc419, 0x706af48f, 0xe963a535, 0x9e6495a3, 0x0edb8832, 0x79dcb8a4, 0xe0d5e91e, 0x97d2d988, 0x09b64c2b, 0x7eb17cbd, 0xe7b82d07, 0x90bf1d91,
			0x1db71064, 0x6ab020f2, 0xf3b97148, 0x84be41de, 0x1adad47d, 0x6ddde4eb, 0xf4d4b551, 0x83d385c7, 0x136c9856, 0x646ba8c0, 0xfd62f97a, 0x8a65c9ec, 0x14015c4f, 0x63066cd9, 0xfa0f3d63, 0x8d080df5,
			0x3b6e20c8, 0x4c69105e, 0xd56041e4, 0xa2677172, 0x3c03e4d1, 0x4b04d447, 0xd20d85fd, 0xa50ab56b, 0x35b5a8fa, 0x42b2986c, 0xdbbbc9d6, 0xacbcf940, 0x32d86ce3, 0x45df5c75, 0xdcd60dcf, 0xabd13d59,
			0x26d930ac, 0x51de003a, 0xc8d75180, 0xbfd06116, 0x21b4f4b5, 0x56b3c423, 0xcfba9599, 0xb8bda50f, 0x2802b89e, 0x5f058808, 0xc60cd9b2, 0xb10be924, 0x2f6f7c87, 0x58684c11, 0xc1611dab, 0xb6662d3d,
			0x76dc4190, 0x01db7106, 0x98d220bc, 0xefd5102a, 0x71b18589, 0x06b6b51f, 0x9fbfe4a5, 0xe8b8d433, 0x7807c9a2, 0x0f00f934, 0x9609a88e, 0xe10e9818, 0x7f6a0dbb, 0x086d3d2d, 0x91646c97, 0xe6635c01,
			0x6b6b51f4, 0x1c6c6162, 0x856530d8, 0xf262004e, 0x6c0695ed, 0x1b01a57b, 0x8208f4c1, 0xf50fc457, 0x65b0d9c6, 0x12b7e950, 0x8bbeb8ea, 0xfcb9887c, 0x62dd1ddf, 0x15da2d49, 0x8cd37cf3, 0xfbd44c65,
			0x4db26158, 0x3ab551ce, 0xa3bc0074, 0xd4bb30e2, 0x4adfa541, 0x3dd895d7, 0xa4d1c46d, 0xd3d6f4fb, 0x4369e96a, 0x346ed9fc, 0xad678846, 0xda60b8d0, 0x44042d73, 0x33031de5, 0xaa0a4c5f, 0xdd0d7cc9,
			0x5005713c, 0x270241aa, 0xbe0b1010, 0xc90c2086, 0x5768b525, 0x206f85b3, 0xb966d409, 0xce61e49f, 0x5edef90e, 0x29d9c998, 0xb0d09822, 0xc7d7a8b4, 0x59b33d17, 0x2eb40d81, 0xb7bd5c3b, 0xc0ba6cad,
			0xedb88320, 0x9abfb3b6, 0x03b6e20c, 0x74b1d29a, 0xead54739, 0x9dd277af, 0x04db2615, 0x73dc1683, 0xe3630b12, 0x94643b84, 0x0d6d6a3e, 0x7a6a5aa8, 0xe40ecf0b, 0x9309ff9d, 0x0a00ae27, 0x7d079eb1,
			0xf00f9344, 0x8708a3d2, 0x1e01f268, 0x6906c2fe, 0xf762575d, 0x806567cb, 0x196c3671, 0x6e6b06e7, 0xfed41b76, 0x89d32be0, 0x10da7a5a, 0x67dd4acc, 0xf9b9df6f, 0x8ebeeff9, 0x17b7be43, 0x60b08ed5,
			0xd6d6a3e8, 0xa1d1937e, 0x38d8c2c4, 0x4fdff252, 0xd1bb67f1, 0xa6bc5767, 0x3fb506dd, 0x48b2364b, 0xd80d2bda, 0xaf0a1b4c, 0x36034af6, 0x41047a60, 0xdf60efc3, 0xa867df55, 0x316e8eef, 0x4669be79,
			0xcb61b38c, 0xbc66831a, 0x256fd2a0, 0x5268e236, 0xcc0c7795, 0xbb0b4703, 0x220216b9, 0x5505262f, 0xc5ba3bbe, 0xb2bd0b28, 0x2bb45a92, 0x5cb36a04, 0xc2d7ffa7, 0xb5d0cf31, 0x2cd99e8b, 0x5bdeae1d,
			0x9b64c2b0, 0xec63f226, 0x756aa39c, 0x026d930a, 0x9c0906a9, 0xeb0e363f, 0x72076785, 0x05005713, 0x95bf4a82, 0xe2b87a14, 0x7bb12bae, 0x0cb61b38, 0x92d28e9b, 0xe5d5be0d, 0x7cdcefb7, 0x0bdbdf21,
			0x86d3d2d4, 0xf1d4e242, 0x68ddb3f8, 0x1fda836e, 0x81be16cd, 0xf6b9265b, 0x6fb077e1, 0x18b74777, 0x88085ae6, 0xff0f6a70, 0x66063bca, 0x11010b5c, 0x8f659eff, 0xf862ae69, 0x616bffd3, 0x166ccf45,
			0xa00ae278, 0xd70dd2ee, 0x4e048354, 0x3903b3c2, 0xa7672661, 0xd06016f7, 0x4969474d, 0x3e6e77db, 0xaed16a4a, 0xd9d65adc, 0x40df0b66, 0x37d83bf0, 0xa9bcae53, 0xdebb9ec5, 0x47b2cf7f, 0x30b5ffe9,
			0xbdbdf21c, 0xcabac28a, 0x53b39330, 0x24b4a3a6, 0xbad03605, 0xcdd70693, 0x54de5729, 0x23d967bf, 0xb3667a2e, 0xc4614ab8, 0x5d681b02, 0x2a6f2b94, 0xb40bbe37, 0xc30c8ea1, 0x5a05df1b, 0x2d02ef8d
		];

		private static readonly uint[] s_striHashTable =
		[
			0x00000000, 0x04C11DB7, 0x09823B6E, 0x0D4326D9, 0x130476DC, 0x17C56B6B, 0x1A864DB2, 0x1E475005, 0x2608EDB8, 0x22C9F00F, 0x2F8AD6D6, 0x2B4BCB61, 0x350C9B64, 0x31CD86D3, 0x3C8EA00A, 0x384FBDBD,
			0x4C11DB70, 0x48D0C6C7, 0x4593E01E, 0x4152FDA9, 0x5F15ADAC, 0x5BD4B01B, 0x569796C2, 0x52568B75, 0x6A1936C8, 0x6ED82B7F, 0x639B0DA6, 0x675A1011, 0x791D4014, 0x7DDC5DA3, 0x709F7B7A, 0x745E66CD,
			0x9823B6E0, 0x9CE2AB57, 0x91A18D8E, 0x95609039, 0x8B27C03C, 0x8FE6DD8B, 0x82A5FB52, 0x8664E6E5, 0xBE2B5B58, 0xBAEA46EF, 0xB7A96036, 0xB3687D81, 0xAD2F2D84, 0xA9EE3033, 0xA4AD16EA, 0xA06C0B5D,
			0xD4326D90, 0xD0F37027, 0xDDB056FE, 0xD9714B49, 0xC7361B4C, 0xC3F706FB, 0xCEB42022, 0xCA753D95, 0xF23A8028, 0xF6FB9D9F, 0xFBB8BB46, 0xFF79A6F1, 0xE13EF6F4, 0xE5FFEB43, 0xE8BCCD9A, 0xEC7DD02D,
			0x34867077, 0x30476DC0, 0x3D044B19, 0x39C556AE, 0x278206AB, 0x23431B1C, 0x2E003DC5, 0x2AC12072, 0x128E9DCF, 0x164F8078, 0x1B0CA6A1, 0x1FCDBB16, 0x018AEB13, 0x054BF6A4, 0x0808D07D, 0x0CC9CDCA,
			0x7897AB07, 0x7C56B6B0, 0x71159069, 0x75D48DDE, 0x6B93DDDB, 0x6F52C06C, 0x6211E6B5, 0x66D0FB02, 0x5E9F46BF, 0x5A5E5B08, 0x571D7DD1, 0x53DC6066, 0x4D9B3063, 0x495A2DD4, 0x44190B0D, 0x40D816BA,
			0xACA5C697, 0xA864DB20, 0xA527FDF9, 0xA1E6E04E, 0xBFA1B04B, 0xBB60ADFC, 0xB6238B25, 0xB2E29692, 0x8AAD2B2F, 0x8E6C3698, 0x832F1041, 0x87EE0DF6, 0x99A95DF3, 0x9D684044, 0x902B669D, 0x94EA7B2A,
			0xE0B41DE7, 0xE4750050, 0xE9362689, 0xEDF73B3E, 0xF3B06B3B, 0xF771768C, 0xFA325055, 0xFEF34DE2, 0xC6BCF05F, 0xC27DEDE8, 0xCF3ECB31, 0xCBFFD686, 0xD5B88683, 0xD1799B34, 0xDC3ABDED, 0xD8FBA05A,
			0x690CE0EE, 0x6DCDFD59, 0x608EDB80, 0x644FC637, 0x7A089632, 0x7EC98B85, 0x738AAD5C, 0x774BB0EB, 0x4F040D56, 0x4BC510E1, 0x46863638, 0x42472B8F, 0x5C007B8A, 0x58C1663D, 0x558240E4, 0x51435D53,
			0x251D3B9E, 0x21DC2629, 0x2C9F00F0, 0x285E1D47, 0x36194D42, 0x32D850F5, 0x3F9B762C, 0x3B5A6B9B, 0x0315D626, 0x07D4CB91, 0x0A97ED48, 0x0E56F0FF, 0x1011A0FA, 0x14D0BD4D, 0x19939B94, 0x1D528623,
			0xF12F560E, 0xF5EE4BB9, 0xF8AD6D60, 0xFC6C70D7, 0xE22B20D2, 0xE6EA3D65, 0xEBA91BBC, 0xEF68060B, 0xD727BBB6, 0xD3E6A601, 0xDEA580D8, 0xDA649D6F, 0xC423CD6A, 0xC0E2D0DD, 0xCDA1F604, 0xC960EBB3,
			0xBD3E8D7E, 0xB9FF90C9, 0xB4BCB610, 0xB07DABA7, 0xAE3AFBA2, 0xAAFBE615, 0xA7B8C0CC, 0xA379DD7B, 0x9B3660C6, 0x9FF77D71, 0x92B45BA8, 0x9675461F, 0x8832161A, 0x8CF30BAD, 0x81B02D74, 0x857130C3,
			0x5D8A9099, 0x594B8D2E, 0x5408ABF7, 0x50C9B640, 0x4E8EE645, 0x4A4FFBF2, 0x470CDD2B, 0x43CDC09C, 0x7B827D21, 0x7F436096, 0x7200464F, 0x76C15BF8, 0x68860BFD, 0x6C47164A, 0x61043093, 0x65C52D24,
			0x119B4BE9, 0x155A565E, 0x18197087, 0x1CD86D30, 0x029F3D35, 0x065E2082, 0x0B1D065B, 0x0FDC1BEC, 0x3793A651, 0x3352BBE6, 0x3E119D3F, 0x3AD08088, 0x2497D08D, 0x2056CD3A, 0x2D15EBE3, 0x29D4F654,
			0xC5A92679, 0xC1683BCE, 0xCC2B1D17, 0xC8EA00A0, 0xD6AD50A5, 0xD26C4D12, 0xDF2F6BCB, 0xDBEE767C, 0xE3A1CBC1, 0xE760D676, 0xEA23F0AF, 0xEEE2ED18, 0xF0A5BD1D, 0xF464A0AA, 0xF9278673, 0xFDE69BC4,
			0x89B8FD09, 0x8D79E0BE, 0x803AC667, 0x84FBDBD0, 0x9ABC8BD5, 0x9E7D9662, 0x933EB0BB, 0x97FFAD0C, 0xAFB010B1, 0xAB710D06, 0xA6322BDF, 0xA2F33668, 0xBCB4666D, 0xB8757BDA, 0xB5365D03, 0xB1F740B4
		];

		/// <summary>
		/// Compute a crc from the given string and an initial CRC value.
		/// </summary>
		/// <param name="text">Text used to generate CRC</param>
		/// <param name="crc">Initial CRC values</param>
		/// <returns>Compute CRC value</returns>
		public static uint StriHash(string text, uint crc = 0)
		{
			string upper = text.ToUpperInvariant();
			foreach (char c in upper)
			{
				crc = (crc >> 8) ^ s_striHashTable[(crc ^ c) & 0xFF];
				crc = (crc >> 8) ^ s_striHashTable[(crc ^ (c >> 8)) & 0xFF];
			}
			return crc;
		}

		/// <summary>
		/// Compute a crc from the given string and an initial CRC value
		/// </summary>
		/// <param name="text">Text used to generate CRC</param>
		/// <param name="crc">Initial CRC values</param>
		/// <returns>Compute CRC value</returns>
		public static uint Compute(byte[] text, uint crc = 0)
		{
			crc = ~crc;
			foreach (byte c in text)
			{
				crc = (crc >> 8) ^ s_table[(crc ^ c) & 0xFF];
				crc = (crc >> 8) ^ s_table[(crc) & 0xFF];
				crc = (crc >> 8) ^ s_table[(crc) & 0xFF];
				crc = (crc >> 8) ^ s_table[(crc) & 0xFF];
			}
			return ~crc;
		}
	}

	/// <summary>
	/// Helper structure for verse name mangling
	/// </summary>
	internal readonly struct VerseNameMangling
	{
		private const string MangledPrefix = "__verse_0x";
		private static readonly string[] s_internalNames = [MangledPrefix, "RetVal", "_RetVal", "$TEMP", "_Self"];

		private static bool ShouldMangleCasedName(string name)
		{
			return !s_internalNames.Any(x => name.StartsWith(x, StringComparison.Ordinal));
		}

		/// <summary>
		/// If required, generate the mangled name for a cased name
		/// </summary>
		/// <param name="name">Name to be mangled</param>
		/// <returns>True if the name needed to be mangled, false if not.</returns>
		public static (bool WasMangled, string Result) MangleCasedName(string name)
		{
			if (ShouldMangleCasedName(name))
			{
				byte[] utf8Name = Encoding.UTF8.GetBytes(name);
				uint crc = BinaryPrimitives.ReverseEndianness(Crc.Compute(utf8Name));
				return (true, $"{MangledPrefix}{crc:X8}_{name}");
			}
			else
			{
				return (false, name);
			}
		}

		/// <summary>
		/// Given a verse name, encode it
		/// </summary>
		/// <param name="name">Name to be encoded</param>
		/// <returns>Encoded name</returns>
		public static string EncodeVerseName(string name)
		{
			StringBuilder builder = new();
			builder.AppendEncodedVerseName(name);
			return builder.ToString();
		}
	}
}
