// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Resources;
using System.Runtime.InteropServices;

// These are the assembly properties for all tools
[assembly: AssemblyCompany("Epic Games, Inc.")]
[assembly: AssemblyProduct("UnrealEngine")]
[assembly: AssemblyCopyright("Copyright Epic Games, Inc. All Rights Reserved.")]
[assembly: AssemblyTrademark("")]

// Use a neutral culture to avoid some localization issues
[assembly: AssemblyCulture("")]

[assembly: ComVisible(false)]
[assembly: NeutralResourcesLanguage("en-US")]

#if !SPECIFIC_VERSION
// Automatically generate a version number based on the time of compilation
[assembly: AssemblyVersion("5.8.0.0")]
[assembly: AssemblyFileVersion("5.8.0.0")]
[assembly: AssemblyInformationalVersion("5.8.0.0")]
#endif
