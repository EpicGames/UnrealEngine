<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">

  <PropertyGroup>
    <UnrealEngineCSharpTargetsImported>true</UnrealEngineCSharpTargetsImported>
  </PropertyGroup>

  <!-- Apply RulesAssembly.PreprocessorDefines (UE_5_X_OR_LATER, etc.) only to
       UBT plugin csprojs, not to their transitively-referenced shared deps,
       so MSBuild's input-hash up-to-date check doesn't invalidate the shared
       deps when UBT vs UAT pass different DefineConstants. UBT plugin csprojs
       that set their own DefineConstants must append ($(DefineConstants);...)
       rather than overwrite, or these defines will be lost. -->
  <PropertyGroup Condition="'$(UbtPluginDefineConstants)' != '' And $(MSBuildProjectFile.EndsWith('.ubtplugin.csproj'))">
    <DefineConstants>$(DefineConstants);$(UbtPluginDefineConstants)</DefineConstants>
  </PropertyGroup>

  <!-- Visual Studio versions prior to 2022 are no longer supported -->
  <Target Name="CheckVisualStudioVersion" BeforeTargets="Build" Condition="'$(VisualStudioVersion)'!=''">
    <PropertyGroup>
      <MinimumVisualStudioVersion>17.0</MinimumVisualStudioVersion>
    </PropertyGroup>
    <Error
      Text="Visual Studio $(VisualStudioVersion) is not supported for Unreal Engine .NET projects. Please update to Visual Studio $(MinimumVisualStudioVersion)"
      Condition="$(VisualStudioVersion) &lt; $(MinimumVisualStudioVersion)"/>
  </Target>

  <!-- Task to write project dependencies to a file -->
  <Target Name="Scan">
    <ItemGroup Condition="'$(IsCrossTargetingBuild)' == 'true'">
      <_ScanTfm Include="$(TargetFrameworks)" />
    </ItemGroup>
    <MSBuild Projects="$(MSBuildProjectFullPath)"
             Targets="Scan"
             Properties="TargetFramework=%(_ScanTfm.Identity)"
             BuildInParallel="false"
             Condition="'$(IsCrossTargetingBuild)' == 'true'" />
  </Target>

  <Target Name="ProjectDependencies" BeforeTargets="Scan" Condition="'$(IsCrossTargetingBuild)' != 'true'">
    <PropertyGroup>
      <DependsPath Condition="'$(DependsPath)' == ''">$(OutputPath)\$(TargetFileName).dep.csv</DependsPath>
      <DependsOverwrite Condition="'$(DependsOverwrite)' == ''">true</DependsOverwrite>
      <DependsEncoding Condition="'$(DependsEncoding)' == ''">UTF-8</DependsEncoding>
    </PropertyGroup>

    <ItemGroup>
      <Dependencies Include="$(MSBuildThisFileFullPath)" />
      <Dependencies Include="$(MSBuildProjectFullPath);
          @(Compile);
          @(EmbeddedResource);
          @(ClCompile);
          @(ClInclude);
          @(Content);
          @(ContentWithTargetPath);"
          Condition="!$([System.String]::Copy('%(FullPath)').Contains('.nuget'))"
          Exclude="$(IntermediateOutputPath)*"
      />
    </ItemGroup>

    <GetFileHash Files="@(Dependencies->Distinct())">
      <Output TaskParameter="Items" ItemName="HashedDependencies" />
    </GetFileHash>

    <WriteLinesToFile File="$(OutputPath)\$(TargetFileName).dep.csv" Lines="@(HashedDependencies->'%(FullPath),%(FileHash)')" Overwrite="$(DependsOverwrite)" WriteOnlyWhenDifferent="true" Encoding="$(DependsEncoding)" />
  </Target>

  <!-- Task to delete additional files on clean -->
  <Target Name="AdditionalClean" AfterTargets="Clean" Condition="'$(IsCrossTargetingBuild)' != 'true'">
    <ItemGroup>
      <_DeleteFiles Include="$(OutputPath)\$(TargetFileName).dep.csv" />
    </ItemGroup>
    <Delete Files="@(_DeleteFiles)" />
  </Target>

</Project>
