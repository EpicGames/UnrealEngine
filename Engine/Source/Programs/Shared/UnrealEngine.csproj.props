<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">

  <!-- Add this file to the dependency list for the Scan target -->
  <ItemGroup>
    <Dependencies Include="$(MSBuildThisFileFullPath)" />
  </ItemGroup>

  <!-- MSBuild Targets -->
  <Import Project="UnrealEngine.CSharp.targets" Condition="'$(UnrealEngineCSharpTargetsImported)' != 'true'"/>

  <PropertyGroup>
    <!-- Allow running with newer .NET runtime -->
    <RollForward>LatestMajor</RollForward>
  </PropertyGroup>

  <!-- Platform properties -->
  <PropertyGroup>
    <IsWindows Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Windows)))' == 'true'">true</IsWindows>
    <IsOSX Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::OSX)))' == 'true'">true</IsOSX>
    <IsLinux Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Linux)))' == 'true'">true</IsLinux>

    <IsX64 Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::OSArchitecture)' == 'X64'">true</IsX64>
    <IsArm64 Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::OSArchitecture)' == 'Arm64'">true</IsArm64>

    <EngineDir Condition="'$(EngineDir)' == ''">$([System.IO.Path]::GetFullPath('$(MSBuildThisFileDirectory)..\..\..'))</EngineDir>
    <EnginePath Condition="'$(EnginePath)' == ''">$([System.IO.Path]::GetFullPath('$(MSBuildThisFileDirectory)..\..\..'))</EnginePath>
  </PropertyGroup>

  <!-- Default properties -->
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Configuration Condition="'$(Configuration)' == ''">Development</Configuration>
    <Configurations>Debug;Release;Development;Analyze</Configurations>
    <DebugType>portable</DebugType>
    <Deterministic>true</Deterministic>
    <RunAnalyzersDuringBuild>false</RunAnalyzersDuringBuild>
    <EnforceCodeStyleInBuild>false</EnforceCodeStyleInBuild>
    <NuGetAudit>false</NuGetAudit>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)' == 'Debug|AnyCPU'">
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)' == 'Release|AnyCPU'">
    <Optimize>true</Optimize>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)' == 'Development|AnyCPU'">
    <Optimize>true</Optimize>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)' == 'Analyze'">
    <RunAnalyzersDuringBuild>true</RunAnalyzersDuringBuild>
    <EnforceCodeStyleInBuild>true</EnforceCodeStyleInBuild>
    <TreatWarningsAsErrors>false</TreatWarningsAsErrors>
    <GenerateDocumentationFile>true</GenerateDocumentationFile>
    <NuGetAudit>true</NuGetAudit>
    <NuGetAuditLevel>critical</NuGetAuditLevel>
  </PropertyGroup>

</Project>
